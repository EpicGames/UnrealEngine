// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/SoftObjectPath.h"
#include "Modules/ModuleInterface.h"
#include "AISystemBase.generated.h"

UCLASS(abstract, config = Engine, defaultconfig, MinimalAPI)
class UAISystemBase : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual ~UAISystemBase()= default;
		
	/** 
	 * Called when world initializes all actors and prepares them to start gameplay
	 * @param bTimeGotReset whether the WorldSettings's TimeSeconds are reset to zero
	 */
	virtual void InitializeActorsForPlay(bool bTimeGotReset) PURE_VIRTUAL(UAISystemBase::InitializeActorsForPlay, );

	/**
	 * Event called on world origin location changes
	 *
	 * @param	OldOriginLocation			Previous world origin location
	 * @param	NewOriginLocation			New world origin location
	 */
	virtual void WorldOriginLocationChanged(FIntVector OldOriginLocation, FIntVector NewOriginLocation) PURE_VIRTUAL(UAISystemBase::WorldOriginLocationChanged, );

	/**
	 * Called by UWorld::CleanupWorld.
	 * Should be called by overriding functions.
	 * @param bSessionEnded whether to notify the viewport that the game session has ended
	 */
	ENGINE_API virtual void CleanupWorld(bool bSessionEnded = true, bool bCleanupResources = true);
	UE_DEPRECATED(5.1, "NewWorld was unused and not always calculated correctly and we expect it is not needed; let us know on UDN if it is necessary.")
	ENGINE_API virtual void CleanupWorld(bool bSessionEnded, bool bCleanupResources, UWorld* NewWorld);
	
	/** 
	 * Called by UWorld::BeginPlay to indicate the gameplay has started.
	 * Should be called by overriding functions.
	 */
	ENGINE_API virtual void StartPlay();

	/**
	 * Handles FGameModeEvents::OnGameModeMatchStateSetEvent().Broadcast(MatchState);
	 */
	ENGINE_API virtual void OnMatchStateSet(FName NewMatchState);

private:
	/** List of specific AI system class to create, can be game-specific */
	UPROPERTY(globalconfig, EditAnywhere, Category = "AISystem", noclear, meta = (MetaClass = "/Script/AIModule.AISystem", DisplayName = "AISystem Class"))
	FSoftClassPath AISystemClassName;

	/** Name of a module used to spawn the AI system. If not empty, this module has to implement IAISystemModule */
	UPROPERTY(globalconfig, EditAnywhere, Category = "AISystem", noclear, meta = (MetaClass = "/Script/AIModule.AISystem", DisplayName = "AISystem Module"))
	FName AISystemModuleName;

	FDelegateHandle OnMatchStateSetHandle;

	/** Whether the AI system class should be spawned when connecting as a client */
	UPROPERTY(globalconfig, noclear)
	bool bInstantiateAISystemOnClient;
	
public:
	static ENGINE_API FSoftClassPath GetAISystemClassName();
	static ENGINE_API FName GetAISystemModuleName();
	static ENGINE_API bool ShouldInstantiateInNetMode(ENetMode NetMode);
};

class IAISystemModule : public IModuleInterface
{
public:
	virtual UAISystemBase* CreateAISystemInstance(UWorld* World) = 0;
};
