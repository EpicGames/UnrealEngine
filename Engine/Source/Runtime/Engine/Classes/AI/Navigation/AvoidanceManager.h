// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Misc/CoreMisc.h"
#include "UObject/ScriptMacros.h"
#include "AvoidanceManager.generated.h"

class INavEdgeProviderInterface;
class IRVOAvoidanceInterface;
class UAvoidanceManager;
class UMovementComponent;
class UCharacterMovementComponent;

DECLARE_CYCLE_STAT_EXTERN(TEXT("Avoidance Time"),STAT_AI_ObstacleAvoidance,STATGROUP_AI, );

class UAvoidanceManager;

USTRUCT()
struct FNavAvoidanceData
{
	GENERATED_USTRUCT_BODY()

	/** Current location */
	FVector Center;

	/** Current velocity */
	FVector Velocity;

	/** RVO data is automatically cleared if it's not overwritten first. This makes it easier to use safely. */
	float RemainingTimeToLive;

	/** Radius (object is treated as a cylinder) */
	float Radius;

	/** Height (object is treated as a cylinder) */
	float HalfHeight;

	/** Weight for RVO (set by user) */
	float Weight;

	/** Weight is treated as a hard 1.0 while this is active. This is set by code. */
	double OverrideWeightTime;

	/** Group data */
	int32 GroupMask;

	/** Avoid agents is they belong to one of specified groups */
	int32 GroupsToAvoid;

	/** Do NOT avoid agents is they belong to one of specified groups, takes priority over GroupsToAvoid */
	int32 GroupsToIgnore;

	/** Radius of the area to consider for avoidance */
	float TestRadius2D;

	FNavAvoidanceData() {}
	FNavAvoidanceData(UAvoidanceManager* Manager, IRVOAvoidanceInterface* AvoidanceComp);

	/** Init function for internal use to guard against data changes not being reflected in blueprint-accessible creation functions */
	void Init(UAvoidanceManager* Avoidance, const FVector& InCenter, float InRadius, float InHalfHeight,
		const FVector& InVelocity, float InWeight = 0.5f,
		int32 InGroupMask = 1, int32 InGroupsToAvoid = 0xffffffff, int32 InGroupsToIgnore = 0,
		float InTestRadius2D = 500.0f);

	inline bool ShouldBeIgnored() const
	{
		return (RemainingTimeToLive <= 0.0f);
	}

	inline bool ShouldIgnoreGroup(int32 OtherGroupMask) const
	{
		return ((GroupsToAvoid& OtherGroupMask) == 0) || ((GroupsToIgnore & OtherGroupMask) != 0);
	}
};

struct FVelocityAvoidanceCone
{
	FPlane ConePlane[2];			//Left and right cone planes - these should point in toward each other. Technically, this is a convex hull, it's just unbounded.
};

UCLASS(config=Engine, Blueprintable, MinimalAPI)
class UAvoidanceManager : public UObject, public FSelfRegisteringExec
{
	GENERATED_UCLASS_BODY()
		
	/** How long an avoidance UID must not be updated before the system will put it back in the pool. Actual delay is up to 150% of this value. */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float DefaultTimeToLive;

	/** How long to stay on course (barring collision) after making an avoidance move */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float LockTimeAfterAvoid;

	/** How long to stay on course (barring collision) after making an unobstructed move (should be > 0.0, but can be less than a full frame) */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float LockTimeAfterClean;

	/** This is how far forward in time (seconds) we extend our velocity cones and thus our prediction */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float DeltaTimeToPredict;

	/** Multiply the radius of all STORED avoidance objects by this value to allow a little extra room for avoidance maneuvers. */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float ArtificialRadiusExpansion;

	/** Deprecated - use HeightCheckMargin, generally a much smaller value. */
	UPROPERTY()
	float TestHeightDifference_DEPRECATED;

	/** Allowable height margin between obstacles and agents. This is over and above the difference in agent heights. */
	UPROPERTY(EditAnywhere, Category="Avoidance", config, meta=(ClampMin = "0.0"))
	float HeightCheckMargin;

	/** Get the number of avoidance objects currently in the manager. */
	UFUNCTION(BlueprintCallable, Category="AI")
	ENGINE_API int32 GetObjectCount();

	/** Get appropriate UID for use when reporting to this function or requesting RVO assistance. */
	UFUNCTION(BlueprintCallable, Category="AI")
	ENGINE_API int32 GetNewAvoidanceUID();

	/** Register with the given avoidance manager.
	 *  @param AvoidanceWeight	When avoiding each other, actors divert course in proportion to their relative weights. Range is 0.0 to 1.0. Special: at 1.0, actor will not divert course at all.
	 */
	UFUNCTION(BlueprintCallable, Category="AI")
	ENGINE_API bool RegisterMovementComponent(class UMovementComponent* MovementComp, float AvoidanceWeight = 0.5f);

	ENGINE_API bool RegisterMovementComponent(UCharacterMovementComponent* MovementComp, float AvoidanceWeight = 0.5f);

	/** Get your latest data. */
	FNavAvoidanceData* GetAvoidanceObjectForUID(int32 AvoidanceUID) { return AvoidanceObjects.Find(AvoidanceUID); }
	const FNavAvoidanceData* GetAvoidanceObjectForUID(int32 AvoidanceUID) const { return AvoidanceObjects.Find(AvoidanceUID); }

	/** Calculate avoidance velocity for component (avoids collisions with the supplied component) */
	UFUNCTION(BlueprintCallable, Category="AI")
	ENGINE_API FVector GetAvoidanceVelocityForComponent(UMovementComponent* MovementComp);

	/** Fast-track GetAvoidanceVelocityForComponent(UMovementComponent) implementation
	 *	for most common case of UCharacterMovementComponent*/
	ENGINE_API FVector GetAvoidanceVelocityForComponent(UCharacterMovementComponent* MovementComp);

	/** Only use if you want manual velocity planning. Provide your AvoidanceUID in order to avoid colliding with yourself. */
	ENGINE_API FVector GetAvoidanceVelocityIgnoringUID(const FNavAvoidanceData& AvoidanceData, float DeltaTime, int32 IgnoreThisUID);

	/** Only use if you want manual velocity planning. Will not ignore your own volume if you are registered. */
	ENGINE_API FVector GetAvoidanceVelocity(const FNavAvoidanceData& AvoidanceData, float DeltaTime);

	/** Update the RVO avoidance data for the participating UMovementComponent */
	ENGINE_API void UpdateRVO(UMovementComponent* MovementComp);

	/** Fast-track UpdateRVO(UMovementComponent) implementation for most common case of UCharacterMovementComponent*/
	ENGINE_API void UpdateRVO(UCharacterMovementComponent* MovementComp);

	/** For Duration seconds, set this object to ignore all others. */
	ENGINE_API void OverrideToMaxWeight(int32 AvoidanceUID, float Duration);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ENGINE_API bool IsDebugOnForUID(int32 AvoidanceUID);
	ENGINE_API bool IsDebugOnForAll();
	ENGINE_API bool IsDebugEnabled(int32 AvoidanceUID);
	ENGINE_API void AvoidanceDebugForUID(int32 AvoidanceUID, bool TurnOn);
	ENGINE_API void AvoidanceDebugForAll(bool TurnOn);
	static ENGINE_API void AvoidanceSystemToggle(bool TurnOn);

	/** Exec command handlers */
	ENGINE_API void HandleToggleDebugAll( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API void HandleToggleAvoidance( const TCHAR* Cmd, FOutputDevice& Ar );
#endif

	ENGINE_API void SetNavEdgeProvider(INavEdgeProviderInterface* InEdgeProvider);

	/** Used to manually mark data associated with given ID as 'dead' (and reusable) */
	ENGINE_API void RemoveAvoidanceObject(const int32 AvoidanceUID);

	/** Returns true if the avoidance objects are auto purged in RemoveOutdatedObjects */
	bool IsAutoPurgeEnabled() const { return bAutoPurgeOutdatedObjects; }

protected:
	//~ Begin FExec Interface
	ENGINE_API virtual bool Exec_Dev(UWorld* Inworld,const TCHAR* Cmd,FOutputDevice& Ar) override;
	//~ End FExec Interface

	/** Handle for efficient management of RemoveOutdatedObjects timer */
	FTimerHandle TimerHandle_RemoveOutdatedObjects;

	/** Cleanup AvoidanceObjects, called by timer */
	ENGINE_API void RemoveOutdatedObjects();

	/** Try to set a timer for RemoveOutdatedObjects */
	ENGINE_API void RequestUpdateTimer();

	/** This is called by our blueprint-accessible function after it has packed the data into an object. */
	ENGINE_API void UpdateRVO_Internal(int32 AvoidanceUID, const FNavAvoidanceData& AvoidanceData);

	/** This is called by our blueprint-accessible functions, and permits the user to ignore self, or not. Important in case the user isn't in the avoidance manager. */
	ENGINE_API FVector GetAvoidanceVelocity_Internal(const FNavAvoidanceData& AvoidanceData, float DeltaTime, int32 *IgnoreThisUID = NULL);

	/** All objects currently part of the avoidance solution. This is pretty transient stuff. */
	TMap<int32, FNavAvoidanceData> AvoidanceObjects;

	/** This is a pool of keys to be used when new objects are created. */
	TArray<int32> NewKeyPool;

	/** Keeping this here to avoid constant allocation */
	TArray<FVelocityAvoidanceCone> AllCones;

	/** Provider of navigation edges to consider for avoidance */
	TWeakObjectPtr<UObject> EdgeProviderOb;
	INavEdgeProviderInterface* EdgeProviderInterface;

	/** set when RemoveOutdatedObjects timer is already requested */
	uint32 bRequestedUpdateTimer : 1;

	/** If set then UAvoidanceManager::RemoveOutdatedObjects is responsisble
	 *	for removing dead entries in AvoidanceObjects. If set to FALSE then the
	 *	dead agents need to be removed manually with RemoveAvoidanceObject call */
	uint32 bAutoPurgeOutdatedObjects : 1;

	UE_DEPRECATED(5.3, "Please use bAutoPurgeOutdatedObjects instead.")
	uint32 bAutoPurceOutdatedObjects : 1;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Print out debug information when we predict using any of these IDs as our IgnoreUID */
	TArray<int32> DebugUIDs;
	bool bDebugAll;
	
	/** main switch for avoidance system */
	static ENGINE_API bool bSystemActive;
#endif
};
