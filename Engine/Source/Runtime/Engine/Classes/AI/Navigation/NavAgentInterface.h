// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "NavAgentInterface.generated.h"

class AActor;
class IPathFollowingAgentInterface;
struct FNavAgentProperties;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavAgentInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavAgentInterface
{
	GENERATED_IINTERFACE_BODY()

	/**
	 *	Retrieves FNavAgentProperties expressing navigation props and caps of represented agent
	 *	@NOTE the function will be renamed to GetNavAgentProperties in 4.8. Current name was introduced
	 *		to help with deprecating old GetNavAgentProperties function
	 */
	ENGINE_API virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const;

	/**
	 *	Retrieves Agent's location
	 */
	virtual FVector GetNavAgentLocation() const PURE_VIRTUAL(INavAgentInterface::GetNavAgentLocation, return FVector::ZeroVector;);

	/** Allow actor to specify additional offset (relative to NavLocation) when it's used as move goal */
	virtual FVector GetMoveGoalOffset(const AActor* MovingActor) const { return FVector::ZeroVector; }

	/** Get cylinder for testing if actor has been reached
	 * @param MoveOffset - destination (relative to actor's nav location)
	 * @param GoalOffset - cylinder center (relative to actor's nav location)
	 * @param GoalRadius - cylinder radius
	 * @param GoalHalfHeight - cylinder half height
	 */
	virtual void GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const { }

	/** Allows delaying repath requests */
	virtual bool ShouldPostponePathUpdates() const { return false; }

	/** Checks if the agent is actively following a navigation path */
	virtual bool IsFollowingAPath() const { return false; }

	/** Retrieves PathFollowingAgent for this NavAgent */
	virtual IPathFollowingAgentInterface* GetPathFollowingAgent() const { return nullptr; }
};

