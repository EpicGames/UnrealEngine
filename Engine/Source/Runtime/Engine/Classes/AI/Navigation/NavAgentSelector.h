// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAgentSelector.generated.h"

USTRUCT()
struct FNavAgentSelector
{
	GENERATED_USTRUCT_BODY()

	static const uint32 InitializedBit = 0x80000000;
	static const uint32 AllAgentsMask = 0x7fffffff;
	static const int32 AgentsMaxCount = 31; // Must be less than or equal to the maximum number of agents defined by AllAgentsMask.

#if CPP
	union
	{
		struct
		{
#endif
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent0 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent1 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent2 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent3 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent4 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent5 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent6 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent7 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent8 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent9 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent10 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent11 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent12 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent13 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent14 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent15 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent16 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent17 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent18 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent19 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent20 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent21 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent22 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent23 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent24 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent25 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent26 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent27 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent28 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent29 : 1;
			UPROPERTY(EditAnywhere, Category = Default)
			uint32 bSupportsAgent30 : 1;
#if CPP
		};
		uint32 PackedBits;
	};
#endif

	ENGINE_API explicit FNavAgentSelector(const uint32 InBits = AllAgentsMask);

	inline bool Contains(int32 AgentIndex) const
	{
		return (AgentIndex >= 0 && AgentIndex < AgentsMaxCount) ? !!(PackedBits & (1 << AgentIndex)) : false;
	}

	inline bool ContainsAnyAgent() const
	{
		return PackedBits & AllAgentsMask;
	}


	inline void Set(int32 AgentIndex)
	{
		if (AgentIndex >= 0 && AgentIndex < AgentsMaxCount)
		{
			PackedBits |= (1 << AgentIndex);
		}
	}

	inline bool IsInitialized() const
	{
		return (PackedBits & InitializedBit) != 0;
	}

	inline void MarkInitialized()
	{
		PackedBits |= InitializedBit;
	}

	inline void Empty()
	{
		PackedBits = 0;
	}

	bool IsSame(const FNavAgentSelector& Other) const
	{
		return (~InitializedBit & PackedBits) == (~InitializedBit & Other.PackedBits);
	}

	ENGINE_API bool Serialize(FArchive& Ar);

	uint32 GetAgentBits() const 
	{
		return (~InitializedBit & PackedBits);
	}
};

template<>
struct TStructOpsTypeTraits< FNavAgentSelector > : public TStructOpsTypeTraitsBase2< FNavAgentSelector >
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

