// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "NavAreaBase.generated.h"

struct FNavAgentProperties;

// a stub class. Actual implementation in NavigationSystem module.
UCLASS(DefaultToInstanced, abstract, Config = Engine, MinimalAPI)
class UNavAreaBase : public UObject
{
	GENERATED_BODY()

protected:
	uint8 bIsMetaArea : 1;

public:
	ENGINE_API UNavAreaBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// leftover from NavigationSystem extraction from the Engine code
	virtual bool IsLowArea() const { return false; }
	virtual bool IsMetaArea() const { return (bIsMetaArea != 0); }

	/**
	*	Picks an navigation area class that should be used for Actor when
	*	queried by NavAgent. 
	*/
	static TSubclassOf<UNavAreaBase> PickAreaClassForAgent(TSubclassOf<UNavAreaBase> AreaClass, const AActor& Actor, const FNavAgentProperties& NavAgent)
	{
		const UNavAreaBase* CDO = AreaClass.Get() ? AreaClass->GetDefaultObject<const UNavAreaBase>() : nullptr;
		return CDO && CDO->IsMetaArea()
			? CDO->PickAreaClassForAgent(Actor, NavAgent)
			: AreaClass;
	}
	

protected:
	/**
	*	Picks an navigation area class that should be used for Actor when
	*	queried by NavAgent. Call it via the UNavAreaBase::PickAreaClass
	*/
	ENGINE_API virtual TSubclassOf<UNavAreaBase> PickAreaClassForAgent(const AActor& Actor, const FNavAgentProperties& NavAgent) const;
};

