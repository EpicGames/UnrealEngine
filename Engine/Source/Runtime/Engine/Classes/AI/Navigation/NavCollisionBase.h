// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/NavigationSystemHelpers.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavCollisionBase.generated.h"


class FPrimitiveDrawInterface;

struct FNavCollisionConvex
{
	TNavStatArray<FVector> VertexBuffer;
	TNavStatArray<int32> IndexBuffer;
};

UCLASS(abstract, config=Engine, MinimalAPI)
class UNavCollisionBase : public UObject
{
	GENERATED_BODY()

protected:
	DECLARE_DELEGATE_RetVal_OneParam(UNavCollisionBase*, FConstructNew, UObject& /*Outer*/);

	static ENGINE_API FConstructNew ConstructNewInstanceDelegate;
	struct FDelegateInitializer
	{
		FDelegateInitializer();
	};
	static ENGINE_API FDelegateInitializer DelegateInitializer;

	/** If set, mesh will be used as dynamic obstacle (don't create navmesh on top, much faster adding/removing) */
	UPROPERTY(EditAnywhere, Category = Navigation, config, meta = (EditCondition = "!bUseSurfaceArea"))
	uint32 bIsDynamicObstacle : 1;

	/** Apply AreaClass to walkable surface triangles during rasterization instead of default area.
	 *  Mutually exclusive with bIsDynamicObstacle. See AreaClass documentation for details. */
	UPROPERTY(EditAnywhere, Category = Navigation, config, meta = (EditCondition = "!bIsDynamicObstacle"))
	uint32 bUseSurfaceArea : 1;

	/** convex collisions are ready to use */
	uint32 bHasConvexGeometry : 1;

	FNavCollisionConvex TriMeshCollision;
	FNavCollisionConvex ConvexCollision;

public:
	static UNavCollisionBase* ConstructNew(UObject& Outer) { return ConstructNewInstanceDelegate.Execute(Outer); }

	ENGINE_API UNavCollisionBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Tries to read data from DDC, and if that fails gathers navigation
	*	collision data, stores it and uploads to DDC */
	virtual void Setup(class UBodySetup* BodySetup) PURE_VIRTUAL(UNavCollisionBase::Setup, );

	[[nodiscard]] virtual FBox GetBounds() const PURE_VIRTUAL(UNavCollisionBase::GetBounds, static FBox InvalidBox; return InvalidBox; );

	/** Export collision data */
	virtual bool ExportGeometry(const FTransform& LocalToWorld, FNavigableGeometryExport& GeoExport) const PURE_VIRTUAL(UNavCollisionBase::ExportGeometry, return false; );

	/** Get data for dynamic obstacle and collisions with surface navarea. */
	virtual void GetNavigationModifier(FCompositeNavModifier& Modifier, const FTransform& LocalToWorld) PURE_VIRTUAL(UNavCollisionBase::GetNavigationModifier, );

	/** draw cylinder and box collisions */
	virtual void DrawSimpleGeom(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FColor Color) {}

#if WITH_EDITOR
	virtual void InvalidateCollision() PURE_VIRTUAL(UNavCollisionBase::InvalidateCollision, );
#endif // WITH_EDITOR

	bool IsDynamicObstacle() const { return bIsDynamicObstacle; }
	bool HasConvexGeometry() const { return bHasConvexGeometry; }
	const FNavCollisionConvex& GetTriMeshCollision() const { return TriMeshCollision; }
	const FNavCollisionConvex& GetConvexCollision() const { return ConvexCollision;	}
	FNavCollisionConvex& GetMutableTriMeshCollision() { return TriMeshCollision; }
	FNavCollisionConvex& GetMutableConvexCollision() { return ConvexCollision; }

	/** Returns true if surface area is being used and this collision has a custom surface area class for walkable triangles. */
	virtual bool HasSurfaceAreaClass() const
	{
		return false;
	}
};
