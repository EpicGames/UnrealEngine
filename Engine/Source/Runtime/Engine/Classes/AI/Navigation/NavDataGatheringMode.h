// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavDataGatheringMode.generated.h"

UENUM()
enum class ENavDataGatheringMode : uint8
{
	Default,
	Instant,
	Lazy
};
