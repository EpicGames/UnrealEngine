// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "NavEdgeProviderInterface.generated.h"

struct FNavEdgeSegment
{
	FVector P0, P1;
};

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UNavEdgeProviderInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavEdgeProviderInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual bool GetEdges(const FVector& Center, float Range, TArray<FNavEdgeSegment>& Edges)
	{
		return false;
	}
};
