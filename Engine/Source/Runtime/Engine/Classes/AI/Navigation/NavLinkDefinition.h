// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "NavLinkDefinition.generated.h"

class UNavAreaBase;

UENUM()
namespace ENavLinkDirection
{
	enum Type : int
	{
		BothWays,
		LeftToRight,
		RightToLeft,
	};
}

USTRUCT(BlueprintType)
struct FNavigationLinkBase
{
	GENERATED_USTRUCT_BODY()

	/** if greater than 0 nav system will attempt to project navlink's start point on geometry below */
	UPROPERTY(EditAnywhere, Category=Default, meta=(ClampMin = "0.0"))
	float LeftProjectHeight;

	/** if greater than 0 nav system will attempt to project navlink's end point on geometry below */
	UPROPERTY(EditAnywhere, Category=Default, meta=(ClampMin = "0.0", DisplayName="Right Project Height"))
	float MaxFallDownLength;

	/** Needs to be 0 for recast data generator */
	UE_DEPRECATED(5.3, "Use FNavLinkId::Invalid instead")
	static constexpr uint32 InvalidUserId = 0;

	/** ID passed to navigation data generator */
	UE_DEPRECATED(5.3, "Use NavLinkId instead, this id is no longer used in the engine")
	uint32 UserId;

	FNavLinkId NavLinkId;

	UPROPERTY(EditAnywhere, Category=Default, meta=(ClampMin = "1.0"))
	float SnapRadius;

	UPROPERTY(EditAnywhere, Category=Default, meta=(ClampMin = "1.0", EditCondition="bUseSnapHeight"))
	float SnapHeight;

	/** restrict link only to specified agents */
	UPROPERTY(EditAnywhere, Category=Default)
	FNavAgentSelector SupportedAgents;

	// DEPRECATED AGENT CONFIG
#if CPP
	union
	{
		struct
		{
#endif
	UPROPERTY()
	uint32 bSupportsAgent0 : 1;
	UPROPERTY()
	uint32 bSupportsAgent1 : 1;
	UPROPERTY()
	uint32 bSupportsAgent2 : 1;
	UPROPERTY()
	uint32 bSupportsAgent3 : 1;
	UPROPERTY()
	uint32 bSupportsAgent4 : 1;
	UPROPERTY()
	uint32 bSupportsAgent5 : 1;
	UPROPERTY()
	uint32 bSupportsAgent6 : 1;
	UPROPERTY()
	uint32 bSupportsAgent7 : 1;
	UPROPERTY()
	uint32 bSupportsAgent8 : 1;
	UPROPERTY()
	uint32 bSupportsAgent9 : 1;
	UPROPERTY()
	uint32 bSupportsAgent10 : 1;
	UPROPERTY()
	uint32 bSupportsAgent11 : 1;
	UPROPERTY()
	uint32 bSupportsAgent12 : 1;
	UPROPERTY()
	uint32 bSupportsAgent13 : 1;
	UPROPERTY()
	uint32 bSupportsAgent14 : 1;
	UPROPERTY()
	uint32 bSupportsAgent15 : 1;
#if CPP
		};
		uint32 SupportedAgentsBits;
	};
#endif

#if WITH_EDITORONLY_DATA
	/** this is an editor-only property to put descriptions in navlinks setup, to be able to identify it easier */
	UPROPERTY(EditAnywhere, Category=Default)
	FString Description;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category=Default, BlueprintReadWrite)
	TEnumAsByte<ENavLinkDirection::Type> Direction;

	UPROPERTY(EditAnywhere, Category=Default, meta=(InlineEditConditionToggle))
	uint8 bUseSnapHeight : 1;

	/** If set, link will try to snap to the cheapest area in given radius */
	UPROPERTY(EditAnywhere, Category = Default)
	uint8 bSnapToCheapestArea : 1;
	
	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag0 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag1 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag2 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag3 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag4 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag5 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag6 : 1;

	/** custom flag, check DescribeCustomFlags for details */
	UPROPERTY()
	uint8 bCustomFlag7 : 1;

	/** Was this link generated by the navigation system */
	uint8 bIsGenerated : 1;

	ENGINE_API FNavigationLinkBase();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNavigationLinkBase(const FNavigationLinkBase&) = default;
	FNavigationLinkBase(FNavigationLinkBase&& Other) = default;
	FNavigationLinkBase& operator=(const FNavigationLinkBase& Other) = default;
	FNavigationLinkBase& operator=(FNavigationLinkBase&& Other) = default;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void InitializeAreaClass(const bool bForceRefresh = false);
	ENGINE_API void SetAreaClass(UClass* InAreaClass);
	ENGINE_API UClass* GetAreaClass() const;
	ENGINE_API bool HasMetaArea() const;

#if WITH_EDITORONLY_DATA
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif

#if WITH_EDITOR
	/** set up bCustomFlagX properties and expose them for edit
	  * @param NavLinkPropertiesOwnerClass - optional object holding FNavigationLinkBase structs, defaults to UNavLinkDefinition
	  */
	static ENGINE_API void DescribeCustomFlags(const TArray<FString>& EditableFlagNames, UClass* NavLinkPropertiesOwnerClass = nullptr);
#endif // WITH_EDITOR

private:
	/** Area type of this link (empty = default) */
	UPROPERTY(EditAnywhere, Category = Default)
	TSubclassOf<UNavAreaBase> AreaClass;

	TWeakObjectPtr<UClass> AreaClassOb;
};

template<>
struct TStructOpsTypeTraits< FNavigationLinkBase > : public TStructOpsTypeTraitsBase2< FNavigationLinkBase >
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithPostSerialize = true,
	};
#endif
};

USTRUCT(BlueprintType)
struct FNavigationLink : public FNavigationLinkBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Default, BlueprintReadWrite, meta=(MakeEditWidget=""))
	FVector Left;

	UPROPERTY(EditAnywhere, Category=Default, BlueprintReadWrite, meta=(MakeEditWidget=""))
	FVector Right;

	FNavigationLink()
		: Left(0,-50, 0), Right(0, 50, 0)
	{}

	FNavigationLink(const FVector& InLeft, const FVector& InRight) 
		: Left(InLeft), Right(InRight)
	{}

	inline FNavigationLink Transform(const FTransform& Transformation) const
	{
		FNavigationLink Result = *this;
		Result.Left = Transformation.TransformPosition(Result.Left);
		Result.Right = Transformation.TransformPosition(Result.Right);

		return Result;
	}

	inline FNavigationLink Translate(const FVector& Translation) const
	{
		FNavigationLink Result = *this;
		Result.Left += Translation;
		Result.Right += Translation;

		return Result;
	}

	inline FNavigationLink Rotate(const FRotator& Rotation) const
	{
		FNavigationLink Result = *this;
		
		Result.Left = Rotation.RotateVector(Result.Left);
		Result.Right = Rotation.RotateVector(Result.Right);

		return Result;
	}
};

template<>
struct TStructOpsTypeTraits< FNavigationLink > : public TStructOpsTypeTraits< FNavigationLinkBase >
{
};

USTRUCT()
struct FNavigationSegmentLink : public FNavigationLinkBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Default, meta=(MakeEditWidget=""))
	FVector LeftStart;

	UPROPERTY(EditAnywhere, Category=Default, meta=(MakeEditWidget=""))
	FVector LeftEnd;

	UPROPERTY(EditAnywhere, Category=Default, meta=(MakeEditWidget=""))
	FVector RightStart;

	UPROPERTY(EditAnywhere, Category=Default, meta=(MakeEditWidget=""))
	FVector RightEnd;

	FNavigationSegmentLink() 
		: LeftStart(-25, -50, 0), LeftEnd(25, -50,0), RightStart(-25, 50, 0), RightEnd(25, 50, 0) 
	{}

	FNavigationSegmentLink(const FVector& InLeftStart, const FVector& InLeftEnd, const FVector& InRightStart, const FVector& InRightEnd)
		: LeftStart(InLeftStart), LeftEnd(InLeftEnd), RightStart(InRightStart), RightEnd(InRightEnd)
	{}

	inline FNavigationSegmentLink Transform(const FTransform& Transformation) const
	{
		FNavigationSegmentLink Result = *this;
		Result.LeftStart = Transformation.TransformPosition(Result.LeftStart);
		Result.LeftEnd = Transformation.TransformPosition(Result.LeftEnd);
		Result.RightStart = Transformation.TransformPosition(Result.RightStart);
		Result.RightEnd = Transformation.TransformPosition(Result.RightEnd);

		return Result;
	}

	inline FNavigationSegmentLink Translate(const FVector& Translation) const
	{
		FNavigationSegmentLink Result = *this;
		Result.LeftStart += Translation;
		Result.LeftEnd += Translation;
		Result.RightStart += Translation;
		Result.RightEnd += Translation;

		return Result;
	}
};

template<>
struct TStructOpsTypeTraits< FNavigationSegmentLink > : public TStructOpsTypeTraits< FNavigationLinkBase >
{
};

/** Class containing definition of a navigation area */
UCLASS(abstract, Config=Engine, Blueprintable, MinimalAPI)
class UNavLinkDefinition : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=OffMeshLinks)
	TArray<FNavigationLink> Links;

	UPROPERTY(EditAnywhere, Category=OffMeshLinks)
	TArray<FNavigationSegmentLink> SegmentLinks;

	static ENGINE_API const TArray<FNavigationLink>& GetLinksDefinition(class UClass* LinkDefinitionClass);
	static ENGINE_API const TArray<FNavigationSegmentLink>& GetSegmentLinksDefinition(class UClass* LinkDefinitionClass);

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	ENGINE_API void InitializeAreaClass() const;
	ENGINE_API bool HasMetaAreaClass() const;
	ENGINE_API bool HasAdjustableLinks() const;

private:
	uint32 bHasInitializedAreaClasses : 1;
	uint32 bHasDeterminedMetaAreaClass : 1;
	uint32 bHasMetaAreaClass : 1;
	uint32 bHasDeterminedAdjustableLinks : 1;
	uint32 bHasAdjustableLinks : 1;
};
