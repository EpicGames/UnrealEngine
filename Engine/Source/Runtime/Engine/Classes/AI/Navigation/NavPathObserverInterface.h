// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "NavPathObserverInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavPathObserverInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavPathObserverInterface
{
	GENERATED_IINTERFACE_BODY()

	/** 
	 *	Called on Navigation Agent when given PathGenerator updated it's path.
	 */
	virtual void OnPathUpdated(class INavigationPathGenerator* PathGenerator) PURE_VIRTUAL(INavPathObserverInterface::OnPathUpdated,);

	/** 
	 *	Called when path generated by PathGenerator becomes invalid
	 */
	virtual void OnPathInvalid(class INavigationPathGenerator* PathGenerator) PURE_VIRTUAL(INavPathObserverInterface::OnPathInvalid,);

	/** 
	 *	Called when path generation by PathGenerator fails
	 */
	virtual void OnPathFailed(class INavigationPathGenerator* PathGenerator) PURE_VIRTUAL(INavPathObserverInterface::OnPathFailed,);
};
