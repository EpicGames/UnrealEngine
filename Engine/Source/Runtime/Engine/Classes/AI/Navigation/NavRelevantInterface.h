// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AI/Navigation/NavDataGatheringMode.h"
#include "NavRelevantInterface.generated.h"

struct FNavigableGeometryExport;
struct FNavigationRelevantData;
class UBodySetup;

/** Determines if, and how, a navigation element should export collision for AI navigation */
UENUM()
namespace EHasCustomNavigableGeometry
{
enum Type : int
{
	/** Element custom geometry export callback is not called, but the default collision export is performed using its convex/trimesh collision. */
	No,

	/** The custom geometry export callback is called and indicates if the default collision export should also be performed. */
	Yes,

	/** The custom geometry export callback is called even if the mesh is non-collidable and wouldn't normally affect the navigation data. */
	EvenIfNotCollidable,

	/** Neither the custom geometry export delegate nor the default export will be called (can still add modifiers through the NavigationData export callback). */
	DontExport,
};
}

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavRelevantInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavRelevantInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Prepares navigation modifiers */
	virtual void GetNavigationData(FNavigationRelevantData& Data) const {}

	/** Get bounds for navigation octree */
	virtual FBox GetNavigationBounds() const { return FBox(ForceInit); }

	/** Indicates if this instance knows how to export sub-sections of self */
	virtual bool SupportsGatheringGeometrySlices() const { return false; }
	
	/** Indicates if the area covered by the navigation bounds of the object should not be dirtied when inserting, or removing, the object in the navigation octree.
	 * In this case we expect that object to manually dirty areas (e.g. using UpdateNavigationElementBoundsDelegate).
	 */
	virtual bool ShouldSkipDirtyAreaOnAddOrRemove() const { return false; }

	/** This function is called "on demand", whenever that specified piece of geometry is needed for navigation generation */
	virtual void GatherGeometrySlice(FNavigableGeometryExport& GeomExport, const FBox& SliceBox) const {}

	virtual ENavDataGatheringMode GetGeometryGatheringMode() const { return ENavDataGatheringMode::Default; }

	/** Called on Game-thread to give implementer a chance to perform actions that require game-thread to run (e.g. precaching physics data) */
	virtual void PrepareGeometryExportSync() {}
	
	/** Returns associated body setup (if any) for default geometry export.
	 *	@return Associated UBodySetup if any, nullptr otherwise. */
	virtual UBodySetup* GetNavigableGeometryBodySetup() { return nullptr; }
	
	/** Returns transform to used for default geometry export.
	 *	@return Transform to used when exporting geometry. */
	virtual FTransform GetNavigableGeometryTransform() const { return FTransform::Identity; }

	/** If true then DoCustomNavigableGeometryExport will be called to collect navigable geometry for the object implementing this interface. */
	virtual EHasCustomNavigableGeometry::Type HasCustomNavigableGeometry() const { return EHasCustomNavigableGeometry::No; }

	/** Collects the custom navigable geometry of the object.
	 *	@return true if regular navigable geometry exporting (using GetBodySetup) should be run as well */
	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const { return true; }

	/** Update bounds, called after the object moved */
	virtual void UpdateNavigationBounds() {}

	/** Are modifiers active? */
	virtual bool IsNavigationRelevant() const { return true; }

	/** Get navigation parent
	 *  Adds modifiers to existing octree node, GetNavigationBounds and IsNavigationRelevant won't be checked
	 */
	virtual UObject* GetNavigationParent() const { return nullptr; }
};
