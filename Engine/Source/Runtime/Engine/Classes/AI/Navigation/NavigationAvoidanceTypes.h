// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NavigationAvoidanceTypes.generated.h"

USTRUCT(BlueprintType)
struct FNavAvoidanceMask
{
	GENERATED_USTRUCT_BODY()

#if CPP
	union
	{
		struct
		{
#endif
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup0 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup1 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup2 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup3 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup4 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup5 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup6 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup7 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup8 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup9 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup10 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup11 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup12 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup13 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup14 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup15 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup16 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup17 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup18 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup19 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup20 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup21 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup22 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup23 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup24 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup25 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup26 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup27 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup28 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup29 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup30 : 1;
			UPROPERTY(Category = "Group", EditAnywhere, BlueprintReadWrite)
			uint32 bGroup31 : 1;
#if CPP
		};
		int32 Packed = 0;
	};
#endif

	inline bool HasGroup(uint8 GroupID) const
	{
		return (Packed & (1 << GroupID)) != 0;
	}

	inline void SetGroup(uint8 GroupID)
	{
		Packed |= (1 << GroupID);
	}

	inline void ClearGroup(uint8 GroupID)
	{
		Packed &= ~(1 << GroupID);
	}

	inline void ClearAll()
	{
		Packed = 0;
	}

	inline void SetFlagsDirectly(uint32 NewFlagset)
	{
		Packed = NewFlagset;
	}
};
