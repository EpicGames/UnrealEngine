// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavAgentSelector.h"

struct FNavigationBounds
{
	uint32 UniqueID;
	FBox AreaBox;
	FNavAgentSelector SupportedAgents;
	TWeakObjectPtr<ULevel> Level;		// The level this bounds belongs to

	bool operator==(const FNavigationBounds& Other) const 
	{ 
		return UniqueID == Other.UniqueID; 
	}

	friend uint32 GetTypeHash(const FNavigationBounds& NavBounds)
	{
		return GetTypeHash(NavBounds.UniqueID);
	}
};

struct FNavigationBoundsUpdateRequest 
{
	FNavigationBounds NavBounds;
	
	enum Type
	{
		Added,
		Removed,
		Updated,
	};

	Type UpdateRequest;
};