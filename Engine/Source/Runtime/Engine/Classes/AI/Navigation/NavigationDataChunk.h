// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NavigationDataChunk.generated.h"

/** 
 * 
 */
UCLASS(NotBlueprintable, abstract, MinimalAPI)
class UNavigationDataChunk : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Name of NavigationData actor that owns this chunk */
	UPROPERTY()
	FName NavigationDataName;
};
