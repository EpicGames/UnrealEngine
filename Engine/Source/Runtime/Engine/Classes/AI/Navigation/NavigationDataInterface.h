// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavQueryFilter.h"
#include "NavigationDataInterface.generated.h"


struct FNavigableGeometryExport;
struct FNavigationRelevantData;


UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavigationDataInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavigationDataInterface
{
	GENERATED_IINTERFACE_BODY()
public:
	/**	Tries to move current nav location towards target constrained to navigable area.
	 *	@param OutLocation if successful this variable will be filed with result
	 *	@return true if successful, false otherwise
	 */
	virtual bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const PURE_VIRTUAL(INavigationDataInterface::FindMoveAlongSurface, return false;);

	/**	Tries to project given Point to this navigation type, within given Extent.
	*	@param OutLocation if successful this variable will be filed with result
	*	@return true if successful, false otherwise
	*/
	virtual bool ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const PURE_VIRTUAL(INavigationDataInterface::ProjectPoint, return false;);

	/** Determines whether the specified NavNodeRef is still valid
	*   @param NodeRef the NavNodeRef to test for validity
	*   @return true if valid, false otherwise
	*/
	virtual bool IsNodeRefValid(NavNodeRef NodeRef) const PURE_VIRTUAL(INavigationDataInterface::IsNodeRefValid, return true;);
};
