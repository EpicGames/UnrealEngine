// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationDataResolution.generated.h"

UENUM()
enum class ENavigationDataResolution : uint8
{
	Low = 0,
	Default = 1,
	High = 2,
	Invalid = 3 UMETA(DisplayName="None"),
	MAX = 3,
};