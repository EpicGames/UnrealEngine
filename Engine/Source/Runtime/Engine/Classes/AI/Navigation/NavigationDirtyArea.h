// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Box.h"
#include "UObject/WeakObjectPtr.h"

class UObject;
struct FNavigationElement;

enum class ENavigationDirtyFlag : uint8
{
	None				= 0,
	Geometry			= (1 << 0),
	DynamicModifier		= (1 << 1),
	UseAgentHeight		= (1 << 2),
	NavigationBounds	= (1 << 3),

	All = Geometry | DynamicModifier, // all rebuild steps here without additional flags
};
ENUM_CLASS_FLAGS(ENavigationDirtyFlag);

struct FNavigationDirtyArea
{
	FBox Bounds = FBox(ForceInit);
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Use OptionalSourceElement instead.")
	TWeakObjectPtr<UObject> OptionalSourceObject;
#endif // WITH_EDITORONLY_DATA
	TSharedPtr<const FNavigationElement> OptionalSourceElement;
	ENavigationDirtyFlag Flags = ENavigationDirtyFlag::None;

	FNavigationDirtyArea() = default;
	ENGINE_API FNavigationDirtyArea(const FBox& InBounds, ENavigationDirtyFlag InFlags, const TSharedPtr<const FNavigationElement>& InOptionalSourceElement = nullptr);

	UE_DEPRECATED(5.5, "Use the constructor taking ENavigationDirtyFlag and FNavigationElement instead.")
	ENGINE_API FNavigationDirtyArea(const FBox& InBounds, int32 InFlags, UObject* const InOptionalSourceObject = nullptr);

	ENGINE_API FNavigationDirtyArea(const FNavigationDirtyArea& Other);
	ENGINE_API FNavigationDirtyArea(FNavigationDirtyArea&& Other);

	ENGINE_API FNavigationDirtyArea& operator=(const FNavigationDirtyArea& Other);
	ENGINE_API FNavigationDirtyArea& operator=(FNavigationDirtyArea&& Other);

	bool HasFlag(const ENavigationDirtyFlag Flag) const
	{
		return (Flags & Flag) != ENavigationDirtyFlag::None;
	}

	bool operator==(const FNavigationDirtyArea& Other) const
	{ 
		return Flags == Other.Flags && OptionalSourceElement == Other.OptionalSourceElement && Bounds.Equals(Other.Bounds);
	}
	
	bool operator!=( const FNavigationDirtyArea& Other) const
	{
		return !(*this == Other);
	}

	ENGINE_API FString GetSourceDescription() const;
};
