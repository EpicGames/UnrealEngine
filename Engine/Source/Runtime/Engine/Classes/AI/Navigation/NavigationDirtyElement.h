// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "UObject/WeakObjectPtr.h"

enum class ENavigationDirtyFlag : uint8;
class INavRelevantInterface;
struct FNavigationElement;

struct FNavigationDirtyElement
{
	/**
	 * If not empty and the associated navigation element controls the dirty areas explicitly (i.e. DirtyAreasOnRegistration is 'false'),
	 * the list will be used to indicate the areas that need rebuilding.
	 * Otherwise, the default behavior, element's bounds will be used.
	 */
	TArray<FBox> ExplicitAreasToDirty;

	/** Navigation element associated with this dirty element */
	TSharedRef<const FNavigationElement> NavigationElement;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Use NavigationElement instead.")
	FWeakObjectPtr Owner;

	UE_DEPRECATED(5.5, "Use NavigationElement instead.")
	INavRelevantInterface* NavInterface = nullptr;
#endif // WITH_EDITORONLY_DATA

	/** bounds of already existing entry for this actor */
	FBox PrevBounds = FBox(ForceInit);

	/** override for update flags */
	ENavigationDirtyFlag FlagsOverride;

	/** flags of already existing entry for this actor */
	ENavigationDirtyFlag PrevFlags;

	/** prev flags & bounds data are set */
	uint8 bHasPrevData : 1 = false;

	/** request was invalidated while queued, use prev values to dirty area */
	uint8 bInvalidRequest : 1 = false;

	/** requested during visibility change of the owning level (loading/unloading) */
	uint8 bIsFromVisibilityChange : 1 = false;

	/** part of the base navmesh */
	uint8 bIsInBaseNavmesh : 1 = false;

	UE_DEPRECATED(5.5, "The default constructor will be remove. Use the version with FNavigationElement instead.")
	ENGINE_API FNavigationDirtyElement();
	UE_DEPRECATED(5.5, "Use the version with FNavigationElement instead.")
	ENGINE_API explicit FNavigationDirtyElement(UObject* InOwner);
	UE_DEPRECATED(5.5, "Use the version with FNavigationElement instead.")
	ENGINE_API FNavigationDirtyElement(UObject* InOwner, INavRelevantInterface* InNavInterface, int32 InFlagsOverride = 0, const bool bUseWorldPartitionedDynamicMode = false);
	ENGINE_API FNavigationDirtyElement(const TSharedRef<const FNavigationElement>& InNavigationElement, ENavigationDirtyFlag InFlagsOverride, const bool bUseWorldPartitionedDynamicMode = false);
	ENGINE_API explicit FNavigationDirtyElement(const TSharedRef<const FNavigationElement>& InNavigationElement, const bool bUseWorldPartitionedDynamicMode = false);

	ENGINE_API FNavigationDirtyElement(const FNavigationDirtyElement& Other);

	ENGINE_API FNavigationDirtyElement& operator=(const FNavigationDirtyElement& Other);

	UE_DEPRECATED(5.5, "This operator will no longer be used.")
	ENGINE_API bool operator==(const UObject*& OtherOwner) const;

	ENGINE_API friend uint32 GetTypeHash(const FNavigationDirtyElement& Info);
};
