// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Box.h"
#include "Math/Transform.h"
#include "UObject/WeakObjectPtr.h"

class INavRelevantInterface;
class UBodySetup;
class UObject;
struct FNavigationElement;
struct FNavigationRelevantData;
struct FNavigableGeometryExport;
enum class ENavDataGatheringMode : uint8;
namespace EHasCustomNavigableGeometry
{
	enum Type : int;
}

DECLARE_DELEGATE_TwoParams(FNavigationDataExport, const FNavigationElement&, FNavigationRelevantData&);
DECLARE_DELEGATE_ThreeParams(FCustomGeometryExport, const FNavigationElement&, FNavigableGeometryExport&, bool&);
DECLARE_DELEGATE_ThreeParams(FGeometrySliceExport, const FNavigationElement&, FNavigableGeometryExport&, const FBox&);

/**
 * Structure used to identify a unique navigation element registered in the Navigation system.
 * The handle can be created to represent two use cases:
 *  1. Single UObject representing the navigation element
 *     constructed from a UObject raw or weak pointer
 *  2. Single UObject managing multiple non-UObject navigation elements
 *     constructed from a UObject raw or weak pointer and using the optional constructor parameter to identity a unique sub-element
 * @see FNavigationElement
 */
struct FNavigationElementHandle
{
	static ENGINE_API const FNavigationElementHandle Invalid;

	FNavigationElementHandle() = default;

	inline explicit FNavigationElementHandle(const UObject* Object, const uint64 SubElementId = INDEX_NONE);
	inline explicit FNavigationElementHandle(const TWeakObjectPtr<const UObject>& WeakObject, const uint64 SubElementId = INDEX_NONE);

	inline bool operator==(const FNavigationElementHandle& Other) const;
	inline bool operator!=(const FNavigationElementHandle& Other) const;

	/**
	 * Conversion operator used to convert the handle to a boolean based on its validity.
	 * @see IsValid
	 */
	inline explicit operator bool() const;

	/** @return true if the handle has been properly assigned. */
	inline bool IsValid() const;

	/** Invalidates the current handle so further calls to IsValid will return false. */
	inline void Invalidate();

	/** Hash function to use TSet/TMap */
	inline friend uint32 GetTypeHash(const FNavigationElementHandle& Handle);

	/** Stringifies FNavigationElementHandle */
	ENGINE_API friend FString LexToString(const FNavigationElementHandle& Handle);

private:

	/**
	 * Main mandatory part of the handle to associate it with either a UObject with a 1:1 relation
	 * with the registered navigation element, or a UObject that manages multiple non-UObject navigation elements.
	 */
	TWeakObjectPtr<const UObject> OwnerUObject;

	/**
	 * Second optional part of the handle used when the associated UObject manages multiple navigation elements.
	 */
	uint64 SubElementId = INDEX_NONE;
};


/**
 * Structure registered in the navigation system that holds the required properties and delegates
 * to gather navigation data (navigable geometry, NavArea modifiers, NavLinks, etc.) and be stored
 * in the navigation octree.
 *
 * It represents a single element spatially located in a defined area in the level.
 * That element can be created to represent two use cases:
 *  1. Single UObject representing the navigation element
 *     constructed from a UObject raw or weak pointer
 *  2. Single UObject managing multiple non-UObject navigation elements
 *     constructed from a UObject raw or weak pointer and using the optional constructor parameter to identity a unique sub-element
 * @see FNavigationElementHandle
 */
struct FNavigationElement : TSharedFromThis<FNavigationElement>
{
private:
	/** Temporary solution to provide access to deprecated constructors. */
	friend struct FNavigationDirtyElement;
    struct FPrivateToken { explicit FPrivateToken() = default; };
	static TSharedRef<FNavigationElement> MakeFromUObject_DEPRECATED(UObject* InOwner)
	{
		return MakeShared<FNavigationElement>(FPrivateToken{}, InOwner, INDEX_NONE);
	}

public:

    // Public, but can only be called by FNavigationElement and friends, because it needs access to FPrivateToken
    ENGINE_API explicit FNavigationElement(FPrivateToken, const UObject* Object, uint64 SubElementId /*= INDEX_NONE*/);

	FNavigationElement() = delete;
	ENGINE_API explicit FNavigationElement(const UObject& Object, uint64 SubElementId = INDEX_NONE);
	ENGINE_API explicit FNavigationElement(const INavRelevantInterface& NavRelevant, uint64 SubElementId = INDEX_NONE);

	UE_DEPRECATED(5.5, "Temporary constructor to allow deprecation of other navigation types. Use the version taking an object reference instead.")
	explicit FNavigationElement(const UObject* Object, const uint64 SubElementId = INDEX_NONE)
		: FNavigationElement(FPrivateToken{}, Object, SubElementId)
	{
	}

	/** Factory helper function to create and initialize a sharable element from INavRelevantInterface. */
	static ENGINE_API TSharedRef<const FNavigationElement> CreateFromNavRelevantInterface(const INavRelevantInterface& NavRelevantInterface);

	/**
	 * Returns a weak pointer to the UObject associated with this navigation element.
	 * @return Weak pointer to the associated UObject if any.
	 * @note If called from the Game thread all regular TWeakObjectPtr concerns and limitation applies.
	 */
	inline TWeakObjectPtr<const UObject> GetWeakUObject() const;

	/**
	 * Returns a handle that can be used to uniquely identify this element.
	 * @return Handle representing the current element.
	 */
	ENGINE_API FNavigationElementHandle GetHandle() const;

	/** @return Transform to use for the default geometry export. */
	inline const FTransform& GetTransform() const;

	/** Sets the Transform to use for the default geometry export. */
	inline void SetTransform(const FTransform& InGeometryTransform);

	/** @return The bounds to use when registering the element in the navigation octree. */
	inline FBox GetBounds() const;

	/** Sets the Bounds to use when registering the element in the navigation octree. */
	inline void SetBounds(const FBox& InBounds);

	/** @return The associated body setup (if any) to use for the default geometry export. */
	inline UBodySetup* GetBodySetup() const;

	/** Sets the body setup to use for the default geometry export. */
	ENGINE_API void SetBodySetup(UBodySetup* InBodySetup);

	/** @return The UObject for which the associated octree node will be used to hold the current element navigation data. */
	inline const TWeakObjectPtr<const UObject>& GetNavigationParent() const;

	/** Sets the UObject for which the associated octree node will be used to hold the current element navigation data. */
	inline void SetNavigationParent(const UObject* InNavigationParent);

	/**
	 * @return The type of geometry export to use for the current element
	 * @see EHasCustomNavigableGeometry
	 */
	inline EHasCustomNavigableGeometry::Type GetGeometryExportType() const;

	/**
	 * Sets the type of geometry export to use for the current element
	 * @see GeometryExportType
	 */
	inline void SetGeometryExportType(const EHasCustomNavigableGeometry::Type InCustomNavigableGeometry);

	/**
	 * @return The mode that indicates when the geometry gathering must be executed.
	 * @see GeometryGatheringMode
	 */
	inline ENavDataGatheringMode GetGeometryGatheringMode() const;

	/**
	 * Sets the mode that indicates when the geometry gathering must be executed.
	 * @see GeometryGatheringMode
	 */
	inline void SetGeometryGatheringMode(const ENavDataGatheringMode InGeometryGatheringMode);

	/**
	 * @return True the area covered by the navigation bounds should be dirtied when inserting,
	 * or removing, the element in the navigation octree (default behavior).
	 * @see DirtyAreaOnRegistration
	 */
	inline bool GetDirtyAreaOnRegistration() const;

	/**
	 * Sets whether the area covered by the navigation bounds should be dirtied when inserting,
	 * or removing, the element in the navigation octree (default behavior).
	 * @see DirtyAreaOnRegistration
	 */
	inline void SetDirtyAreaOnRegistration(const bool bInDirtyAreaOnRegistration);

	/**
	 * @return True if the element was created from a UObject that is associated with a data layer
	 * in the list of runtime data layers that should be included in the base navigation data (cooked)
	 * or directly placed in the level.
	 */
	inline bool IsInBaseNavigationData() const;

	/**
	 * @return True if the element was created from a UObject while its level was pending
	 * being made invisible or visible (i.e., loading/unloading).
	 */
	inline bool IsFromLevelVisibilityChange() const;

	/** @return Name describing the element based on the associated UObject name and sub-element ID (if any). */
	ENGINE_API FString GetName() const;

	/** @return Name describing the element based on the associated UObject pathname and sub-element ID (if any). */
	ENGINE_API FString GetPathName() const;

	/** @return Name describing the element based on the associated UObject fullname and sub-element ID (if any). */
	ENGINE_API FString GetFullName() const;

	/** @return Name describing the specified element based on the name of its associated UObject and sub-element ID (if any). */
	inline friend FString GetNameSafe(const FNavigationElement* Element);

	/** @return Name describing the specified element based on the pathname of its associated UObject and sub-element ID (if any). */
	inline friend FString GetPathNameSafe(const FNavigationElement* Element);

	/** @return Name describing the specified element based on the fullname of its associated UObject and sub-element ID (if any). */
	inline friend FString GetFullNameSafe(const FNavigationElement* Element);

	/** Hash function to use TSet/TMap */
	ENGINE_API friend uint32 GetTypeHash(const FNavigationElement& Element);

	/** Stringifies FNavigationElementHandle */
	ENGINE_API friend FString LexToString(const FNavigationElement& Element);

	/** Delegate used to gather navigation data like NavLinks, NavAreaModifiers, etc. */
	FNavigationDataExport NavigationDataExportDelegate;

	/**
	 * Delegate used during the geometry export based on the specified GeometryExportType
	 * @see GeometryExportType
	 */
	FCustomGeometryExport CustomGeometryExportDelegate;

	/** Delegate that can be used by very large elements to gather a limited piece of geometry for navigation generation in a given area. */
	FGeometrySliceExport GeometrySliceExportDelegate;

private:

	ENGINE_API explicit FNavigationElement(const UObject& Object, uint64 SubElementId, bool bTryInitializeFromInterface);
	void InitializeFromInterface(const INavRelevantInterface* NavRelevantInterface);
	
	/**
	 * The associated UObject provided to create the element.
	 * It is still currently used after creation for: ShouldSkipObjectPredicate (dirty areas), MetaModifier, logging and validation.
	 * This is always set when creating the object and should be valid for the lifetime of the element if it is properly unregistered
	 * based on the UObject lifetime (e.g., a Component will unregister its associated element when it gets unregistered from the world).
	 */
	TWeakObjectPtr<const UObject> OwnerUObject;

	/**
	 * Optional ID that can be used when a UObject owns and manages multiple sub-elements which should be represented individually
	 * in the navigation system (i.e., each sub-element has its own bounds and should be registered in an octree node).
	 */
	uint64 SubElementId = INDEX_NONE;

	/** Associated body setup (if any) used by the default geometry export. */
	TWeakObjectPtr<UBodySetup> BodySetup = nullptr;

	/** Transform used by the default geometry export. */
	FTransform GeometryTransform = FTransform::Identity;

	/** Bounds used to register the element in the navigation octree. */
	FBox Bounds;

	/**
	 * Indicates that this element is not registered as a new node in the navigation octree, but instead
	 * adds its navigation data to the parent octree node.
	 */
	TWeakObjectPtr<const UObject> NavigationParent = nullptr;

	/**
	 * Indicates the type of geometry export to use for the current element.
	 * @see EHasCustomNavigableGeometry
	 */
	EHasCustomNavigableGeometry::Type GeometryExportType;

	/**
	 * Indicates when the geometry gathering must be executed:
	 * - Instant: The geometry export is executed as soon as the element is registered in the navigation octree.
	 * - Lazy: The geometry export is executed only when it is required to rebuild the navigation data intersecting with a given area (e.g., Tile generator).
	 * - Default: Use the default gathering mode defined by the navigation system.
	 */
	ENavDataGatheringMode GeometryGatheringMode;

	/**
	 * Indicates if the area covered by the navigation bounds should be dirtied when inserting, or removing, the element in the navigation octree (default behavior).
	 * When returning false it is expected that the element will explicitly dirty areas (e.g. using UpdateNavigationElementBoundsDelegate).
	 */
	bool bDirtyAreaOnRegistration = true;

	/**
	 * Indicates that the element was created from a UObject that is associated with a data layer
	 * in the list of runtime data layers that should be included in the base navigation data (cooked)
	 * or directly placed in the level.
	 */
	bool bIsInBaseNavigationData = false;

	/** Indicates that the element was created from a UObject while its level was pending being made invisible or visible (i.e., loading/unloading). */
	bool bIsFromLevelVisibilityChange = false;
};


//----------------------------------------------------------------------//
// Inlines
//----------------------------------------------------------------------//
uint32 GetTypeHash(const FNavigationElementHandle& Key)
{
	return HashCombine(GetTypeHash(Key.OwnerUObject), GetTypeHash(Key.SubElementId));
}

FString GetNameSafe(const FNavigationElement* Element)
{
	return Element ? Element->GetName() : TEXT("None");
}

FString GetPathNameSafe(const FNavigationElement* Element)
{
	return Element ? Element->GetPathName() : TEXT("None");
}

FString GetFullNameSafe(const FNavigationElement* Element)
{
	return Element ? Element->GetFullName() : TEXT("None");
}


//----------------------------------------------------------------------//
// Inlines (FNavigationElementHandle)
//----------------------------------------------------------------------//

FNavigationElementHandle::FNavigationElementHandle(const UObject* Object, const uint64 SubElementId /*= INDEX_NONE*/)
	: OwnerUObject(Object)
	, SubElementId(SubElementId)
{
}

FNavigationElementHandle::FNavigationElementHandle(const TWeakObjectPtr<const UObject>& WeakObject, const uint64 SubElementId /*= INDEX_NONE*/)
	: OwnerUObject(WeakObject)
	, SubElementId(SubElementId)
{
}

bool FNavigationElementHandle::operator==(const FNavigationElementHandle& Other) const
{
	return OwnerUObject.HasSameIndexAndSerialNumber(Other.OwnerUObject) && SubElementId == Other.SubElementId;
}

bool FNavigationElementHandle::operator!=(const FNavigationElementHandle& Other) const
{
	return !(*this == Other);
}

FNavigationElementHandle::operator bool() const
{
	return IsValid();
}

bool FNavigationElementHandle::IsValid() const
{
	return *this != Invalid;
}

void FNavigationElementHandle::Invalidate()
{
	*this = Invalid;
}


//----------------------------------------------------------------------//
// Inlines (FNavigationElement)
//----------------------------------------------------------------------//

TWeakObjectPtr<const UObject> FNavigationElement::GetWeakUObject() const
{
	return OwnerUObject;
}

bool FNavigationElement::IsFromLevelVisibilityChange() const
{
	return bIsFromLevelVisibilityChange;
}

bool FNavigationElement::IsInBaseNavigationData() const
{
	return bIsInBaseNavigationData;
}

const FTransform& FNavigationElement::GetTransform() const
{
	return GeometryTransform;
}

void FNavigationElement::SetTransform(const FTransform& InGeometryTransform)
{
	GeometryTransform = InGeometryTransform;
}

FBox FNavigationElement::GetBounds() const
{
	return Bounds;
}

void FNavigationElement::SetBounds(const FBox& InBounds)
{
	Bounds = InBounds;
}

UBodySetup* FNavigationElement::GetBodySetup() const
{
	return BodySetup.Get();
}

const TWeakObjectPtr<const UObject>& FNavigationElement::GetNavigationParent() const
{
	return NavigationParent;
}

void FNavigationElement::SetNavigationParent(const UObject* InNavigationParent)
{
	NavigationParent = InNavigationParent;
}

EHasCustomNavigableGeometry::Type FNavigationElement::GetGeometryExportType() const
{
	return GeometryExportType;
}

void FNavigationElement::SetGeometryExportType(const EHasCustomNavigableGeometry::Type InCustomNavigableGeometry)
{
	GeometryExportType = InCustomNavigableGeometry;
}

ENavDataGatheringMode FNavigationElement::GetGeometryGatheringMode() const
{
	return GeometryGatheringMode;
}

void FNavigationElement::SetGeometryGatheringMode(const ENavDataGatheringMode InGeometryGatheringMode)
{
	GeometryGatheringMode = InGeometryGatheringMode;
}

bool FNavigationElement::GetDirtyAreaOnRegistration() const
{
	return bDirtyAreaOnRegistration;
}

void FNavigationElement::SetDirtyAreaOnRegistration(const bool bInDirtyAreaOnRegistration)
{
	bDirtyAreaOnRegistration = bInDirtyAreaOnRegistration;
}
