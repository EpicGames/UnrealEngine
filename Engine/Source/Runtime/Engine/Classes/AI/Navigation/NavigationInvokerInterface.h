// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "NavigationInvokerInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavigationInvokerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavigationInvokerInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Called to retrieve the center of the area that require navmesh. Must be implemented by child class. */
	virtual FVector GetNavigationInvokerLocation() const PURE_VIRTUAL(, return FNavigationSystem::InvalidLocation;);
};
