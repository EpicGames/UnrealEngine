// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationInvokerPriority.generated.h"

UENUM()
enum class ENavigationInvokerPriority : uint8
{
	// Enum values are used (keep in ascending order)
	VeryLow = 1,
	Low = 2,
	Default = 3,
	High = 4,
	VeryHigh = 5,
	MAX = 6
};