// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/NavigationModifier.h"
#include "NavigationDirtyArea.h"

struct FNavigationElement;

struct FNavigationRelevantDataFilter 
{
	/** pass when actor has geometry */
	uint32 bIncludeGeometry : 1;
	/** pass when actor has any offmesh link modifier */
	uint32 bIncludeOffmeshLinks : 1;
	/** pass when actor has any area modifier */
	uint32 bIncludeAreas : 1;
	/** pass when actor has any modifier with meta area */
	uint32 bIncludeMetaAreas : 1;
	/** fail if from level loading (only valid in WP dynamic mode) */
	uint32 bExcludeLoadedData : 1;

	FNavigationRelevantDataFilter() 
		: bIncludeGeometry(false)
		, bIncludeOffmeshLinks(false)
		, bIncludeAreas(false)
		, bIncludeMetaAreas(false)
		, bExcludeLoadedData(false)
	{}
};

// @todo consider optional structures that can contain a delegate instead of 
// actual copy of collision data
struct FNavigationRelevantData : public TSharedFromThis<FNavigationRelevantData, ESPMode::ThreadSafe>
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterNavDataDelegate, const struct FNavDataConfig*);

	/** CollisionData should always start with this struct for validation purposes */
	struct FCollisionDataHeader
	{
		int32 DataSize;

		static bool IsValid(const uint8* RawData, int32 RawDataSize);
	};

	/** exported geometry (used by recast navmesh as FRecastGeometryCache) */
	TNavStatArray<uint8> CollisionData;

	/** cached voxels (used by recast navmesh as FRecastVoxelCache) */
	TNavStatArray<uint8> VoxelData;

	/** bounds of geometry (unreal coords) */
	FBox Bounds;

	/** Gathers per instance data for navigation geometry in a specified area box */
	FNavDataPerInstanceTransformDelegate NavDataPerInstanceTransformDelegate;

	/** called to check if hosted geometry should be used for given FNavDataConfig. If not set then "true" is assumed.
	 *  Might want to set bUseVirtualGeometryFilteringAndDirtying to true in the Navmesh class you are excluding geometry from.
	 *  This will improve cpu performance by stopping the navmesh from dirtying tiles requested by actors being excluded by this delegate.
	 */
	FFilterNavDataDelegate ShouldUseGeometryDelegate;

	/** additional modifiers: areas and external links */
	FCompositeNavModifier Modifiers;

#if WITH_EDITORONLY_DATA
	/** UObject these data represents */
	UE_DEPRECATED(5.5, "Use SourceElement instead.")
	TWeakObjectPtr<UObject> SourceObject;
#endif // WITH_EDITORONLY_DATA

	/** Navigation element this data represents */
	TSharedRef<const FNavigationElement> SourceElement;

	/** get set to true when lazy navigation exporting is enabled and this navigation data has "potential" of
	*	containing geometry data. First access will result in gathering the data and setting this flag back to false.
	*	Mind that this flag can go back to 'true' if related data gets cleared out. */
	uint32 bPendingLazyGeometryGathering : 1;
	uint32 bPendingLazyModifiersGathering : 1;
	uint32 bPendingChildLazyModifiersGathering : 1;

	uint32 bSupportsGatheringGeometrySlices : 1;

	/** Indicates that this data will not dirty the navmesh when added or removed from the octree. */
	uint32 bShouldSkipDirtyAreaOnAddOrRemove : 1;

	/** From level loading (only valid in WP dynamic mode) */
	uint32 bLoadedData : 1;

	FNavigationRelevantData() = delete;

	UE_DEPRECATED(5.5, "Use the constructor using FNavigationElement instead.")
	ENGINE_API FNavigationRelevantData(UObject& Source);

	explicit FNavigationRelevantData(const TSharedRef<const FNavigationElement>& Source)
		: SourceElement(Source)
		, bPendingLazyGeometryGathering(false)
		, bPendingLazyModifiersGathering(false)
		, bPendingChildLazyModifiersGathering(false)
		, bSupportsGatheringGeometrySlices(false)
		, bShouldSkipDirtyAreaOnAddOrRemove(false)
		, bLoadedData(false)
	{}

	ENGINE_API FNavigationRelevantData(const FNavigationRelevantData& Other);
	ENGINE_API FNavigationRelevantData(FNavigationRelevantData&& Other);

	ENGINE_API FNavigationRelevantData& operator=(FNavigationRelevantData&& Other);
	ENGINE_API FNavigationRelevantData& operator=(const FNavigationRelevantData& Other);

	inline bool HasGeometry() const { return VoxelData.Num() || CollisionData.Num(); }
	inline bool HasModifiers() const { return !Modifiers.IsEmpty(); }
	inline bool HasDynamicModifiers() const { return Modifiers.IsDynamic(); }
	inline bool IsPendingLazyGeometryGathering() const { return bPendingLazyGeometryGathering; }
	inline bool IsPendingLazyModifiersGathering() const { return bPendingLazyModifiersGathering; }
	inline bool IsPendingChildLazyModifiersGathering() const { return bPendingChildLazyModifiersGathering; }
	inline bool NeedAnyPendingLazyModifiersGathering() const { return bPendingLazyModifiersGathering || bPendingChildLazyModifiersGathering; }
	inline bool SupportsGatheringGeometrySlices() const { return bSupportsGatheringGeometrySlices; }

	/**
	 * Indicates that this object will not dirty the navmesh when added or removed from the octree.
	 * In this case we expect it to manually dirty areas (e.g. using OnObjectBoundsChanged).
	 */
	inline bool ShouldSkipDirtyAreaOnAddOrRemove() const { return bShouldSkipDirtyAreaOnAddOrRemove; }

	inline bool IsEmpty() const { return !HasGeometry() && !HasModifiers(); }
	inline SIZE_T GetAllocatedSize() const { return CollisionData.GetAllocatedSize() + VoxelData.GetAllocatedSize() + Modifiers.GetAllocatedSize(); }
	inline SIZE_T GetGeometryAllocatedSize() const { return CollisionData.GetAllocatedSize() + VoxelData.GetAllocatedSize(); }
	inline ENavigationDirtyFlag GetDirtyFlag() const
	{
		const bool bSetGeometryFlag = HasGeometry() || IsPendingLazyGeometryGathering() ||
			Modifiers.GetFillCollisionUnderneathForNavmesh() || Modifiers.GetMaskFillCollisionUnderneathForNavmesh() ||
			(Modifiers.GetNavMeshResolution() != ENavigationDataResolution::Invalid);
		
		return (bSetGeometryFlag
				? ENavigationDirtyFlag::Geometry
				: ENavigationDirtyFlag::None)
			| ((HasDynamicModifiers() || NeedAnyPendingLazyModifiersGathering())
				? ENavigationDirtyFlag::DynamicModifier
				: ENavigationDirtyFlag::None)
			| (Modifiers.HasAgentHeightAdjust()
				? ENavigationDirtyFlag::UseAgentHeight
				: ENavigationDirtyFlag::None);
	}

	ENGINE_API FCompositeNavModifier GetModifierForAgent(const FNavAgentProperties* NavAgent = nullptr) const;

	ENGINE_API bool HasPerInstanceTransforms() const;
	ENGINE_API bool IsMatchingFilter(const FNavigationRelevantDataFilter& Filter) const;
	ENGINE_API void Shrink();
	ENGINE_API bool IsCollisionDataValid() const;

	void ValidateAndShrink()
	{
		if (IsCollisionDataValid())
		{
			Shrink();
		}
		else
		{
			CollisionData.Empty();
		}
	}

	UE_DEPRECATED(5.5, "Use SourceElement instead.")
	ENGINE_API const UObject* GetOwner() const;

	UE_DEPRECATED(5.5, "Use SourceElement instead.")
	ENGINE_API TWeakObjectPtr<UObject> GetOwnerPtr() const;
};
