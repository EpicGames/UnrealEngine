// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Misc/CoreStats.h"
#include "UObject/SoftObjectPath.h"
#include "NavigationTypes.generated.h"

#define INVALID_NAVQUERYID uint32(0)
#define INVALID_NAVDATA uint32(0)
#define INVALID_NAVEXTENT (FVector::ZeroVector)

#define DEFAULT_NAV_QUERY_EXTENT_HORIZONTAL 50.f
#define DEFAULT_NAV_QUERY_EXTENT_VERTICAL 250.f

class AActor;
class INavAgentInterface;
class INavRelevantInterface;
class ULevel;
struct FNavigationPath;

/** uniform identifier type for navigation data elements may it be a polygon or graph node */
typedef uint64 NavNodeRef;
#define INVALID_NAVNODEREF NavNodeRef(0)

namespace FNavigationSystem
{
	/** used as a fallback value for navigation agent radius, when none specified via UNavigationSystemV1::SupportedAgents */
	extern ENGINE_API const float FallbackAgentRadius;

	/** used as a fallback value for navigation agent height, when none specified via UNavigationSystemV1::SupportedAgents */
	extern ENGINE_API const float FallbackAgentHeight;

	inline constexpr FBox InvalidBoundingBox(ForceInit, UE::Math::TBoxConstInit{});

	inline constexpr FVector InvalidLocation = FVector(TNumericLimits<FVector::FReal>::Max(), UE::Math::TVectorConstInit{});

	inline bool IsValidLocation(const FVector& TestLocation)
	{
		return TestLocation != InvalidLocation;
	}

	inline bool BoxesAreSame(const FBox& A, const FBox& B)
	{
		return FVector::PointsAreSame(A.Min, B.Min) && FVector::PointsAreSame(A.Max, B.Max);
	}

	/** Returns true if the visibility of the owning level is currently changing (loading/unloading). */
	ENGINE_API bool IsLevelVisibilityChanging(const UObject* Object);
	
	/** Objects placed directly in the level and objects placed in the base navmesh data layers are in the base navmesh. */
	ENGINE_API bool IsInBaseNavmesh(const UObject* Object);
}

UENUM()
enum class ENavigationOptionFlag : uint8
{
	Default,
	Enable UMETA(DisplayName = "Yes"), // UHT was complaining when tried to use True as value instead of Enable
	Disable UMETA(DisplayName = "No"),
	MAX UMETA(Hidden)
};

//////////////////////////////////////////////////////////////////////////
// Navigation data generation

UENUM()
enum class ENavDataGatheringModeConfig : uint8
{
	Invalid UMETA(Hidden),
	Instant,
	Lazy
};

UENUM()
enum class ENavigationSortPendingTilesMethod : uint8
{
	SortWithSeedLocations,
	SortByPriority,
	None
};

//
// Used to gather per instance transforms in a specific area
//
DECLARE_DELEGATE_TwoParams(FNavDataPerInstanceTransformDelegate, const FBox&, TArray<FTransform>&);

/*
 * There should be one of these Ids generated and stored per Actor or Component that inherits from INavLinkCustomInterface. Primarily
 * its a way of differentiating between Components that inherit from INavLinkCustomInterface as the ActorInstanceGuid
 * alone can not do this.See UNavLinkCustomComponent as an example. For Level Instances these will be different for each component in an actor but repeated
 * in repeated Level Instances.FNavLinkId::GenerateUniqueId() uses this to make the UniqueId
 */
USTRUCT()
struct FNavLinkAuxiliaryId
{
	GENERATED_BODY()

	FNavLinkAuxiliaryId() = default;

	uint64 GetId() const
	{
		return Id;
	}

	bool operator==(const FNavLinkAuxiliaryId& Other) const { return Id == Other.Id; }
	bool operator!=(const FNavLinkAuxiliaryId& Other) const { return !this->operator==(Other); }


	UE_DEPRECATED(5.4, "This function is not deterministic in all instances during cooking. Use the version which takes PathName.")
	static ENGINE_API FNavLinkAuxiliaryId GenerateUniqueAuxiliaryId();

	/**
	 * Helper function: returns unique Auxiliary ID for custom links.
	 **/
	static ENGINE_API FNavLinkAuxiliaryId GenerateUniqueAuxiliaryId(FStringView PathName);

private:
	FNavLinkAuxiliaryId(uint64 InId)
		: Id(InId)
	{}

public:
	static ENGINE_API const FNavLinkAuxiliaryId Invalid;

private:
	UPROPERTY()
	uint64 Id = InvalidLinkId;

	static constexpr uint64 InvalidLinkId = 0;
};

/**
 * NavLink Id type. Legacy Ids were generated using the incremental ID system which has been deprecated but will be supported for quite some time. The new system uses CityHash using GenerateUniqueId()
 * Only NavLinks generated this way should have the most significant bit of the id set to 1.
 */
USTRUCT()
struct FNavLinkId
{
	GENERATED_BODY()

	FNavLinkId() = default;

	explicit FNavLinkId(uint64 InId)
		: Id(InId)
	{}

	bool IsValid() const
	{
		return Id != InvalidLinkId;
	}

	bool operator==(const FNavLinkId& Other) const
	{
		return Id == Other.Id;
	}


	bool operator!=(const FNavLinkId& Other) const
	{
		return !this->operator==(Other);
	}
	
	uint64 GetId() const
	{
		return Id;
	}

	void SetId(uint64 InId)
	{
		Id = InId;
	}

	/**
	 * New non Legacy NavLinkIds have the most significant bit set to 1. Only Id's formed from GenerateUniqueId() should have the most significant bit set to 1. We should only see extremely rare clashes
	 * here anyway with 63 bits for the city hash in the sort of number of CustomLinks we are likely to have and there is a system in place to handle this for dynamic meshes at run time.
	 * For static meshes the AuxiliaryId / ActorInstanceGuid will need to be generated again in the editor (probably easiest by deleting and readding the actor / component in the editor
	 * at the moment).
	 */
	bool IsLegacyId() const
	{
		return (Id & NavLinkIdBitMask) == 0;
	}

	/**
	 *  Helper function: returns unique ID number for custom links.
	 */
	static ENGINE_API FNavLinkId GenerateUniqueId();
	
	/**
	 *  Helper function: returns unique ID number for custom links.
	 */
	static ENGINE_API FNavLinkId GenerateUniqueId(FNavLinkAuxiliaryId AuxiliaryId, FGuid ActorInstanceGuid);

	friend inline uint32 GetTypeHash(FNavLinkId const& Value)
	{
		return GetTypeHash(Value.GetId());
	}

public:
	static ENGINE_API const FNavLinkId Invalid;

private:
	UPROPERTY()
	uint64 Id = InvalidLinkId;

	static constexpr uint64 InvalidLinkId = 0;
	static constexpr uint64 NavLinkIdBitMask = 1ULL << 63; // Newer non incremental 64 bit NavLinkIds will have a 1 in the most significant bit.
};


//////////////////////////////////////////////////////////////////////////
// Path

struct FNavigationPortalEdge
{
	FVector Left;
	FVector Right;
	NavNodeRef ToRef;

	FNavigationPortalEdge() : Left(0.f), Right(0.f), ToRef(INVALID_NAVNODEREF)
	{}

	FNavigationPortalEdge(const FVector& InLeft, const FVector& InRight, NavNodeRef InToRef)
		: Left(InLeft), Right(InRight), ToRef(InToRef)
	{}

	inline FVector GetPoint(const int32 Index) const
	{
		check(Index >= 0 && Index < 2);
		return ((FVector*)&Left)[Index];
	}

	inline FVector::FReal GetLength() const { return FVector::Dist(Left, Right); }

	inline FVector GetMiddlePoint() const { return Left + (Right - Left) / 2; }
};

/** Describes a point in navigation data */
struct FNavLocation
{
	/** location relative to path's base */
	FVector Location;

	/** node reference in navigation data */
	NavNodeRef NodeRef;

	FNavLocation() : Location(FVector::ZeroVector), NodeRef(INVALID_NAVNODEREF) {}
	explicit FNavLocation(const FVector& InLocation, NavNodeRef InNodeRef = INVALID_NAVNODEREF) 
		: Location(InLocation), NodeRef(InNodeRef) {}

	/** checks if location has associated navigation node ref */
	inline bool HasNodeRef() const { return NodeRef != INVALID_NAVNODEREF; }

	inline operator FVector() const { return Location; }

	bool operator==(const FNavLocation& Other) const
	{
		return Location == Other.Location && NodeRef == Other.NodeRef;
	}
};

/** Describes node in navigation path */
struct FNavPathPoint : public FNavLocation
{
	/** extra node flags */
	uint32 Flags;

	UE_DEPRECATED(5.3, "Use CustomNavLinkId instead. This id is no longer used in the engine.")
	uint32 CustomLinkId;

	/** unique Id of custom navigation link starting at this point */
	FNavLinkId CustomNavLinkId;

	FNavPathPoint() : Flags(0)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CustomLinkId = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FNavPathPoint(const FVector& InLocation, NavNodeRef InNodeRef = INVALID_NAVNODEREF, uint32 InFlags = 0) 
		: FNavLocation(InLocation, InNodeRef), Flags(InFlags)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CustomLinkId = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNavPathPoint(const FNavPathPoint&) = default;
	FNavPathPoint(FNavPathPoint&& Other) = default;
	FNavPathPoint& operator=(const FNavPathPoint& Other) = default;
	FNavPathPoint& operator=(FNavPathPoint&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool operator==(const FNavPathPoint& Other) const
	{
		return Flags == Other.Flags && CustomNavLinkId == Other.CustomNavLinkId && FNavLocation::operator==(Other);
	}
};

/** path type data */
struct FNavPathType
{
	explicit FNavPathType(const FNavPathType* Parent = nullptr) : Id(++NextUniqueId), ParentType(Parent) {}

	bool operator==(const FNavPathType& Other) const
	{
		return Id == Other.Id || (ParentType != nullptr && *ParentType == Other);
	}

	bool IsA(const FNavPathType& Other) const
	{
		return (Id == Other.Id) || (ParentType && ParentType->IsA(Other));
	}

private:
	static ENGINE_API uint32 NextUniqueId;
	uint32 Id;
	const FNavPathType* ParentType;
};

UENUM()
namespace ENavPathEvent
{
	enum Type : int
	{
		Cleared,
		NewPath,
		UpdatedDueToGoalMoved,
		UpdatedDueToNavigationChanged,
		Invalidated,
		RePathFailed,
		MetaPathUpdate,
		Custom,
	};
}

namespace ENavPathUpdateType
{
	enum Type
	{
		GoalMoved,
		NavigationChanged,
		MetaPathUpdate,
		Custom,
	};
}

namespace EPathObservationResult
{
	enum Type
	{
		NoLongerObserving,
		NoChange,
		RequestRepath,
	};
}

namespace ENavAreaEvent
{
	enum Type
	{
		Registered,
		Unregistered
	};
}

typedef TSharedRef<struct FNavigationPath, ESPMode::ThreadSafe> FNavPathSharedRef;
typedef TSharedPtr<struct FNavigationPath, ESPMode::ThreadSafe> FNavPathSharedPtr;
typedef TWeakPtr<struct FNavigationPath, ESPMode::ThreadSafe> FNavPathWeakPtr;

/** Movement capabilities, determining available movement options for Pawns and used by AI for reachability tests. */
USTRUCT(BlueprintType)
struct FMovementProperties
{
	GENERATED_USTRUCT_BODY()

	/** If true, this Pawn is capable of crouching. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	uint8 bCanCrouch:1;

	/** If true, this Pawn is capable of jumping. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	uint8 bCanJump:1;

	/** If true, this Pawn is capable of walking or moving on the ground. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	uint8 bCanWalk:1;

	/** If true, this Pawn is capable of swimming or moving through fluid volumes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	uint8 bCanSwim:1;

	/** If true, this Pawn is capable of flying. By default, Pawn can only use NavData with matching bCanFly values.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	uint8 bCanFly:1;

	FMovementProperties()
		: bCanCrouch(false)
		, bCanJump(false)
		, bCanWalk(false)
		, bCanSwim(false)
		, bCanFly(false)
	{
	}
};

/**
 * Struct to hold properties a user might set for navigation movement
 */
USTRUCT(BlueprintType)
struct FNavMovementProperties
{
	GENERATED_BODY()

	/** Braking distance override used with acceleration driven path following (bUseAccelerationForPaths) */
	UPROPERTY(EditAnywhere, Category = NavMovement, meta = (EditCondition = "bUseFixedBrakingDistanceForPaths"))
	float FixedPathBrakingDistance = 0;

	/** If set to true, NavAgentProperties' radius and height will be updated with Owner's collision capsule size */
	UPROPERTY(EditAnywhere, Category = NavMovement)
	bool bUpdateNavAgentWithOwnersCollision = true;

	/** If set, pathfollowing will control character movement via acceleration values. If false, it will set velocities directly. */
	UPROPERTY(EditAnywhere, Category = NavMovement, GlobalConfig)
	bool bUseAccelerationForPaths = false;

	/** If set, FixedPathBrakingDistance will be used for path following deceleration */
	UPROPERTY(EditAnywhere, Category = NavMovement, meta = (EditCondition = "bUseAccelerationForPaths"))
	bool bUseFixedBrakingDistanceForPaths = false;

	/** If set, StopActiveMovement call will abort current path following request */
	UPROPERTY(EditAnywhere, Category = NavMovement)
	bool bStopMovementAbortPaths = true;
};

/** Properties of representation of an 'agent' (or Pawn) used by AI navigation/pathfinding. */
USTRUCT(BlueprintType)
struct FNavAgentProperties : public FMovementProperties
{
	GENERATED_USTRUCT_BODY()

	/** Radius of the capsule used for navigation/pathfinding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties, meta=(DisplayName="Nav Agent Radius"))
	float AgentRadius;

	/** Total height of the capsule used for navigation/pathfinding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties, meta=(DisplayName="Nav Agent Height"))
	float AgentHeight;

	/** Step height to use, or -1 for default value from navdata's config. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties, meta=(DisplayName="Nav Agent Step Height"))
	float AgentStepHeight;

	/** Scale factor to apply to height of bounds when searching for navmesh to project to when nav walking */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties)
	float NavWalkingSearchHeightScale;

	/** Type of navigation data used by agent, null means "any" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementProperties, meta=(MetaClass = "/Script/NavigationSystem.NavigationData"))
	FSoftClassPath PreferredNavData;
	
	FNavAgentProperties(float Radius = -1.f, float Height = -1.f)
		: AgentRadius(Radius), AgentHeight(Height), AgentStepHeight(-1), NavWalkingSearchHeightScale(0.5f)
	{}

	ENGINE_API FNavAgentProperties(const FNavAgentProperties& Other);
	ENGINE_API FNavAgentProperties& operator=(const FNavAgentProperties& Other);

	ENGINE_API void UpdateWithCollisionComponent(class UShapeComponent* CollisionComponent);

	inline bool IsValid() const { return AgentRadius >= 0.f && AgentHeight >= 0.f; }
	inline bool HasStepHeightOverride() const { return AgentStepHeight >= 0.0f; }

	ENGINE_API bool IsNavDataMatching(const FNavAgentProperties& Other) const;

	inline bool IsEquivalent(const FNavAgentProperties& Other, float Precision = 5.f) const
	{
		return FGenericPlatformMath::Abs(AgentRadius - Other.AgentRadius) < Precision
			&& FGenericPlatformMath::Abs(AgentHeight - Other.AgentHeight) < Precision
			&& ((HasStepHeightOverride() == false)
				|| (Other.HasStepHeightOverride() == false)
				|| FGenericPlatformMath::Abs(AgentStepHeight - Other.AgentStepHeight) < Precision)
			&& IsNavDataMatching(Other);
	}
	
	bool operator==(const FNavAgentProperties& Other) const
	{
		return IsEquivalent(Other);
	}

	FVector GetExtent() const
	{
		return IsValid() 
			? FVector(AgentRadius, AgentRadius, AgentHeight / 2)
			: INVALID_NAVEXTENT;
	}

	ENGINE_API void SetPreferredNavData(TSubclassOf<AActor> NavDataClass);

	static ENGINE_API const FNavAgentProperties DefaultProperties;

	friend inline uint32 GetTypeHash(const FNavAgentProperties& A)
	{
		return ((int16(A.AgentRadius) << 16) | int16(A.AgentHeight)) ^ int32(A.AgentStepHeight);
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType)
struct FNavDataConfig : public FNavAgentProperties
{
	GENERATED_USTRUCT_BODY()

	/** Internal/debug name of this agent */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Display)
	FName Name;

	/** Color used to represent this agent in the editor and for debugging */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Display)
	FColor Color;

	/** Rough size of this agent, used when projecting unto navigation mesh */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Querying)
	FVector DefaultQueryExtent;

protected:
	/** Class to use when spawning navigation data instance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Navigation, meta = (MetaClass = "/Script/NavigationSystem.NavigationData"))
	TSoftClassPtr<AActor> NavDataClass;

public:	
	ENGINE_API FNavDataConfig(float Radius = FNavigationSystem::FallbackAgentRadius, float Height = FNavigationSystem::FallbackAgentHeight);

	ENGINE_API FNavDataConfig(const FNavDataConfig& Other);
	ENGINE_API FNavDataConfig& operator=(const FNavDataConfig& Other);

	ENGINE_API bool IsValid() const; 
	ENGINE_API void Invalidate();

	ENGINE_API void SetNavDataClass(UClass* InNavDataClass);
	ENGINE_API void SetNavDataClass(TSoftClassPtr<AActor> InNavDataClass);
	
	template<typename T>
	TSubclassOf<T> GetNavDataClass() const { return TSubclassOf<T>(NavDataClass.Get()); }

	ENGINE_API FString GetDescription() const;

#if WITH_EDITOR
	static FName GetNavigationDataClassPropertyName()
	{
		static const FName NAME_NavigationDataClass = GET_MEMBER_NAME_CHECKED(FNavDataConfig, NavDataClass);
		return NAME_NavigationDataClass;
	}
#endif // WITH_EDITOR
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

struct FNavigationProjectionWork
{
	// source point
	const FVector Point;
	
	// projection range
	FBox ProjectionLimit;

	// result point with nav Id
	FNavLocation OutLocation;

	// if set, projection function scoring will be biased for 2D work (e.g. in case of navmesh, findNearestPoly2D)
	uint32 bHintProjection2D : 1;

	// result of projection function. 'true' means nav projection was successful
	// and OutLocation contains Point projected to the nav data's surface
	uint32 bResult : 1;

	// if set, data in this structure is valid
	uint32 bIsValid : 1;

	explicit FNavigationProjectionWork(const FVector& StartPoint, const FBox& CustomProjectionLimits = FBox(ForceInit))
		: Point(StartPoint), ProjectionLimit(CustomProjectionLimits), bHintProjection2D(false), bResult(false), bIsValid(true)
	{}

	FNavigationProjectionWork()
		: Point(FNavigationSystem::InvalidLocation), ProjectionLimit(ForceInit), bHintProjection2D(false), bResult(false), bIsValid(false)
	{}
};

struct FRayStartEnd
{
	const FVector RayStart;
	const FVector RayEnd;
	explicit FRayStartEnd(const FVector& InRayStart = FNavigationSystem::InvalidLocation, const FVector& InRayEnd = FNavigationSystem::InvalidLocation)
		: RayStart(InRayStart), RayEnd(InRayEnd)
	{}
};

struct FNavigationRaycastWork : FRayStartEnd
{
	/** depending on bDidHit HitLocation contains either actual hit location or RayEnd*/
	FNavLocation HitLocation;
	bool bDidHit;
	/** when bDidHit is false, bIsRayEndInCorridor indicates if the projection of RayEnd is located in the corridor explored from the ray. 
    *  When bIsRayEndInCorridor is false, it means that RayEnd failed to project to the NavigationData or on a navigation node that is not part of the explored corridor (e.g. different height) */
	bool bIsRayEndInCorridor;

	FNavigationRaycastWork(const FVector& InRayStart, const FVector& InRayEnd)
		: FRayStartEnd(InRayStart, InRayEnd), HitLocation(InRayEnd), bDidHit(false), bIsRayEndInCorridor(false)
	{}
};

struct FNavigationFindMoveAlongSurfaceWork
{
	FNavLocation StartLocation;

	/** If the move along the surface was found (ie, bResult is set to TRUE), stores the navmesh surface location and poly where it ends */
	FNavLocation ResultNavLocation;

	FVector DesiredEndLocation;

	/** If set, stores the result of the underlying find surface move call
	 *  TRUE if the underlying find surface move call succeeded -- in this case ResultNavLocation will store the navmesh surface location and poly where it ends
	 *  FALSE if the underlying find surface move call failed */
	TOptional<bool> bResult;

	FNavigationFindMoveAlongSurfaceWork(const FNavLocation& InStartLocation = FNavLocation(), const FVector& InDesiredEndLocation = FNavigationSystem::InvalidLocation)
		: StartLocation(InStartLocation), ResultNavLocation(), DesiredEndLocation(InDesiredEndLocation), bResult()
	{}
};

UENUM()
namespace ENavigationQueryResult
{
	enum Type : int
	{
		Invalid,
		Error,
		Fail,
		Success
	};
}

/**
*	Delegate used to communicate that path finding query has been finished.
*	@param uint32 unique Query ID of given query
*	@param ENavigationQueryResult enum expressed query result.
*	@param FNavPathSharedPtr resulting path. Valid only for ENavigationQueryResult == ENavigationQueryResult::Fail
*		(may contain path leading as close to destination as possible)
*		and ENavigationQueryResult == ENavigationQueryResult::Success
*/
DECLARE_DELEGATE_ThreeParams(FNavPathQueryDelegate, uint32, ENavigationQueryResult::Type, FNavPathSharedPtr);

//////////////////////////////////////////////////////////////////////////
// Memory stating

namespace NavMeshMemory
{
#if STATS
	// @todo could be made a more generic solution
	class FNavigationMemoryStat : public FDefaultAllocator
	{
	public:
		typedef FDefaultAllocator Super;

		class ForAnyElementType : public FDefaultAllocator::ForAnyElementType
		{
		public:
			typedef FDefaultAllocator::ForAnyElementType Super;
		private:
			int64 AllocatedSize;
		public:

			ForAnyElementType()
				: AllocatedSize(0)
			{

			}

			/** Destructor. */
			~ForAnyElementType()
			{
				if (AllocatedSize)
				{
					DEC_DWORD_STAT_BY(STAT_NavigationMemory, AllocatedSize);
				}
			}

			inline void ResizeAllocation(int32 PreviousNumElements, int32 NumElements, int32 NumBytesPerElement, uint32 AlignmentOfElement)
			{
				// Maintain existing behavior, call the default aligned version of this function.
				// We currently rely on this as we are often storing actual structs into uint8's here.
				ResizeAllocation(PreviousNumElements, NumElements, NumBytesPerElement);
			}

			inline void ResizeAllocation(SizeType PreviousNumElements, SizeType NumElements, SIZE_T NumBytesPerElement)
			{
				const int64 NewSize = NumElements * NumBytesPerElement;
				INC_DWORD_STAT_BY(STAT_NavigationMemory, NewSize - AllocatedSize);
				AllocatedSize = NewSize;

				Super::ResizeAllocation(PreviousNumElements, NumElements, NumBytesPerElement);
			}

		private:
			ForAnyElementType(const ForAnyElementType&);
			ForAnyElementType& operator=(const ForAnyElementType&);
		};

		template<typename ElementType>
		class ForElementType : public ForAnyElementType
		{
		public:
			ElementType* GetAllocation() const
			{
				return (ElementType*)ForAnyElementType::GetAllocation();
			}
		};
	};

	typedef FNavigationMemoryStat FNavAllocator;
#else
	typedef FDefaultAllocator FNavAllocator;
#endif
}

#if STATS

template <>
struct TAllocatorTraits<NavMeshMemory::FNavigationMemoryStat> : TAllocatorTraits<NavMeshMemory::FNavigationMemoryStat::Super>
{
};

#endif

template<typename InElementType>
class TNavStatArray : public TArray<InElementType, NavMeshMemory::FNavAllocator>
{
public:
	typedef TArray<InElementType, NavMeshMemory::FNavAllocator> Super;
};


//----------------------------------------------------------------------//
// generic "landscape" support
//----------------------------------------------------------------------//
struct FNavHeightfieldSamples
{
	TNavStatArray<int16> Heights;
	TBitArray<> Holes;

	ENGINE_API FNavHeightfieldSamples();
	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	ENGINE_API void Empty();
	inline bool IsEmpty() const { return Heights.IsEmpty() && Holes.IsEmpty();  }
};
