// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "PathFollowingAgentInterface.generated.h"

struct FHitResult;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UPathFollowingAgentInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IPathFollowingAgentInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Called when pathfollowing agent is not able to move anymore */
	virtual void OnUnableToMove(const UObject& Instigator) PURE_VIRTUAL(IPathFollowingAgentInterface::OnUnableToMove, );
	
	/** Called when falling movement starts. */
	virtual void OnStartedFalling() {}

	/** Called when falling movement ends. */
	virtual void OnLanded() {}

	/** Called when movement is blocked by a collision with another actor.  */
	virtual void OnMoveBlockedBy(const FHitResult& BlockingImpact) {}

	/** Returns true if the agent is currently following a NavLink on their path */
	virtual bool IsFollowingNavLink() const
	{
		return false;
	} 
};
