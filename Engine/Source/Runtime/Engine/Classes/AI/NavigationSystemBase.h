// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/NavigationModifier.h"
#include "Engine/World.h"
#include "NavigationSystemBase.generated.h"

class UNavigationSystemBase;
class UNavigationSystemConfig;
class AActor;
class UActorComponent;
class USceneComponent;
class INavigationDataInterface;
class IPathFollowingAgentInterface;
class AWorldSettings;
class ULevel;
class AController;
class UNavAreaBase;
struct FNavigationElement;
struct FNavigationElementHandle;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNavigation, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNavigationDataBuild, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNavigationHistory, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNavInvokers, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNavLink, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAStar, Display, All);

UENUM()
enum class FNavigationSystemRunMode : uint8
{
	InvalidMode,
	GameMode,
	EditorMode,
	SimulationMode UE_DEPRECATED(5.3, "No longer used.") UMETA(Hidden, DisplayName = "DEPRECATED No longer used, to be removed soon"),
	PIEMode,
	InferFromWorldMode,
	EditorWorldPartitionBuildMode
};

namespace ENavigationLockReason
{
	enum Type
	{
		Unknown = 1 << 0,
		AllowUnregister = 1 << 1,

		MaterialUpdate = 1 << 2,
		LightingUpdate = 1 << 3,
		ContinuousEditorMove = 1 << 4,
		SpawnOnDragEnter = 1 << 5,
	};
}

class FNavigationLockContext
{
public:
	FNavigationLockContext(ENavigationLockReason::Type Reason = ENavigationLockReason::Unknown, bool bApplyLock = true)
		: MyWorld(NULL), LockReason((uint8)Reason), bSingleWorld(false), bIsLocked(false)
	{
		if (bApplyLock)
		{
			LockUpdates();
		}
	}

	FNavigationLockContext(UWorld* InWorld, ENavigationLockReason::Type Reason = ENavigationLockReason::Unknown, bool bApplyLock = true)
		: MyWorld(InWorld), LockReason((uint8)Reason), bSingleWorld(true), bIsLocked(false)
	{
		if (bApplyLock)
		{
			LockUpdates();
		}
	}

	~FNavigationLockContext()
	{
		UnlockUpdates();
	}

private:
	UWorld* MyWorld;
	uint8 LockReason;
	uint8 bSingleWorld : 1;
	uint8 bIsLocked : 1;

	ENGINE_API void LockUpdates();
	ENGINE_API void UnlockUpdates();
};

namespace FNavigationSystem
{
	/** Creates an instance of NavigationSystem (class being specified by WorldSetting's NavigationSystemConfig)
	 *	A new instance will be created only if given WorldOwner doesn't have one yet.
	 *	The new instance will be assigned to the given WorldOwner (via SetNavigationSystem 
	 *	call) and depending on value of bInitializeForWorld the InitializeForWorld 
	 *	function will be called on the new NavigationSystem instance.
	 *	(@see UWorld.NavigationSystem)
	 *	@param RunMode if set to a valid value (other than FNavigationSystemRunMode::InvalidMode) 
	 *		will also configure the created NavigationSystem instance for that mode
	 *	@param NavigationSystemConfig is used to pick the navigation system's class and set it up. If null
	 *		then WorldOwner.WorldSettings.NavigationSystemConfig will be used
	 */
	ENGINE_API void AddNavigationSystemToWorld(UWorld& WorldOwner, const FNavigationSystemRunMode RunMode = FNavigationSystemRunMode::InvalidMode, UNavigationSystemConfig* NavigationSystemConfig = nullptr, const bool bInitializeForWorld = true, const bool bOverridePreviousNavSys = false);

	/** Discards all navigation data chunks in all sub-levels */
	ENGINE_API void DiscardNavigationDataChunks(UWorld& InWorld);

	ENGINE_API bool IsEditorRunMode(const FNavigationSystemRunMode Mode);
	
	template<typename TNavSys>
	inline TNavSys* GetCurrent(UWorld* World)
	{
		return World ? Cast<TNavSys>(World->GetNavigationSystem()) : (TNavSys*)nullptr;
	}

	template<typename TNavSys>
	inline const TNavSys* GetCurrent(const UWorld* World)
	{
		return World ? Cast<TNavSys>(World->GetNavigationSystem()) : (const TNavSys*)nullptr;
	}

	ENGINE_API UWorld* GetWorldFromContextObject(UObject* WorldContextObject);

	template<typename TNavSys>
	TNavSys* GetCurrent(UObject* WorldContextObject)
	{
		UWorld* World = GetWorldFromContextObject(WorldContextObject);
		return GetCurrent<TNavSys>(World);
	}

	ENGINE_API void UpdateActorData(AActor& Actor);
	ENGINE_API void UpdateComponentData(UActorComponent& Comp);
	ENGINE_API void UpdateActorAndComponentData(AActor& Actor, bool bUpdateAttachedActors = true);
	ENGINE_API void UpdateComponentDataAfterMove(USceneComponent& Comp);
	//ENGINE_API bool HasComponentData(UActorComponent& Comp);
	ENGINE_API void OnActorBoundsChanged(AActor& Actor);
	ENGINE_API void OnPostEditActorMove(AActor& Actor);

	UE_DEPRECATED(5.4, "Use OnObjectBoundsChanged taking UObject and list of dirty areas as parameters instead.")
	ENGINE_API void OnComponentBoundsChanged(UActorComponent& Comp, const FBox& NewBounds, const FBox& DirtyArea);
	ENGINE_API void OnComponentTransformChanged(USceneComponent& Comp);

	ENGINE_API void OnActorRegistered(AActor& Actor);
	ENGINE_API void OnActorUnregistered(AActor& Actor);

	ENGINE_API void OnComponentRegistered(UActorComponent& Comp);
	ENGINE_API void OnComponentUnregistered(UActorComponent& Comp);
	
	ENGINE_API void RegisterComponent(UActorComponent& Comp);
	ENGINE_API void UnregisterComponent(UActorComponent& Comp);

	ENGINE_API void RegisterNavRelevantObject(UObject& Object);
	ENGINE_API void UpdateNavRelevantObject(UObject& Object);
	ENGINE_API void UnregisterNavRelevantObject(UObject& Object);
	ENGINE_API void OnObjectBoundsChanged(UObject& Object, const FBox& NewBounds, TConstArrayView<FBox> DirtyAreas);

	ENGINE_API bool SupportsDynamicChanges(UWorld* World);
	ENGINE_API FNavigationElementHandle AddNavigationElement(UWorld* World, FNavigationElement&& Element);
	ENGINE_API void UpdateNavigationElement(UWorld* World, FNavigationElementHandle ElementHandle, FNavigationElement&& Element);
	ENGINE_API void RemoveNavigationElement(UWorld* World, FNavigationElementHandle ElementHandle);
	ENGINE_API void OnNavigationElementBoundsChanged(UWorld* World, FNavigationElementHandle ElementHandle, const FBox& NewBounds, TConstArrayView<FBox> DirtyAreas);

	ENGINE_API void RemoveActorData(AActor& Actor);

	ENGINE_API bool HasComponentData(UActorComponent& Comp);

	ENGINE_API const FNavDataConfig& GetDefaultSupportedAgent();
	ENGINE_API const FNavDataConfig& GetBiggestSupportedAgent(const UWorld* World);
	ENGINE_API double GetWorldPartitionNavigationDataBuilderOverlap(const UWorld& World);

	ENGINE_API TSubclassOf<UNavAreaBase> GetDefaultWalkableArea();
	ENGINE_API TSubclassOf<UNavAreaBase> GetDefaultObstacleArea();

	/**	Retrieves the transform the Navigation System is using to convert coords
	 *	from FromCoordType to ToCoordType */
	ENGINE_API const FTransform& GetCoordTransform(const ENavigationCoordSystem::Type FromCoordType, const ENavigationCoordSystem::Type ToCoordType);

	ENGINE_API bool WantsComponentChangeNotifies();

	ENGINE_API INavigationDataInterface* GetNavDataForActor(const AActor& Actor);
	ENGINE_API TSubclassOf<AActor> GetDefaultNavDataClass();

	ENGINE_API void VerifyNavigationRenderingComponents(UWorld& World, const bool bShow);
	ENGINE_API void Build(UWorld& World);

#if WITH_EDITOR
	ENGINE_API void OnPIEStart(UWorld& World);
	ENGINE_API void OnPIEEnd(UWorld& World);
	ENGINE_API void SetNavigationAutoUpdateEnabled(const bool bNewEnable, UNavigationSystemBase* InNavigationSystem);
	ENGINE_API void UpdateLevelCollision(ULevel& Level);
#endif // WITH_EDITOR

	enum class ECleanupMode : uint8
	{
		CleanupWithWorld,
		CleanupUnsafe,
	};

	// pathfollowing
	ENGINE_API bool IsFollowingAPath(const AController& Controller);
	ENGINE_API void StopMovement(const AController& Controller);
	ENGINE_API IPathFollowingAgentInterface* FindPathFollowingAgentForActor(const AActor& Actor);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FSupportsDynamicChangesSignature, UWorld* /*World*/);
	DECLARE_DELEGATE_RetVal_TwoParams(FNavigationElementHandle, FAddNavigationElementSignature, UWorld* /*World*/, FNavigationElement&& /*Element*/);
	DECLARE_DELEGATE_TwoParams(FRemoveNavigationElementSignature, UWorld* /*World*/, FNavigationElementHandle /*ElementHandle*/);
	DECLARE_DELEGATE_ThreeParams(FUpdateNavigationElementSignature, UWorld* /*World*/, FNavigationElementHandle /*ElementHandle*/, FNavigationElement&& /*Element*/)
	DECLARE_DELEGATE_FourParams(FUpdateNavigationElementBoundsSignature, UWorld* /*World*/, FNavigationElementHandle /*ElementHandle*/, const FBox& /*NewBounds*/, TConstArrayView<FBox> /*DirtyAreas*/)

	DECLARE_DELEGATE_OneParam(FObjectBasedSignature, UObject& /*Object*/);
	DECLARE_DELEGATE_ThreeParams(FObjectBoundsChangedSignature, UObject& /*Object*/, const FBox& /*NewBounds*/, TConstArrayView<FBox> /*DirtyAreas*/)
	DECLARE_DELEGATE_OneParam(FActorBasedSignature, AActor& /*Actor*/);
	DECLARE_DELEGATE_OneParam(FActorComponentBasedSignature, UActorComponent& /*Comp*/);
	DECLARE_DELEGATE_OneParam(FSceneComponentBasedSignature, USceneComponent& /*Comp*/);
	DECLARE_DELEGATE_OneParam(FWorldBasedSignature, UWorld& /*World*/);
	DECLARE_DELEGATE_OneParam(FLevelBasedSignature, ULevel& /*Level*/);
	DECLARE_DELEGATE_OneParam(FControllerBasedSignature, const AController& /*Controller*/);
	DECLARE_DELEGATE_TwoParams(FNavigationAutoUpdateEnableSignature, const bool /*bNewEnable*/, UNavigationSystemBase* /*InNavigationSystem*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FBoolControllerBasedSignature, const AController& /*Controller*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FBoolActorComponentBasedSignature, UActorComponent& /*Comp*/);
	DECLARE_DELEGATE_RetVal(TSubclassOf<UNavAreaBase>, FNavAreaBasedSignature);
	DECLARE_DELEGATE_RetVal(const FNavDataConfig&, FNavDataConfigBasedSignature);
	DECLARE_DELEGATE_RetVal_OneParam(const FNavDataConfig&, FNavDataConfigAndWorldSignature, const UWorld* /*World*/);
	DECLARE_DELEGATE_RetVal_OneParam(double, FDoubleWorldBasedSignature, const UWorld& /*World*/);
	DECLARE_DELEGATE_TwoParams(FWorldByteBasedSignature, UWorld& /*World*/, uint8 /*Flags*/);
	DECLARE_DELEGATE_TwoParams(FActorBooleBasedSignature, AActor& /*Actor*/, bool /*bUpdateAttachedActors*/);
	DECLARE_DELEGATE_RetVal_OneParam(INavigationDataInterface*, FNavDataForPropsSignature, const FNavAgentProperties& /*AgentProperties*/);
	DECLARE_DELEGATE_RetVal_OneParam(INavigationDataInterface*, FNavDataForActorSignature, const AActor& /*Actor*/);
	DECLARE_DELEGATE_RetVal(TSubclassOf<AActor>, FNavDataClassFetchSignature);
	DECLARE_DELEGATE_TwoParams(FWorldBoolBasedSignature, UWorld& /*World*/, const bool /*bShow*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNavigationInitSignature, const UNavigationSystemBase&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNavAreaGenericEvent, const UWorld&, const UClass*);

	UE_DEPRECATED(5.4, "Use FObjectBoundsChangedSignature delegate taking UObject and list of dirty areas as parameters instead.")
	DECLARE_DELEGATE_ThreeParams(FComponentBoundsChangeSignature, UActorComponent& /*Comp*/, const FBox& /*NewBounds*/, const FBox& /*DirtyArea*/)
}


UCLASS(Abstract, config = Engine, defaultconfig, Transient, MinimalAPI)
class UNavigationSystemBase : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UNavigationSystemBase()= default;

	virtual void Tick(float DeltaSeconds) PURE_VIRTUAL(UNavigationSystemBase::Tick, );
	virtual void CleanUp(const FNavigationSystem::ECleanupMode Mode) PURE_VIRTUAL(UNavigationSystemBase::CleanUp, );
	virtual void Configure(const UNavigationSystemConfig& Config) PURE_VIRTUAL(UNavigationSystemBase::Configure, );
	/** Called when there's a need to extend current navigation system's config with information in NewConfig */
	virtual void AppendConfig(const UNavigationSystemConfig& NewConfig) PURE_VIRTUAL(UNavigationSystemBase::AppendConfig, );

	/**
	*	Called when owner-UWorld initializes actors
	*/
	virtual void OnInitializeActors() {}

	virtual bool IsNavigationBuilt(const AWorldSettings* Settings) const { return false; }

	virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) PURE_VIRTUAL(UNavigationSystemBase::ApplyWorldOffset, );

	virtual void InitializeForWorld(UWorld& World, FNavigationSystemRunMode Mode) PURE_VIRTUAL(UNavigationSystemBase::InitializeForWorld, );

	/** 
	 *	If you're using NavigationSysstem module consider calling 
	 *	FNavigationSystem::GetCurrent<UNavigationSystemV1>()->GetDefaultNavDataInstance 
	 *	instead.
	 */
	virtual INavigationDataInterface* GetMainNavData() const { return nullptr; }

	virtual void SetBuildBounds(const FBox& Bounds) PURE_VIRTUAL(UNavigationSystemBase::SetBuildBounds, );

	virtual FBox GetNavigableWorldBounds() const PURE_VIRTUAL(UNavigationSystemBase::GetNavigableWorldBounds, return FBox(ForceInit););
	
	virtual bool ContainsNavData(const FBox& Bounds) const PURE_VIRTUAL(UNavigationSystemBase::ContainsNavData, return false;);
	virtual FBox ComputeNavDataBounds() const PURE_VIRTUAL(UNavigationSystemBase::GetNavigableWorldBounds, return FBox(ForceInit););
	
	virtual void AddNavigationDataChunk(class ANavigationDataChunkActor& DataChunkActor) {}
	virtual void RemoveNavigationDataChunk(class ANavigationDataChunkActor& DataChunkActor) {}
	virtual void FillNavigationDataChunkActor(const FBox& QueryBounds, class ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds) {}

	virtual bool IsWorldInitDone() const PURE_VIRTUAL(UNavigationSystemBase::IsWorldInitDone, return false;);

	/**
	 * Indicates whether a navigation system instance should be created for a given world.
	 * @param World The world in which the navigation would be used
	 * @return whether the navigation system should be created for the specified world or not
	 */
	virtual bool ShouldCreateNavigationSystemInstance(const UWorld* World) const PURE_VIRTUAL(UNavigationSystemBase::ShouldCreateNavigationSystemInstance, return true;);

	static ENGINE_API FNavigationSystem::FOnNavigationInitSignature& OnNavigationInitStartStaticDelegate();
	static ENGINE_API FNavigationSystem::FOnNavigationInitSignature& OnNavigationInitDoneStaticDelegate();
	static ENGINE_API FNavigationSystem::FOnNavAreaGenericEvent& OnNavAreaRegisteredDelegate();
	static ENGINE_API FNavigationSystem::FOnNavAreaGenericEvent& OnNavAreaUnregisteredDelegate();

protected:
	/**	Sets the Transform the Navigation System will use when converting from FromCoordType
	 *	to ToCoordType
	 *	@param bAddInverse if true (default) will also set coord transform in 
	 *		the reverse order using Transform.Inverse() */
	static ENGINE_API void SetCoordTransform(const ENavigationCoordSystem::Type FromCoordType, const ENavigationCoordSystem::Type ToCoordType, const FTransform& Transform, bool bAddInverse = true);
	static ENGINE_API void SetWantsComponentChangeNotifies(const bool bEnable);
	static ENGINE_API void SetDefaultWalkableArea(TSubclassOf<UNavAreaBase> InAreaClass);
	static ENGINE_API void SetDefaultObstacleArea(TSubclassOf<UNavAreaBase> InAreaClass);

	static ENGINE_API void ResetEventDelegates();

	static ENGINE_API FNavigationSystem::FSupportsDynamicChangesSignature& GetSupportsDynamicChangesDelegate();
	static ENGINE_API FNavigationSystem::FAddNavigationElementSignature& GetAddNavigationElementDelegate();
	static ENGINE_API FNavigationSystem::FUpdateNavigationElementSignature& GetUpdateNavigationElementDelegate();
	static ENGINE_API FNavigationSystem::FRemoveNavigationElementSignature& GetRemoveNavigationElementDelegate();
	static ENGINE_API FNavigationSystem::FUpdateNavigationElementBoundsSignature& GetUpdateNavigationElementBoundsDelegate();

	static ENGINE_API FNavigationSystem::FObjectBasedSignature& RegisterNavRelevantObjectDelegate();
	static ENGINE_API FNavigationSystem::FObjectBasedSignature& UpdateNavRelevantObjectDelegate();
	static ENGINE_API FNavigationSystem::FObjectBasedSignature& UnregisterNavRelevantObjectDelegate();
	static ENGINE_API FNavigationSystem::FObjectBoundsChangedSignature& OnObjectBoundsChangedDelegate();

	static ENGINE_API FNavigationSystem::FActorBasedSignature& UpdateActorDataDelegate();
	static ENGINE_API FNavigationSystem::FActorComponentBasedSignature& UpdateComponentDataDelegate();
	static ENGINE_API FNavigationSystem::FSceneComponentBasedSignature& UpdateComponentDataAfterMoveDelegate();
	static ENGINE_API FNavigationSystem::FActorBasedSignature& OnActorBoundsChangedDelegate();
	static ENGINE_API FNavigationSystem::FActorBasedSignature& OnPostEditActorMoveDelegate();
	static ENGINE_API FNavigationSystem::FSceneComponentBasedSignature& OnComponentTransformChangedDelegate();
	static ENGINE_API FNavigationSystem::FActorBasedSignature& OnActorRegisteredDelegate();
	static ENGINE_API FNavigationSystem::FActorBasedSignature& OnActorUnregisteredDelegate();
	static ENGINE_API FNavigationSystem::FActorComponentBasedSignature& OnComponentRegisteredDelegate();
	static ENGINE_API FNavigationSystem::FActorComponentBasedSignature& OnComponentUnregisteredDelegate();
	static ENGINE_API FNavigationSystem::FActorComponentBasedSignature& RegisterComponentDelegate();
	static ENGINE_API FNavigationSystem::FActorComponentBasedSignature& UnregisterComponentDelegate();
	static ENGINE_API FNavigationSystem::FActorBasedSignature& RemoveActorDataDelegate();
	static ENGINE_API FNavigationSystem::FBoolActorComponentBasedSignature& HasComponentDataDelegate();
	static ENGINE_API FNavigationSystem::FNavDataConfigBasedSignature& GetDefaultSupportedAgentDelegate();
	static ENGINE_API FNavigationSystem::FNavDataConfigAndWorldSignature& GetBiggestSupportedAgentDelegate();
	static ENGINE_API FNavigationSystem::FActorBooleBasedSignature& UpdateActorAndComponentDataDelegate();
	static ENGINE_API FNavigationSystem::FNavDataForActorSignature& GetNavDataForActorDelegate();
	static ENGINE_API FNavigationSystem::FNavDataClassFetchSignature& GetDefaultNavDataClassDelegate();
	static ENGINE_API FNavigationSystem::FWorldBoolBasedSignature& VerifyNavigationRenderingComponentsDelegate();
	static ENGINE_API FNavigationSystem::FWorldBasedSignature& BuildDelegate();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static ENGINE_API FNavigationSystem::FComponentBoundsChangeSignature& OnComponentBoundsChangedDelegate();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#if WITH_EDITOR
	static ENGINE_API FNavigationSystem::FWorldBasedSignature& OnPIEStartDelegate();
	static ENGINE_API FNavigationSystem::FWorldBasedSignature& OnPIEEndDelegate();
	static ENGINE_API FNavigationSystem::FLevelBasedSignature& UpdateLevelCollisionDelegate();
	static ENGINE_API FNavigationSystem::FNavigationAutoUpdateEnableSignature& SetNavigationAutoUpdateEnableDelegate();
	static ENGINE_API FNavigationSystem::FWorldByteBasedSignature& AddNavigationUpdateLockDelegate();
	static ENGINE_API FNavigationSystem::FWorldByteBasedSignature& RemoveNavigationUpdateLockDelegate();
	static ENGINE_API FNavigationSystem::FDoubleWorldBasedSignature& GetWorldPartitionNavigationDataBuilderOverlapDelegate();
#endif // WITH_EDITOR
};


class IPathFollowingManagerInterface
{
protected:
	static ENGINE_API FNavigationSystem::FControllerBasedSignature& StopMovementDelegate();
	static ENGINE_API FNavigationSystem::FBoolControllerBasedSignature& IsFollowingAPathDelegate();
};
