// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "NavigationSystemConfig.generated.h"

class UNavigationSystemBase;
class UWorld;


UCLASS(config = Engine, defaultconfig, EditInlineNew, DisplayName = "Generic Navigation System Config", collapseCategories, MinimalAPI)
class UNavigationSystemConfig : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Navigation, meta = (MetaClass = "/Script/Engine.NavigationSystemBase", NoResetToDefault))
	FSoftClassPath NavigationSystemClass;

	/** NavigationSystem's properties in Project Settings define all possible supported agents,
	 *	but a specific navigation system can choose to support only a subset of agents.*/
	UPROPERTY(EditAnywhere, Category = Navigation)
	FNavAgentSelector SupportedAgentsMask;

	/** If not None indicates which of navigation datas and supported agents are 
	 * going to be used as the default ones. If navigation agent of this type does 
	 * not exist or is not enabled then the first available nav data will be used 
	 * as the default one */
	UPROPERTY(EditAnywhere, Category = Navigation)
	FName DefaultAgentName;

protected:
	/** If true it means the navigation system settings are overridden from another source (like a NavConfigOverrideActor) */
	UPROPERTY(VisibleAnywhere, Category = Navigation)
	uint32 bIsOverriden : 1; 
	
public:
	ENGINE_API UNavigationSystemConfig(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API virtual UNavigationSystemBase* CreateAndConfigureNavigationSystem(UWorld& World) const;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	void SetIsOverriden(const bool bInNewValue) { bIsOverriden = bInNewValue; }

	static ENGINE_API TSubclassOf<UNavigationSystemConfig> GetDefaultConfigClass();
};

UCLASS(MinimalAPI, HideCategories=Navigation)
class UNullNavSysConfig : public UNavigationSystemConfig
{
	GENERATED_BODY()
public:
	UNullNavSysConfig(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
};
