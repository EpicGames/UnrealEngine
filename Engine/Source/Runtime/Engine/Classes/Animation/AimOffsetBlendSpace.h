// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Blend Space. Contains a grid of data points with weights from sample points in the space
 *
 */

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/BlendSpace.h"

#include "AimOffsetBlendSpace.generated.h"

/**
 * An Aim Offset is an asset that stores a blendable series of poses to help a character aim a weapon.
 */
UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UAimOffsetBlendSpace : public UBlendSpace
{
	GENERATED_UCLASS_BODY()
		
	virtual bool IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const override;
	virtual bool IsValidAdditive() const override;
};
