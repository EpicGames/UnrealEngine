// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Blend Space 1D. Contains 1 axis blend 'space'
 *
 */

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/BlendSpace1D.h"
#include "AimOffsetBlendSpace1D.generated.h"

 /**
  * An Aim Offset is an asset that stores a blendable series of poses to help a character aim a weapon.
  */
UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UAimOffsetBlendSpace1D : public UBlendSpace1D
{
	GENERATED_UCLASS_BODY()

	virtual bool IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const override;
	virtual bool IsValidAdditive() const override;
};
