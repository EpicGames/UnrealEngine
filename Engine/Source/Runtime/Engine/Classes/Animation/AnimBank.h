// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationAsset.h"
#include "Animation/TransformProviderData.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "HLOD/HLODBatchingPolicy.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "IO/IoHash.h"
#include "Logging/LogMacros.h"
#include "PhysicsEngine/BodyInstance.h"
#include "RenderCommandFence.h"
#include "SkinningDefinitions.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AnimTrackPool.h"
#include "AnimSequenceTransformProviderData.h"
#include "AnimBank.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimBank, Log, All);

class FQueuedThreadPool;
enum class EQueuedWorkPriority : uint8;
struct FPropertyChangedEvent;

struct FRenderBounds;
struct FReferenceSkeleton;
class USkinnedAsset;
class UInstancedSkinnedMeshComponent;
class FInstancedSkinningSceneExtensionProxy;
class FAnimBankBuildAsyncCacheTask;
class IAnimBankTransformProvider;

struct FSkinnedAssetMapping
{
	// Bone transforms in global pose.
	TArray<FTransform> MeshGlobalRefPose;

	// Inverse global space transforms
	TArray<FVector3f> PositionKeys;
	TArray<FQuat4f> RotationKeys;

	uint32 BoneCount = 0;
};

inline FArchive& operator<<(FArchive& Ar, FSkinnedAssetMapping& AssetMapping)
{
	Ar << AssetMapping.MeshGlobalRefPose;
	Ar << AssetMapping.PositionKeys;
	Ar << AssetMapping.RotationKeys;
	Ar << AssetMapping.BoneCount;
	return Ar;
}

struct FAnimBankEntry
{
	TArray<FVector3f>	PositionKeys;
	TArray<FQuat4f>		RotationKeys;
	TArray<FVector3f>	ScalingKeys;

	/**
		Note: This is almost fully conservative, but since it is derived from 
		bone positions on the skeleton (not skinning all verts across all frames)
		it could have some edge cases for (presumably) strange content.

		This hasn't been an issue in practice yet, so we won't worry about it,
		and each anim bank sequence has an optional BoundsScale that can be adjusted
		to account for certain cases that might fail.

		One possible future idea if needed, is to calculate a per-bone influence radius
		in the skeleton mesh build, where each bone has a bounding sphere of all weighted
		vertex positions. Then we could try something like the following to make the
		bounds possibly fit this content better.

		InitialAnimatedBoundsMin(AssetBounds.Origin - AssetBounds.BoxExtent);
		InitialAnimatedBoundsMax(AssetBounds.Origin + AssetBounds.BoxExtent);
		For each Key,Bone:
			AnimatedBoundsMin = Min(AnimatedBoundsMin, InitialAnimatedBoundsMin + Bone.Pos[Key] - Bone.RefPos)
			AnimatedBoundsMax = Max(AnimatedBoundsMax, InitialAnimatedBoundsMax + Bone.Pos[Key] - Bone.RefPos)
	*/
	FBoxSphereBounds SampledBounds;

	float Position		= 0.0f;
	float PlayRate		= 1.0f;

	uint32 FrameCount	= 0u;
	uint32 KeyCount		= 0u;
	uint32 Flags		= 0u;

	inline float GetPlayLength() const
	{
		const float SampleInterval = 1.0f / ANIM_BANK_SAMPLE_RATE;
		return float(FrameCount - 1) * SampleInterval;
	}

	inline bool IsLooping() const
	{
		return (Flags & ANIM_BANK_FLAG_LOOPING) != 0;
	}

	inline bool IsAutoStart() const
	{
		return (Flags & ANIM_BANK_FLAG_AUTOSTART) != 0;
	}
};

inline FArchive& operator<<(FArchive& Ar, FAnimBankEntry& BankEntry)
{
	Ar << BankEntry.PositionKeys;
	Ar << BankEntry.RotationKeys;
	Ar << BankEntry.ScalingKeys;
	Ar << BankEntry.SampledBounds;
	Ar << BankEntry.Position;
	Ar << BankEntry.PlayRate;
	Ar << BankEntry.FrameCount;
	Ar << BankEntry.KeyCount;
	Ar << BankEntry.Flags;
	return Ar;
}

struct FAnimBankData
{
	FSkinnedAssetMapping Mapping;
	TArray<FAnimBankEntry> Entries;
};

inline FArchive& operator<<(FArchive& Ar, FAnimBankData& BankData)
{
	Ar << BankData.Mapping;
	Ar << BankData.Entries;
	return Ar;
}

USTRUCT(BlueprintType)
struct FAnimBankSequence
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class UAnimSequence> Sequence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "Looping"))
	uint32 bLooping : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "AutoStart"))
	uint32 bAutoStart : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "Position"))
	float Position;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "PlayRate"))
	float PlayRate;

	/**
	 * Scales the bounds of the instances playing this sequence.
	 * This is useful when the animation moves the vertices of the mesh outside of its bounds.
	 * Warning: Increasing the bounds will reduce performance!
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (UIMin = "1", UIMax = "10.0"))
	float BoundsScale;

	FAnimBankSequence()
	{
		Sequence	= nullptr;
		BoundsScale	= 1.0f;
		PlayRate	= 1.0f;
		bLooping 	= true;
		bAutoStart	= true;
		Position	= 0.0f;
	}

	ENGINE_API void ValidatePosition();
};

UCLASS(BlueprintType, hidecategories=Object, editinlinenew, MinimalAPI)
class UAnimBank : public UAnimationAsset, public IInterface_AsyncCompilation
{
	GENERATED_BODY()

private:
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnGPUDataChanged);
#endif

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sequences, meta = (ShowOnlyInnerProperties))
	TArray<struct FAnimBankSequence> Sequences;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mapping, meta = (ShowOnlyInnerProperties))
	TObjectPtr<USkinnedAsset> Asset;

public:
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	void InitResources();
	void ReleaseResources();

	inline const FAnimBankData& GetData() const
	{
#if WITH_EDITOR
		ensure(!IsCompiling());
#endif
		return Data;
	}

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimBank")
	ENGINE_API float GetSequencePlayLength(int32 SequenceIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimBank")
	ENGINE_API bool IsSequenceLooping(int32 SequenceIndex) const;

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimBank")
	inline float GetSequenceCount() const
	{
#if WITH_EDITORONLY_DATA
		return static_cast<float>(Sequences.Num());
#else
		return static_cast<float>(GetData().Entries.Num());
#endif
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	virtual void ClearAllCachedCookedPlatformData() override;

	virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;

	/** Returns whether or not the asset is currently being compiled */
	bool IsCompiling() const override;

	/** Try to cancel any pending async tasks.
	*  Returns true if there is no more async tasks pending, false otherwise.
	*/
	bool TryCancelAsyncTasks();

	/** Returns false if there is currently an async task running */
	bool IsAsyncTaskComplete() const;

	/**
	* Wait until all async tasks are complete, up to a time limit
	* Returns true if all tasks are completed
	**/
	bool WaitForAsyncTasks(float TimeLimitSeconds);

	/** Make sure all async tasks are completed before returning */
	void FinishAsyncTasks();

	typedef FOnGPUDataChanged::FDelegate FOnRebuild;
	FDelegateHandle RegisterOnGPUDataChanged(const FOnRebuild& Delegate);
	void UnregisterOnGPUDataChanged(FDelegateUserObject Unregister);
	void UnregisterOnGPUDataChanged(FDelegateHandle Handle);
	void NotifyOnGPUDataChanged();
#endif

private:
#if WITH_EDITOR
	friend class FAnimBankCompilingManager;
	void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority);

	FIoHash CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform);
	FIoHash BeginCacheDerivedData(const ITargetPlatform* TargetPlatform);
	bool PollCacheDerivedData(const FIoHash& KeyHash) const;
	void EndCacheDerivedData(const FIoHash& KeyHash);

	/** Synchronously cache and return derived data for the target platform. */
	FAnimBankData& CacheDerivedData(const ITargetPlatform* TargetPlatform);
#endif

private:
	bool bIsInitialized = false;

	FAnimBankData Data;
	FRenderCommandFence ReleaseResourcesFence;

#if WITH_EDITOR
	FIoHash DataKeyHash;
	TMap<FIoHash, TUniquePtr<FAnimBankData>> DataByPlatformKeyHash;
	TMap<FIoHash, TPimplPtr<FAnimBankBuildAsyncCacheTask>> CacheTasksByKeyHash;

	FOnGPUDataChanged OnGPUDataChanged;

	DECLARE_EVENT_OneParam(UAnimBank, FOnDependenciesChanged, UAnimBank*);
	static FOnDependenciesChanged OnDependenciesChanged;
#endif
};

struct FSoftAnimBankItem;

USTRUCT(BlueprintType)
struct FAnimBankItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<UAnimBank> BankAsset = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	int32 SequenceIndex = 0;

	ENGINE_API FAnimBankItem();
	ENGINE_API FAnimBankItem(const FAnimBankItem& InBankItem);
	ENGINE_API explicit FAnimBankItem(const FSoftAnimBankItem& InBankItem);

	ENGINE_API bool operator!=(const FAnimBankItem& Other) const;
	ENGINE_API bool operator==(const FAnimBankItem& Other) const;
};

inline uint32 GetTypeHash(const FAnimBankItem& Key)
{
	return HashCombine(GetTypeHash(Key.BankAsset.Get()), GetTypeHash(Key.SequenceIndex));
}

inline uint32 GetTypeHash(const TArray<FAnimBankItem>& InBankItems)
{
	uint32 Hash = 0;
	for (const FAnimBankItem& BankItem : InBankItems)
	{
		Hash = HashCombine(Hash, GetTypeHash(BankItem));
	}
	return Hash;
}

USTRUCT(BlueprintType)
struct FSoftAnimBankItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TSoftObjectPtr<UAnimBank> BankAsset = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	int32 SequenceIndex = 0;

	ENGINE_API FSoftAnimBankItem();
	ENGINE_API FSoftAnimBankItem(const FSoftAnimBankItem& InBankItem);
	ENGINE_API explicit FSoftAnimBankItem(const FAnimBankItem& InBankItem);

	ENGINE_API bool operator!=(const FSoftAnimBankItem& Other) const;
	ENGINE_API bool operator==(const FSoftAnimBankItem& Other) const;
};

inline uint32 GetTypeHash(const FSoftAnimBankItem& Key)
{
	return HashCombine(GetTypeHash(Key.BankAsset.Get()), GetTypeHash(Key.SequenceIndex));
}

inline uint32 GetTypeHash(const TArray<FSoftAnimBankItem>& InBankItems)
{
	uint32 Hash = 0;
	for (const FSoftAnimBankItem& BankItem : InBankItems)
	{
		Hash = HashCombine(Hash, GetTypeHash(BankItem));
	}
	return Hash;
}

USTRUCT()
struct FSkinnedMeshComponentDescriptorBase
{
	GENERATED_BODY()
	ENGINE_API FSkinnedMeshComponentDescriptorBase();
	ENGINE_API explicit FSkinnedMeshComponentDescriptorBase(ENoInit);
	ENGINE_API explicit FSkinnedMeshComponentDescriptorBase(const FSkinnedMeshComponentDescriptorBase&);
	ENGINE_API virtual ~FSkinnedMeshComponentDescriptorBase();

	ENGINE_API UInstancedSkinnedMeshComponent* CreateComponent(UObject* Outer, FName Name = NAME_None, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) const;

	ENGINE_API virtual void InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance = true);
	ENGINE_API virtual void InitComponent(UInstancedSkinnedMeshComponent* ISMComponent) const;

	ENGINE_API bool operator!=(const FSkinnedMeshComponentDescriptorBase& Other) const;
	ENGINE_API bool operator==(const FSkinnedMeshComponentDescriptorBase& Other) const;

public:
	UPROPERTY()
	mutable uint32 Hash = 0;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<EComponentMobility::Type> Mobility;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TSubclassOf<UInstancedSkinnedMeshComponent> ComponentClass;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceMinDrawDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceStartCullDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceEndCullDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastDynamicShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastStaticShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastVolumetricTranslucentShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastContactShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bSelfShadowOnly : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastFarShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastInsetShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastCinematicShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastHiddenShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (EditCondition = "bCastShadow"))
	uint8 bCastShadowAsTwoSided : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	bool bVisibleInRayTracing = true;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	bool bVisibleInReflections = true;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	bool bAffectDynamicIndirectLighting = true;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	bool bAffectDistanceFieldLighting = true;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisible : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRenderStatic : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEvaluateWorldPositionOffset : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	FBodyInstance BodyInstance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 WorldPositionOffsetDisableDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 NanitePixelProgrammableDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<enum EDetailMode> DetailMode;

	UPROPERTY()
	FBox PrimitiveBoundsOverride;

	UPROPERTY()
	bool bIsInstanceDataGPUOnly = false;

	UPROPERTY()
	int32 NumInstancesGPUOnly = 0;

	UPROPERTY()
	int32 NumCustomDataFloatsGPUOnly = 0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bIncludeInHLOD : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	EHLODBatchingPolicy HLODBatchingPolicy;
#endif
};

USTRUCT()
struct FSkinnedMeshComponentDescriptor : public FSkinnedMeshComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSkinnedMeshComponentDescriptor();
	ENGINE_API explicit FSkinnedMeshComponentDescriptor(ENoInit);
	ENGINE_API explicit FSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor&);
	ENGINE_API explicit FSkinnedMeshComponentDescriptor(const FSoftSkinnedMeshComponentDescriptor&);
	ENGINE_API virtual ~FSkinnedMeshComponentDescriptor();

	ENGINE_API UInstancedSkinnedMeshComponent* CreateComponent(UObject* Outer, FName Name = NAME_None, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) const;

	ENGINE_API virtual void InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance = true);
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(UInstancedSkinnedMeshComponent* ISMComponent) const;

	ENGINE_API void PostLoadFixup(UObject* Loader);

	ENGINE_API bool operator!=(const FSkinnedMeshComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FSkinnedMeshComponentDescriptor& Other) const;

	friend inline uint32 GetTypeHash(const FSkinnedMeshComponentDescriptor& Key)
	{
		return Key.GetTypeHash();
	}

	uint32 GetTypeHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TObjectPtr<class USkinnedAsset> SkinnedAsset;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "SkinnedAsset"))
	TObjectPtr<class UTransformProviderData> TransformProvider;
};

USTRUCT()
struct FSoftSkinnedMeshComponentDescriptor : public FSkinnedMeshComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSoftSkinnedMeshComponentDescriptor();
	ENGINE_API explicit FSoftSkinnedMeshComponentDescriptor(ENoInit);
	ENGINE_API explicit FSoftSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor&);
	ENGINE_API explicit FSoftSkinnedMeshComponentDescriptor(const FSoftSkinnedMeshComponentDescriptor&);
	ENGINE_API virtual ~FSoftSkinnedMeshComponentDescriptor();

	ENGINE_API UInstancedSkinnedMeshComponent* CreateComponent(UObject* Outer, FName Name = NAME_None, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) const;

	ENGINE_API virtual void InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance = true);
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(UInstancedSkinnedMeshComponent* ISMComponent) const;

	ENGINE_API void PostLoadFixup(UObject* Loader);

	ENGINE_API bool operator!=(const FSoftSkinnedMeshComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FSoftSkinnedMeshComponentDescriptor& Other) const;

	friend inline uint32 GetTypeHash(const FSoftSkinnedMeshComponentDescriptor& Key)
	{
		return Key.GetTypeHash();
	}

	uint32 GetTypeHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TSoftObjectPtr<class USkinnedAsset> SkinnedAsset;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "SkinnedAsset"))
	TSoftObjectPtr<class UTransformProviderData> TransformProvider;
};

class FAnimBankTransformProviderProxy;
class UAnimBankDataInstance;

class FAnimBankTrackPackedData
{
public:
	/////////////////////////////////////////////////////////////////////////////////////////////////
	// AutoPlay

	static FAnimBankTrackPackedData Pack(FAnimSequenceTrackAutoPlayData AutoPlayData)
	{
		FAnimBankTrackPackedData Data;
		Data.ItemIndex            = AutoPlayData.SequenceIndex;
		Data.bAutoPlay            = 1;
		Data.LoopMode             = static_cast<uint32>(AutoPlayData.LoopMode);
		Data.bHasPreviousPosition = true;
		Data.Position             = AutoPlayData.Position;
		Data.PreviousPosition     = AutoPlayData.Position;
		Data.PlayRate             = AutoPlayData.PlayRate;
		return Data;
	}

	bool IsAutoPlay() const
	{
		return bAutoPlay;
	}

	FAnimSequenceTrackAutoPlayData UnpackAutoPlay() const
	{
		check(IsAutoPlay());

		FAnimSequenceTrackAutoPlayData Data;
		Data.SequenceIndex = ItemIndex;
		Data.Position  = Position;
		Data.PlayRate  = PlayRate;
		Data.LoopMode  = static_cast<EAnimSequenceTrackLoopMode>(LoopMode);
		return Data;
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////
	// Manual

	static FAnimBankTrackPackedData Pack(FAnimSequenceTrackManualData ManualData)
	{
		FAnimBankTrackPackedData Data;
		Data.ItemIndex            = ManualData.SequenceIndex;
		Data.bAutoPlay            = 0;
		Data.LoopMode             = 0;
		Data.bHasPreviousPosition = false;
		Data.Position             = ManualData.Position;
		Data.PreviousPosition     = ManualData.Position;
		Data.PlayRate             = 0.0f;
		return Data;
	}

	bool IsManual() const
	{
		return !bAutoPlay;
	}

	FAnimSequenceTrackManualData UnpackManual() const
	{
		check(IsManual());

		FAnimSequenceTrackManualData Data;
		Data.SequenceIndex        = ItemIndex;
		Data.Position             = Position;
		return Data;
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////

	int32 GetItemIndex() const
	{
		return ItemIndex;
	}

	float GetPosition() const
	{
		return Position;
	}

	float GetPreviousPosition() const
	{
		return PreviousPosition;
	}

	bool HasPreviousPosition() const
	{
		return bHasPreviousPosition;
	}

	EAnimSequenceTrackLoopMode GetLoopMode() const
	{
		return static_cast<EAnimSequenceTrackLoopMode>(LoopMode);
	}

	ENGINE_API void Update(float SequenceLength, bool bLooping, float DeltaTime);

	ENGINE_API void Wrap(float SequenceLength, bool bLooping);

	ENGINE_API void Serialize(FArchive& Ar);

	void SetData(const FAnimBankTrackPackedData& RHS)
	{
		*this = RHS;
	}

private:
	uint32 ItemIndex            : 20;
	uint32 LoopMode             : 2;
	uint32 bAutoPlay            : 1;
	uint32 bHasPreviousPosition : 1;
	uint32 Unused               : 8 = 0;
	float Position;
	float PreviousPosition;
	float PlayRate;

	friend class FAnimBankTransformProvider;
};

inline FArchive& operator <<(FArchive& Ar, FAnimBankTrackPackedData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

using FAnimBankTrackPool = TAnimTrackPool<FAnimBankTrackPackedData>;

class FAnimBankRenderData
{
public:
	ENGINE_API ~FAnimBankRenderData();

private:
	void Patch(FAnimBankTrackPool::FPatch&& Patch);
	void InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy);
	void RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy);

	FAnimBankTrackPool Tracks;
	Experimental::TRobinHoodHashSet<FInstancedSkinningSceneExtensionProxy*> Proxies;

	friend FAnimBankTransformProviderProxy;
	friend UAnimBankDataInstance;
};

class FAnimBankTransformProviderProxy : public FTransformProviderRenderProxy
{
	friend class UAnimBankDataInstance;
	friend class UAnimBankData;

	ENGINE_API FAnimBankTransformProviderProxy(TConstArrayView<FAnimBankItem> AnimBankItems, FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy);
	ENGINE_API FAnimBankTransformProviderProxy(TConstArrayView<FAnimBankItem> AnimBankItems, FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy, FAnimBankRenderData& RenderData);

public:
	virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;

	FAnimBankTrackPool& GetTracks()
	{
		return *Tracks;
	}

	TConstArrayView<FAnimBankItem> GetAnimBankItems() const
	{
		return AnimBankItems;
	}

	const FInstancedSkinningSceneExtensionProxy* GetSceneExtensionProxy() const
	{
		return SceneExtensionProxy;
	}

private:
	TConstArrayView<FAnimBankItem> AnimBankItems;
	TUniquePtr<FAnimBankTrackPool> TrackAllocator;
	FAnimBankRenderData* RenderData = nullptr;
	FAnimBankTrackPool* Tracks;
	FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy = nullptr;
	const USkinnedAsset* SkinnedAsset = nullptr;
	IAnimBankTransformProvider* AnimBankTransformProvider = nullptr;
};

class IAnimBankTransformProvider
{
public:
	virtual ~IAnimBankTransformProvider() = default;

	virtual void RegisterAnimBank(FAnimBankTransformProviderProxy* Proxy) {}
	virtual void UnregisterAnimBank(FAnimBankTransformProviderProxy* Proxy) {}
};

UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UAnimBankData : public UTransformProviderData
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const override;
	virtual bool IsGpuOnly() const override { return true; }
	virtual const FGuid& GetTransformProviderID() const override;
	virtual uint32 GetUniqueAnimationCount() const override;
	virtual bool UsesSkeletonBatching() const override;
	virtual bool HasAnimationBounds() const override;
	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const override;
	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const override;
	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const override;
	virtual bool IsCompiling() const override;

#if WITH_EDITOR
	void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (ShowOnlyInnerProperties))
	TArray<struct FAnimBankItem> AnimBankItems;

	// Returns the play length for an anim bank item.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimBank")
	ENGINE_API float GetSequencePlayLength(int32 ItemIndex) const;

protected:
	bool IsAnimBankValid(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const;
};

namespace UE::AnimBank
{

// Convert a list of transforms from bone/local space to mesh/global space by walking through the
// hierarchy of a reference skeleton.
ENGINE_API void ConvertLocalToGlobalSpaceTransforms(
	const FReferenceSkeleton& InRefSkeleton,
	const TArray<FTransform>& InLocalSpaceTransforms,
	TArray<FTransform>& OutGlobalSpaceTransforms
);

ENGINE_API void BuildSkinnedAssetMapping(const USkinnedAsset& Asset, FSkinnedAssetMapping& Mapping);

}