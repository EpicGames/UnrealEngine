// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Engine/Blueprint.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "AnimBlueprint.generated.h"

class SWidget;
class UAnimationAsset;
class USkeletalMesh;
class USkeleton;
class UPoseWatch;
class UPoseWatchFolder;
struct FAnimBlueprintDebugData;
class UAnimGraphNodeBinding;
class UClass;

USTRUCT()
struct FAnimGroupInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FLinearColor Color;

	FAnimGroupInfo()
		: Color(FLinearColor::White)
	{
	}
};

USTRUCT()
struct FAnimParentNodeAssetOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UAnimationAsset> NewAsset;
	UPROPERTY()
	FGuid ParentNodeGuid;

	FAnimParentNodeAssetOverride(FGuid InGuid, UAnimationAsset* InNewAsset)
		: NewAsset(InNewAsset)
		, ParentNodeGuid(InGuid)
	{}

	FAnimParentNodeAssetOverride()
		: NewAsset(NULL)
	{}

	bool operator ==(const FAnimParentNodeAssetOverride& Other)
	{
		return ParentNodeGuid == Other.ParentNodeGuid;
	}
};

/** The method by which a preview animation blueprint is applied */
UENUM()
enum class EPreviewAnimationBlueprintApplicationMethod : uint8
{
	/** Apply the preview animation blueprint using LinkAnimClassLayers */
	LinkedLayers,

	/** Apply the preview animation blueprint using SetLinkedAnimGraphByTag */
	LinkedAnimGraph,
};

/**
 * An Anim Blueprint is essentially a specialized Blueprint whose graphs control the animation of a Skeletal Mesh.
 * It can perform blending of animations, directly control the bones of the skeleton, and output a final pose
 * for a Skeletal Mesh each frame.
 */
UCLASS(BlueprintType, MinimalAPI)
class UAnimBlueprint : public UBlueprint, public IInterface_PreviewMeshProvider
{
	GENERATED_UCLASS_BODY()

	/**
	 * This is the target skeleton asset for anim instances created from this blueprint; all animations
	 * referenced by the BP should be compatible with this skeleton.  For advanced use only, it is easy
	 * to cause errors if this is modified without updating or replacing all referenced animations.
	 */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, AdvancedDisplay, Category=ClassOptions)
	TObjectPtr<USkeleton> TargetSkeleton;

	// List of animation sync groups
	UPROPERTY()
	TArray<FAnimGroupInfo> Groups;

	// This is an anim blueprint that acts as a set of template functionality without being tied to a specific skeleton.
	// Implies a null TargetSkeleton.
	UPROPERTY(AssetRegistrySearchable)
	bool bIsTemplate;
	
	/**
	 * Allows this anim Blueprint to update its native update, blend tree, montages and asset players on
	 * a worker thread. The compiler will attempt to pick up any issues that may occur with threaded update.
	 * For updates to run in multiple threads both this flag and the project setting "Allow Multi Threaded 
	 * Animation Update" should be set.
	 */
	UPROPERTY(EditAnywhere, Category = Optimization)
	bool bUseMultiThreadedAnimationUpdate;

	/**
	 * Selecting this option will cause the compiler to emit warnings whenever a call into Blueprint
	 * is made from the animation graph. This can help track down optimizations that need to be made.
	 */
	UPROPERTY(EditAnywhere, Category = Optimization)
	bool bWarnAboutBlueprintUsage;

	/** If true, linked animation layers will be instantiated only once per AnimClass instead of once per AnimInstance, AnimClass and AnimGroup.
	Extra instances will be created if two or more active anim graph override the same layer Function */
	UPROPERTY(EditDefaultsOnly, Category = Optimization)
	uint8 bEnableLinkedAnimLayerInstanceSharing : 1;

	// @todo document
	ENGINE_API class UAnimBlueprintGeneratedClass* GetAnimBlueprintGeneratedClass() const;

	// @todo document
	ENGINE_API class UAnimBlueprintGeneratedClass* GetAnimBlueprintSkeletonClass() const;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR

	ENGINE_API virtual UClass* GetBlueprintClass() const override;

	// Inspects the hierarchy and looks for an override for the requested node GUID
	// @param NodeGuid - Guid of the node to search for
	// @param bIgnoreSelf - Ignore this blueprint and only search parents, handy for finding parent overrides
	ENGINE_API FAnimParentNodeAssetOverride* GetAssetOverrideForNode(FGuid NodeGuid, bool bIgnoreSelf = false) const ;

	// Inspects the hierarchy and builds a list of all asset overrides for this blueprint
	// @param OutOverrides - Array to fill with overrides
	// @return bool - Whether any overrides were found
	ENGINE_API bool GetAssetOverrides(TArray<FAnimParentNodeAssetOverride*>& OutOverrides);

	// UBlueprint interface
	virtual bool SupportedByDefaultBlueprintFactory() const override
	{
		return false;
	}

	virtual bool IsValidForBytecodeOnlyRecompile() const override { return false; }
	ENGINE_API virtual bool CanAlwaysRecompileWhilePlayingInEditor() const override;
	// End of UBlueprint interface

	// Finds the index of the specified group, or creates a new entry for it (unless the name is NAME_None, which will return INDEX_NONE)
	ENGINE_API int32 FindOrAddGroup(FName GroupName);

	/** Returns the most base anim blueprint for a given blueprint (if it is inherited from another anim blueprint, returning null if only native / non-anim BP classes are it's parent) */
	static ENGINE_API UAnimBlueprint* FindRootAnimBlueprint(const UAnimBlueprint* DerivedBlueprint);

	/** Returns the parent anim blueprint for a given blueprint (if it is inherited from another anim blueprint, returning null if only native / non-anim BP classes are it's parent) */
	static ENGINE_API UAnimBlueprint* GetParentAnimBlueprint(const UAnimBlueprint* DerivedBlueprint);
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOverrideChangedMulticaster, FGuid, UAnimationAsset*);

	typedef FOnOverrideChangedMulticaster::FDelegate FOnOverrideChanged;

	void RegisterOnOverrideChanged(const FOnOverrideChanged& Delegate)
	{
		OnOverrideChanged.Add(Delegate);
	}

	void UnregisterOnOverrideChanged(SWidget* Widget)
	{
		OnOverrideChanged.RemoveAll(Widget);
	}

	void NotifyOverrideChange(FAnimParentNodeAssetOverride& Override)
	{
		OnOverrideChanged.Broadcast(Override.ParentNodeGuid, Override.NewAsset);
	}

	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const override;
	ENGINE_API virtual void SetObjectBeingDebugged(UObject* NewObject) override;
	ENGINE_API virtual bool SupportsAnimLayers() const override;
	ENGINE_API virtual bool SupportsEventGraphs() const override;
	ENGINE_API virtual bool SupportsDelegates() const override;
	ENGINE_API virtual bool SupportsMacros() const override;
	ENGINE_API virtual bool SupportsInputEvents() const override;
	ENGINE_API virtual bool AllowFunctionOverride(const UFunction* const InFunction) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	ENGINE_API virtual void GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

protected:
	// Broadcast when an override is changed, allowing derived blueprints to be updated
	FOnOverrideChangedMulticaster OnOverrideChanged;
#endif	// #if WITH_EDITOR

public:
	/** IInterface_PreviewMeshProvider interface */
	ENGINE_API virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override;
	ENGINE_API virtual USkeletalMesh* GetPreviewMesh(bool bFindIfNotSet = false) override;
	ENGINE_API virtual USkeletalMesh* GetPreviewMesh() const override;

	/** Preview anim blueprint support */
	ENGINE_API void SetPreviewAnimationBlueprint(UAnimBlueprint* InPreviewAnimationBlueprint);
	ENGINE_API UAnimBlueprint* GetPreviewAnimationBlueprint() const;

	ENGINE_API void SetPreviewAnimationBlueprintApplicationMethod(EPreviewAnimationBlueprintApplicationMethod InMethod);
	ENGINE_API EPreviewAnimationBlueprintApplicationMethod GetPreviewAnimationBlueprintApplicationMethod() const;

	ENGINE_API void SetPreviewAnimationBlueprintTag(FName InTag);
	ENGINE_API FName GetPreviewAnimationBlueprintTag() const;

public:
	/** Check if the anim instance is the active debug object for this anim BP */
	ENGINE_API bool IsObjectBeingDebugged(const UObject* AnimInstance) const;

	/** Get the debug data for this anim BP */
	ENGINE_API FAnimBlueprintDebugData* GetDebugData() const;

#if WITH_EDITORONLY_DATA
public:
	// Queue a refresh of the set of anim blueprint extensions that this anim blueprint hosts.
	// Usually called from anim graph nodes to ensure that extensions that are no longer required are cleaned up.
	void RequestRefreshExtensions() { bRefreshExtensions = true; }

	// Check if the anim BP is compatible with this one (for linked instancing). Checks target skeleton, template flags
	// blueprint type.
	// Note compatibility is directional - e.g. template anim BPs can be instanced within any 'regular' anim BP, but not
	// vice versa
	// @param	InAnimBlueprint		The anim blueprint to check for compatibility
	ENGINE_API bool IsCompatible(const UAnimBlueprint* InAnimBlueprint) const;
	
	// Check if the asset path of a skeleton, template and interface flags are compatible with this anim blueprint
	// (for linked instancing)
	// @param	InSkeletonAsset		The asset path of the skeleton asset used by the anim blueprint
	// @param	bInIsTemplate		Whether the anim blueprint to check is a template
	// @param	bInIsInterface		Whether the anim blueprint to check is an interface
	ENGINE_API bool IsCompatibleByAssetString(const FString& InSkeletonAsset, bool bInIsTemplate, bool bInIsInterface) const;
	
	// Get the default binding type that any new nodes will use when created
	UClass* GetDefaultBindingClass() const { return DefaultBindingClass; }

	virtual void NotifyGraphRenamed(class UEdGraph* Graph, FName OldName, FName NewName) override;

	// Event that is broadcast to inform observers that the node title has changed
	// The default SAnimationGraphNode uses this to invalidate cached node title text
	DECLARE_EVENT_ThreeParams(UAnimBlueprint, FOnGraphRenamedEvent, UEdGraph*, FName, FName);
	FOnGraphRenamedEvent& OnGraphRenamedEvent() { return GraphRenameEvent; }

private:
	FOnGraphRenamedEvent GraphRenameEvent;

public:
	// Array of overrides to asset containing nodes in the parent that have been overridden
	UPROPERTY()
	TArray<FAnimParentNodeAssetOverride> ParentAssetOverrides;

	// Array of active pose watches (pose watches allows us to see the bone pose at a 
	// particular point of the anim graph and control debug draw for unselected anim nodes).
	UPROPERTY()
	TArray<TObjectPtr<UPoseWatchFolder>> PoseWatchFolders;
	
	UPROPERTY()
	TArray<TObjectPtr<UPoseWatch>> PoseWatches;

private:
	friend class FAnimBlueprintCompilerContext;
	
	/** The default skeletal mesh to use when previewing this asset - this only applies when you open Persona using this asset*/
	UPROPERTY(duplicatetransient, AssetRegistrySearchable)
	TSoftObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	/** 
	 * An animation Blueprint to overlay with this Blueprint. When working on layers, this allows this Blueprint to be previewed in the context of another 'outer' anim blueprint. 
	 * Setting this is the equivalent of running the preview animation blueprint on the preview mesh, then calling SetLayerOverlay with this anim blueprint.
	 */
	UPROPERTY(duplicatetransient, AssetRegistrySearchable)
	TSoftObjectPtr<class UAnimBlueprint> PreviewAnimationBlueprint;

	/** The default binding type that any new nodes will use when created */
	UPROPERTY(EditAnywhere, Category=Bindings, meta=(AllowedClasses="/Script/AnimGraph.AnimGraphNodeBinding", ShowDisplayNames=true, NoClear))
	TObjectPtr<UClass> DefaultBindingClass;

	/** The method by which a preview animation blueprint is applied, either as an overlay layer, or as a linked instance */
	UPROPERTY()
	EPreviewAnimationBlueprintApplicationMethod PreviewAnimationBlueprintApplicationMethod;

	/** The tag to use when applying a preview animation blueprint via LinkAnimGraphByTag */
	UPROPERTY()
	FName PreviewAnimationBlueprintTag;

	/** If set, then extensions need to be refreshed according to spawned nodes */
	bool bRefreshExtensions;
#endif // WITH_EDITORONLY_DATA
};
