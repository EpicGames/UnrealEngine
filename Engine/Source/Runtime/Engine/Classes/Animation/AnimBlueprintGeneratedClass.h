// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Animation/AnimTypes.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/BlendSpace.h"
#include "Animation/ExposedValueHandler.h"
#include "Engine/PoseWatchRenderData.h"

#include "AnimBlueprintGeneratedClass.generated.h"

class UAnimGraphNode_Base;
class UAnimGraphNode_StateMachineBase;
class UAnimInstance;
class UAnimStateNode;
class UAnimStateNodeBase;
class UAnimStateAliasNode;
class UAnimStateTransitionNode;
class UEdGraph;
class USkeleton;
class UPoseWatch;
class UPoseWatchPoseElement;
struct FAnimSubsystem;
struct FAnimSubsystemInstance;
struct FPropertyAccessLibrary;

// Represents the debugging information for a single state within a state machine
USTRUCT()
struct FStateMachineStateDebugData
{
	GENERATED_BODY()

public:
	FStateMachineStateDebugData()
		: StateMachineIndex(INDEX_NONE)
		, StateIndex(INDEX_NONE)
		, Weight(0.0f)
		, ElapsedTime(0.0f)
	{
	}

	FStateMachineStateDebugData(int32 InStateMachineIndex, int32 InStateIndex, float InWeight, float InElapsedTime)
		: StateMachineIndex(InStateMachineIndex)
		, StateIndex(InStateIndex)
		, Weight(InWeight)
		, ElapsedTime(InElapsedTime)
	{}

	// The index of the state machine
	int32 StateMachineIndex;

	// The index of the state
	int32 StateIndex;

	// The last recorded weight for this state
	float Weight;

	// The time that this state has been active (only valid if this is the current state)
	float ElapsedTime;
};

// This structure represents debugging information for a single state machine
USTRUCT()
struct FStateMachineDebugData
{
	GENERATED_BODY()

public:
	FStateMachineDebugData()
		: MachineIndex(INDEX_NONE)
	{}

	struct FStateAliasTransitionStateIndexPair
	{
		int32 TransitionIndex;
		int32 AssociatedStateIndex;
	};

	// Map from state nodes to their state entry in a state machine
	TMap<TWeakObjectPtr<UEdGraphNode>, int32> NodeToStateIndex;
	TMap<int32, TWeakObjectPtr<UAnimStateNodeBase>> StateIndexToNode;

	// Transition nodes may be associated w/ multiple transition indicies when the source state is an alias
	TMultiMap<TWeakObjectPtr<UEdGraphNode>, int32> NodeToTransitionIndex;

	// Mapping between an alias and any transition indices it might be associated to (Both as source and target).
	TMultiMap<TWeakObjectPtr<UAnimStateAliasNode>, FStateAliasTransitionStateIndexPair> StateAliasNodeToTransitionStatePairs;

	// The animation node that leads into this state machine (A3 only)
	TWeakObjectPtr<UAnimGraphNode_StateMachineBase> MachineInstanceNode;

	// Index of this machine in the StateMachines array
	int32 MachineIndex;

public:
	ENGINE_API UEdGraphNode* FindNodeFromStateIndex(int32 StateIndex) const;
	ENGINE_API UEdGraphNode* FindNodeFromTransitionIndex(int32 TransitionIndex) const;
};

// This structure represents debugging information for a frame snapshot
USTRUCT()
struct FAnimationFrameSnapshot
{
	GENERATED_USTRUCT_BODY()

	FAnimationFrameSnapshot()
#if WITH_EDITORONLY_DATA
		: TimeStamp(0.0)
#endif
	{
	}
#if WITH_EDITORONLY_DATA
public:
	// The snapshot of data saved from the animation
	TArray<uint8> SerializedData;

	// The time stamp for when this snapshot was taken (relative to the life timer of the object being recorded)
	double TimeStamp;

public:
	void InitializeFromInstance(UAnimInstance* Instance);
	ENGINE_API void CopyToInstance(UAnimInstance* Instance);
#endif
};

struct FAnimBlueprintDebugData_NodeVisit
{
	int32 SourceID;
	int32 TargetID;
	float Weight;

	FAnimBlueprintDebugData_NodeVisit(int32 InSourceID, int32 InTargetID, float InWeight)
		: SourceID(InSourceID)
		, TargetID(InTargetID)
		, Weight(InWeight)
	{
	}
};


struct FAnimBlueprintDebugData_AttributeRecord
{
	FName Attribute;
	int32 OtherNode;

	FAnimBlueprintDebugData_AttributeRecord(int32 InOtherNode, FName InAttribute)
		: Attribute(InAttribute)
		, OtherNode(InOtherNode)
	{}
};

// This structure represents animation-related debugging information for an entire AnimBlueprint
// (general debug information for the event graph, etc... is still contained in a FBlueprintDebugData structure)
USTRUCT()
struct FAnimBlueprintDebugData
{
	GENERATED_USTRUCT_BODY()

	FAnimBlueprintDebugData()
#if WITH_EDITORONLY_DATA
		: SnapshotBuffer(NULL)
		, SnapshotIndex(INDEX_NONE)
#endif
	{
	}

#if WITH_EDITORONLY_DATA
public:
	// Map from state machine graphs to their corresponding debug data
	TMap<TWeakObjectPtr<const UEdGraph>, FStateMachineDebugData> StateMachineDebugData;

	// Map from state graphs to their node
	TMap<TWeakObjectPtr<const UEdGraph>, TWeakObjectPtr<UAnimStateNode> > StateGraphToNodeMap;

	// Map from transition graphs to their node
	TMap<TWeakObjectPtr<const UEdGraph>, TWeakObjectPtr<UAnimStateTransitionNode> > TransitionGraphToNodeMap;

	// Map from custom transition blend graphs to their node
	TMap<TWeakObjectPtr<const UEdGraph>, TWeakObjectPtr<UAnimStateTransitionNode> > TransitionBlendGraphToNodeMap;

	// Map from animation node to their property index
	TMap<TWeakObjectPtr<const UAnimGraphNode_Base>, int32> NodePropertyToIndexMap;

	// Map from node property index to source editor node
	TMap<int32, TWeakObjectPtr<const UEdGraphNode> > NodePropertyIndexToNodeMap;

	// Map from animation node GUID to property index
	TMap<FGuid, int32> NodeGuidToIndexMap;

	// Map from animation node to attributes
	TMap<TWeakObjectPtr<const UAnimGraphNode_Base>, TArray<FName>> NodeAttributes;

	// The debug data for each state machine state
	TArray<FStateMachineStateDebugData> StateData;	
	
	// History of snapshots of animation data
	TSimpleRingBuffer<FAnimationFrameSnapshot>* SnapshotBuffer;

	// Mapping from graph pins to their folded properties.
	// Graph pins are unique per node instance and thus suitable as identifier for the properties.
	TMap<FEdGraphPinReference, FProperty*> GraphPinToFoldedPropertyMap;

	// Node visit structure
	using FNodeVisit = FAnimBlueprintDebugData_NodeVisit;

	// History of activated nodes
	TArray<FNodeVisit> UpdatedNodesThisFrame;

	// Record of attribute transfer between nodes
	using FAttributeRecord = FAnimBlueprintDebugData_AttributeRecord;

	// History of node attributes that are output from and input to nodes
	TMap<int32, TArray<FAttributeRecord>> NodeInputAttributesThisFrame;
	TMap<int32, TArray<FAttributeRecord>> NodeOutputAttributesThisFrame;

	// History of node syncs - maps from player node index to graph-determined group name
	TMap<int32, FName> NodeSyncsThisFrame;

	// Values output by nodes
	struct FNodeValue
	{
		FString Text;
		int32 NodeID;

		FNodeValue(const FString& InText, int32 InNodeID)
			: Text(InText)
			, NodeID(InNodeID)
		{}
	};

	// Values output by nodes
	TArray<FNodeValue> NodeValuesThisFrame;

	// Record of a sequence player's state
	struct FSequencePlayerRecord
	{
		FSequencePlayerRecord(int32 InNodeID, float InPosition, float InLength, int32 InFrameCount)
			: NodeID(InNodeID)
			, Position(InPosition)
			, Length(InLength) 
			, FrameCount(InFrameCount)
		{}

		int32 NodeID;
		float Position;
		float Length;
		int32 FrameCount;
	};

	// All sequence player records this frame
	TArray<FSequencePlayerRecord> SequencePlayerRecordsThisFrame;

	// Record of a blend space player's state
	struct FBlendSpacePlayerRecord
	{
		FBlendSpacePlayerRecord(int32 InNodeID, const UBlendSpace* InBlendSpace, const FVector& InPosition, const FVector& InFilteredPosition)
			: NodeID(InNodeID)
			, BlendSpace(InBlendSpace)
			, Position(InPosition)
			, FilteredPosition(InFilteredPosition)
		{}

		int32 NodeID;
		TWeakObjectPtr<const UBlendSpace> BlendSpace;
		FVector Position;
		FVector FilteredPosition;
	};

	// All blend space player records this frame
	TArray<FBlendSpacePlayerRecord> BlendSpacePlayerRecordsThisFrame;

	// Active pose watches to track
	TArray<FAnimNodePoseWatch> AnimNodePoseWatch;

	// Index of snapshot
	int32 SnapshotIndex;
public:

	~FAnimBlueprintDebugData()
	{
		if (SnapshotBuffer != NULL)
		{
			delete SnapshotBuffer;
		}
		SnapshotBuffer = NULL;
	}



	bool IsReplayingSnapshot() const { return SnapshotIndex != INDEX_NONE; }
	ENGINE_API void TakeSnapshot(UAnimInstance* Instance);
	ENGINE_API float GetSnapshotLengthInSeconds();
	ENGINE_API int32 GetSnapshotLengthInFrames();
	ENGINE_API void SetSnapshotIndexByTime(UAnimInstance* Instance, double TargetTime);
	ENGINE_API void SetSnapshotIndex(UAnimInstance* Instance, int32 NewIndex);
	ENGINE_API void ResetSnapshotBuffer();

	ENGINE_API void ResetNodeVisitSites();
	ENGINE_API void RecordNodeVisit(int32 TargetNodeIndex, int32 SourceNodeIndex, float BlendWeight);
	ENGINE_API void RecordNodeVisitArray(const TArray<FNodeVisit>& Nodes);
	ENGINE_API void RecordNodeAttribute(int32 TargetNodeIndex, int32 SourceNodeIndex, FName InAttribute);
	ENGINE_API void RecordNodeAttributeMaps(const TMap<int32, TArray<FAttributeRecord>>& InInputAttributes, const TMap<int32, TArray<FAttributeRecord>>& InOutputAttributes);
	ENGINE_API void RecordNodeSync(int32 InSourceNodeIndex, FName InSyncGroup);
	ENGINE_API void RecordNodeSyncsArray(const TMap<int32, FName>& InNodeSyncs);
	ENGINE_API void RecordStateData(int32 StateMachineIndex, int32 StateIndex, float Weight, float ElapsedTime);
	ENGINE_API void RecordNodeValue(int32 InNodeID, const FString& InText);
	ENGINE_API void RecordSequencePlayer(int32 InNodeID, float InPosition, float InLength, int32 InFrameCount);
	ENGINE_API void RecordBlendSpacePlayer(int32 InNodeID, const UBlendSpace* InBlendSpace, const FVector& InPosition, const FVector& InFilteredPosition);

	ENGINE_API void AddPoseWatch(int32 NodeID, UPoseWatchPoseElement* const InPoseWatchPoseElement);
	ENGINE_API void RemovePoseWatch(int32 NodeID);
	ENGINE_API void ForEachActiveVisiblePoseWatchPoseElement(const TFunctionRef<void(FAnimNodePoseWatch&)>& InFunction);
	ENGINE_API void DisableAllPoseWatches();

	ENGINE_API TArrayView<const FName> GetNodeAttributes(TWeakObjectPtr<UAnimGraphNode_Base> InAnimGraphNode) const;
#endif
};

// 'Marker' structure for mutable data. This is used as a base struct for mutable data to be inserted into by the anim
// BP compiler.
USTRUCT()
struct FAnimBlueprintMutableData
{
	GENERATED_BODY()
};

// 'Marker' structure for constant data. This is used as a base struct for constant data to be inserted into by the anim
// BP compiler if there is no existing archetype sparse class data.
USTRUCT()
struct FAnimBlueprintConstantData
{
	GENERATED_BODY()
};

#if WITH_EDITORONLY_DATA
namespace EPropertySearchMode
{
	enum Type
	{
		OnlyThis,
		Hierarchy
	};
}
#endif

// Struct type generated by the anim BP compiler. Used for sparse class data and mutable data area.
// Only really needed to hide the struct from the content browser (via IsAsset override)
UCLASS(MinimalAPI)
class UAnimBlueprintGeneratedStruct : public UScriptStruct
{
	GENERATED_BODY()

	// UObject interface
	virtual bool IsAsset() const override { return false; }
};

UCLASS(MinimalAPI)
class UAnimBlueprintGeneratedClass : public UBlueprintGeneratedClass, public IAnimClassInterface
{
	GENERATED_UCLASS_BODY()

	friend class FAnimBlueprintCompilerContext;
	friend class FAnimBlueprintGeneratedClassCompiledData;
	friend class FKismetDebugUtilities;
	friend class UAnimBlueprintExtension_Base;

	// List of state machines present in this blueprint class
	UPROPERTY()
	TArray<FBakedAnimationStateMachine> BakedStateMachines;

	/** Target skeleton for this blueprint class */
	UPROPERTY(AssetRegistrySearchable)
	TObjectPtr<USkeleton> TargetSkeleton;

	/** A list of anim notifies that state machines (or anything else) may reference */
	UPROPERTY()
	TArray<FAnimNotifyEvent> AnimNotifies;

	// Indices for each of the saved pose nodes that require updating, in the order they need to get updates, per layer
	UPROPERTY()
	TMap<FName, FCachedPoseIndices> OrderedSavedPoseIndicesMap;

	// The various anim functions that this class holds (created during GenerateAnimationBlueprintFunctions)
	TArray<FAnimBlueprintFunction> AnimBlueprintFunctions;

	// The arrays of anim nodes; this is transient generated data (created during Link)
	TArray<FStructProperty*> AnimNodeProperties;
	TArray<FStructProperty*> LinkedAnimGraphNodeProperties;
	TArray<FStructProperty*> LinkedAnimLayerNodeProperties;
	TArray<FStructProperty*> PreUpdateNodeProperties;
	TArray<FStructProperty*> DynamicResetNodeProperties;
	TArray<FStructProperty*> StateMachineNodeProperties;
	TArray<FStructProperty*> InitializationNodeProperties;

	// Array of sync group names in the order that they are requested during compile
	UPROPERTY()
	TArray<FName> SyncGroupNames;

#if WITH_EDITORONLY_DATA
	// Deprecated - moved to FAnimSubsystem_Base
	UPROPERTY()
	TArray<FExposedValueHandler> EvaluateGraphExposedInputs_DEPRECATED;
#endif

	// Indices for any Asset Player found within a specific (named) Anim Layer Graph, or implemented Anim Interface Graph
	UPROPERTY()
	TMap<FName, FGraphAssetPlayerInformation> GraphAssetPlayerInformation;

	// Per layer graph blending options
	UPROPERTY()
	TMap<FName, FAnimGraphBlendOptions> GraphBlendOptions;

private:
	// Constant/folded anim node data
	UPROPERTY()
	TArray<FAnimNodeData> AnimNodeData;

	// Map from anim node struct to info about that struct (used to accelerate property name lookups)
	UPROPERTY()
	TMap<TObjectPtr<const UScriptStruct>, FAnimNodeStructData> NodeTypeMap;

	// Cached properties used to access 'folded' anim node properties
	TArray<FProperty*> MutableProperties;
	TArray<FProperty*> ConstantProperties;

	// Cached properties used to access subsystem properties
	TArray<FStructProperty*> ConstantSubsystemProperties;
	TArray<FStructProperty*> MutableSubsystemProperties;

	// Property for the object's mutable data area
	FStructProperty* MutableNodeDataProperty = nullptr;

	// Pointers to each subsystem, for easier debugging
	TArray<const FAnimSubsystem*> Subsystems;

#if WITH_EDITORONLY_DATA
	// Flag indicating the persistent result of calling VerifyNodeDataLayout() on load/compile
	bool bDataLayoutValid = true;
#endif
	
public:
	// IAnimClassInterface interface
	virtual const TArray<FBakedAnimationStateMachine>& GetBakedStateMachines() const override { return GetRootClass()->GetBakedStateMachines_Direct(); }
	virtual USkeleton* GetTargetSkeleton() const override { return TargetSkeleton; }
	virtual const TArray<FAnimNotifyEvent>& GetAnimNotifies() const override { return GetRootClass()->GetAnimNotifies_Direct(); }
	virtual const TArray<FStructProperty*>& GetAnimNodeProperties() const override { return AnimNodeProperties; }
	virtual const TArray<FStructProperty*>& GetLinkedAnimGraphNodeProperties() const override { return LinkedAnimGraphNodeProperties; }
	virtual const TArray<FStructProperty*>& GetLinkedAnimLayerNodeProperties() const override { return LinkedAnimLayerNodeProperties; }
	virtual const TArray<FStructProperty*>& GetPreUpdateNodeProperties() const override { return PreUpdateNodeProperties; }
	virtual const TArray<FStructProperty*>& GetDynamicResetNodeProperties() const override { return DynamicResetNodeProperties; }
	virtual const TArray<FStructProperty*>& GetStateMachineNodeProperties() const override { return StateMachineNodeProperties; }
	virtual const TArray<FStructProperty*>& GetInitializationNodeProperties() const override { return InitializationNodeProperties; }
	virtual const TArray<FName>& GetSyncGroupNames() const override { return GetRootClass()->GetSyncGroupNames_Direct(); }
	virtual const TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseNodeIndicesMap() const override { return GetRootClass()->GetOrderedSavedPoseNodeIndicesMap_Direct(); }
	virtual int32 GetSyncGroupIndex(FName SyncGroupName) const override { return GetSyncGroupNames().IndexOfByKey(SyncGroupName); }

	virtual const TArray<FAnimBlueprintFunction>& GetAnimBlueprintFunctions() const override { return AnimBlueprintFunctions; }
	virtual const TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation() const override { return GetRootClass()->GetGraphAssetPlayerInformation_Direct(); }
	virtual const TMap<FName, FAnimGraphBlendOptions>& GetGraphBlendOptions() const override { return GetRootClass()->GetGraphBlendOptions_Direct(); }

private:
	virtual const TArray<FBakedAnimationStateMachine>& GetBakedStateMachines_Direct() const override { return BakedStateMachines; }
	virtual const TArray<FAnimNotifyEvent>& GetAnimNotifies_Direct() const override { return AnimNotifies; }
	virtual const TArray<FName>& GetSyncGroupNames_Direct() const override { return SyncGroupNames; }
	virtual const TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseNodeIndicesMap_Direct() const override { return OrderedSavedPoseIndicesMap; }
	virtual const TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation_Direct() const override { return GraphAssetPlayerInformation; }
	virtual const TMap<FName, FAnimGraphBlendOptions>& GetGraphBlendOptions_Direct() const override { return GraphBlendOptions; }
	
private:
	ENGINE_API virtual const void* GetConstantNodeValueRaw(int32 InIndex) const override;
	ENGINE_API virtual const void* GetMutableNodeValueRaw(int32 InIndex, const UObject* InObject) const override;
	ENGINE_API virtual const FAnimBlueprintMutableData* GetMutableNodeData(const UObject* InObject) const override;
	ENGINE_API virtual FAnimBlueprintMutableData* GetMutableNodeData(UObject* InObject) const override;
	ENGINE_API virtual const void* GetConstantNodeData() const override;
	virtual TArrayView<const FAnimNodeData> GetNodeData() const override { return AnimNodeData; }

	ENGINE_API virtual int32 GetAnimNodePropertyIndex(const UScriptStruct* InNodeType, FName InPropertyName) const override;
	ENGINE_API virtual int32 GetAnimNodePropertyCount(const UScriptStruct* InNodeType) const override;
	
	ENGINE_API virtual void ForEachSubsystem(TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemContext&)> InFunction) const override;
	ENGINE_API virtual void ForEachSubsystem(UObject* InObject, TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemInstanceContext&)> InFunction) const override;
	ENGINE_API virtual const FAnimSubsystem* FindSubsystem(UScriptStruct* InSubsystemType) const override;

#if WITH_EDITORONLY_DATA
	virtual bool IsDataLayoutValid() const override { return bDataLayoutValid; };
#endif
	
	// Called internally post-load defaults and by the compiler after compilation is completed 
	ENGINE_API void OnPostLoadDefaults(UObject* Object);

	// Called by the compiler to make sure that data tables are initialized. This is needed to patch the sparse class
	// data for child anim BP overrides 
	ENGINE_API void InitializeAnimNodeData(UObject* DefaultObject, bool bForce) const;

#if WITH_EDITORONLY_DATA
	// Verify that the serialized NodeTypeMap can be used with the current set of native node data layouts
	// Sets internal bDataLayoutValid flag
	ENGINE_API bool VerifyNodeDataLayout();
#endif

public:
#if WITH_EDITORONLY_DATA
	FAnimBlueprintDebugData AnimBlueprintDebugData;

	FAnimBlueprintDebugData& GetAnimBlueprintDebugData()
	{
		return AnimBlueprintDebugData;
	}

	template<typename StructType>
	const int32* GetNodePropertyIndexFromHierarchy(const UAnimGraphNode_Base* Node)
	{
		TArray<const UBlueprintGeneratedClass*> BlueprintHierarchy;
		GetGeneratedClassesHierarchy(this, BlueprintHierarchy);

		for (const UBlueprintGeneratedClass* Blueprint : BlueprintHierarchy)
		{
			if (const UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(Blueprint))
			{
				const int32* SearchIndex = AnimBlueprintClass->AnimBlueprintDebugData.NodePropertyToIndexMap.Find(Node);
				if (SearchIndex)
				{
					return SearchIndex;
				}
			}

		}
		return NULL;
	}

	template<typename StructType>
	const int32* GetNodePropertyIndex(const UAnimGraphNode_Base* Node, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		return (SearchMode == EPropertySearchMode::OnlyThis) ? AnimBlueprintDebugData.NodePropertyToIndexMap.Find(Node) : GetNodePropertyIndexFromHierarchy<StructType>(Node);
	}

	template<typename StructType>
	int32 GetLinkIDForNode(const UAnimGraphNode_Base* Node, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		const int32* pIndex = GetNodePropertyIndex<StructType>(Node, SearchMode);
		if (pIndex)
		{
			return (AnimNodeProperties.Num() - 1 - *pIndex); //@TODO: Crazysauce
		}
		return -1;
	}

	template<typename StructType>
	FStructProperty* GetPropertyForNode(const UAnimGraphNode_Base* Node, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		const int32* pIndex = GetNodePropertyIndex<StructType>(Node, SearchMode);
		if (pIndex)
		{
			if (FStructProperty* AnimationProperty = AnimNodeProperties[AnimNodeProperties.Num() - 1 - *pIndex])
			{
				if (AnimationProperty->Struct->IsChildOf(StructType::StaticStruct()))
				{
					return AnimationProperty;
				}
			}
		}

		return NULL;
	}

	template<typename StructType>
	StructType* GetPropertyInstance(UObject* Object, const UAnimGraphNode_Base* Node, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		FStructProperty* AnimationProperty = GetPropertyForNode<StructType>(Node);
		if (AnimationProperty)
		{
			return AnimationProperty->ContainerPtrToValuePtr<StructType>((void*)Object);
		}

		return NULL;
	}

	template<typename StructType>
	StructType* GetPropertyInstance(UObject* Object, FGuid NodeGuid, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		const int32* pIndex = GetNodePropertyIndexFromGuid(NodeGuid, SearchMode);
		if (pIndex)
		{
			if (FStructProperty* AnimProperty = AnimNodeProperties[AnimNodeProperties.Num() - 1 - *pIndex])
			{
				if (AnimProperty->Struct->IsChildOf(StructType::StaticStruct()))
				{
					return AnimProperty->ContainerPtrToValuePtr<StructType>((void*)Object);
				}
			}
		}

		return NULL;
	}

	template<typename StructType>
	StructType& GetPropertyInstanceChecked(UObject* Object, const UAnimGraphNode_Base* Node, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis)
	{
		const int32 Index = AnimBlueprintDebugData.NodePropertyToIndexMap.FindChecked(Node);
		FStructProperty* AnimationProperty = AnimNodeProperties[AnimNodeProperties.Num() - 1 - Index];
		check(AnimationProperty);
		check(AnimationProperty->Struct->IsChildOf(StructType::StaticStruct()));
		return *AnimationProperty->ContainerPtrToValuePtr<StructType>((void*)Object);
	}

	// Gets the property index from the original UAnimGraphNode's GUID. Does not remap to property order.
	ENGINE_API const int32* GetNodePropertyIndexFromGuid(FGuid Guid, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis);

	// Gets the remapped property index from the original UAnimGraphNode's GUID. Can be used to index the AnimNodeProperties array.
	ENGINE_API int32 GetNodeIndexFromGuid(FGuid Guid, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis);

	ENGINE_API const UEdGraphNode* GetVisualNodeFromNodePropertyIndex(int32 PropertyIndex, EPropertySearchMode::Type SearchMode = EPropertySearchMode::OnlyThis) const;
#endif

	// Called after Link to patch up references to the nodes in the CDO
	ENGINE_API void LinkFunctionsToDefaultObjectNodes(UObject* DefaultObject);

	// Populates AnimBlueprintFunctions according to the UFunction(s) on this class
	ENGINE_API void GenerateAnimationBlueprintFunctions();

	// Build the properties that we cache for our constant data
	ENGINE_API void BuildConstantProperties();

	// Get the fixed names of our generated structs
	static ENGINE_API FName GetConstantsStructName();
	static ENGINE_API FName GetMutablesStructName();
	
#if WITH_EDITOR
	void HandleReinitializeObjectAfterCompile(UObject* Object) const override;
	ENGINE_API virtual void PrepareToConformSparseClassData(UScriptStruct* SparseClassDataArchetypeStruct) override;
	ENGINE_API virtual void ConformSparseClassData(UObject* Object) override;
#endif

	// UObject interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UStruct interface
	ENGINE_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	// End of UStruct interface

	// UClass interface
	ENGINE_API virtual void PurgeClass(bool bRecompilingOnLoad) override;
	ENGINE_API virtual uint8* GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const override;
	ENGINE_API virtual void PostLoadDefaultObject(UObject* Object) override;
	ENGINE_API virtual void PostLoad() override;
	// End of UClass interface
};

template<typename NodeType>
NodeType* GetNodeFromPropertyIndex(UObject* AnimInstanceObject, const IAnimClassInterface* AnimBlueprintClass, int32 PropertyIndex)
{
	if (PropertyIndex != INDEX_NONE)
	{
		FStructProperty* NodeProperty = AnimBlueprintClass->GetAnimNodeProperties()[AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - PropertyIndex]; //@TODO: Crazysauce
		check(NodeProperty->Struct == NodeType::StaticStruct());
		return NodeProperty->ContainerPtrToValuePtr<NodeType>(AnimInstanceObject);
	}

	return NULL;
}
