// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimEnums.h"
#include "Animation/AnimCompressionTypes.h"
#include "AnimEncoding.h"
#include "AnimBoneCompressionCodec.generated.h"

class UAnimBoneCompressionCodec;
class UAnimSequence;
struct FBlendedCurve;
class FBoneData;

namespace UE::Anim
{
struct FAnimPoseDecompressionData;
}


/*
 * Base class for all bone compression codecs.
 */
UCLASS(abstract, hidecategories = Object, EditInlineNew, MinimalAPI)
class UAnimBoneCompressionCodec : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Description for this codec. */
	UPROPERTY(Category = Compression, EditAnywhere)
	FString Description;

	//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
	/** Returns whether or not we can use this codec to compress. */
	virtual bool IsCodecValid() const { return true; }

	/** Returns whether or not this codec is high fidelity. High fidelity codecs requires the original raw data without any sanitizing by the engine. */
	virtual bool IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const { return false; }

	/** Compresses the curve data from an animation sequence. */
	virtual bool Compress(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) PURE_VIRTUAL(UAnimCurveCompressionCodec::Compress, return false;);

	/**
	 * Estimates the peak memory usage in bytes needed to compress the given data with this codec.
	 * This is used to make informed scheduling decisions during asset cooking. Estimates that are too low may cause
	 * out-of-memory conditions. Estimates that are too high can unnecessarily limit the number of concurrent cook
	 * processes.
	 * 
	 * Values below zero indicate that no estimate has been given and high memory usage is hence assumed.
	 * 
	 * @param AnimSequence	The Animation Sequence to estimate the memory usage for.
	 * @returns Estimated peak memory usage in bytes. Values < 0 indicate maximum memory usage.
	 */
	ENGINE_API virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const;

	/*
	 * Called to generate a unique DDC key for this codec instance and input anim sequence and TargetPlatform
	 * A suitable key should be generated from: the InstanceGuid, a codec version, and all relevant properties that drive the behavior.
	 */
	ENGINE_API virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar);

#endif

	/** Allow us to convert DDC serialized path back into codec object */
	ENGINE_API virtual UAnimBoneCompressionCodec* GetCodec(const FString& DDCHandle);

	/** Returns a unique path relative to the parent settings asset */
	ENGINE_API virtual FString GetCodecDDCHandle() const;

	/** Allocates the necessary anim data structure used for decompression. The engine will serialize the data in/out and bind it. */
	virtual TUniquePtr<ICompressedAnimData> AllocateAnimData() const PURE_VIRTUAL(UAnimCurveCompressionCodec::AllocateAnimData, return TUniquePtr<ICompressedAnimData>(););

	/**
	 * Handles Byte-swapping incoming animation data from a MemoryReader
	 *
	 * @param	Seq					An Animation Sequence to contain the read data.
	 * @param	MemoryReader		The MemoryReader object to read from.
	 * @param	SourceArVersion		The version of the archive that the data is coming from.
	 */
	virtual void ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const PURE_VIRTUAL(UAnimCurveCompressionCodec::ByteSwapIn, );

	/**
	 * Handles Byte-swapping outgoing animation data to an array of BYTEs
	 *
	 * @param	Seq					An Animation Sequence to write.
	 * @param	SerializedData		The output buffer.
	 * @param	bForceByteSwapping	true is byte swapping is not optional.
	 */
	virtual void ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const PURE_VIRTUAL(UAnimCurveCompressionCodec::ByteSwapOut, );

	/**
	 * Decompresses all the specified bone tracks.
	 * The caller is responsible for pre-filling the output pose with sensible values (e.g. reference/bind/additive identity pose) as
	 * the codec will only decompress and write out tracks that are contained in the compressed data.
	 */
	virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const PURE_VIRTUAL(UAnimCurveCompressionCodec::DecompressPose, );

	/** Decompress a single bone. */
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const PURE_VIRTUAL(UAnimCurveCompressionCodec::DecompressBone, );

	/**
	 * Decompresses all the specified bone tracks.
	 * The caller is responsible for pre-filling the output pose with sensible values (e.g. reference/bind/additive identity pose) as
	 * the codec will only decompress and write out tracks that are contained in the compressed data.
	 */
	ENGINE_API virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const UE::Anim::FAnimPoseDecompressionData& DecompressionData) const;
};
