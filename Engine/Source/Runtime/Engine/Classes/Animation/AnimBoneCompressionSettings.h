// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/SharedMutex.h"
#include "AnimBoneCompressionSettings.generated.h"

class UAnimBoneCompressionCodec;
class UAnimSequenceBase;

#if WITH_EDITORONLY_DATA
struct FCompressibleAnimData;
struct FCompressibleAnimDataResult;
namespace UE::Anim::Compression { struct FAnimDDCKeyArgs; }
#endif // WITH_EDITORONLY_DATA

/*
 * This object is used to wrap a bone compression codec. It allows a clean integration in the editor by avoiding the need
 * to create asset types and factory wrappers for every codec.
 */
UCLASS(hidecategories = Object, MinimalAPI)
class UAnimBoneCompressionSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** A list of animation bone compression codecs to try. Empty entries are ignored but the array cannot be empty. */
	UPROPERTY(Category = Compression, Instanced, EditAnywhere, meta = (NoElementDuplicate))
	TArray<TObjectPtr<UAnimBoneCompressionCodec>> Codecs;

#if WITH_EDITORONLY_DATA
	/** When compressing, the best codec below this error threshold will be used. */
	UPROPERTY(Category = Compression, EditAnywhere, meta = (ClampMin = "0"))
	float ErrorThreshold;

	/** Any codec (even one that increases the size) with a lower error will be used until it falls below the threshold. */
	UPROPERTY(Category = Compression, EditAnywhere)
	bool bForceBelowThreshold;
#endif

	/** Allow us to convert DDC serialized path back into codec object */
	ENGINE_API UAnimBoneCompressionCodec* GetCodec(const FString& DDCHandle);

	// UObject overrides
	ENGINE_API virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph) override;

	//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
	// UObject overrides
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	/** Returns whether or not we can use these settings to compress. */
	ENGINE_API bool AreSettingsValid() const;

	/** Returns whether or not a codec within is high fidelity. @see UAnimBoneCompressionCodec::IsHighFidelity */
	ENGINE_API bool IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const;

	/*
	 * Compresses the animation bones inside the supplied sequence.
	 * The resultant compressed data is applied to the OutCompressedData structure.
	 */
	ENGINE_API bool Compress(const FCompressibleAnimData& AnimSeq, FCompressibleAnimDataResult& OutCompressedData) const;

	/** Generates a DDC key that takes into account the current settings, selected codec, input anim sequence and TargetPlatform */
	ENGINE_API void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar);
#endif

private:
	/** Guards the Codecs array against concurrent read (GetCodec) and write (PostLoadSubobjects/InstanceSubobjects) access during async loading. */
	mutable UE::FSharedMutex CodecsLock;
};
