// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimEncoding.h"

namespace UE::Anim
{

/*
 * Struct containing the generated pairs to decompress a Pose, with the output atoms arranged in separate arrays
 */
struct FAnimPoseDecompressionData
{
	FAnimPoseDecompressionData(const BoneTrackArray& InRotationPairs
			, const BoneTrackArray& InTranslationPairs
			, const BoneTrackArray& InScalePairs
			, TArrayView<FQuat>& InOutAtomRotations
			, TArrayView<FVector>& InOutAtomTranslations
			, TArrayView<FVector>& InOutAtomScales3D)
		: RotationPairs(InRotationPairs)
		, TranslationPairs(InTranslationPairs)
		, ScalePairs(InScalePairs)
		, OutAtomRotations(InOutAtomRotations)
		, OutAtomTranslations(InOutAtomTranslations)
		, OutAtomScales3D(InOutAtomScales3D)
	{}

	const BoneTrackArray& GetRotationPairs() const
	{
		return RotationPairs;
	}
	const BoneTrackArray& GetTranslationPairs() const
	{
		return TranslationPairs;
	}
	const BoneTrackArray& GetScalePairs() const
	{
		return ScalePairs;
	}

	TArrayView<FQuat>& GetOutAtomRotations() const
	{
		return OutAtomRotations;
	}
	TArrayView<FVector>& GetOutAtomTranslations() const
	{
		return OutAtomTranslations;
	}
	TArrayView<FVector>& GetOutAtomScales3D() const
	{
		return OutAtomScales3D;
	}


protected:
	const BoneTrackArray& RotationPairs;
	const BoneTrackArray& TranslationPairs;
	const BoneTrackArray& ScalePairs;
	TArrayView<FQuat>& OutAtomRotations;
	TArrayView<FVector>& OutAtomTranslations;
	TArrayView<FVector>& OutAtomScales3D;
};

} // end namespace UE::Anim

/*
* Helper macro to iterate two NON OVERLAPPING arrays/array views using restricted raw pointers
* - ItFirstType and ItFirstMember refer to the first array type and variable. It will be accessed through It pointer (all pointers are incremented automatically)
* - ItSecondType and ItSecondMember refer to the second array type variable. It can be accessed through ItRet pointer (all pointers are incremented automatically)
* - ItNumElements is the number of elements that we want to process and that have to exist on BOTH arrays
* Inside the Loop, we can use two pointers :
* - ItFirst : Pointer to the element currently iterated on the first array
* - ItSecond : Pointer to the element currently iterated on the second array 
* 
* Example of iteration over a FQuat array and writing values in a FTransform array
*         ITERATE_ATOMS_START(FQuat, DecompressionData.OutAtomRotations, FTransform, OutAtoms, NumTransforms)
*             ItSecond->SetRotation(*ItFirst);
*         ITERATE_ATOMS_END()
*/
#define ITERATE_NON_OVERLAPPING_ARRAYS_START(ItFirstType, ItFirstMember, ItSecondType, ItSecondMember, ItNumElements) \
if (ItNumElements > 0) \
{ \
	check(ItFirstMember.Num() >= ItNumElements && ItSecondMember.Num() >= ItNumElements); \
	const ItFirstType* RESTRICT ItFirst = ItFirstMember.GetData(); \
	const ItFirstType* RESTRICT ItFirstEnd = ItFirst + ItNumElements; \
	ItSecondType* RESTRICT ItSecond = ItSecondMember.GetData(); \
	for (; ItFirst != ItFirstEnd; ++ItFirst, ++ItSecond) \
	{

#define ITERATE_NON_OVERLAPPING_ARRAYS_END() }}
