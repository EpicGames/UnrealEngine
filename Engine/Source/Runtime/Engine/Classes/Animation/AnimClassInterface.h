// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/AnimSubsystem.h"
#include "UObject/FieldPath.h"

#include "AnimClassInterface.generated.h"

class USkeleton;
struct FExposedValueHandler;
struct FPropertyAccessLibrary;
struct FAnimNodeConstantData;
class IAnimClassInterface;
struct FAnimBlueprintConstantData;
struct FAnimBlueprintMutableData;
struct FAnimNode_Base;
struct FAnimNodeData;
enum class EAnimNodeDataFlags : uint32;
namespace UE { namespace Anim { struct FNodeDataId; } }

/** Describes the input and output of an anim blueprint 'function' */
USTRUCT()
struct FAnimBlueprintFunction
{
	GENERATED_BODY()

	FAnimBlueprintFunction()
		: Name(NAME_None)
		, Group(NAME_None)
		, OutputPoseNodeIndex(INDEX_NONE)
		, OutputPoseNodeProperty(nullptr)
		, bImplemented(false)
	{}

	FAnimBlueprintFunction(const FName& InName)
		: Name(InName)
		, Group(NAME_None)
		, OutputPoseNodeIndex(INDEX_NONE)
		, OutputPoseNodeProperty(nullptr)
		, bImplemented(false)
	{}

	bool operator==(const FAnimBlueprintFunction& InFunction) const
	{
		return Name == InFunction.Name;
	}

	/** The name of the function */
	UPROPERTY()
	FName Name;

	/** The group of the function */
	UPROPERTY()
	FName Group;

	/** Index of the output node */
	UPROPERTY()
	int32 OutputPoseNodeIndex;

	/** The names of the input poses */
	UPROPERTY()
	TArray<FName> InputPoseNames;

	/** Indices of the input nodes */
	UPROPERTY()
	TArray<int32> InputPoseNodeIndices;

	/** The property of the output node, patched up during link */
	FStructProperty* OutputPoseNodeProperty;

	/** The properties of the input nodes, patched up during link */
	TArray< FStructProperty* > InputPoseNodeProperties;

	// A named input property
	struct FInputPropertyData
	{
		// The name of the property
		FName Name = NAME_None;

		// The input property (on the stub function)
		FProperty* FunctionProperty = nullptr;

		// The input property itself (on this class, not the stub function)
		FProperty* ClassProperty = nullptr;
	};

	/** The input properties */
	TArray< FInputPropertyData > InputPropertyData;

	/** Whether this function is actually implemented by this class - it could just be a stub */
	UPROPERTY(transient)
	bool bImplemented;
};

/** Wrapper struct as we dont support nested containers */
USTRUCT()
struct FCachedPoseIndices
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> OrderedSavedPoseNodeIndices;

	bool operator==(const FCachedPoseIndices& InOther) const
	{
		return OrderedSavedPoseNodeIndices == InOther.OrderedSavedPoseNodeIndices;
	}
};

/** Contains indices for any Asset Player nodes found for a specific Name Anim Graph (only and specifically harvested for Anim Graph Layers and Implemented Anim Layer Graphs) */
USTRUCT()
struct FGraphAssetPlayerInformation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<int32> PlayerNodeIndices;
};

/** Blending options for animation graphs in Linked Animation Blueprints. */
USTRUCT()
struct FAnimGraphBlendOptions
{
	GENERATED_USTRUCT_BODY()

	/**
	* Time to blend this graph in using Inertialization. Specify -1.0 to defer to the BlendOutTime of the previous graph.
	* To blend this graph in you must place an Inertialization node after the Linked Anim Graph node or Linked Anim Layer node that uses this graph.
	*/
	UPROPERTY(EditAnywhere, Category = GraphBlending)
	float BlendInTime;       

	/**
	* Optional blend profile to use when blending this graph in (if BlendInTime > 0)
	*/
	UPROPERTY(EditAnywhere, Category = GraphBlending, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendInProfile;

	/**
	* Time to blend this graph out using Inertialization. Specify -1.0 to defer to the BlendInTime of the next graph.
	* To blend this graph out you must place an Inertialization node after the Linked Anim Graph node or Linked Anim Layer node that uses this graph.
	*/
	UPROPERTY(EditAnywhere, Category = GraphBlending)
	float BlendOutTime;

	/**
	* Optional blend profile to use when blending this graph out (if BlendOutTime > 0)
	*/
	UPROPERTY(EditAnywhere, Category = GraphBlending, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendOutProfile;

	FAnimGraphBlendOptions()
		: BlendInTime(-1.0f)
		, BlendInProfile(nullptr)
		, BlendOutTime(-1.0f)
		, BlendOutProfile(nullptr)
	{}
};

// How to proceed when enumerating subsystems
enum class EAnimSubsystemEnumeration
{
	Stop,
	Continue
};

UINTERFACE(MinimalAPI)
class UAnimClassInterface : public UInterface
{
	GENERATED_BODY()
};

typedef TFieldPath<FStructProperty> FStructPropertyPath;

class IAnimClassInterface
{
	GENERATED_BODY()
public:
	virtual const TArray<FBakedAnimationStateMachine>& GetBakedStateMachines() const = 0;
	virtual const TArray<FAnimNotifyEvent>& GetAnimNotifies() const = 0;
	virtual const TArray<FStructProperty*>& GetAnimNodeProperties() const = 0;
	UE_DEPRECATED(4.24, "Function has been renamed, please use GetLinkedAnimGraphNodeProperties")
	virtual const TArray<FStructProperty*>& GetSubInstanceNodeProperties() const { return GetLinkedAnimGraphNodeProperties(); }
	virtual const TArray<FStructProperty*>& GetLinkedAnimGraphNodeProperties() const = 0;
	UE_DEPRECATED(4.24, "Function has been renamed, please use GetLinkedLayerNodeProperties")
	virtual const TArray<FStructProperty*>& GetLayerNodeProperties() const { return GetLinkedAnimLayerNodeProperties(); }
	virtual const TArray<FStructProperty*>& GetLinkedAnimLayerNodeProperties() const = 0;
	virtual const TArray<FStructProperty*>& GetPreUpdateNodeProperties() const = 0;
	virtual const TArray<FStructProperty*>& GetDynamicResetNodeProperties() const = 0;
	virtual const TArray<FStructProperty*>& GetStateMachineNodeProperties() const = 0;
	virtual const TArray<FStructProperty*>& GetInitializationNodeProperties() const = 0;
	virtual const TArray<FName>& GetSyncGroupNames() const = 0;
	virtual const TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseNodeIndicesMap() const = 0;
	virtual const TArray<FAnimBlueprintFunction>& GetAnimBlueprintFunctions() const = 0;
	virtual const TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation() const = 0;
	virtual const TMap<FName, FAnimGraphBlendOptions>& GetGraphBlendOptions() const = 0;
	virtual USkeleton* GetTargetSkeleton() const = 0;
	virtual int32 GetSyncGroupIndex(FName SyncGroupName) const = 0;

	// Iterate over each subsystem for this class, supplying both the constant (FAnimSubsystem) and mutable (FAnimSubsystemInstance) data
	virtual void ForEachSubsystem(TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemContext&)> InFunction) const  = 0;
	virtual void ForEachSubsystem(UObject* InObject, TFunctionRef<EAnimSubsystemEnumeration(const FAnimSubsystemInstanceContext&)> InFunction) const = 0;

	// Find a subsystem's class-resident data. If no subsystem of the type exists this will return nullptr.
	// @param	InSubsystemType	The subsystem's type
	virtual const FAnimSubsystem* FindSubsystem(UScriptStruct* InSubsystemType) const = 0;

	// Get a subsystem's class-resident data. If no subsystem of the type exists this will return nullptr.
	template<typename SubsystemType>
	const SubsystemType* FindSubsystem() const
	{
		const FAnimSubsystem* Subsystem = FindSubsystem(SubsystemType::StaticStruct());
		return static_cast<const SubsystemType*>(Subsystem);
	}
	
	// Get a subsystem's class-resident data. If no subsystem of the type exists this will assert.
	template<typename SubsystemType>
	const SubsystemType& GetSubsystem() const
	{
		const FAnimSubsystem* Subsystem = FindSubsystem(SubsystemType::StaticStruct());
		check(Subsystem);
		return static_cast<const SubsystemType&>(*Subsystem);
	}

	// Check whether a node at the specified index has the specified flags
	static ENGINE_API bool HasNodeAnyFlags(IAnimClassInterface* InAnimClassInterface, int32 InNodeIndex, EAnimNodeDataFlags InNodeDataFlags);

protected:
	// These direct accessors are here to allow internal access that doesnt redirect to the root class
	virtual const TArray<FBakedAnimationStateMachine>& GetBakedStateMachines_Direct() const = 0;
	virtual const TArray<FAnimNotifyEvent>& GetAnimNotifies_Direct() const = 0;
	virtual const TArray<FName>& GetSyncGroupNames_Direct() const = 0;
	virtual const TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseNodeIndicesMap_Direct() const = 0;
	virtual const TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation_Direct() const = 0;
	virtual const TMap<FName, FAnimGraphBlendOptions>& GetGraphBlendOptions_Direct() const = 0;
protected:
	friend class UAnimBlueprintGeneratedClass;
	friend struct FAnimNodeData;
	friend struct UE::Anim::FNodeDataId;

	// Access the various constant and mutable values
	virtual const void* GetConstantNodeValueRaw(int32 InIndex) const = 0;
	virtual const void* GetMutableNodeValueRaw(int32 InIndex, const UObject* InObject) const = 0;

	// Get the struct that holds the mutable data
	// @param	InObject	The anim instance object that holds the mutable data
	virtual const FAnimBlueprintMutableData* GetMutableNodeData(const UObject* InObject) const = 0;
	virtual FAnimBlueprintMutableData* GetMutableNodeData(UObject* InObject) const = 0;

	// Get the struct that holds the constant data
	virtual const void* GetConstantNodeData() const = 0;

	// Get the anim node data used for each node's constant/folded data
	virtual TArrayView<const FAnimNodeData> GetNodeData() const = 0;

	// Get the (editor-only data) index of the property
	virtual int32 GetAnimNodePropertyIndex(const UScriptStruct* InNodeType, FName InPropertyName) const = 0;

	// Get the number of properties (including editor only properties) that the anim node type has
	virtual int32 GetAnimNodePropertyCount(const UScriptStruct* InNodeType) const = 0;

#if WITH_EDITORONLY_DATA
	// Check that the serialized NodeTypeMap can be used with the current set of native node data layouts
	virtual bool IsDataLayoutValid() const = 0;
#endif
public:

	// Get the root anim class interface (i.e. if this is a derived class).
	// Some properties that are derived from the compiled anim graph are routed to the 'Root' class
	// as child classes don't get fully compiled. Instead they just override various asset players leaving the
	// full compilation up to the base class. 
	ENGINE_API const IAnimClassInterface* GetRootClass() const;

	static ENGINE_API IAnimClassInterface* GetFromClass(UClass* InClass);

	static ENGINE_API const IAnimClassInterface* GetFromClass(const UClass* InClass);

	static ENGINE_API UClass* GetActualAnimClass(IAnimClassInterface* AnimClassInterface);

	static ENGINE_API const UClass* GetActualAnimClass(const IAnimClassInterface* AnimClassInterface);

	static ENGINE_API const FAnimBlueprintFunction* FindAnimBlueprintFunction(IAnimClassInterface* AnimClassInterface, const FName& InFunctionName);

	/**
	 * Check if a function is an anim function on this class
	 * @param	InAnimClassInterface	The interface to check
	 * @param	InFunction				The function to check
	 * @return true if the supplied function is an anim function on the specified class
	 */
	static ENGINE_API bool IsAnimBlueprintFunction(IAnimClassInterface* InAnimClassInterface, const UFunction* InFunction);

	// Get the object ptr given an anim node
	static ENGINE_API const UObject* GetObjectPtrFromAnimNode(const IAnimClassInterface* InAnimClassInterface, const FAnimNode_Base* InNode);

	// Get an anim node of the specified type given the object & node index
	// Asserts if InObject is nullptr
	// @return nullptr if the node index was out of bounds or the incorrect type
	static ENGINE_API const FAnimNode_Base* GetAnimNodeFromObjectPtr(const UObject* InObject, int32 InNodeIndex, UScriptStruct* InNodeType);

	// Get an anim node of the specified type given the object & node index
	// Asserts if InObject is nullptr, the node index is out of bounds or the node is the incorrect type
	template<typename NodeType>
	static const NodeType& GetAnimNodeFromObjectPtrChecked(const UObject* InObject, int32 InNodeIndex)
	{
		const FAnimNode_Base* NodePtr = GetAnimNodeFromObjectPtr(InObject, InNodeIndex, NodeType::StaticStruct());
		check(NodePtr);
		return static_cast<NodeType&>(*NodePtr);
	}
	
	UE_DEPRECATED(4.23, "Please use GetAnimBlueprintFunctions()")
	virtual int32 GetRootAnimNodeIndex() const { return INDEX_NONE; }

	UE_DEPRECATED(4.23, "Please use GetAnimBlueprintFunctions()")
	virtual FStructProperty* GetRootAnimNodeProperty() const { return nullptr; }
};
