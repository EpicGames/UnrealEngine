// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Abstract base class of animation made of multiple sequences.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/AnimCompositeBase.h"
#include "AnimComposite.generated.h"

class UAnimSequence;
struct FCompactPose;

/**
* Animation Composites serve as a way to combine multiple animations together and treat them as a single unit.
*/
UCLASS(config=Engine, hidecategories=UObject, MinimalAPI, BlueprintType)
class UAnimComposite : public UAnimCompositeBase
{
	GENERATED_UCLASS_BODY()

public:
	/** Serializable data that stores section/anim pairing **/
	UPROPERTY()
	struct FAnimTrack AnimationTrack;

#if WITH_EDITORONLY_DATA
	/** Preview Base pose for additive BlendSpace **/
	UPROPERTY(EditAnywhere, Category=AdditiveSettings)
	TObjectPtr<UAnimSequence> PreviewBasePose;
#endif // WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	//~ Begin UAnimSequenceBase Interface
	ENGINE_API virtual void HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const override;

	virtual void GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const override;
	virtual EAdditiveAnimationType GetAdditiveAnimType() const override;
	virtual bool IsValidAdditive() const override { return GetAdditiveAnimType() != AAT_None; }
	virtual void EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock) override;
	virtual bool HasRootMotion() const override;
	virtual FTransform ExtractRootMotion(const FAnimExtractContext& ExtractionContext) const override final;
	virtual FTransform ExtractRootMotionFromRange(double StartTime, double EndTime, const FAnimExtractContext& ExtractionContext) const override final;
	virtual FTransform ExtractRootTrackTransform(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const override final;
	virtual void GetAnimNotifiesFromDeltaPositions(const float& PreviousPosition, const float & CurrentPosition, FAnimNotifyContext& NotifyContext) const override;
	virtual bool IsNotifyAvailable() const override;
	//~ End UAnimSequenceBase Interface
	//~ Begin UAnimSequence Interface
#if WITH_EDITOR
	virtual class UAnimSequence* GetAdditiveBasePose() const override;
	virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;
	virtual void UpdateCommonTargetFrameRate() override;	
#endif
	//~ End UAnimSequence Interface

	//~ Begin UAnimCompositeBase Interface
	virtual void InvalidateRecursiveAsset() override;
	virtual bool ContainRecursive(TArray<UAnimCompositeBase*>& CurrentAccumulatedList) override;
	virtual void SetCompositeLength(float InLength) override;
	virtual void PostLoad() override;
	virtual FFrameRate GetSamplingFrameRate() const override;
	//~End UAnimCompositeBase Interface
};

