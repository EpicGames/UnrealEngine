// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Abstract base class of animation composite base
 * This contains Composite Section data and some necessary interface to make this work
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimSequenceBase.h"
#include "HAL/IConsoleManager.h"
#include "AnimCompositeBase.generated.h"

class UAnimCompositeBase;
class UAnimSequence;
struct FCompactPose;

#if WITH_EDITOR
namespace UE { namespace Anim
{
	extern TAutoConsoleVariable<bool> CVarOutputMontageFrameRateWarning;
}}

#endif // WITH_EDITOR

/** Struct defining a RootMotionExtractionStep.
 * When extracting RootMotion we can encounter looping animations (wrap around), or different animations.
 * We break those up into different steps, to help with RootMotion extraction, 
 * as we can only extract a contiguous range per AnimSequence.
 */
USTRUCT()
struct FRootMotionExtractionStep
{
	GENERATED_USTRUCT_BODY()

	/** AnimSequence ref */
	UPROPERTY()
	TObjectPtr<UAnimSequence> AnimSequence;

	/** Start position to extract root motion from. */
	UPROPERTY()
	float StartPosition;

	/** End position to extract root motion to. */
	UPROPERTY()
	float EndPosition;

	FRootMotionExtractionStep() 
		: AnimSequence(nullptr)
		, StartPosition(0.f)
		, EndPosition(0.f)
		{
		}

	FRootMotionExtractionStep(UAnimSequence * InAnimSequence, float InStartPosition, float InEndPosition) 
		: AnimSequence(InAnimSequence)
		, StartPosition(InStartPosition)
		, EndPosition(InEndPosition)
	{
	}
};

/** this is anim segment that defines what animation and how **/
USTRUCT(BlueprintType)
struct FAnimSegment
{
	GENERATED_USTRUCT_BODY()

	FAnimSegment(const FAnimSegment&) = default;
	FAnimSegment(FAnimSegment&&) = default;
	FAnimSegment& operator=(const FAnimSegment&) = default;
	FAnimSegment& operator=(FAnimSegment&&) = default;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=AnimSegment, meta=(DisplayName = "Cached Animation Asset length"))
	float CachedPlayLength = 0.f;
#endif

#if WITH_EDITOR
	friend class UEditorAnimSegment;
	friend class UEditorAnimCompositeSegment;
	ENGINE_API void UpdateCachedPlayLength();
#endif // WITH_EDITOR

protected:
	/** Anim Reference to play - only allow AnimSequence or AnimComposite **/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimSegment, meta=(DisplayName = "Animation Reference"))
	TObjectPtr<UAnimSequenceBase> AnimReference;
public:

	ENGINE_API void SetAnimReference(UAnimSequenceBase* InAnimReference, bool bInitialize = false);
	const TObjectPtr<UAnimSequenceBase>& GetAnimReference() const 
	{
		return AnimReference;
	}

#if WITH_EDITOR
	ENGINE_API bool IsPlayLengthOutOfDate() const;
#endif // WITH_EDITOR
	
	/** Start Pos within this AnimCompositeBase */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=AnimSegment, meta=(DisplayName = "Starting Position"))
	float StartPos;

	/** Time to start playing AnimSequence at. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimSegment, meta=(DisplayName = "Start Time"))
	float AnimStartTime;

	/** Time to end playing the AnimSequence at. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimSegment, meta=(DisplayName = "End Time"))
	float AnimEndTime;

	/** Playback speed of this animation. If you'd like to reverse, set -1*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimSegment, meta=(DisplayName = "Play Rate"))
	float AnimPlayRate;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimSegment, meta=(DisplayName = "Loop Count"))
	int32 LoopingCount;

	FAnimSegment()
		: AnimReference(nullptr)
		, StartPos(0.f)
		, AnimStartTime(0.f)
		, AnimEndTime(0.f)
		, AnimPlayRate(1.f)
		, LoopingCount(1)
		, bValid(true)
	{
	}

	/** Ensures PlayRate is non Zero */
	float GetValidPlayRate() const
	{
		float SeqPlayRate = AnimReference ? AnimReference->RateScale : 1.0f;
		float FinalPlayRate = SeqPlayRate * AnimPlayRate;
		return (FMath::IsNearlyZero(FinalPlayRate) ? 1.f : FinalPlayRate);
	}

	float GetLength() const
	{
		return (float(LoopingCount) * (AnimEndTime - AnimStartTime)) / FMath::Abs(GetValidPlayRate());
	}

	/** End Position within this AnimCompositeBase */
	float GetEndPos() const
	{
		return StartPos + GetLength();
	}

	bool IsInRange(float CurPos) const
	{
		return ((CurPos >= StartPos) && (CurPos <= GetEndPos()));
	}

	/*
	 * Return true if it's included within the input range
	 */
	bool IsIncluded(float InStartPos, float InEndPos) const
	{
		float EndPos = StartPos + GetLength(); 
		// InStartPos is between Start and End, it is included
		if (StartPos <= InStartPos && EndPos > InStartPos)
		{
			return true;
		}
		// InEndPos is between Start and End, it is also included
		if (StartPos < InEndPos && EndPos >= InEndPos)
		{
			return true;
		}
		// if it is within Start and End, it is also included
		if (StartPos >= InStartPos && EndPos <= InEndPos)
		{
			return true;
		}

		return false;
	}
	/**
	 * Get Animation Data.
	 */
	ENGINE_API UAnimSequenceBase* GetAnimationData(float PositionInTrack, float& PositionInAnim) const;

	/** Converts 'Track Position' to position on AnimSequence.
	 * Note: doesn't check that position is in valid range, must do that before calling this function! */
	ENGINE_API float ConvertTrackPosToAnimPos(const float& TrackPosition) const;
	
	/**
	* Retrieves AnimNotifies between two Track time positions. ]PreviousTrackPosition, CurrentTrackPosition]
	* Between PreviousTrackPosition (exclusive) and CurrentTrackPosition (inclusive).
	* Supports playing backwards (CurrentTrackPosition<PreviousTrackPosition).
	* Only supports contiguous range, does NOT support looping and wrapping over.
	*/
	ENGINE_API void GetAnimNotifiesFromTrackPositions(const float& PreviousTrackPosition, const float& CurrentTrackPosition, FAnimNotifyContext& NotifyContext) const;

	/** 
	 * Given a Track delta position [StartTrackPosition, EndTrackPosition]
	 * See if this AnimSegment overlaps any of it, and if it does, break it up into RootMotionExtractionSteps.
	 * Supports animation playing forward and backward. Track segment should be a contiguous range, not wrapping over due to looping.
	 */
	void GetRootMotionExtractionStepsForTrackRange(TArray<FRootMotionExtractionStep> & RootMotionExtractionSteps, const float StartPosition, const float EndPosition) const;

	/**
	 * return true if valid, false otherwise, only invalid if we contains recursive reference
	 **/
	bool IsValid() const { return bValid;  }

	/** 
	 * return true if anim notify is available 
	 */
	bool IsNotifyAvailable() const { return IsValid() && AnimReference && AnimReference->IsNotifyAvailable(); }
private:

	/**
	 * This gets invalidated if this section started recursive
	**/
	bool bValid;

	friend struct FAnimTrack;
};

/** This is list of anim segments for this track 
 * For now this is only one TArray, but in the future 
 * we should define more transition/blending behaviors
 **/
USTRUCT(BlueprintType)
struct FAnimTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=AnimTrack, EditFixedSize)
	TArray<FAnimSegment>	AnimSegments;

	FAnimTrack() {}
	ENGINE_API float GetLength() const;
	ENGINE_API bool IsAdditive() const;
	bool IsRotationOffsetAdditive() const;

	ENGINE_API int32 GetTrackAdditiveType() const;

	/** Returns whether any of the animation sequences this track uses has root motion */
	bool HasRootMotion() const;

	/** 
	 * Given a Track delta position [StartTrackPosition, EndTrackPosition]
	 * See if any AnimSegment overlaps any of it, and if it does, break it up into RootMotionExtractionPieces.
	 * Supports animation playing forward and backward. Track segment should be a contiguous range, not wrapping over due to looping.
	 */
	ENGINE_API void GetRootMotionExtractionStepsForTrackRange(TArray<FRootMotionExtractionStep> & RootMotionExtractionSteps, const float StartTrackPosition, const float EndTrackPosition) const;

	/** Ensure segment times are correctly formed (no gaps and no extra time at the end of the anim reference) */
	void ValidateSegmentTimes();

	/** return true if valid to add */
	ENGINE_API bool IsValidToAdd(const UAnimSequenceBase* SequenceBase, FText* OutReason = nullptr) const;

	/** Gets the index of the segment at the given absolute montage time. */	
	ENGINE_API int32 GetSegmentIndexAtTime(float InTime) const;

	/** Get the segment at the given absolute montage time */
	ENGINE_API FAnimSegment* GetSegmentAtTime(float InTime);
	ENGINE_API const FAnimSegment* GetSegmentAtTime(float InTime) const;

	/** Get animation pose function */	
	ENGINE_API void GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const;

	/** Enable Root motion setting from montage */
	void EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock);

#if WITH_EDITOR
	bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive) const;
	void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap);

	/** Moves anim segments so that there are no gaps between one finishing
	 *  and the next starting, preserving the order of AnimSegments
	 */
	ENGINE_API void CollapseAnimSegments();

	/** Sorts AnimSegments based on the start time of each segment */
	ENGINE_API void SortAnimSegments();

	/** Get Addiitve Base Pose if additive */
	class UAnimSequence* GetAdditiveBasePose() const;
#endif

	// this is to prevent anybody adding recursive asset to anim composite
	// as a result of anim composite being a part of anim sequence base
	void InvalidateRecursiveAsset(class UAnimCompositeBase* CheckAsset);

	// this is recursive function that look thorough internal assets 
	// and clear the reference if recursive is found. 
	// We're going to remove the top reference if found
	bool ContainRecursive(const TArray<UAnimCompositeBase*>& CurrentAccumulatedList);

	/**
	* Retrieves AnimNotifies between two Track time positions. ]PreviousTrackPosition, CurrentTrackPosition]
	* Between PreviousTrackPosition (exclusive) and CurrentTrackPosition (inclusive).
	* Supports playing backwards (CurrentTrackPosition<PreviousTrackPosition).
	* Only supports contiguous range, does NOT support looping and wrapping over.
	*/
	ENGINE_API void GetAnimNotifiesFromTrackPositions(const float& PreviousTrackPosition, const float& CurrentTrackPosition, FAnimNotifyContext& NotifyContext) const;

	/** return true if anim notify is available */
	bool IsNotifyAvailable() const;

	ENGINE_API int32 GetTotalBytesUsed() const;
};

UCLASS(abstract, MinimalAPI)
class UAnimCompositeBase : public UAnimSequenceBase
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin UAnimSequenceBase Interface
	virtual FFrameRate GetSamplingFrameRate() const override;
	//~ End UAnimSequenceBase Interface

	// Extracts root motion from the supplied FAnimTrack between the Start End range specified
	ENGINE_API void ExtractRootMotionFromTrack(const FAnimTrack &SlotAnimTrack, float StartTrackPosition, float EndTrackPosition, const FAnimExtractContext& Context, FRootMotionMovementParams& RootMotion) const;

	// this is to prevent anybody adding recursive asset to anim composite
	// as a result of anim composite being a part of anim sequence base
	virtual void InvalidateRecursiveAsset() PURE_VIRTUAL(UAnimCompositeBase::InvalidateRecursiveAsset, );

	// this is recursive function that look thorough internal assets 
	// and clear the reference if recursive is found. 
	// We're going to remove the top reference if found
	virtual bool ContainRecursive(TArray<UAnimCompositeBase*>& CurrentAccumulatedList) PURE_VIRTUAL(UAnimCompositeBase::ContainRecursive, return false; );

	virtual void SetCompositeLength(float InLength) PURE_VIRTUAL(UAnimCompositeBase::SetCompositeLength, );

#if WITH_EDITOR
	virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel) override;

	virtual void UpdateCommonTargetFrameRate() PURE_VIRTUAL(UAnimCompositeBase::UpdateCommonTargetFrameRate, );	
#endif // WITH_EDITOR
	FFrameRate GetCommonTargetFrameRate() const { return CommonTargetFrameRate; }
protected:
	/** Frame-rate used to represent this Animation Montage (best fitting for placed Animation Sequences)*/
	UPROPERTY(VisibleAnywhere, Category = AnimationComposite)
	FFrameRate CommonTargetFrameRate;
};

