// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Baseclass for animation compression algorithms.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/Function.h"
#include "Templates/Greater.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "AnimationUtils.h"
#include "AnimEnums.h"
#include "AnimationCompression.h"
#include "AnimCompress.generated.h"

struct FCompressibleAnimData;

//Helper function for ddc key generation
uint8 MakeBitForFlag(uint32 Item, uint32 Position);

// Logic for tracking top N error items for later display
template<typename DataType, typename SortType, int MaxItems>
struct FMaxErrorStatTracker
{
public:
	FMaxErrorStatTracker()
		: CurrentLowestError(0.f)
	{
		Items.Reserve(MaxItems);
	}

	bool CanUseErrorStat(SortType NewError)
	{
		return Items.Num() < MaxItems || NewError > CurrentLowestError;
	}

	template <typename... ArgsType>
	void StoreErrorStat(SortType NewError, ArgsType&&... Args)
	{
		bool bModified = false;

		if (Items.Num() < MaxItems)
		{
			Items.Emplace(Forward<ArgsType>(Args)...);
			bModified = true;
		}
		else if(NewError > CurrentLowestError)
		{
			Items[MaxItems - 1] = DataType(Forward<ArgsType>(Args)...);
			bModified = true;
		}

		if (bModified)
		{
			Algo::Sort(Items, TGreater<>());
			CurrentLowestError = Items.Last().GetErrorValue();
		}
	}

	void LogErrorStat()
	{
		for (int ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex)
		{
			UE_LOGF(LogAnimationCompression, Display, "%i) %ls", ItemIndex+1, *Items[ItemIndex].ToText().ToString());
		}
	}

	const DataType& GetMaxErrorItem() const
	{
		return Items[0];
	}

	bool IsValid() const
	{
		return Items.Num() > 0;
	}

private:
	//Storage of tracked items
	TArray<DataType> Items;

	//For ease cache current lowest error value
	SortType CurrentLowestError;
};

struct FErrorTrackerWorstBone
{
	FErrorTrackerWorstBone()
		: BoneError(0)
		, BoneErrorTime(0)
		, BoneErrorBone(0)
		, BoneErrorBoneName(NAME_None)
		, BoneErrorAnimName(NAME_None)
	{}

	FErrorTrackerWorstBone(float InBoneError, float InBoneErrorTime, int32 InBoneErrorBone, FName InBoneErrorBoneName, FName InBoneErrorAnimName)
		: BoneError(InBoneError)
		, BoneErrorTime(InBoneErrorTime)
		, BoneErrorBone(InBoneErrorBone)
		, BoneErrorBoneName(InBoneErrorBoneName)
		, BoneErrorAnimName(InBoneErrorAnimName)
	{}

	bool operator<(const FErrorTrackerWorstBone& Rhs) const
	{
		return BoneError < Rhs.BoneError;
	}

	float GetErrorValue() const { return BoneError; }

	FText ToText() const
	{
		FNumberFormattingOptions Options;
		Options.MinimumIntegralDigits = 1;
		Options.MinimumFractionalDigits = 3;

		FFormatNamedArguments Args;
		Args.Add(TEXT("BoneError"), FText::AsNumber(BoneError, &Options));
		Args.Add(TEXT("BoneErrorAnimName"), FText::FromName(BoneErrorAnimName));
		Args.Add(TEXT("BoneErrorBoneName"), FText::FromName(BoneErrorBoneName));
		Args.Add(TEXT("BoneErrorBone"), BoneErrorBone);
		Args.Add(TEXT("BoneErrorTime"), FText::AsNumber(BoneErrorTime, &Options));

		return FText::Format(NSLOCTEXT("Engine", "CompressionWorstBoneSummary", "{BoneError} in Animation {BoneErrorAnimName}, Bone : {BoneErrorBoneName}(#{BoneErrorBone}), at Time {BoneErrorTime}"), Args);
	}

	// Error of this bone
	float BoneError;

	// Time in the sequence that the error occurred at
	float BoneErrorTime;

	// Bone index the error occurred on
	int32 BoneErrorBone;

	// Bone name the error occurred on 
	FName BoneErrorBoneName;

	// Animation the error occurred on
	FName BoneErrorAnimName;
};

struct FErrorTrackerWorstAnimation
{
	FErrorTrackerWorstAnimation()
		: AvgError(0)
		, AnimName(NAME_None)
	{}

	FErrorTrackerWorstAnimation(float InAvgError, FName InMaxErrorAnimName)
		: AvgError(InAvgError)
		, AnimName(InMaxErrorAnimName)
	{}

	bool operator<(const FErrorTrackerWorstAnimation& Rhs) const
	{
		return AvgError < Rhs.AvgError;
	}

	float GetErrorValue() const { return AvgError; }

	FText ToText() const
	{
		FNumberFormattingOptions Options;
		Options.MinimumIntegralDigits = 1;
		Options.MinimumFractionalDigits = 3;

		FFormatNamedArguments Args;
		Args.Add(TEXT("AvgError"), FText::AsNumber(AvgError, &Options));
		Args.Add(TEXT("AnimName"), FText::FromName(AnimName));

		return FText::Format(NSLOCTEXT("Engine", "CompressionWorstAnimationSummary", "{AvgError} in Animation {AnimName}"), Args);
	}

private:

	// Average error of this animation
	float AvgError;

	// Animation being tracked
	FName AnimName;
};

class FCompressionMemorySummary
{
public:
	ENGINE_API FCompressionMemorySummary(bool bInEnabled);

	ENGINE_API void GatherPreCompressionStats(int32 RawSize, int32 PreviousCompressionSize);

	ENGINE_API void GatherPostCompressionStats(const FCompressedAnimSequence& CompressedData, const TArray<FBoneData>& BoneData, const FName AnimFName, double CompressionTime, bool bInPerformedCompression);

	ENGINE_API ~FCompressionMemorySummary();

private:
	bool bEnabled;
	bool bUsed;
	bool bPerformedCompression;
	int64 TotalRaw;
	int64 TotalBeforeCompressed;
	int64 TotalAfterCompressed;
	int32 NumberOfAnimations;

	// Total time spent compressing animations
	double TotalCompressionExecutionTime;

	// Stats across all animations
	float ErrorTotal;
	float ErrorCount;
	float AverageError;

	// Track the largest errors on a single bone
	FMaxErrorStatTracker<FErrorTrackerWorstBone, float, 10> WorstBoneError;

	// Track the animations with the largest average error
	FMaxErrorStatTracker<FErrorTrackerWorstAnimation, float, 10> WorstAnimationError;
};

#if WITH_EDITOR
namespace UE
{
	namespace Anim
	{
		namespace Compression
		{		
			// This is a version string that mimics the old versioning scheme. If you
			// want to bump this version, generate a new guid using VS->Tools->Create GUID and
			// return it here. Ex.
			static const FString AnimationCompressionVersionString = "0439926D560447329623BE4394FA11A6";
			
			struct FAnimationCompressionMemorySummaryScope
			{
				FAnimationCompressionMemorySummaryScope()
				{
					bool bExpected = false;
					check(ScopeExists.compare_exchange_strong(bExpected, true));
					CompressionSummary = MakeUnique<FCompressionMemorySummary>(true);
				}

				~FAnimationCompressionMemorySummaryScope()
				{
					bool bExpected = true;
					check(ScopeExists.compare_exchange_strong(bExpected, false));
					CompressionSummary.Reset();
				}

				static bool ShouldStoreCompressionResults()
				{
					return ScopeExists.load();
				}
	
				static FCompressionMemorySummary& CompressionResultSummary()
				{
					check(ScopeExists.load());
					return *CompressionSummary.Get();
				}

				static ENGINE_API std::atomic<bool> ScopeExists;
				static ENGINE_API TUniquePtr<FCompressionMemorySummary> CompressionSummary;
			};
		}
	}
}
#endif // WITH_EDITOR


UCLASS(abstract, hidecategories=Object, MinimalAPI, EditInlineNew)
class UAnimCompress : public UAnimBoneCompressionCodec
{
	GENERATED_UCLASS_BODY()

	/** Compression algorithms requiring a skeleton should set this value to true. */
	UPROPERTY()
	uint32 bNeedsSkeleton:1;

	/** Format for bitwise compression of translation data. */
	UPROPERTY(Category = Compression, EditAnywhere)
	TEnumAsByte<AnimationCompressionFormat> TranslationCompressionFormat;

	/** Format for bitwise compression of rotation data. */
	UPROPERTY(Category = Compression, EditAnywhere)
	TEnumAsByte<AnimationCompressionFormat> RotationCompressionFormat;

	/** Format for bitwise compression of scale data. */
	UPROPERTY(Category = Compression, EditAnywhere)
	TEnumAsByte<AnimationCompressionFormat> ScaleCompressionFormat;

public:
#if WITH_EDITORONLY_DATA
	/** UAnimBoneCompressionCodec implementation */
	virtual bool Compress(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
#endif

	virtual TUniquePtr<ICompressedAnimData> AllocateAnimData() const override;

	virtual void ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const override;
	virtual void ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const override;

	virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const override;
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const override;

protected:
#if WITH_EDITOR
	/**
	 * Implemented by child classes, this function reduces the number of keyframes in
	 * the specified sequence, given the specified skeleton (if needed).
	 *
	 * @return		true if the keyframe reduction was successful.
	 */
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) PURE_VIRTUAL(UAnimCompress::DoReduction,return false;);
#endif // WITH_EDITOR

	/**
	 * Common compression utility to remove 'redundant' position keys based on the provided delta threshold
	 *
	 * @param	Track			Position tracks to reduce
	 * @param	MaxPosDelta		Maximum local-space threshold for stationary motion
	 */
	static void FilterTrivialPositionKeys(
		TArray<struct FTranslationTrack>& Track,
		float MaxPosDelta);

	/**
	 * Common compression utility to remove 'redundant' position keys in a single track based on the provided delta threshold
	 *
	 * @param	Track			Track to reduce
	 * @param	MaxPosDelta		Maximum local-space threshold for stationary motion
	 */
	static void FilterTrivialPositionKeys(
		struct FTranslationTrack& Track,
		float MaxPosDelta);

	/**
	 * Common compression utility to remove 'redundant' rotation keys in a set of tracks based on the provided delta threshold
	 *
	 * @param	InputTracks		Array of rotation track elements to reduce
	 * @param	MaxRotDelta		Maximum angle threshold to consider stationary motion
	 */
	static void FilterTrivialRotationKeys(
		TArray<struct FRotationTrack>& InputTracks,
		float MaxRotDelta);

	/**
	 * Common compression utility to remove 'redundant' rotation keys in a set of tracks based on the provided delta threshold
	 *
	 * @param	Track			Track to reduce
	 * @param	MaxRotDelta		Maximum angle threshold to consider stationary motion
	 */
	static void FilterTrivialRotationKeys(
		struct FRotationTrack& Track,
		float MaxRotDelta);

	/**
	 * Common compression utility to remove 'redundant' Scale keys based on the provided delta threshold
	 *
	 * @param	Track			Scale tracks to reduce
	 * @param	MaxPosDelta		Maximum local-space threshold for stationary motion
	 */
	static void FilterTrivialScaleKeys(
		TArray<struct FScaleTrack>& Track,
		float MaxScaleDelta);

	/**
	 * Common compression utility to remove 'redundant' Scale keys in a single track based on the provided delta threshold
	 *
	 * @param	Track			Track to reduce
	 * @param	MaxPosDelta		Maximum local-space threshold for stationary motion
	 */
	static void FilterTrivialScaleKeys(
		struct FScaleTrack& Track,
		float MaxScaleDelta);

	
	/**
	 * Common compression utility to remove 'redundant' keys based on the provided delta thresholds
	 *
	 * @param	PositionTracks	Array of position track elements to reduce
	 * @param	RotationTracks	Array of rotation track elements to reduce
	 * @param	ScaleTracks		Array of scale track elements to reduce
	 * @param	MaxPosDelta		Maximum local-space threshold for stationary motion
	 * @param	MaxRotDelta		Maximum angle threshold to consider stationary motion
	 * @param	MaxScaleDelta	Maximum scale threshold to consider stationary motion
	 */
	static void FilterTrivialKeys(
		TArray<struct FTranslationTrack>& PositionTracks,
		TArray<struct FRotationTrack>& RotationTracks,
		TArray<struct FScaleTrack>& ScaleTracks,
		float MaxPosDelta,
		float MaxRotDelta, 
		float MaxScaleDelta);

	/**
	 * Common compression utility to retain only intermittent position keys. For example,
	 * calling with an Interval of 3 would keep every third key in the set and discard the rest
	 *
	 * @param	PositionTracks	Array of position track elements to reduce
	 * @param	StartIndex		Index at which to begin reduction
	 * @param	Interval		Interval of keys to retain
	 */
	static void FilterIntermittentPositionKeys(
		TArray<struct FTranslationTrack>& PositionTracks,
		int32 StartIndex,
		int32 Interval);

	/**
	 * Common compression utility to retain only intermittent position keys. For example,
	 * calling with an Interval of 3 would keep every third key in the set and discard the rest
	 *
	 * @param	Track			Track to reduce
	 * @param	StartIndex		Index at which to begin reduction
	 * @param	Interval		Interval of keys to retain
	 */
	static void FilterIntermittentPositionKeys(
		struct FTranslationTrack& Track,
		int32 StartIndex,
		int32 Interval);

	/**
	 * Common compression utility to retain only intermittent rotation keys. For example,
	 * calling with an Interval of 3 would keep every third key in the set and discard the rest
	 *
	 * @param	RotationTracks	Array of rotation track elements to reduce
	 * @param	StartIndex		Index at which to begin reduction
	 * @param	Interval		Interval of keys to retain
	 */
	static void FilterIntermittentRotationKeys(
		TArray<struct FRotationTrack>& RotationTracks,
		int32 StartIndex,
		int32 Interval);

	/**
	 * Common compression utility to retain only intermittent rotation keys. For example,
	 * calling with an Interval of 3 would keep every third key in the set and discard the rest
	 *
	 * @param	Track			Track to reduce
	 * @param	StartIndex		Index at which to begin reduction
	 * @param	Interval		Interval of keys to retain
	 */
	static void FilterIntermittentRotationKeys(
		struct FRotationTrack& Track,
		int32 StartIndex,
		int32 Interval);

	/**
	 * Common compression utility to retain only intermittent animation keys. For example,
	 * calling with an Interval of 3 would keep every third key in the set and discard the rest
	 *
	 * @param	PositionTracks	Array of position track elements to reduce
	 * @param	RotationTracks	Array of rotation track elements to reduce
	 * @param	StartIndex		Index at which to begin reduction
	 * @param	Interval		Interval of keys to retain
	 */
	static void FilterIntermittentKeys(
		TArray<struct FTranslationTrack>& PositionTracks,
		TArray<struct FRotationTrack>& RotationTracks,
		int32 StartIndex,
		int32 Interval);

	/**
	 * Common compression utility to populate individual rotation and translation track
	 * arrays from a set of raw animation tracks. Used as a precurser to animation compression.
	 *
	 * @param	RawAnimData			Array of raw animation tracks
	 * @param	SequenceLength		The duration of the animation in seconds
	 * @param	OutTranslationData	Translation tracks to fill
	 * @param	OutRotationData		Rotation tracks to fill
	 * @param	OutScaleData		Scale tracks to fill
	 */
	static void SeparateRawDataIntoTracks(
		const TArray<struct FRawAnimSequenceTrack>& RawAnimData,
		float SequenceLength,
		TArray<struct FTranslationTrack>& OutTranslationData,
		TArray<struct FRotationTrack>& OutRotationData, 
		TArray<struct FScaleTrack>& OutScaleData);

	/**
	 * Common compression utility to walk an array of rotation tracks and enforce
	 * that all adjacent rotation keys are represented by shortest-arc quaternion pairs.
	 *
	 * @param	RotationData	Array of rotation track elements to reduce.
	 */
	static void PrecalculateShortestQuaternionRoutes(TArray<struct FRotationTrack>& RotationData);

public:

	/**
	 * Encodes individual key arrays into an AnimSequence using the desired bit packing formats.
	 *
	 * @param	Seq							Pointer to an Animation Sequence which will contain the bit-packed data .
	 * @param	TargetTranslationFormat		The format to use when encoding translation keys.
	 * @param	TargetRotationFormat		The format to use when encoding rotation keys.
	 * @param	TargetScaleFormat			The format to use when encoding scale keys.	 
	 * @param	TranslationData				Translation Tracks to bit-pack into the Animation Sequence.
	 * @param	RotationData				Rotation Tracks to bit-pack into the Animation Sequence.
	 * @param	ScaleData					Scale Tracks to bit-pack into the Animation Sequence.	 
	 * @param	IncludeKeyTable				true if the compressed data should also contain a table of frame indices for each key. (required by some codecs)
	 */
	static void BitwiseCompressAnimationTracks(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		AnimationCompressionFormat TargetTranslationFormat, 
		AnimationCompressionFormat TargetRotationFormat,
		AnimationCompressionFormat TargetScaleFormat,
		const TArray<FTranslationTrack>& TranslationData,
		const TArray<FRotationTrack>& RotationData,
		const TArray<FScaleTrack>& ScaleData,
		bool IncludeKeyTable = false);

#if WITH_EDITOR
protected:
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
#endif // WITH_EDITOR

	/**
	 * Utility function to append data to a byte stream.
	 *
	 * @param	ByteStream					Byte stream to append to
	 * @param	Src							Pointer to the source data to append
	 * @param	Len							Length in bytes of the source data to append
	 */
	static void UnalignedWriteToStream(TArray<uint8>& ByteStream, const void* Src, SIZE_T Len);

	/**
	* Utility function to write data to a byte stream.
	*
	* @param	ByteStream					Byte stream to write to
	* @param	StreamOffset				Offset in stream to start writing to
	* @param	Src							Pointer to the source data to write
	* @param	Len							Length in bytes of the source data to write
	*/
	static void UnalignedWriteToStream(TArray<uint8>& ByteStream, int32& StreamOffset, const void* Src, SIZE_T Len);

	/**
	 * Utility function to append a packed FVector to a byte stream.
	 *
	 * @param	ByteStream					Byte stream to append to
	 * @param	Format						Compression format to pack with
	 * @param	Vec							The FVector to pack
	 * @param	Mins						The range minimum of the input value to pack for range normalization
	 * @param	Ranges						The range extent of the input value to pack for range normalization
	 */
	static void PackVectorToStream(
		TArray<uint8>& ByteStream,
		AnimationCompressionFormat Format,
		const FVector3f& Vec,
		const float* Mins,
		const float* Ranges);

	/**
	* Utility function to append a packed FQuat to a byte stream.
	*
	* @param	ByteStream					Byte stream to append to
	* @param	Format						Compression format to pack with
	* @param	Vec							The FQuat to pack
	* @param	Mins						The range minimum of the input value to pack for range normalization
	* @param	Ranges						The range extent of the input value to pack for range normalization
	*/
	static void PackQuaternionToStream(
		TArray<uint8>& ByteStream,
		AnimationCompressionFormat Format,
		const FQuat4f& Quat,
		const float* Mins,
		const float* Ranges);

	/**
	 * Pads a byte stream to force a particular alignment for the data to follow.
	 *
	 * @param	ByteStream					Byte stream to append to
	 * @param	Alignment					Required alignment
	 * @param	Sentinel					If we need to add padding to meet the requested alignment, this is the padding value used
	 */
	static void PadByteStream(TArray<uint8>& ByteStream, const int32 Alignment, uint8 Sentinel);

	/**
	 * Default animation padding value.
	 */
	static constexpr uint8 AnimationPadSentinel = 85; //(1<<1)+(1<<3)+(1<<5)+(1<<7)
};



