// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Bitwise animation compression only; performs no key reduction.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCompress.h"
#include "AnimCompress_BitwiseCompressOnly.generated.h"

UCLASS(MinimalAPI)
class UAnimCompress_BitwiseCompressOnly : public UAnimCompress
{
	GENERATED_UCLASS_BODY()


protected:
	//~ Begin UAnimCompress Interface
#if WITH_EDITOR
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
#endif // WITH_EDITOR
	//~ Begin UAnimCompress Interface
};



