// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Reverts any animation compression, restoring the animation to the raw data.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCompress.h"
#include "Animation/AnimCompress_BitwiseCompressOnly.h"
#include "AnimCompress_LeastDestructive.generated.h"

UCLASS()
class UAnimCompress_LeastDestructive : public UAnimCompress_BitwiseCompressOnly
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	// Least destructive is high fidelity
	virtual bool IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const override { return true; }
#endif
};
