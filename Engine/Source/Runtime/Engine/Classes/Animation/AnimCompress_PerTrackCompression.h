// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Keyframe reduction algorithm that removes keys which are linear interpolations of surrounding keys, as
 * well as choosing the best bitwise compression for each track independently.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimSequence.h"
#include "AnimationUtils.h"
#include "Animation/AnimCompress_RemoveLinearKeys.h"
#include "AnimCompress_PerTrackCompression.generated.h"

UCLASS(hidecategories=AnimCompress)
class UAnimCompress_PerTrackCompression : public UAnimCompress_RemoveLinearKeys
{
	GENERATED_UCLASS_BODY()

	/** Maximum threshold to use when replacing a component with zero. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	float MaxZeroingThreshold;

	/** Maximum position difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	float MaxPosDiffBitwise;

	/** Maximum angle difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	float MaxAngleDiffBitwise;

	/** Maximum position difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	float MaxScaleDiffBitwise;

	/** Which encoding formats is the per-track compressor allowed to try on rotation keys */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	TArray<TEnumAsByte<enum AnimationCompressionFormat> > AllowedRotationFormats;

	/** Which encoding formats is the per-track compressor allowed to try on translation keys */
	UPROPERTY(EditAnywhere, Category=PerTrack)
	TArray<TEnumAsByte<enum AnimationCompressionFormat> > AllowedTranslationFormats;

	/** Which encoding formats is the per-track compressor allowed to try on scale keys */
	UPROPERTY(EditAnywhere, Category=PerTrack, meta = (InvalidEnumValues = "ACF_Fixed32NoW,ACF_Float32NoW"))
	TArray<TEnumAsByte<enum AnimationCompressionFormat> > AllowedScaleFormats;

	/** If true, resample the animation to ResampleFramerate frames per second */
	UPROPERTY(EditAnywhere, Category=Resampling)
	uint32 bResampleAnimation:1;

	/** When bResampleAnimation is true, this defines the desired framerate */
	UPROPERTY(EditAnywhere, Category=Resampling, meta=(ClampMin = "1.0", ClampMax = "30.0", editcondition = "bResampleAnimation"))
	float ResampledFramerate;

	/** Animations with fewer keys than MinKeysForResampling will not be resampled. */
	UPROPERTY(EditAnywhere, Category=Resampling, meta=(editcondition = "bResampleAnimation"))
	int32 MinKeysForResampling;

	/** If true, adjust the error thresholds based on the 'height' within the skeleton */
	UPROPERTY(EditAnywhere, Category=AdaptiveError)
	uint32 bUseAdaptiveError:1;

	/** If true, uses MinEffectorDiff as the threshold for end effectors */
	UPROPERTY(EditAnywhere, Category=AdaptiveError, meta=(editcondition = "bUseAdaptiveError"))
	uint32 bUseOverrideForEndEffectors:1;

	/** A bias added to the track height before using it to calculate the adaptive error */
	UPROPERTY(EditAnywhere, Category=AdaptiveError, meta=(editcondition = "bUseAdaptiveError"))
	int32 TrackHeightBias;

	/**
	 * Reduces the error tolerance the further up the tree that a key occurs
	 * EffectiveErrorTolerance = Max(BaseErrorTolerance / Power(ParentingDivisor, Max(Height+Bias,0) * ParentingDivisorExponent), ZeroingThreshold)
	 * Only has an effect bUseAdaptiveError is true
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError, meta=(ClampMin = "1.0", editcondition = "bUseAdaptiveError"))
	float ParentingDivisor;

	/**
	 * Reduces the error tolerance the further up the tree that a key occurs
	 * EffectiveErrorTolerance = Max(BaseErrorTolerance / Power(ParentingDivisor, Max(Height+Bias,0) * ParentingDivisorExponent), ZeroingThreshold)
	 * Only has an effect bUseAdaptiveError is true
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError, meta=(ClampMin = "0.1", editcondition = "bUseAdaptiveError"))
	float ParentingDivisorExponent;

	/**
	 * If true, the adaptive error system will determine how much error to allow for each track, based on the
	 * error introduced in end effectors due to errors in the track.
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError2)
	uint32 bUseAdaptiveError2:1;

	/**
	 * This ratio determines how much error in end effector rotation can come from a given track's rotation error or translation error.
	 * If 1, all of it must come from rotation error, if 0.5, half can come from each, and if 0.0, all must come from translation error.
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError2, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bUseAdaptiveError2"))
	float RotationErrorSourceRatio;

	/**
	 * This ratio determines how much error in end effector translation can come from a given track's rotation error or translation error.
	 * If 1, all of it must come from rotation error, if 0.5, half can come from each, and if 0.0, all must come from translation error.
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError2, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bUseAdaptiveError2"))
	float TranslationErrorSourceRatio;

	/**
	 * This ratio determines how much error in end effector scale can come from a given track's rotation error or scale error.
	 * If 1, all of it must come from rotation error, if 0.5, half can come from each, and if 0.0, all must come from scale error.
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError2, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bUseAdaptiveError2"))
	float ScaleErrorSourceRatio;

	/**
	 * A fraction that determines how much of the total error budget can be introduced by any particular track
	 */
	UPROPERTY(EditAnywhere, Category=AdaptiveError2, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bUseAdaptiveError2"))
	float MaxErrorPerTrackRatio;

	/**
	 * How big of a perturbation should be made when probing error propagation
	 */
	UPROPERTY()
	float PerturbationProbeSize;


public:
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const override;

protected:
	//~ Begin UAnimCompress Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
#endif // WITH_EDITOR
	//~ Begin UAnimCompress Interface

#if WITH_EDITOR
	//~ Begin UAnimCompress_RemoveLinearKeys Interface
	virtual void CompressUsingUnderlyingCompressor(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		const TArray<FTranslationTrack>& TranslationData,
		const TArray<FRotationTrack>& RotationData,
		const TArray<FScaleTrack>& ScaleData,
		const bool bFinalPass) override;

	virtual void* FilterBeforeMainKeyRemoval(
		const FCompressibleAnimData& CompressibleAnimData,
		TArray<FTranslationTrack>& TranslationData,
		TArray<FRotationTrack>& RotationData,
		TArray<FScaleTrack>& ScaleData) override;
	//~ End UAnimCompress_RemoveLinearKeys Interface
#endif // WITH_EDITOR
};



