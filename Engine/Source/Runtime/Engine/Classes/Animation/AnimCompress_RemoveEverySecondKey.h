// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Keyframe reduction algorithm that simply removes every second key.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCompress.h"
#include "AnimCompress_RemoveEverySecondKey.generated.h"

UCLASS(MinimalAPI)
class UAnimCompress_RemoveEverySecondKey : public UAnimCompress
{
	GENERATED_UCLASS_BODY()

	/** Animations with fewer than MinKeys will not lose any keys. */
	UPROPERTY(EditAnywhere, Category=AnimationCompressionAlgorithm_RemoveEverySecondKey, meta=(UIMin=1, ClampMin=1))
	int32 MinKeys;

	/**
	 * If bStartAtSecondKey is true, remove keys 1,3,5,etc.
	 * If bStartAtSecondKey is false, remove keys 0,2,4,etc.
	 */
	UPROPERTY(EditAnywhere, Category=AnimationCompressionAlgorithm_RemoveEverySecondKey)
	uint32 bStartAtSecondKey:1;


protected:
	//~ Begin UAnimCompress Interface
#if WITH_EDITOR
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
#endif // WITH_EDITOR
	//~ Begin UAnimCompress Interface
};



