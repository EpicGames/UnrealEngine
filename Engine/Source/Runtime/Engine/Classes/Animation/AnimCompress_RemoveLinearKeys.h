// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Keyframe reduction algorithm that simply removes keys which are linear interpolations of surrounding keys.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimSequence.h"
#include "AnimationUtils.h"
#include "Animation/AnimCompress.h"
#include "AnimCompress_RemoveLinearKeys.generated.h"


UCLASS(MinimalAPI)
class UAnimCompress_RemoveLinearKeys : public UAnimCompress
{
	GENERATED_UCLASS_BODY()

	/** Maximum position difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float MaxPosDiff;

	/** Maximum angle difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float MaxAngleDiff;

	/** Maximum Scale difference to use when testing if an animation key may be removed. Lower values retain more keys, but yield less compression. */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float MaxScaleDiff;

	/** 
	 * As keys are tested for removal, we monitor the effects all the way down to the end effectors. 
	 * If their position changes by more than this amount as a result of removing a key, the key will be retained.
	 * This value is used for all bones except the end-effectors parent.
	 */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float MaxEffectorDiff;

	/** 
	 * As keys are tested for removal, we monitor the effects all the way down to the end effectors. 
	 * If their position changes by more than this amount as a result of removing a key, the key will be retained.
	 * This value is used for the end-effectors parent, allowing tighter restrictions near the end of a skeletal chain.
	 */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float MinEffectorDiff;

	/** 
	 * Error threshold for End Effectors with Sockets attached to them.
	 * Typically more important bone, where we want to be less aggressive with compression.
	 */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	float EffectorDiffSocket;

	/** 
	 * A scale value which increases the likelihood that a bone will retain a key if it's parent also had a key at the same time position. 
	 * Higher values can remove shaking artifacts from the animation, at the cost of compression.
	 */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval, meta = (ClampMin = "1.0"))
	float ParentKeyScale;

	/** 
	 * true = As the animation is compressed, adjust animated nodes to compensate for compression error.
	 * false= Do not adjust animated nodes.
	 */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	uint32 bRetarget:1;

	/**
	  * Controls whether the final filtering step will occur, or only the retargetting after bitwise compression.
	  * If both this and bRetarget are false, then the linear compressor will do no better than the underlying bitwise compressor, extremely slowly.
	  */
	UPROPERTY(EditAnywhere, Category=LinearKeyRemoval)
	uint32 bActuallyFilterLinearKeys:1;

	//~ Begin UAnimBoneCompressionCodec Interface
#if WITH_EDITORONLY_DATA
	virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const override;
#endif // WITH_EDITORONLY_DATA
	//~ End UAnimBoneCompressionCodec Interface

protected:
	//~ Begin UAnimCompress Interface
#if WITH_EDITOR
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
#endif // WITH_EDITOR
	//~ Begin UAnimCompress Interface

#if WITH_EDITOR
	/**
	 * Pre-filters the tracks before running the main key removal algorithm
	 */
	virtual void* FilterBeforeMainKeyRemoval(
		const FCompressibleAnimData& CompressibleAnimData,
		TArray<FTranslationTrack>& TranslationData,
		TArray<FRotationTrack>& RotationData, 
		TArray<FScaleTrack>& ScaleData);

	/**
	 * Compresses the tracks passed in using the underlying compressor for this key removal codec
	 */
	virtual void CompressUsingUnderlyingCompressor(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		const TArray<FTranslationTrack>& TranslationData,
		const TArray<FRotationTrack>& RotationData,
		const TArray<FScaleTrack>& ScaleData,
		const bool bFinalPass);

	/**
	 * Updates the world bone transforms for a range of bone indices
	 */
	void UpdateWorldBoneTransformRange(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		const TArray<FTransform>& RefPose,
		const TArray<FTranslationTrack>& PositionTracks,
		const TArray<FRotationTrack>& RotationTracks,
		const TArray<FScaleTrack>& ScaleTracks,
		int32 StartingBoneIndex,
		int32 EndingBoneIndex,
		bool UseRaw,
		TArray<FTransform>& OutputWorldBones);

	/**
	 * To guide the key removal process, we need to maintain a table of world transforms
	 * for the bones we are investigating. This helper function fills a row of the 
	 * table for a specified bone.
	 */
	void UpdateWorldBoneTransformTable(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		const TArray<FTransform>& RefPose,
		int32 BoneIndex,
		bool UseRaw,
		TArray<FTransform>& OutputWorldBones);

	/**
	 * Creates a list of the bone atom result for every frame of a given track
	 */
	static void UpdateBoneAtomList(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		int32 BoneIndex,
		int32 TrackIndex,
		int32 NumFrames,
		float TimePerFrame,
		TArray<FTransform>& BoneAtoms);

	/**
	 * If the passed in animation sequence is additive, converts it to absolute (using the frame 0 pose) and returns true
	 * (indicating it should be converted back to relative later with ConvertToRelativeSpace)
	 *
	 * @param AnimSeq			The animation sequence being compressed
	 *
	 */
	void ConvertFromRelativeSpace(FCompressibleAnimData& CompressibleAnimData);

	/**
	 * Converts an absolute animation sequence and matching track data to a relative (additive) one.
	 *
	 * @param AnimSeq			The animation sequence being compressed
	 * @param TranslationData	Translation Tracks to convert to relative space
	 * @param RotationData		Rotation Tracks to convert to relative space
	 * @param ScaleData			Scale Tracks to convert to relative space
	 *
	 */
	void ConvertToRelativeSpaceBoth(FCompressibleAnimData& CompressibleAnimData, TArray<FTranslationTrack>& TranslationData, TArray<FRotationTrack>& RotationData, TArray<FScaleTrack>& ScaleData);

	/**
	 * Converts an absolute animation sequence to a relative (additive) one.
	 *
	 * @param AnimSeq			The animation sequence being compressed and converted
	 *
	 */
	void ConvertToRelativeSpace(FCompressibleAnimData& CompressibleAnimData) const;

	/**
	 * Converts track data to relative (additive) space.
	 *
	 * @param AnimSeq			The animation sequence being compressed
	 * @param TranslationData	Translation Tracks to convert to relative space
	 * @param RotationData		Rotation Tracks to convert to relative space
	 * @param ScaleData			Scale Tracks to convert to relative space
	 *
	 */
	void ConvertToRelativeSpace(const FCompressibleAnimData& CompressibleAnimData, TArray<FTranslationTrack>& TranslationData, TArray<FRotationTrack>& RotationData, TArray<FScaleTrack>& ScaleData) const;


	/**
	 * Locates spans of keys within the position and rotation tracks provided which can be estimated
	 * through linear interpolation of the surrounding keys. The remaining key values are bit packed into
	 * the animation sequence provided
	 *
	 * @param	AnimSeq		The animation sequence being compressed
	 * @param	BoneData	BoneData array describing the hierarchy of the animated skeleton
	 * @param	PositionTracks		Translation Tracks to compress and bit-pack into the Animation Sequence.
	 * @param	RotationTracks		Rotation Tracks to compress and bit-pack into the Animation Sequence.
	 * @param	ScaleTracks			Scale Tracks to compress and bit-pack into the Animation Sequence.	 
	 * @return				None.
	 */
	void ProcessAnimationTracks(
		const FCompressibleAnimData& CompressibleAnimData,
		FCompressibleAnimDataResult& OutCompressedData,
		TArray<FTranslationTrack>& PositionTracks,
		TArray<FRotationTrack>& RotationTracks, 
		TArray<FScaleTrack>& ScaleTracks);

#endif // WITH_EDITOR
};



