// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Removes trivial frames -- frames of tracks when position or orientation is constant
 * over the entire animation -- from the raw animation data.  If both position and rotation
 * go down to a single frame, the time is stripped out as well.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCompress.h"
#include "AnimCompress_RemoveTrivialKeys.generated.h"

UCLASS(MinimalAPI)
class UAnimCompress_RemoveTrivialKeys : public UAnimCompress
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=AnimationCompressionAlgorithm_RemoveTrivialKeys)
	float MaxPosDiff;

	UPROPERTY(EditAnywhere, Category=AnimationCompressionAlgorithm_RemoveTrivialKeys)
	float MaxAngleDiff;

	UPROPERTY(EditAnywhere, Category=AnimationCompressionAlgorithm_RemoveTrivialKeys)
	float MaxScaleDiff;

protected:
	//~ Begin UAnimCompress Interface
#if WITH_EDITOR
	virtual bool DoReduction(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
#endif // WITH_EDITOR
	//~ Begin UAnimCompress Interface
};



