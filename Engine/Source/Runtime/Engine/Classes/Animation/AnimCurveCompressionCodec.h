// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimCompressionTypes.h"
#include "AnimCurveCompressionCodec.generated.h"

class UAnimCurveCompressionCodec;
class UAnimSequence;
struct FBlendedCurve;

#if WITH_EDITORONLY_DATA
/** Holds the result from animation curve compression */
struct FAnimCurveCompressionResult
{
	/** The animation curves as raw compressed bytes */
	TArray<uint8> CompressedBytes;

	/** The codec used by the compressed bytes */
	UAnimCurveCompressionCodec* Codec;

	/** Default constructor */
	FAnimCurveCompressionResult() : CompressedBytes(), Codec(nullptr) {}
};
#endif

/*
 * Base class for all curve compression codecs.
 */
UCLASS(abstract, hidecategories = Object, EditInlineNew, MinimalAPI)
class UAnimCurveCompressionCodec : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Allow us to convert DDC serialized path back into codec object */
	virtual UAnimCurveCompressionCodec* GetCodec(const FString& Path) { return this; }

	//////////////////////////////////////////////////////////////////////////

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/*
	 * Called on load and when cooking to validate that the compressed data is good.
	 * Codecs should perform necessary validation and emit an error when appropriate.
	 * e.g: UE_LOGF(LogAnimationCompression, Error, "Bad data!");
	 * 
	 * Returns true when the data is valid, false otherwise.
	 */
	virtual bool ValidateCompressedData(UObject* DataOwner, const FCompressedAnimSequence& AnimSeq) const { return true; }

#if WITH_EDITORONLY_DATA
	/** Returns whether or not we can use this codec to compress. */
	virtual bool IsCodecValid() const { return true; }

	/** Compresses the curve data from an animation sequence. */
	virtual bool Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult) PURE_VIRTUAL(UAnimCurveCompressionCodec::Compress, return false;);

	/**
	 * Estimates the peak memory usage in bytes needed to compress the given data with this codec.
	 * This is used to make informed scheduling decisions during asset cooking. Estimates that are too low may cause
	 * out-of-memory conditions. Estimates that are too high can unnecessarily limit the number of concurrent cook
	 * processes.
	 *
	 * Values below zero indicate that no estimate has been given and high memory usage is hence assumed.
	 *
	 * @param AnimSequence	The Animation Sequence to estimate the memory usage for.
	 * @returns Estimated peak memory usage in bytes. Values < 0 indicate maximum memory usage.
	 */
	ENGINE_API virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const;

	/*
	 * Called to generate a unique DDC key for this codec instance.
	 * A suitable key should be generated from: the InstanceGuid, a codec version, and all relevant properties that drive the behavior.
	 */
	ENGINE_API virtual void PopulateDDCKey(FArchive& Ar);
#endif

	/*
	 * Decompresses all the active blended curves.
	 * Note: Codecs should _NOT_ rely on any member properties during decompression. Decompression
	 * behavior should entirely be driven by code and the compressed data.
	 */
	virtual void DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const PURE_VIRTUAL(UAnimCurveCompressionCodec::DecompressCurves, );

	/*
	 * Decompress a single curve.
	 * Note: Codecs should _NOT_ rely on any member properties during decompression. Decompression
	 * behavior should entirely be driven by code and the compressed data.
	 */
	virtual float DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName CurveName, float CurrentTime) const PURE_VIRTUAL(UAnimCurveCompressionCodec::DecompressCurve, return 0.0f;);
};
