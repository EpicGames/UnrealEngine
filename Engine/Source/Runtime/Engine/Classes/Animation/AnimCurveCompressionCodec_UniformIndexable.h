// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Stores the raw rich curves as FCompressedRichCurve internally with optional key reduction and key time quantization.
*/

#include "Animation/Skeleton.h"
#include "CoreMinimal.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "AnimCurveCompressionCodec_UniformIndexable.generated.h"

struct FAnimCurveBufferAccess
{
private:
	int32 NumSamples;
	float SampleRate;

	const float* CompressedBuffer;

	const FFloatCurve* RawCurve;
	bool bUseCompressedData;
public:

	ENGINE_API FAnimCurveBufferAccess(const UAnimSequenceBase* InSequence, FName InName);

	bool IsValid() const
	{
		return CompressedBuffer || RawCurve;
	}

	int32 GetNumSamples() const { return NumSamples; }

	ENGINE_API float GetValue(int32 SampleIndex) const;

	ENGINE_API float GetTime(int32 SampleIndex) const;
};

UCLASS(meta = (DisplayName = "Uniform Indexable"), MinimalAPI)
class UAnimCurveCompressionCodec_UniformIndexable : public UAnimCurveCompressionCodec
{
	GENERATED_UCLASS_BODY()

	//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
	// UAnimCurveCompressionCodec overrides
	ENGINE_API virtual bool Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult) override;
	ENGINE_API virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const override;
	ENGINE_API virtual void PopulateDDCKey(FArchive& Ar) override;
#endif
	
	ENGINE_API virtual void DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const override;
	ENGINE_API virtual float DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName CurveName, float CurrentTime) const override;

	ENGINE_API bool GetCurveBufferAndSamples(const FCompressedAnimSequence& AnimSeq, FName CurveName, const float*& OutCurveBuffer, int32& OutSamples, float& OutSampleRate);
};
