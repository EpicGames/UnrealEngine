// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Stores the raw rich curves as FCompressedRichCurve internally with optional key reduction and key time quantization.
*/

#include "CoreMinimal.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "AnimCurveCompressionCodec_UniformlySampled.generated.h"

UCLASS(meta = (DisplayName = "Uniformly Sampled"), MinimalAPI)
class UAnimCurveCompressionCodec_UniformlySampled : public UAnimCurveCompressionCodec
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** Whether to use the animation sequence sample rate or an explicit value */
	UPROPERTY(Category = Compression, EditAnywhere)
	bool UseAnimSequenceSampleRate;

	/** Sample rate to use when uniformly sampling */
	UPROPERTY(Category = Compression, EditAnywhere, meta = (ClampMin = "0", EditCondition = "!UseAnimSequenceSampleRate"))
	float SampleRate;
#endif

	//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
	// UAnimCurveCompressionCodec overrides
	ENGINE_API virtual bool Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult) override;
	ENGINE_API virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const override;
	ENGINE_API virtual void PopulateDDCKey(FArchive& Ar) override;
#endif
	
	ENGINE_API virtual void DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const override;
	ENGINE_API virtual float DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName CurveName, float CurrentTime) const override;
};
