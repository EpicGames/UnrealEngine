// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimCompressionTypes.h"
#include "AnimCurveCompressionSettings.generated.h"

class UAnimCurveCompressionCodec;
class UAnimSequence;

/*
 * This object is used to wrap a curve compression codec. It allows a clean integration in the editor by avoiding the need
 * to create asset types and factory wrappers for every codec.
 */
UCLASS(hidecategories = Object, MinimalAPI)
class UAnimCurveCompressionSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** An animation curve compression codec. */
	UPROPERTY(Category = Compression, Export, EditAnywhere, NoClear, meta = (EditInline))
	TObjectPtr<UAnimCurveCompressionCodec> Codec;

	//////////////////////////////////////////////////////////////////////////

	/** Allow us to convert DDC serialized path back into codec object */
	ENGINE_API UAnimCurveCompressionCodec* GetCodec(const FString& Path);

#if WITH_EDITORONLY_DATA
	// UObject overrides
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	/** Returns whether or not we can use these settings to compress. */
	ENGINE_API bool AreSettingsValid() const;

	/*
	 * Compresses the animation curves inside the supplied sequence data.
	 * The resultant compressed data is applied to the OutCompressedData structure.
	 */
	ENGINE_API bool Compress(const FCompressibleAnimData& AnimSeq, FCompressedAnimSequence& OutCompressedData) const;

	/** Generates a DDC key that takes into account the current settings and selected codec. */
	ENGINE_API void PopulateDDCKey(FArchive& Ar);
#endif
};
