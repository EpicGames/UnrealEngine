// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "IAnimationDataModel.h"

#include "AnimDataModel.generated.h"

/**
 * The Model represents the source data for animations. It contains both bone animation data as well as animated curves.
 * They are currently only a sub-object of a AnimSequenceBase instance. The instance derives all runtime data from the source data. 
 */
UCLASS(BlueprintType, meta=(DebugTreeLeaf), MinimalAPI)
class UAnimDataModel : public UObject, public IAnimationDataModel
{
	GENERATED_BODY()
public:
	/** Begin UObject overrides */
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual bool IsEditorOnly() const override { return true; }
	/** End UObject overrides */
	
	/** Begin IAnimationDataModel overrides */
	ENGINE_API virtual double GetPlayLength() const override;
	ENGINE_API virtual int32 GetNumberOfFrames() const override;
	ENGINE_API virtual int32 GetNumberOfKeys() const override;
	ENGINE_API virtual FFrameRate GetFrameRate() const override;
	
	ENGINE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS	
	virtual const TArray<FBoneAnimationTrack>& GetBoneAnimationTracks() const override;
	ENGINE_API virtual const FBoneAnimationTrack& GetBoneTrackByIndex(int32 TrackIndex) const override;
	ENGINE_API virtual const FBoneAnimationTrack& GetBoneTrackByName(FName TrackName) const override;
	ENGINE_API virtual int32 GetBoneTrackIndex(const FBoneAnimationTrack& Track) const override;
	ENGINE_API virtual int32 GetBoneTrackIndexByName(FName TrackName) const override;
	ENGINE_API virtual bool IsValidBoneTrackIndex(int32 TrackIndex) const override;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	virtual int32 GetNumBoneTracks() const override;
	ENGINE_API virtual void GetBoneTrackNames(TArray<FName>& OutNames) const override;
	ENGINE_API virtual FTransform GetBoneTrackTransform(FName TrackName, const FFrameNumber& FrameNumber) const override;
	ENGINE_API virtual void GetBoneTrackTransforms(FName TrackName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& OutTransforms) const override;	
	ENGINE_API virtual void GetBoneTrackTransforms(FName TrackName, TArray<FTransform>& OutTransforms) const override;
	ENGINE_API virtual void GetBoneTracksTransform(const TArray<FName>& TrackNames, const FFrameNumber& FrameNumber, TArray<FTransform>& OutTransforms) const override;	
	ENGINE_API virtual FTransform EvaluateBoneTrackTransform(FName TrackName, const FFrameTime& FrameTime, const EAnimInterpolationType& Interpolation) const override;
	ENGINE_API virtual bool IsValidBoneTrackName(const FName& TrackName) const override;	
	ENGINE_API virtual const FAnimCurveBase* FindCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FFloatCurve* FindFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FTransformCurve* FindTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FRichCurve* FindRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FAnimationCurveData& GetCurveData() const override;
	ENGINE_API virtual int32 GetNumberOfTransformCurves() const override;
	ENGINE_API virtual int32 GetNumberOfFloatCurves() const override;
	ENGINE_API virtual const TArray<struct FFloatCurve>& GetFloatCurves() const override;
	ENGINE_API virtual const TArray<struct FTransformCurve>& GetTransformCurves() const override;	
	ENGINE_API virtual const FAnimCurveBase& GetCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FFloatCurve& GetFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FTransformCurve& GetTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual const FRichCurve& GetRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	ENGINE_API virtual TArrayView<const FAnimatedBoneAttribute> GetAttributes() const override;
	ENGINE_API virtual int32 GetNumberOfAttributes() const override;
	ENGINE_API virtual int32 GetNumberOfAttributesForBoneIndex(const int32 BoneIndex) const override;
	ENGINE_API virtual void GetAttributesForBone(const FName& BoneName, TArray<const FAnimatedBoneAttribute*>& OutBoneAttributes) const override;
	ENGINE_API virtual const FAnimatedBoneAttribute& GetAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const override;
	ENGINE_API virtual const FAnimatedBoneAttribute* FindAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const override;
	ENGINE_API virtual UAnimSequence* GetAnimationSequence() const override;
	virtual FAnimDataModelModifiedEvent& GetModifiedEvent() override { return ModifiedEvent; }
	ENGINE_API virtual FGuid GenerateGuid(const FGuidGenerationSettings& InSettings) const override;
#if WITH_EDITOR
	ENGINE_API virtual FString GenerateDebugStateString() const override;
	ENGINE_API virtual void Evaluate(FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const override;
#endif
	ENGINE_API virtual TScriptInterface<IAnimationDataController> GetController() override;
	virtual bool HasBeenPopulated() const override { return bPopulated; }
	ENGINE_API virtual void IterateBoneKeys(const FName& BoneName, TFunction<bool(const FVector3f& Pos, const FQuat4f&, const FVector3f, const FFrameNumber&)> IterationFunction) const override;
protected:
	ENGINE_API virtual IAnimationDataModel::FModelNotifier& GetNotifier() override;
	virtual FAnimDataModelModifiedDynamicEvent& GetModifiedDynamicEvent() override { return ModifiedEventDynamic; }
	virtual void OnNotify(const EAnimDataModelNotifyType& NotifyType, const FAnimDataModelNotifPayload& Payload) override {}
	virtual void LockEvaluationAndModification() const override final {}
	virtual bool TryLockEvaluationAndModification() const override final { return true; }
	virtual void UnlockEvaluationAndModification() const override final {}	
	virtual bool& GetPopulationFlag() override final
	{
		return bPopulated;
	}
	/** End IAnimationDataModel overrides */

private:
	/** Helper functionality used by UAnimDataController to retrieve mutable data */ 
	ENGINE_API FBoneAnimationTrack* FindMutableBoneTrackByName(FName Name);
	ENGINE_API FBoneAnimationTrack& GetMutableBoneTrackByName(FName Name);
	ENGINE_API FTransformCurve* FindMutableTransformCurveById(const FAnimationCurveIdentifier& CurveIdentifier);
	ENGINE_API FFloatCurve* FindMutableFloatCurveById(const FAnimationCurveIdentifier& CurveIdentifier);
	ENGINE_API FAnimCurveBase* FindMutableCurveById(const FAnimationCurveIdentifier& CurveIdentifier);	   
	ENGINE_API FRichCurve* GetMutableRichCurve(const FAnimationCurveIdentifier& CurveIdentifier);

	ENGINE_API USkeleton* GetSkeleton() const;
	template <typename HasherType>
	void GenerateStateHash(HasherType& Hasher, const FGuidGenerationSettings& InSettings) const;

private:
	UPROPERTY(Transient)
	int32 BracketCounter = 0;

	/** Dynamic delegate event allows scripting to register to any broadcasted notify. */
	UPROPERTY(BlueprintAssignable, Transient, Category = AnimationDataModel, meta = (ScriptName = "ModifiedEvent", AllowPrivateAccess = "true"))
	FAnimDataModelModifiedDynamicEvent ModifiedEventDynamic;
	
	/** Native delegate event allows for registerings to any broadcasted notify. */
	FAnimDataModelModifiedEvent ModifiedEvent;

	/** All individual bone animation tracks */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Animation Data Model", meta = (AllowPrivateAccess = "true"))
	TArray<FBoneAnimationTrack> BoneAnimationTracks;

	/** Total playable length of the contained animation data */
	UE_DEPRECATED(5.1, "PlayLength is deprecated use GetPlayLength instead, as it is now calculated with Number of Frames * FrameRate instead of stored as a value")
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	float PlayLength;
	
	/** Rate at which the animated data is sampled */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	FFrameRate FrameRate;

	/** Total number of sampled animated frames */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	int32 NumberOfFrames;

	/** Total number of sampled animated keys */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	int32 NumberOfKeys;
	
	/** Container with all animated curve data */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	FAnimationCurveData CurveData;
	
	/** Container with all animated (bone) attribute data */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Data Model", meta = (AllowPrivateAccess = "true"))
	TArray<FAnimatedBoneAttribute> AnimatedBoneAttributes;

	UPROPERTY()
	bool bPopulated = false;

	TUniquePtr<IAnimationDataModel::FModelNotifier> Notifier;

	friend class UAnimDataController;
	friend class FAnimDataControllerTestBase;
};

