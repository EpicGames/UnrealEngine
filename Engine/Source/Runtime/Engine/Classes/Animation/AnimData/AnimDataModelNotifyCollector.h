// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/AnimDataNotifications.h"
#include "Containers/Set.h"
#include "Containers/Array.h"

enum class EAnimDataModelNotifyType : uint8;

namespace UE {
namespace Anim {

#if WITH_EDITORONLY_DATA

/** Helper structure for keeping track of which notifies of type EAnimDataModelNotifyType are broadcasted
between top-level EAnimDataModelNotifyType::BracketOpened and EAnimDataModelNotifyType::BracketClosed notifies */
struct FAnimDataModelNotifyCollector
{
	FAnimDataModelNotifyCollector() : BracketDepth(0), bDataModified(false) {}

	/** Handle a broadcasted notify, reset if we are opening a new top-level bracket*/
	void Handle(EAnimDataModelNotifyType NotifyType)
	{
		if (BracketDepth == 0)
		{
			Reset();
			bDataModified = false;
		}
		
		NotifyTypes.Add(NotifyType);

		if (NotifyType == EAnimDataModelNotifyType::BracketOpened)
		{
			++BracketDepth;
		}
		else if (NotifyType == EAnimDataModelNotifyType::BracketClosed)
		{
			--BracketDepth;
		}
	}
	
	/** Returns whether or not the notify of the provided types was broadcasted */
	bool Contains(EAnimDataModelNotifyType NotifyType) const
	{
		return NotifyTypes.Find(NotifyType) != nullptr;
	}

	/** Returns whether or not any of the provided notify types were broadcasted */
	bool Contains(const TArray<EAnimDataModelNotifyType>& TestNotifyTypes) const
	{
		for (EAnimDataModelNotifyType Notify : TestNotifyTypes)
		{
			if (NotifyTypes.Find(Notify) != nullptr)
			{
				return true;
			}
		}

		return false;
	}

	/** Returns whether or not a bracket is still open */
	bool IsWithinBracket() const { return BracketDepth > 0; }

	/** Returns whether or not all brackets have been closed */
	bool IsNotWithinBracket() const { return BracketDepth == 0; }

	void MarkDataModified() { bDataModified = true; }
	bool WasDataModified() { return bDataModified; }
protected:
	void Reset()
	{
		NotifyTypes.Empty();
	}
protected:
	TSet<EAnimDataModelNotifyType> NotifyTypes;
	int32 BracketDepth;
	bool bDataModified;
};

#endif // WITH_EDITORONLY_DATA

} // namespace Anim	

} // namespace UE
