// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/FrameRate.h"

#include "Animation/SmartName.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AnimData/AttributeIdentifier.h"

#include "AnimDataNotifications.generated.h"

class IAnimationDataModel;

UENUM(BlueprintType)
enum class EAnimDataModelNotifyType : uint8
{
	/** Indicates a bracket has been opened. Type of payload: FBracketPayload */
	BracketOpened,

	/** Indicates a bracket has been closed. Type of payload: FEmptyPayload */
	BracketClosed,

	/** Indicates a new bone track has been added. Type of payload: FAnimationTrackAddedPayload */
	TrackAdded, 

	/** Indicates the keys of a bone track have been changed. Type of payload: FAnimationTrackChangedPayload */
	TrackChanged,

	/** Indicates a bone track has been removed. Type of payload: FAnimationTrackRemovedPayload */
	TrackRemoved,	

	/** Indicates the play length of the animated data has changed. Type of payload: FSequenceLengthChangedPayload */
	SequenceLengthChanged,

	/** Indicates the sampling rate of the animated data has changed. Type of payload: FFrameRateChangedPayload */
	FrameRateChanged,

	/** Indicates a new curve has been added. Type of payload: FCurveAddedPayload */
	CurveAdded,

	/** Indicates a curve its data has been changed. Type of payload: FCurveChangedPayload */
	CurveChanged,

	/** Indicates a curve has been removed. Type of payload: FCurveRemovedPayload */
	CurveRemoved,

	/** Indicates a curve its flags have changed. Type of payload: FCurveFlagsChangedPayload */
	CurveFlagsChanged,

	/** Indicates a curve has been renamed. Type of payload: FCurveRenamedPayload */
	CurveRenamed,

	/** Indicates a curve has been scaled. Type of payload: FCurveScaledPayload */
	CurveScaled,

	/** Indicates a curve its color has changed. Type of payload: FCurveChangedPayload */
	CurveColorChanged,

	/** Indicates a curve has been removed. Type of payload: FCurveChangedPayload */
	CurveCommentChanged,

	/** Indicates a new attribute has been added. Type of payload: FAttributeAddedPayload */
	AttributeAdded,
	
	/** Indicates a new attribute has been removed. Type of payload: FAttributeRemovedPayload */
	AttributeRemoved,

	/** Indicates an attribute has been changed. Type of payload: FAttributeChangedPayload */
	AttributeChanged,
	
	/** Indicates the data model has been populated from the source UAnimSequence. Type of payload: FEmptyPayload */
	Populated,

	/** Indicates all data stored on the model has been reset. Type of payload: FEmptyPayload */
	Reset,

	/** Indicates that the skeleton changed. Type of payload: FEmptyPayload */
	SkeletonChanged,

	Invalid // The max for this enum (used for guarding)
};

USTRUCT(BlueprintType)
struct FEmptyPayload
{
	GENERATED_BODY()
};

USTRUCT(BlueprintType)
struct FBracketPayload : public FEmptyPayload
{
	GENERATED_BODY()
	
	/** Description of bracket-ed operation applied to the model */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FString Description;
};

USTRUCT(BlueprintType)
struct FAnimationTrackPayload : public FEmptyPayload
{
	GENERATED_BODY()

	/** Name of the track (bone) */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FName Name;
};

typedef FAnimationTrackPayload FAnimationTrackRemovedPayload;
typedef FAnimationTrackPayload FAnimationTrackChangedPayload;

USTRUCT(BlueprintType)
struct FAnimationTrackAddedPayload : public FAnimationTrackPayload
{
	GENERATED_BODY()
	
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimationTrackAddedPayload() = default;
	FAnimationTrackAddedPayload(const FAnimationTrackAddedPayload&) = default;
	FAnimationTrackAddedPayload(FAnimationTrackAddedPayload&&) = default;
	FAnimationTrackAddedPayload& operator=(const FAnimationTrackAddedPayload&) = default;
	FAnimationTrackAddedPayload& operator=(FAnimationTrackAddedPayload&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Index of the track (bone) which was added */
	UE_DEPRECATED(5.2, "FAnimationTrackAddedPayload::TrackIndex has been deprecated")
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	int32 TrackIndex = INDEX_NONE;
};

USTRUCT(BlueprintType)
struct FSequenceLengthChangedPayload : public FEmptyPayload
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSequenceLengthChangedPayload() = default;
	FSequenceLengthChangedPayload(const FSequenceLengthChangedPayload&) = default;
	FSequenceLengthChangedPayload(FSequenceLengthChangedPayload&&) = default;
	FSequenceLengthChangedPayload& operator=(const FSequenceLengthChangedPayload&) = default;
	FSequenceLengthChangedPayload& operator=(FSequenceLengthChangedPayload&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Previous playable length for the Model */
	UE_DEPRECATED(5.0, "Time in seconds is deprecated use FFrameNumber members (PreviousNumberOfFrames) instead")
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	float PreviousLength = 0.f;

	/** Time at which the change in length has been made */
	UE_DEPRECATED(5.0, "Time in seconds is deprecated use FFrameNumber members (Frame0) instead")
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	float T0 = 0.f;

	/** Length of time which is inserted or removed starting at T0 */
	UE_DEPRECATED(5.0, "Time in seconds is deprecated use FFrameNumber members (Frame1) instead")
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	float T1 = 0.f;

	/** Previous playable number of frames for the Model */
    UPROPERTY(BlueprintReadOnly, Category = Payload)
    FFrameNumber PreviousNumberOfFrames;

    /** Frame number at which the change in frames has been made */
    UPROPERTY(BlueprintReadOnly, Category = Payload)
    FFrameNumber Frame0;

    /** Amount of frames which is inserted or removed starting at Frame0 */
    UPROPERTY(BlueprintReadOnly, Category = Payload)
	FFrameNumber Frame1;
};

USTRUCT(BlueprintType)
struct FFrameRateChangedPayload : public FEmptyPayload
{
	GENERATED_BODY()

	/** Previous sampling rate for the Model */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FFrameRate PreviousFrameRate;
};

USTRUCT(BlueprintType)
struct FCurvePayload : public FEmptyPayload
{
	GENERATED_BODY()

	/** Identifier of the curve */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FAnimationCurveIdentifier Identifier;
};

typedef FCurvePayload FCurveAddedPayload;
typedef FCurvePayload FCurveRemovedPayload;
typedef FCurvePayload FCurveChangedPayload;

USTRUCT(BlueprintType)
struct FCurveScaledPayload : public FCurvePayload
{
	GENERATED_BODY()

	/** Factor with which the curve was scaled */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	float Factor = 0.f;
	
	/** Time used as the origin when scaling the curve */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	float Origin = 0.f;
};

USTRUCT(BlueprintType)
struct FCurveRenamedPayload : public FCurvePayload
{
	GENERATED_BODY()
	
	/** Identifier of the curve after it was renamed */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FAnimationCurveIdentifier NewIdentifier;
};

USTRUCT(BlueprintType)
struct FCurveFlagsChangedPayload : public FCurvePayload
{
	GENERATED_BODY()

	/** Old flags mask for the curve */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	int32 OldFlags = 0;
};

USTRUCT(BlueprintType)
struct FAttributePayload : public FEmptyPayload
{
	GENERATED_BODY()

	/** Identifier of the attribute */
	UPROPERTY(BlueprintReadOnly, Category = Payload)
	FAnimationAttributeIdentifier Identifier;
};

typedef FAttributePayload FAttributeAddedPayload;
typedef FAttributePayload FAttributeRemovedPayload;
typedef FAttributePayload FAttributeChangedPayload;

USTRUCT(BlueprintType)
struct FAnimDataModelNotifPayload
{
	GENERATED_BODY()
		
	FAnimDataModelNotifPayload() : Data(nullptr), Struct(nullptr) {}
	FAnimDataModelNotifPayload(const int8* InData, UScriptStruct* InStruct) : Data(InData), Struct(InStruct) {}

	/** Returns the typed contained payload data if the stored type matches*/
	template<typename PayloadType>
	const PayloadType& GetPayload() const
	{
		const UScriptStruct* ScriptStruct = PayloadType::StaticStruct();
		ensure(ScriptStruct == Struct || Struct->IsChildOf(ScriptStruct));
		return *((const PayloadType*)Data);
	}

	const int8* GetData() const { return Data; };
	UScriptStruct* GetStruct() const { return Struct; };
protected:
	/**  ptr to the actual payload data */
	const int8* Data;
	
	/** UScriptStruct type for which Data contains, used for verifying GetPayload */
	UScriptStruct* Struct;
};

UCLASS(MinimalAPI)
class UAnimationDataModelNotifiesExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = AnimationAsset, meta = (ScriptMethod))
	static void CopyPayload(UPARAM(ref)const FAnimDataModelNotifPayload& Payload, UScriptStruct* ExpectedStruct, UPARAM(ref)FEmptyPayload& OutPayload)
	{
		if (Payload.GetStruct() == ExpectedStruct)
		{
			ExpectedStruct->CopyScriptStruct(&OutPayload, Payload.GetData());
		}
	}

	UFUNCTION(BlueprintCallable, Category = AnimationAsset, meta = (ScriptMethod))
	static const FEmptyPayload& GetPayload(UPARAM(ref)const FAnimDataModelNotifPayload& Payload)
	{
		return *((const FEmptyPayload*)Payload.GetData());
	}
#endif // WITH_EDITOR
};

DECLARE_MULTICAST_DELEGATE_ThreeParams(FAnimDataModelModifiedEvent, const EAnimDataModelNotifyType& /* type */, IAnimationDataModel* /* Model */, const FAnimDataModelNotifPayload& /* payload */);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FAnimDataModelModifiedDynamicEvent, EAnimDataModelNotifyType, NotifType, TScriptInterface<IAnimationDataModel>, Model, const FAnimDataModelNotifPayload&, Payload);
