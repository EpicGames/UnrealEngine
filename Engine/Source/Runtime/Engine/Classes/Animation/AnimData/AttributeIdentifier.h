// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimationAsset.h"
#include "EngineLogs.h"

#include "AttributeIdentifier.generated.h"

/** Script-friendly structure for identifying an attribute (curve).
* Wrapping the attribute name, bone name and index, and underlying data type for use within the AnimDataController API. */
USTRUCT(BlueprintType)
struct FAnimationAttributeIdentifier
{
	GENERATED_USTRUCT_BODY();

	FAnimationAttributeIdentifier() : Name(NAME_None), BoneName(NAME_None), BoneIndex(INDEX_NONE), ScriptStruct(nullptr), ScriptStructPath(nullptr) {}
	
	FAnimationAttributeIdentifier(const FName& InName, const int32 InBoneIndex, const FName InBoneName, UScriptStruct* InStruct) : Name(InName), BoneName(InBoneName), BoneIndex(InBoneIndex), ScriptStruct(InStruct), ScriptStructPath(InStruct) {}
	
	FAnimationAttributeIdentifier(const FName& InName, const int32 InBoneIndex, const FName InBoneName, const FSoftObjectPath InStructPath)
		: Name(InName)
		, BoneName(InBoneName)
		, BoneIndex(InBoneIndex)
		, ScriptStruct(nullptr)
		, ScriptStructPath(InStructPath)
	{
		ScriptStruct = Cast<UScriptStruct>(ScriptStructPath.ResolveObject());
	}

	const FName& GetName() const { return Name; }
	const FName& GetBoneName() const { return BoneName; }
	int32 GetBoneIndex() const { return BoneIndex; }
	UScriptStruct* GetType() const { return ScriptStruct; }
	FSoftObjectPath GetScriptStructPath() const { return ScriptStructPath; }

	bool IsValid() const
	{
		return Name != NAME_None && BoneName != NAME_None && BoneIndex != INDEX_NONE && ScriptStruct != nullptr;
	}

	bool operator==(const FAnimationAttributeIdentifier& Other) const
	{
		return (Name == Other.Name && BoneIndex == Other.BoneIndex
			&& BoneName == Other.BoneName
			&& ScriptStruct == Other.ScriptStruct);
	}

	friend uint32 GetTypeHash(const FAnimationAttributeIdentifier& Id)
	{
		return HashCombine(GetTypeHash(Id.BoneIndex), GetTypeHash(Id.Name));
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Name;
		Ar << BoneName;
		Ar << BoneIndex;
		Ar << ScriptStructPath;

		if (Ar.IsLoading())
		{			
			ScriptStruct = Cast<UScriptStruct>(ScriptStructPath.ResolveObject());
			ensure(ScriptStruct != nullptr);
		}

		return true;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("Name: %s Type: %s BoneName: %s BoneIndex :%i"), *Name.ToString(), *ScriptStructPath.ToString(), *BoneName.ToString(), BoneIndex);
	}

protected:
	UPROPERTY(BlueprintReadOnly, Category = AttributeIdentifier)
	FName Name;

	UPROPERTY(BlueprintReadOnly, Category = AttributeIdentifier)
	FName BoneName;

	UPROPERTY(BlueprintReadOnly, Category = AttributeIdentifier)
	int32 BoneIndex;

	UPROPERTY(BlueprintReadOnly, Transient, Category = AttributeIdentifier)
	TObjectPtr<UScriptStruct> ScriptStruct;

	UPROPERTY(BlueprintReadOnly, Category = AttributeIdentifier)
	FSoftObjectPath ScriptStructPath;

	friend class UAnimationAttributeIdentifierExtensions;
};

USTRUCT()
struct FAnimationAttributeIdentifierArray
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FAnimationAttributeIdentifier> AttributeIdentifiers;
};

template<>
struct TStructOpsTypeTraits<FAnimationAttributeIdentifier> : public TStructOpsTypeTraitsBase2<FAnimationAttributeIdentifier>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Script-exposed functionality for wrapping native functionality and constructing valid FAnimationAttributeIdentifier instances */
UCLASS(MinimalAPI)
class UAnimationAttributeIdentifierExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	/**
	* Constructs a valid FAnimationAttributeIdentifier instance. Ensuring that the underlying BoneName exists on the Skeleton for the provided AnimationAsset.
	*
	* @param	AnimationAsset			Animation asset to retrieve Skeleton from
	* @param	AttributeName			Name of the attribute 
	* @param	BoneName				Name of the bone this attribute should be attributed to
	* @param	AttributeType			Type of the underlying data (to-be) stored by this attribute
	* @param	bValidateExistsOnAsset	Whether or not the attribute should exist on the AnimationAsset. False by default.
	*
	* @return	Valid FAnimationCurveIdentifier if the operation was successful
	*/
	UFUNCTION(BlueprintCallable, Category = Attributes, meta=(ScriptMethod))
	static ENGINE_API FAnimationAttributeIdentifier CreateAttributeIdentifier(UAnimationAsset* AnimationAsset, const FName AttributeName, const FName BoneName, UScriptStruct* AttributeType, bool bValidateExistsOnAsset = false);
#endif // WITH_EDITOR

	/**
	* @return	Whether or not the Attribute Identifier is valid
	*/
	UFUNCTION(BlueprintPure, Category = Attribute, meta = (ScriptMethod))
	static bool IsValid(UPARAM(ref) FAnimationAttributeIdentifier& Identifier)
	{
		return Identifier.IsValid();
	}
};
