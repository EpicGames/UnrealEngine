// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *	Bone Mask Filter data holder class for A2Node_PerBone. This has to be in Engine, not Editor class since we need this data to exists in runtime
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BoneMaskFilter.generated.h"

USTRUCT()
struct FBranchFilter
{
	GENERATED_USTRUCT_BODY()

	/** Bone Name to filter **/
	UPROPERTY(EditAnywhere, Category=Branch)
	FName BoneName;

	/** Blend Depth **/
	UPROPERTY(EditAnywhere, Category=Branch)
	int32 BlendDepth = 0;
};

USTRUCT()
struct FInputBlendPose
{
	GENERATED_USTRUCT_BODY()

	/** Bone Name to filter **/
	UPROPERTY(EditAnywhere, Category=Filter)
	TArray<FBranchFilter> BranchFilters;
};

UCLASS(hidecategories=Object, BlueprintType,MinimalAPI)
class UBoneMaskFilter : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=BoneMask)
	TArray<FInputBlendPose>	BlendPoses;
};



