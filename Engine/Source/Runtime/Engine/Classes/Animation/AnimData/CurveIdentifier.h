// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/SmartName.h"
#include "Animation/AnimCurveTypes.h"

#include "CurveIdentifier.generated.h"

/** Enum used to determine a component channel of a transform curve */
UENUM(BlueprintType)
enum class ETransformCurveChannel : uint8
{
	Position,
	Rotation,
	Scale,
	Invalid
};

/** Enum used to determine an axis channel of a vector curve */
UENUM(BlueprintType)
enum class EVectorCurveChannel : uint8
{
	X,
	Y,
	Z,
	Invalid
};

/** Script-friendly structure for identifying an animation curve.
* Wrapping the unique type and smart-name for use within the AnimDataController API. */
USTRUCT(BlueprintType)
struct FAnimationCurveIdentifier
{
	GENERATED_BODY();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimationCurveIdentifier() = default;
	~FAnimationCurveIdentifier() = default;
	FAnimationCurveIdentifier(const FAnimationCurveIdentifier&) = default;
	FAnimationCurveIdentifier(FAnimationCurveIdentifier&&) = default;
	FAnimationCurveIdentifier& operator=(const FAnimationCurveIdentifier&) = default;
	FAnimationCurveIdentifier& operator=(FAnimationCurveIdentifier&&) = default;

	FAnimationCurveIdentifier(const FName& InName, ERawCurveTrackTypes InCurveType)
		: CurveName(InName)
		, CurveType(InCurveType)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	bool operator==(const FAnimationCurveIdentifier& Other) const
	{
		return (CurveName == Other.CurveName && CurveType == Other.CurveType 
			&& Channel == Other.Channel && Axis == Other.Axis);
	}

	bool operator!=(const FAnimationCurveIdentifier& Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return CurveName != NAME_None && CurveType != ERawCurveTrackTypes::RCT_MAX;
	}

	friend uint32 GetTypeHash(const FAnimationCurveIdentifier& CurveId)
	{
		return HashCombine(HashCombine(HashCombine((uint32)CurveId.Channel, (uint32)CurveId.Axis), (uint32)CurveId.CurveType), GetTypeHash(CurveId.CurveName));
	}
	
	ENGINE_API void PostSerialize(const FArchive& Ar);

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	FSmartName InternalName_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	UPROPERTY()
	FName CurveName = NAME_None;

	UPROPERTY()
	ERawCurveTrackTypes CurveType = ERawCurveTrackTypes::RCT_MAX;

	UPROPERTY()
	ETransformCurveChannel Channel = ETransformCurveChannel::Invalid;
	
	UPROPERTY()
	EVectorCurveChannel Axis = EVectorCurveChannel::Invalid;
};

template<>
struct TStructOpsTypeTraits<FAnimationCurveIdentifier> : public TStructOpsTypeTraitsBase2<FAnimationCurveIdentifier>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithPostSerialize = true,
	};
};

/** Script-exposed functionality for wrapping native functionality and constructing valid FAnimationCurveIdentifier instances */
UCLASS(MinimalAPI)
class UAnimationCurveIdentifierExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	* @return	Whether or not the Curve Identifier is valid
	*/
	UFUNCTION(BlueprintPure, Category = Curve, meta = (ScriptMethod))
	static bool IsValid(UPARAM(ref) FAnimationCurveIdentifier& Identifier)
	{
		return Identifier.IsValid();
	}

	/**
	* @return	The name used for displaying the Curve Identifier
	*/
	UFUNCTION(BlueprintPure, Category = Curve, meta = (ScriptMethod))
	static FName GetName(UPARAM(ref) FAnimationCurveIdentifier& Identifier)
	{
		return Identifier.CurveName;
	}

	/**
	* @return	The animation curve type for the curve represented by the Curve Identifier
	*/
	UFUNCTION(BlueprintPure, Category = Curve, meta = (ScriptMethod))
	static ERawCurveTrackTypes GetType(UPARAM(ref) FAnimationCurveIdentifier& Identifier)
	{
		return Identifier.CurveType;
	}

#if WITH_EDITOR
	/**
	* Constructs a valid FAnimationCurveIdentifier instance.
	*
	* @param	InOutIdentifier		The identifier to set up
	* @param	Name				Name of the curve to find or add
	* @param	CurveType			Type of the curve to find or add
	*/
	UFUNCTION(BlueprintCallable, Category = Curves, meta = (ScriptMethod))
	static ENGINE_API void SetCurveIdentifier(UPARAM(ref) FAnimationCurveIdentifier& InOutIdentifier, FName Name, ERawCurveTrackTypes CurveType);

	UFUNCTION(BlueprintCallable, Category = Curves, meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage="Please use SetCurveIdentifier."))
	static ENGINE_API FAnimationCurveIdentifier GetCurveIdentifier(USkeleton* InSkeleton, FName Name, ERawCurveTrackTypes CurveType);
	
	/**
	* Tries to construct a valid FAnimationCurveIdentifier instance. It tries to find the underlying SmartName on the provided Skeleton for the provided curve type.
	*
	* @param	InSkeleton			Skeleton on which to look for the curve name
	* @param	Name				Name of the curve to find
	* @param	CurveType			Type of the curve to find
	*
	* @return	Valid FAnimationCurveIdentifier if the name exists on the skeleton and the operation was successful, invalid otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = Curves, meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage="Curve identifiers are no longer retrievable globally from the skeleton, they are specified per-animation."))
	static ENGINE_API FAnimationCurveIdentifier FindCurveIdentifier(const USkeleton* InSkeleton, FName Name, ERawCurveTrackTypes CurveType);

	/**
	* Retrieves all curve identifiers for a specific curve types from the provided Skeleton
	*
	* @param	InSkeleton			Skeleton from which to retrieve the curve identifiers
	* @param	CurveType			Type of the curve identifers to retrieve
	*
	* @return	Array of FAnimationCurveIdentifier instances each representing a unique curve if the operation was successful, empyty array otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = Curves, meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage="Curve identifiers are no longer retrievable globally from the skeleton, they are specified per-animation."))
	static ENGINE_API TArray<FAnimationCurveIdentifier> GetCurveIdentifiers(USkeleton* InSkeleton, ERawCurveTrackTypes CurveType);

	/**
	* Converts a valid FAnimationCurveIdentifier instance with RCT_Transform curve type to target a child curve.
	*
	* @param	InOutIdentifier		[out] Identifier to be converted
	* @param	Channel				Channel to target
	* @param	Axis				Axis within channel to target
	*
	* @return	Valid FAnimationCurveIdentifier if the operation was successful
	*/
	UFUNCTION(BlueprintCallable, Category = Curves, meta = (ScriptMethod))
	static ENGINE_API bool GetTransformChildCurveIdentifier(UPARAM(ref) FAnimationCurveIdentifier& InOutIdentifier, ETransformCurveChannel Channel, EVectorCurveChannel Axis);
#endif // WITH_EDITOR
};

