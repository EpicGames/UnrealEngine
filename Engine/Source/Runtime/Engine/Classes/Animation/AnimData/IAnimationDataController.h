// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "CurveIdentifier.h"
#include "Misc/FrameRate.h"
#include "AttributeIdentifier.h"
#include "IAnimationDataModel.h"
#include "Logging/LogVerbosity.h"
#include "Algo/Transform.h"
#include "Animation/AnimCurveTypes.h"

#include "IAnimationDataController.generated.h"

#define LOCTEXT_NAMESPACE "IAnimationDataController"

class FText;
class UAssetUserData;
class IAnimationDataModel;

struct FCurveAttributes;
struct FAnimationCurveIdentifier;
struct FAnimationAttributeIdentifier;

namespace UE {
namespace Anim {
	class FOpenBracketAction;
	class FCloseBracketAction;
}}

/**
 * The Controller is the sole authority to perform changes on the Animation Data Model. Any mutation to the model made will
 * cause a subsequent notify (EAnimDataModelNotifyType) to be broadcast-ed from the Model's ModifiedEvent. Alongside of it is a 
 * payload containing information relevant to the mutation. These notifies should be relied upon to update any dependent views 
 * or generated (derived) data.
 */
UINTERFACE(BlueprintType, meta=(CannotImplementInterfaceInBlueprint), MinimalAPI)
class UAnimationDataController : public UInterface
{
	GENERATED_BODY()
};

class IAnimationDataController
{
public:
	GENERATED_BODY()

#if WITH_EDITOR
	/** RAII helper to define a scoped-based bracket, opens and closes a controller bracket automatically */
    struct FScopedBracket
	{
		FScopedBracket(IAnimationDataController* InController, const FText& InDescription, bool bInShouldTransact=true)
            : Controller(*InController), bShouldTransact(bInShouldTransact)
		{
			Controller.OpenBracket(InDescription, bShouldTransact);
		}

		FScopedBracket(IAnimationDataController& InController, const FText& InDescription, bool bInShouldTransact=true)
            : Controller(InController), bShouldTransact(bInShouldTransact)
		{
			Controller.OpenBracket(InDescription, bShouldTransact);
		}
		
		FScopedBracket(const TScriptInterface<IAnimationDataController>& InController, const FText& InDescription, bool bInShouldTransact=true)
            : Controller(*InController), bShouldTransact(bInShouldTransact)
		{
			Controller.OpenBracket(InDescription, bShouldTransact);
		}
		
		~FScopedBracket()
		{
			Controller.CloseBracket(bShouldTransact);
		}
	private:
		IAnimationDataController& Controller;
		bool bShouldTransact;
	};
#endif // WITH_EDITOR

	/**
	* Sets the AnimDataModel instance this controller is supposed to be targeting
	*
	* @param	InModel		IAnimationDataModel instance to target
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void SetModel(TScriptInterface<IAnimationDataModel> InModel) = 0;

	/**
	* @return		The IAnimationDataModel instance this controller is currently targeting
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual TScriptInterface<IAnimationDataModel> GetModelInterface() const = 0;

	/**
	* @return		The IAnimationDataModel instance this controller is currently targeting
	*/
	virtual const IAnimationDataModel* const GetModel() const = 0;

	/**
	* Opens an interaction bracket, used for combining a set of controller actions. Broadcasts a EAnimDataModelNotifyType::BracketOpened notify,
	* this can be used by any Views or dependent systems to halt any unnecessary or invalid operations until the (last) bracket is closed.
	*
	* @param	InTitle				Description of the bracket, e.g. "Generating Curve Data"
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void OpenBracket(const FText& InTitle, bool bShouldTransact = true) = 0;

	/**
	* Closes a previously opened interaction bracket, used for combining a set of controller actions. Broadcasts a EAnimDataModelNotifyType::BracketClosed notify.
	*
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void CloseBracket(bool bShouldTransact = true) = 0;

	/**
	* Sets the total play-able length in seconds. Broadcasts a EAnimDataModelNotifyType::SequenceLengthChanged notify if successful.
	* The number of frames and keys for the provided length is recalculated according to the current value of UAnimDataModel::FrameRate.
	*
	* @param	NewLengthInFrames	Total new play-able number of frames value (according to frame rate), has to be positive and non-zero
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void SetNumberOfFrames(FFrameNumber NewLengthInFrames, bool bShouldTransact = true) = 0;

	UE_DEPRECATED(5.1, "SetPlayLength with length in seconds in deprecated use SetNumberOfFrames using FFrameNumber instead")
	UFUNCTION(BlueprintCallable, Category = AnimationData, meta=(DeprecatedFunction, DeprecationMessage="SetPlayLength is deprecated use SetNumberOfFrames instead."))
	virtual void SetPlayLength(float Length, bool bShouldTransact = true) = 0;

	/*** Sets the total play-able length in seconds. Broadcasts a EAnimDataModelNotifyType::SequenceLengthChanged notify if successful.
	* T0 and T1 are expected to represent the window of time that was either added or removed. E.g. for insertion T0 indicates the time
	* at which additional time starts and T1 were it ends. For removal T0 indicates the time at which time should be started to remove, and T1 indicates the end. Giving a total of T1 - T0 added or removed length.
	* The number of frames and keys for the provided length is recalculated according to the current value of UAnimDataModel::FrameRate.
	* @param	NewLengthInFrames	Total new play-able number of frames value (according to frame rate), has to be positive and non-zero
	* @param	T0					Point between 0 and NewLengthInFrames at which the change in length starts
	* @param	T1					Point between 0 and NewLengthInFrames at which the change in length ends
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
    virtual void ResizeNumberOfFrames(FFrameNumber NewLengthInFrames, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) = 0;

	UE_DEPRECATED(5.1, "ResizePlayLength with length in seconds in deprecated use ResizeNumberOfFrames using FFrameNumber instead")
	UFUNCTION(BlueprintCallable, Category = AnimationData, meta=(DeprecatedFunction, DeprecationMessage="ResizePlayLength is deprecated use ResizeNumberOfFrames instead."))
	virtual void ResizePlayLength(float NewLength, float T0, float T1, bool bShouldTransact = true) = 0;

	/**
	* Sets the total play-able length in seconds and resizes curves. Broadcasts EAnimDataModelNotifyType::SequenceLengthChanged
	* and EAnimDataModelNotifyType::CurveChanged notifies if successful.
	* T0 and T1 are expected to represent the window of time that was either added or removed. E.g. for insertion T0 indicates the time
	* at which additional time starts and T1 were it ends. For removal T0 indicates the time at which time should be started to remove, and T1 indicates the end. Giving a total of T1 - T0 added or removed length.
	* The number of frames and keys for the provided length is recalculated according to the current value of UAnimDataModel::FrameRate.
	*
	* @param	NewLengthInFrames	Total new play-able number of frames value (according to frame rate), has to be positive and non-zero
	* @param	T0					Point between 0 and NewLengthInFrames at which the change in length starts
	* @param	T1					Point between 0 and NewLengthInFrames at which the change in length ends
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void ResizeInFrames(FFrameNumber NewLengthInFrames, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) = 0;

	UE_DEPRECATED(5.1, "Resize with length in seconds in deprecated use ResizeInFrames using FFrameNumber instead")
	UFUNCTION(BlueprintCallable, Category = AnimationData, meta=(DeprecatedFunction, DeprecationMessage="Resize is deprecated use ResizeInFrames instead."))
	virtual void Resize(float Length, float T0, float T1, bool bShouldTransact = true) = 0;
	
	/**
	* Sets the frame rate according to which the bone animation is expected to be sampled. Broadcasts a EAnimDataModelNotifyType::FrameRateChanged notify if successful.
	* The number of frames and keys for the provided frame rate is recalculated according to the current value of UAnimDataModel::PlayLength.
	*
	* @param	FrameRate			The new sampling frame rate, has to be positive and non-zero
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void SetFrameRate(FFrameRate FrameRate, bool bShouldTransact = true) = 0;

	/**
	* Adds a new bone animation track for the provided name. Broadcasts a EAnimDataModelNotifyType::TrackAdded notify if successful.
	*
	* @param	BoneName			Bone name for which a track should be added
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	The index at which the bone track was added, INDEX_NONE if adding it failed
	*/
	UE_DEPRECATED(5.2, "AddBoneTrack returning index is deprecated use AddBoneCurve returning bool instead.")
	UFUNCTION(BlueprintCallable, Category = AnimationData,  meta=(DeprecatedFunction, DeprecationMessage="AddBoneTrack returning index is deprecated use AddBoneCurve returning bool instead."))
	virtual int32 AddBoneTrack(FName BoneName, bool bShouldTransact = true) { AddBoneCurve(BoneName, bShouldTransact); return INDEX_NONE; }

	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual bool AddBoneCurve(FName BoneName, bool bShouldTransact = true) = 0;

	/**
	* Inserts a new bone animation track for the provided name, at the provided index. Broadcasts a EAnimDataModelNotifyType::TrackAdded notify if successful.
	* The bone name is verified with the AnimModel's outer target USkeleton to ensure the bone exists.
	*
	* @param	BoneName			Bone name for which a track should be inserted
	* @param	DesiredIndex		Index at which the track should be inserted
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	The index at which the bone track was inserted, INDEX_NONE if the insertion failed
	*/
	UE_DEPRECATED(5.1, "InsertBoneTrack has been deprecated, use AddBoneTrack instead")
	UFUNCTION(BlueprintCallable, Category = AnimationData, meta=(DeprecatedFunction, DeprecationMessage="InsertBoneTrack is deprecated use AddBoneTrack instead."))
	virtual int32 InsertBoneTrack(FName BoneName, int32 DesiredIndex, bool bShouldTransact = true) = 0;

	/**
	* Removes an existing bone animation track with the provided name. Broadcasts a EAnimDataModelNotifyType::TrackRemoved notify if successful.
	*
	* @param	BoneName			Bone name of the track which should be removed
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the removal was successful
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual bool RemoveBoneTrack(FName BoneName, bool bShouldTransact = true) = 0;

	/**
	* Removes all existing Bone Animation tracks. Broadcasts a EAnimDataModelNotifyType::TrackRemoved for each removed track, wrapped within BracketOpened/BracketClosed notifies.
	*
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual void RemoveAllBoneTracks(bool bShouldTransact = true) = 0;

	/**
	* Sets keys for an existing bone animation track with the provided name. Broadcasts a EAnimDataModelNotifyType::TrackChanged notify if successful.
	* The provided number of keys provided is expected to match for each component, and be non-zero.
	*
	* @param	BoneName			Bone name of the track for which the keys should be set
	* @param	PositionalKeys		Array of keys for the translation component
	* @param	RotationalKeys		Array of keys for the rotation component
	* @param	ScalingKeys			Array of keys for the scale component
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the keys were successfully set
	*/
	virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) = 0;

	/**
	* Sets keys for an existing bone animation track with the provided name. Broadcasts a EAnimDataModelNotifyType::TrackChanged notify if successful.
	* The provided number of keys provided is expected to match for each component, and be non-zero.
	*
	* @param	BoneName			Bone name of the track for which the keys should be set
	* @param	PositionalKeys		Array of keys for the translation component
	* @param	RotationalKeys		Array of keys for the rotation component
	* @param	ScalingKeys			Array of keys for the scale component
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the keys were successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationData)
	virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) = 0;

	/**
	* Sets a range of keys for an existing bone animation track with the provided name. Broadcasts a EAnimDataModelNotifyType::TrackChanged notify if successful.
	* The provided number of keys provided is expected to match for each component, be between FrameLowerBound and FrameUpperBound (inclusive), and be non-zero.
	*
	* @param	BoneName			Bone name of the track for which the keys should be set
	* @param	KeyRangeToSet			Range of frames to set keys for
	* @param	PositionalKeys		Array of keys for the translation component
	* @param	RotationalKeys		Array of keys for the rotation component
	* @param	ScalingKeys			Array of keys for the scale component
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the keys were successfully set
	*/
	virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) = 0;

	/**
	* Sets a range of keys for an existing bone animation track with the provided name. Broadcasts a EAnimDataModelNotifyType::TrackChanged notify if successful.
	* The provided number of keys provided is expected to match for each component, be between FrameLowerBound and FrameUpperBound (inclusive), and be non-zero.
	*
	* @param	BoneName			Bone name of the track for which the keys should be set
	* @param	KeyRangeToSet			Range of frames to set keys for
	* @param	PositionalKeys		Array of keys for the translation component
	* @param	RotationalKeys		Array of keys for the rotation component
	* @param	ScalingKeys			Array of keys for the scale component
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the keys were successfully set
	*/
	virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) = 0;

	/**
	* Adds a new curve with the provided information. Broadcasts a EAnimDataModelNotifyType::CurveAdded notify if successful.
	*
	* @param	CurveId				Identifier for the to-be-added curve
	* @param	CurveFlags			Flags to be set for the curve
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve was successfully added
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool AddCurve(const FAnimationCurveIdentifier& CurveId, int32 CurveFlags = 0x00000004, bool bShouldTransact = true) = 0;

	/**
	* Duplicated the curve with the identifier. Broadcasts a EAnimDataModelNotifyType::CurveAdded notify if successful.
	*
	* @param	CopyCurveId			Identifier for the to-be-duplicated curve
	* @param	NewCurveId			Identifier for the to-be-added curve
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve was successfully duplicated
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool DuplicateCurve(const FAnimationCurveIdentifier& CopyCurveId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) = 0;
	

	/**
	* Remove the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveRemoved notify if successful.
	*
	* @param	CurveId				Identifier for the to-be-removed curve
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve was successfully removed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool RemoveCurve(const FAnimationCurveIdentifier& CurveId, bool bShouldTransact = true) = 0;

	/**
	* Removes all the curves of the provided type. Broadcasts a EAnimDataModelNotifyType::CurveRemoved for each removed curve, wrapped within BracketOpened/BracketClosed notifies.
	*
	* @param	SupportedCurveType	Type for which all curves are to be removed
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual void RemoveAllCurvesOfType(ERawCurveTrackTypes SupportedCurveType, bool bShouldTransact = true) = 0;

	/**
	* Set an individual flag for the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveFlagsChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the curve for which the flag state is to be set
	* @param	Flag				Flag for which the state is supposed to be set
	* @param	bState				State of the flag to be, true=set/false=not set
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the flag state was successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveFlag(const FAnimationCurveIdentifier& CurveId, EAnimAssetCurveFlags Flag, bool bState = true, bool bShouldTransact = true) = 0;

	/**
	* Replace the flags for the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveFlagsChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the curve for which the flag state is to be set
	* @param	Flags				Flag mask with which the existing flags are to be replaced
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the flag mask was successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveFlags(const FAnimationCurveIdentifier& CurveId, int32 Flags, bool bShouldTransact = true) = 0;

	/**
	* Replace the keys for the transform curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the transform curve for which the keys are to be set
	* @param	TransformValues		Transform Values with which the existing values are to be replaced
	* @param	TimeKeys			Time Keys with which the existing keys are to be replaced
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the transform curve keys were successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetTransformCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FTransform>& TransformValues, const TArray<float>& TimeKeys, bool bShouldTransact = true) = 0;
		
	/**
	* Sets a single key for the transform curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	* In case a key for any of the individual transform channel curves already exists the value is replaced.
	*
	* @param	CurveId			    Identifier for the transform curve for which the key is to be set
	* @param	Time				Time of the key to be set
	* @param	Value				Value of the key to be set
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the transform curve key was successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, const FTransform& Value, bool bShouldTransact = true) = 0;

	/**
	* Removes a single key for the transform curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the transform curve for which the key is to be removed
	* @param	Time				Time of the key to be removed
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the transform curve key was successfully removed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool RemoveTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) = 0;

	/**
	* Renames the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveRenamed notify if successful.
	*
	* @param	CurveToRenameId		Identifier for the curve to be renamed
	* @param	NewCurveId			Time of the key to be removed
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve was successfully renamed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool RenameCurve(const FAnimationCurveIdentifier& CurveToRenameId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) = 0;

	/**
	* Changes the color of the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveColorChanged notify if successful.
	* Currently changing curve colors is only supported for float curves.
	*
	* @param	CurveId				Identifier of the curve to change the color for
	* @param	Color				Color to which the curve is to be set
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve color was successfully changed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveColor(const FAnimationCurveIdentifier& CurveId, FLinearColor Color, bool bShouldTransact = true) = 0;

	/**
	* Changes the comment of the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveCommentChanged notify if successful.
	* Currently changing curve comments is only supported for float curves.
	*
	* @param	CurveId				Identifier of the curve to change the comment for
	* @param	Comment				Comment to which the curve is to be set
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve comment was successfully changed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveComment(const FAnimationCurveIdentifier& CurveId, const FString& Comment, bool bShouldTransact = true) = 0;

	/**
	* Scales the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveScaled notify if successful.
	*
	* @param	CurveId				Identifier of the curve to scale
	* @param	Origin				Time to use as the origin when scaling the curve
	* @param	Factor				Factor with which the curve is supposed to be scaled
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not scaling the curve was successful
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool ScaleCurve(const FAnimationCurveIdentifier& CurveId, float Origin, float Factor, bool bShouldTransact = true) = 0;

	/**
	* Sets a single key for the curve with provided identifier and name. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	* In case a key for the provided key time already exists the key is replaced.
	*
	* @param	CurveId			    Identifier for the curve for which the key is to be set
	* @param	Key					Key to be set
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve key was successfully set
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveKey(const FAnimationCurveIdentifier& CurveId, const FRichCurveKey& Key, bool bShouldTransact = true) = 0;
	
	/**
	* Remove a single key from the curve with provided identifier and name. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the curve for which the key is to be removed
	* @param	Time				Time of the key to be removed
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve key was successfully removed
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool RemoveCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) = 0;

	/**
	* Replace the keys for the curve with provided identifier and name. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	*
	* @param	CurveId			    Identifier for the curve for which the keys are to be replaced
	* @param	CurveKeys			Keys with which the existing keys are to be replaced
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not replacing curve keys was successful
	*/
	UFUNCTION(BlueprintCallable, Category = CurveData)
	virtual bool SetCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FRichCurveKey>& CurveKeys, bool bShouldTransact = true) = 0;

	/**
	* Changes the attributes of the curve with provided identifier. Broadcasts a EAnimDataModelNotifyType::CurveChanged notify if successful.
	*
	* @param	CurveId				Identifier of the curve to change the color for
	* @param	Attributes			Attribute values to be applied
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the curve attributes were set successfully
	*/
	virtual bool SetCurveAttributes(const FAnimationCurveIdentifier& CurveId, const FCurveAttributes& Attributes, bool bShouldTransact = true) = 0;

	/**
	* Updates the display name values for any stored curve, with the names being retrieved from the provided skeleton. Broadcasts a EAnimDataModelNotifyType::CurveRenamed for each to-be-updated curve name, wrapped within BracketOpened/BracketClosed notifies.
	*
	* @param	Skeleton			Skeleton to retrieve the display name values from
	* @param	SupportedCurveType	Curve type for which the names should be updated
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UE_DEPRECATED(5.3, "This function is no longer used.")
	UFUNCTION(BlueprintCallable, Category = CurveData, meta=(DeprecatedFunction, DeprecationMessage="This function is no longer used."))
	virtual void UpdateCurveNamesFromSkeleton(const USkeleton* Skeleton, ERawCurveTrackTypes SupportedCurveType, bool bShouldTransact = true) {}

	/**
	* Updates the curve names with the provided skeleton, if a display name is not found it will be added thus modifying the skeleton. Broadcasts a EAnimDataModelNotifyType::CurveRenamed for each curve name for which the UID was different or if it was added as a new smart-name, wrapped within BracketOpened/BracketClosed notifies.
	*
	* @param	Skeleton			Skeleton to retrieve the display name values from
	* @param	SupportedCurveType	Curve type for which the names should be updated
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	UE_DEPRECATED(5.3, "This function is no longer used.")
	UFUNCTION(BlueprintCallable, Category = CurveData, meta=(DeprecatedFunction, DeprecationMessage="This function is no longer used."))
	virtual void FindOrAddCurveNamesOnSkeleton(USkeleton* Skeleton, ERawCurveTrackTypes SupportedCurveType, bool bShouldTransact = true) {}

	/**
	* Removes any bone track for which the name was not found in the provided skeleton. Broadcasts a EAnimDataModelNotifyType::TrackRemoved for each track which was not found in the skeleton, wrapped within BracketOpened/BracketClosed notifies.
	*
	* @param	Skeleton			Skeleton to retrieve the display name values from
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	virtual bool RemoveBoneTracksMissingFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) = 0;

	/**
	* Removes any bone attribute for which the name was not found in the provided skeleton. Broadcasts a EAnimDataModelNotifyType::AttributeRemoved for each attribute which was not found in the skeleton, wrapped within BracketOpened/BracketClosed notifies.
	* Updates any bone attribute for which the bone index is different in the provided skeleton. Broadcasts a EAnimDataModelNotifyType::AttributeAdded and EAnimDataModelNotifyType::AttributeRemove for each attribute which was remapped
	*
	* @param	Skeleton			Skeleton to retrieve the bone information from
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	virtual void UpdateAttributesFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) = 0;

	/**
	* Broadcast a EAnimDataModelNotifyType::Populated notify.
	*/
	virtual void NotifyPopulated() = 0;	

	/**
	* Resets all data stored in the model, broadcasts a EAnimDataModelNotifyType::Reset and wraps all actions within BracketOpened/BracketClosed notifies.
	*	- Bone tracks, broadcasts a EAnimDataModelNotifyType::TrackRemoved for each
	*	- Curves, broadcasts a EAnimDataModelNotifyType::CurveRemoves for each
	*	- Play length to one frame at 30fps, broadcasts a EAnimDataModelNotifyType::PlayLengthChanged
	*	- Frame rate to 30fps, broadcasts a EAnimDataModelNotifyType::FrameRateChanged
	*
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*/
	virtual void ResetModel(bool bShouldTransact = true) = 0;

	/**
	* Adds a new attribute with the provided information. Broadcasts a EAnimDataModelNotifyType::AttributeAdded notify if successful.
	*
	* @param	AttributeIdentifier		Identifier for the to-be-added attribute
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the attribute was successfully added
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual bool AddAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) = 0;

	/**
	* Removes an attribute, if found, with the provided information. Broadcasts a EAnimDataModelNotifyType::AttributeRemoved notify if successful.
	*
	* @param	AttributeIdentifier		Identifier for the to-be-removed attribute
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the attribute was successfully removed
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual bool RemoveAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) = 0;

	/**
	* Removes all attributes for the specified bone name, if any. Broadcasts a EAnimDataModelNotifyType::AttributeRemoved notify for each removed attribute.
	*
	* @param	BoneName			Name of the bone to remove attributes for
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Total number of removes attributes
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual int32 RemoveAllAttributesForBone(const FName& BoneName, bool bShouldTransact = true) = 0;

	/**
	* Removes all stored attributes. Broadcasts a EAnimDataModelNotifyType::AttributeRemoved notify for each removed attribute.
	*
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	*
	* @return	Total number of removes attributes
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual int32 RemoveAllAttributes(bool bShouldTransact = true) = 0;	

	/**
	* Sets a single key for the attribute with provided identifier. Broadcasts a EAnimDataModelNotifyType::AttributeChanged notify if successful.
	* In case a key for the provided key time already exists the key is replaced.
	*
	* @param	AttributeIdentifier		Identifier for the attribute for which the key is to be set
	* @param	Time					Time of the to-be-set key
	* @param	KeyValue				Value (templated) of the to-be-set key
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the key was successfully set
	*/
	template<typename AttributeType>
	bool SetTypedAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const AttributeType& KeyValue, bool bShouldTransact = true)
	{
		return SetAttributeKey(AttributeIdentifier, Time, static_cast<const void*>(&KeyValue), AttributeType::StaticStruct(), bShouldTransact);
	}

	/**
	* Sets a single key for the attribute with provided identifier. Broadcasts a EAnimDataModelNotifyType::AttributeChanged notify if successful.
	* In case a key for the provided key time already exists the key is replaced.
	*
	* @param	AttributeIdentifier		Identifier for the attribute for which the key is to be set
	* @param	Time					Time of the to-be-set key
	* @param	KeyValue				Value of the to-be-set key
	* @param	TypeStruct				UScriptStruct describing the type of KeyValue
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the key was successfully set
	*/
	virtual bool SetAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact = true) = 0;
	
	/**
	* Replace the keys for the attribute with provided identifier. Broadcasts a EAnimDataModelNotifyType::AttributeChanged notify if successful.
	*
	* @param	AttributeIdentifier		Identifier for the attribute for which the keys are to be replaced
	* @param	Times					Times with which the existing key timings are to be replaced
	* @param	KeyValues				Values with which the existing key values are to be replaced
	* @param	TypeStruct				UScriptStruct describing the type of KeyValues
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not replacing the attribute keys was successful
	*/
	virtual bool SetAttributeKeys(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact = true) = 0;

	/**
	* Replace the keys for the attribute with provided identifier. Broadcasts a EAnimDataModelNotifyType::AttributeChanged notify if successful.
	*
	* @param	AttributeIdentifier		Identifier for the attribute for which the keys are to be replaced
	* @param	Times					Times with which the existing key timings are to be replaced
	* @param	KeyValues				Values (templated) with which the existing key values are to be replaced
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not replacing the attribute keys was successful
	*/
	template<typename AttributeType>
	bool SetTypedAttributeKeys(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const AttributeType> KeyValues, bool bShouldTransact = true)
	{
		TArray<const void*> KeyValuePtrs;
		Algo::Transform(KeyValues, KeyValuePtrs, [](const AttributeType& Value)
		{
			return static_cast<const void*>(&Value);
		});

		return SetAttributeKeys(AttributeIdentifier, Times, MakeArrayView(KeyValuePtrs), AttributeType::StaticStruct(), bShouldTransact);
	}

	/**
	* Remove a single key from the attribute with provided identifier. Broadcasts a EAnimDataModelNotifyType::AttributeChanged notify if successful.
	*
	* @param	AttributeIdentifier		Identifier for the attribute from which the key is to be removed
	* @param	Time					Time of the key to be removed
	* @param	bShouldTransact			Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the attribute key was successfully removed
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual bool RemoveAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, bool bShouldTransact = true) = 0;

	/**
	* Duplicated the attribute (curve) with the identifier. Broadcasts a EAnimDataModelNotifyType::AttributeAdded notify if successful.
	*
	* @param	AttributeIdentifier			Identifier for the to-be-duplicated attribute
	* @param	NewAttributeIdentifier		Identifier for the to-be-added attribute
	* @param	bShouldTransact				Whether or not any undo-redo changes should be generated
	*
	* @return	Whether or not the attribute was successfully duplicated
	*/
	UFUNCTION(BlueprintCallable, Category = AttributeData)
	virtual bool DuplicateAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, const FAnimationAttributeIdentifier& NewAttributeIdentifier, bool bShouldTransact = true) = 0;

	/** Updates/removes/remaps contained animation data according to the newly assigned skeleton
	 *
	* @param	TargetSkeleton		Skeleton to which the contained animation data is to be mapped
	* @param	bShouldTransact		Whether or not any undo-redo changes should be generated
	* 
	 */
	virtual void UpdateWithSkeleton(USkeleton* TargetSkeleton, bool bShouldTransact = true) = 0;

	/** Copies any animation relevant data from an already existing IAnimationDataModel object */
	virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> InModel) = 0;

	/** Initializes model data structures */
	virtual void InitializeModel() = 0;

	/** Returns the final frame number calculating according to the Model its frame-rate, additionally outputs log information for invalid/loss of precision */
	FFrameNumber ConvertSecondsToFrameNumber(double Seconds) const
	{
		ValidateModel();
		
		const FFrameRate& ModelFrameRate = GetModel()->GetFrameRate();
		
		static const FNumberFormattingOptions DurationFormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(8)
			.SetMaximumFractionalDigits(8);
	
		const FFrameTime FrameTime = ModelFrameRate.AsFrameTime(Seconds);
		// Check for either small sub-frame value or near zero seconds representation (of sub-frame only)
		if (!FMath::IsNearlyZero(FrameTime.GetSubFrame(), UE_KINDA_SMALL_NUMBER) &&
			!FMath::IsNearlyZero(ModelFrameRate.AsSeconds(FFrameTime(0, FrameTime.GetSubFrame())), UE_DOUBLE_KINDA_SMALL_NUMBER))
		{
			ReportWarningf(LOCTEXT("SecondsToFrameNumberPrecisionWarning", "Insufficient precision while converting seconds to frames: {0} seconds {1} frames using {2} (sub-frame in seconds {3})"), FText::AsNumber(Seconds), FText::AsNumber(FrameTime.AsDecimal(), &DurationFormatOptions), ModelFrameRate.ToPrettyText(), FText::AsNumber(ModelFrameRate.AsSeconds(FFrameTime(0, FrameTime.GetSubFrame())), &DurationFormatOptions));
		}

		return FrameTime.GetFrame();
	}	
protected:
	/** Functionality used by FOpenBracketAction and FCloseBracketAction to broadcast their equivalent notifies without actually opening a bracket. */
	virtual void NotifyBracketOpen() = 0;
	virtual void NotifyBracketClosed() = 0;
	
	virtual bool ShouldTransact(bool bInLocalShouldTransact) const = 0;

	template <typename FmtType, typename... Types>
    void ReportWarningf(const FmtType& Fmt, Types... Args) const
	{
		ReportWarning(FText::Format(Fmt, Args...));
	}

	template <typename FmtType, typename... Types>
    void ReportErrorf(const FmtType& Fmt, Types... Args) const
	{
		ReportError(FText::Format(Fmt, Args...));
	}

	void Report(ELogVerbosity::Type Verbosity, const FText& Message) const
	{
		ReportMessage(GetModelInterface().GetObject(), Message, Verbosity);
	}

	template <typename FmtType, typename... Types>
    void Reportf(ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args) const
    {
		ReportMessage(GetModelInterface().GetObject(), FText::Format(Fmt, Args...), Verbosity);
    }

	void ReportWarning(const FText& Message) const
	{
		ReportMessage(GetModelInterface().GetObject(), Message, ELogVerbosity::Warning);
	}
	
	void ReportError(const FText& Message) const
    {
    	ReportMessage(GetModelInterface().GetObject(), Message, ELogVerbosity::Error);
    }

	/** Returns whether or not the supplied curve type is supported by the controller functionality */
	static bool IsSupportedCurveType(ERawCurveTrackTypes CurveType)
	{
		const TArray<ERawCurveTrackTypes> SupportedTypes = { ERawCurveTrackTypes::RCT_Float, ERawCurveTrackTypes::RCT_Transform };
		return SupportedTypes.Contains(CurveType);
	}

	/** Ensures that a valid model is currently targeted */
	void ValidateModel() const
	{
		checkf(GetModel() != nullptr, TEXT("Invalid Model"));
	}
		
	/** Verifies whether or not the Model's outer object is (or is derived from) the specified UClass */
	bool CheckOuterClass(UClass* InClass) const
	{
		ValidateModel();
	
		if (const UObject* ModelOuter = GetModelInterface().GetObject()->GetOuter())
		{
			if (const UClass* OuterClass = ModelOuter->GetClass())
			{
				if (OuterClass == InClass || OuterClass->IsChildOf(InClass))
				{
					return true;
				}
				else
				{
					ReportErrorf(NSLOCTEXT("IAnimationDataController", "NoValidOuterClassError", "Incorrect outer object class found for Animation Data Model {0}, expected {1} actual {2}"), FText::FromString(GetModelInterface().GetObject()->GetName()), FText::FromString(InClass->GetName()), FText::FromString(OuterClass->GetName()));
				}
			}
		}
		else
		{
			ReportErrorf(NSLOCTEXT("IAnimationDataController", "NoValidOuterObjectFoundError", "No valid outer object found for Animation Data Model {0}"), FText::FromString(GetModelInterface().GetObject()->GetName()));
		}

		return false;
	}
	
	/** Returns the string representation of the provided curve enum type value */
	static FString GetCurveTypeValueName(ERawCurveTrackTypes InType)
	{
		FString ValueString;
		if (const UEnum* Enum = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.ERawCurveTrackTypes")))
		{
			ValueString = Enum->GetNameStringByValue(static_cast<int64>(InType));
		}

		return ValueString;
	}
	
	static constexpr int32 DefaultCurveFlags = EAnimAssetCurveFlags::AACF_Editable;

public:
	template <typename FmtType, typename... Types>
	static void ReportObjectWarningf(const UObject* ErrorObject, const FmtType& Fmt, Types... Args)
	{
		ReportMessage(ErrorObject, FText::Format(Fmt, Args...), ELogVerbosity::Warning);
	}

	template <typename FmtType, typename... Types>
	static void ReportObjectErrorf(const UObject* ErrorObject, const FmtType& Fmt, Types... Args)
	{
		ReportMessage(ErrorObject, FText::Format(Fmt, Args...), ELogVerbosity::Error);
	}
	
	template <typename FmtType, typename... Types>
	static void Reportf(ELogVerbosity::Type LogVerbosity, const UObject* ErrorObject, const FmtType& Fmt, Types... Args)
	{
		ReportMessage(ErrorObject, FText::Format(Fmt, Args...), ELogVerbosity::Error);
	}

	static ENGINE_API void ReportMessage(const UObject* ErrorObject, const FText& InMessage, ELogVerbosity::Type LogVerbosity);	
private:
	friend class FAnimDataControllerTestBase;
	friend UE::Anim::FOpenBracketAction;
	friend UE::Anim::FCloseBracketAction;
};

// Add this to subclasses for change transactor. Note, this code uses UnrealEd types and this file is in Engine
// Needs to include "ChangeTransactor.h"
#define DECLARE_TRANSACTION_CONTROLLER \
		typedef TUniquePtr<UE::FScopedCompoundTransaction> FTransaction; \
		FTransaction ConditionalTransaction(const FText& Description, bool bCondition) \
		{ \
			if (UE::FChangeTransactor::CanTransactChanges() && ShouldTransact(bCondition)) \
			{  \
				return MakeUnique<UE::FScopedCompoundTransaction>(ChangeTransactor, Description); \
			} \
			return nullptr; \
		} \
		typedef TUniquePtr<IAnimationDataController::FScopedBracket> FBracket;  \
		FBracket ConditionalBracket(const FText& Description, bool bCondition) \
		{	 \
			return MakeUnique<IAnimationDataController::FScopedBracket>(this, Description, UE::FChangeTransactor::CanTransactChanges() && ShouldTransact(bCondition));	 \
		} \
		template<typename ActionClass, class... ActionArguments> \
		void ConditionalAction(bool bCondition, ActionArguments&&... Arguments) \
		{ \
			if (UE::FChangeTransactor::CanTransactChanges() && ShouldTransact(bCondition)) \
			{  \
				ChangeTransactor.AddTransactionChange<ActionClass>(Forward<ActionArguments>(Arguments)...); \
			} \
		} \
	protected: \
		UE::FChangeTransactor ChangeTransactor; \


#define CONDITIONAL_TRANSACTION(Text) FTransaction Transaction = ConditionalTransaction(Test, bShouldTransact);
#define CONDITIONAL_BRACKET(Text) FBracket Bracket = ConditionalBracket(Text, bShouldTransact);
#define CONDITIONAL_ACTION(ActionClass, ...) ConditionalAction<ActionClass>(bShouldTransact, __VA_ARGS__);

#undef LOCTEXT_NAMESPACE // "IAnimationDataController"

