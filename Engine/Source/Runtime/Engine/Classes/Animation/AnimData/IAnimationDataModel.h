// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AnimDataNotifications.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributeCurve.h"
#include "Features/IModularFeature.h"

class IAnimationDataController;

#include "IAnimationDataModel.generated.h"

namespace UE::Anim::DataModel
{
	/** Structure used to supply necessary animation (pose) evaluation information */
	struct FEvaluationContext
	{
		FEvaluationContext() = delete;
		FEvaluationContext(double InTime, FFrameRate InSampleRate, FName InRetargetSource, const TArray<FTransform>& InRetargetTransforms, EAnimInterpolationType InInterpolationType = EAnimInterpolationType::Linear)
			: SampleFrameRate(InSampleRate), SampleTime(InSampleRate.AsFrameTime(InTime)), RetargetSource(InRetargetSource), RetargetTransforms(InRetargetTransforms), InterpolationType(InInterpolationType)
		{		
		}

		FEvaluationContext(FFrameTime InTime, FFrameRate InSampleRate, FName InRetargetSource, const TArray<FTransform>& InRetargetTransforms, EAnimInterpolationType InInterpolationType = EAnimInterpolationType::Linear)
			: SampleFrameRate(InSampleRate), SampleTime(InTime), RetargetSource(InRetargetSource), RetargetTransforms(InRetargetTransforms), InterpolationType(InInterpolationType)
		{		
		}

		/** Sampling frame rate used to calculate SampleTime */
		const FFrameRate SampleFrameRate;
		/** Time at which the animation data should be evaluated */
		const FFrameTime SampleTime;
		/** (Source) Name used for retargeting */
		const FName RetargetSource;
		/** Per-bone pose to use as basis when retargeting */
		const TArray<FTransform>& RetargetTransforms;
		/** Type of interpolation to be used when evaluating animation data */
		const EAnimInterpolationType InterpolationType;
	};

	/** Modular feature allowing plugins to provide an implementation of IAnimationDataModel */
	class IAnimationDataModels : public IModularFeature
	{
	public:
		virtual ~IAnimationDataModels() = default;

		static FName GetModularFeatureName()
		{
			static FName FeatureName = FName("AnimationDataModels");
			return FeatureName;
		}

		/** Returns UClass (if possible) for the provided animation asset */
		virtual UClass* GetModelClass(UAnimSequenceBase* OwningAnimationAsset) const = 0;
		
		static ENGINE_API UClass* FindClassForAnimationAsset(UAnimSequenceBase* AnimSequenceBase);
	};
}

/**
* Structure encapsulating a single bone animation track.
*/
USTRUCT(BlueprintType)
struct FBoneAnimationTrack
{
	GENERATED_BODY()

	/** Internally stored data representing the animation bone data */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	FRawAnimSequenceTrack InternalTrackData;

	/** Index corresponding to the bone this track corresponds to within the target USkeleton */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	int32 BoneTreeIndex = INDEX_NONE;

	/** Name of the bone this track corresponds to */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	FName Name;
};

/**
* Structure encapsulating animated curve data. Currently only contains Float and Transform curves.
*/
USTRUCT(BlueprintType)
struct FAnimationCurveData
{
	GENERATED_BODY()

	/** Float-based animation curves */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	TArray<FFloatCurve>	FloatCurves;

	/** FTransform-based animation curves, used for animation layer editing */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	TArray<FTransformCurve>	TransformCurves;
};

/**
* Structure encapsulating animated (bone) attribute data.
*/
USTRUCT(BlueprintType)
struct FAnimatedBoneAttribute
{
	GENERATED_BODY()

	/** Identifier to reference this attribute by */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	FAnimationAttributeIdentifier Identifier;	

	/** Curve containing the (animated) attribute data */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Model")
	FAttributeCurve Curve;
};

struct FTrackToSkeletonMap;
struct FAnimationCurveIdentifier;

UINTERFACE(BlueprintType, meta=(CannotImplementInterfaceInBlueprint), MinimalAPI)
class UAnimationDataModel : public UInterface
{
	GENERATED_BODY()
};

class IAnimationDataModel
{		
public:
	GENERATED_BODY()

#if WITH_EDITOR
	// RAII helper to allow repopulating data model without warnings during reimporting
	struct FReimportScope
	{
		FReimportScope(IAnimationDataModel* InModel) : bPopulationFlagRef(InModel->GetPopulationFlag())
		{
			bOriginalValue = bPopulationFlagRef;
			if (bOriginalValue)
			{
				bPopulationFlagRef = false;
			}
		}

		~FReimportScope()
		{
			// Only reset back to true if that was the original state and currently is false			
			if (!bPopulationFlagRef && bOriginalValue)
			{
				bPopulationFlagRef = bOriginalValue;
			}
		}
	private:
		bool& bPopulationFlagRef;
		bool bOriginalValue;
	};
#endif // WITH_EDITOR

	/**
	* @return	Total length of play-able animation data 
	*/
	UFUNCTION(BlueprintCallable, Category=AnimationDataModel)
	virtual double GetPlayLength() const = 0;
	
	/**
	* @return	Total number of frames of animation data stored 
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetNumberOfFrames() const = 0;

	/**
	* @return	Total number of animation data keys stored 
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetNumberOfKeys() const = 0;

	/**
	* @return	Frame rate at which the animation data is key-ed 
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual FFrameRate GetFrameRate() const = 0;

	/**
	* @return	Array containing all bone animation tracks 
	*/
	UE_DEPRECATED(5.2, "GetBoneAnimationTracks has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual const TArray<FBoneAnimationTrack>& GetBoneAnimationTracks() const = 0;

	virtual FTransform EvaluateBoneTrackTransform(FName TrackName, const FFrameTime& FrameTime, const EAnimInterpolationType& Interpolation) const = 0;
	virtual FTransform GetBoneTrackTransform(FName TrackName, const FFrameNumber& FrameNumber) const = 0;
	virtual void GetBoneTrackTransforms(FName TrackName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& OutTransforms) const = 0;
	virtual void GetBoneTrackTransforms(FName TrackName, TArray<FTransform>& OutTransforms) const = 0;
	virtual void GetBoneTracksTransform(const TArray<FName>& TrackNames, const FFrameNumber& FrameNumber, TArray<FTransform>& OutTransforms) const = 0;
	
	/**
	* @return	Bone animation track for the provided index
	*/
	UE_DEPRECATED(5.2, "GetBoneTrackByIndex has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual const FBoneAnimationTrack& GetBoneTrackByIndex(int32 TrackIndex) const = 0;

	/**
	* @return	Bone animation track for the provided (bone) name
	*/
	UE_DEPRECATED(5.2, "GetBoneTrackByName has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual const FBoneAnimationTrack& GetBoneTrackByName(FName TrackName) const = 0;

	/**
	* @return	Internal track index for the provided bone animation track if found, otherwise returns INDEX_NONE 
	*/
	UE_DEPRECATED(5.2, "FindBoneTrackByIndex has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetBoneTrackIndex(const FBoneAnimationTrack& Track) const = 0;

	/**
	* @return	Internal track index for the provided (bone) name if found, otherwise returns INDEX_NONE 
	*/
	UE_DEPRECATED(5.2, "GetBoneTrackIndexByName has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetBoneTrackIndexByName(FName TrackName) const = 0;

	/**
	* @return	Whether or not the provided track index is valid 
	*/
	UE_DEPRECATED(5.2, "IsValidBoneTrackIndex has been deprecated")
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual bool IsValidBoneTrackIndex(int32 TrackIndex) const = 0;

	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual bool IsValidBoneTrackName(const FName& TrackName) const = 0;

	/**
	* @return	Total number of bone animation tracks
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetNumBoneTracks() const = 0;

	/**
	* Populates the provided array with all contained (bone) track names
	*
	* @param	OutNames	[out] Array containing all bone track names
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual void GetBoneTrackNames(TArray<FName>& OutNames) const = 0;

	/** Returns all contained curve animation data */
	virtual const FAnimationCurveData& GetCurveData() const = 0;

	/**
	* @return	Total number of stored FTransform curves
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetNumberOfTransformCurves() const = 0;

	/**
	* @return	Total number of stored float curves
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual int32 GetNumberOfFloatCurves() const = 0;

	/**
	* @return	Array containing all stored float curves 
	*/
	virtual const TArray<struct FFloatCurve>& GetFloatCurves() const = 0;

	/**
	* @return	Array containing all stored FTransform curves 
	*/
	virtual const TArray<struct FTransformCurve>& GetTransformCurves() const = 0;

	/**
	* @return	Curve ptr for the provided identifier if valid, otherwise returns a nullptr 
	*/
	virtual const FAnimCurveBase* FindCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Float Curve ptr for the provided identifier if valid, otherwise returns a nullptr
	*/
	virtual const FFloatCurve* FindFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Transform Curve ptr for the provided identifier if valid, otherwise returns a nullptr
	*/
	virtual const FTransformCurve* FindTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Rich curve ptr for the provided identifier if valid, otherwise returns a nullptr
	*/
	virtual const FRichCurve* FindRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Curve object for the provided identifier if valid
	*/
	virtual const FAnimCurveBase& GetCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Float Curve object for the provided identifier if valid
	*/
	virtual const FFloatCurve& GetFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Transform Curve object for the provided identifier if valid
	*/
	virtual const FTransformCurve& GetTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;

	/**
	* @return	Rich Curve object for the provided identifier if valid
	*/
	virtual const FRichCurve& GetRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const = 0;
		
	/**
	* @return	Animated (bone) attributes stored
	*/
	virtual TArrayView<const FAnimatedBoneAttribute> GetAttributes() const = 0;

	/**
	* @return	Number of animated (bone) attributes stored
	*/
	virtual int32 GetNumberOfAttributes() const = 0;

	/**
	* @return	Number of animated (bone) attributes stored for the specified bone index
	*/
	virtual int32 GetNumberOfAttributesForBoneIndex(const int32 BoneIndex) const = 0;

	/**
	* @return	All animated (bone) attributes stored for the specified bone name
	*/
	virtual void GetAttributesForBone(const FName& BoneName, TArray<const FAnimatedBoneAttribute*>& OutBoneAttributes) const = 0;

	/**
	* @return	Animated (bone) attribute object for the provided identifier if valid
	*/
	virtual const FAnimatedBoneAttribute& GetAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const = 0;

	/**
	* @return	Animated (bone) attribute ptr for the provided identifier if valid, otherwise returns a nullptr 
	*/
	virtual const FAnimatedBoneAttribute* FindAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const = 0;
		
	/**
	* @return	The outer UAnimSequence object if found, otherwise returns a nullptr 
	*/
	UFUNCTION(BlueprintCallable, Category = AnimationDataModel)
	virtual UAnimSequence* GetAnimationSequence() const = 0;

	/**
	* @return	Multicast delegate which is broadcasted to propagated changes to any internal data, see FAnimDataModelModifiedEvent and EAnimDataModelNotifyType
	*/
	virtual FAnimDataModelModifiedEvent& GetModifiedEvent() = 0;

	struct FGuidGenerationSettings
	{
		FGuidGenerationSettings() : bIncludeBoneData(1), bIncludeCurveData(1), bIncludeAttributeData(1), bIncludeTimingData(1) {}

		uint8 bIncludeBoneData : 1;
		uint8 bIncludeCurveData : 1;
		uint8 bIncludeAttributeData : 1;
		uint8 bIncludeTimingData : 1;
	};

	/**
	* @return	GUID representing a configured sub-set of the contained data and state 
	*
	* @param	InSettings		Settings used to determine which data should be included when generating the returned GUID
	*/
	virtual FGuid GenerateGuid(const FGuidGenerationSettings& InSettings) const = 0;

	/**
	* @return	GUID representing the contained data and state 
	*/
	virtual FGuid GenerateGuid() const { return GenerateGuid(FGuidGenerationSettings()); }

#if WITH_EDITOR
	/**
	* @return	String representing the data reported by GenerateGuid, used to debug indeterminism in GenerateGuid.
	*/
	virtual FString GenerateDebugStateString() const
	{
		return FString::Printf(TEXT("DebugStateString not implemented. Guid == %s"), *GenerateGuid().ToString());
	}
#endif

	/**
	* @return	IAnimationDataController instance used for mutating this model
	*/
	virtual TScriptInterface<IAnimationDataController> GetController() = 0;

	/**
	* Evaluates the contained Animation Data to populated an Animation Pose with (Bones, Curves and Attributes).
	*
	* @param	InOutPoseData		Container for the to-be-evaluated pose related data
	* @param	EvaluationContext	Context describing how the data should be evaluated (timing etc.)
	*/
#if WITH_EDITOR
	virtual void Evaluate(FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const = 0;
#endif // WITH_EDITOR

	virtual bool HasBeenPopulated() const = 0;
	virtual void IterateBoneKeys(const FName& BoneName, TFunction<bool(const FVector3f& Pos, const FQuat4f&, const FVector3f, const FFrameNumber&)> IterationFunction) const = 0;

	struct FEvaluationAndModificationLock
	{
		FEvaluationAndModificationLock(IAnimationDataModel& InModel) : Model(InModel) { InModel.LockEvaluationAndModification(); bLocked = true; }
		FEvaluationAndModificationLock(IAnimationDataModel& InModel, const TFunction<bool()>& SpinFunc) : Model(InModel)
		{
			while(SpinFunc())
			{
				bLocked = InModel.TryLockEvaluationAndModification();
				if (bLocked)
				{
					break;
				}
			}
		}
		~FEvaluationAndModificationLock()  { if(bLocked) {Model.UnlockEvaluationAndModification(); } }
	private:
		IAnimationDataModel& Model;
		bool bLocked = false;
	};
protected:
	virtual FAnimDataModelModifiedDynamicEvent& GetModifiedDynamicEvent() = 0;
	virtual void OnNotify(const EAnimDataModelNotifyType& NotifyType, const FAnimDataModelNotifPayload& Payload) = 0;

	virtual void LockEvaluationAndModification() const = 0;
	virtual bool TryLockEvaluationAndModification() const = 0;
	virtual void UnlockEvaluationAndModification() const = 0;
	virtual bool& GetPopulationFlag() = 0;
	
	struct FModelNotifier;
	virtual IAnimationDataModel::FModelNotifier& GetNotifier() = 0;

	struct FModelNotifier
	{
		virtual ~FModelNotifier() = default;	
		
		/**
		* Broadcasts a new EAnimDataModelNotifyType with the provided payload data alongside it.
		*
		* @param	NotifyType			Type of notify to broadcast
		* @param	PayloadData			Typed payload data
		*/
		template<typename T>
		void Notify(EAnimDataModelNotifyType NotifyType, const T& PayloadData)
		{
			UScriptStruct* TypeScriptStruct = T::StaticStruct();

			const FAnimDataModelNotifPayload Payload((int8*)&PayloadData, TypeScriptStruct);
			DataModel->OnNotify(NotifyType, Payload);
			
			ModifiedEvent.Broadcast(NotifyType, DataModel.GetInterface(), Payload);
			
			if (ModifiedEventDynamic.IsBound())
			{
				ModifiedEventDynamic.Broadcast(NotifyType, DataModel, Payload);
			}

			// Only regenerate transient data when not in a bracket, or at the end of one
			{
				if (NotifyType == EAnimDataModelNotifyType::BracketOpened)
				{
					++BracketCounter;
				}
				if (NotifyType == EAnimDataModelNotifyType::BracketClosed)
				{
					--BracketCounter;
				}

				check(BracketCounter >= 0);
			}
		}

		/**
		* Broadcasts a new EAnimDataModelNotifyType alongside of an empty payload.
		*
		* @param	NotifyType			Type of notify to broadcast
		*/
		void Notify(EAnimDataModelNotifyType NotifyType)
		{
			const FEmptyPayload EmptyPayload;
			Notify<FEmptyPayload>(NotifyType, EmptyPayload);
		}
	
		FModelNotifier(TScriptInterface<IAnimationDataModel> ModelInterface) :
			DataModel(ModelInterface),				
			ModifiedEvent(ModelInterface->GetModifiedEvent()),
			ModifiedEventDynamic(ModelInterface->GetModifiedDynamicEvent())
		{					
		}

		int32 GetBracketDepth() const
		{
			return BracketCounter;
		}

	protected:
		TScriptInterface<IAnimationDataModel> DataModel;
		FAnimDataModelModifiedEvent& ModifiedEvent; 
		FAnimDataModelModifiedDynamicEvent& ModifiedEventDynamic;
		int32 BracketCounter = 0;
	};
};


