// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimEnums.generated.h"

UENUM()
/** Root Bone Lock options when extracting Root Motion. */
namespace ERootMotionRootLock
{
	enum Type : int
	{
		/** Use reference pose root bone position. */
		RefPose,

		/** Use root bone position on first frame of animation. */
		AnimFirstFrame,

		/** FTransform::Identity. */
		Zero
	};
}

UENUM()
namespace ERootMotionMode
{
	enum Type : int
	{
		/** Leave root motion in animation. */
		NoRootMotionExtraction,

		/** Extract root motion but do not apply it. */
		IgnoreRootMotion,

		/** Root motion is taken from all animations contributing to the final pose, not suitable for network multiplayer setups. */
		RootMotionFromEverything,

		/** Root motion is only taken from montages, suitable for network multiplayer setups. */
		RootMotionFromMontagesOnly,
	};
}

/**
* For an additive animation, indicates what the animation is relative to.
*/
UENUM()
enum EAdditiveBasePoseType : int
{
	/** Will be deprecated. */
	ABPT_None UMETA(DisplayName = "None"),
	/** Use the Skeleton's ref pose as base. */
	ABPT_RefPose UMETA(DisplayName = "Skeleton Reference Pose"),
	/** Use a whole animation as a base pose. BasePoseSeq must be set. */
	ABPT_AnimScaled UMETA(DisplayName = "Selected animation scaled"),
	/** Use one frame of an animation as a base pose. BasePoseSeq and RefFrameIndex must be set (RefFrameIndex will be clamped). */
	ABPT_AnimFrame UMETA(DisplayName = "Selected animation frame"),
	/** Use one frame of this animation. RefFrameIndex must be set (RefFrameIndex will be clamped). */
	ABPT_LocalAnimFrame UMETA(DisplayName = "Frame from this animation"),
	ABPT_MAX,
};


/**
* Indicates animation data compression format.
*/
UENUM()
enum AnimationCompressionFormat : int
{
	ACF_None,
	ACF_Float96NoW,
	ACF_Fixed48NoW,
	ACF_IntervalFixed32NoW,
	ACF_Fixed32NoW,
	ACF_Float32NoW,
	ACF_Identity,
	ACF_MAX UMETA(Hidden),
};