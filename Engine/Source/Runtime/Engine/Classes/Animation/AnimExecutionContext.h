// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimExecutionContext.generated.h"

struct FAnimationBaseContext;
struct FAnimationInitializeContext;
struct FAnimationUpdateContext;
struct FPoseContext;
struct FComponentSpacePoseContext;

// The result of an anim node context conversion 
UENUM(BlueprintType)
enum class EAnimExecutionContextConversionResult : uint8
{
	Succeeded = 1,
	Failed = 0,
};

// Context used to expose anim graph execution to BP function libraries
USTRUCT(BlueprintType)
struct FAnimExecutionContext
{
	GENERATED_BODY()

public:
	// Internal data, weakly referenced
	struct FData
	{
	public:
		FData(const FAnimationBaseContext& InContext);
		
		FData(const FAnimationInitializeContext& InContext);

		FData(const FAnimationUpdateContext& InContext);

		FData(FPoseContext& InContext);

		FData(FComponentSpacePoseContext& InContext);

	private:
		friend struct FAnimExecutionContext;
		friend struct FAnimInitializationContext;
		friend struct FAnimUpdateContext;
		friend struct FAnimPoseContext;
		friend struct FAnimComponentSpacePoseContext;
		
		enum class EContextType
		{
			None,
			Base,
			Initialize,
			Update,
			Pose,
			ComponentSpacePose,
		};
	
		// The context used when executing this node, e.g. FAnimationUpdateContext, FPoseContext etc.
		FAnimationBaseContext* Context = nullptr;

		// The phase we are in
		EContextType ContextType = EContextType::None;	
	};
	
	FAnimExecutionContext() = default;
	FAnimExecutionContext(const FAnimExecutionContext& InOther) = default;
	FAnimExecutionContext& operator=(const FAnimExecutionContext& InOther) = default;
	
	FAnimExecutionContext(TWeakPtr<FData> InData)
		: Data(InData)
	{}

	// Is this a valid context? 
	bool IsValid() const
	{
		return Data.IsValid();
	}

	// Convert to a derived type
	template<typename OtherContextType>
	static OtherContextType ConvertToType(const FAnimExecutionContext& InContext, EAnimExecutionContextConversionResult& OutResult)
	{
		static_assert(TIsDerivedFrom<OtherContextType, FAnimExecutionContext>::IsDerived, "Argument OtherContextType must derive from FAnimExecutionContext");
		
		if(TSharedPtr<FData> PinnedData = InContext.Data.Pin())
		{
			if(OtherContextType::InternalContextTypeId == PinnedData->ContextType)
			{
				OutResult = EAnimExecutionContextConversionResult::Succeeded;
				
				OtherContextType Context;
				Context.Data = InContext.Data;
				return Context;
			}
		}

		OutResult = EAnimExecutionContextConversionResult::Failed;
		
		return OtherContextType();
	}

	// Access internal context. Will return nullptr if invalid
	FAnimationBaseContext* GetBaseContext() const
	{
		if(TSharedPtr<FData> PinnedData = Data.Pin())
		{
			return PinnedData->Context;
		}

		return nullptr;
	}

protected:
	// Access internal context. Will return nullptr if invalid or an incorrect type is requested
	template<typename OtherContextType, typename InternalContextType>
	InternalContextType* GetInternalContext() const
	{
		if(TSharedPtr<FData> PinnedData = Data.Pin())
		{
			if(OtherContextType::InternalContextTypeId == PinnedData->ContextType)
			{
				return static_cast<InternalContextType*>(PinnedData->Context);
			}
		}

		return nullptr;
	}
	
protected:
	// Internal data
	TWeakPtr<FData> Data; 
};

USTRUCT()
struct FAnimInitializationContext : public FAnimExecutionContext
{
	GENERATED_BODY()

public:
	static const FData::EContextType InternalContextTypeId = FData::EContextType::Initialize;

	FAnimInitializationContext() = default;
	
	FAnimInitializationContext(TWeakPtr<FData> InData)
		: FAnimExecutionContext(InData)
	{
		check(InData.IsValid());
		check(InData.Pin()->ContextType == InternalContextTypeId);
	}

	ENGINE_API FAnimationInitializeContext* GetContext() const;
};

USTRUCT()
struct FAnimUpdateContext : public FAnimExecutionContext
{
	GENERATED_BODY()

public:
	static const FData::EContextType InternalContextTypeId = FData::EContextType::Update;

	FAnimUpdateContext() = default;
	
	FAnimUpdateContext(TWeakPtr<FData> InData)
		: FAnimExecutionContext(InData)
	{
		check(InData.IsValid());
		check(InData.Pin()->ContextType == InternalContextTypeId);
	}

	ENGINE_API FAnimationUpdateContext* GetContext() const;
};

USTRUCT()
struct FAnimPoseContext : public FAnimExecutionContext
{
	GENERATED_BODY()

public:
	static const FData::EContextType InternalContextTypeId = FData::EContextType::Pose;

	FAnimPoseContext() = default;
	
	FAnimPoseContext(TWeakPtr<FData> InData)
		: FAnimExecutionContext(InData)
	{
		check(InData.IsValid());
		check(InData.Pin()->ContextType == InternalContextTypeId);
	}

	ENGINE_API FPoseContext* GetContext() const;
};

USTRUCT()
struct FAnimComponentSpacePoseContext : public FAnimExecutionContext
{
	GENERATED_BODY()

public:
	static const FData::EContextType InternalContextTypeId = FData::EContextType::ComponentSpacePose;

	FAnimComponentSpacePoseContext() = default;
	
	FAnimComponentSpacePoseContext(TWeakPtr<FData> InData)
		: FAnimExecutionContext(InData)
	{
		check(InData.IsValid());
		check(InData.Pin()->ContextType == InternalContextTypeId);
	}

	ENGINE_API FComponentSpacePoseContext* GetContext() const;
};
