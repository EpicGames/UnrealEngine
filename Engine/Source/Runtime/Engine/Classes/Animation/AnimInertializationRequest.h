// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h" // Required for EAlphaBlendOption

#include "AnimInertializationRequest.generated.h"

class UBlendProfile;
class UCurveFloat;

USTRUCT()
struct FInertializationRequest
{
	GENERATED_BODY()

	ENGINE_API FInertializationRequest();
	ENGINE_API FInertializationRequest(float InDuration, const UBlendProfile* InBlendProfile);

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FInertializationRequest() = default;
	FInertializationRequest(const FInertializationRequest&) = default;
	FInertializationRequest(FInertializationRequest&&) = default;
	FInertializationRequest& operator=(const FInertializationRequest&) = default;
	FInertializationRequest& operator=(FInertializationRequest&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ENGINE_API void Clear();

	// Comparison operator used to test for equality in the array of animation requests to that
	// only unique requests are added. This does not take into account the properties that are 
	// used only for debugging and only used when ANIM_TRACE_ENABLED
	friend bool operator==(const FInertializationRequest& A, const FInertializationRequest& B)
	{
		return
			(A.Duration == B.Duration) &&
			(A.BlendProfile == B.BlendProfile) &&
			(A.bUseBlendMode == B.bUseBlendMode) &&
			(A.BlendMode == B.BlendMode) &&
			(A.CustomBlendCurve == B.CustomBlendCurve) &&
			(A.Tag == B.Tag);
	}

	friend bool operator!=(const FInertializationRequest& A, const FInertializationRequest& B)
	{
		return !(A == B);
	}

	// Blend duration of the inertialization request.
	UPROPERTY(Transient)
	float Duration = -1.0f;

	// Blend profile to control per-joint blend times.
	UPROPERTY(Transient)
	TObjectPtr<const UBlendProfile> BlendProfile = nullptr;

	// If to use the provided blend mode.
	UPROPERTY(Transient)
	bool bUseBlendMode = false;

	// Blend mode to use.
	UPROPERTY(Transient)
	EAlphaBlendOption BlendMode = EAlphaBlendOption::Linear;

	// Custom blend curve to use when use of the blend mode is active.
	UPROPERTY(Transient)
	TObjectPtr<UCurveFloat> CustomBlendCurve = nullptr;

	// Inertialization / Dead blend node tag to force a particular node to handle the request.
	UPROPERTY(Transient)
	FName Tag = NAME_None;

// if ANIM_TRACE_ENABLED - these properties are only used for debugging when ANIM_TRACE_ENABLED == 1
	
	UE_DEPRECATED(5.4, "Use DescriptionString instead.")
	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Use DescriptionString instead."))
	FText Description_DEPRECATED;

	// Description of the request
	UPROPERTY(Transient)
	FString DescriptionString;

	// Node id from which this request was made.
	UPROPERTY(Transient)
	int32 NodeId = INDEX_NONE;

	// Anim instance from which this request was made.
	UPROPERTY(Transient)
	TObjectPtr<UObject> AnimInstance = nullptr;

// endif ANIM_TRACE_ENABLED
};
