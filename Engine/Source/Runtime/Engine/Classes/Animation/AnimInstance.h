// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/AnimSubsystemInstance.h"
#include "Animation/AnimSync.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimInertializationRequest.h"
#include "AnimInstance.generated.h"

// Disable debugging information for shipping and test builds.
#define ENABLE_ANIM_DRAW_DEBUG (1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

// Post Compile Validation requires WITH_EDITOR
#define ANIMINST_PostCompileValidation WITH_EDITOR

struct FDisplayDebugManager;
class FDebugDisplayInfo;
class IAnimClassInterface;
class UAnimInstance;
class UCanvas;
class USkeletalMeshComponent;
class USkeleton;
struct FAlphaBlend;
struct FAlphaBlendArgs;
struct FAnimInstanceProxy;
struct FAnimMontageInstance;
struct FAnimationEvaluationContext;
struct FAnimNode_AssetPlayerBase;
struct FAnimNode_AssetPlayerRelevancyBase;
struct FAnimNode_StateMachine;
struct FAnimNode_LinkedInputPose;
struct FBakedAnimationStateMachine;
struct FCompactPose;
struct FCurveEvaluationOption;
class FCompilerResultsLog;
struct FBlendedHeapCurve;
struct FBoneContainer;
struct FSmartNameMapping;
struct FAnimNode_LinkedAnimLayer;
struct FNodeDebugData;
enum class ETransitionRequestQueueMode : uint8;
enum class ETransitionRequestOverwriteMode : uint8;
class UAnimMontage;

typedef TArray<FTransform> FTransformArrayA2;

namespace UE::Anim
{
	struct FHeapAttributeContainer;
	// DEPRECATED use FInertializationRequest instead
	using FSlotInertializationRequest = TPair<float, const UBlendProfile*>;
	struct FCurveFilterSettings;
}	// namespace UE::Anim

struct FParallelEvaluationData
{
	FBlendedHeapCurve& OutCurve;
	FCompactPose& OutPose;
	UE::Anim::FHeapAttributeContainer& OutAttributes;
};

UENUM()
enum class EMontagePlayReturnType : uint8
{
	//Return value is the length of the montage (in seconds)
	MontageLength,
	//Return value is the play duration of the montage (length / play rate, in seconds)
	Duration,
};

DECLARE_DELEGATE_OneParam(FOnMontageStarted, UAnimMontage*)
DECLARE_DELEGATE_TwoParams(FOnMontageEnded, UAnimMontage*, bool /*bInterrupted*/)
DECLARE_DELEGATE_TwoParams(FOnMontageBlendingOutStarted, UAnimMontage*, bool /*bInterrupted*/)
DECLARE_DELEGATE_OneParam(FOnMontageBlendedInEnded, UAnimMontage*)
DECLARE_DELEGATE_ThreeParams(FOnMontageSectionChanged, UAnimMontage*, FName /*SectionName*/ , bool /*bLooped*/)

// Experimental 5.8, FComputeDeltaFrameTime is experimental, and may change in the future.
DECLARE_DELEGATE_RetVal_ThreeParams(float, FComputeDeltaFrameTime, int32 /*InstanceId*/, float /*CurrentTime*/, float /*DeltaTime*/)

/**
* Delegate for when Montage is started
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMontageStartedMCDelegate, UAnimMontage*, Montage);

/**
* Delegate for when Montage is completed, whether interrupted or finished
* Weight of this montage is 0.f, so it stops contributing to output pose
*
* bInterrupted = true if it was not property finished
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMontageEndedMCDelegate, UAnimMontage*, Montage, bool, bInterrupted);

/** Delegate for when all montage instances have ended. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAllMontageInstancesEndedMCDelegate);

/**
* Delegate for when Montage started to blend out, whether interrupted or finished
* DesiredWeight of this montage becomes 0.f, but this still contributes to the output pose
*
* bInterrupted = true if it was not property finished
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMontageBlendingOutStartedMCDelegate, UAnimMontage*, Montage, bool, bInterrupted);

/** Delegate for when Montage finished blending in */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMontageBlendedInEndedMCDelegate, UAnimMontage*, Montage);

/** Delegate for when Montage changes sections */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnMontageSectionChangedMCDelegate, UAnimMontage*, Montage, FName, SectionName, bool, bLooped);

/** Delegate that native code can hook to provide additional transition logic */
DECLARE_DELEGATE_RetVal(bool, FCanTakeTransition);

/** Delegate that native code can hook into to handle state entry/exit */
DECLARE_DELEGATE_ThreeParams(FOnGraphStateChanged, const struct FAnimNode_StateMachine& /*Machine*/, int32 /*PrevStateIndex*/, int32 /*NextStateIndex*/);

/** Delegate that allows users to insert custom animation curve values - for now, it's only single, not sure how to make this to multi delegate and retrieve value sequentially, so */
DECLARE_DELEGATE_OneParam(FOnAddCustomAnimationCurves, UAnimInstance*)

/** Delegate called by 'PlayMontageNotify' and 'PlayMontageNotifyWindow' **/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FPlayMontageAnimNotifyDelegate, FName, NotifyName, const FBranchingPointNotifyPayload&, BranchingPointPayload);


USTRUCT()
struct FA2Pose
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FTransform> Bones;

	FA2Pose() {}
};

/** Component space poses. */
USTRUCT()
struct FA2CSPose : public FA2Pose
{
	GENERATED_USTRUCT_BODY()

private:
	/** Pointer to current BoneContainer. */
	const struct FBoneContainer* BoneContainer;

	/** Once evaluated to be mesh space, this flag will be set. */
	UPROPERTY()
	TArray<uint8> ComponentSpaceFlags;

public:
	FA2CSPose()
		: BoneContainer(NULL)
	{
	}

	/** Constructor - needs LocalPoses. */
	ENGINE_API void AllocateLocalPoses(const FBoneContainer& InBoneContainer, const FA2Pose & LocalPose);

	/** Constructor - needs LocalPoses. */
	ENGINE_API void AllocateLocalPoses(const FBoneContainer& InBoneContainer, const FTransformArrayA2 & LocalBones);

	/** Returns if this struct is valid. */
	ENGINE_API bool IsValid() const;

	/** Get parent bone index for given bone index. */
	ENGINE_API int32 GetParentBoneIndex(const int32 BoneIndex) const;

	/** Returns local transform for the bone index. **/
	ENGINE_API FTransform GetLocalSpaceTransform(int32 BoneIndex);

	/** Do not access Bones array directly; use this instead. This will fill up gradually mesh space bases. */
	ENGINE_API FTransform GetComponentSpaceTransform(int32 BoneIndex);

	/** convert to local poses **/
	ENGINE_API void ConvertToLocalPoses(FA2Pose & LocalPoses) const;

private:
	/** Calculate all transform till parent **/
	ENGINE_API void CalculateComponentSpaceTransform(int32 Index);
	ENGINE_API void SetComponentSpaceTransform(int32 Index, const FTransform& NewTransform);

	/**
	 * Convert Bone to Local Space.
	 */
	ENGINE_API void ConvertBoneToLocalSpace(int32 BoneIndex);


	ENGINE_API void SetLocalSpaceTransform(int32 Index, const FTransform& NewTransform);

	// This is not really best way to protect SetComponentSpaceTransform, but we'd like to make sure that isn't called by anywhere else.
	friend class FAnimationRuntime;
};

/** Helper struct to store a Queued Montage BlendingOut event. */
struct FQueuedMontageBlendingOutEvent
{
	TObjectPtr<class UAnimMontage> Montage;
	bool bInterrupted;
	FOnMontageBlendingOutStarted Delegate;

	FQueuedMontageBlendingOutEvent()
		: bInterrupted(false)
	{}

	FQueuedMontageBlendingOutEvent(class UAnimMontage* InMontage, bool InbInterrupted, FOnMontageBlendingOutStarted InDelegate)
		: Montage(InMontage)
		, bInterrupted(InbInterrupted)
		, Delegate(InDelegate)
	{}
};

struct FQueuedMontageBlendedInEvent
{
	TObjectPtr<class UAnimMontage> Montage;
	FOnMontageBlendedInEnded Delegate;

	FQueuedMontageBlendedInEvent(class UAnimMontage* InMontage, FOnMontageBlendedInEnded InDelegate)
		: Montage(InMontage)
		, Delegate(InDelegate)
	{}
};

/** Helper struct to store a Queued Montage Ended event. */
struct FQueuedMontageEndedEvent
{
	TObjectPtr<class UAnimMontage> Montage;
	int32 MontageInstanceID;
	bool bInterrupted;
	FOnMontageEnded Delegate;

	FQueuedMontageEndedEvent()
		: MontageInstanceID(INDEX_NONE)
		, bInterrupted(false)
	{}

	FQueuedMontageEndedEvent(class UAnimMontage* InMontage, int32 InInstanceID, bool InbInterrupted, FOnMontageEnded InDelegate)
		: Montage(InMontage)
		, MontageInstanceID(InInstanceID)
		, bInterrupted(InbInterrupted)
		, Delegate(InDelegate)
	{}
};

/** Helper struct to store a Queued Montage Ended event. */
struct FQueuedMontageSectionChangedEvent
{
	TObjectPtr<class UAnimMontage> Montage;
	int32 MontageInstanceID;
	FName SectionName;
	bool bLooped;
	FOnMontageSectionChanged Delegate;

	FQueuedMontageSectionChangedEvent()
		: Montage(nullptr)
		, MontageInstanceID(INDEX_NONE)
		, SectionName(NAME_None)
		, bLooped(false)
	{}

	FQueuedMontageSectionChangedEvent(class UAnimMontage* InMontage, int32 InInstanceID, FName SectionName, bool InbLooped, FOnMontageSectionChanged InDelegate)
		: Montage(InMontage)
		, MontageInstanceID(InInstanceID)
		, SectionName(SectionName)
		, bLooped(InbLooped)
		, Delegate(InDelegate)
	{}
};

/** Binding allowing native transition rule evaluation */
struct FNativeTransitionBinding
{
	/** State machine to bind to */
	FName MachineName;

	/** Previous state the transition comes from */
	FName PreviousStateName;

	/** Next state the transition goes to */
	FName NextStateName;

	/** Delegate to use when checking transition */
	FCanTakeTransition NativeTransitionDelegate;

#if WITH_EDITORONLY_DATA
	/** Name of this transition rule */
	FName TransitionName;
#endif

	FNativeTransitionBinding(const FName& InMachineName, const FName& InPreviousStateName, const FName& InNextStateName, const FCanTakeTransition& InNativeTransitionDelegate, const FName& InTransitionName = NAME_None)
		: MachineName(InMachineName)
		, PreviousStateName(InPreviousStateName)
		, NextStateName(InNextStateName)
		, NativeTransitionDelegate(InNativeTransitionDelegate)
#if WITH_EDITORONLY_DATA
		, TransitionName(InTransitionName)
#endif
	{
	}
};

/** Binding allowing native notification of state changes */
struct FNativeStateBinding
{
	/** State machine to bind to */
	FName MachineName;

	/** State to bind to */
	FName StateName;

	/** Delegate to use when checking transition */
	FOnGraphStateChanged NativeStateDelegate;

#if WITH_EDITORONLY_DATA
	/** Name of this binding */
	FName BindingName;
#endif

	FNativeStateBinding(const FName& InMachineName, const FName& InStateName, const FOnGraphStateChanged& InNativeStateDelegate, const FName& InBindingName = NAME_None)
		: MachineName(InMachineName)
		, StateName(InStateName)
		, NativeStateDelegate(InNativeStateDelegate)
#if WITH_EDITORONLY_DATA
		, BindingName(InBindingName)
#endif
	{
	}
};

/** Tracks state of active slot nodes in the graph */
struct FMontageActiveSlotTracker
{
	/** Local weight of Montages being played (local to the slot node) */
	float MontageLocalWeight;

	/** Global weight of this slot node */
	float NodeGlobalWeight;

	//Is the montage slot part of the active graph this tick
	bool  bIsRelevantThisTick;

	//Was the montage slot part of the active graph last tick
	bool  bWasRelevantOnPreviousTick;

	FMontageActiveSlotTracker()
		: MontageLocalWeight(0.f)
		, NodeGlobalWeight(0.f)
		, bIsRelevantThisTick(false)
		, bWasRelevantOnPreviousTick(false) 
	{}
};

UCLASS(transient, Blueprintable, hideCategories=AnimInstance, BlueprintType, Within=SkeletalMeshComponent, MinimalAPI)
class UAnimInstance : public UObject
{
	GENERATED_UCLASS_BODY()

	typedef FAnimInstanceProxy ProxyType;

	ENGINE_API ~UAnimInstance();

	/** This is used to extract animation. If Mesh exists, this will be overwritten by Mesh->Skeleton */
	UPROPERTY(transient)
	TObjectPtr<USkeleton> CurrentSkeleton;

	// Sets where this blueprint pulls Root Motion from
	UPROPERTY(Category = RootMotion, EditDefaultsOnly)
	TEnumAsByte<ERootMotionMode::Type> RootMotionMode;

	/**
	 * Allows this anim instance to update its native update, blend tree, montages and asset players on
	 * a worker thread. This flag is propagated from the UAnimBlueprint to this instance by the compiler.
	 * The compiler will attempt to pick up any issues that may occur with threaded update.
	 * For updates to run in multiple threads both this flag and the project setting "Allow Multi Threaded 
	 * Animation Update" should be set.
	 */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	uint8 bUseMultiThreadedAnimationUpdate : 1;

	/** If this AnimInstance has nodes using 'CopyPoseFromMesh' this will be true. */
	UPROPERTY(meta = (BlueprintCompilerGeneratedDefaults))
	uint8 bUsingCopyPoseFromMesh : 1;

	/** Flag to check back on the game thread that indicates we need to run PostUpdateAnimation() in the post-eval call */
	UE_DEPRECATED(5.7, "Public access to bNeedsUpdate is deprecated, and it will become private in a future release. Please update code to use the NeedsUpdate() getter instead.")
	uint8 bNeedsUpdate : 1;

	/** Flag to check if created by LinkedAnimGraph in ReinitializeLinkedAnimInstance */
	uint8 bCreatedByLinkedAnimGraph : 1;

	/** Whether to process notifies from any linked anim instances */
	UPROPERTY(EditDefaultsOnly, Category = Notifies)
	uint8 bReceiveNotifiesFromLinkedInstances : 1;

	/** Whether to propagate notifies to any linked anim instances */
	UPROPERTY(EditDefaultsOnly, Category = Notifies)
	uint8 bPropagateNotifiesToLinkedInstances : 1;

	/** If true, linked instances will use the main instance's montage data. (i.e. playing a montage on a main instance will play it on the linked layer too.) */
	UPROPERTY(EditDefaultsOnly, Category = "Montage")
	uint8 bUseMainInstanceMontageEvaluationData: 1;

private:
	// Should UpdateAnimation be called
	uint8 bUpdateAnimationEnabled : 1;

	/** True when Montages are being ticked, and Montage Events should be queued. 
	 * When Montage are being ticked, we queue AnimNotifies and Events. We trigger notifies first, then Montage events. */
	UPROPERTY(Transient)
	uint8 bQueueMontageEvents : 1;

	/** Flag used to query if the animation instance has been uninitialized via UninitializeAnimation() call. */
	uint8 bUninitialized : 1;
	
#if DO_CHECK
	/** Guard flag used for checking whether we are in user callbacks for initialization */
	uint8 bInitializing : 1;

	/** Used to guard against recursive calls to UpdateAnimation */
	uint8 bPostUpdatingAnimation : 1;

	/** Used to guard against recursive calls to UpdateAnimation */
	uint8 bUpdatingAnimation : 1;
#endif

	/** Used to prevent firing notifies queued from the anim graph if we have already done the animation update this tick (non-autonomous). */
	uint8 bHasDispatchedAnimGraphNotifiesThisTick : 1;

	/** Used to prevent firing notifies queued from montages if we have already done the montage animation update this tick (non-autonomous). */
	uint8 bHasDispatchedMontageNotifiesThisTick : 1;
	
public:

	// @todo document
	ENGINE_API void MakeMontageTickRecord(FAnimTickRecord& TickRecord, class UAnimMontage* Montage, float MoveDelta, float Weight, TArray<FPassedMarker>& MarkersPassedThisTick, FMarkerTickRecord& MarkerTickRecord);

	/** Get global weight in AnimGraph for this slot node.
	* Note: this is the weight of the node, not the weight of any potential montage it is playing. */
	ENGINE_API float GetSlotNodeGlobalWeight(const FName& SlotNodeName) const;

	// Should Extract Root Motion or not. Return true if we do. 
	bool ShouldExtractRootMotion() const { return RootMotionMode == ERootMotionMode::RootMotionFromEverything || RootMotionMode == ERootMotionMode::IgnoreRootMotion; }

	/** Get Global weight of any montages this slot node is playing.
	* If this slot is not currently playing a montage, it will return 0. */
	ENGINE_API float GetSlotMontageGlobalWeight(const FName& SlotNodeName) const;

	/** Get local weight of any montages this slot node is playing.
	* If this slot is not currently playing a montage, it will return 0.
	* This is double buffered, will return last frame data if called from Update or Evaluate. */
	ENGINE_API float GetSlotMontageLocalWeight(const FName& SlotNodeName) const;

	/** Get local weight of any montages this slot is playing.
	* If this slot is not current playing a montage, it will return 0.
	* This will return up to date data if called during Update or Evaluate. */
	ENGINE_API float CalcSlotMontageLocalWeight(const FName& SlotNodeName) const;

	// kismet event functions

	UFUNCTION(BlueprintCallable, Category = "Animation") 
	ENGINE_API virtual APawn* TryGetPawnOwner() const;

	/** 
	 * Takes a snapshot of the current skeletal mesh component pose & saves it internally.
	 * This snapshot can then be retrieved by name in the animation blueprint for blending. 
	 * The snapshot is taken at the current LOD, so if for example you took the snapshot at LOD1 and then used it at LOD0 any bones not in LOD1 will use the reference pose 
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Pose")
	ENGINE_API virtual void SavePoseSnapshot(FName SnapshotName);

	/** Add an empty pose snapshot to the internal snapshot cache (or recycle an existing pose snapshot if the name is already in use) */
	ENGINE_API FPoseSnapshot& AddPoseSnapshot(FName SnapshotName);

	/** Remove a previously saved pose snapshot from the internal snapshot cache */
	UFUNCTION(BlueprintCallable, Category = "Animation|Pose")
	ENGINE_API void RemovePoseSnapshot(FName SnapshotName);

	/** Get a cached pose snapshot by name */
	ENGINE_API const FPoseSnapshot* GetPoseSnapshot(FName SnapshotName) const;

	/**
	 * Takes a snapshot of the current skeletal mesh component pose and saves it to the specified snapshot.
	 * The snapshot is taken at the current LOD, so if for example you took the snapshot at LOD1 
	 * and then used it at LOD0 any bones not in LOD1 will use the reference pose 
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Pose")
	ENGINE_API virtual void SnapshotPose(UPARAM(ref) FPoseSnapshot& Snapshot);
	
	/** Get the sync group we are currently reading from */
	ENGINE_API const TMap<FName, FAnimGroupInstance>& GetSyncGroupMapRead() const;

	/** Get the ungrouped active player we are currently reading from */
	ENGINE_API const TArray<FAnimTickRecord>& GetUngroupedActivePlayersRead();

	/** Get the current value of all animation curves **/
	ENGINE_API const TMap<FName, float>& GetAnimationCurves(EAnimCurveType InCurveType) const;

	/** Gather debug data from this instance proxy and the blend tree for display */
	ENGINE_API void GatherDebugData(FNodeDebugData& DebugData);

	// Can this animation instance run Update or Evaluation work in parallel
	virtual bool CanRunParallelWork() const { return true; }

	// Are we being evaluated on a worker thread
	ENGINE_API bool IsRunningParallelEvaluation() const;

	// Can does this anim instance need an update (parallel or not)?
	ENGINE_API bool NeedsUpdate() const;

	// Get whether we have initialized our animation instance or not. This will return false if UninitializeAnimation() has been called.
	ENGINE_API bool IsInitialized() const;
	
	/** Get whether to process notifies from any linked anim instances */
	UFUNCTION(BlueprintPure, Category = "Animation|Notifies")
	bool GetReceiveNotifiesFromLinkedInstances() const { return bReceiveNotifiesFromLinkedInstances; }

	/** Set whether to process notifies from any linked anim instances */
	UFUNCTION(BlueprintCallable, Category = "Animation|Notifies")
	void SetReceiveNotifiesFromLinkedInstances(bool bSet) { bReceiveNotifiesFromLinkedInstances = bSet; }

	/** Get whether to propagate notifies to any linked anim instances */
	UFUNCTION(BlueprintPure, Category = "Animation|Notifies")
	bool GetPropagateNotifiesToLinkedInstances() const { return bPropagateNotifiesToLinkedInstances; }

	/** Set whether to propagate notifies to any linked anim instances */
	UFUNCTION(BlueprintCallable, Category = "Animation|Notifies")
	void SetPropagateNotifiesToLinkedInstances(bool bSet) { bPropagateNotifiesToLinkedInstances = bSet; }

	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs|Montage")
	bool IsUsingMainInstanceMontageEvaluationData() const { return bUseMainInstanceMontageEvaluationData; }

	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs|Montage")
	void SetUseMainInstanceMontageEvaluationData(bool bSet) { bUseMainInstanceMontageEvaluationData = bSet; }

private:
	// Does this anim instance need immediate update (rather than parallel)?
	ENGINE_API bool NeedsImmediateUpdate(float DeltaSeconds, bool bNeedsValidRootMotion) const;

public:
	/** Returns the owning actor of this AnimInstance */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API AActor* GetOwningActor() const;
	
	// Returns the skeletal mesh component that has created this AnimInstance
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API USkeletalMeshComponent* GetOwningComponent() const;

	/** Get the 'main' anim instance, i.e. the one that is hosted on the skeletal mesh component */
	UFUNCTION(BlueprintCallable, Category = "Animation", meta = (DisplayName="Get Main Anim Instance", BlueprintThreadSafe))
	ENGINE_API UAnimInstance* Blueprint_GetMainAnimInstance() const;

public:

	/** Executed when the Animation is initialized */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void BlueprintInitializeAnimation();

	/** Executed when the Animation is updated */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void BlueprintUpdateAnimation(float DeltaTimeX);

	/** Executed after the Animation is evaluated */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void BlueprintPostEvaluateAnimation();

	/** Executed when begin play is called on the owning component */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void BlueprintBeginPlay();

	/** Executed when the all Linked Animation Layers are initialized */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void BlueprintLinkedAnimationLayersInitialized();

	/** Executed when the Animation Blueprint is updated on a worker thread, just prior to graph update */
	UFUNCTION(BlueprintImplementableEvent, meta=(BlueprintThreadSafe))
	ENGINE_API void BlueprintThreadSafeUpdateAnimation(float DeltaTime);

	ENGINE_API bool CanTransitionSignature() const;
	
	/*********************************************************************************************
	* SlotAnimation
	********************************************************************************************* */
public:

	/** Play normal animation asset on the slot node by creating a dynamic UAnimMontage. You can only play one asset (whether montage or animsequence) at a time per SlotGroup. */
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API UAnimMontage* PlaySlotAnimationAsDynamicMontage(UAnimSequenceBase* Asset, FName SlotNodeName, float BlendInTime = 0.25f, float BlendOutTime = 0.25f, float InPlayRate = 1.f, int32 LoopCount = 1, float BlendOutTriggerTime = -1.f, float InTimeToStartMontageAt = 0.f);

	/** Play normal animation asset on the slot node by creating a dynamic UAnimMontage with blend in arguments. You can only play one asset (whether montage or animsequence) at a time per SlotGroup. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API UAnimMontage* PlaySlotAnimationAsDynamicMontage_WithBlendArgs(UAnimSequenceBase* Asset, FName SlotNodeName, const FAlphaBlendArgs& BlendIn, const FAlphaBlendArgs& BlendOut, float InPlayRate = 1.f, int32 LoopCount = 1, float BlendOutTriggerTime = -1.f, float InTimeToStartMontageAt = 0.f);

	/** Play normal animation asset on the slot node by creating a dynamic UAnimMontage with blend in settings. You can only play one asset (whether montage or animsequence) at a time per SlotGroup. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API UAnimMontage* PlaySlotAnimationAsDynamicMontage_WithBlendSettings(UAnimSequenceBase* Asset, FName SlotNodeName, const FMontageBlendSettings& BlendInSettings, const FMontageBlendSettings& BlendOutSettings, float InPlayRate = 1.f, int32 LoopCount = 1, float BlendOutTriggerTime = -1.f, float InTimeToStartMontageAt = 0.f);

	/** Stops currently playing slot animation slot or all*/
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API void StopSlotAnimation(float InBlendOutTime = 0.25f, FName SlotNodeName = NAME_None);

	/** Return true if it's playing the slot animation */
	UFUNCTION(BlueprintPure, Category="Animation|Montage")
	ENGINE_API bool IsPlayingSlotAnimation(const UAnimSequenceBase* Asset, FName SlotNodeName) const;

	/** Return true if this instance playing the slot animation, also returning the montage it is playing on */
	ENGINE_API bool IsPlayingSlotAnimation(const UAnimSequenceBase* Asset, FName SlotNodeName, UAnimMontage*& OutMontage) const;

	/** Return true if this instance has an active montage in the given slot. A UAnimMontage that is playing in the slot and blending out is not determined to be "active". */
	UFUNCTION(BlueprintPure, Category="Animation|Montage", meta=(BlueprintThreadSafe))
	ENGINE_API bool IsSlotActive(FName SlotNodeName) const;
	
	/** Get local weight of any montages this slot node is playing. If this slot is not currently playing a montage, it will return 0. */
	UFUNCTION(BlueprintPure, Category="Animation|Montage", meta=(BlueprintThreadSafe, DisplayName="GetSlotLocalWeight"))
	ENGINE_API float Blueprint_GetSlotMontageLocalWeight(FName SlotNodeName) const;
	
	/*********************************************************************************************
	 * AnimMontage
	 ********************************************************************************************* */
public:
	/** Plays an animation montage. Returns the length of the animation montage in seconds. Returns 0.f if failed to play. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API float Montage_Play(UAnimMontage* MontageToPlay, float InPlayRate = 1.f, EMontagePlayReturnType ReturnValueType = EMontagePlayReturnType::MontageLength, float InTimeToStartMontageAt=0.f, bool bStopAllMontages = true);

	/** Plays an animation montage. Same as Montage_Play, but you can specify an AlphaBlend for Blend In settings. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API float Montage_PlayWithBlendIn(UAnimMontage* MontageToPlay, const FAlphaBlendArgs& BlendIn, float InPlayRate = 1.f, EMontagePlayReturnType ReturnValueType = EMontagePlayReturnType::MontageLength, float InTimeToStartMontageAt=0.f, bool bStopAllMontages = true);

	/** Plays an animation montage. Same as Montage_Play, but you can overwrite all of the montage's default blend in settings. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API float Montage_PlayWithBlendSettings(UAnimMontage* MontageToPlay, const FMontageBlendSettings& BlendInSettings, float InPlayRate = 1.f, EMontagePlayReturnType ReturnValueType = EMontagePlayReturnType::MontageLength, float InTimeToStartMontageAt=0.f, bool bStopAllMontages = true);

	/** Stops the animation montage. If reference is NULL, it will stop ALL active montages. */
	/** Stopped montages will blend out using their montage asset's BlendOut, with InBlendOutTime as the BlendTime */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_Stop(float InBlendOutTime, const UAnimMontage* Montage = NULL);

	/** Same as Montage_Stop. Uses values from the AlphaBlendArgs. Other settings come from the montage asset*/
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_StopWithBlendOut(const FAlphaBlendArgs& BlendOut, const UAnimMontage* Montage = nullptr);

	/** Same as Montage_Stop, but all blend settings are provided instead of using the ones on the montage asset*/
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_StopWithBlendSettings(const FMontageBlendSettings& BlendOutSettings, const UAnimMontage* Montage = nullptr);

	/** Stops all active montages belonging to a group. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_StopGroupByName(float InBlendOutTime, FName GroupName);

	/** Pauses the animation montage. If reference is NULL, it will pause ALL active montages. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_Pause(const UAnimMontage* Montage = NULL);

	/** Resumes a paused animation montage. If reference is NULL, it will resume ALL active montages. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_Resume(const UAnimMontage* Montage);

	/** Makes a montage jump to a named section. If Montage reference is NULL, it will do that to all active montages. */
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API void Montage_JumpToSection(FName SectionName, const UAnimMontage* Montage = NULL);

	/** Makes a montage jump to the end of a named section. If Montage reference is NULL, it will do that to all active montages. */
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API void Montage_JumpToSectionsEnd(FName SectionName, const UAnimMontage* Montage = NULL);

	/** Relink new next section AFTER SectionNameToChange in run-time
	 *	You can link section order the way you like in editor, but in run-time if you'd like to change it dynamically, 
	 *	use this function to relink the next section
	 *	For example, you can have Start->Loop->Loop->Loop.... but when you want it to end, you can relink
	 *	next section of Loop to be End to finish the montage, in which case, it stops looping by Loop->End. 
	 
	 * @param SectionNameToChange : This should be the name of the Montage Section after which you want to insert a new next section
	 * @param NextSection	: new next section 
	 */
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API void Montage_SetNextSection(FName SectionNameToChange, FName NextSection, const UAnimMontage* Montage = NULL);

	/** Change AnimMontage play rate. NewPlayRate = 1.0 is the default playback rate. */
	UFUNCTION(BlueprintCallable, Category="Animation|Montage")
	ENGINE_API void Montage_SetPlayRate(const UAnimMontage* Montage, float NewPlayRate = 1.f);

	/** Returns true if the animation montage is active. If the Montage reference is NULL, it will return true if any Montage is active. */
	UFUNCTION(BlueprintPure, Category="Animation|Montage")
	ENGINE_API bool Montage_IsActive(const UAnimMontage* Montage) const;

	/** Returns true if the animation montage is currently active and playing. 
	If reference is NULL, it will return true is ANY montage is currently active and playing. */
	UFUNCTION(BlueprintPure, Category="Animation|Montage")
	ENGINE_API bool Montage_IsPlaying(const UAnimMontage* Montage) const;

	/** Returns the name of the current animation montage section. */
	UFUNCTION(BlueprintPure, Category="Animation|Montage")
	ENGINE_API FName Montage_GetCurrentSection(const UAnimMontage* Montage = NULL) const;

	/** Get Current Montage Position */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API float Montage_GetPosition(const UAnimMontage* Montage) const;
	
	/** Set position. */
	UFUNCTION(BlueprintCallable, Category = "Animation|Montage")
	ENGINE_API void Montage_SetPosition(const UAnimMontage* Montage, float NewPosition);
	
	/** return true if Montage is not currently active. (not valid or blending out) */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API bool Montage_GetIsStopped(const UAnimMontage* Montage) const;

	/** Get the current blend time of the Montage.
	If Montage reference is NULL, it will return the current blend time on the first active Montage found. */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API float Montage_GetBlendTime(const UAnimMontage* Montage) const;

	/** Get PlayRate for Montage. This does not account for RateScale, so it may not reflect the actual play rate seen in game (see Montage_GetEffectivePlayRate).
	If Montage reference is NULL, PlayRate for any Active Montage will be returned.
	If Montage is not playing, 0 is returned. */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API float Montage_GetPlayRate(const UAnimMontage* Montage) const;

	/** Get scaled PlayRate for Montage. This accounts for RateScale, so it will reflect the actual play rate seen in game.
	If Montage reference is NULL, scaled PlayRate for any Active Montage will be returned.
	If Montage is not playing, 0 is returned. */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API float Montage_GetEffectivePlayRate(const UAnimMontage* Montage) const;

	/** Returns true if there is an animation montage is currently active and playing that was created from the provided animation. */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API bool DynamicMontage_IsPlayingFrom(const UAnimSequenceBase* Animation) const;

	/*********************************************************************************************
	* AnimMontage sync. See notes in AnimMontage.h
	********************************************************************************************* */

	/** Synchronize a montage to another anim instance's montage. Both montages must be playing already
	* @param MontageFollower : The montage that will follow the leader in OtherAnimInstance
	* @param OtherAnimInstance	: The other anim instance we want to synchronize to. Can be set to self
	* @param MontageLeader	: The montage we want to follow in the other anim instance */
	UFUNCTION(BlueprintCallable, Category = "Montage")
	ENGINE_API void MontageSync_Follow(const UAnimMontage* MontageFollower, const UAnimInstance* OtherAnimInstance, const UAnimMontage* MontageLeader);

	/** Stop following the montage's leader in this anim instance
	* @param MontageFollower : The montage we want to stop synchronizing */
	UFUNCTION(BlueprintCallable, Category = "Montage")
	ENGINE_API void MontageSync_StopFollowing(const UAnimMontage* MontageFollower);

	/** Returns true if any montage is playing currently. Doesn't mean it's active though, it could be blending out. */
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API bool IsAnyMontagePlaying() const;

	/** Get a current Active Montage in this AnimInstance. 
		Note that there might be multiple Active at the same time. This will only return the first active one it finds. **/
	UFUNCTION(BlueprintPure, Category = "Animation|Montage")
	ENGINE_API UAnimMontage* GetCurrentActiveMontage() const;

	/** Called when a montage starts blending out, whether interrupted or finished */
	UPROPERTY(BlueprintAssignable)
	FOnMontageBlendingOutStartedMCDelegate OnMontageBlendingOut;

	/** Called when a montage finishes blending in */
	UPROPERTY(BlueprintAssignable)
	FOnMontageBlendedInEndedMCDelegate OnMontageBlendedIn;
	
	/** Called when a montage has started */
	UPROPERTY(BlueprintAssignable)
	FOnMontageStartedMCDelegate OnMontageStarted;

	/** Called when a montage has ended, whether interrupted or finished*/
	UPROPERTY(BlueprintAssignable)
	FOnMontageEndedMCDelegate OnMontageEnded;

	/** Called when all Montage instances have ended. */
	UPROPERTY(BlueprintAssignable)
	FOnAllMontageInstancesEndedMCDelegate OnAllMontageInstancesEnded;

	/** Called when a montage section changes */
	UPROPERTY(BlueprintAssignable)
	FOnMontageSectionChangedMCDelegate OnMontageSectionChanged;
	
	/*********************************************************************************************
	* AnimMontage native C++ interface
	********************************************************************************************* */
public:	
	ENGINE_API void Montage_SetEndDelegate(FOnMontageEnded & InOnMontageEnded, UAnimMontage* Montage = NULL);

	/** Get pointer to Ended delegate for Montage.
	    If Montage reference is NULL, it will pick the first active montage found.*/
	ENGINE_API FOnMontageEnded* Montage_GetEndedDelegate(UAnimMontage* Montage = nullptr);
	
	ENGINE_API void Montage_SetBlendingOutDelegate(FOnMontageBlendingOutStarted& InOnMontageBlendingOut, UAnimMontage* Montage = NULL);

	ENGINE_API void Montage_SetBlendedInDelegate(FOnMontageBlendedInEnded& InOnMontageBlendingIn, UAnimMontage* Montage = nullptr);
	
	/** Get pointer to BlendingOutStarted delegate for Montage.
	If Montage reference is NULL, it will pick the first active montage found. */
	ENGINE_API FOnMontageBlendingOutStarted* Montage_GetBlendingOutDelegate(UAnimMontage* Montage = NULL);

	/**  Experimental: Returns the delegate if Montage is found, otherwise returns first one found */
	UE_EXPERIMENTAL(5.8, "Experimental custom way to drive the montage frame time forward, API may change in the future")
	ENGINE_API FComputeDeltaFrameTime* Montage_GetComputeFrameDeltaTimeDelegate(const UAnimMontage* Montage = nullptr);
	
	ENGINE_API void Montage_SetSectionChangedDelegate(FOnMontageSectionChanged& InOnMontageSectionChanged, UAnimMontage* Montage = NULL);
	
	/** Get pointer to SectionChanged delegate for Montage.
	If Montage reference is NULL, it will pick the first active montage found. */
	ENGINE_API FOnMontageSectionChanged* Montage_GetSectionChangedDelegate(UAnimMontage* Montage = NULL);
	
	/** Get next sectionID for given section ID */
	ENGINE_API int32 Montage_GetNextSectionID(const UAnimMontage* Montage, int32 const & CurrentSectionID) const;

	/** Get Currently active montage instance.
		Note that there might be multiple Active at the same time. This will only return the first active one it finds. **/
	ENGINE_API FAnimMontageInstance* GetActiveMontageInstance() const;

	/** Get Active FAnimMontageInstance for given Montage asset. Will return NULL if Montage is not currently Active. */
	ENGINE_API FAnimMontageInstance* GetActiveInstanceForMontage(const UAnimMontage* Montage) const;

	/** Get FAnimMontageInstance for given Montage asset (even when blending out). Will return NULL if Montage is not currently playing. */
	ENGINE_API FAnimMontageInstance* GetInstanceForMontage(const UAnimMontage* Montage) const;

	/** Get the FAnimMontageInstance currently running that matches this ID.  Will return NULL if no instance is found. */
	ENGINE_API FAnimMontageInstance* GetMontageInstanceForID(int32 MontageInstanceID);

	/** Stop all montages that are active **/
	ENGINE_API void StopAllMontages(float BlendOut);

	/** AnimMontage instances that are running currently
	* - only one is primarily active per group, and the other ones are blending out
	*/
	TArray<struct FAnimMontageInstance*> MontageInstances;

	ENGINE_API virtual void OnMontageInstanceStopped(FAnimMontageInstance & StoppedMontageInstance);
	ENGINE_API void ClearMontageInstanceReferences(FAnimMontageInstance& InMontageInstance);

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedInputPoseNode")
	FAnimNode_LinkedInputPose* GetSubInputNode(FName InSubInput = NAME_None, FName InGraph = NAME_None) { return GetLinkedInputPoseNode(InSubInput, InGraph); }

	/** 
	 * Get a linked input pose node by name, given a named graph.
	 * @param	InSubInput	The name of the linked input pose. If this is NAME_None, then we assume that the desired input is FAnimNode_LinkedInputPose::DefaultInputPoseName.
	 * @param	InGraph		The name of the graph in which to find the linked input. If this is NAME_None, then we assume that the desired graph is "AnimGraph", the default.
	 */
	ENGINE_API FAnimNode_LinkedInputPose* GetLinkedInputPoseNode(FName InSubInput = NAME_None, FName InGraph = NAME_None);

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimGraphInstanceByTag")
	UAnimInstance* GetSubInstanceByTag(FName InTag) const { return GetLinkedAnimGraphInstanceByTag(InTag); }

	/** Runs through all nodes, attempting to find the first linked instance by name/tag */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs")
	ENGINE_API UAnimInstance* GetLinkedAnimGraphInstanceByTag(FName InTag) const;

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimGraphInstancesByTag")
	ENGINE_API void GetSubInstancesByTag(FName InTag, TArray<UAnimInstance*>& OutSubInstances) const;

	/** Runs through all nodes, attempting to find all linked instances that match the name/tag */
	UE_DEPRECATED(5.0, "Tags are unique so this function is no longer supported. Please use GetLinkedAnimGraphInstanceByTag instead")
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs", meta=(DeprecatedFunction, DeprecationMessage="Tags are unique so this function is no longer supported. Please use GetLinkedAnimGraphInstanceByTag instead"))
	ENGINE_API void GetLinkedAnimGraphInstancesByTag(FName InTag, TArray<UAnimInstance*>& OutLinkedInstances) const;

	UE_DEPRECATED(4.24, "Function renamed, please use LinkAnimGraphByTag")
	void SetSubInstanceClassByTag(FName InTag, TSubclassOf<UAnimInstance> InClass) { LinkAnimGraphByTag(InTag, InClass); }

	/** Runs through all nodes, attempting to find a linked instance by name/tag, then sets the class of each node if the tag matches */
	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs")
	ENGINE_API void LinkAnimGraphByTag(FName InTag, TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use LinkAnimClassLayers")
	void SetLayerOverlay(TSubclassOf<UAnimInstance> InClass) { LinkAnimClassLayers(InClass); }

	/**
	 * Runs through all layer nodes, attempting to find layer nodes that are implemented by the specified class, then sets up a linked instance of the class for each.
	 * Allocates one linked instance to run each of the groups specified in the class, so state is shared. If a layer is not grouped (ie. NAME_None), then state is not shared
	 * and a separate linked instance is allocated for each layer node.
	 * If InClass is null, then all layers are reset to their defaults.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs")
	ENGINE_API virtual void LinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use UnlinkAnimClassLayers")
	void ClearLayerOverlay(TSubclassOf<UAnimInstance> InClass) { UnlinkAnimClassLayers(InClass); }

	/**
	 * Runs through all layer nodes, attempting to find layer nodes that are currently running the specified class, then resets each to its default value.
	 * State sharing rules are as with SetLayerOverlay.
	 * If InClass is null, does nothing.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs")
	ENGINE_API virtual void UnlinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimLayerInstanceByGroup")
	UAnimInstance* GetLayerSubInstanceByGroup(FName InGroup) const { return GetLinkedAnimLayerInstanceByGroup(InGroup); }

	/** Gets the layer linked instance corresponding to the specified group */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs")
	ENGINE_API UAnimInstance* GetLinkedAnimLayerInstanceByGroup(FName InGroup) const;

	/** Runs through all nodes, attempting to find all distinct layer linked instances in the group */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs")
	ENGINE_API void GetLinkedAnimLayerInstancesByGroup(FName InGroup, TArray<UAnimInstance*>& OutLinkedInstances) const;

	/** Gets layer linked instance that matches group and class */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs")
	ENGINE_API UAnimInstance* GetLinkedAnimLayerInstanceByGroupAndClass(FName InGroup, TSubclassOf<UAnimInstance> InClass) const;

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimLayerInstanceByClass")
	UAnimInstance* GetLayerSubInstanceByClass(TSubclassOf<UAnimInstance> InClass) const { return GetLinkedAnimLayerInstanceByClass(InClass); }

	/** Gets the first layer linked instance corresponding to the specified class, optionally if bCheckForChildClass is true, it will check IsChildOf on InClass. */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs")
	ENGINE_API UAnimInstance* GetLinkedAnimLayerInstanceByClass(TSubclassOf<UAnimInstance> InClass, bool bCheckForChildClass = false) const;

	/** Sets up initial layer groupings */
	ENGINE_API void InitializeGroupedLayers(bool bInDeferSubGraphInitialization);

	/** Allows other UObjects to bind custom event notifies similarly to the AnimBP */
	ENGINE_API void AddExternalNotifyHandler(UObject* ExternalHandlerObject, FName NotifyEventName);
	/** Other UObjects should call this to remove themselves from the callbacks */
	ENGINE_API void RemoveExternalNotifyHandler(UObject* ExternalHandlerObject, FName NotifyEventName);

	// Find a subsystem's instance-resident data. If no subsystem of the type exists this will return nullptr.
	// @param	InSubsystemType	The subsystem's type
	ENGINE_API FAnimSubsystemInstance* FindSubsystem(UScriptStruct* InSubsystemType);

	// Get a subsystem's instance-resident data. If no subsystem of the type exists this will return nullptr.
	template<typename SubsystemType>
	SubsystemType* FindSubsystem()
	{
		FAnimSubsystemInstance* Subsystem = FindSubsystem(SubsystemType::StaticStruct());
		return static_cast<SubsystemType*>(Subsystem);
	}
	
	// Get a subsystem's instance-resident data. If no subsystem of the type exists this will assert.
	template<typename SubsystemType>
	SubsystemType& GetSubsystem()
	{
		FAnimSubsystemInstance* Subsystem = FindSubsystem(SubsystemType::StaticStruct());
		check(Subsystem);
		return static_cast<SubsystemType&>(*Subsystem);
	}

public:

#if ENABLE_ANIM_DRAW_DEBUG
		ENGINE_API void AnimDrawDebugOnScreenMessage(const FString& DebugMessage, const FColor& Color, const FVector2D& TextScale = FVector2D::UnitVector, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugInWorldMessage(const FString& DebugMessage, const FVector& TextLocation, const FColor& Color, float TextScale);
		ENGINE_API void AnimDrawDebugLine(const FVector& StartLoc, const FVector& EndLoc, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugCoordinateSystem(FVector const& AxisLoc, FRotator const& AxisRot, float Scale = 1.f, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugPlane(const FTransform& BaseTransform, float Radii, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugPoint(const FVector& Loc, float Size, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
		ENGINE_API void AnimDrawDebugCircle(const FVector& Center, float Radius, int32 Segments, const FColor& Color, const FVector& UpVector = FVector::UpVector, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f);
		ENGINE_API void AnimDrawDebugCone(const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, float AngleHeight, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f);
		ENGINE_API void AnimDrawDebugCapsule(const FVector& Center, float HalfHeight, float Radius, const FRotator& Rotation, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
#else
		void AnimDrawDebugOnScreenMessage(const FString& DebugMessage, const FColor& Color, const FVector2D& TextScale = FVector2D::UnitVector, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugInWorldMessage(const FString& DebugMessage, const FVector& TextLocation, const FColor& Color, float TextScale) {}
		void AnimDrawDebugLine(const FVector& StartLoc, const FVector& EndLoc, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugCoordinateSystem(FVector const& AxisLoc, FRotator const& AxisRot, float Scale = 1.f, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugPlane(const FTransform& BaseTransform, float Radii, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugPoint(const FVector& Loc, float Size, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
		void AnimDrawDebugCircle(const FVector& Center, float Radius, int32 Segments, const FColor& Color, const FVector& UpVector = FVector::UpVector, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f) {}
		void AnimDrawDebugCone(const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, float AngleHeight, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f) {}
		void AnimDrawDebugCapsule(const FVector& Center, float HalfHeight, float Radius, const FRotator& Rotation, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
#endif // ENABLE_ANIM_DRAW_DEBUG

private:
	/** Helper function to perform layer overlay actions (set, clear) */
	ENGINE_API void PerformLinkedLayerOverlayOperation(TSubclassOf<UAnimInstance> InClass, TFunctionRef<UClass*(UClass*, FAnimNode_LinkedAnimLayer*)> InClassSelectorFunction, bool bInDeferSubGraphInitialization = false);

protected:
	/** Map between Active Montages and their FAnimMontageInstance */
	TMap<class UAnimMontage*, struct FAnimMontageInstance*> ActiveMontagesMap;

	/**  Inertialization requests gathered this frame. Gets reset in UpdateMontageEvaluationData */
	UPROPERTY(Transient)
	TMap<FName, FInertializationRequest> SlotGroupInertializationRequestDataMap;

	/** Stop all active montages belonging to 'InGroupName' */
	ENGINE_API void StopAllMontagesByGroupName(FName InGroupName, const FMontageBlendSettings& BlendOutSettings);

	/** Update weight of montages  **/
	ENGINE_API virtual void Montage_UpdateWeight(float DeltaSeconds);
	/** Advance montages **/
	ENGINE_API virtual void Montage_Advance(float DeltaSeconds);

	ENGINE_API void Montage_StopInternal(TFunctionRef<FMontageBlendSettings(const FAnimMontageInstance*)> AlphaBlendSelectorFunction, const UAnimMontage* Montage = nullptr);
	ENGINE_API virtual float Montage_PlayInternal(UAnimMontage* MontageToPlay, const FMontageBlendSettings& BlendInSettings, float InPlayRate = 1.f, EMontagePlayReturnType ReturnValueType = EMontagePlayReturnType::MontageLength, float InTimeToStartMontageAt = 0.f, bool bStopAllMontages = true);

public:

	/** Flush completed montages when animation tick is paused. 
		This can be used to prevent montages from infinitely stacking up when not ticking. */
	ENGINE_API void ConditionalFlushCompletedMontages();

	/**  Builds an inertialization request from the montage's group, provided duration and optional blend profile*/
	ENGINE_API void RequestMontageInertialization(const UAnimMontage* Montage, float Duration, const UBlendProfile* BlendProfile = nullptr);

	/**  Makes an inertialization request from the montage's group. */
	ENGINE_API void RequestMontageInertialization(const UAnimMontage* Montage, const FInertializationRequest& Request);

	/**  Requests an inertial blend during the next anim graph update. Requires your anim graph to have a slot node belonging to the specified group name */
	UFUNCTION(BlueprintCallable, Category = "Animation|Inertial Blending")
	ENGINE_API void RequestSlotGroupInertialization(FName InSlotGroupName, float Duration, const UBlendProfile* BlendProfile = nullptr);
	
	/**  Returns active inertialization requests on the anim instance (does not forward the call to the proxy), used for legacy montage support in UAF */
	ENGINE_API const TMap<FName, FInertializationRequest>& GetActiveInertializationRequests() const;
	
	/**  Clears all active inertialization requests on the anim instance (does not forward the call to the proxy), used for legacy montage support in UAF */
	ENGINE_API void ClearActiveInertializationRequests();

	/** Queue a Montage BlendingOut Event to be triggered. */
	ENGINE_API void QueueMontageBlendingOutEvent(const FQueuedMontageBlendingOutEvent& MontageBlendingOutEvent);

	/** Queue a Montage BlendedIn Event to be triggered. */
	ENGINE_API void QueueMontageBlendedInEvent(const FQueuedMontageBlendedInEvent& MontageBlendedInEvent);

	/** Queue a Montage Ended Event to be triggered. */
	ENGINE_API void QueueMontageEndedEvent(const FQueuedMontageEndedEvent& MontageEndedEvent);

	/** Queue a Montage Section Changed Event to be triggered. */
	ENGINE_API void QueueMontageSectionChangedEvent(const FQueuedMontageSectionChangedEvent& MontageSectionChangedEvent);
	
private:
	/** Trigger queued Montage events. */
	ENGINE_API void TriggerQueuedMontageEvents();

	/** Queued Montage BlendingOut events. */
	TArray<FQueuedMontageBlendingOutEvent> QueuedMontageBlendingOutEvents;

	/** Queued Montage BlendedIn events. */
	TArray<FQueuedMontageBlendedInEvent> QueuedMontageBlendedInEvents;

	/** Queued Montage Ended Events */
	TArray<FQueuedMontageEndedEvent> QueuedMontageEndedEvents;

	/** Queued Montage Section Changed Events */
	TArray<FQueuedMontageSectionChangedEvent> QueuedMontageSectionChangedEvents;
	
	/** Trigger a Montage BlendingOut event */
	ENGINE_API void TriggerMontageBlendingOutEvent(const FQueuedMontageBlendingOutEvent& MontageBlendingOutEvent);

	/** Trigger a Montage BlendingIn event */
	ENGINE_API void TriggerMontageBlendedInEvent(const FQueuedMontageBlendedInEvent& MontageBlendedInEvent);

	/** Trigger a Montage Ended event */
	ENGINE_API void TriggerMontageEndedEvent(const FQueuedMontageEndedEvent& MontageEndedEvent);

	/** Trigger a Montage Ended event */
	ENGINE_API void TriggerMontageSectionChangedEvent(const FQueuedMontageSectionChangedEvent& MontageSectionChangedEvent);
	
public:

#if DO_CHECK
	/** Is this animation currently running post update */
	bool IsPostUpdatingAnimation() const { return static_cast<bool>(bPostUpdatingAnimation); }

	/** Is this animation currently running update */
	bool IsUpdatingAnimation() const { return static_cast<bool>(bUpdatingAnimation); }
#endif

	/** Set RootMotionMode */
	UFUNCTION(BlueprintCallable, Category = "Animation|Root Motion")
	ENGINE_API void SetRootMotionMode(TEnumAsByte<ERootMotionMode::Type> Value);

	/** 
	 * NOTE: Derived anim getters
	 *
	 * Anim getter functions can be defined for any instance deriving UAnimInstance.
	 * To do this the function must be marked BlueprintPure, and have the AnimGetter metadata entry set to
	 * "true". Following the instructions below, getters should appear correctly in the blueprint node context
	 * menu for the derived classes
	 *
	 * A context string can be provided in the GetterContext metadata and can contain any (or none) of the
	 * following entries separated by a pipe (|)
	 * Transition  - Only available in a transition rule
	 * AnimGraph   - Only available in an animgraph (also covers state anim graphs)
	 * CustomBlend - Only available in a custom blend graph
	 *
	 * Anim getters support a number of automatic parameters that will be baked at compile time to be passed
	 * to the functions. They will not appear as pins on the graph node. They are as follows:
	 * AssetPlayerIndex - Index of an asset player node to operate on, one getter will be added to the blueprint action list per asset node available
	 * MachineIndex     - Index of a state machine in the animation blueprint, one getter will be added to the blueprint action list per state machine
	 * StateIndex       - Index of a state inside a state machine, also requires MachineIndex. One getter will be added to the blueprint action list per state
	 * TransitionIndex  - Index of a transition inside a state machine, also requires MachineIndex. One getter will be added to the blueprint action list per transition
	 */

	/** Gets the length in seconds of the asset referenced in an asset player node */
	UFUNCTION(BlueprintPure, Category="Animation|Asset Player", meta=(DisplayName="Length", BlueprintInternalUseOnly="true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceAssetPlayerLength(int32 AssetPlayerIndex);

	/** Get the current accumulated time in seconds for an asset player node */
	UFUNCTION(BlueprintPure, Category="Animation|Asset Player", meta = (DisplayName = "Current Time", BlueprintInternalUseOnly = "true", AnimGetter = "true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceAssetPlayerTime(int32 AssetPlayerIndex);

	/** Get the current accumulated time as a fraction for an asset player node */
	UFUNCTION(BlueprintPure, Category="Animation|Asset Player", meta=(DisplayName="Current Time (ratio)", BlueprintInternalUseOnly="true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceAssetPlayerTimeFraction(int32 AssetPlayerIndex);

	/** Get the time in seconds from the end of an animation in an asset player node */
	UFUNCTION(BlueprintPure, Category="Animation|Asset Player", meta=(DisplayName="Time Remaining", BlueprintInternalUseOnly="true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceAssetPlayerTimeFromEnd(int32 AssetPlayerIndex);

	/** Get the time as a fraction of the asset length of an animation in an asset player node */
	UFUNCTION(BlueprintPure, Category="Animation|Asset Player", meta=(DisplayName="Time Remaining (ratio)", BlueprintInternalUseOnly="true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceAssetPlayerTimeFromEndFraction(int32 AssetPlayerIndex);

	/** Get the blend weight of a specified state machine */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (DisplayName = "Machine Weight", BlueprintInternalUseOnly = "true", AnimGetter = "true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceMachineWeight(int32 MachineIndex);

	/** Get the blend weight of a specified state */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (DisplayName="State Weight", BlueprintInternalUseOnly = "true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceStateWeight(int32 MachineIndex, int32 StateIndex);

	/** Get the current elapsed time of a state within the specified state machine */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (DisplayName="Current State Time", BlueprintInternalUseOnly = "true", AnimGetter="true", GetterContext="Transition", BlueprintThreadSafe))
	ENGINE_API float GetInstanceCurrentStateElapsedTime(int32 MachineIndex);

	/** Get the crossfade duration of a specified transition */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (DisplayName="Get Transition Crossfade Duration", BlueprintInternalUseOnly = "true", AnimGetter="true", BlueprintThreadSafe))
	ENGINE_API float GetInstanceTransitionCrossfadeDuration(int32 MachineIndex, int32 TransitionIndex);

	/** Get the elapsed time in seconds of a specified transition */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (DisplayName="Get Transition Time Elapsed", BlueprintInternalUseOnly = "true", AnimGetter="true", GetterContext="CustomBlend", BlueprintThreadSafe))
	ENGINE_API float GetInstanceTransitionTimeElapsed(int32 MachineIndex, int32 TransitionIndex);

	/** Get the elapsed time as a fraction of the crossfade duration of a specified transition */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (DisplayName="Get Transition Time Elapsed (ratio)", BlueprintInternalUseOnly = "true", AnimGetter="true", GetterContext="CustomBlend", BlueprintThreadSafe))
	ENGINE_API float GetInstanceTransitionTimeElapsedFraction(int32 MachineIndex, int32 TransitionIndex);

	/** Get the time remaining in seconds for the most relevant animation in the source state */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter="true", GetterContext="Transition", BlueprintThreadSafe))
	ENGINE_API float GetRelevantAnimTimeRemaining(int32 MachineIndex, int32 StateIndex);

	/** Get the time remaining as a fraction of the duration for the most relevant animation in the source state */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe))
	ENGINE_API float GetRelevantAnimTimeRemainingFraction(int32 MachineIndex, int32 StateIndex);

	/** Get the length in seconds of the most relevant animation in the source state */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe))
	ENGINE_API float GetRelevantAnimLength(int32 MachineIndex, int32 StateIndex);

	/** Get the current accumulated time in seconds for the most relevant animation in the source state */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe))
	ENGINE_API float GetRelevantAnimTime(int32 MachineIndex, int32 StateIndex);

	/** Get the current accumulated time as a fraction of the length of the most relevant animation in the source state */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe))
	ENGINE_API float GetRelevantAnimTimeFraction(int32 MachineIndex, int32 StateIndex);

	/** Get whether a particular notify state was active in any state machine last tick.*/
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
	ENGINE_API bool WasAnimNotifyStateActiveInAnyState(TSubclassOf<UAnimNotifyState> AnimNotifyStateType);

	/** Get whether a particular notify state is active in a specific state machine last tick.  */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
	ENGINE_API bool WasAnimNotifyStateActiveInStateMachine(int32 MachineIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType);

	/** Get whether a particular notify state is active in a specific state last tick. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
	ENGINE_API bool WasAnimNotifyStateActiveInSourceState(int32 MachineIndex, int32 StateIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType);

	/** Get whether the most relevant animation was in a particular notify state last tick. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyTriggeredInSourceState(int32 MachineIndex, int32 StateIndex,  TSubclassOf<UAnimNotify> AnimNotifyType);

	/** Get whether the most relevant animation triggered the animation notify with the specified name last tick.. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyNameTriggeredInSourceState(int32 MachineIndex, int32 StateIndex, FName NotifyName);

	/** Get whether a particular notify type was active in a specific state machine last tick.  */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyTriggeredInStateMachine(int32 MachineIndex, TSubclassOf<UAnimNotify> AnimNotifyType);

	/** Get whether the given state machine triggered the animation notify with the specified name last tick. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyNameTriggeredInStateMachine(int32 MachineIndex, FName NotifyName);
	
	/**  Get whether an animation notify of a given type was triggered last tick. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyTriggeredInAnyState(TSubclassOf<UAnimNotify> AnimNotifyType);

	/** Get whether the animation notify with the specified name triggered last tick. */
	UFUNCTION(BlueprintPure, Category = "Animation|State Machines", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Window,TransitionWindow,NotifyState,NotifyStateTransition"))
    ENGINE_API bool WasAnimNotifyNameTriggeredInAnyState(FName NotifyName);
	
	/** Gets the runtime instance of the specified state machine by Name */
	ENGINE_API const FAnimNode_StateMachine* GetStateMachineInstanceFromName(FName MachineName) const;

	/** Get the machine description for the specified instance. Does not rely on PRIVATE_MachineDescription being initialized */
	ENGINE_API const FBakedAnimationStateMachine* GetMachineDescription(IAnimClassInterface* AnimBlueprintClass, FAnimNode_StateMachine* MachineInstance);

	ENGINE_API void GetStateMachineIndexAndDescription(FName InMachineName, int32& OutMachineIndex, const FBakedAnimationStateMachine** OutMachineDescription);

	/** Returns the baked sync group index from the compile step */
	ENGINE_API int32 GetSyncGroupIndexFromName(FName SyncGroupName) const;

	/** Gets the index of the state machine matching MachineName */
	ENGINE_API int32 GetStateMachineIndex(FName MachineName) const;

	/** Gets the runtime instance of the specified state machine */
	ENGINE_API const FAnimNode_StateMachine* GetStateMachineInstance(int32 MachineIndex) const;

	/** 
	 * Get the index of the specified instance asset player. Useful to pass to GetInstanceAssetPlayerLength (etc.).
	 * Passing NAME_None to InstanceName will return the first (assumed only) player instance index found.
	 */
	ENGINE_API int32 GetInstanceAssetPlayerIndex(FName MachineName, FName StateName, FName InstanceName = NAME_None) const;

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<const FAnimNode_AssetPlayerBase*> GetInstanceAssetPlayers(const FName& GraphName) const;

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<FAnimNode_AssetPlayerBase*> GetMutableInstanceAssetPlayers(const FName& GraphName);

	/** Returns all Animation Nodes implementing FAnimNode_AssetPlayerRelevancyBase within the specified (named) Animation Graph */
	ENGINE_API TArray<const FAnimNode_AssetPlayerRelevancyBase*> GetInstanceRelevantAssetPlayers(const FName& GraphName) const;

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerRelevancyBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<FAnimNode_AssetPlayerRelevancyBase*> GetMutableInstanceRelevantAssetPlayers(const FName& GraphName);

	/** Gets the runtime instance desc of the state machine specified by name */
	ENGINE_API const FBakedAnimationStateMachine* GetStateMachineInstanceDesc(FName MachineName) const;

	/** Gets the most relevant asset player in a specified state */
	ENGINE_API const FAnimNode_AssetPlayerRelevancyBase* GetRelevantAssetPlayerInterfaceFromState(int32 MachineIndex, int32 StateIndex) const;

	//////////////////////////////////////////////////////////////////////////

public:
	
	/** Returns a blend profile by name from our current skeleton. Null if not found. */
	UFUNCTION(BlueprintPure, Category="Animation|Skeleton", meta=(BlueprintThreadSafe))
	ENGINE_API const UBlendProfile* GetBlendProfileByName(FName InBlendProfileName) const;
	
	/** Returns the value of a named curve. */
	UFUNCTION(BlueprintPure, Category="Animation|Curves", meta=(BlueprintThreadSafe))
	ENGINE_API float GetCurveValue(FName CurveName) const;

	/** 
	* Returns whether a named curve was found, its value, and a default value when it's not found.
	* @param	AnimInstance	The anim instance to find this curve value for.
	* @param	CurveName		The name of the curve.
	* @param	DefaultValue	Value to use when the curve is not found.
	* @param	OutValue		The curve's value.
	*/
	UFUNCTION(BlueprintPure, Category="Animation|Curves", meta=(BlueprintThreadSafe))
	ENGINE_API bool GetCurveValueWithDefault(FName CurveName, float DefaultValue, float& OutValue) const;

	/** This returns last up-to-date list of active curve names */
	UFUNCTION(BlueprintPure, Category = "Animation|Curves", meta=(BlueprintThreadSafe))
	ENGINE_API void GetActiveCurveNames(EAnimCurveType CurveType, TArray<FName>& OutNames) const;

	/* This returns all curve names. This is the same as calling GetActiveCurveNames with CurveType == AttributeCurve */
	UFUNCTION(BlueprintPure, Category = "Animation|Curves", meta=(BlueprintThreadSafe))
	ENGINE_API void GetAllCurveNames(TArray<FName>& OutNames) const;

	/** Returns value of named curved in OutValue, returns whether the curve was actually found or not. */
	ENGINE_API bool GetCurveValue(FName CurveName, float& OutValue) const;

	/** Overrides the value of a named curve. Will be reset next evaluation */
	ENGINE_API void OverrideCurveValue(FName CurveName, float Value);

	/** Returns the name of a currently active state in a state machine. */
	UFUNCTION(BlueprintPure, Category="Animation|State Machines", meta=(BlueprintInternalUseOnly = "true", AnimGetter = "true", BlueprintThreadSafe))
	ENGINE_API FName GetCurrentStateName(int32 MachineIndex) const;

	/** Sets a morph target to a certain weight. */
	UFUNCTION(BlueprintCallable, Category="Animation|Morph Targets")
	ENGINE_API void SetMorphTarget(FName MorphTargetName, float Value);

	/** Clears the current morph targets. */
	UFUNCTION(BlueprintCallable, Category="Animation|Morph Targets")
	ENGINE_API void ClearMorphTargets();

	UE_DEPRECATED(5.0, "Please use UKismetAnimationLibrary::CalculateDirection instead")
	UFUNCTION(BlueprintCallable, Category="Animation", meta=(BlueprintThreadSafe))
	ENGINE_API float CalculateDirection(const FVector& Velocity, const FRotator& BaseRotation) const;

	//--- AI communication start ---//
	/** locks indicated AI resources of animated pawn
	 *	DEPRECATED. Use LockAIResourcesWithAnimation instead */
	UFUNCTION(BlueprintCallable, Category = "Animation", BlueprintAuthorityOnly, Meta=(DeprecatedFunction, DeprecationMessage="Use LockAIResourcesWithAnimation instead"))
	ENGINE_API void LockAIResources(bool bLockMovement, bool LockAILogic);

	/** unlocks indicated AI resources of animated pawn. Will unlock only animation-locked resources.
	 *	DEPRECATED. Use UnlockAIResourcesWithAnimation instead */
	UFUNCTION(BlueprintCallable, Category = "Animation", BlueprintAuthorityOnly, Meta=(DeprecatedFunction, DeprecationMessage="Use UnlockAIResourcesWithAnimation instead"))
	ENGINE_API void UnlockAIResources(bool bUnlockMovement, bool UnlockAILogic);
	//--- AI communication end ---//

	UFUNCTION(BlueprintCallable, Category = "Animation|Synchronization")
	ENGINE_API void ResetSync();

	UFUNCTION(BlueprintCallable, Category = "Animation|Synchronization", meta=(BlueprintThreadSafe))
	ENGINE_API bool GetTimeToClosestMarker(FName SyncGroup, FName MarkerName, float& OutMarkerTime) const;

	UFUNCTION(BlueprintCallable, Category = "Animation|Synchronization", meta=(BlueprintThreadSafe))
	ENGINE_API bool HasMarkerBeenHitThisFrame(FName SyncGroup, FName MarkerName) const;

	UFUNCTION(BlueprintCallable, Category = "Animation|Synchronization", meta=(BlueprintThreadSafe))
	ENGINE_API bool IsSyncGroupBetweenMarkers(FName InSyncGroupName, FName PreviousMarker, FName NextMarker, bool bRespectMarkerOrder = true) const;

	UFUNCTION(BlueprintCallable, Category = "Animation|Synchronization", meta=(BlueprintThreadSafe))
	ENGINE_API FMarkerSyncAnimPosition GetSyncGroupPosition(FName InSyncGroupName) const;

	/** Attempts to queue a transition request, returns true if the request was successful */
	UFUNCTION(BlueprintCallable, Category="Animation", meta = (BlueprintThreadSafe, Keywords = "Event,Request,Transition"))
	ENGINE_API bool RequestTransitionEvent(const FName EventName, const double RequestTimeout, const ETransitionRequestQueueMode QueueMode, const ETransitionRequestOverwriteMode OverwriteMode);

	/** Removes all queued transition requests with the given event name */
	UFUNCTION(BlueprintCallable, Category = "Animation", meta = (BlueprintThreadSafe, Keywords = "Event,Request,Transition"))
	ENGINE_API void ClearTransitionEvents(const FName EventName);

	/** Removes all queued transition requests */
	UFUNCTION(BlueprintCallable, Category = "Animation", meta = (BlueprintThreadSafe, Keywords = "Event,Request,Transition"))
	ENGINE_API void ClearAllTransitionEvents();

	/** Returns whether or not the given event transition request has been queued */
	UFUNCTION(BlueprintPure, Category = "Transitions", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Event,Request,Transition"))
	ENGINE_API bool QueryTransitionEvent(int32 MachineIndex, int32 TransitionIndex, FName EventName);

	/** Behaves like QueryTransitionEvent but additionally marks the event for consumption */
	UFUNCTION(BlueprintPure, Category = "Transitions", meta = (BlueprintInternalUseOnly = "true", AnimGetter = "true", GetterContext = "Transition", BlueprintThreadSafe, Keywords = "Event,Request,Transition"))
	ENGINE_API bool QueryAndMarkTransitionEvent(int32 MachineIndex, int32 TransitionIndex, FName EventName);

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void PostInitProperties() override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface

#if WITH_EDITORONLY_DATA // ANIMINST_PostCompileValidation
	/** Name of Class to do Post Compile Validation.
	* See Class UAnimBlueprintPostCompileValidation. */
	UPROPERTY()
	FSoftClassPath PostCompileValidationClassName;

	/** Warn if AnimNodes are not using fast path during AnimBP compilation. */
	virtual bool PCV_ShouldWarnAboutNodesNotUsingFastPath() const { return false; }
	virtual bool PCV_ShouldNotifyAboutNodesNotUsingFastPath() const { return false; }

	// Called on the newly created CDO during anim blueprint compilation to allow subclasses a chance to replace animations (experimental)
	virtual void ApplyAnimOverridesToCDO(FCompilerResultsLog& MessageLog) {}
#endif // WITH_EDITORONLY_DATA

	/** Called when skipping an animation update because of URO. */
	virtual void OnUROSkipTickAnimation() {}

	UE_DEPRECATED(4.22, "This function is deprecated, please use OnUROPreInterpolation_AnyThread")
	virtual void OnUROPreInterpolation() {}

	/** 
	 * Called before URO interpolation is performed. Useful for modifying bone space transforms etc. before interpolation is performed.
	 * Note that this can be called on a worker thread. 
	 */
	virtual void OnUROPreInterpolation_AnyThread(FAnimationEvaluationContext& InOutContext) {}

	/** Flag passed to UpdateAnimation, determines the path we follow */
	enum class EUpdateAnimationFlag : uint8
	{
		/** Enforces a parallel update, regardless of state */
		ForceParallelUpdate,
		/** Use state to determine whether or not to immediately or update in parallel */
		Default
	};

	/** Controls which notifies are triggered (used by TriggerAnimNotifies()). Useful mainly when handling active anim notify state lifetimes. */
	enum class ENotifyTriggerMode : uint8
	{
		/**
		 * Uses the global dispatch policy (a.Notifies.DispatchPolicy) for eligibility:
		 *   1: only trigger notifies from sources (anim graph or montage) that updated before the trigger request.
		 *   0: trigger notifies from all sources.
		 */
		Default,

		/** Override: Only trigger notifies queued by the anim graph. Ignores global dispatch policy. */
		ForceAnimGraphOnly,

		/** Override: Only trigger notifies queued by montages. Ignores global dispatch policy. */
		ForceMontageOnly,

		/** Override: Trigger notifies from all sources (anim graph/montages). Ignores global dispatch policy. */
		ForceAllSources,
	};
	
	// Animation phase trigger
	// start with initialize
	// update happens in every tick. Can happen in parallel with others if conditions are right.
	// evaluate happens when condition is met - i.e. depending on your skeletalmeshcomponent update flag
	// post eval happens after evaluation is done
	// uninitialize happens when owner is unregistered
	// @param	bInDeferRootNodeInitialization	When set to true, defer init of the blend tree until the first Update() call
	ENGINE_API void InitializeAnimation(bool bInDeferRootNodeInitialization = false);

	ENGINE_API void InitializeMontageOnly();

	/** Called on the game thread before UpdateAnimation is called on linked instances */
	virtual void PreUpdateLinkedInstances(float DeltaSeconds) {}

	/** Update Animation code-paths, updates and advances animation state, returns whether or not the actual update should have been called immediately */
	ENGINE_API void UpdateAnimation(float DeltaSeconds, bool bNeedsValidRootMotion, EUpdateAnimationFlag UpdateFlag = EUpdateAnimationFlag::Default );

	/** Run update animation work on a worker thread */
	ENGINE_API void ParallelUpdateAnimation();

	/** Called after updates are completed, dispatches notifies etc. */
	ENGINE_API void PostUpdateAnimation();

	/** Called on the game thread pre-evaluation. */
	ENGINE_API void PreEvaluateAnimation();

	/** Check whether evaluation can be performed on the supplied skeletal mesh. Can be called from worker threads. */
	ENGINE_API bool ParallelCanEvaluate(const USkeletalMesh* InSkeletalMesh) const;

	/** Perform evaluation. Can be called from worker threads. */
	ENGINE_API void ParallelEvaluateAnimation(bool bForceRefPose, const USkeletalMesh* InSkeletalMesh, FParallelEvaluationData& OutAnimationPoseData);

	UE_DEPRECATED(4.26, "Please use ParallelEvaluateAnimation with different signature.")
	ENGINE_API void ParallelEvaluateAnimation(bool bForceRefPose, const USkeletalMesh* InSkeletalMesh, FBlendedHeapCurve& OutCurve, FCompactPose& OutPose);

	// Called post evaluation if an evaluation took place
	ENGINE_API void PostEvaluateAnimation();
	// Called after PostEvaluateAnimation, whether or not an evaluation took place
	ENGINE_API void FinalizeAnimation(bool bWasEvaluated);
	ENGINE_API void UninitializeAnimation();

	// the below functions are the native overrides for each phase
	// Native initialization override point
	ENGINE_API virtual void NativeInitializeAnimation();
	// Native update override point. It is usually a good idea to simply gather data in this step and 
	// for the bulk of the work to be done in NativeThreadSafeUpdateAnimation.
	ENGINE_API virtual void NativeUpdateAnimation(float DeltaSeconds);
	// Native thread safe update override point. Executed on a worker thread just prior to graph update 
	// for linked anim instances, only called when the hosting node(s) are relevant
	ENGINE_API virtual void NativeThreadSafeUpdateAnimation(float DeltaSeconds);
	// Native Post Evaluate override point
	ENGINE_API virtual void NativePostEvaluateAnimation();
	// Native Finalize override point
	ENGINE_API virtual void NativeFinalizeAnimation(bool bWasEvaluated);
	// Native Uninitialize override point
	ENGINE_API virtual void NativeUninitializeAnimation();
	// Native override point executed when all Linked Animation Layers are initialized
	ENGINE_API virtual void NativeLinkedAnimationLayersInitialized();

	// Enable / Disable animation update. This is provided as an optimization to disable linked instances that aren't relevant. Disabling an instance whose graph is still evaluated will assert and causes issues.
	void EnableUpdateAnimation(bool bEnable) { bUpdateAnimationEnabled = bEnable; }
	bool IsUpdateAnimationEnabled() const { return bUpdateAnimationEnabled != 0; }

	// Executed when begin play is called on the owning component
	ENGINE_API virtual void NativeBeginPlay();

	// Sets up a native transition delegate between states with PrevStateName and NextStateName, in the state machine with name MachineName.
	// Note that a transition already has to exist for this to succeed
	ENGINE_API void AddNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, const FCanTakeTransition& NativeTransitionDelegate, const FName& TransitionName = NAME_None);

	// Check for whether a native rule is bound to the specified transition
	ENGINE_API bool HasNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, FName& OutBindingName);

	// Sets up a native state entry delegate from state with StateName, in the state machine with name MachineName.
	ENGINE_API void AddNativeStateEntryBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeEnteredDelegate);
	
	// Check for whether a native entry delegate is bound to the specified state
	ENGINE_API bool HasNativeStateEntryBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName);

	// Sets up a native state exit delegate from state with StateName, in the state machine with name MachineName.
	ENGINE_API void AddNativeStateExitBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeExitedDelegate);

	// Check for whether a native exit delegate is bound to the specified state
	ENGINE_API bool HasNativeStateExitBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName);

	// Debug output for this anim instance. Info for SyncGroups, Graph, Montages, etc.
	ENGINE_API void DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);

	// Display debug info about AnimInstance. Can be overridden to add custom info from child classes.
	ENGINE_API virtual void DisplayDebugInstance(FDisplayDebugManager& DisplayDebugManager, float& Indent);

	/** Reset any dynamics running simulation-style updates (e.g. on teleport, time skip etc.) */
	UFUNCTION(BlueprintCallable, Category = "Animation|Dynamics")
	ENGINE_API void ResetDynamics(ETeleportType InTeleportType);

	UE_DEPRECATED(4.20, "Please use ResetDynamics with a ETeleportType argument")
	ENGINE_API void ResetDynamics();

	/** 
	 * Get the 'animation' LOD level, which by default is the PredictedLODLevel of this anim instance's skeletal mesh component.
	 * This function is used by the anim graph to determine the LOD level at which to run.
	 * @return the current LOD level
	 */
	UFUNCTION(BlueprintPure, Category = "Animation")
	ENGINE_API virtual int32 GetLODLevel() const;

public:
	/** Access a read only version of the Updater Counter from the AnimInstanceProxy on the GameThread. */
	ENGINE_API const FGraphTraversalCounter& GetUpdateCounter() const; 

	/** Access the required bones array */
	ENGINE_API FBoneContainer& GetRequiredBones();
	ENGINE_API const FBoneContainer& GetRequiredBones() const;
	ENGINE_API const FBoneContainer& GetRequiredBonesOnAnyThread() const;

	/** Pending teleport type, set in ResetDynamics and cleared in UpdateAnimation */
	ETeleportType PendingDynamicResetTeleportType;
	
	/** ID for this anim instance */
	uint32 AnimInstanceID = 0;

	/** Animation Notifies that has been triggered in the latest tick **/
	UPROPERTY(transient)
	FAnimNotifyQueue NotifyQueue;

	/** Currently Active AnimNotifyState, stored as a copy of the event as we need to
		call NotifyEnd on the event after a deletion in the editor. After this the event
		is removed correctly. */
	UPROPERTY(transient)
	TArray<FAnimNotifyEvent> ActiveAnimNotifyState;

	UPROPERTY(transient)
	TArray<FAnimNotifyEventReference> ActiveAnimNotifyEventReference;
	
private:
	/** Reset Animation Curves */
	ENGINE_API void ResetAnimationCurves();

public: 
	/** Pushes blended heap curve to output curves in the proxy using required bones cached data */
	ENGINE_API void UpdateCurvesToEvaluationContext(const FAnimationEvaluationContext& InOutContext);

	/** Update curves once evaluation has taken place. Mostly pushes curves to materials/morphs */
	ENGINE_API void UpdateCurvesPostEvaluation();

	/** Swap curves out for evaluation */
	ENGINE_API void SwapCurveWithEvaluationContext(FAnimationEvaluationContext& InOutContext);

	/** Update all internal curves from Blended Curve */
	ENGINE_API void UpdateCurves(const FBlendedHeapCurve& InCurves);

	/** Copy curves from external source */
	ENGINE_API void CopyCurveValues(const UAnimInstance& InSourceInstance);

	/** Refresh currently existing curves */
	ENGINE_API void RefreshCurves(USkeletalMeshComponent* Component);

	/** Check whether we have active morph target curves */
	ENGINE_API bool HasMorphTargetCurves() const;

	/** Check whether we have any active curves */
	ENGINE_API bool HasActiveCurves() const;

	/** Get the current delta time */
	UFUNCTION(BlueprintPure, Category="Animation", meta=(BlueprintThreadSafe))
	ENGINE_API float GetDeltaSeconds() const;
	
	/** 
	 * Append the type of curve to the OutCurveList specified by Curve Flags
	 */
	ENGINE_API void AppendAnimationCurveList(EAnimCurveType Type, TMap<FName, float>& InOutCurveList) const;


	UE_DEPRECATED(4.19, "This function is deprecated. Use AppendAnimationCurveList instead.")
	ENGINE_API void GetAnimationCurveList(EAnimCurveType Type, TMap<FName, float>& InOutCurveList) const;
	/**
	 *	Return the list of curves that are specified by type 
	 */
	ENGINE_API const TMap<FName, float>& GetAnimationCurveList(EAnimCurveType Type) const;

#if WITH_EDITORONLY_DATA
	// Maximum playback position ever reached (only used when debugging in Persona)
	double LifeTimer;

	// Current scrubbing playback position (only used when debugging in Persona)
	double CurrentLifeTimerScrubPosition;
#endif

public:
	FGraphTraversalCounter DebugDataCounter;

private:

	// Used to prevent indefinitely flushing montages on a single frame. 
	uint32 LastMontageFlushFrame = 0u;
	
	TMap<FName, FMontageActiveSlotTracker> SlotWeightTracker;
	TMap<FName, FSimpleMulticastDelegate> ExternalNotifyHandlers;

	ENGINE_API bool CheckOnInstanceAndMainInstance(TFunctionRef<bool (FAnimInstanceProxy* )> ProxyLambdaFunc);

public:

	// Gets the last frame any completed montages where flushed. Only important if a.Montage.FlushCompletedMontagesOnPlay is enabled.
	uint32 GetLastMontageFlushFrame() const;

	/** 
	 * Recalculate Required Bones [RequiredBones]
	 * Is called when bRequiredBonesUpToDate = false
	 */
	ENGINE_API void RecalcRequiredBones();

	/**
	* Recalculate Required Curves based on Required Bones [RequiredBones]
	*/
	ENGINE_API void RecalcRequiredCurves(const UE::Anim::FCurveFilterSettings& CurveFilterSettings);

	/** Returns the associated skeletal mesh component. This will always be non-NULL. */
	inline USkeletalMeshComponent* GetSkelMeshComponent() const
	{
		return GetSkelMeshComponentChecked();
	}

	ENGINE_API virtual UWorld* GetWorld() const override;

	/** Trigger AnimNotifies **/
	ENGINE_API void TriggerAnimNotifies(float DeltaSeconds, ENotifyTriggerMode Mode = ENotifyTriggerMode::Default);

	/** Trigger an AnimNotify.  Note that this version does not provide any context for the event **/ 
	ENGINE_API void TriggerSingleAnimNotify(const FAnimNotifyEvent* AnimNotifyEvent);

	/** Trigger an AnimNotify using an EventReference that provide context used in derived notify events**/ 
	ENGINE_API void TriggerSingleAnimNotify(FAnimNotifyEventReference& EventReference);

	/** Triggers end on active notify states and clears the array */
	ENGINE_API void EndNotifyStates();

	/** Given a machine index, record a state machine weight for this frame */
	ENGINE_API void RecordMachineWeight(const int32 InMachineClassIndex, const float InMachineWeight);
	
	/** 
	 * Add curve float data, using a curve name.
	 */
	ENGINE_API void AddCurveValue(const FName& CurveName, float Value, bool bMorphtarget = false, bool bMaterial = false);

	/** Given a machine and state index, record a state weight for this frame */
	ENGINE_API void RecordStateWeight(const int32 InMachineClassIndex, const int32 InStateIndex, const float InStateWeight, const float InElapsedTime);

protected:
#if WITH_EDITORONLY_DATA
	// Returns true if a snapshot is being played back and the remainder of Update should be skipped.
	ENGINE_API bool UpdateSnapshotAndSkipRemainingUpdate();
#endif

	/** Implementable custom function to handle notifies */
	ENGINE_API virtual bool HandleNotify(const FAnimNotifyEvent& AnimNotifyEvent);

	/** Returns the associated skeletal mesh component. Will assert if invalid. */
	ENGINE_API USkeletalMeshComponent* GetSkelMeshComponentChecked() const;

	/** Returns the associated skeletal mesh component, or NULL if invalid/unset. */
	ENGINE_API USkeletalMeshComponent* GetSkelMeshComponentUnchecked() const;

	// Root Motion
public:
	/** Get current RootMotion FAnimMontageInstance if any. NULL otherwise. */
	ENGINE_API FAnimMontageInstance * GetRootMotionMontageInstance() const;

	/** Get current accumulated root motion, removing it from the AnimInstance in the process */
	ENGINE_API FRootMotionMovementParams ConsumeExtractedRootMotion(float Alpha);

	/**  
	 * Queue blended root motion. This is used to blend in root motion transforms according to 
	 * the correctly-updated slot weight (after the animation graph has been updated).
	 */
	ENGINE_API void QueueRootMotionBlend(const FTransform& RootTransform, const FName& SlotName, float Weight);

private:
	/** Active Root Motion Montage Instance, if any. */
	struct FAnimMontageInstance* RootMotionMontageInstance;

	/** Temporarily queued root motion blend */
	struct FQueuedRootMotionBlend
	{
		FQueuedRootMotionBlend(const FTransform& InTransform, const FName& InSlotName, float InWeight)
			: Transform(InTransform)
			, SlotName(InSlotName)
			, Weight(InWeight)
		{}

		FTransform Transform;
		FName SlotName;
		float Weight;
	};

	/** 
	 * Blend queue for blended root motion. This is used to blend in root motion transforms according to 
	 * the correctly-updated slot weight (after the animation graph has been updated).
	 */
	TArray<FQueuedRootMotionBlend> RootMotionBlendQueue;

	// Root motion read from proxy (where it is calculated) and stored here to avoid potential stalls by calling GetProxyOnGameThread
	FRootMotionMovementParams ExtractedRootMotion;

private:
	// update montage
	ENGINE_API void UpdateMontage(float DeltaSeconds);
	ENGINE_API void UpdateMontageSyncGroup();

public:
	ENGINE_API void TickMontageOnly(float DeltaSeconds);

protected:
	// Updates the montage data used for evaluation based on the current playing montages
	ENGINE_API void UpdateMontageEvaluationData();

	/** Called to setup for updates */
	ENGINE_API virtual void PreUpdateAnimation(float DeltaSeconds);

	/** update animation curves to component */
	ENGINE_API void UpdateCurvesToComponents(USkeletalMeshComponent* Component);

	/** Override point for derived classes to create their own proxy objects (allows custom allocation) */
	ENGINE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy();

	/** Override point for derived classes to destroy their own proxy objects (allows custom allocation) */
	ENGINE_API virtual void DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy);

	/** Access the proxy but block if a task is currently in progress as it wouldn't be safe to access it 
	 *	This is protected static member for allowing derived to access
	 */
	template <typename T /*= FAnimInstanceProxy*/>	// @TODO: Cant default parameters to this function on Xbox One until we move off the VS2012 compiler
	inline static T* GetProxyOnGameThreadStatic(UAnimInstance* InAnimInstance)
	{
		if (InAnimInstance)
		{
			check(IsInGameThread());
			UObject* OuterObj = InAnimInstance->GetOuter();
			if (IsSkeletalMeshComponent(OuterObj))
			{
				HandleExistingParallelEvaluationTask(InAnimInstance->GetSkelMeshComponent());
			}
			if (InAnimInstance->AnimInstanceProxy == nullptr)
			{
				InAnimInstance->AnimInstanceProxy = InAnimInstance->CreateAnimInstanceProxy();
			}
			return static_cast<T*>(InAnimInstance->AnimInstanceProxy);
		}

		return nullptr;
	}
	/** Access the proxy but block if a task is currently in progress as it wouldn't be safe to access it */
	template <typename T /*= FAnimInstanceProxy*/>	// @TODO: Cant default parameters to this function on Xbox One until we move off the VS2012 compiler
	inline T& GetProxyOnGameThread()
	{
		return *GetProxyOnGameThreadStatic<T>(this);
	}

	/** Access the proxy but block if a task is currently in progress as it wouldn't be safe to access it */
	template <typename T/* = FAnimInstanceProxy*/>	// @TODO: Cant default parameters to this function on Xbox One until we move off the VS2012 compiler
	inline const T& GetProxyOnGameThread() const
	{
		check(IsInGameThread());
		if(IsSkeletalMeshComponent(GetOuter()))
		{
			HandleExistingParallelEvaluationTask(GetSkelMeshComponent());
		}
		if(AnimInstanceProxy == nullptr)
		{
			AnimInstanceProxy = const_cast<UAnimInstance*>(this)->CreateAnimInstanceProxy();
		}
		return *static_cast<const T*>(AnimInstanceProxy);
	}

	/** Access the proxy but block if a task is currently in progress (and we are on the game thread) as it wouldn't be safe to access it */
	template <typename T/* = FAnimInstanceProxy*/>	// @TODO: Cant default parameters to this function on Xbox One until we move off the VS2012 compiler
	inline T& GetProxyOnAnyThread()
	{
		if(IsSkeletalMeshComponent(GetOuter()))
		{
			if(IsInGameThread())
			{
				HandleExistingParallelEvaluationTask(GetSkelMeshComponent());
			}
		}
		if(AnimInstanceProxy == nullptr)
		{
			AnimInstanceProxy = CreateAnimInstanceProxy();
		}
		return *static_cast<T*>(AnimInstanceProxy);
	}

	/** Access the proxy but block if a task is currently in progress (and we are on the game thread) as it wouldn't be safe to access it */
	template <typename T/* = FAnimInstanceProxy*/>	// @TODO: Cant default parameters to this function on Xbox One until we move off the VS2012 compiler
	inline const T& GetProxyOnAnyThread() const
	{
		if(IsSkeletalMeshComponent(GetOuter()))
		{
			if(IsInGameThread())
			{
				HandleExistingParallelEvaluationTask(GetSkelMeshComponent());
			}
		}
		if(AnimInstanceProxy == nullptr)
		{
			AnimInstanceProxy = const_cast<UAnimInstance*>(this)->CreateAnimInstanceProxy();
		}
		return *static_cast<const T*>(AnimInstanceProxy);
	}

	friend struct FAnimNode_LinkedAnimGraph;
	friend struct FAnimNode_LinkedAnimLayer;
	friend struct FAnimInstanceProxy;
	
public:
	/** Return whether this AnimNotifyState should be triggered */
	ENGINE_API virtual bool ShouldTriggerAnimNotifyState(const UAnimNotifyState* AnimNotifyState) const;

protected:
	static ENGINE_API bool IsSkeletalMeshComponent(const UObject* Object);
	static ENGINE_API void HandleExistingParallelEvaluationTask(USkeletalMeshComponent* Component);

	/** Proxy object, nothing should access this from an externally-callable API as it is used as a scratch area on worker threads */
	mutable FAnimInstanceProxy* AnimInstanceProxy;

public:
	/** Called when a montage hits a 'PlayMontageNotify' or 'PlayMontageNotifyWindow' begin */
	FPlayMontageAnimNotifyDelegate OnPlayMontageNotifyBegin;

	/** Called when a montage hits a 'PlayMontageNotify' or 'PlayMontageNotifyWindow' end */
	FPlayMontageAnimNotifyDelegate OnPlayMontageNotifyEnd;

public:
	/** Dispatch AnimEvents (AnimNotifies, Montage Events) queued during UpdateAnimation() */
	ENGINE_API void DispatchQueuedAnimEvents();

	/** Clear AnimEvents (AnimNotifies, Montage Events) queued. Note this is mainly needed for very specific scenarios where our notify queue is populated more than once per tick, avoid calling it if not needed. */
	ENGINE_API void ClearQueuedAnimEvents(bool bShouldUpdateActiveAnimNotifiesSinceLastTick);
	
#if WITH_EDITOR
	// Helper function to handle reinstancing in editor
	ENGINE_API virtual void HandleObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Returns true if this anim instance is actively being debugged, false otherwise */
	ENGINE_API bool IsBeingDebugged() const;
#endif
};
