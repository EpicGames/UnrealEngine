// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AnimLayerInterface.generated.h"

/** An interface used to specify the inputs and outputs of an animation blueprint */
UINTERFACE(MinimalAPI)
class UAnimLayerInterface : public UInterface
{
	GENERATED_BODY()
};

class IAnimLayerInterface
{
	GENERATED_BODY()
};
