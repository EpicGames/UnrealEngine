// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimLinkableElement.generated.h"

class UAnimMontage;
class UAnimSequenceBase;
struct FAnimSegment;

/** Supported types of time for a linked element */
UENUM()
namespace EAnimLinkMethod
{
	enum Type : int
	{
		/** Element stays at a specific time without moving. */
		Absolute,
		/** Element moves with its segment, but not when the segment changes size. */
		Relative,
		/** Element moves with its segment and will stay at a certain proportion through the segment. */
		Proportional
	};
}

/** Used to describe an element that can be linked to a segment in a montage or sequence.
 *	Usage: 
 *		Inherit from FAnimLinkableElement and make sure to call LinkMontage or LinkSequence
 *		both on creation and on loading the element. From there SetTime and GetTime should be
 *		used to control where this element is in the montage or sequence.
 *	
 *		For more advanced usage, see this implementation used in FAnimNotifyEvent where
 *		we have a secondary link to handle a duration
 *		@see FAnimNotifyEvent
 */
USTRUCT()
struct FAnimLinkableElement
{
	GENERATED_USTRUCT_BODY()

	FAnimLinkableElement()
	: LinkedMontage(nullptr)
	, SlotIndex(0)
	, SegmentIndex(INDEX_NONE)
	, LinkMethod(EAnimLinkMethod::Absolute)
	, CachedLinkMethod(LinkMethod)
	, SegmentBeginTime(0.f)
	, SegmentLength(0.f)
	, LinkValue(0.f)
	, LinkedSequence(nullptr)
	{
	}

	virtual ~FAnimLinkableElement() = default;

	/** Update the timing information for this element. Will not search for a new segment unless this element does not have one */
	ENGINE_API void Update();

	/** Link this element to an animation object (Sequence or Montage)
	 *	@param AnimSequenceBase The object to link to
	 *	@param AbsTime The absolute time to place this element
	 *	@param InSlotIndex Slot index for montages (ignored otherwise)
	 */
	ENGINE_API void Link(UAnimSequenceBase* AnimSequenceBase, float AbsTime, int32 InSlotIndex = 0);

	/** Clear the linking information from this element, leaves montage link intact */
	ENGINE_API void Clear();

	/** Called when the properties of this element are changed */
	ENGINE_API void OnChanged(float NewMontageTime);

	/** Gets the current time for this element 
	 * @param ReferenceFrame What kind of time to return
	 */
	ENGINE_API float GetTime(EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) const;

	/** Sets the time of this element
	 * @param NewTime The time to set this element to
	 * @param ReferenceFrame The kind of time being passed to this method
	 */
	ENGINE_API virtual void SetTime(float NewTime, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute);

	/** Gets the sequence this element is linked to */
	const UAnimSequenceBase* GetLinkedSequence() const {return LinkedSequence;}

	/** Gets the Montage this element is linked to, if any */
	const UAnimMontage* GetLinkedMontage() const { return LinkedMontage; }

	/** Changes the way this element is linked to its segment
	 * @param NewLinkMethod The new linking method to use
	 */
	ENGINE_API void ChangeLinkMethod(EAnimLinkMethod::Type NewLinkMethod);

	/** Change the montage slot that this element is linked to
	 * @param NewSlotIndex The new slot to link to
	 */
	ENGINE_API void ChangeSlotIndex(int32 NewSlotIndex);

	/** Get the method used to link this element to its segment */
	EAnimLinkMethod::Type GetLinkMethod() const {return LinkMethod;}

	/** Get the slot index this element is currently linked to */
	int32 GetSlotIndex() const {return SlotIndex;}
	
	/** Get the index of the segment this element is currently linked to */
	int32 GetSegmentIndex() const {return SegmentIndex;}

	/** Directly sets segment index
	 *	@param NewSegmentIndex New segment index
	 */
	void SetSegmentIndex(int32 NewSegmentIndex) {SegmentIndex = NewSegmentIndex;}

	/** Relinks this element if internal state requires relinking */
	ENGINE_API bool ConditionalRelink();

	/** Refreshes the current segment data (Begin time, length etc.) and validate the link time
	 *  Intended to update the internal state when segment lengths/times could have changed
	 */
	ENGINE_API void RefreshSegmentOnLoad();

protected:

	/** Gets the segment in the current montage in the current slot that is at the time of this element */
	FAnimSegment* GetSegmentAtCurrentTime();

	/** The montage that this element is currently linked to */
	UPROPERTY()
	TObjectPtr<UAnimMontage> LinkedMontage;

	/** The slot index we are currently using within LinkedMontage */
	UPROPERTY(EditAnywhere, Category=AnimLink)
	int32 SlotIndex;

	/** The index of the segment we are linked to within the slot we are using */
	UPROPERTY()
	int32 SegmentIndex;

	/** The method we are using to calculate our times */
	UPROPERTY(EditAnywhere, Category=AnimLink)
	TEnumAsByte<EAnimLinkMethod::Type> LinkMethod;

	/** Cached link method used to transform the time when LinkMethod changes, always relates to the currently stored time */
	UPROPERTY()
	TEnumAsByte<EAnimLinkMethod::Type> CachedLinkMethod;

	/** The absolute time in the montage that our currently linked segment begins */
	UPROPERTY()
	float SegmentBeginTime;

	/** The absolute length of our currently linked segment */
	UPROPERTY()
	float SegmentLength;

	/** The time of this montage. This will differ depending upon the method we are using to link the time for this element */
	UPROPERTY()
	float LinkValue;

	/** 
	 * The Animation Sequence that this montage element will link to, when the sequence changes
	 * in either length or rate; the element will correctly place itself in relation to the sequence
	 */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category=AnimLink)
	TObjectPtr<UAnimSequenceBase> LinkedSequence;

private:

	/** Gets a time from an absolute LinkValue
	 * @param NewLinkMethod The type of time to return
	 */
	float GetTimeFromAbsolute(EAnimLinkMethod::Type ReferenceFrame) const ;

	/** Gets a time from a relative LinkValue
	 * @param NewLinkMethod The type of time to return
	 */
	float GetTimeFromRelative(EAnimLinkMethod::Type ReferenceFrame) const ;

	/** Gets a time from a proportional LinkValue
	 * @param NewLinkMethod The type of time to return
	 */
	float GetTimeFromProportional(EAnimLinkMethod::Type ReferenceFrame) const;

	/** Set the LinkValue to a converted absolute time
	 * @param NewTime The new time to set
	 * @param NewLinkMethod The type of time passed to this method
	 */
	void SetTimeFromAbsolute(float NewTime, EAnimLinkMethod::Type ReferenceFrame);

	/** Set the LinkValue to a converted relative time
	 * @param NewTime The new time to set
	 * @param NewLinkMethod The type of time passed to this method
	 */
	void SetTimeFromRelative(float NewTime, EAnimLinkMethod::Type ReferenceFrame);

	/** Set the LinkValue to a converted proportional time
	 * @param NewTime The new time to set
	 * @param NewLinkMethod The type of time passed to this method
	 */
	void SetTimeFromProportional(float NewTime, EAnimLinkMethod::Type ReferenceFrame);

	/** Sets the LinkValue based on NewTime. Internal version to ensure overridden behavior isn't used when transforming a time
	 * @param NewTime New time to set
	 * @param ReferenceFrame The kind of time passed in to this method
	 */
	void SetTime_Internal(float NewTime, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute);
};
