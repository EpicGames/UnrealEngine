// Copyright Epic Games, Inc. All Rights Reserved.

/* 
 * Base Class for UAnimMetaData that can be implemented for each game needs
 * this data will be saved to animation asset as well as montage sections, and you can query that data and decide what to do
 *
 * Refer : GetMetaData/GetSectionMetaData
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AnimMetaData.generated.h"

UCLASS(Blueprintable, abstract, editinlinenew, hidecategories=Object, collapsecategories, MinimalAPI)
class UAnimMetaData : public UObject
{
	GENERATED_UCLASS_BODY()
};



