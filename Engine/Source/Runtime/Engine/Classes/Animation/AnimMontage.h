// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Abstract base class of animation made of multiple sequences.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/AnimLinkableElement.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "AlphaBlend.h"
#include "Animation/AnimCompositeBase.h"
#include "Animation/TimeStretchCurve.h"
#include "AnimMontage.generated.h"

class UAnimInstance;
class UAnimMontage;
class UAnimNotifyState;
class UAnimSequence;
class UBlendProfile;
class USkeletalMeshComponent;

enum class EBlendProfileMode : uint8;


/**
 * Section data for each track. Reference of data will be stored in the child class for the way they want
 * AnimComposite vs AnimMontage have different requirement for the actual data reference
 * This only contains composite section information. (vertical sequences)
 */
USTRUCT()
struct FCompositeSection : public FAnimLinkableElement
{
	GENERATED_USTRUCT_BODY()

	/** Section Name */
	UPROPERTY(EditAnywhere, Category=Section)
	FName SectionName;

#if WITH_EDITORONLY_DATA
	/** Start Time **/
	UPROPERTY()
	float StartTime_DEPRECATED;
#endif

	/** Should this animation loop. */
	UPROPERTY(VisibleAnywhere, Category=Section)
	FName NextSectionName;

	/** Meta data that can be saved with the asset 
	 * 
	 * You can query by GetMetaData function
	 */
	UPROPERTY(Category=Section, Instanced, EditAnywhere)
	TArray<TObjectPtr<class UAnimMetaData>> MetaData;

public:
	FCompositeSection()
		: FAnimLinkableElement()
		, SectionName(NAME_None)
#if WITH_EDITORONLY_DATA
		, StartTime_DEPRECATED(0.0f)
#endif
		, NextSectionName(NAME_None)
	{
	}

	/** Get available Metadata for this section
	 */
	const TArray<class UAnimMetaData*>& GetMetaData() const { return MetaData; }
};

/**
 * Each slot data referenced by Animation Slot 
 * contains slot name, and animation data 
 */
USTRUCT(BlueprintType)
struct FSlotAnimationTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Slot)
	FName SlotName;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Slot)
	FAnimTrack AnimTrack;

	ENGINE_API FSlotAnimationTrack();
};

/** 
 * Remove FBranchingPoint when VER_UE4_MONTAGE_BRANCHING_POINT_REMOVAL is removed.
 */
USTRUCT()
struct FBranchingPoint : public FAnimLinkableElement
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=BranchingPoint)
	FName EventName;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float DisplayTime_DEPRECATED = 0.f;
#endif

	/** An offset from the DisplayTime to the actual time we will trigger the notify, as we cannot always trigger it exactly at the time the user wants */
	UPROPERTY()
	float TriggerTimeOffset = 0.f;

	/** Returns the time this branching point should be triggered */
	float GetTriggerTime() const { return GetTime() + TriggerTimeOffset; }
};

/**  */
UENUM()
namespace EAnimNotifyEventType
{
	enum Type : int
	{
		/** */
		Begin,
		/** */
		End,
	};
}

UENUM()
enum class EMontageBlendMode : uint8
{
	//Uses standard weight based blend
	Standard,
	//Uses inertialization. Requires an inertialization node somewhere in the graph after any slot node used by this montage.
	Inertialization,
};

/** AnimNotifies marked as BranchingPoints will create these markers on their Begin/End times.
	They create stopping points when the Montage is being ticked to dispatch events. */
USTRUCT()
struct FBranchingPointMarker
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 NotifyIndex;
	
	UPROPERTY()
	float TriggerTime;

	UPROPERTY()
	TEnumAsByte<EAnimNotifyEventType::Type> NotifyEventType;

	FBranchingPointMarker()
		: NotifyIndex(INDEX_NONE)
		, TriggerTime(0.f)
		, NotifyEventType(EAnimNotifyEventType::Begin)
	{
	}

	FBranchingPointMarker(int32 InNotifyIndex, float InTriggerTime, EAnimNotifyEventType::Type InNotifyEventType)
		: NotifyIndex(InNotifyIndex)
		, TriggerTime(InTriggerTime)
		, NotifyEventType(InNotifyEventType)
	{
	}
};

UENUM()
enum class EMontageSubStepResult : uint8
{
	Moved,
	NotMoved,
	InvalidSection,
	InvalidMontage,
};

/**
 * Delegate for when Montage is completed, whether interrupted or finished
 * Weight of this montage is 0.f, so it stops contributing to output pose
 *
 * bInterrupted = true if it was not property finished
 */
DECLARE_DELEGATE_TwoParams( FOnMontageEnded, class UAnimMontage*, bool /*bInterrupted*/) 
/**
 * Delegate for when Montage started to blend out, whether interrupted or finished
 * DesiredWeight of this montage becomes 0.f, but this still contributes to the output pose
 * 
 * bInterrupted = true if it was not property finished
 */
DECLARE_DELEGATE_TwoParams( FOnMontageBlendingOutStarted, class UAnimMontage*, bool /*bInterrupted*/) 

DECLARE_DELEGATE_OneParam(FOnMontageBlendedInEnded, class UAnimMontage*)

/**
 * Delegate for when Montage changes section.
 *
 * bLooped = true if a requested to change to the current section.
 */
DECLARE_DELEGATE_ThreeParams( FOnMontageSectionChanged, class UAnimMontage*, FName /*SectionName*/ , bool /*bInterrupted*/)

/* Delegate allows a c++ client to get a callback when it's time to advance the montage frame time.
 * It then returns the DeltaTime to use. Currently used to drive montages according to a music clock timeline
 * rather than advancing through world time. Note that playrate both at the montage and segment level
 * are basically ignored by this system, because the current use does not require it.
 */
// Experimental 5.8, FComputeDeltaFrameTime is experimental, and may change in the future.
DECLARE_DELEGATE_RetVal_ThreeParams(float, FComputeDeltaFrameTime, int32 /*InstanceId*/, float /*CurrentTime*/, float /*DeltaTime*/)

/**
	Helper struct to sub step through Montages when advancing time.
	These require stopping at sections and branching points to potential jumps and loops.
	And also stepping through TimeStretchMarkers to adjust play rate based on TimeStretchCurve.
 */
struct FMontageSubStepper
{
private:
	const struct FAnimMontageInstance* MontageInstance;
	const class UAnimMontage* Montage;

	float TimeRemaining;
	float Cached_CombinedPlayRate;
	float PlayRate;
	float DeltaMove;
	bool bPlayingForward;

	int32 CurrentSectionIndex;
	float CurrentSectionStartTime;
	float CurrentSectionLength;
	bool bReachedEndOfSection;
	bool bHasValidTimeStretchCurveData;

	int32 TimeStretchMarkerIndex;

	mutable TArray<float> SectionStartPositions_Target;
	mutable TArray<float> SectionEndPositions_Target;

	float Cached_P_Target;
	float Cached_P_Original;

	FTimeStretchCurveInstance TimeStretchCurveInstance;

public:
	FMontageSubStepper()
		: MontageInstance(nullptr)
		, Montage(nullptr)
		, TimeRemaining(0.f)
		, Cached_CombinedPlayRate(0.f)
		, PlayRate(0.f)
		, DeltaMove(0.f)
		, bPlayingForward(true)
		, CurrentSectionIndex(INDEX_NONE)
		, CurrentSectionStartTime(0.f)
		, CurrentSectionLength(0.f)
		, bReachedEndOfSection(false)
		, bHasValidTimeStretchCurveData(false)
		, TimeStretchMarkerIndex(INDEX_NONE)
		, Cached_P_Target(FLT_MAX)
		, Cached_P_Original(FLT_MAX)
	{}

	void Initialize(const struct FAnimMontageInstance& InAnimInstance);

	void AddEvaluationTime(float InDeltaTime) { TimeRemaining += InDeltaTime; }
	bool HasTimeRemaining() const { return (TimeRemaining > UE_SMALL_NUMBER); }
	float GetRemainingTime() const { return TimeRemaining; }
	EMontageSubStepResult Advance(float& InOut_P_Original, const FBranchingPointMarker** OutBranchingPointMarkerPtr);
	bool HasReachedEndOfSection() const { return bReachedEndOfSection; }
	float GetRemainingPlayTimeToSectionEnd(const float In_P_Original);

	bool GetbPlayingForward() const { return bPlayingForward; }
	float GetDeltaMove() const { return DeltaMove; }
	int32 GetCurrentSectionIndex() const { return CurrentSectionIndex; }

	/** Invalidate Cached_CombinedPlayRate to force data to be recached in 'ConditionallyUpdateCachedData' */
	void ClearCachedData() { Cached_CombinedPlayRate = FLT_MAX; }

private:
	/** 
		Updates markers P_Marker_Original and P_Marker_Target
		*only* if T_Target has changed. 
	*/
	void ConditionallyUpdateTimeStretchCurveCachedData();

	/** 
		Finds montage position in 'target' space, given current position in 'original' space.
		This means given a montage position, we find its playback time.
		This should only be used for montage position, as we cache results and lazily update it for performance.
	*/
	float FindMontagePosition_Target(float In_P_Original);

	/**
		Finds montage position in 'original' space, given current position in 'target' space.
		This means given a montage playback time, we find its actual position.
		This should only be used for montage position, as we cache results and lazily update it for performance.
	*/
	float FindMontagePosition_Original(float In_P_Target);

	/** 
		Gets current section end position in target space. 
		this is lazily cached, as it can be called every frame to test when to blend out. 
	*/
	float GetCurrSectionEndPosition_Target() const;

	/** 
		Gets current section start position in target space.
		this is lazily cached, as it can be called every frame to test when to blend out. 
	*/
	float GetCurrSectionStartPosition_Target() const;
};

/**
* Montage blend settings. Can be used to overwrite default Montage settings on Play/Stop
*/
USTRUCT(BlueprintType)
struct FMontageBlendSettings
{
	GENERATED_BODY()

	ENGINE_API FMontageBlendSettings();
	ENGINE_API FMontageBlendSettings(float BlendTime);
	ENGINE_API FMontageBlendSettings(const FAlphaBlendArgs& BlendArgs);

	/** Blend Profile to use for this blend */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (DisplayAfter = "Blend"))
	TObjectPtr<UBlendProfile> BlendProfile;

	/** AlphaBlend options (time, curve, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (DisplayAfter = "BlendMode"))
	FAlphaBlendArgs Blend;

	/** Type of blend mode (Standard vs Inertial) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend")
	EMontageBlendMode BlendMode;
};

USTRUCT()
struct FAnimMontageInstance
{
	GENERATED_USTRUCT_BODY()

	friend struct FMontageSubStepper;

	// Montage reference
	UPROPERTY()
	TObjectPtr<class UAnimMontage> Montage;

	// delegates
	FOnMontageEnded OnMontageEnded;
	FOnMontageBlendingOutStarted OnMontageBlendingOutStarted;
	FOnMontageBlendedInEnded OnMontageBlendedInEnded;
	FOnMontageSectionChanged OnMontageSectionChanged;

	UE_EXPERIMENTAL(5.8, "FComputeDeltaFrameTime is experimental, and may change in the future.")
	FComputeDeltaFrameTime OnComputeDeltaFrameTime;

	UPROPERTY()
	bool bPlaying;

	// Blend Time multiplier to allow extending and narrowing blendtimes
	UPROPERTY(transient)
	float DefaultBlendTimeMultiplier;

	// transient value of time position and delta in the last frame known
	FDeltaTimeRecord DeltaTimeRecord;

	// marker tick record
	FMarkerTickRecord MarkerTickRecord;

	// markers that passed in this tick
	TArray<FPassedMarker> MarkersPassedThisTick;

	// Whether this in this tick's call to Advance we used marker based sync
	bool bDidUseMarkerSyncThisTick;

	// enable auto blend out. This is instance set up. You can override
	bool bEnableAutoBlendOut;

private:
	struct FMontageSubStepper MontageSubStepper;

	// list of next sections per section - index of array is section id
	UPROPERTY()
	TArray<int32> NextSections;

	// list of prev sections per section - index of array is section id
	UPROPERTY()
	TArray<int32> PrevSections;

	// reference to AnimInstance
	TWeakObjectPtr<UAnimInstance> AnimInstance;

	// Unique ID for this instance
	int32 InstanceID;

	/** Currently Active AnimNotifyState, stored as a copy of the event as we need to
		call NotifyEnd on the event after a deletion in the editor. After this the event
		is removed correctly. */
	UPROPERTY(Transient)
	TArray<FAnimNotifyEvent> ActiveStateBranchingPoints;

	UPROPERTY()
	float Position;

	UPROPERTY()
	float PlayRate;

	UPROPERTY(transient)
	FAlphaBlend Blend;

	// need to save if it's interrupted or not
	// this information is crucial for gameplay
	bool bInterrupted;

	// transient PreviousWeight - Weight of previous tick
	float PreviousWeight;

	// transient NotifyWeight   - Weight for spawned notifies, modified slightly to make sure
	//                          - we spawn all notifies
	float NotifyWeight;

	// The current start linear alpha value of the blend. This is not stored inside the FAlphaBlend struct.
	float BlendStartAlpha;

	// sync group name
	FName SyncGroupName;

	// Active blend profile.
	UBlendProfile* ActiveBlendProfile;
	EBlendProfileMode ActiveBlendProfileMode;

	/**
	 * Optional evaluation range to use next update (ignoring the real delta time).
	 * Used by external systems that are setting animation times directly. Will fire off notifies and other events provided the animation system is ticking.
	 */
	TOptional<float> ForcedNextFromPosition;
	TOptional<float> ForcedNextToPosition;
	TOptional<float> ForcedLoopBoundaryPosition;
	TOptional<float> ForcedLoopResumePosition;

	UPROPERTY(Transient)
	int32 DisableRootMotionCount;

public:
	/** Montage to Montage Synchronization.
	 *
	 * A montage can only have a single leader. A leader can have multiple followers.
	 * Loops cause no harm.
	 * If Follower gets ticked before Leader, then synchronization will be performed with a frame of lag.
	 *		Essentially correcting the previous frame. Which is enough for simple cases (i.e. no timeline jumps from notifies).
	 * If Follower gets ticked after Leader, then synchronization will be exact and support more complex cases (i.e. timeline jumps).
	 *		This can be enforced by setting up tick pre-requisites if desired.
	 */
	ENGINE_API void MontageSync_Follow(struct FAnimMontageInstance* NewLeaderMontageInstance);
	/** Stop leading, release all followers. */
	ENGINE_API void MontageSync_StopLeading();
	/** Stop following our leader */
	ENGINE_API void MontageSync_StopFollowing();
	/** PreUpdate - Sync if updated before Leader. */
	ENGINE_API void MontageSync_PreUpdate();
	/** PostUpdate - Sync if updated after Leader. */
	ENGINE_API void MontageSync_PostUpdate();

	/** Get Weight */
	float GetWeight() const { return Blend.GetBlendedValue(); }
	float GetDesiredWeight() const { return Blend.GetDesiredValue(); }
	float GetBlendTime() const { return Blend.GetBlendTime(); }
	UE_DEPRECATED(4.26, "Please use GetSyncGroupName")
	int32 GetSyncGroupIndex() const { return INDEX_NONE;  }
	FName GetSyncGroupName() const { return SyncGroupName;  }

	/** Set the weight */
	void SetWeight(float InValue) { Blend.SetAlpha(InValue); }
	/** Set the Desired Weight */
	void SetDesiredWeight(float InValue) { Blend.SetDesiredValue(InValue); }

	/** Get the current blend info. */
	const FAlphaBlend& GetBlend() const { return Blend; }

private:
	/** Followers this Montage will synchronize */
	TArray<struct FAnimMontageInstance*> MontageSyncFollowers;
	/** Leader this Montage will follow */
	struct FAnimMontageInstance* MontageSyncLeader;
	/** Frame counter to sync montages once per frame */
	uint32 MontageSyncUpdateFrameCounter;

	/** true if montage has been updated this frame */
	ENGINE_API bool MontageSync_HasBeenUpdatedThisFrame() const;
	/** This frame's counter, to track which Montages have been updated */
	ENGINE_API uint32 MontageSync_GetFrameCounter() const;
	/** Synchronize ourselves to our leader */
	ENGINE_API void MontageSync_PerformSyncToLeader();

	/** Initialize Blend Setup from Montage */
	ENGINE_API void InitializeBlend(const FAlphaBlend& InAlphaBlend);

	/**  Notify may invalidate current montage instance. Inputs should be memory not belonging to calling FAnimMontageInstance.*/
	static ENGINE_API bool ValidateInstanceAfterNotifyState(const TWeakObjectPtr<UAnimInstance>& InAnimInstance, const UAnimNotifyState* InNotifyStateClass);

public:
	ENGINE_API FAnimMontageInstance();

	ENGINE_API FAnimMontageInstance(UAnimInstance * InAnimInstance);

	//~ Begin montage instance Interfaces

	// Blend in with the supplied play rate. Other blend settings will come from the Montage asset.
	ENGINE_API void Play(float InPlayRate = 1.f);
	// Blend in with the supplied blend settings
	ENGINE_API void Play(float InPlayRate, const FMontageBlendSettings& BlendInSettings);

	// Blend out with the supplied FAlphaBlend. Other blend settings will come from the Montage asset.
	ENGINE_API void Stop(const FAlphaBlend& InBlendOut, bool bInterrupt=true);
	// Blend out with the supplied blend settings
	ENGINE_API void Stop(const FMontageBlendSettings& InBlendOutSettings, bool bInterrupt=true);

	ENGINE_API void Pause();
	ENGINE_API void Initialize(class UAnimMontage * InMontage);

	ENGINE_API bool JumpToSectionName(FName const & SectionName, bool bEndOfSection = false);
	ENGINE_API bool SetNextSectionName(FName const & SectionName, FName const & NewNextSectionName);
	ENGINE_API bool SetNextSectionID(int32 const & SectionID, int32 const & NewNextSectionID);

	bool IsValid() const { return (Montage!=nullptr); }
	bool IsPlaying() const { return IsValid() && bPlaying; }
	void SetPlaying(bool bInPlaying) { bPlaying = bInPlaying; }
	bool IsStopped() const { return Blend.GetDesiredValue() == 0.f; }

	/** Returns true if this montage is active (valid and not blending out) */
	bool IsActive() const { return (IsValid() && !IsStopped()); }

	ENGINE_API void Terminate();

	/** return true if it can use marker sync */
	ENGINE_API bool CanUseMarkerSync() const;

	/**
	 *  Getters
	 */
	int32 GetInstanceID() const { return InstanceID; }
	float GetPosition() const { return Position; };
	float GetPlayRate() const { return PlayRate; }
	float GetDeltaMoved() const { return DeltaTimeRecord.Delta; }
	float GetPreviousPosition() const { return DeltaTimeRecord.GetPrevious();  }
	float GetBlendStartAlpha() const { return BlendStartAlpha; }
	const FAnimMontageInstance* GetMontageSyncLeader() const { return MontageSyncLeader; } 
	const UBlendProfile* GetActiveBlendProfile() const { return ActiveBlendProfile; }
	const EBlendProfileMode GetActiveBlendProfileMode() const { return ActiveBlendProfileMode; }

	/** 
	 * Setters
	 */
	void SetPosition(float const & InPosition) { Position = InPosition; MarkerTickRecord.Reset(); }
	void SetPlayRate(float const & InPlayRate) { PlayRate = InPlayRate; }

	// Disable RootMotion
	void PushDisableRootMotion() { DisableRootMotionCount++; }
	void PopDisableRootMotion() { DisableRootMotionCount--; }
	bool IsRootMotionDisabled() const {	return DisableRootMotionCount > 0; }

	/** Set the next position of this animation for the next animation update tick. Will trigger events and notifies since last position. */
	void SetNextPositionWithEvents(float InPosition) { ForcedNextFromPosition.Reset();  ForcedNextToPosition = InPosition; }
	/** Set the evaluation range of this animation for the next animation update tick. Will trigger events and notifies for that range. */
	void SetNextPositionWithEvents(float InFromPosition, float InToPosition) { ForcedNextFromPosition = InFromPosition; ForcedNextToPosition = InToPosition; }

	/**
	 * Montage Tick happens in 2 phases
	 *
	 * first is to update weight of current montage only
	 * this will make sure that all nodes will get up-to-date weight information
	 * when update comes in for them
	 *
	 * second is normal tick. This tick has to happen later when all node ticks
	 * to accumulate and update curve data/notifies/branching points
	 */
	ENGINE_API void UpdateWeight(float DeltaTime);

#if WITH_EDITOR	
	ENGINE_API void EditorOnly_PreAdvance();
#endif

	/** Simulate is same as Advance, but without calling any events or touching any of the instance data. So it performs a simulation of advancing the timeline. */
	ENGINE_API bool SimulateAdvance(float DeltaTime, float& InOutPosition, struct FRootMotionMovementParams & OutRootMotionParams) const;
	ENGINE_API void Advance(float DeltaTime, struct FRootMotionMovementParams * OutRootMotionParams, bool bBlendRootMotion);

	ENGINE_API FName GetCurrentSection() const;
	ENGINE_API FName GetNextSection() const;
	ENGINE_API int32 GetNextSectionID(int32 const & CurrentSectionID) const;
	ENGINE_API FName GetSectionNameFromID(int32 const & SectionID) const;

	// reference has to be managed manually
	ENGINE_API void AddReferencedObjects( FReferenceCollector& Collector );

	/** Delegate function handlers
	 */
	ENGINE_API void HandleEvents(float PreviousTrackPos, float CurrentTrackPos, const FBranchingPointMarker* BranchingPointMarker);

private:
	/** Called by blueprint functions that modify the montages current position. */
	ENGINE_API void OnMontagePositionChanged(FName const & ToSectionName);
	
	/** Updates ActiveStateBranchingPoints array and triggers Begin/End notifications based on CurrentTrackPosition */
	/** Returns false if montage instance was destroyed during branching point update*/
	ENGINE_API bool UpdateActiveStateBranchingPoints(float CurrentTrackPosition);

	/** Trigger associated events when Montage ticking reaches given FBranchingPointMarker */
	ENGINE_API void BranchingPointEventHandler(const FBranchingPointMarker* BranchingPointMarker);
	ENGINE_API void RefreshNextPrevSections();

	ENGINE_API float GetRemainingPlayTimeToSectionEnd(const FMontageSubStepper& MontageSubStepper) const;

public:
	/** static functions that are used by sequencer montage support*/
	static ENGINE_API UAnimMontage* SetSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bPlaying);
	static ENGINE_API UAnimMontage* PreviewSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bFireNotifies, bool bPlaying);
	static ENGINE_API UAnimMontage* SetSequencerMontagePosition(FName SlotName, UAnimInstance* AnimInstance, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bPlaying);
	static ENGINE_API UAnimMontage* PreviewSequencerMontagePosition(FName SlotName, USkeletalMeshComponent* SkeletalMeshComponent, UAnimInstance* AnimInstance, int32& InOutInstanceId, UAnimSequenceBase* InAnimSequence, float InFromPosition, float InToPosition, float Weight, bool bLooping, bool bFireNotifies, bool bPlaying);
};

/**
 * Any property you're adding to AnimMontage and parent class has to be considered for Child Asset
 *
 * Child Asset is considered to be only asset mapping feature using everything else in the class
 * For example, you can just use all parent's setting  for the montage, but only remap assets
 * This isn't magic bullet unfortunately and it is consistent effort of keeping the data synced with parent
 * If you add new property, please make sure those property has to be copied for children.
 * If it does, please add the copy in the function RefreshParentAssetData
 */
UCLASS(config=Engine, hidecategories=(UObject, Length), MinimalAPI, BlueprintType, meta= (LoadBehavior = "LazyOnDemand"))
class UAnimMontage : public UAnimCompositeBase
{
	GENERATED_UCLASS_BODY()

	friend struct FAnimMontageInstance;
	friend class UAnimMontageFactory;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BlendOption)
	EMontageBlendMode BlendModeIn;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BlendOption)
	EMontageBlendMode BlendModeOut;

	/** Blend in option. */
	UPROPERTY(EditAnywhere, Category=BlendOption)
	FAlphaBlend BlendIn;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float BlendInTime_DEPRECATED;
#endif

	/** Blend out option. This is only used when it blends out itself. If it's interrupted by other montages, it will use new montage's BlendIn option to blend out. */
	UPROPERTY(EditAnywhere, Category=BlendOption)
	FAlphaBlend BlendOut;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float BlendOutTime_DEPRECATED;
#endif

	/** Time from Sequence End to trigger blend out.
	 * <0 means using BlendOutTime, so BlendOut finishes as Montage ends.
	 * >=0 means using 'SequenceEnd - BlendOutTriggerTime' to trigger blend out. */
	UPROPERTY(EditAnywhere, Category = BlendOption)
	float BlendOutTriggerTime;

	UFUNCTION(BlueprintPure, Category = "Montage")
	FAlphaBlendArgs GetBlendInArgs() const { return FAlphaBlendArgs(BlendIn); }

	UFUNCTION(BlueprintPure, Category = "Montage")
	FAlphaBlendArgs GetBlendOutArgs() const { return FAlphaBlendArgs(BlendOut); }

	UFUNCTION(BlueprintPure, Category = "Montage")
	float GetDefaultBlendInTime() const { return BlendIn.GetBlendTime(); }

	UFUNCTION(BlueprintCallable, Category = "Montage")
	float GetDefaultBlendOutTime() const { return BlendOut.GetBlendTime(); }

	/** If you're using marker based sync for this montage, make sure to add sync group name. For now we only support one group */
	UPROPERTY(EditAnywhere, Category = SyncGroup)
	FName SyncGroup;

	/** Index of the slot track used for collecting sync markers */
	UPROPERTY(EditAnywhere, Category = SyncGroup)
	int32 SyncSlotIndex;

	UPROPERTY()
	struct FMarkerSyncData	MarkerData;

	// composite section. 
	UPROPERTY()
	TArray<FCompositeSection> CompositeSections;
	
	// slot data, each slot contains anim track
	UPROPERTY(BlueprintReadOnly, Category = "Montage")
	TArray<struct FSlotAnimationTrack> SlotAnimTracks;

#if WITH_EDITORONLY_DATA
	// Remove this when VER_UE4_MONTAGE_BRANCHING_POINT_REMOVAL is removed.
	UPROPERTY()
	TArray<struct FBranchingPoint> BranchingPoints_DEPRECATED;
#endif

	/** If this is on, it will allow extracting root motion translation. DEPRECATED in 4.5 root motion is controlled by anim sequences **/
	UPROPERTY()
	bool bEnableRootMotionTranslation;

	/** If this is on, it will allow extracting root motion rotation. DEPRECATED in 4.5 root motion is controlled by anim sequences **/
	UPROPERTY()
	bool bEnableRootMotionRotation;

	/** When it hits end, it automatically blends out. If this is false, it won't blend out but keep the last pose until stopped explicitly */
	UPROPERTY(EditAnywhere, Category = BlendOption)
	bool bEnableAutoBlendOut;

	/** The blend profile to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BlendOption, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendProfileIn;

	/** The blend profile to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BlendOption, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendProfileOut;

	/** Root Bone will be locked to that position when extracting root motion. DEPRECATED in 4.5 root motion is controlled by anim sequences **/
	UPROPERTY()
	TEnumAsByte<ERootMotionRootLock::Type> RootMotionRootLock;

#if WITH_EDITORONLY_DATA
	/** Preview Base pose for additive BlendSpace **/
	UPROPERTY(EditAnywhere, Category = AdditiveSettings)
	TObjectPtr<UAnimSequence> PreviewBasePose;
#endif // WITH_EDITORONLY_DATA

	// Add new slot track to this montage
	ENGINE_API FSlotAnimationTrack& AddSlot(FName SlotName);

	/** return true if valid slot */
	ENGINE_API bool IsValidSlot(FName InSlotName) const;

	UFUNCTION(BlueprintPure, Category = "Montage")
	ENGINE_API bool IsDynamicMontage() const;

	UFUNCTION(BlueprintPure, Category = "Montage")
	ENGINE_API UAnimSequenceBase* GetFirstAnimReference() const;

public:
	//~ Begin UObject Interface
#if WITH_EDITORONLY_DATA
	virtual void Serialize(FArchive& Ar) override;
#endif // WITH_EDITORONLY_DATA
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	virtual FFrameRate GetSamplingFrameRate() const override;

	// Gets the sequence length of the montage by calculating it from the lengths of the segments in the montage
	ENGINE_API float CalculateSequenceLength();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

	//~ Begin AnimSequenceBase Interface
	virtual bool IsValidAdditive() const override;
#if WITH_EDITOR
	virtual EAnimEventTriggerOffsets::Type CalculateOffsetForNotify(float NotifyDisplayTime) const override;
#endif // WITH_EDITOR
	virtual void GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker) const override;
	virtual FMarkerSyncAnimPosition GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, const UMirrorDataTable* MirrorTable) const override;
	virtual void TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const override;
	virtual TArray<FName>* GetUniqueMarkerNames() override { return &MarkerData.UniqueMarkerNames; }
	virtual void RefreshCacheData() override;
	virtual bool CanBeUsedInComposition() const { return false; }
	virtual void GetAnimationPose(FAnimationPoseData& OutPoseData, const FAnimExtractContext& ExtractionContext) const override { check(false); /* Should never be called, montages dont use this API */ }
	//~ End AnimSequenceBase Interface

#if WITH_EDITOR
	//~ Begin UAnimationAsset Interface
	virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;
	//~ End UAnimationAsset Interface

	/** Update all linkable elements contained in the montage */
	ENGINE_API void UpdateLinkableElements();

	/** Update linkable elements that rely on a specific segment. This will update linkable elements for the segment specified
	 *	and elements linked to segments after the segment specified
	 *	@param SlotIdx The slot that the segment is contained in
	 *	@param SegmentIdx The index of the segment within the specified slot
	 */
	ENGINE_API void UpdateLinkableElements(int32 SlotIdx, int32 SegmentIdx);
#endif
	/** Check if this slot has a valid additive animation for the specified slot.
	 * The slot name should not include the group name.
	 * i.e. for "DefaultGroup.DefaultSlot", the slot name is "DefaultSlot".
	 */
	UFUNCTION(BlueprintPure, Category = "Montage")
	bool IsValidAdditiveSlot(const FName& SlotNodeName) const;

	/** Get FCompositeSection with InSectionName */
	ENGINE_API FCompositeSection& GetAnimCompositeSection(int32 SectionIndex);
	ENGINE_API const FCompositeSection& GetAnimCompositeSection(int32 SectionIndex) const;

	// @todo document
	ENGINE_API void GetSectionStartAndEndTime(int32 SectionIndex, float& OutStartTime, float& OutEndTime) const;
	
	// @todo document
	ENGINE_API float GetSectionLength(int32 SectionIndex) const;
	
	/** Get SectionIndex from SectionName. Returns INDEX_None if not found */
	UFUNCTION(BlueprintPure, Category = "Montage")
	ENGINE_API int32 GetSectionIndex(FName InSectionName) const;
	
	/** Get SectionName from SectionIndex. Returns NAME_None if not found */
	UFUNCTION(BlueprintPure, Category = "Montage")
	ENGINE_API FName GetSectionName(int32 SectionIndex) const;

	/** Returns the number of sections this montage has */
	UFUNCTION(BlueprintPure, Category = "Montage")
	int32 GetNumSections() const { return CompositeSections.Num(); }

	/** @return true if valid section */
	UFUNCTION(BlueprintCallable, Category = "Montage")
	ENGINE_API bool IsValidSectionName(FName InSectionName) const;

	// @todo document
	ENGINE_API bool IsValidSectionIndex(int32 SectionIndex) const;

	/** Return Section Index from Position */
	ENGINE_API int32 GetSectionIndexFromPosition(float Position) const;
	
	/**
	 * Get Section Metadata for the montage including metadata belong to the anim reference
	 * This will remove redundant entry if found - i.e. multiple same anim reference is used
	 * 
	 * @param : SectionName - Name of section you'd like to get meta data for. 
	 *						- If SectionName == NONE, it will return all the section data
	 * @param : bIncludeSequence - if true, it returns all metadata of the animation within that section
	 *						- whether partial or full
	 * @param : SlotName - this only matters if bIncludeSequence is true.
	 *						- If true, and if SlotName is given, it will only look for SlotName.
	 *						- If true and if SlotName is none, then it will look for all slot nodes
	 ***/

	ENGINE_API const TArray<class UAnimMetaData*> GetSectionMetaData(FName SectionName, bool bIncludeSequence=true, FName SlotName = NAME_None);

	/** Get Section Index from CurrentTime with PosWithinCompositeSection */
	ENGINE_API int32 GetAnimCompositeSectionIndexFromPos(float CurrentTime, float& PosWithinCompositeSection) const;

	/** Return time left to end of section from given position. -1.f if not a valid position */
	ENGINE_API float GetSectionTimeLeftFromPos(float Position);

	/** Utility function to calculate Animation Pos from Section, PosWithinCompositeSection */
	float CalculatePos(FCompositeSection &Section, float PosWithinCompositeSection) const;
	
	/** Prototype function to get animation data - this will need rework */
	ENGINE_API const FAnimTrack* GetAnimationData(FName SlotName) const;

	/** Returns whether the anim sequences this montage have root motion enabled */
	virtual bool HasRootMotion() const override;

	/** Extract RootMotion Transform from a contiguous Track position range.
	 * *CONTIGUOUS* means that if playing forward StartTractPosition < EndTrackPosition.
	 * No wrapping over if looping. No jumping across different sections.
	 * So the AnimMontage has to break the update into contiguous pieces to handle those cases.
	 *
	 * This does handle Montage playing backwards (StartTrackPosition > EndTrackPosition).
	 *
	 * It will break down the range into steps if needed to handle looping animations, or different animations.
	 * These steps will be processed sequentially, and output the RootMotion transform in component space.
	 */
	ENGINE_API FTransform ExtractRootMotionFromTrackRange(float StartTrackPosition, float EndTrackPosition, const FAnimExtractContext& Context) const;

	/** Get the Montage's Group Name. This is the group from the first slot.  */
	UFUNCTION(BlueprintPure, Category = "Montage")
	ENGINE_API FName GetGroupName() const;

	/** true if valid, false otherwise. Will log warning if not valid. */
	bool HasValidSlotSetup() const;

private:
	/** 
	 * Utility function to check if CurrentTime is between FirstIndex and SecondIndex of CompositeSections
	 * return true if it is
	 */
	bool IsWithinPos(int32 FirstIndex, int32 SecondIndex, float CurrentTime) const;

	/** Calculates a trigger offset based on the supplied time taking into account only the montages sections */
	EAnimEventTriggerOffsets::Type CalculateOffsetFromSections(float Time) const;


public:
#if WITH_EDITOR
	/**
	 * Add Composite section with InSectionName
	 * returns index of added item 
	 * returns INDEX_NONE if failed. - i.e. InSectionName is not unique
	 */
	ENGINE_API int32 AddAnimCompositeSection(FName InSectionName, float StartPos);
	
	/** 
	 * Delete Composite section with InSectionName
	 * return true if success, false otherwise
	 */
	ENGINE_API bool DeleteAnimCompositeSection(int32 SectionIndex);

private:
	/** Sort CompositeSections in the order of StartPos */
	void SortAnimCompositeSectionByPos();

	/** Refresh Parent Asset Data to the child */
	virtual void RefreshParentAssetData() override;
	
	/** Propagate the changes to children */
	void PropagateChanges();

private:
	DECLARE_MULTICAST_DELEGATE(FOnMontageChangedMulticaster);
	FOnMontageChangedMulticaster OnMontageChanged;

public:
	typedef FOnMontageChangedMulticaster::FDelegate FOnMontageChanged;

	/** Registers a delegate to be called after notification has changed*/
	ENGINE_API void RegisterOnMontageChanged(const FOnMontageChanged& Delegate);
	ENGINE_API void UnregisterOnMontageChanged(FDelegateUserObject Unregister);
#endif	//WITH_EDITOR

private:

	/** Convert all branching points to AnimNotifies */
	void ConvertBranchingPointsToAnimNotifies();
	/** Recreate BranchingPoint markers from AnimNotifies marked 'BranchingPoints' */
	void RefreshBranchingPointMarkers();
	void AddBranchingPointMarker(FBranchingPointMarker Marker, TMap<float, FAnimNotifyEvent*>& TriggerTimes);
	
	/** Cached list of Branching Point markers */
	UPROPERTY()
	TArray<FBranchingPointMarker> BranchingPointMarkers;
public:

	/** Keep track of which AnimNotify_State are marked as BranchingPoints, so we can update their state when the Montage is ticked */
	UPROPERTY()
	TArray<int32> BranchingPointStateNotifyIndices;

	/** Find first branching point marker between track positions */
	const FBranchingPointMarker* FindFirstBranchingPointMarker(float StartTrackPos, float EndTrackPos) const;
	
	/** Filter out notifies from array that are marked as 'BranchingPoints' */
	UE_DEPRECATED(4.19, "Use the GetAnimNotifiesFromTrackPositions that takes FAnimNotifyEventReferences instead")
	void FilterOutNotifyBranchingPoints(TArray<const FAnimNotifyEvent*>& InAnimNotifies);

	/** Filter out notifies from array that are marked as 'BranchingPoints' */
	void FilterOutNotifyBranchingPoints(TArray<FAnimNotifyEventReference>& InAnimNotifies);

	bool CanUseMarkerSync() const { return MarkerData.AuthoredSyncMarkers.Num() > 0; }

	// update markers
	void CollectMarkers();

	//~Begin UAnimCompositeBase Interface
	virtual void InvalidateRecursiveAsset() override;
	virtual bool ContainRecursive(TArray<UAnimCompositeBase*>& CurrentAccumulatedList) override;
	virtual void SetCompositeLength(float InLength) override;
#if WITH_EDITOR
	virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel) override;
#endif // WITH_EDITOR
	//~End UAnimCompositeBase Interface

	/** Utility function to create dynamic montage from AnimSequence */
	ENGINE_API static UAnimMontage* CreateSlotAnimationAsDynamicMontage(UAnimSequenceBase* Asset, FName SlotNodeName, float BlendInTime = 0.25f, float BlendOutTime = 0.25f, float InPlayRate = 1.f, int32 LoopCount = 1, float BlendOutTriggerTime = -1.f, float InTimeToStartMontageAt = 0.f);

	/** Utility function to create dynamic montage from AnimSequence with blend in settings */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	static ENGINE_API UAnimMontage* CreateSlotAnimationAsDynamicMontage_WithBlendSettings(UAnimSequenceBase* Asset, FName SlotNodeName, const FMontageBlendSettings& BlendInSettings, const FMontageBlendSettings& BlendOutSettings, float InPlayRate = 1.f, int32 LoopCount = 1, float InBlendOutTriggerTime = -1.f);

	/** Utility function to create dynamic montage from AnimSequence */
	ENGINE_API static UAnimMontage* CreateSlotAnimationAsDynamicMontage_WithFractionalLoops(UAnimSequenceBase* Asset, FName SlotNodeName, float BlendInTime = 0.25f, float BlendOutTime = 0.25f, float LoopCount = 1.0f, float BlendOutTriggerTime = -1.f);

	//~Begin Time Stretch Curve
public:

	/** Time stretch curve will only be used when the montage has a non-default play rate  */
	UPROPERTY(EditAnywhere, Category = TimeStretchCurve)
	FTimeStretchCurve TimeStretchCurve;

	/** Name of optional TimeStretchCurveName to look for in Montage. Time stretch curve will only be used when the montage has a non-default play rate */
	UPROPERTY(EditAnywhere, Category = TimeStretchCurve)
	FName TimeStretchCurveName;

private:
#if WITH_EDITOR
	void BakeTimeStretchCurve();
	//~End Time Stretch Curve
	virtual void UpdateCommonTargetFrameRate() override;
#endif // WITH_EDITOR
};
