// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimMontage.h"

struct FMontageEvaluationState
{
	FMontageEvaluationState(UAnimMontage* InMontage, float InPosition, FDeltaTimeRecord InDeltaTimeRecord, bool bInIsPlaying, bool bInIsActive, const FAlphaBlend& InBlendInfo, const UBlendProfile* InActiveBlendProfile, float InBlendStartAlpha)
		: Montage(InMontage)
		, BlendInfo(InBlendInfo)
		, ActiveBlendProfile(InActiveBlendProfile)
		, MontagePosition(InPosition)
		, DeltaTimeRecord(InDeltaTimeRecord)
		, BlendStartAlpha(InBlendStartAlpha)
		, bIsPlaying(bInIsPlaying)
		, bIsActive(bInIsActive)
	{
	}

	// The montage to evaluate
	TWeakObjectPtr<UAnimMontage> Montage;

	// The current blend information.
	FAlphaBlend BlendInfo;

	// The active blend profile. Montages have a profile for blending in and blending out.
	const UBlendProfile* ActiveBlendProfile;

	// The position to evaluate this montage at
	float MontagePosition;

	// The previous MontagePosition and delta leading into current
	FDeltaTimeRecord DeltaTimeRecord;

	// The linear alpha value where to start blending from. So not the blended value that already has been curve sampled.
	float BlendStartAlpha;

	// Whether this montage is playing
	bool bIsPlaying;

	// Whether this montage is valid and not stopped
	bool bIsActive;
};
