// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"

// Alpha blending options helper functions for anim nodes
// Assumes that the specified node contains the members:
// AlphaInputType, ActualAlpha, AlphaScaleBias, Alpha, bAlphaBoolEnabled, AlphaCurveName
struct FAnimNodeAlphaOptions
{
	// Per-tick update
	template<typename AnimNodeType>
	static bool Update(AnimNodeType& InAnimNode, const FAnimationUpdateContext& InContext)
	{
		// Determine Actual Alpha.
		switch (InAnimNode.AlphaInputType)
		{
		case EAnimAlphaInputType::Float:
			InAnimNode.ActualAlpha = InAnimNode.AlphaScaleBias.ApplyTo(InAnimNode.AlphaScaleBiasClamp.ApplyTo(InAnimNode.Alpha, InContext.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool:
			InAnimNode.ActualAlpha = InAnimNode.AlphaBoolBlend.ApplyTo(InAnimNode.bAlphaBoolEnabled, InContext.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve:
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(InContext.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				InAnimNode.ActualAlpha = InAnimNode.AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(InAnimNode.AlphaCurveName), InContext.GetDeltaTime());
			}
			break;
		};

		// Make sure Alpha is clamped between 0 and 1.
		InAnimNode.ActualAlpha = FMath::Clamp<float>(InAnimNode.ActualAlpha, 0.f, 1.f);

		return FAnimWeight::IsRelevant(InAnimNode.ActualAlpha);
	}
};