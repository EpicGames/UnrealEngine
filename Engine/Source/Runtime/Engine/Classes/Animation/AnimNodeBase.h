// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Engine/EngineTypes.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimCurveTypes.h"
#include "BonePose.h"
#include "Stats/StatsHierarchical.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimNodeData.h"
#include "Animation/ExposedValueHandler.h"
#include "AnimNodeFunctionRef.h"

#include "AnimNodeBase.generated.h"

#define DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Method) \
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

class IAnimClassInterface;
class UAnimBlueprint;
class UAnimInstance;
struct FAnimInstanceProxy;
struct FAnimNode_Base;
class UProperty;
struct FPropertyAccessLibrary;
struct FAnimNodeConstantData;

/** Persistent state shared during animation tree update  */
struct FAnimationUpdateSharedContext
{
	FAnimationUpdateSharedContext() = default;

	// Non-copyable
	FAnimationUpdateSharedContext(FAnimationUpdateSharedContext& ) = delete;
	FAnimationUpdateSharedContext& operator=(const FAnimationUpdateSharedContext&) = delete;

	// Message stack used for storing scoped messages and tags during execution
	UE::Anim::FMessageStack MessageStack;

	void CopyForCachedUpdate(FAnimationUpdateSharedContext& Source)
	{
		MessageStack.CopyForCachedUpdate(Source.MessageStack);
	}
};

/** Base class for update/evaluate contexts */
struct FAnimationBaseContext
{
public:
	FAnimInstanceProxy* AnimInstanceProxy;

	FAnimationUpdateSharedContext* SharedContext;

	ENGINE_API FAnimationBaseContext();

protected:
	// DEPRECATED - Please use constructor that uses an FAnimInstanceProxy*
	ENGINE_API FAnimationBaseContext(UAnimInstance* InAnimInstance);

	ENGINE_API FAnimationBaseContext(FAnimInstanceProxy* InAnimInstanceProxy, FAnimationUpdateSharedContext* InSharedContext = nullptr);

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimationBaseContext(FAnimationBaseContext&&) = default;
	FAnimationBaseContext(const FAnimationBaseContext&) = default;
	FAnimationBaseContext& operator=(FAnimationBaseContext&&) = default;
	FAnimationBaseContext& operator=(const FAnimationBaseContext&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	// Get the Blueprint IAnimClassInterface associated with this context, if there is one.
	// Note: This can return NULL, so check the result.
	ENGINE_API IAnimClassInterface* GetAnimClass() const;

	// Get the anim instance associated with the current proxy
	ENGINE_API UObject* GetAnimInstanceObject() const;

#if WITH_EDITORONLY_DATA
	// Get the AnimBlueprint associated with this context, if there is one.
	// Note: This can return NULL, so check the result.
	ENGINE_API UAnimBlueprint* GetAnimBlueprint() const;
#endif //WITH_EDITORONLY_DATA

	// Get the innermost scoped message of the specified type
	template<typename TGraphMessageType>
	TGraphMessageType* GetMessage() const
	{
		if (ensure(SharedContext != nullptr))
		{
			TGraphMessageType* Message = nullptr;

			SharedContext->MessageStack.TopMessage<TGraphMessageType>([&Message](TGraphMessageType& InMessage)
			{
				Message = &InMessage;
			});

			return Message;
		}
		
		return nullptr;
	}

	// Find the innermost scoped message of the specified type matching the condition of InFunction
	template<typename TGraphMessageType>
	TGraphMessageType* FindMessage(TFunctionRef<bool(TGraphMessageType&)> InFunction) const
	{
		if (ensure(SharedContext != nullptr))
		{
			TGraphMessageType* Message = nullptr;
			SharedContext->MessageStack.ForEachMessage<TGraphMessageType>([&Message, &InFunction](TGraphMessageType& InMessage)
			{
				if (InFunction(InMessage))
				{
					Message = &InMessage;
					return UE::Anim::FMessageStack::EEnumerate::Stop;
				}
				return UE::Anim::FMessageStack::EEnumerate::Continue;
			});
				
			return Message;
		}

		return nullptr;
	}

	// Get the innermost scoped message of the specified type
	template<typename TGraphMessageType>
	TGraphMessageType& GetMessageChecked() const
	{
		check(SharedContext != nullptr);

		TGraphMessageType* Message = nullptr;

		SharedContext->MessageStack.TopMessage<TGraphMessageType>([&Message](TGraphMessageType& InMessage)
		{
			Message = &InMessage;
		});

		check(Message != nullptr);

		return *Message;
	}
	
	void SetNodeId(int32 InNodeId)
	{ 
		PreviousNodeId = CurrentNodeId;
		CurrentNodeId = InNodeId;
	}

	void SetNodeIds(const FAnimationBaseContext& InContext)
	{ 
		CurrentNodeId = InContext.CurrentNodeId;
		PreviousNodeId = InContext.PreviousNodeId;
	}

	// Get the current node Id, set when we recurse into graph traversal functions from pose links
	int32 GetCurrentNodeId() const { return CurrentNodeId; }

	// Get the previous node Id, set when we recurse into graph traversal functions from pose links
	int32 GetPreviousNodeId() const { return PreviousNodeId; }

	// Get whether the graph branch of this context is active (i.e. NOT blending out). 
	bool IsActive() const { return bIsActive; }

protected:
	
	// Whether this context belongs to graph branch (i.e. NOT blending out).
	bool bIsActive = true;

	// The current node ID, set when we recurse into graph traversal functions from pose links
	int32 CurrentNodeId;

	// The previous node ID, set when we recurse into graph traversal functions from pose links
	int32 PreviousNodeId;

protected:

	/** Interface for node contexts to register log messages with the proxy */
	ENGINE_API void LogMessageInternal(FName InLogType, const TSharedRef<FTokenizedMessage>& InMessage) const;
};


/** Initialization context passed around during animation tree initialization */
struct FAnimationInitializeContext : public FAnimationBaseContext
{
public:
	FAnimationInitializeContext(FAnimInstanceProxy* InAnimInstanceProxy, FAnimationUpdateSharedContext* InSharedContext = nullptr)
		: FAnimationBaseContext(InAnimInstanceProxy, InSharedContext)
	{
	}
};

/**
 * Context passed around when RequiredBones array changed and cached bones indices have to be refreshed.
 * (RequiredBones array changed because of an LOD switch for example)
 */
struct FAnimationCacheBonesContext : public FAnimationBaseContext
{
public:
	FAnimationCacheBonesContext(FAnimInstanceProxy* InAnimInstanceProxy)
		: FAnimationBaseContext(InAnimInstanceProxy)
	{
	}

	FAnimationCacheBonesContext WithNodeId(int32 InNodeId) const
	{ 
		FAnimationCacheBonesContext Result(*this);
		Result.SetNodeId(InNodeId);
		return Result; 
	}
};

/** Update context passed around during animation tree update */
struct FAnimationUpdateContext : public FAnimationBaseContext
{
private:
	float CurrentWeight;
	float RootMotionWeightModifier;

	float DeltaTime;

public:
	FAnimationUpdateContext(FAnimInstanceProxy* InAnimInstanceProxy = nullptr)
		: FAnimationBaseContext(InAnimInstanceProxy)
		, CurrentWeight(1.0f)
		, RootMotionWeightModifier(1.0f)
		, DeltaTime(0.0f)
	{
	}

	FAnimationUpdateContext(FAnimInstanceProxy* InAnimInstanceProxy, float InDeltaTime, FAnimationUpdateSharedContext* InSharedContext = nullptr)
		: FAnimationBaseContext(InAnimInstanceProxy, InSharedContext)
		, CurrentWeight(1.0f)
		, RootMotionWeightModifier(1.0f)
		, DeltaTime(InDeltaTime)
	{
	}


	FAnimationUpdateContext(const FAnimationUpdateContext& Copy, FAnimInstanceProxy* InAnimInstanceProxy)
		: FAnimationBaseContext(InAnimInstanceProxy, Copy.SharedContext)
		, CurrentWeight(Copy.CurrentWeight)
		, RootMotionWeightModifier(Copy.RootMotionWeightModifier)
		, DeltaTime(Copy.DeltaTime)
	{
		CurrentNodeId = Copy.CurrentNodeId;
		PreviousNodeId = Copy.PreviousNodeId;
	}

public:
	FAnimationUpdateContext WithOtherProxy(FAnimInstanceProxy* InAnimInstanceProxy) const
	{
		return FAnimationUpdateContext(*this, InAnimInstanceProxy);
	}

	FAnimationUpdateContext WithOtherSharedContext(FAnimationUpdateSharedContext* InSharedContext) const
	{
		FAnimationUpdateContext Result(*this);
		Result.SharedContext = InSharedContext;

		// This is currently only used in the case of cached poses, where we dont want to preserve the previous node, so clear it here
	//	Result.PreviousNodeId = INDEX_NONE;

		return Result;
	}

	FAnimationUpdateContext AsInactive() const
	{
		FAnimationUpdateContext Result(*this);
		Result.bIsActive = false;

		return Result;
	}

	FAnimationUpdateContext FractionalWeight(float WeightMultiplier) const
	{
		FAnimationUpdateContext Result(*this);
		Result.CurrentWeight = CurrentWeight * WeightMultiplier;

		return Result;
	}

	FAnimationUpdateContext FractionalWeightAndRootMotion(float WeightMultiplier, float RootMotionMultiplier) const
	{
		FAnimationUpdateContext Result(*this);
		Result.CurrentWeight = CurrentWeight * WeightMultiplier;
		Result.RootMotionWeightModifier = RootMotionWeightModifier * RootMotionMultiplier;

		return Result;
	}

	FAnimationUpdateContext FractionalWeightAndTime(float WeightMultiplier, float TimeMultiplier) const
	{
		FAnimationUpdateContext Result(*this);
		Result.DeltaTime = DeltaTime * TimeMultiplier;
		Result.CurrentWeight = CurrentWeight * WeightMultiplier;
		return Result;
	}

	FAnimationUpdateContext FractionalWeightTimeAndRootMotion(float WeightMultiplier, float TimeMultiplier, float RootMotionMultiplier) const
	{
		FAnimationUpdateContext Result(*this);
		Result.DeltaTime = DeltaTime * TimeMultiplier;
		Result.CurrentWeight = CurrentWeight * WeightMultiplier;
		Result.RootMotionWeightModifier = RootMotionWeightModifier * RootMotionMultiplier;

		return Result;
	}

	FAnimationUpdateContext WithNodeId(int32 InNodeId) const
	{ 
		FAnimationUpdateContext Result(*this);
		Result.SetNodeId(InNodeId);
		return Result; 
	}

	// Returns persistent state that is tracked through animation tree update
	FAnimationUpdateSharedContext* GetSharedContext() const
	{
		return SharedContext;
	}

	// Returns the final blend weight contribution for this stage
	float GetFinalBlendWeight() const { return CurrentWeight; }

	// Returns the weight modifier for root motion (as root motion weight wont always match blend weight)
	float GetRootMotionWeightModifier() const { return RootMotionWeightModifier; }

	// Returns the delta time for this update, in seconds
	float GetDeltaTime() const { return DeltaTime; }

	// Log update message
	void LogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const { LogMessageInternal("Update", InMessage); }
	void LogMessage(EMessageSeverity::Type InSeverity, FText InMessage) const { LogMessage(FTokenizedMessage::Create(InSeverity, InMessage)); }
};


/** Evaluation context passed around during animation tree evaluation */
struct FPoseContext : public FAnimationBaseContext
{
public:
	/* These Pose/Curve/Attributes are allocated using MemStack. You should not use it outside of stack. */
	FCompactPose	Pose;
	FBlendedCurve	Curve;
	UE::Anim::FStackAttributeContainer CustomAttributes;

public:
	friend class FScopedExpectsAdditiveOverride;
	
	// This constructor allocates a new uninitialized pose for the specified anim instance
	FPoseContext(FAnimInstanceProxy* InAnimInstanceProxy, bool bInExpectsAdditivePose = false)
		: FAnimationBaseContext(InAnimInstanceProxy)
		, bExpectsAdditivePose(bInExpectsAdditivePose)
	{
		InitializeImpl(InAnimInstanceProxy);
	}

	// This constructor allocates a new uninitialized pose, copying non-pose state from the source context
	FPoseContext(const FPoseContext& SourceContext, bool bInOverrideExpectsAdditivePose = false)
		: FAnimationBaseContext(SourceContext.AnimInstanceProxy)
		, bExpectsAdditivePose(SourceContext.bExpectsAdditivePose || bInOverrideExpectsAdditivePose)
	{
		InitializeImpl(SourceContext.AnimInstanceProxy);

		CurrentNodeId = SourceContext.CurrentNodeId;
		PreviousNodeId = SourceContext.PreviousNodeId;
	}

	// This constructor allocates a new uninitialized pose, using the provided BoneContainer (when there is no AnimInstanceProxy available)
	FPoseContext(const FBoneContainer& InRequiredBones, bool bInExpectsAdditivePose = false)
		: FAnimationBaseContext()
		, bExpectsAdditivePose(bInExpectsAdditivePose)
	{
		checkSlow(InRequiredBones.IsValid());
		Pose.SetBoneContainer(&InRequiredBones);
		Curve.InitFrom(InRequiredBones);
	}

	// Log evaluation message
	void LogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const { LogMessageInternal("Evaluate", InMessage); }
	void LogMessage(EMessageSeverity::Type InSeverity, FText InMessage) const { LogMessage(FTokenizedMessage::Create(InSeverity, InMessage)); }

	void ResetToRefPose()
	{
		if (bExpectsAdditivePose)
		{
			Pose.ResetToAdditiveIdentity();
		}
		else
		{
			Pose.ResetToRefPose();
		}
	}

	void ResetToAdditiveIdentity()
	{
		Pose.ResetToAdditiveIdentity();
	}

	bool ContainsNaN() const
	{
		return Pose.ContainsNaN();
	}

	bool IsNormalized() const
	{
		return Pose.IsNormalized();
	}

	FPoseContext& operator=(const FPoseContext& Other)
	{
		if (AnimInstanceProxy != Other.AnimInstanceProxy)
		{
			InitializeImpl(AnimInstanceProxy);
		}

		Pose = Other.Pose;
		Curve = Other.Curve;
		CustomAttributes = Other.CustomAttributes;
		bExpectsAdditivePose = Other.bExpectsAdditivePose;
		return *this;
	}

	// Is this pose expected to be additive
	bool ExpectsAdditivePose() const { return bExpectsAdditivePose; }

private:
	ENGINE_API void InitializeImpl(FAnimInstanceProxy* InAnimInstanceProxy);

	// Is this pose expected to be an additive pose
	bool bExpectsAdditivePose;
};

// Helper for modifying and resetting ExpectsAdditivePose on a FPoseContext
class FScopedExpectsAdditiveOverride
{
public:
	FScopedExpectsAdditiveOverride(FPoseContext& InContext, bool bInExpectsAdditive)
		: Context(InContext)
	{
		bPreviousValue = Context.ExpectsAdditivePose();
		Context.bExpectsAdditivePose = bInExpectsAdditive;
	}
	
	~FScopedExpectsAdditiveOverride()
	{
		Context.bExpectsAdditivePose = bPreviousValue;
	}
private:
	FPoseContext& Context;
	bool bPreviousValue;
};
	


/** Evaluation context passed around during animation tree evaluation */
struct FComponentSpacePoseContext : public FAnimationBaseContext
{
public:
	FCSPose<FCompactPose>	Pose;
	FBlendedCurve			Curve;
	UE::Anim::FStackAttributeContainer CustomAttributes;

public:
	// This constructor allocates a new uninitialized pose for the specified anim instance
	FComponentSpacePoseContext(FAnimInstanceProxy* InAnimInstanceProxy)
		: FAnimationBaseContext(InAnimInstanceProxy)
	{
		// No need to initialize, done through FA2CSPose::AllocateLocalPoses
	}

	// This constructor allocates a new uninitialized pose, copying non-pose state from the source context
	FComponentSpacePoseContext(const FComponentSpacePoseContext& SourceContext)
		: FAnimationBaseContext(SourceContext.AnimInstanceProxy)
	{
		// No need to initialize, done through FA2CSPose::AllocateLocalPoses

		CurrentNodeId = SourceContext.CurrentNodeId;
		PreviousNodeId = SourceContext.PreviousNodeId;
	}

	// Note: this copy assignment operator copies the whole object but the copy constructor only copies part of the object.
	FComponentSpacePoseContext& operator=(const FComponentSpacePoseContext&) = default;

	// Log evaluation message
	void LogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const { LogMessageInternal("Evaluate", InMessage); }
	void LogMessage(EMessageSeverity::Type InSeverity, FText InMessage) const { LogMessage(FTokenizedMessage::Create(InSeverity, InMessage)); }

	ENGINE_API void ResetToRefPose();

	ENGINE_API bool ContainsNaN() const;
	ENGINE_API bool IsNormalized() const;
};

/**
 * We pass array items by reference, which is scary as TArray can move items around in memory.
 * So we make sure to allocate enough here so it doesn't happen and crash on us.
 */
#define ANIM_NODE_DEBUG_MAX_CHAIN 50
#define ANIM_NODE_DEBUG_MAX_CHILDREN 12
#define ANIM_NODE_DEBUG_MAX_CACHEPOSE 20

struct FNodeDebugData
{
private:
	struct DebugItem
	{
		DebugItem(FString Data, bool bInPoseSource) : DebugData(Data), bPoseSource(bInPoseSource) {}

		/** This node item's debug text to display. */
		FString DebugData;

		/** Whether we are supplying a pose instead of modifying one (e.g. an playing animation). */
		bool bPoseSource;

		/** Nodes that we are connected to. */
		TArray<FNodeDebugData> ChildNodeChain;
	};

	/** This nodes final contribution weight (based on its own weight and the weight of its parents). */
	float AbsoluteWeight;

	/** Nodes that we are dependent on. */
	TArray<DebugItem> NodeChain;

	/** Additional info provided, used in GetNodeName. States machines can provide the state names for the Root Nodes to use for example. */
	FString NodeDescription;

	/** Pointer to RootNode */
	FNodeDebugData* RootNodePtr;

	/** SaveCachePose Nodes */
	TArray<FNodeDebugData> SaveCachePoseNodes;

public:
	struct FFlattenedDebugData
	{
		FFlattenedDebugData(FString Line, float AbsWeight, int32 InIndent, int32 InChainID, bool bInPoseSource) : DebugLine(Line), AbsoluteWeight(AbsWeight), Indent(InIndent), ChainID(InChainID), bPoseSource(bInPoseSource){}
		FString DebugLine;
		float AbsoluteWeight;
		int32 Indent;
		int32 ChainID;
		bool bPoseSource;

		bool IsOnActiveBranch() { return FAnimWeight::IsRelevant(AbsoluteWeight); }
	};

	FNodeDebugData(const class UAnimInstance* InAnimInstance) 
		: AbsoluteWeight(1.f), RootNodePtr(this), AnimInstance(InAnimInstance)
	{
		SaveCachePoseNodes.Reserve(ANIM_NODE_DEBUG_MAX_CACHEPOSE);
	}
	
	FNodeDebugData(const class UAnimInstance* InAnimInstance, const float AbsWeight, FString InNodeDescription, FNodeDebugData* InRootNodePtr)
		: AbsoluteWeight(AbsWeight)
		, NodeDescription(InNodeDescription)
		, RootNodePtr(InRootNodePtr)
		, AnimInstance(InAnimInstance) 
	{}

	ENGINE_API void AddDebugItem(FString DebugData, bool bPoseSource = false);
	ENGINE_API FNodeDebugData& BranchFlow(float BranchWeight, FString InNodeDescription = FString());
	ENGINE_API FNodeDebugData* GetCachePoseDebugData(float GlobalWeight);

	template<class Type>
	FString GetNodeName(Type* Node)
	{
		FString FinalString = FString::Printf(TEXT("%s<W:%.1f%%> %s"), *Node->StaticStruct()->GetName(), AbsoluteWeight*100.f, *NodeDescription);
		NodeDescription.Empty();
		return FinalString;
	}

	ENGINE_API void GetFlattenedDebugData(TArray<FFlattenedDebugData>& FlattenedDebugData, int32 Indent, int32& ChainID);

	TArray<FFlattenedDebugData> GetFlattenedDebugData()
	{
		TArray<FFlattenedDebugData> Data;
		int32 ChainID = 0;
		GetFlattenedDebugData(Data, 0, ChainID);
		return Data;
	}

	// Anim instance that we are generating debug data for
	const UAnimInstance* AnimInstance;
};

/** The display mode of editable values on an animation node. */
UENUM()
namespace EPinHidingMode
{
	enum Type : int
	{
		/** Never show this property as a pin, it is only editable in the details panel (default for everything but FPoseLink properties). */
		NeverAsPin,

		/** Hide this property by default, but allow the user to expose it as a pin via the details panel. */
		PinHiddenByDefault,

		/** Show this property as a pin by default, but allow the user to hide it via the details panel. */
		PinShownByDefault,

		/** Always show this property as a pin; it never makes sense to edit it in the details panel (default for FPoseLink properties). */
		AlwaysAsPin
	};
}

#define ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG 0

/** A pose link to another node */
USTRUCT(BlueprintInternalUseOnly)
struct FPoseLinkBase
{
	GENERATED_USTRUCT_BODY()

protected:
	/** The non serialized node pointer. */
	FAnimNode_Base* LinkedNode;

public:
	/** Serialized link ID, used to build the non-serialized pointer map. */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	int32 LinkID;

#if WITH_EDITORONLY_DATA
	/** The source link ID, used for debug visualization. */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	int32 SourceLinkID;
#endif

#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	FGraphTraversalCounter InitializationCounter;
	FGraphTraversalCounter CachedBonesCounter;
	FGraphTraversalCounter UpdateCounter;
	FGraphTraversalCounter EvaluationCounter;
#endif

protected:
#if DO_CHECK
	/** Flag to prevent reentry when dealing with circular trees. */
	bool bProcessed;
#endif

public:
	FPoseLinkBase()
		: LinkedNode(nullptr)
		, LinkID(INDEX_NONE)
#if WITH_EDITORONLY_DATA
		, SourceLinkID(INDEX_NONE)
#endif
#if DO_CHECK
		, bProcessed(false)
#endif
	{
	}

	// Interface

	ENGINE_API void Initialize(const FAnimationInitializeContext& Context);
	ENGINE_API void CacheBones(const FAnimationCacheBonesContext& Context);
	ENGINE_API void Update(const FAnimationUpdateContext& Context);
	ENGINE_API void GatherDebugData(FNodeDebugData& DebugData);

	/** Try to re-establish the linked node pointer. */
	ENGINE_API void AttemptRelink(const FAnimationBaseContext& Context);

	/** This only used by custom handlers, and it is advanced feature. */
	ENGINE_API void SetLinkNode(FAnimNode_Base* NewLinkNode);

	/** This only used when dynamic linking other graphs to this one. */
	ENGINE_API void SetDynamicLinkNode(struct FPoseLinkBase* InPoseLink);

	/** This only used by custom handlers, and it is advanced feature. */
	ENGINE_API FAnimNode_Base* GetLinkNode();
};

#define ENABLE_ANIMNODE_POSE_DEBUG 0

/** A local-space pose link to another node */
USTRUCT(BlueprintInternalUseOnly)
struct FPoseLink : public FPoseLinkBase
{
	GENERATED_USTRUCT_BODY()

public:
	// Interface
	ENGINE_API void Evaluate(FPoseContext& Output);

#if ENABLE_ANIMNODE_POSE_DEBUG
private:
	// forwarded pose data from the wired node which current node's skeletal control is not applied yet
	FCompactHeapPose CurrentPose;
#endif //#if ENABLE_ANIMNODE_POSE_DEBUG
};

/** A component-space pose link to another node */
USTRUCT(BlueprintInternalUseOnly)
struct FComponentSpacePoseLink : public FPoseLinkBase
{
	GENERATED_USTRUCT_BODY()

public:
	// Interface
	ENGINE_API void EvaluateComponentSpace(FComponentSpacePoseContext& Output);
};

/**
 * This is the base of all runtime animation nodes
 *
 * To create a new animation node:
 *   Create a struct derived from FAnimNode_Base - this is your runtime node
 *   Create a class derived from UAnimGraphNode_Base, containing an instance of your runtime node as a member - this is your visual/editor-only node
 */
USTRUCT()
struct FAnimNode_Base
{
	GENERATED_BODY()

	/** 
	 * Called when the node first runs. If the node is inside a state machine or cached pose branch then this can be called multiple times. 
	 * This can be called on any thread.
	 * @param	Context		Context structure providing access to relevant data
	 */
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context);

	/** 
	 * Called to cache any bones that this node needs to track (e.g. in a FBoneReference). 
	 * This is usually called at startup when LOD switches occur.
	 * This can be called on any thread.
	 * @param	Context		Context structure providing access to relevant data
	 */
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context);

	/** 
	 * Called to update the state of the graph relative to this node.
	 * Generally this should configure any weights (etc.) that could affect the poses that
	 * will need to be evaluated. This function is what usually executes EvaluateGraphExposedInputs.
	 * This can be called on any thread.
	 * @param	Context		Context structure providing access to relevant data
	 */
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context);

	/** 
	 * Called to evaluate local-space bones transforms according to the weights set up in Update().
	 * You should implement either Evaluate or EvaluateComponentSpace, but not both of these.
	 * This can be called on any thread.
	 * @param	Output		Output structure to write pose or curve data to. Also provides access to relevant data as a context.
	 */
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output);

	/** 
	 * Called to evaluate component-space bone transforms according to the weights set up in Update().
	 * You should implement either Evaluate or EvaluateComponentSpace, but not both of these.
	 * This can be called on any thread.
	 * @param	Output		Output structure to write pose or curve data to. Also provides access to relevant data as a context.
	 */	
	ENGINE_API virtual void EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output);

	/**
	 * Called to gather on-screen debug data. 
	 * This is called on the game thread.
	 * @param	DebugData	Debug data structure used to output any relevant data
	 */
	virtual void GatherDebugData(FNodeDebugData& DebugData)
	{ 
		DebugData.AddDebugItem(FString::Printf(TEXT("Non Overriden GatherDebugData! (%s)"), *DebugData.GetNodeName(this)));
	}

	/**
	 * Whether this node can run its Update() call on a worker thread.
	 * This is called on the game thread.
	 * If any node in a graph returns false from this function, then ALL nodes will update on the game thread.
	 */
	virtual bool CanUpdateInWorkerThread() const { return true; }

	/**
	 * Override this to indicate that PreUpdate() should be called on the game thread (usually to 
	 * gather non-thread safe data) before Update() is called.
	 * Note that this is called at load on the UAnimInstance CDO to avoid needing to call this at runtime.
	 * This is called on the game thread.
	 */
	virtual bool HasPreUpdate() const { return false; }

	/** Override this to perform game-thread work prior to non-game thread Update() being called */
	virtual void PreUpdate(const UAnimInstance* InAnimInstance) {}

	/**
	 * For nodes that implement some kind of simulation, return true here so ResetDynamics() gets called
	 * when things like teleports, time skips etc. occur that might require special handling.
	 * Note that this is called at load on the UAnimInstance CDO to avoid needing to call this at runtime.
	 * This is called on the game thread.
	 */
	virtual bool NeedsDynamicReset() const { return false; }

	/** Called to help dynamics-based updates to recover correctly from large movements/teleports */
	ENGINE_API virtual void ResetDynamics(ETeleportType InTeleportType);

	/** Called after compilation */
	virtual void PostCompile(const class USkeleton* InSkeleton) {}

	/** 
	 * For nodes that need some kind of initialization that is not dependent on node relevancy 
	 * (i.e. it is insufficient or inefficient to use Initialize_AnyThread), return true here.
	 * Note that this is called at load on the UAnimInstance CDO to avoid needing to call this at runtime.
	 */
	virtual bool NeedsOnInitializeAnimInstance() const { return false; }

	virtual ~FAnimNode_Base() = default;

	/** Deprecated functions */
	UE_DEPRECATED(4.20, "Please use ResetDynamics with an ETeleportPhysics flag instead")
	virtual void ResetDynamics() {}

	// The default handler for graph-exposed inputs:
	ENGINE_API const FExposedValueHandler& GetEvaluateGraphExposedInputs() const;

	// Get this node's index. The node index provides a unique key into its location within the class data
	int32 GetNodeIndex() const
	{
		check(NodeData);
		return NodeData->GetNodeIndex();
	}

	// Get the anim class that this node is hosted within
	const IAnimClassInterface* GetAnimClassInterface() const
	{
		check(NodeData);
		return &NodeData->GetAnimClassInterface();
	}
	
protected:
	// Get anim node constant/folded data of the specified type given the identifier. Do not use directly - use GET_ANIM_NODE_DATA
	template<typename DataType>
	const DataType& GetData(UE::Anim::FNodeDataId InId, const UObject* InObject = nullptr) const
	{
#if WITH_EDITORONLY_DATA
		if(NodeData)
		{
			return *static_cast<const DataType*>(NodeData->GetData(InId, this, InObject));
		}
		else
		{
			return *InId.GetProperty()->ContainerPtrToValuePtr<const DataType>(this);
		}
#else
		check(NodeData);
		return *static_cast<const DataType*>(NodeData->GetData(InId, this, InObject));
#endif
	}

	// Get anim node constant/folded data of the specified type given the identifier. Do not use directly - use GET_MUTABLE_ANIM_NODE_DATA
	// Note: will assert if data is not held on the instance/dynamic. Use GetInstanceDataPtr/GET_INSTANCE_ANIM_NODE_DATA_PTR if the value
	// might not be mutable, which will return null.
#if WITH_EDITORONLY_DATA
	template<typename DataType>
	DataType& GetMutableData(UE::Anim::FNodeDataId InId, UObject* InObject = nullptr)
	{
		if(NodeData)
		{
			return *static_cast<DataType*>(NodeData->GetMutableData(InId, this, InObject));
		}
		else
		{
			return *InId.GetProperty()->ContainerPtrToValuePtr<DataType>(this);
		}
	}
#endif

	// Get anim node mutable data of the specified type given the identifier. Do not use directly - use GET_INSTANCE_ANIM_NODE_DATA_PTR
	// @return nullptr if the data is not mutable/dynamic
	template<typename DataType>
	DataType* GetInstanceDataPtr(UE::Anim::FNodeDataId InId, UObject* InObject = nullptr)
	{
#if WITH_EDITORONLY_DATA	
		if(NodeData)
		{
			return static_cast<DataType*>(NodeData->GetInstanceData(InId, this, InObject));
		}
		else
		{
			return InId.GetProperty()->ContainerPtrToValuePtr<DataType>(this);
		}
#else
		check(NodeData);
		return static_cast<DataType*>(NodeData->GetInstanceData(InId, this, InObject));
#endif
	}
	
protected:
	/** return true if enabled, otherwise, return false. This is utility function that can be used per node level */
	ENGINE_API bool IsLODEnabled(FAnimInstanceProxy* AnimInstanceProxy);

	/** Get the LOD level at which this node is enabled. Node is enabled if the current LOD is less than or equal to this threshold. */
	virtual int32 GetLODThreshold() const { return INDEX_NONE; }

	/** Called once, from game thread as the parent anim instance is created */
	ENGINE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance);

	friend struct FAnimInstanceProxy;

private:
	// Access functions
	ENGINE_API const FAnimNodeFunctionRef& GetInitialUpdateFunction() const;
	ENGINE_API const FAnimNodeFunctionRef& GetBecomeRelevantFunction() const;
	ENGINE_API const FAnimNodeFunctionRef& GetUpdateFunction() const;
	
private:
	friend class IAnimClassInterface;
	friend class UAnimBlueprintGeneratedClass;
	friend struct UE::Anim::FNodeDataId;
	friend struct UE::Anim::FNodeFunctionCaller;
	friend class UAnimGraphNode_Base;
	friend struct FPoseLinkBase;

	// Set the cached ptr to the constant/folded data for this node
	void SetNodeData(const FAnimNodeData& InNodeData) { NodeData = &InNodeData; }

	// Reference to the constant/folded data for this node
	const FAnimNodeData* NodeData = nullptr;

#if WITH_EDITORONLY_DATA
	// Function called on initial update
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef InitialUpdateFunction;

	// Function called on become relevant
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef BecomeRelevantFunction;

	// Function called on update
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef UpdateFunction;
#endif
};

#if WITH_EDITORONLY_DATA
#define VERIFY_ANIM_NODE_MEMBER_TYPE(Type, Identifier) static_assert(std::is_same_v<decltype(Identifier), Type>, "Incorrect return type used");
#else
#define VERIFY_ANIM_NODE_MEMBER_TYPE(Type, Identifier)
#endif

#define GET_ANIM_NODE_DATA_ID_INTERNAL(Type, Identifier) \
	[this]() -> UE::Anim::FNodeDataId \
	{ \
		VERIFY_ANIM_NODE_MEMBER_TYPE(Type, Identifier) \
		static UE::Anim::FNodeDataId CachedId_##Identifier; \
		if(!CachedId_##Identifier.IsValid()) \
		{ \
			static const FName AnimName_##Identifier(#Identifier); \
			CachedId_##Identifier = UE::Anim::FNodeDataId(AnimName_##Identifier, this, StaticStruct()); \
		} \
		return CachedId_##Identifier; \
	}() \

// Get some (potentially folded) anim node data. Only usable from within an anim node.
// This caches the node data ID in static contained in a local lambda for improved performance
#define GET_ANIM_NODE_DATA(Type, Identifier) (GetData<Type>(GET_ANIM_NODE_DATA_ID_INTERNAL(Type, Identifier)))

// Get some anim node data that should be held on an instance. Only usable from within an anim node.
// @return nullptr if the data is not held on an instance (i.e. it is in constant sparse class data)
// This caches the node data ID in static contained in a local lambda for improved performance
#define GET_INSTANCE_ANIM_NODE_DATA_PTR(Type, Identifier) (GetInstanceDataPtr<Type>(GET_ANIM_NODE_DATA_ID_INTERNAL(Type, Identifier)))

#if WITH_EDITORONLY_DATA
// Editor-only way of accessing mutable anim node data but with internal checks
#define GET_MUTABLE_ANIM_NODE_DATA(Type, Identifier) (GetMutableData<Type>(GET_ANIM_NODE_DATA_ID_INTERNAL(Type, Identifier)))
#endif
