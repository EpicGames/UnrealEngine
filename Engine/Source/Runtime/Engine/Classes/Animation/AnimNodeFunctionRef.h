// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeFunctionRef.generated.h"

struct FAnimNode_StateMachine;
struct FPoseLink;
struct FPoseLinkBase;
struct FComponentSpacePoseLink;
struct FAnimNode_Base;
struct FAnimationBaseContext;
struct FAnimationInitializeContext;
struct FAnimationUpdateContext;
struct FPoseContext;
struct FComponentSpacePoseContext;
struct FAnimInstanceProxy;

/**
 * Cached function name/ptr that is resolved at init time
 */
USTRUCT()
struct FAnimNodeFunctionRef
{
	GENERATED_BODY()

public:
	// Cache the function ptr from the name
	ENGINE_API void Initialize(const UClass* InClass);
	
	// Call the function
	ENGINE_API void Call(UObject* InObject, void* InParameters = nullptr) const;

	// Set the function via name
	void SetFromFunctionName(FName InName) { FunctionName = InName; ClassName = NAME_None; }

	// Set the function via a function
	ENGINE_API void SetFromFunction(UFunction* InFunction);
	
	// Get the function name
	FName GetFunctionName() const { return FunctionName; }
	
	// Get the function we reference
	UFunction* GetFunction() const { return Function; }
	
	// Check if we reference a valid function
	bool IsValid() const { return Function != nullptr; }

	// Override operator== as we only need to compare class/function names
	bool operator==(const FAnimNodeFunctionRef& InOther) const
	{
		return ClassName == InOther.ClassName && FunctionName == InOther.FunctionName;
	}
	
private:
	// The name of the class to call the function with. If this is NAME_None, we assume this is a 'thiscall', if it is valid then we assume (and verify) we should call the function on a function library CDO.
	UPROPERTY()
	FName ClassName = NAME_None;

	// The name of the function to call
	UPROPERTY()
	FName FunctionName = NAME_None;	

	// The class to use to call the function with, recovered by looking for a class of name FunctionName
	UPROPERTY(Transient)
	TObjectPtr<const UClass> Class = nullptr;
	
	// The function to call, recovered by looking for a function of name FunctionName
	UPROPERTY(Transient)
	TObjectPtr<UFunction> Function = nullptr;
};

template<>
struct TStructOpsTypeTraits<FAnimNodeFunctionRef> : public TStructOpsTypeTraitsBase2<FAnimNodeFunctionRef>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

namespace UE { namespace Anim {

// Wrapper used to call anim node functions
struct FNodeFunctionCaller
{
private:
	friend struct ::FPoseLinkBase;
	friend struct ::FPoseLink;
	friend struct ::FComponentSpacePoseLink;
	friend struct ::FAnimInstanceProxy;
	friend struct ::FAnimNode_StateMachine;
	
	// Call the InitialUpdate function of this node
	static void InitialUpdate(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode);

	// Call the BecomeRelevant function of this node
	static void BecomeRelevant(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode);

	// Call the Update function of this node
	static void Update(const FAnimationUpdateContext& InContext, FAnimNode_Base& InNode);

public:
	// Call a generic function for this node
	ENGINE_API static void CallFunction(const FAnimNodeFunctionRef& InFunction, const FAnimationBaseContext& InContext, FAnimNode_Base& InNode);
};

}}
