// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineLogs.h"
#include "AnimNodeReference.generated.h"

struct FAnimNode_Base;
class IAnimClassInterface;
class UAnimInstance;

// The result of an anim node reference conversion 
UENUM(BlueprintType)
enum class EAnimNodeReferenceConversionResult : uint8
{
	Succeeded = 1,
	Failed = 0,
};

// A reference to an anim node. Does not persist, only valid for the call in which it was retrieved.
USTRUCT(BlueprintType)
struct FAnimNodeReference
{
	GENERATED_BODY()

public:
	typedef FAnimNode_Base FInternalNodeType; 
		
	FAnimNodeReference() = default;

	ENGINE_API FAnimNodeReference(UAnimInstance* InAnimInstance, FAnimNode_Base& InNode);

	ENGINE_API FAnimNodeReference(UAnimInstance* InAnimInstance, int32 InIndex);
	
	// Get the node we wrap. If the context is invalid or the node is not of the specified type then this will return nullptr.
	template<typename NodeType>
	NodeType* GetAnimNodePtr() const
	{
		if(AnimNode && AnimNodeStruct && AnimNodeStruct->IsChildOf(NodeType::StaticStruct()))
		{
			return static_cast<NodeType*>(AnimNode);
		}

		return nullptr;
	}

	// Get the node we wrap. If the reference is invalid or node is not of the specified type then this will return assert.
	template<typename NodeType>
	NodeType& GetAnimNode() const
	{
		check(AnimNodeStruct);
		check(AnimNode);
		check(AnimNodeStruct->IsChildOf(NodeType::StaticStruct()));
		return *static_cast<NodeType*>(AnimNode);
	}

	// Call a function if this context is valid
	template<typename NodeType>
	void CallAnimNodeFunction(const TCHAR* InFunctionNameForErrorReporting, TFunctionRef<void(NodeType&)> InFunction) const
	{
		if(NodeType* NodePtr = GetAnimNodePtr<NodeType>())
		{
			InFunction(*NodePtr);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "%ls called on an invalid context or with an invalid type", InFunctionNameForErrorReporting);
		}
	}
	
	// Convert to a derived type
	template<typename OtherContextType>
	static OtherContextType ConvertToType(const FAnimNodeReference& InReference, EAnimNodeReferenceConversionResult& OutResult)
	{
		static_assert(TIsDerivedFrom<OtherContextType, FAnimNodeReference>::IsDerived, "Argument ContextType must derive from FAnimNodeReference");
		
		if(InReference.AnimNodeStruct && InReference.AnimNodeStruct->IsChildOf(OtherContextType::FInternalNodeType::StaticStruct()))
		{
			OutResult = EAnimNodeReferenceConversionResult::Succeeded;
			
			OtherContextType Context;
			Context.AnimNode = InReference.AnimNode;
			Context.AnimNodeStruct = InReference.AnimNodeStruct;
			return Context;
		}

		OutResult = EAnimNodeReferenceConversionResult::Failed;
		
		return OtherContextType();
	}

private:
	// The node we wrap
	FAnimNode_Base* AnimNode = nullptr;

	// The struct type of the anim node
	UScriptStruct* AnimNodeStruct = nullptr;
};
