// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNodeSpaceConversions.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ConvertComponentToLocalSpace : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FComponentSpacePoseLink ComponentPose;

public:
	ENGINE_API FAnimNode_ConvertComponentToLocalSpace();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ConvertLocalToComponentSpace : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink LocalPose;

public:
	ENGINE_API FAnimNode_ConvertLocalToComponentSpace();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ENGINE_API virtual void EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output) override;
	// End of FAnimNode_Base interface
};
