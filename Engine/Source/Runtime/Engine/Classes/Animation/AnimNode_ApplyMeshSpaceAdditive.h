// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_ApplyMeshSpaceAdditive.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ApplyMeshSpaceAdditive : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink Base;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink Additive;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	uint8 bRootSpaceAdditive : 1;

	/** The data type used to control the alpha blending of the additive pose.
		Note: Changing this value will disconnect alpha input pins.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EAnimAlphaInputType AlphaInputType;

	/** The float value that controls the alpha blending when the alpha input type is set to 'Float' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	float Alpha;

	/** The boolean value that controls the alpha blending when the alpha input type is set to 'Bool' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault, DisplayName = "bEnabled"))
	uint8 bAlphaBoolEnabled : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	/*
	* Max LOD that this node is allowed to run
	* For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	* when the component LOD becomes 3, it will stop update/evaluate
	* currently transition would be issue and that has to be re-visited
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	float ActualAlpha;

public:
	ENGINE_API FAnimNode_ApplyMeshSpaceAdditive();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface
};
