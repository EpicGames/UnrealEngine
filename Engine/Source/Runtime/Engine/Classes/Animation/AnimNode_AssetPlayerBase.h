// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNode_RelevantAssetPlayerBase.h"
#include "Animation/AnimSync.h"
#include "AnimNode_AssetPlayerBase.generated.h"

/* Base class for any asset playing anim node */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_AssetPlayerBase : public FAnimNode_AssetPlayerRelevancyBase
{
	GENERATED_BODY()

	friend class UAnimGraphNode_AssetPlayerBase;

	FAnimNode_AssetPlayerBase() = default;

	/** Initialize function for setup purposes */
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;

	/** Update the node, marked final so we can always handle blendweight caching.
	 *  Derived classes should implement UpdateAssetPlayer
	 */
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) final override;

	/** Update method for the asset player, to be implemented by derived classes */
	virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) {};

	// Create a tick record for this node
	ENGINE_API void CreateTickRecordForNode(const FAnimationUpdateContext& Context, UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, bool bIsEvaluator);

	// Get the sync group name we are using
	virtual FName GetGroupName() const { return NAME_None; }

	// Get the sync group role we are using
	virtual EAnimGroupRole::Type GetGroupRole() const { return EAnimGroupRole::CanBeLeader; }

	// Get the sync group method we are using
	virtual EAnimSyncMethod GetGroupMethod() const { return EAnimSyncMethod::DoNotSync; }

	// Get the flag that determines if this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const { return false; }

	// Set the sync group name we are using
	virtual bool SetGroupName(FName InGroupName) { return false; }

	// Set the sync group role we are using
	virtual bool SetGroupRole(EAnimGroupRole::Type InRole) { return false; }

	// Set the sync group method we are using
	virtual bool SetGroupMethod(EAnimSyncMethod InMethod) { return false; }

	// Set the flag that determines if this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position. 
	virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) { return false; }
	
	// --- FAnimNode_RelevantAssetPlayerBase ---
	ENGINE_API virtual float GetAccumulatedTime() const override;
	ENGINE_API virtual void SetAccumulatedTime(float NewTime) override;
	ENGINE_API virtual float GetCachedBlendWeight() const override;
	ENGINE_API virtual void ClearCachedBlendWeight() override;
	ENGINE_API virtual float GetCurrentAssetTimePlayRateAdjusted() const override;
	ENGINE_API virtual const FDeltaTimeRecord* GetDeltaTimeRecord() const override;
	// --- End of FAnimNode_RelevantAssetPlayerBase ---

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 GroupIndex_DEPRECATED = INDEX_NONE;

	UPROPERTY()
	EAnimSyncGroupScope GroupScope_DEPRECATED = EAnimSyncGroupScope::Local;
#endif
	
protected:
	/** Track whether we have been full weight previously. Reset when we reach 0 weight*/
	bool bHasBeenFullWeight = false;
	
	/** If true, an asset player instance ID has been successfully set */
	bool bHasValidAssetPlayerInstanceID = false;
	
	/** Instance ID for this asset player node. Used to identify concurrently playing instances of the same animation during notify triggering. */
	uint32 AssetPlayerInstanceID = 0;
	
	/** Store data about current marker position when using marker based syncing*/
	FMarkerTickRecord MarkerTickRecord;

	/** Last encountered blendweight for this node */
	UPROPERTY(BlueprintReadWrite, Transient, Category=DoNotEdit)
	float BlendWeight = 0.0f;

	/** Accumulated time used to reference the asset in this node */
	UPROPERTY(BlueprintReadWrite, Transient, Category=DoNotEdit)
	float InternalTimeAccumulator = 0.0f;
	
	/** Previous frame InternalTimeAccumulator value and effective delta time leading into the current frame */
	FDeltaTimeRecord DeltaTimeRecord;
	
	/** Get sync parameters for the tick record produced by this asset player */
	ENGINE_API UE::Anim::FAnimSyncParams GetSyncParams(bool bRequestedInertialization) const;
};
