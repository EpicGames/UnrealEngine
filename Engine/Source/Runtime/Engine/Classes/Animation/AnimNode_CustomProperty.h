// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimStats.h"
#include "AnimNode_CustomProperty.generated.h"

/** 
 * Custom property node that you'd like to expand pin by reflecting internal instance (we call TargetInstance here)
 * 
 *  Used by sub anim instance or control rig node 
 *	where you have internal instance and would like to reflect to AnimNode as a pin
 * 
 *  To make pin working, you need storage inside of AnimInstance (SourceProperties/SourcePropertyNames)
 *  So this creates storage inside of AnimInstance with the unique custom property name
 *	and it copies to the actually TargetInstance here to allow the information be transferred in runtime (DestProperties/DestPropertyNames)
 * 
 *  TargetInstance - UObject derived instance that has certain dest properties
 *  Source - AnimInstance's copy properties that is used to store the data 
 */
USTRUCT()
struct FAnimNode_CustomProperty : public FAnimNode_Base
{
	GENERATED_BODY()

public:

	ENGINE_API FAnimNode_CustomProperty();
	ENGINE_API FAnimNode_CustomProperty(const FAnimNode_CustomProperty&);
	ENGINE_API ~FAnimNode_CustomProperty();

	/* Set Target Instance */
	ENGINE_API void SetTargetInstance(UObject* InInstance);

	/* Get Target Instance by type for convenience */
	template<class T>
	T* GetTargetInstance() const
	{
		if (IsValid(TargetInstance))
		{
			return Cast<T>(TargetInstance);
		}

		return nullptr;
	}

	// We only subscribe to the OnInitializeAnimInstance path because we need to cache our source object, so we only
	// override these methods in editor at the moment
#if WITH_EDITOR	
	// FAnimNode_Base interface
	ENGINE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }

	// Handle object reinstancing in editor
	ENGINE_API void HandleObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);
#endif

protected:
	/** List of source properties to use, 1-1 with Dest names below, built by the compiler */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	TArray<FName> SourcePropertyNames;

	/** List of destination properties to use, 1-1 with Source names above, built by the compiler */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	TArray<FName> DestPropertyNames;

	/** This is the actual instance allocated at runtime that will run. Set by child class. */
	UPROPERTY(Transient)
	TObjectPtr<UObject> TargetInstance;

	/** List of properties on the calling Source Instances instance to push from  */
	TArray<FProperty*> SourceProperties;

	/** List of properties on the TargetInstance to push to, built from name list when initialised */
	TArray<FProperty*> DestProperties;
	
	/* Initialize property links from the source instance, in this case AnimInstance 
	 * Compiler creates those properties during compile time */
	ENGINE_API virtual void InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass);

	/* Propagate the Source Instances' properties to Target Instance*/
	ENGINE_API virtual void PropagateInputProperties(const UObject* InSourceInstance);

	/** Get Target Class */
	virtual UClass* GetTargetClass() const PURE_VIRTUAL(FAnimNode_CustomProperty::GetTargetClass, return nullptr;);

#if WITH_EDITOR
	/**
	 * Handle object reinstancing override point.
	 * When objects are replaced in editor, the FCoreUObjectDelegates::OnObjectsReplaced is called before reference
	 * replacement, so we cannot handle replacement until later. This call is made on the first PreUpdate after object
	 * replacement.
	 */
	ENGINE_API virtual void HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:
	/** This is the source instance, cached to help with re-instancing */
	UPROPERTY(Transient)
	TObjectPtr<UObject> SourceInstance;
#endif

	// Stats
#if ANIMNODE_STATS_VERBOSE
	// Cached StatID for this node
	TStatId StatID;
	virtual void InitializeStatID() { StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Anim>(FString("Unknown")); }
#endif // ANIMNODE_STATS_VERBOSE

	friend class UAnimGraphNode_CustomProperty;
	friend class UAnimInstance;
	friend class UControlRigLayerInstance;
};

template<>
struct TStructOpsTypeTraits<FAnimNode_CustomProperty> : public TStructOpsTypeTraitsBase2<FAnimNode_CustomProperty>
{
	enum
	{
		WithPureVirtual = true,
	};
};
