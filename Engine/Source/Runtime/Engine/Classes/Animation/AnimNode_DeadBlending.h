// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_Inertialization.h"
#include "AnimNode_DeadBlending.generated.h"

/**
 * Dead Blending Node
 *
 * Dead blending is an alternative method of inertialization that extrapolates the animation being transitioned from 
 * forward in time and then performs a normal cross-fade blend between this extrapolated animation and the new animation
 * being transitioned to.
 * 
 * For more background see: https://theorangeduck.com/page/dead-blending
 * 
 * This node works by extrapolating forward the animation being transition from using the animation's velocities at
 * the point of transition, with an exponential decay which reduces those velocities over time to avoid the pose 
 * becoming invalid.
 * 
 * The rate of this decay is set automatically based on how much the velocities of the animation being transitioned 
 * from are moving toward the pose of the animation being transitioned to. If they are moving in the wrong direction or
 * too quickly they will have a larger decay rate, while if they are in the correct direction and moving slowly relative 
 * to the difference they will have a smaller decay rate.
 * 
 * These decay rates can be controlled by the `ExtrapolationHalfLife`, `ExtrapolationHalfLifeMin` and 
 * `ExtrapolationHalfLifeMax` parameters, which specify the approximate average, min, and max decay periods.
 * More specifically they specify the "half-life" - or how it takes for the velocities to be decayed by half.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_DeadBlending : public FAnimNode_Base
#if CPP
	, public IBoneReferenceSkeletonProvider
#endif
{
	GENERATED_BODY()
	friend class UAnimGraphNode_DeadBlending;

	// Input Pose
	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

#if WITH_EDITORONLY_DATA
	void SetTag(FName InTag) { Tag = InTag; }
#endif
	FName GetTag() { return Tag; }

private:

	// When enabled, the default blend settings will always be used rather than those coming from the inertialization request.
	UPROPERTY(EditAnywhere, Category = Blending)
	bool bAlwaysUseDefaultBlendSettings = false;

	// The default blend duration to use when "Always Use Default Blend Settings" is set to true.
	UPROPERTY(EditAnywhere, Category = Blending)
	float DefaultBlendDuration = 0.25f;

	// Default blend profile to use when no blend profile is supplied with the inertialization request.
	UPROPERTY(EditAnywhere, Category = Blending, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> DefaultBlendProfile = nullptr;

	// Default blend mode to use when no blend mode is supplied with the inertialization request.
	UPROPERTY(EditAnywhere, Category = Blending, DisplayName = "Default Blend Mode")
	EAlphaBlendOption DefaultBlendMode = UE::Anim::DefaultBlendOption;

	// Default custom blend curve to use along with the default blend mode.
	UPROPERTY(EditAnywhere, Category = Blending)
	TObjectPtr<UCurveFloat> DefaultCustomBlendCurve = nullptr;

	// Multiplier that can be used to scale the overall blend durations coming from inertialization requests.
	UPROPERTY(EditAnywhere, Category = Blending, meta = (Min = "0.0", UIMin = "0.0"))
	float BlendTimeMultiplier = 1.0f;

	/**
	 * When enabled, bone scales will be linearly interpolated and extrapolated. This is slightly more performant and
	 * consistent with the rest of Unreal but visually gives the appearance of the rate of change of scale being affected
	 * by the overall size of the bone. Note: this option must be enabled if you want this node to support negative scales.
	 */
	UPROPERTY(EditAnywhere, Category = Blending)
	bool bLinearlyInterpolateScales = false;

	// List of curves that should not use inertial blending. These curves will change instantly when the animation switches.
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FName> FilteredCurves;

	/**
	 * List of curves that will not be included in the extrapolation. Curves in this list will effectively act like they have had their value reset 
	 * to zero at the point of the transition, and will be blended in with the curve values in the animation that is being transitioned to.
	 */ 
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FName> ExtrapolationFilteredCurves;

	// List of bones that should not use inertial blending. These bones will change instantly when the animation switches.
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FBoneReference> FilteredBones;

	/**
	 * The average half-life of decay in seconds to use when extrapolating the animation. To get the final half-life of 
	 * decay, this value will be scaled by the amount to which the velocities of the animation being transitioned from 
	 * are moving toward the animation being transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"), DisplayName = "Extrapolation Half Life")
	float ExtrapolationHalfLife = 1.0f;

	/**
	 * The minimum half-life of decay in seconds to use when extrapolating the animation. This will be used when the 
	 * velocities of the animation being transitioned from are very small or moving away from the animation being 
	 * transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"), DisplayName = "Minimum Extrapolation Half Life")
	float ExtrapolationHalfLifeMin = 0.05f;

	/**
	 * The maximum half-life of decay in seconds to use when extrapolating the animation. This will dictate the longest 
	 * decay duration possible when velocities of the animation being transitioned from are small and moving towards the 
	 * animation being transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"), DisplayName = "Maximum Extrapolation Half Life")
	float ExtrapolationHalfLifeMax = 1.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone translations in centimeters per second. Smaller values 
	 * may help prevent the pose breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"))
	float MaximumTranslationVelocity = 500.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone rotations in degrees per second. Smaller values
	 * may help prevent the pose breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"))
	float MaximumRotationVelocity = 360.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone scales. Smaller values may help prevent the pose 
	 * breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"))
	float MaximumScaleVelocity = 4.0f;

	/**
	 * The maximum velocity to allow for extrapolation of curves. Smaller values may help prevent extreme curve values 
	 * during blending but too small values can make the blending of curves less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Min = "0.0", UIMin = "0.0"))
	float MaximumCurveVelocity = 100.0f;

	/**
	* Clear any active blends if we just became relevant, to avoid carrying over undesired blends.
	*/	
	UPROPERTY(EditAnywhere, Category = Blending)
	bool bResetOnBecomingRelevant = true;

	/**
	* When enabled this option will forward inertialization requests through any downstream UseCachedPose nodes which 
	* have had their update skipped (e.g. because they have already been updated in another location). This can be
	* useful in the case where the same cached pose is used in multiple places, and having an inertialization request 
	* that goes with it caught in only one of those places would create popping.
	*/
	UPROPERTY(EditAnywhere, Category = Requests)
	bool bForwardRequestsThroughSkippedCachedPoseNodes = false;

	/* UAnimGraphNode_Deadblending tag to match at runtime for filtering inertialization requests */
	UPROPERTY()
	FName Tag = NAME_None;

#if WITH_EDITORONLY_DATA
	
	// This setting can be used to show what the extrapolation of the animation looks like.
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowExtrapolations = false;

#endif // WITH_EDITORONLY_DATA

public: // FAnimNode_DeadBlending

	ENGINE_API FAnimNode_DeadBlending();

	/**
	 * Request to activate inertialization. If multiple requests are made on the same inertialization node, the request 
	 * with the minimum blend time will be used.
	 */
	ENGINE_API virtual void RequestInertialization(const FInertializationRequest& Request);

public: // FAnimNode_Base

	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ENGINE_API virtual bool NeedsDynamicReset() const override;

private:
	
	/**
	 * Deactivates the inertialization and frees any temporary memory.
	 */
	void Deactivate();

	/**
	 * Records the pose and velocity of the animation being transitioned from, and computes the extrapolation half-lives.
	 * 
	 * @param InPose		The current pose for the animation being transitioned to.
	 * @param InCurves		The current curves for the animation being transitioned to.
	 * @param InAttributes	The current attributes for the animation being transitioned to.
	 * @param SrcPosePrev	The pose recorded as output of the inertializer two frames ago.
	 * @param SrcPoseCurr	The pose recorded as output of the inertializer on the previous frame.
	 */
	void InitFrom(
		const FCompactPose& InPose, 
		const FBlendedCurve& InCurves, 
		const UE::Anim::FStackAttributeContainer& Attributes,
		const FInertializationSparsePose& SrcPosePrev,
		const FInertializationSparsePose& SrcPoseCurr);

	/**
	 * Computes the extrapolated pose and blends it with the input pose.
	 * 
	 * @param InOutPose			The current pose to blend with the extrapolated pose.
	 * @param InOutCurves		The current curves to blend with the extrapolated curves.
	 * @param InOutAttributes	The current attributes to blend with the extrapolated attributes.
	 */
	void ApplyTo(FCompactPose& InOutPose, FBlendedCurve& InOutCurves, UE::Anim::FStackAttributeContainer& InOutAttributes);

public: // IBoneReferenceSkeletonProvider
	ENGINE_API class USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;

private:

	// Cached curve filter built from FilteredCurves
	UE::Anim::FCurveFilter CurveFilter;

	// Cached curve filter built from ExtrapolationFilteredCurves
	UE::Anim::FCurveFilter ExtrapolatedCurveFilter;

	// Cache compact pose bone index for FilteredBones
	TArray<FCompactPoseBoneIndex, TInlineAllocator<8>> BoneFilter;

	// Snapshots of the actor pose generated as output.
	FInertializationSparsePose PrevPoseSnapshot;
	FInertializationSparsePose CurrPoseSnapshot;

	// Pending inertialization requests.
	UPROPERTY(Transient)
	TArray<FInertializationRequest> RequestQueue;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;

private:

	// Recorded pose state at point of transition.

	TArray<int32> BoneIndices;
	
	TArray<FVector> BoneTranslations;
	TArray<FQuat> BoneRotations;
	TArray<FQuat4f> BoneRotationDirections;
	TArray<FVector> BoneScales;

	TArray<FVector3f> BoneTranslationVelocities;
	TArray<FVector3f> BoneRotationVelocities;
	TArray<FVector3f> BoneScaleVelocities;

	TArray<FVector3f> BoneTranslationDecayHalfLives;
	TArray<FVector3f> BoneRotationDecayHalfLives;
	TArray<FVector3f> BoneScaleDecayHalfLives;

	FVector3f RootTranslationVelocity;
	FVector3f RootRotationVelocity;
	FVector3f RootScaleVelocity;

	// Recorded curve state at the point of transition.

	struct FDeadBlendingCurveElement : public UE::Anim::FCurveElement
	{
		float Velocity = 0.0f;
		float HalfLife = 0.0f;

		FDeadBlendingCurveElement() = default;
	};

	// Recorded curve state at the point of transition
	TBaseBlendedCurve<TInlineAllocator<8>, FDeadBlendingCurveElement> CurveData;

	// Temporary storage for curve data of the Destination Pose
	TBaseBlendedCurve<TInlineAllocator<8>, UE::Anim::FCurveElement> PoseCurveData;

private:

	// Variable used to store the elapsed delta time between calls to evaluate.
	float DeltaTime = 0.0f;

	// Current Inertialization state.
	EInertializationState InertializationState = EInertializationState::Inactive;

	// Time since the last request for inertialization.
	float InertializationTime = 0.0f;

	// Current inertialization duration (used for curves).
	float InertializationDuration = 0.0f;

	// Current inertialization durations for each bone, indexed by skeleton bone index (used for per-bone blending).
	TCustomBoneIndexArray<float, FSkeletonPoseBoneIndex> InertializationDurationPerBone;

	// Maximum of InertializationDuration and all entries in InertializationDurationPerBone (used for knowing when to shutdown the inertialization).
	float InertializationMaxDuration = 0.0f;

	// Current blend mode being used for Inertialization.
	EAlphaBlendOption InertializationBlendMode = EAlphaBlendOption::Linear;

	// Custom blend curve being used by the current blend mode.
	UPROPERTY(Transient)
	TObjectPtr<UCurveFloat> InertializationCustomBlendCurve = nullptr;


// if ANIM_TRACE_ENABLED - these properties are only used for debugging when ANIM_TRACE_ENABLED == 1

	// Description for the current inertialization request
	FString InertializationRequestDescription;

	// Node Id for the current inertialization request
	int32 InertializationRequestNodeId = INDEX_NONE;

	// Anim Instance for the current inertialization request
	UPROPERTY(Transient)
	TObjectPtr<UObject> InertializationRequestAnimInstance = nullptr;

// endif ANIM_TRACE_ENABLED
};
