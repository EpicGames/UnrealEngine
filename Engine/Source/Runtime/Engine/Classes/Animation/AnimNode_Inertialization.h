// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimInertializationRequest.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"

#include "AnimNode_Inertialization.generated.h"


// Inertialization: High-Performance Animation Transitions in 'Gears of War'
// David Bollo
// Game Developer Conference 2018
//
// https://www.gdcvault.com/play/1025165/Inertialization
// https://www.gdcvault.com/play/1025331/Inertialization


namespace UE::Anim
{

// Event that can be subscribed to request inertialization-based blends
class IInertializationRequester : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(IInertializationRequester, ENGINE_API);

public:
	static ENGINE_API const FName Attribute;

	// Request to activate inertialization for a duration.
	// If multiple requests are made on the same inertialization node, the minimum requested time will be used.
	virtual void RequestInertialization(float InRequestedDuration, const UBlendProfile* InBlendProfile = nullptr) = 0;

	// Request to activate inertialization.
	// If multiple requests are made on the same inertialization node, the minimum requested time will be used.
	ENGINE_API virtual void RequestInertialization(const FInertializationRequest& InInertializationRequest);

	// Add a record of this request
	virtual void AddDebugRecord(const FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId) = 0;

	virtual FName GetTag() const = 0;
};

}	// namespace UE::Anim

UENUM()
enum class EInertializationState : uint8
{
	Inactive,		// Inertialization inactive
	Pending,		// Inertialization request pending... prepare to capture the pose difference and then switch to active
	Active			// Inertialization active... apply the previously captured pose difference
};

UENUM()
enum class UE_DEPRECATED(5.4, "Internal private pose storage is now used by inertialization.") EInertializationBoneState : uint8
{
	Invalid,		// Invalid bone (ie: bone was present in the skeleton but was not present in the pose when it was captured)
	Valid,			// Valid bone
	Excluded		// Valid bone that is to be excluded from the inertialization request
};


UENUM()
enum class EInertializationSpace : uint8
{
	Default,		// Inertialize in local space (default)
	WorldSpace,		// Inertialize translation and rotation in world space (to conceal discontinuities in actor transform such snapping to a new attach parent)
	WorldRotation	// Inertialize rotation only in world space (to conceal discontinuities in actor orientation)
};

struct FInertializationCurve
{
	FBlendedHeapCurve BlendedCurve;

	FInertializationCurve() = default;

	FInertializationCurve(const FInertializationCurve& Other)
	{
		*this = Other;
	}

	FInertializationCurve(FInertializationCurve&& Other)
	{
		*this = MoveTemp(Other);
	}

	FInertializationCurve& operator=(const FInertializationCurve& Other)
	{
		BlendedCurve.CopyFrom(Other.BlendedCurve);
		return *this;
	}

	FInertializationCurve& operator=(FInertializationCurve&& Other)
	{
		BlendedCurve.MoveFrom(Other.BlendedCurve);
		return *this;
	}

	template <typename OtherAllocator>
	void InitFrom(const TBaseBlendedCurve<OtherAllocator>& Other)
	{
		BlendedCurve.CopyFrom(Other);
	}
};

USTRUCT()
struct UE_DEPRECATED(5.4, "Internal private pose storage is now used by inertialization.") FInertializationPose
{
	GENERATED_BODY()

	FTransform ComponentTransform;
	TArray<FTransform> BoneTransforms;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<EInertializationBoneState> BoneStates;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FInertializationCurve Curves;
	FName AttachParentName;
	float DeltaTime;

	FInertializationPose()
		: ComponentTransform(FTransform::Identity)
		, AttachParentName(NAME_None)
		, DeltaTime(0.0f)
	{
	}
	

	FInertializationPose(const FInertializationPose&) = default;
	FInertializationPose(FInertializationPose&&) = default;
	FInertializationPose& operator=(const FInertializationPose&) = default;
	FInertializationPose& operator=(FInertializationPose&&) = default;

	void InitFrom(const FCompactPose& Pose, const FBlendedCurve& InCurves, const FTransform& InComponentTransform, const FName& InAttachParentName, float InDeltaTime);
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <>
struct TUseBitwiseSwap<FInertializationPose>
{
	enum { Value = false };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Internal private structure used for storing a pose snapshots sparsely (i.e. when we may not have the full set of transform for every bone)
struct FInertializationSparsePose
{
	friend struct FAnimNode_Inertialization;
	friend struct FAnimNode_DeadBlending;

private:

	// Transform of the component at the point of the snapshot
	FTransform ComponentTransform;
	
	// Has Root Motion
	bool bHasRootMotion = false;

	// Root Motion Delta at the point of the snapshot
	FTransform RootMotionDelta;

	// For each SkeletonPoseBoneIndex this array stores the index into the BoneTranslations, BoneRotations, and 
	// BoneScales arrays which contains that bone's data. Or INDEX_NONE if this bone's data is not in the snapshot.
	TArray<int32> BoneIndices;
	
	// Bone translation Data
	TArray<FVector> BoneTranslations;
	
	// Bone Rotation Data
	TArray<FQuat> BoneRotations;
	
	// Bone Scale Data
	TArray<FVector> BoneScales;

    // Curve Data
	FInertializationCurve Curves;

	// Attached Parent object Name
	FName AttachParentName = NAME_None;
	
	// Delta Time since last snapshot
	float DeltaTime = 0.0f;

	void InitFrom(
		const FCompactPose& Pose, 
		const FBlendedCurve& InCurves, 
		const UE::Anim::FStackAttributeContainer& Attributes, 
		const FTransform& InComponentTransform, 
		const FName InAttachParentName, 
		const float InDeltaTime);

	bool IsEmpty() const;
	void Empty();
};

USTRUCT()
struct UE_DEPRECATED(5.4, "Internal private pose difference storage is now used by inertialization.") FInertializationBoneDiff
{
	GENERATED_BODY()

	FVector TranslationDirection;
	FVector RotationAxis;
	FVector ScaleAxis;

	float TranslationMagnitude;
	float TranslationSpeed;

	float RotationAngle;
	float RotationSpeed;

	float ScaleMagnitude;
	float ScaleSpeed;

	FInertializationBoneDiff()
		: TranslationDirection(FVector::ZeroVector)
		, RotationAxis(FVector::ZeroVector)
		, ScaleAxis(FVector::ZeroVector)
		, TranslationMagnitude(0.0f)
		, TranslationSpeed(0.0f)
		, RotationAngle(0.0f)
		, RotationSpeed(0.0f)
		, ScaleMagnitude(0.0f)
		, ScaleSpeed(0.0f)
	{
	}

	void Clear()
	{
		TranslationDirection = FVector::ZeroVector;
		RotationAxis = FVector::ZeroVector;
		ScaleAxis = FVector::ZeroVector;
		TranslationMagnitude = 0.0f;
		TranslationSpeed = 0.0f;
		RotationAngle = 0.0f;
		RotationSpeed = 0.0f;
		ScaleMagnitude = 0.0f;
		ScaleSpeed = 0.0f;
	}
};

struct FInertializationCurveDiffElement : public UE::Anim::FCurveElement
{
	float Delta = 0.0f;
	float Derivative = 0.0f;

	FInertializationCurveDiffElement() = default;

	void Clear()
	{
		Value = 0.0f;
		Delta = 0.0f;
		Derivative = 0.0f;
	}
};

USTRUCT()
struct UE_DEPRECATED(5.4, "Internal private pose difference storage is now used by inertialization.") FInertializationPoseDiff
{
	GENERATED_BODY()

	FInertializationPoseDiff()
		: InertializationSpace(EInertializationSpace::Default)
	{
	}

	void Reset(uint32 NumBonesSlack = 0)
	{
		BoneDiffs.Empty(NumBonesSlack);
		CurveDiffs.Empty();
		InertializationSpace = EInertializationSpace::Default;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	void InitFrom(const FCompactPose& Pose, const FBlendedCurve& Curves, const FTransform& ComponentTransform, const FName& AttachParentName, const FInertializationPose& Prev1, const FInertializationPose& Prev2, const UE::Anim::FCurveFilter& CurveFilter);
	void ApplyTo(FCompactPose& Pose, FBlendedCurve& Curves, float InertializationElapsedTime, float InertializationDuration, TArrayView<const float> InertializationDurationPerBone) const;

	EInertializationSpace GetInertializationSpace() const
	{
		return InertializationSpace;
	}

private:

	TArray<FInertializationBoneDiff> BoneDiffs;
	TBaseBlendedCurve<FDefaultAllocator, FInertializationCurveDiffElement> CurveDiffs;
	EInertializationSpace InertializationSpace;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Inertialization : public FAnimNode_Base
#if CPP
	, public IBoneReferenceSkeletonProvider
#endif
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

#if WITH_EDITORONLY_DATA
	void SetTag(FName InTag) { Tag = InTag; }
#endif
	FName GetTag() { return Tag; }

private:

	// Optional default blend profile to use when no blend profile is supplied with the inertialization request
	UPROPERTY(EditAnywhere, Category = BlendProfile, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> DefaultBlendProfile = nullptr;

	// List of curves that should not use inertial blending. These curves will instantly change when inertialization begins.
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FName> FilteredCurves;

	// List of bones that should not use inertial blending. These bones will change instantly when the animation switches.
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FBoneReference> FilteredBones;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "Preallocate Memory has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Preallocate Memory has been deprecated."))
	bool bPreallocateMemory_DEPRECATED = false;
#endif

	/**
	* Clear any active blends if we just became relevant, to avoid carrying over undesired blends.
	*/	
	UPROPERTY(EditAnywhere, Category = Blending)
	bool bResetOnBecomingRelevant = false;

	/**
	* When enabled this option will forward inertialization requests through any downstream UseCachedPose nodes which 
	* have had their update skipped (e.g. because they have already been updated in another location). This can be
	* useful in the case where the same cached pose is used in multiple places, and having an inertialization request 
	* that goes with it caught in only one of those places would create popping.
	*/
	UPROPERTY(EditAnywhere, Category = Requests)
	bool bForwardRequestsThroughSkippedCachedPoseNodes = true;

	UPROPERTY()
	FName Tag = NAME_None;

public: // FAnimNode_Inertialization

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimNode_Inertialization() = default;
	~FAnimNode_Inertialization() = default;
	FAnimNode_Inertialization(const FAnimNode_Inertialization&) = default;
	FAnimNode_Inertialization(FAnimNode_Inertialization&&) = default;
	FAnimNode_Inertialization& operator=(const FAnimNode_Inertialization&) = default;
	FAnimNode_Inertialization& operator=(FAnimNode_Inertialization&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// Request to activate inertialization for a duration.
	// If multiple requests are made on the same inertialization node, the minimum requested time will be used.
	//
	ENGINE_API virtual void RequestInertialization(float Duration, const UBlendProfile* BlendProfile);

	// Request to activate inertialization.
	// If multiple requests are made on the same inertialization node, the minimum requested time will be used.
	//
	ENGINE_API virtual void RequestInertialization(const FInertializationRequest& InertializationRequest);

	// Log an error when a node wants to inertialize but no inertialization ancestor node exists
	//
	static ENGINE_API void LogRequestError(const FAnimationUpdateContext& Context, const int32 NodePropertyIndex);
	static ENGINE_API void LogRequestError(const FAnimationUpdateContext& Context, const FPoseLinkBase& RequesterPoseLink);

public: // FAnimNode_Base

	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;

	ENGINE_API virtual bool NeedsDynamicReset() const override;
	ENGINE_API virtual void ResetDynamics(ETeleportType InTeleportType) override;

private:

	/**
	 * Deactivates the inertialization and frees any temporary memory.
	 */
	void Deactivate();

	/**
	 * Computes the inertialization pose difference between the current pose and the previous pose and computes the velocity of this difference.
	 *
	 * @param InPose				The current pose for the animation being transitioned to.
	 * @param InCurves				The current curves for the animation being transitioned to.
	 * @param InAttributes			The current attributes for the animation being transitioned to.
	 * @param ComponentTransform	The component transform of the current pose
	 * @param AttachParentName		The name of the attached parent object
	 * @param PreviousPose1			The pose recorded as output of the inertializer on the previous frame.
	 * @param PreviousPose2			The pose recorded as output of the inertializer two frames ago.
	 */
	void InitFrom(
		const FCompactPose& InPose, 
		const FBlendedCurve& InCurves, 
		const UE::Anim::FStackAttributeContainer& InAttributes,
		const FTransform& ComponentTransform, 
		const FName AttachParentName, 
		const FInertializationSparsePose& PreviousPose1, 
		const FInertializationSparsePose& PreviousPose2);

	/**
	 * Applies the inertialization difference to the given pose (decaying to zero as ElapsedTime approaches Duration)
	 *
	 * @param InOutPose		    The current pose to blend with the extrapolated pose.
	 * @param InOutCurves	    The current curves to blend with the extrapolated curves.
	 * @param InOutAttributes	The current attributes to blend with the extrapolated attributes.
	 */
	void ApplyTo(FCompactPose& InOutPose, FBlendedCurve& InOutCurves, UE::Anim::FStackAttributeContainer& InOutAttributes);

	// Snapshots of the actor pose generated as output.
	FInertializationSparsePose PrevPoseSnapshot;
	FInertializationSparsePose CurrPoseSnapshot;

	// Elapsed delta time between calls to evaluate
	float DeltaTime = 0.0f;

	// Pending inertialization requests
	UPROPERTY(Transient)
	TArray<FInertializationRequest> RequestQueue;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;

	// Inertialization state
	EInertializationState InertializationState = EInertializationState::Inactive;

	// Amount of time elapsed during the Inertialization
	float InertializationElapsedTime = 0.0f;

	// Inertialization duration for the main inertialization request (used for curve blending and deficit tracking)
	float InertializationDuration = 0.0f;

	// Inertialization durations indexed by skeleton bone index (used for per-bone blending)
	TCustomBoneIndexArray<float, FSkeletonPoseBoneIndex> InertializationDurationPerBone;

	// Maximum of InertializationDuration and all entries in InertializationDurationPerBone (used for knowing when to shutdown the inertialization)
	float InertializationMaxDuration = 0.0f;

	// Inertialization deficit (for tracking and reducing 'pose melting' when thrashing inertialization requests)
	float InertializationDeficit = 0.0f;

	// Inertialization pose differences
	TArray<int32> BoneIndices;
	TArray<FVector3f> BoneTranslationDiffDirection;
	TArray<float> BoneTranslationDiffMagnitude;
	TArray<float> BoneTranslationDiffSpeed;
	TArray<FVector3f> BoneRotationDiffAxis;
	TArray<float> BoneRotationDiffAngle;
	TArray<float> BoneRotationDiffSpeed;
	TArray<FVector3f> BoneScaleDiffAxis;
	TArray<float> BoneScaleDiffMagnitude;
	TArray<float> BoneScaleDiffSpeed;
	FVector3f RootTranslationVelocityDiffDirection;
	float RootTranslationVelocityDiffMagnitude;
	FVector3f RootRotationVelocityDiffDirection;
	float RootRotationVelocityDiffMagnitude;
	FVector3f RootScaleVelocityDiffDirection;
	float RootScaleVelocityDiffMagnitude;

	// Curve differences
	TBaseBlendedCurve<FDefaultAllocator, FInertializationCurveDiffElement> CurveDiffs;

	// Temporary storage for curve data of the Destination Pose
	TBaseBlendedCurve<TInlineAllocator<8>, UE::Anim::FCurveElement> PoseCurveData;

public: // IBoneReferenceSkeletonProvider
	ENGINE_API class USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;

private:

	// Cached curve filter built from FilteredCurves
	UE::Anim::FCurveFilter CurveFilter;

	// Cache compact pose bone index for FilteredBones
	TArray<FCompactPoseBoneIndex, TInlineAllocator<8>> BoneFilter;

// if ANIM_TRACE_ENABLED - these properties are only used for debugging when ANIM_TRACE_ENABLED == 1

	// Description for the current inertialization request
	FString InertializationRequestDescription;

	// Node Id for the current inertialization request
	int32 InertializationRequestNodeId = INDEX_NONE;

	// Anim Instance for the current inertialization request
	UPROPERTY(Transient)
	TObjectPtr<UObject> InertializationRequestAnimInstance = nullptr;

// endif ANIM_TRACE_ENABLED

};
