// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimLayerInterface.h"

#include "AnimNode_LinkedAnimGraph.generated.h"

struct FAnimInstanceProxy;
class UUserDefinedStruct;
struct FAnimBlueprintFunction;
class IAnimClassInterface;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LinkedAnimGraph : public FAnimNode_CustomProperty
{
	GENERATED_BODY()

public:

	ENGINE_API FAnimNode_LinkedAnimGraph();

	/** 
	 *  Input poses for the node, intentionally not accessible because if there's no input
	 *  nodes in the target class we don't want to show these as pins
	 */
	UPROPERTY()
	TArray<FPoseLink> InputPoses;

	/** List of input pose names, 1-1 with pose links about, built by the compiler */
	UPROPERTY()
	TArray<FName> InputPoseNames;

	/** The class spawned for this linked instance */
	UPROPERTY(EditAnywhere, Category = Settings)
	TSubclassOf<UAnimInstance> InstanceClass;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Tag_DEPRECATED;
#endif
	
	// The root node of the dynamically-linked graph
	FAnimNode_Base* LinkedRoot;

	// Our node index
	int32 NodeIndex;

	// Cached node index for our linked function
	int32 CachedLinkedNodeIndex;

protected:
	// Inertial blending duration to request next update (pulled from the prior state's blend out)
	float PendingBlendOutDuration;

	// Optional blend profile to use during inertial blending (pulled from the prior state's blend out)
	UPROPERTY(Transient)
	TObjectPtr<const UBlendProfile> PendingBlendOutProfile;

	// Inertial blending duration to request next update (pulled from the new state's blend in)
	float PendingBlendInDuration;

	// Optional blend profile to use during inertial blending (pulled from the new state's blend in)
	UPROPERTY(Transient)
	TObjectPtr<const UBlendProfile> PendingBlendInProfile;

public:
	/** Whether named notifies will be received by this linked instance from other instances (outer or other linked instances) */
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bReceiveNotifiesFromLinkedInstances : 1;

	/** Whether named notifies will be propagated from this linked instance to other instances (outer or other linked instances) */
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bPropagateNotifiesToLinkedInstances : 1;

	/** Dynamically set the anim class of this linked instance */
	ENGINE_API void SetAnimClass(TSubclassOf<UAnimInstance> InClass, const UAnimInstance* InOwningAnimInstance);

	/** Get the function name we should be linking with when we call DynamicLink/Unlink */
	ENGINE_API virtual FName GetDynamicLinkFunctionName() const;

	/** Get the dynamic link target */
	ENGINE_API virtual UAnimInstance* GetDynamicLinkTarget(UAnimInstance* InOwningAnimInstance) const;

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;

	// Initializes only the sub-graph that this node is linked to
	ENGINE_API void InitializeSubGraph_AnyThread(const FAnimationInitializeContext& Context);

	// Caches bones only for the sub graph that this node is linked to
	ENGINE_API void CacheBonesSubGraph_AnyThread(const FAnimationCacheBonesContext& Context);

protected:

	ENGINE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	// End of FAnimNode_Base interface

	// Re-create the linked instances for this node
	ENGINE_API void ReinitializeLinkedAnimInstance(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InNewAnimInstance = nullptr);

	// Shutdown the currently running instance
	ENGINE_API void TeardownInstance(const UAnimInstance* InOwningAnimInstance);

	// Check if the currently linked instance can be teared down
	virtual bool CanTeardownLinkedInstance(const UAnimInstance* LinkedInstance) const {return true;}

	// FAnimNode_CustomProperty interface
	virtual UClass* GetTargetClass() const override 
	{
		return *InstanceClass;
	}

#if WITH_EDITOR
	ENGINE_API virtual void HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
#endif	// #if WITH_EDITOR

	/** Link up pose links dynamically with linked instance */
	ENGINE_API void DynamicLink(UAnimInstance* InOwningAnimInstance);

	/** Break any pose links dynamically with linked instance */
	ENGINE_API void DynamicUnlink(UAnimInstance* InOwningAnimInstance);

	/** Helper function for finding function inputs when linking/unlinking */
	ENGINE_API int32 FindFunctionInputIndex(const FAnimBlueprintFunction& AnimBlueprintFunction, const FName& InInputName);

	/** Request a blend when the active instance changes */
	ENGINE_API void RequestBlend(const IAnimClassInterface* PriorAnimBPClass, const IAnimClassInterface* NewAnimBPClass);

	friend class UAnimInstance;

	// Stats
#if ANIMNODE_STATS_VERBOSE
	ENGINE_API virtual void InitializeStatID() override;
#endif

};

UE_DEPRECATED(4.24, "FAnimNode_SubInstance has been renamed to FAnimNode_LinkedAnimGraph")
typedef FAnimNode_LinkedAnimGraph FAnimNode_SubInstance;
