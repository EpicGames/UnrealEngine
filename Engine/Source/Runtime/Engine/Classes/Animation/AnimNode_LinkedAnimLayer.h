// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimLayerInterface.h"

#include "AnimNode_LinkedAnimLayer.generated.h"

struct FAnimInstanceProxy;
class UUserDefinedStruct;
struct FAnimBlueprintFunction;
class IAnimClassInterface;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LinkedAnimLayer : public FAnimNode_LinkedAnimGraph
{
	GENERATED_BODY()

public:
	/** 
	 * Optional interface. If this is set then this node will only accept (both statically and dynamically) anim instances that implement this interface.
	 * If not set, then this is considered a 'self' layer. This value is set when Layer is changed in the details panel.
	 */
	UPROPERTY()
	TSubclassOf<UAnimLayerInterface> Interface;

	/** The layer in the interface to use */
	UPROPERTY(EditAnywhere, Category = Settings)
	FName Layer;

	/** Set the layer's 'overlay' externally managed linked instance. */
	ENGINE_API void SetLinkedLayerInstance(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InNewLinkedInstance);

	/** FAnimNode_Base interface */
	ENGINE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	ENGINE_API void OnUninitializeAnimInstance(UAnimInstance* InAnimInstance);

	/** FAnimNode_CustomProperty interface */
	ENGINE_API virtual void InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass) override;
	
	/** FAnimNode_LinkedAnimGraph interface */
	ENGINE_API virtual FName GetDynamicLinkFunctionName() const override;
	ENGINE_API virtual UAnimInstance* GetDynamicLinkTarget(UAnimInstance* InOwningAnimInstance) const override;
	virtual UClass* GetTargetClass() const override 
	{
		return *Interface;
	}

#if WITH_EDITOR
	// Event fired when the instance we are running has changed
	FSimpleMulticastDelegate& OnInstanceChanged() { return OnInstanceChangedEvent; }
	ENGINE_API virtual void HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
#endif

protected:
	ENGINE_API void InitializeSelfLayer(const UAnimInstance* SelfAnimInstance);

	// Initialize the source properties to copy from
	ENGINE_API void InitializeSourceProperties(const UAnimInstance* InAnimInstance);

	ENGINE_API virtual bool CanTeardownLinkedInstance(const UAnimInstance* LinkedInstance) const override;

	// Cleanup Shared LinkedLayers Data associated with InPreviousTargetInstance
	ENGINE_API void CleanupSharedLinkedLayersData(const UAnimInstance* InOwningAnimInstance, UAnimInstance* InPreviousTargetInstance);
	
#if WITH_EDITOR
	// Event fired when the instance we are running has changed
	FSimpleMulticastDelegate OnInstanceChangedEvent;
#endif

    // stats
#if ANIMNODE_STATS_VERBOSE
	ENGINE_API virtual void InitializeStatID() override;
#endif
};

UE_DEPRECATED(4.24, "FAnimNode_Layer has been renamed to FAnimNode_LinkedAnimLayer")
typedef FAnimNode_LinkedAnimLayer FAnimNode_Layer;
