// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCurveTypes.h"
#include "BonePose.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_LinkedInputPose.generated.h"

USTRUCT()
struct FAnimNode_LinkedInputPose : public FAnimNode_Base
{
	GENERATED_BODY()

	/** The default name of this input pose */
	static ENGINE_API const FName DefaultInputPoseName;

	FAnimNode_LinkedInputPose()
		: Name(DefaultInputPoseName)
		, Graph(NAME_None)
		, bIsCachedInputPoseInitialized(false)
		, bIsOutputLinked(true)
		, OuterGraphNodeIndex(INDEX_NONE)
		, InputProxy(nullptr)
	{
	}

	/** The name of this linked input pose node's pose, used to identify the input of this graph. */
	UPROPERTY(EditAnywhere, Category = "Inputs", meta = (NeverAsPin))
	FName Name;

	/** The graph that this linked input pose node is in, filled in by the compiler */
	UPROPERTY()
	FName Graph;

	/** Input pose, optionally linked dynamically to another graph */
	UPROPERTY()
	FPoseLink InputPose;

	/** 
	 * If this linked input pose is not dynamically linked, this cached data will be populated by the calling 
	 * linked instance node before this graph is processed.
	 */
	FCompactHeapPose CachedInputPose;
	FBlendedHeapCurve CachedInputCurve;
	UE::Anim::FHeapAttributeContainer CachedAttributes;

	// CachedInputPose can have bone data allocated but uninitialized.
	// This can happen if an anim graph has an Input Pose node with nothing populating it (e.g. if it's played as the only animbp on an actor).
	uint8 bIsCachedInputPoseInitialized : 1;

	// True if this linked input pose output is connected to the graph root
	UPROPERTY(meta = (BlueprintCompilerGeneratedDefaults))
	uint8 bIsOutputLinked : 1;

	// The node index of the currently-linked outer node
	int32 OuterGraphNodeIndex;



	// FAnimNode_Base interface
#if ENABLE_ANIMGRAPH_TRAVERSAL_DEBUG
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
#endif
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	/** Called by linked instance nodes to dynamically link this to an outer graph */
	ENGINE_API void DynamicLink(FAnimInstanceProxy* InInputProxy, FPoseLinkBase* InPoseLink, int32 InOuterGraphNodeIndex);

	/** Called by linked instance nodes to dynamically unlink this to an outer graph */
	ENGINE_API void DynamicUnlink();

private:
	/** The proxy to use when getting inputs, set when dynamically linked */
	FAnimInstanceProxy* InputProxy;
};

UE_DEPRECATED(4.24, "FAnimNode_SubInput has been renamed to FAnimNode_LinkedInputPose")
typedef FAnimNode_LinkedInputPose FAnimNode_SubInput;
