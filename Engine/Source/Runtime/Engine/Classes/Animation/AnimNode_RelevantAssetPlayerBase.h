// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_RelevantAssetPlayerBase.generated.h"

/* Base class for any asset playing anim node */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_AssetPlayerRelevancyBase : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	/** Get the animation asset associated with the node, derived classes should implement this */
	ENGINE_API virtual class UAnimationAsset* GetAnimAsset() const;

	/** Get the currently referenced time within the asset player node */
	ENGINE_API virtual float GetAccumulatedTime() const;

	/** Override the currently accumulated time */
	ENGINE_API virtual void SetAccumulatedTime(float NewTime);

	// Functions to report data to getters, this is required for all asset players (but can't be pure abstract because of struct instantiation generated code).
	ENGINE_API virtual float GetCurrentAssetLength() const;
	ENGINE_API virtual float GetCurrentAssetTime() const;
	ENGINE_API virtual float GetCurrentAssetTimePlayRateAdjusted() const;

	// Does this asset player loop back to the start when it reaches the end?
	ENGINE_API virtual bool IsLooping() const;

	// Check whether this node should be ignored when testing for relevancy in state machines
	ENGINE_API virtual bool GetIgnoreForRelevancyTest() const;

	// Set whether this node should be ignored when testing for relevancy in state machines
	ENGINE_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest);

	/** Get the last encountered blend weight for this node */
	ENGINE_API virtual float GetCachedBlendWeight() const;

	/** Set the cached blendweight to zero */
	ENGINE_API virtual void ClearCachedBlendWeight();

	/** Get the delta time record owned by this asset player (or null) */
	ENGINE_API virtual const FDeltaTimeRecord* GetDeltaTimeRecord() const;
};
