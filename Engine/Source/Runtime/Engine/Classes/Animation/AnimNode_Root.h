// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_Root.generated.h"

// Root node of an animation tree (sink)
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Root : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Result;

#if WITH_EDITORONLY_DATA
protected:
	/** The name of this root node, used to identify the output of this graph. Filled in by the compiler, propagated from the parent graph. */
	UPROPERTY(meta=(FoldProperty, BlueprintCompilerGeneratedDefaults))
	FName Name;

	/** The group of this root node, used to group this output with others when used in a layer. */
	UPROPERTY(meta=(FoldProperty))
	FName LayerGroup = DefaultSharedGroup;

	UPROPERTY(meta= (DeprecatedProperty, DeprecationMessage = "Please, use LayerGroup"))
	FName Group_DEPRECATED;
#endif

public:	
	ENGINE_API FAnimNode_Root();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

#if WITH_EDITORONLY_DATA
	/** Set the name of this root node, used to identify the output of this graph */
	void SetName(FName InName) { Name = InName; }

	/** Set the group of this root node, used to group this output with others when used in a layer. */
	void SetGroup(FName InGroup) { LayerGroup = InGroup; }

#endif

	/** Get the name of this root node, used to identify the output of this graph */
	ENGINE_API FName GetName() const;

	/** Get the group of this root node, used to group this output with others when used in a layer. */
	ENGINE_API FName GetGroup() const;

#if WITH_EDITORONLY_DATA
	/** Default shared group used by layers. */
	ENGINE_API static FName DefaultSharedGroup;
#endif

	friend class UAnimGraphNode_Root;
};
