// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimCurveTypes.h"
#include "BonePose.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimNodeMessages.h"
#include "AnimNode_SaveCachedPose.generated.h"

namespace UE { namespace Anim {

// Event that can be subscribed to receive skipped updates when a cached pose is run.
// When a cached pose update call executes the link with the maximum weight, this event receives information about
// the other links with lesser weights
class FCachedPoseSkippedUpdateHandler : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE(FCachedPoseSkippedUpdateHandler);

public:
	FCachedPoseSkippedUpdateHandler(TUniqueFunction<void(TArrayView<const FMessageStack>)> InFunction)
		: Function(MoveTemp(InFunction))
	{}

	// Called when there are Update() calls that were skipped due to pose caching. 
	void OnUpdatesSkipped(TArrayView<const FMessageStack> InSkippedUpdates) { Function(InSkippedUpdates); }

private:
	// Function to call
	TUniqueFunction<void(TArrayView<const FMessageStack>)> Function;
};

/** RAII helper for cached pose lifetimes (as they are stored on the mem stack) */
struct FCachedPoseScope
{
	FCachedPoseScope();
	~FCachedPoseScope();
};

}}	// namespace UE::Anim

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SaveCachedPose : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Pose;

	/** Intentionally not exposed, set by AnimBlueprintCompiler */
	UPROPERTY()
	FName CachePoseName;

	float GlobalWeight;

protected:
	struct FCachedUpdateContext
	{
		FCachedUpdateContext() = default;
		~FCachedUpdateContext() = default;
		FCachedUpdateContext(FCachedUpdateContext&&) noexcept = default;
		FCachedUpdateContext& operator=(FCachedUpdateContext&&) noexcept = default;

		FCachedUpdateContext(FCachedUpdateContext&) = delete;
		FCachedUpdateContext& operator=(const FCachedUpdateContext&) = delete;

		FAnimationUpdateContext Context;
		TUniquePtr<FAnimationUpdateSharedContext> SharedContext;
	};

	TArray<FCachedUpdateContext> CachedUpdateContexts;

	FGraphTraversalCounter InitializationCounter;
	FGraphTraversalCounter CachedBonesCounter;
	FGraphTraversalCounter UpdateCounter;
	FGraphTraversalCounter EvaluationCounter;

public:	
	ENGINE_API FAnimNode_SaveCachedPose();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	ENGINE_API void PostGraphUpdate();
};

template<>
struct TStructOpsTypeTraits<FAnimNode_SaveCachedPose> : public TStructOpsTypeTraitsBase2<FAnimNode_SaveCachedPose>
{
	enum
	{
		WithCopy = false
	};
};
