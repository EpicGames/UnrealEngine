// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNode_AssetPlayerBase.h"
#include "Animation/InputScaleBias.h"
#include "Animation/AnimSequenceBase.h"
#include "AnimNode_SequencePlayer.generated.h"


// Sequence player node. Not instantiated directly, use FAnimNode_SequencePlayer or FAnimNode_SequencePlayer_Standalone
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequencePlayerBase : public FAnimNode_AssetPlayerBase
{
	GENERATED_BODY()

protected:
	// Corresponding state for PlayRateScaleBiasClampConstants
	UPROPERTY()
	FInputScaleBiasClampState PlayRateScaleBiasClampState;

public:
	// FAnimNode_AssetPlayerBase interface
	ENGINE_API virtual float GetCurrentAssetTime() const override;
	ENGINE_API virtual float GetCurrentAssetTimePlayRateAdjusted() const override;
	ENGINE_API virtual float GetCurrentAssetLength() const override;
	virtual UAnimationAsset* GetAnimAsset() const override { return GetSequence(); }
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	ENGINE_API float GetTimeFromEnd(float CurrentNodeTime) const;
	ENGINE_API float GetEffectiveStartPosition(const FAnimationBaseContext& Context) const;

	// The animation sequence asset to play
	virtual UAnimSequenceBase* GetSequence() const { return nullptr; }

protected:
	// Set the animation sequence asset to play
	virtual bool SetSequence(UAnimSequenceBase* InSequence) { return false; }

	// Set the animation to continue looping when it reaches the end
	virtual bool SetLoopAnimation(bool bInLoopAnimation) { return false; }
	
	// The Basis in which the PlayRate is expressed in. This is used to rescale PlayRate inputs.
	// For example a Basis of 100 means that the PlayRate input will be divided by 100.
	virtual float GetPlayRateBasis() const { return 1.0f; }

	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	virtual float GetPlayRate() const { return 1.0f; }
	
	// Additional scaling, offsetting and clamping of PlayRate input.
	// Performed after PlayRateBasis.
	virtual const FInputScaleBiasClampConstants& GetPlayRateScaleBiasClampConstants() const { static FInputScaleBiasClampConstants Dummy; return Dummy; }

	// The start position [range: 0 - sequence length] to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	virtual float GetStartPosition() const { return 0.0f; }

	// Use pose matching to choose the start position. Requires PoseSearch plugin.
	virtual bool GetStartFromMatchingPose() const { return false; }

	// Set the start time of this node.
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	virtual bool SetStartPosition(float InStartPosition) { return false; }

	// Set the play rate of this node.
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)	
	virtual bool SetPlayRate(float InPlayRate) { return false; }
};

// Sequence player node that can be used with constant folding
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequencePlayer : public FAnimNode_SequencePlayerBase
{
	GENERATED_BODY()

protected:
	friend class UAnimGraphNode_SequencePlayer;

#if WITH_EDITORONLY_DATA
	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// When enabled, acting as the leader, and using marker-based sync, this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty, EditCondition = "GroupRole != EAnimGroupRole::TransitionFollower && GroupRole != EAnimGroupRole::AlwaysFollower", EditConditionHides))
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
	
	// How this node will synchronize with other animations.
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category=Relevancy, meta=(FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;
#endif

	// The animation sequence asset to play
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, DisallowedClasses="/Script/Engine.AnimMontage"))
	TObjectPtr<UAnimSequenceBase> Sequence = nullptr;

#if WITH_EDITORONLY_DATA
	// The Basis in which the PlayRate is expressed in. This is used to rescale PlayRate inputs.
	// For example a Basis of 100 means that the PlayRate input will be divided by 100.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	float PlayRateBasis = 1.0f;

	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	float PlayRate = 1.0f;
	
	// Additional scaling, offsetting and clamping of PlayRate input.
	// Performed after PlayRateBasis.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName="PlayRateScaleBiasClamp", FoldProperty))
	FInputScaleBiasClampConstants PlayRateScaleBiasClampConstants;

	UPROPERTY()
	FInputScaleBiasClamp PlayRateScaleBiasClamp_DEPRECATED;

	// The start position between 0 and the length of the sequence to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	float StartPosition = 0.0f;

	// Should the animation loop back to the start when it reaches the end?
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	bool bLoopAnimation = true;

	// Use pose matching to choose the start position. Requires PoseSearch plugin.
	UPROPERTY(EditAnywhere, Category = PoseMatching, meta = (PinHiddenByDefault, FoldProperty))
	bool bStartFromMatchingPose = false;
#endif

public:
	// FAnimNode_SequencePlayerBase interface
	ENGINE_API virtual bool SetSequence(UAnimSequenceBase* InSequence) override;
	ENGINE_API virtual bool SetLoopAnimation(bool bInLoopAnimation) override;
	ENGINE_API virtual UAnimSequenceBase* GetSequence() const override;
	ENGINE_API virtual float GetPlayRateBasis() const override;
	ENGINE_API virtual float GetPlayRate() const override;
	ENGINE_API virtual const FInputScaleBiasClampConstants& GetPlayRateScaleBiasClampConstants() const override;
	ENGINE_API virtual float GetStartPosition() const override;
	ENGINE_API virtual bool GetStartFromMatchingPose() const override;
	ENGINE_API virtual bool SetStartPosition(float InStartPosition) override;
	ENGINE_API virtual bool SetPlayRate(float InPlayRate) override;

	// FAnimNode_AssetPlayerBase interface
	ENGINE_API virtual FName GetGroupName() const override;
	ENGINE_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	ENGINE_API virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const override;
	ENGINE_API virtual EAnimSyncMethod GetGroupMethod() const override;
	ENGINE_API virtual bool IsLooping() const override;
	ENGINE_API virtual bool GetIgnoreForRelevancyTest() const override;
	ENGINE_API virtual bool SetGroupName(FName InGroupName) override;
	ENGINE_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	ENGINE_API virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) override;
	ENGINE_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	ENGINE_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
};

// Sequence player node that can be used standalone (without constant folding)
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequencePlayer_Standalone : public FAnimNode_SequencePlayerBase
{	
	GENERATED_BODY()

protected:
	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category=Sync)
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category=Sync)
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// When enabled, acting as the leader, and using marker-based sync, this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (EditCondition = "GroupRole != EAnimGroupRole::TransitionFollower && GroupRole != EAnimGroupRole::AlwaysFollower", EditConditionHides))
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
	
	// How this node will synchronize with other animations.
	UPROPERTY(EditAnywhere, Category=Sync)
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category=Relevancy, meta=(PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;

	// The animation sequence asset to play
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, DisallowedClasses="/Script/Engine.AnimMontage"))
	TObjectPtr<UAnimSequenceBase> Sequence = nullptr;

	// The Basis in which the PlayRate is expressed in. This is used to rescale PlayRate inputs.
	// For example a Basis of 100 means that the PlayRate input will be divided by 100.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	float PlayRateBasis = 1.0f;

	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	float PlayRate = 1.0f;
	
	// Additional scaling, offsetting and clamping of PlayRate input.
	// Performed after PlayRateBasis.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName="PlayRateScaleBiasClamp"))
	FInputScaleBiasClampConstants PlayRateScaleBiasClampConstants;

	// The start position between 0 and the length of the sequence to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	float StartPosition = 0.0f;

	// Should the animation loop back to the start when it reaches the end?
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bLoopAnimation = true;

	// Use pose matching to choose the start position. Requires PoseSearch plugin.
	UPROPERTY(EditAnywhere, Category = PoseMatching, meta = (PinHiddenByDefault))
	bool bStartFromMatchingPose = false;

public:
	// FAnimNode_SequencePlayerBase interface
	virtual bool SetSequence(UAnimSequenceBase* InSequence) override { Sequence = InSequence; return true; }
	virtual bool SetLoopAnimation(bool bInLoopAnimation) override { bLoopAnimation = bInLoopAnimation; return true; }
	virtual UAnimSequenceBase* GetSequence() const override { return Sequence; }
	virtual float GetPlayRateBasis() const override { return PlayRateBasis; }
	virtual float GetPlayRate() const override { return PlayRate; }
	virtual const FInputScaleBiasClampConstants& GetPlayRateScaleBiasClampConstants() const override { return PlayRateScaleBiasClampConstants; }
	virtual float GetStartPosition() const override { return StartPosition; }
	virtual bool GetStartFromMatchingPose() const override { return bStartFromMatchingPose; }

	// FAnimNode_AssetPlayerBase interface
	virtual FName GetGroupName() const override { return GroupName; }
	virtual EAnimGroupRole::Type GetGroupRole() const override { return GroupRole; }
	virtual EAnimSyncMethod GetGroupMethod() const override { return Method; }
	virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const override { return bOverridePositionWhenJoiningSyncGroupAsLeader; }
	virtual bool IsLooping() const override { return bLoopAnimation; }
	virtual bool GetIgnoreForRelevancyTest() const override { return bIgnoreForRelevancyTest; }
	virtual bool SetGroupName(FName InGroupName) override { GroupName = InGroupName; return true; }
	virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override { GroupRole = InRole; return true; }
	virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override { Method = InMethod; return true; }
	virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) override { bOverridePositionWhenJoiningSyncGroupAsLeader = InOverridePositionWhenJoiningSyncGroupAsLeader; return true; }	
	virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override { bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest; return true; }
	virtual bool SetStartPosition(float InStartPosition) override { StartPosition = InStartPosition; return  true; }
	virtual bool SetPlayRate(float InPlayRate) override { PlayRate = InPlayRate; return true; }
};
