// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "AlphaBlend.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstance.h"
#include "Animation/BlendProfile.h"
#include "AnimNode_StateMachine.generated.h"

class IAnimClassInterface;
struct FAnimNode_AssetPlayerBase;
struct FAnimNode_AssetPlayerRelevancyBase;
struct FAnimNode_StateMachine;
struct FAnimNode_TransitionPoseEvaluator;

// Information about an active transition on the transition stack
USTRUCT()
struct FAnimationActiveTransitionEntry
{
	GENERATED_USTRUCT_BODY()

	// Elapsed time for this transition
	float ElapsedTime;

	// The transition alpha between next and previous states
	float Alpha;

	// Duration of this cross-fade (may be shorter than the nominal duration specified by the state machine if the target state had non-zero weight at the start)
	float CrossfadeDuration;

	// Cached Pose for this transition
	TArray<FTransform> InputPose;

	// Graph to run that determines the final pose for this transition
	FPoseLink CustomTransitionGraph;

	// To and from state ids
	int32 NextState;

	int32 PreviousState;

	// Notifies are copied from the reference transition info
	int32 StartNotify;

	int32 EndNotify;

	int32 InterruptNotify;
	
	TArray<FAnimNode_TransitionPoseEvaluator*> PoseEvaluators;

	// Blend data used for per-bone animation evaluation
	TArray<FBlendSampleData> StateBlendData;

	TArray<int32, TInlineAllocator<3>> SourceTransitionIndices;

protected:
	// Blend object to handle alpha interpolation
	FAlphaBlend Blend;

public:
	// Blend profile to use for this transition. Specifying this will make the transition evaluate per-bone
	UPROPERTY()
	TObjectPtr<UBlendProfile> BlendProfile;

	// Type of blend to use
	EAlphaBlendOption BlendOption;

	TEnumAsByte<ETransitionLogicType::Type> LogicType;

	// Is this transition active?
	bool bActive;

public:
	FAnimationActiveTransitionEntry();
	FAnimationActiveTransitionEntry(int32 NextStateID, float ExistingWeightOfNextState, int32 PreviousStateID, const FAnimationTransitionBetweenStates& ReferenceTransitionInfo, float CrossfadeTimeAdjustment);

	void InitializeCustomGraphLinks(const FAnimationUpdateContext& Context, const FBakedStateExitTransition& TransitionRule);

	void Update(const FAnimationUpdateContext& Context, int32 CurrentStateIndex, bool &OutFinished);
	
	void UpdateCustomTransitionGraph(const FAnimationUpdateContext& Context, FAnimNode_StateMachine& StateMachine, int32 ActiveTransitionIndex);
	void EvaluateCustomTransitionGraph(FPoseContext& Output, FAnimNode_StateMachine& StateMachine, bool IntermediatePoseIsValid, int32 ActiveTransitionIndex);

	bool Serialize(FArchive& Ar);

protected:
	float CalculateInverseAlpha(EAlphaBlendOption BlendMode, float InFraction) const;
	float CalculateAlpha(float InFraction) const;
};

USTRUCT()
struct FAnimationPotentialTransition
{
	GENERATED_USTRUCT_BODY()

	int32 TargetState;
	float CrossfadeTimeAdjustment;

	const FBakedStateExitTransition* TransitionRule;

	TArray<int32, TInlineAllocator<3>> SourceTransitionIndices;

public:
	FAnimationPotentialTransition();
	bool IsValid() const;
	void Clear();
};

//@TODO: ANIM: Need to implement WithSerializer and Identical for FAnimationActiveTransitionEntry?

// State machine node
USTRUCT()
struct FAnimNode_StateMachine : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:
	// Index into the BakedStateMachines array in the owning UAnimBlueprintGeneratedClass
	UPROPERTY()
	int32 StateMachineIndexInClass;

	// The maximum number of transitions that can be taken by this machine 'simultaneously' in a single frame
	UPROPERTY(EditAnywhere, Category=Settings)
	int32 MaxTransitionsPerFrame;

	// The maximum number of transition requests that can be buffered at any time.
	// The oldest transition requests are dropped to accommodate for newly created requests.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0"))
	int32 MaxTransitionsRequests = 32;

	// When the state machine becomes relevant, it is initialized into the Entry state.
	// It then tries to take any valid transitions to possibly end up in a different state on that same frame.
	// - if true, that new state starts full weight.
	// - if false, a blend is created between the entry state and that new state.
	// In either case all visited State notifications (Begin/End) will be triggered.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bSkipFirstUpdateTransition;

	// Reinitialize the state machine if we have become relevant to the graph
	// after not being ticked on the previous frame(s)
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bReinitializeOnBecomingRelevant;

	// Tag Notifies with meta data such as the active state and mirroring state.  Producing this
	// data has a  slight performance cost.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bCreateNotifyMetaData;

	// Allows a conduit to be used as this state machine's entry state
	// If a valid entry state cannot be found at runtime then this will generate a reference pose!
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bAllowConduitEntryStates;
private:
	// true if it is the first update.
	bool bFirstUpdate;

public:

	int32 GetCurrentState() const
	{
		return CurrentState;
	}

	float GetCurrentStateElapsedTime() const
	{
		return ElapsedTime;
	}

	ENGINE_API FName GetCurrentStateName() const;

	ENGINE_API bool IsTransitionActive(int32 TransIndex) const;

protected:
	// The current state within the state machine
	int32 CurrentState;

	// Elapsed time since entering the current state
	float ElapsedTime;

	// Current Transition Index being evaluated
	int32 EvaluatingTransitionIndex;

	// The state machine description this is an instance of
	const FBakedAnimationStateMachine* PRIVATE_MachineDescription;

	// The set of active transitions, if there are any
	TArray<FAnimationActiveTransitionEntry> ActiveTransitionArray;

	// The set of states in this state machine
	TArray<FPoseLink> StatePoseLinks;
	
	// Used during transitions to make sure we don't double tick a state if it appears multiple times
	TArray<int32> StatesUpdated;

	// Delegates that native code can hook into to handle state entry
	TArray<FOnGraphStateChanged> OnGraphStatesEntered;

	// Delegates that native code can hook into to handle state exits
	TArray<FOnGraphStateChanged> OnGraphStatesExited;

	// All alive transition requests that have been queued
	TArray<FTransitionEvent> QueuedTransitionEvents;

#if WITH_EDITORONLY_DATA
	// The set of transition requests handled this update
	TArray<FTransitionEvent> HandledTransitionEvents;
#endif

private:
	TArray<FPoseContext*> StateCachedPoses;

	FGraphTraversalCounter UpdateCounter;

	TArray<FGraphTraversalCounter> StateCacheBoneCounters;

	// The last time that each respective state was entered
	TArray<double> LastStateEntryTime; 

#if WITH_EDITOR
	int32 DebugForcedStateIndex = INDEX_NONE;
#endif

public:
	FAnimNode_StateMachine()
		: StateMachineIndexInClass(0)
		, MaxTransitionsPerFrame(3)
		, bSkipFirstUpdateTransition(true)
		, bReinitializeOnBecomingRelevant(true)
		, bCreateNotifyMetaData(true)
		, bAllowConduitEntryStates(false)
		, bFirstUpdate(true)
		, CurrentState(INDEX_NONE)
		, ElapsedTime(0.0f)
		, PRIVATE_MachineDescription(NULL)
	{
	}

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	ENGINE_API void ConditionallyCacheBonesForState(int32 StateIndex, FAnimationBaseContext Context);

	// Returns the blend weight of the specified state, as calculated by the last call to Update()
	ENGINE_API float GetStateWeight(int32 StateIndex) const;

	ENGINE_API const FBakedAnimationState& GetStateInfo(int32 StateIndex) const;
	ENGINE_API const FAnimationTransitionBetweenStates& GetTransitionInfo(int32 TransIndex) const;
	
	ENGINE_API bool IsValidTransitionIndex(int32 TransitionIndex) const;

	/** Cache the internal machine description */
	ENGINE_API void CacheMachineDescription(IAnimClassInterface* AnimBlueprintClass);

	ENGINE_API void SetState(const FAnimationBaseContext& Context, int32 NewStateIndex, bool bAllowReEntry);
	ENGINE_API void TransitionToState(const FAnimationUpdateContext& Context, const FAnimationTransitionBetweenStates& TransitionInfo, const FAnimationPotentialTransition* BakedTransitionInfo = nullptr);
	ENGINE_API const int32 GetStateIndex(FName StateName) const;

protected:
	// Tries to get the instance information for the state machine
	ENGINE_API const FBakedAnimationStateMachine* GetMachineDescription() const;

	ENGINE_API void SetStateInternal(int32 NewStateIndex);

	ENGINE_API const FBakedAnimationState& GetStateInfo() const;
	ENGINE_API const int32 GetStateIndex(const FBakedAnimationState& StateInfo) const;
	
	// finds the highest priority valid transition, information pass via the OutPotentialTransition variable.
	// OutVisitedStateIndices will let you know what states were checked, but is also used to make sure we don't get stuck in an infinite loop or recheck states
	ENGINE_API bool FindValidTransition(const FAnimationUpdateContext& Context, 
							const FBakedAnimationState& StateInfo,
							/*OUT*/ FAnimationPotentialTransition& OutPotentialTransition,
							/*OUT*/ TArray<int32, TInlineAllocator<4>>& OutVisitedStateIndices);

	// Helper function that will update the states associated with a transition
	ENGINE_API void UpdateTransitionStates(const FAnimationUpdateContext& Context, FAnimationActiveTransitionEntry& Transition);

	// helper function to test if a state is a conduit
	ENGINE_API bool IsAConduitState(int32 StateIndex) const;

	// helper functions for calling update and evaluate on state nodes
	ENGINE_API void UpdateState(int32 StateIndex, const FAnimationUpdateContext& Context);
	ENGINE_API const FPoseContext& EvaluateState(int32 StateIndex, const FPoseContext& Context);

	// transition type evaluation functions
	ENGINE_API void EvaluateTransitionStandardBlend(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, bool bIntermediatePoseIsValid);
	ENGINE_API void EvaluateTransitionStandardBlendInternal(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, const FPoseContext& PreviousStateResult, const FPoseContext& NextStateResult);
	ENGINE_API void EvaluateTransitionCustomBlend(FPoseContext& Output, FAnimationActiveTransitionEntry& Transition, bool bIntermediatePoseIsValid);

	// Get the time remaining in seconds for the most relevant animation in the source state 
	ENGINE_API float GetRelevantAnimTimeRemaining(const FAnimInstanceProxy* InAnimInstanceProxy, int32 StateIndex) const;
	float GetRelevantAnimTimeRemaining(const FAnimationUpdateContext& Context, int32 StateIndex) const
	{
		return GetRelevantAnimTimeRemaining(Context.AnimInstanceProxy, StateIndex);
	}

	// Get the time remaining as a fraction of the duration for the most relevant animation in the source state 
	ENGINE_API float GetRelevantAnimTimeRemainingFraction(const FAnimInstanceProxy* InAnimInstanceProxy, int32 StateIndex) const;
	float GetRelevantAnimTimeRemainingFraction(const FAnimationUpdateContext& Context, int32 StateIndex) const
	{
		return GetRelevantAnimTimeRemainingFraction(Context.AnimInstanceProxy, StateIndex);
	}

	ENGINE_API const FAnimNode_AssetPlayerRelevancyBase* GetRelevantAssetPlayerInterfaceFromState(const FAnimInstanceProxy* InAnimInstanceProxy, const FBakedAnimationState& StateInfo) const;
	const FAnimNode_AssetPlayerRelevancyBase* GetRelevantAssetPlayerInterfaceFromState(const FAnimationUpdateContext& Context, const FBakedAnimationState& StateInfo) const
	{
		return GetRelevantAssetPlayerInterfaceFromState(Context.AnimInstanceProxy, StateInfo);
	}

	ENGINE_API virtual void LogInertializationRequestError(const FAnimationUpdateContext& Context, int32 PreviousState, int32 NextState);

	/** Queues a new transition request, returns true if the transition request was successfully queued */
	ENGINE_API bool RequestTransitionEvent(const FTransitionEvent& InTransitionEvent);

	/** Removes all queued transition requests with the given event name */
	ENGINE_API void ClearTransitionEvents(const FName& EventName);

	/** Removes all queued transition requests*/
	ENGINE_API void ClearAllTransitionEvents();

	/** Returns whether or not the given event transition request has been queued */
	ENGINE_API bool QueryTransitionEvent(const int32 TransitionIndex, const FName& EventName) const;

	/** Behaves like QueryTransitionEvent but additionally marks the event for consumption */
	ENGINE_API bool QueryAndMarkTransitionEvent(const int32 TransitionIndex, const FName& EventName);

	/** Removes all marked events that are queued */
	ENGINE_API void ConsumeMarkedTransitionEvents();

#if WITH_EDITOR
	void DebugForceState(int32 InStateIndex)
	{
		DebugForcedStateIndex = InStateIndex;
	}
#endif

public:
	friend struct FAnimInstanceProxy;
	friend class UAnimationStateMachineLibrary;
	friend class FAnimationBlueprintEditor;
};
