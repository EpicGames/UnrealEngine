// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCurveTypes.h"
#include "BonePose.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_TransitionPoseEvaluator.generated.h"

/** Indicates which state is being evaluated by this node (source or destination). */
UENUM()
namespace EEvaluatorDataSource
{
	enum Type : int
	{
		EDS_SourcePose UMETA(DisplayName="Source Pose"),
		EDS_DestinationPose UMETA(DisplayName="Destination Pose")
	};
}

/** Determines the behavior this node will use when updating and evaluating. */
UENUM()
namespace EEvaluatorMode
{
	enum Mode : int
	{
		/** DataSource is ticked and evaluated every frame. */
		EM_Standard UMETA(DisplayName="Standard"),

		/** DataSource is never ticked and only evaluated on the first frame. Every frame after uses the cached pose from the first frame. */
		EM_Freeze UMETA(DisplayName="Freeze"),

		/** DataSource is ticked and evaluated for a given number of frames, then freezes after and uses the cached pose for future frames. */
		EM_DelayedFreeze UMETA(DisplayName="Delayed Freeze")
	};
}


/** Animation data node for state machine transitions.
 * Can be set to supply either the animation data from the transition source (From State) or the transition destination (To State).
*/
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_TransitionPoseEvaluator : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	FCompactHeapPose CachedPose;
	FBlendedHeapCurve CachedCurve;
	UE::Anim::FHeapAttributeContainer CachedAttributes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pose, meta=(NeverAsPin, ClampMin="1", UIMin="1"))
	int32 FramesToCachePose;

	int32 CacheFramesRemaining;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pose, meta=(NeverAsPin))
	TEnumAsByte<EEvaluatorDataSource::Type> DataSource;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pose, meta=(NeverAsPin))
	TEnumAsByte<EEvaluatorMode::Mode> EvaluatorMode;

	FGraphTraversalCounter CachedBonesCounter;

public:	
	ENGINE_API FAnimNode_TransitionPoseEvaluator();
	ENGINE_API void SetupCacheFrames();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	ENGINE_API bool InputNodeNeedsUpdate(const FAnimationUpdateContext& Context) const;
	ENGINE_API bool InputNodeNeedsEvaluate() const;
	ENGINE_API void CachePose(const FPoseContext& PoseToCache);
};
