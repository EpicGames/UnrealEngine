// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_UseCachedPose.generated.h"

USTRUCT()
struct FAnimNode_UseCachedPose : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	// Note: This link is intentionally not public; it's wired up during compilation
	UPROPERTY()
	FPoseLink LinkToCachingNode;

	/** Intentionally not exposed, set by AnimBlueprintCompiler */
	UPROPERTY(meta=(BlueprintCompilerGeneratedDefaults))
	FName CachePoseName;

public:	
	ENGINE_API FAnimNode_UseCachedPose();

	// FAnimNode_Base interface
	ENGINE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ENGINE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};
