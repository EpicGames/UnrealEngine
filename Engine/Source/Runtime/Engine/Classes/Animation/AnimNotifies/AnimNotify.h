// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimNotify.generated.h"

class UAnimSequenceBase;
class USkeletalMeshComponent;
struct FAnimNotifyEvent;
#if WITH_EDITORONLY_DATA
class FPrimitiveDrawInterface;
class FCanvas;
class FSceneView;
#endif

USTRUCT()
struct FBranchingPointNotifyPayload
{
public:
	GENERATED_USTRUCT_BODY()

	USkeletalMeshComponent* SkelMeshComponent;
	UAnimSequenceBase* SequenceAsset;
	FAnimNotifyEvent* NotifyEvent;
	int32 MontageInstanceID;
	bool bReachedEnd = false;

	FBranchingPointNotifyPayload()
		: SkelMeshComponent(nullptr)
		, SequenceAsset(nullptr)
		, NotifyEvent(nullptr)
		, MontageInstanceID(INDEX_NONE)
	{}

	FBranchingPointNotifyPayload(USkeletalMeshComponent* InSkelMeshComponent, UAnimSequenceBase* InSequenceAsset, FAnimNotifyEvent* InNotifyEvent, int32 InMontageInstanceID, bool bInReachedEnd = false)
		: SkelMeshComponent(InSkelMeshComponent)
		, SequenceAsset(InSequenceAsset)
		, NotifyEvent(InNotifyEvent)
		, MontageInstanceID(InMontageInstanceID)
		, bReachedEnd(bInReachedEnd)
	{}
};

UCLASS(abstract, editinlinenew, Blueprintable, const, hidecategories=Object, collapsecategories, MinimalAPI)
class UAnimNotify : public UObject
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Implementable event to get a custom name for the notify
	 */
	UFUNCTION(BlueprintNativeEvent)
	ENGINE_API FString GetNotifyName() const;

	UFUNCTION(BlueprintImplementableEvent, meta=(AutoCreateRefTerm="EventReference"))
	ENGINE_API bool Received_Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) const;

#if WITH_EDITORONLY_DATA
	/** Color of Notify in editor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnimNotify)
	FColor NotifyColor;

	/** Whether this notify instance should fire in animation editors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=AnimNotify)
	bool bShouldFireInEditor;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual void OnAnimNotifyCreatedInEditor(FAnimNotifyEvent& ContainingAnimNotifyEvent) {};
	virtual bool CanBePlaced(UAnimSequenceBase* Animation) const { return true; }
	virtual void ValidateAssociatedAssets() {}
	virtual void DrawInEditor(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const {}
	virtual void DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const {}

	/** Override this to prevent firing this notify type in animation editors */
	virtual bool ShouldFireInEditor() { return bShouldFireInEditor; }
#endif

	ENGINE_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference);
	ENGINE_API virtual void BranchingPointNotify(FBranchingPointNotifyPayload& BranchingPointPayload);

	// @todo document 
	virtual FString GetEditorComment() 
	{ 
		return TEXT(""); 
	}

	/** TriggerWeightThreshold to use when creating notifies of this type */
	UFUNCTION(BlueprintNativeEvent)
	ENGINE_API float GetDefaultTriggerWeightThreshold() const;

	// @todo document 
	virtual FLinearColor GetEditorColor() 
	{ 
#if WITH_EDITORONLY_DATA
		return FLinearColor(NotifyColor); 
#else
		return FLinearColor::Black;
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 * We don't instance UAnimNotify objects along with the animations they belong to, but
	 * we still need a way to see which world this UAnimNotify is currently operating on.
	 * So this retrieves a contextual world pointer, from the triggering animation/mesh.  
	 * 
	 * @return NULL if this isn't in the middle of a Received_Notify(), otherwise it's the world belonging to the Mesh passed to Received_Notify()
	 */
	ENGINE_API virtual class UWorld* GetWorld() const override;

	/** UObject Interface */
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	/** End UObject Interface */

	/** This notify is always a branching point when used on Montages. */
	bool bIsNativeBranchingPoint;

protected:
	ENGINE_API UObject* GetContainingAsset() const;

private:
	/* The mesh we're currently triggering a UAnimNotify for (so we can retrieve per instance information) */
	class USkeletalMeshComponent* MeshContext;
};
