// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimNotifyState.generated.h"

class UAnimSequenceBase;
class USkeletalMeshComponent;
struct FAnimNotifyEvent;
struct FBranchingPointNotifyPayload;
#if WITH_EDITORONLY_DATA
class FPrimitiveDrawInterface;
class FCanvas;
class FSceneView;
#endif

/** Flags that control how notify states behave */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EAnimNotifyStateBehaviorFlags : uint8
{
	None					= 0 UMETA(Hidden),
	// If set, each concurrently playing instance of the same animation maintains an independent notify state lifecycle rather than merging them.
	NoMergeOnConcurrentPlay	= 1 << 0,
};
ENUM_CLASS_FLAGS(EAnimNotifyStateBehaviorFlags);

UCLASS(abstract, editinlinenew, Blueprintable, const, hidecategories=Object, collapsecategories, meta=(ShowWorldContextPin), MinimalAPI)
class UAnimNotifyState : public UObject
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Implementable event to get a custom name for the notify
	 */
	UFUNCTION(BlueprintNativeEvent)
	ENGINE_API FString GetNotifyName() const;

	UFUNCTION(BlueprintImplementableEvent, meta=(AutoCreateRefTerm="EventReference"))
	ENGINE_API bool Received_NotifyBegin(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) const;
	
	UFUNCTION(BlueprintImplementableEvent, meta=(AutoCreateRefTerm="EventReference"))
	ENGINE_API bool Received_NotifyTick(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) const;

	UFUNCTION(BlueprintImplementableEvent, meta=(AutoCreateRefTerm="EventReference"))
	ENGINE_API bool Received_NotifyEnd(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference) const;

#if WITH_EDITORONLY_DATA
	/** Color of Notify in editor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnimNotify)
	FColor NotifyColor;
	
	/** Whether this notify state instance should fire in animation editors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=AnimNotify)
	bool bShouldFireInEditor;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual void OnAnimNotifyCreatedInEditor(FAnimNotifyEvent& ContainingAnimNotifyEvent) {};
	virtual bool CanBePlaced(UAnimSequenceBase* Animation) const { return true; }
	virtual void ValidateAssociatedAssets() {}
	virtual void DrawInEditor(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const {}
	virtual void DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const {}

	/** Override this to prevent firing this notify state type in animation editors */
	virtual bool ShouldFireInEditor() { return bShouldFireInEditor; }
#endif

	ENGINE_API virtual void NotifyBegin(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference);
	ENGINE_API virtual void NotifyTick(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference);
	ENGINE_API virtual void NotifyEnd(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference);
	
	ENGINE_API virtual void BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload);
	ENGINE_API virtual void BranchingPointNotifyTick(FBranchingPointNotifyPayload& BranchingPointPayload, float FrameDeltaTime);
	ENGINE_API virtual void BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload);

	// @todo document 
	virtual FString GetEditorComment() 
	{ 
		return TEXT(""); 
	}

	/** TriggerWeightThreshold to use when creating notifies of this type */
	UFUNCTION(BlueprintNativeEvent)
	ENGINE_API float GetDefaultTriggerWeightThreshold() const;

	// @todo document 
	virtual FLinearColor GetEditorColor() 
	{ 
#if WITH_EDITORONLY_DATA
		return FLinearColor(NotifyColor); 
#else
		return FLinearColor::Black;
#endif // WITH_EDITORONLY_DATA
	}

	/** UObject Interface */
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	/** End UObject Interface */

	/** This notify is always a branching point when used on Montages. */
	bool bIsNativeBranchingPoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = AnimNotify, meta=(Bitmask, BitmaskEnum="/Script/Engine.EAnimNotifyStateBehaviorFlags"))
	uint8 NotifyStateBehaviorFlags;
	
protected:
	ENGINE_API UObject* GetContainingAsset() const;
};
