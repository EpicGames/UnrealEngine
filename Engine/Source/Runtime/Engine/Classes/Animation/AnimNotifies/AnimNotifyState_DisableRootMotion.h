// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_DisableRootMotion.generated.h"

UCLASS(editinlinenew, const, hidecategories = Object, collapsecategories, MinimalAPI, meta = (DisplayName = "Disable Root Motion"))
class UAnimNotifyState_DisableRootMotion : public UAnimNotifyState
{
	GENERATED_UCLASS_BODY()

public:
	virtual void BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload) override;
	virtual void BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload) override;

#if WITH_EDITOR
	virtual bool CanBePlaced(UAnimSequenceBase* Animation) const override;
#endif

protected:

};
