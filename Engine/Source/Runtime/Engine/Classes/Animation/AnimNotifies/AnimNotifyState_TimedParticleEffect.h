// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_TimedParticleEffect.generated.h"

class UParticleSystem;
class USkeletalMeshComponent;

// Timed Particle Effect Notify
// Allows a looping particle effect to be played in an animation that will activate
// at the beginning of the notify and deactivate at the end.
UCLASS(Blueprintable, meta = (DisplayName = "Timed Particle Effect"), MinimalAPI)
class UAnimNotifyState_TimedParticleEffect : public UAnimNotifyState
{
	GENERATED_UCLASS_BODY()

	// The particle system template to use when spawning the particle component
	UPROPERTY(EditAnywhere, Category = ParticleSystem, meta = (ToolTip = "The particle system to spawn for the notify state"))
	TObjectPtr<UParticleSystem> PSTemplate;

	// The socket within our mesh component to attach to when we spawn the particle component
	UPROPERTY(EditAnywhere, Category = ParticleSystem, meta = (ToolTip = "The socket or bone to attach the system to"))
	FName SocketName;

	// Offset from the socket / bone location
	UPROPERTY(EditAnywhere, Category = ParticleSystem, meta = (ToolTip = "Offset from the socket or bone to place the particle system"))
	FVector LocationOffset;

	// Offset from the socket / bone rotation
	UPROPERTY(EditAnywhere, Category = ParticleSystem, meta = (ToolTip = "Rotation offset from the socket or bone for the particle system"))
	FRotator RotationOffset;

	// Whether or not we destroy the component at the end of the notify or instead just stop
	// the emitters.
	UPROPERTY(EditAnywhere, Category = ParticleSystem, meta = (DisplayName = "Destroy Immediately", ToolTip = "Whether the particle system should be immediately destroyed at the end of the notify state or be allowed to finish"))
	bool bDestroyAtEnd;

#if WITH_EDITORONLY_DATA
	// The following arrays are used to handle property changes during a state. Because we can't
	// store any stateful data here we can't know which emitter is ours. The best metric we have
	// is an emitter on our Mesh Component with the same template and socket name we have defined.
	// Because these can change at any time we need to track previous versions when we are in an
	// editor build. Refactor when stateful data is possible, tracking our component instead.
	UPROPERTY(transient)
	TArray<TObjectPtr<UParticleSystem>> PreviousPSTemplates;

	UPROPERTY(transient)
	TArray<FName> PreviousSocketNames;
	
#endif

#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
#endif
	
	ENGINE_API virtual void NotifyBegin(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) override;
	ENGINE_API virtual void NotifyTick(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) override;
	ENGINE_API virtual void NotifyEnd(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference) override;

	// Overridden from UAnimNotifyState to provide custom notify name.
	ENGINE_API FString GetNotifyName_Implementation() const override;

protected:
	ENGINE_API bool ValidateParameters(USkeletalMeshComponent* MeshComp);
};
