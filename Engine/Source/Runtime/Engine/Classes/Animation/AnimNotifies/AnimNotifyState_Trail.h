// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_Trail.generated.h"

class UParticleSystem;
class UParticleSystemComponent;
class USkeletalMeshComponent;
class UAnimSequenceBase;

DECLARE_LOG_CATEGORY_EXTERN(LogAnimTrails, Log, All);

UCLASS(editinlinenew, Blueprintable, const, hidecategories = Object, collapsecategories, meta = (ShowWorldContextPin, DisplayName = "Trail"), MinimalAPI)
class UAnimNotifyState_Trail : public UAnimNotifyState
{
	GENERATED_UCLASS_BODY()
	
	/** The particle system to use for this trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	TObjectPtr<UParticleSystem> PSTemplate;

	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API UParticleSystem* OverridePSTemplate(USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation) const;

	ENGINE_API virtual UParticleSystem* GetOverridenPSTemplate(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation) const;

	ENGINE_API virtual float GetCurveWidth(USkeletalMeshComponent* MeshComp) const;

	/** Name of the first socket defining this trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	FName FirstSocketName;

	/** Name of the second socket defining this trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	FName SecondSocketName;
	
	/** 
	Controls the way width scale is applied. In each method a width scale of 1.0 will mean the width is unchanged from the position of the sockets. A width scale of 0.0 will cause a trail of zero width.
	From Centre = Trail width is scaled outwards from the centre point between the two sockets.
	From First = Trail width is scaled outwards from the position of the first socket.
	From Second = Trail width is scaled outwards from the position of the Second socket.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	TEnumAsByte<enum ETrailWidthMode> WidthScaleMode;

	/** Name of the curve to drive the width scale. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	FName WidthScaleCurve;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Trail)
	uint32 bRecycleSpawnedSystems:1;

#if WITH_EDITORONLY_DATA
	/** If true, render the trail geometry (this should typically be on) */
	UPROPERTY(transient, EditAnywhere, Category = Rendering)
	uint32 bRenderGeometry : 1;

	/** If true, render stars at each spawned particle point along the trail */
	UPROPERTY(transient, EditAnywhere, Category = Rendering)
	uint32 bRenderSpawnPoints : 1;

	/** If true, render a line showing the tangent at each spawned particle point along the trail */
	UPROPERTY(transient, EditAnywhere, Category = Rendering)
	uint32 bRenderTangents : 1;

	/** If true, render the tessellated path between spawned particles */
	UPROPERTY(transient, EditAnywhere, Category = Rendering)
	uint32 bRenderTessellation : 1;
#endif // WITH_EDITORONLY_DATA

	/** Helper function for outside code to get PSC that we are using */
	ENGINE_API UParticleSystemComponent* GetParticleSystemComponent(USkeletalMeshComponent* MeshComp) const;

	ENGINE_API virtual void NotifyBegin(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) override;
	ENGINE_API virtual void NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) override;
	ENGINE_API virtual void NotifyEnd(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;

	ENGINE_API bool ValidateInput(USkeletalMeshComponent* MeshComp, bool bReportErrors = false);
};
