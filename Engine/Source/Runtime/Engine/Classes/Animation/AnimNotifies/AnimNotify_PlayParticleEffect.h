// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "AnimNotify_PlayParticleEffect.generated.h"

class UAnimSequenceBase;
class UParticleSystem;
class USkeletalMeshComponent;
class UParticleSystemComponent;

UCLASS(const, hidecategories=Object, collapsecategories, meta=(DisplayName="Play Particle Effect"), MinimalAPI)
class UAnimNotify_PlayParticleEffect : public UAnimNotify
{
	GENERATED_BODY()

public:

	ENGINE_API UAnimNotify_PlayParticleEffect();

	// Begin UObject interface
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End UObject interface

	// Begin UAnimNotify interface
	ENGINE_API virtual FString GetNotifyName_Implementation() const override;

	ENGINE_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
#if WITH_EDITOR
	ENGINE_API virtual void ValidateAssociatedAssets() override;
#endif
	// End UAnimNotify interface

	// Particle System to Spawn
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify", meta=(DisplayName="Particle System"))
	TObjectPtr<UParticleSystem> PSTemplate;

	// Location offset from the socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify")
	FVector LocationOffset;

	// Rotation offset from socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify")
	FRotator RotationOffset;

	// Scale to spawn the particle system at
	UPROPERTY(EditAnywhere, Category="AnimNotify")
	FVector Scale;

private:
	// Cached version of the Rotation Offset already in Quat form
	FQuat RotationOffsetQuat;

protected:
	// Spawns the ParticleSystemComponent. Called from Notify.
	ENGINE_API virtual UParticleSystemComponent* SpawnParticleSystem(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation);

public:

	// Should attach to the bone/socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify")
	uint32 Attached:1; 	//~ Does not follow coding standard due to redirection from BP

	// SocketName to attach to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify")
	FName SocketName;
};
