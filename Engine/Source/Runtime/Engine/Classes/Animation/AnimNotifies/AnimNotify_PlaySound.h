// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "AnimNotify_PlaySound.generated.h"

class UAnimSequenceBase;
class USkeletalMeshComponent;
class USoundBase;

UCLASS(const, hidecategories=Object, collapsecategories, Config = Game, meta=(DisplayName="Play Sound"), MinimalAPI)
class UAnimNotify_PlaySound : public UAnimNotify
{
	GENERATED_BODY()

public:

	ENGINE_API UAnimNotify_PlaySound();

	// Begin UAnimNotify interface
	ENGINE_API virtual FString GetNotifyName_Implementation() const override;
	ENGINE_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
#if WITH_EDITOR
	ENGINE_API virtual void ValidateAssociatedAssets() override;
	// End UAnimNotify interface
	
	//~ Begin UObject interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

	// Sound to Play
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify", meta=(ExposeOnSpawn = true))
	TObjectPtr<USoundBase> Sound;

	// Volume Multiplier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify", meta=(ExposeOnSpawn = true))
	float VolumeMultiplier;

	// Pitch Multiplier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify", meta=(ExposeOnSpawn = true))
	float PitchMultiplier;

	// If this sound should follow its owner. Automatically true if Attach Name is set. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify", meta=(EditCondition="bFollowEditable", HideEditConditionToggle, DisplayPriority=1))
	uint32 bFollow:1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	uint32 bFollowEditable:1;
	
	UPROPERTY(Config, EditAnywhere, Category = "AnimNotify")
	uint32 bPreviewIgnoreAttenuation:1;
#endif

	// Socket or bone name to attach sound to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnimNotify", meta=(ExposeOnSpawn = true, DisplayPriority=0))
	FName AttachName;
};
