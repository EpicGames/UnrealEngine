// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNotify.h"

#include "AnimNotify_ResetDynamics.generated.h"

UCLASS(const, hidecategories=Object, collapsecategories, meta=(DisplayName="Reset Dynamics"), MinimalAPI)
class UAnimNotify_ResetDynamics : public UAnimNotify
{
	GENERATED_BODY()

public:

	ENGINE_API UAnimNotify_ResetDynamics();

	// Begin UAnimNotify interface
	ENGINE_API virtual FString GetNotifyName_Implementation() const override;
	ENGINE_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
	// End UAnimNotify interface

};
