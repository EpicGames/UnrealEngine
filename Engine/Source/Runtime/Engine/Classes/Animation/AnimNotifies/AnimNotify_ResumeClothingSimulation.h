// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNotify.h"

#include "AnimNotify_ResumeClothingSimulation.generated.h"

UCLASS(const, hidecategories=Object, collapsecategories, meta=(DisplayName="Resume Clothing Simulation"), MinimalAPI)
class UAnimNotify_ResumeClothingSimulation : public UAnimNotify
{
	GENERATED_BODY()

public:

	ENGINE_API UAnimNotify_ResumeClothingSimulation();

	// Begin UAnimNotify interface
	ENGINE_API virtual FString GetNotifyName_Implementation() const override;
	ENGINE_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
	// End UAnimNotify interface

};
