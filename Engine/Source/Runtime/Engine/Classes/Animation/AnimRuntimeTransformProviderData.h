// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/TransformProviderData.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "AnimTrackPool.h"
#include "SkinningDefinitions.h"
#include "SkeletalRenderPublic.h"
#include "Containers/IntrusiveDoubleLinkedList.h"
#include "AnimRuntimeTransformProviderData.generated.h"

class UAnimRuntimeTransformProviderData;
class FAnimRuntimeTransformProviderProxy;
class IAnimRuntimeTransformProvider;

class FAnimRuntimeTrackTransformData final : public TIntrusiveDoubleLinkedListNode<FAnimRuntimeTrackTransformData>
{
public:
	~FAnimRuntimeTrackTransformData()
	{
		check(!IsInList());
	}

	FAnimRuntimeTrackTransformData(int32 NumTransforms)
	{
		Transforms.SetNumUninitialized(NumTransforms);
	}

	TArrayView<FCompressedBoneTransform> GetTransforms()
	{
		return Transforms;
	}

	TConstArrayView<FCompressedBoneTransform> GetTransforms() const
	{
		return Transforms;
	}

	int32 Num() const
	{
		return Transforms.Num();
	}

	const FCompressedBoneTransform& operator[] (int32 Index) const
	{
		return Transforms[Index];
	}

	FCompressedBoneTransform& operator[] (int32 Index)
	{
		return Transforms[Index];
	}

	void SetRevisionNumber(uint32 InRevisionNumber)
	{
		RevisionNumber = InRevisionNumber;
	}

	uint32 GetRevisionNumber() const
	{
		return RevisionNumber;
	}

private:
	TArray<FCompressedBoneTransform> Transforms;
	uint32 RevisionNumber = 0;

	friend UAnimRuntimeTransformProviderData;
};

struct FAnimRuntimeTrackData
{
	FAnimRuntimeTrackTransformData* Current = nullptr;
	FAnimRuntimeTrackTransformData* Previous = nullptr;

	void SetData(const FAnimRuntimeTrackData& RHS)
	{
		*this = RHS;
	}
};

inline FArchive& operator <<(FArchive& Ar, FAnimRuntimeTrackData& Data)
{
	// Serialization is a no-op. Defaults to nullptr.
	return Ar;
}

using FAnimRuntimeTrackPool = TAnimTrackPool<FAnimRuntimeTrackData>;

class FAnimRuntimeTransformProviderRenderData
{
public:
	ENGINE_API ~FAnimRuntimeTransformProviderRenderData();

private:
	void Patch(FAnimRuntimeTrackPool::FPatch&& Patch);
	void InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy);
	void RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy);

	FAnimRuntimeTrackPool Tracks;
	Experimental::TRobinHoodHashSet<FInstancedSkinningSceneExtensionProxy*> Proxies;

	UE::FMutex FreeListMutex;
	TIntrusiveDoubleLinkedList<FAnimRuntimeTrackTransformData> FreeList;

	friend FAnimRuntimeTransformProviderProxy;
	friend UAnimRuntimeTransformProviderData;
};

struct FAnimRuntimeTrackAllocation
{
	bool IsValid() const
	{
		return Index != INDEX_NONE && Current != nullptr;
	}

	int32 Index = INDEX_NONE;
	FAnimRuntimeTrackTransformData* Current = nullptr;
	FAnimRuntimeTrackTransformData* Previous = nullptr;
};

class FAnimRuntimeTransformProviderProxy : public FTransformProviderRenderProxy
{
	FAnimRuntimeTransformProviderProxy(FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy, FAnimRuntimeTransformProviderRenderData& InRenderData)
		: SceneExtensionProxy(InSceneExtensionProxy)
		, RenderData(InRenderData)
		, Tracks(InRenderData.Tracks)
	{}

public:
	virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;

	FAnimRuntimeTrackPool& GetTracks()
	{
		return Tracks;
	}

	const FInstancedSkinningSceneExtensionProxy* GetSceneExtensionProxy() const
	{
		return SceneExtensionProxy;
	}

private:
	FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy = nullptr;
	FAnimRuntimeTransformProviderRenderData& RenderData;
	FAnimRuntimeTrackPool& Tracks;
	IAnimRuntimeTransformProvider* AnimRuntimeTransformProvider = nullptr;

	friend class UAnimRuntimeTransformProviderData;
};

class IAnimRuntimeTransformProvider
{
public:
	virtual ~IAnimRuntimeTransformProvider() = default;

	virtual void RegisterProxy(FAnimRuntimeTransformProviderProxy* Proxy) {}
	virtual void UnregisterProxy(FAnimRuntimeTransformProviderProxy* Proxy) {}
};

UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType, Within=InstancedSkinnedMeshComponent)
class UAnimRuntimeTransformProviderData : public UTransformProviderData
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const override;
	virtual const FGuid& GetTransformProviderID() const override;
	virtual uint32 GetUniqueAnimationCount() const override;
	virtual bool UsesSkeletonBatching() const override;
	virtual bool HasAnimationBounds() const override;
	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const override;
	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const override;
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Ar) override;

	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const override;

public:
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimBank")
	static ENGINE_API UAnimRuntimeTransformProviderData* CreateAnimRuntimeTransformProviderData(UInstancedSkinnedMeshComponent* Owner);

	// Allocates a new track and returns a set of current and (optionally) previous transforms. The data should be filled prior to calling SubmitChanges.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimRuntime")
	ENGINE_API int32 AllocateTrack();

	// Allocates a new track and returns a set of current and (optionally) previous transforms. The data should be filled prior to calling SubmitChanges.
	ENGINE_API FAnimRuntimeTrackAllocation AllocateTrack(EPreviousBoneTransformUpdateMode UpdateMode);
	
	// Updates an existing track and returns a set of current and (optionally) previous transforms. The data should be filled prior to calling SubmitChanges.
	ENGINE_API FAnimRuntimeTrackAllocation UpdateTrack(int32 TrackIndex, EPreviousBoneTransformUpdateMode UpdateMode);

	// Deallocates a track. Returns whether the deallocation succeeded.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimRuntime")
	ENGINE_API bool DeallocateTrack(int32 TrackIndex);

	// Submits all pending changes to rendering.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimRuntime")
	ENGINE_API virtual void SubmitChanges() override;

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimRuntime")
	inline bool IsValidIndex(int32 TrackIndex) const
	{
		return Tracks.IsValidIndex(TrackIndex);
	}

private:
	FAnimRuntimeTrackData AcquireTrackData(EPreviousBoneTransformUpdateMode UpdateMode);

	int32 NumTransforms = 0;
	int32 NumPoolInstances = 0;
	FAnimRuntimeTrackPool Tracks;
	TUniquePtr<FAnimRuntimeTransformProviderRenderData> RenderData{ new FAnimRuntimeTransformProviderRenderData };
	TIntrusiveDoubleLinkedList<FAnimRuntimeTrackTransformData> FreeList;
	UE::FMutex SubmitChangesMutex;
};