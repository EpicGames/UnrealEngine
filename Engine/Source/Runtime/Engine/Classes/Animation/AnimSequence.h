// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * One animation sequence of keyframes. Contains a number of tracks of data.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimCompressionTypes.h"
#include "CustomAttributes.h"
#include "Containers/ArrayView.h"
#include "Animation/CustomAttributes.h"
#include "Animation/AnimData/AnimDataNotifications.h"
#include "Animation/AttributeCurve.h"
#include "Async/SharedRecursiveMutex.h"
#include "UObject/PerPlatformProperties.h"
#include "IO/IoHash.h"

#if WITH_EDITOR
#include "AnimData/IAnimationDataModel.h"
#include "Async/Mutex.h"
#endif // WITH_EDITOR

#include "AnimSequence.generated.h"

typedef TArray<FTransform> FTransformArrayA2;

class USkeletalMesh;
class FQueuedThreadPool;
class ITargetPlatform;
enum class EQueuedWorkPriority : uint8;
struct FAnimSequenceDecompressionContext;
struct FCompactPose;

namespace UE { namespace Anim { class FAnimSequenceCompilingManager; namespace Compression { struct FScopedCompressionGuard; } class FAnimationSequenceAsyncCacheTask; } }
namespace UE::UAF { class FDecompressionTools; }

extern ENGINE_API int32 GPerformFrameStripping;

// These two always should go together, but it is not right now. 
// I wonder in the future, we change all compressed to be inside as well, so they all stay together
// When remove tracks, it should be handled together 
USTRUCT()
struct FAnimSequenceTrackContainer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<struct FRawAnimSequenceTrack> AnimationTracks;

	UPROPERTY()
	TArray<FName>						TrackNames;

	// @todo expand this struct to work better and assign data better
	void Initialize(int32 NumNode)
	{
		AnimationTracks.Empty(NumNode);
		AnimationTracks.AddZeroed(NumNode);
		TrackNames.Empty(NumNode);
		TrackNames.AddZeroed(NumNode);
	}

	void Initialize(TArray<FName> InTrackNames)
	{
		TrackNames = MoveTemp(InTrackNames);
		const int32 NumNode = TrackNames.Num();
		AnimationTracks.Empty(NumNode);
		AnimationTracks.AddZeroed(NumNode);
	}

	int32 GetNum() const
	{
		check (TrackNames.Num() == AnimationTracks.Num());
		return (AnimationTracks.Num());
	}
};

/**
 * Keyframe position data for one track.  Pos(i) occurs at Time(i).  Pos.Num() always equals Time.Num().
 */
USTRUCT()
struct FTranslationTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FVector3f> PosKeys;

	UPROPERTY()
	TArray<float> Times;
};

/**
 * Keyframe rotation data for one track.  Rot(i) occurs at Time(i).  Rot.Num() always equals Time.Num().
 */
USTRUCT()
struct FRotationTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FQuat4f> RotKeys;

	UPROPERTY()
	TArray<float> Times;
};

/**
 * Keyframe scale data for one track.  Scale(i) occurs at Time(i).  Rot.Num() always equals Time.Num().
 */
USTRUCT()
struct FScaleTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FVector3f> ScaleKeys;

	UPROPERTY()
	TArray<float> Times;
};


/**
 * Key frame curve data for one track
 * CurveName: Morph Target Name
 * CurveWeights: List of weights for each frame
 */
USTRUCT()
struct FCurveTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName CurveName;

	UPROPERTY()
	TArray<float> CurveWeights;

	/** Returns true if valid curve weight exists in the array*/
	ENGINE_API bool IsValidCurveTrack();
	
	/** This is very simple cut to 1 key method if all is same since I see so many redundant same value in every frame 
	 *  Eventually this can get more complicated 
	 *  Will return true if compressed to 1. Return false otherwise **/
	ENGINE_API bool CompressCurveWeights();
};

USTRUCT()
struct FCompressedTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<uint8> ByteStream;

	UPROPERTY()
	TArray<float> Times;

	UPROPERTY()
	float Mins[3];

	UPROPERTY()
	float Ranges[3];


	FCompressedTrack()
	{
		for (int32 ElementIndex = 0; ElementIndex < 3; ElementIndex++)
		{
			Mins[ElementIndex] = 0;
		}
		for (int32 ElementIndex = 0; ElementIndex < 3; ElementIndex++)
		{
			Ranges[ElementIndex] = 0;
		}
	}

};

/** Enum used to decide whether we should strip animation data on dedicated server */
UENUM()
enum class EStripAnimDataOnDedicatedServerSettings : uint8
{
	/** Strip track data on dedicated server if 'Strip Animation Data on Dedicated Server' option in Project Settings is true and EnableRootMotion is false */
	UseProjectSetting,
	/** Strip track data on dedicated server regardless of the value of 'Strip Animation Data on Dedicated Server' option in Project Settings as long as EnableRootMotion is false  */
	StripAnimDataOnDedicatedServer,
	/** Do not strip track data on dedicated server regardless of the value of 'Strip Animation Data on Dedicated Server' option in Project Settings  */
	DoNotStripAnimDataOnDedicatedServer
};

UCLASS(config=Engine, hidecategories=(UObject, Length), BlueprintType, MinimalAPI)
class UAnimSequence : public UAnimSequenceBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The DCC framerate of the imported file. UI information only, unit are Hz */
	UPROPERTY(AssetRegistrySearchable, meta = (DisplayName = "Import File Framerate"))
	float ImportFileFramerate;

	/** The resample framerate that was computed during import. UI information only, unit are Hz */
	UPROPERTY(AssetRegistrySearchable, meta = (DisplayName = "Import Resample Framerate"))
	int32 ImportResampleFramerate;

protected:
	/** Contains the number of keys expected within the individual animation tracks. */
	UE_DEPRECATED(5.0, "NumFrames is deprecated see UAnimDataModel::GetNumberOfFrames for the number of source data frames, or GetNumberOfSampledKeys for the target keys")
	UPROPERTY()
	int32 NumFrames;

	/** The number of keys expected within the individual (non-uniform) animation tracks. */
	UE_DEPRECATED(5.0, "NumberOfKeys is deprecated see UAnimDataModel::GetNumberOfKeys for the number of source data keys, or GetNumberOfSampledKeys for the target keys")
	UPROPERTY()
	int32 NumberOfKeys;

	/** The frame rate at which the source animation is sampled. */
	UE_DEPRECATED(5.0, "SamplingFrameRate is deprecated see UAnimDataModel::GetFrameRate for the source frame rate, or GetSamplingFrameRate for the target frame rate instead")
	UPROPERTY()
	FFrameRate SamplingFrameRate;

	UE_DEPRECATED(5.0, "RawAnimationData has been deprecated see FBoneAnimationTrack::InternalTrackData")
	TArray<struct FRawAnimSequenceTrack> RawAnimationData;

	// Update this if the contents of RawAnimationData changes
	UE_DEPRECATED(5.1, "RawDataGuid has been deprecated see GenerateGuidFromModel instead")
	UPROPERTY()
	FGuid RawDataGuid;

	/**
	 * This is name of RawAnimationData tracks for editoronly - if we lose skeleton, we'll need relink them
	 */
	UE_DEPRECATED(5.0, "Animation track names has been deprecated see FBoneAnimationTrack::Name")
	UPROPERTY(VisibleAnywhere, DuplicateTransient, Category="Animation")
	TArray<FName> AnimationTrackNames;

	/**
	 * Source RawAnimationData. Only can be overridden by when transform curves are added first time OR imported
	 */
	TArray<struct FRawAnimSequenceTrack> SourceRawAnimationData_DEPRECATED;
public:

	/**
	 * Allow frame stripping to be performed on this animation if the platform requests it
	 * Can be disabled if animation has high frequency movements that are being lost.
	 */
	UPROPERTY(Category = Compression, EditAnywhere)
	bool bAllowFrameStripping;

	/**
	 * Set a scale for error threshold on compression. This is useful if the animation will 
	 * be played back at a different scale (e.g. if you know the animation will be played
	 * on an actor/component that is scaled up by a factor of 10, set this value to 10)
	 */
	UPROPERTY(Category = Compression, EditAnywhere)
	float CompressionErrorThresholdScale;
#endif

	/** The bone compression settings used to compress bones in this sequence. */
	UPROPERTY(Category = Compression, EditAnywhere, meta = (ForceShowEngineContent))
	TObjectPtr<class UAnimBoneCompressionSettings> BoneCompressionSettings;

	/** The curve compression settings used to compress curves in this sequence. */
	UPROPERTY(Category = Compression, EditAnywhere, meta = (ForceShowEngineContent))
	TObjectPtr<class UAnimCurveCompressionSettings> CurveCompressionSettings;

	// CompressedData is only valid in cook/non-editor runtime, see DataByPlatformKeyHash for editor-runtime data
	UE_DEPRECATED(5.6, "CompressedData public access will be removed")
	FCompressedAnimSequence CompressedData;

	UPROPERTY(Category = Compression, EditAnywhere, meta = (ForceShowEngineContent))
	TObjectPtr<class UVariableFrameStrippingSettings> VariableFrameStrippingSettings;

	/** Additive animation type. **/
	UPROPERTY(EditAnywhere, Category=AdditiveSettings, AssetRegistrySearchable)
	TEnumAsByte<enum EAdditiveAnimationType> AdditiveAnimType;

	/* Additive refrerence pose type. Refer above enum type */
	UPROPERTY(EditAnywhere, Category=AdditiveSettings, meta=(DisplayName = "Base Pose Type"))
	TEnumAsByte<enum EAdditiveBasePoseType> RefPoseType;

	/* Additve reference frame if RefPoseType == AnimFrame **/
	UPROPERTY(EditAnywhere, Category = AdditiveSettings)
	int32 RefFrameIndex;
	
	/* Additive reference animation if it's relevant - i.e. AnimScaled or AnimFrame **/
	UPROPERTY(EditAnywhere, Category=AdditiveSettings, meta=(DisplayName = "Base Pose Animation"))
	TObjectPtr<class UAnimSequence> RefPoseSeq;

	/** Base pose to use when retargeting */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category=Animation)
	FName RetargetSource;

#if WITH_EDITORONLY_DATA
	/** If RetargetSource is set to Default (None), this is asset for the base pose to use when retargeting. Transform data will be saved in RetargetSourceAssetReferencePose. */
	UE_DEPRECATED(5.5, "Direct access to RetargetSourceAsset has been deprecated. Please use members GetRetargetSourceAsset & SetRetargetSourceAsset instead.")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category=Animation, meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TSoftObjectPtr<USkeletalMesh> RetargetSourceAsset;
#endif

	/** When using RetargetSourceAsset, use the post stored here */
	UPROPERTY()
	TArray<FTransform> RetargetSourceAssetReferencePose;

	/** This defines how values between keys are calculated **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Animation)
	EAnimInterpolationType Interpolation;
	
	/** If this is on, it will allow extracting of root motion **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = RootMotion)
	bool bEnableRootMotion;

	/** Root Bone will be locked to that position when extracting root motion.**/
	UPROPERTY(EditAnywhere, Category = RootMotion)
	TEnumAsByte<ERootMotionRootLock::Type> RootMotionRootLock;
	
	/** Force Root Bone Lock even if Root Motion is not enabled */
	UPROPERTY(EditAnywhere, Category = RootMotion)
	bool bForceRootLock;

	/** If this is on, it will use a normalized scale value for the root motion extracted: FVector(1.0, 1.0, 1.0) **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = RootMotion)
	bool bUseNormalizedRootMotionScale;

	/** Have we copied root motion settings from an owning montage */
	UPROPERTY()
	bool bRootMotionSettingsCopiedFromMontage;

#if WITH_EDITORONLY_DATA
	/** Saved version number with CompressAnimations commandlet. To help with doing it in multiple passes. */
	UPROPERTY()
	int32 CompressCommandletVersion;

	/**
	 * Do not attempt to override compression scheme when running CompressAnimations commandlet.
	 * Some high frequency animations are too sensitive and shouldn't be changed.
	 */
	UPROPERTY(EditAnywhere, Category=Compression)
	uint32 bDoNotOverrideCompression:1;

	/** Importing data and options used for this mesh */
	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/***  for Reimport **/
	/** Path to the resource used to construct this skeletal mesh */
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	/** Date/Time-stamp of the file from the last import */
	UPROPERTY()
	FString SourceFileTimestamp_DEPRECATED;

	// Track whether we have updated markers so cached data can be updated
	int32 MarkerDataUpdateCounter;
#endif // WITH_EDITORONLY_DATA

	/** Enum used to decide whether we should strip animation data on dedicated server */
	UPROPERTY(EditAnywhere, Category = Compression)
	EStripAnimDataOnDedicatedServerSettings StripAnimDataOnDedicatedServer = EStripAnimDataOnDedicatedServerSettings::UseProjectSetting;

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void WillNeverCacheCookedPlatformDataAgain() override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	ENGINE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif // WITH_EDITOR
	static ENGINE_API void AddReferencedObjects(UObject* This, FReferenceCollector& Collector);
	//~ End UObject Interface

	//~ Begin UAnimationAsset Interface
	ENGINE_API virtual bool IsValidAdditive() const override;
	virtual TArray<FName>* GetUniqueMarkerNames() { return &UniqueMarkerNames; }
#if WITH_EDITOR
	ENGINE_API virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	ENGINE_API virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;
	ENGINE_API virtual void OnSetSkeleton(USkeleton* NewSkeleton) override;
#endif
	//~ End UAnimationAsset Interface

	//~ Begin UAnimSequenceBase Interface
	ENGINE_API virtual void HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const override;
	virtual bool HasRootMotion() const override { return bEnableRootMotion; }
	ENGINE_API virtual void RefreshCacheData() override;
	virtual EAdditiveAnimationType GetAdditiveAnimType() const override { return AdditiveAnimType; }
	ENGINE_API virtual int32 GetNumberOfSampledKeys() const override;
	virtual FFrameRate GetSamplingFrameRate() const override { return PlatformTargetFrameRate.Default; }
	ENGINE_API virtual void EvaluateCurveData(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const override;
	ENGINE_API virtual float EvaluateCurveData(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const override;
	// This will only check the current platform its compressed data (if valid)
	ENGINE_API virtual bool HasCurveData(FName CurveName, bool bForceUseRawData) const override;

	//~ End UAnimSequenceBase Interface

	// Extract Root Motion transform from the animation
	ENGINE_API virtual FTransform ExtractRootMotion(const FAnimExtractContext& ExtractionContext) const override final;
	// Extract Root Motion transform from a contiguous position range (no looping)
	ENGINE_API virtual FTransform ExtractRootMotionFromRange(double StartTime, double EndTime, const FAnimExtractContext& ExtractionContext) const override final;
	// Extract the transform from the root track for the given animation position
	ENGINE_API virtual FTransform ExtractRootTrackTransform(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const override final;

	// Begin Transform related functions 
	ENGINE_API virtual void GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const override;
	
	/**
	* Get Bone Transform of the Time given, relative to Parent for all RequiredBones
	* This returns different transform based on additive or not. Or what kind of additive.
	*
	* @param	OutAnimationPoseData  [out] Animation Pose related data to populate
	* @param	ExtractionContext	  Extraction Context (position, looping, root motion, etc.)
	* @param	bForceUseRawData	  Whether or not to forcefully try to sample the animation data model	
	*/
	ENGINE_API void GetBonePose(struct FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, bool bForceUseRawData = false) const;

#if WITH_EDITORONLY_DATA
protected:
	ENGINE_API void UpdateCompressedCurveName(const FName& OldCurveName, const FName& NewCurveName);
#endif

public:
#if WITH_EDITOR
	// Assigns the passed skeletal mesh to the retarget source
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetRetargetSourceAsset(USkeletalMesh* InRetargetSourceAsset);

	// Resets the retarget source asset
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void ClearRetargetSourceAsset();

	// Returns the retarget source asset soft object pointer.
	UFUNCTION(BlueprintPure, Category = "Animation")
	ENGINE_API const TSoftObjectPtr<USkeletalMesh>& GetRetargetSourceAsset() const;

	// Update the retarget data pose from the source, if it exist, else clears the retarget data pose saved in RetargetSourceAssetReferencePose.
	// Warning : This function calls LoadSynchronous at the retarget source asset soft object pointer, so it can not be used at PostLoad
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void UpdateRetargetSourceAssetData();
#endif

private:
#if WITH_EDITORONLY_DATA
	/** Updates the stored sampling frame-rate using the sequence length and number of sampling keys */
	ENGINE_API void UpdateFrameRate_DeprecatedLoad();
#endif

public:
	ENGINE_API const TArray<FTransform>& GetRetargetTransforms() const;
	ENGINE_API FName GetRetargetTransformsSourceName() const;

	/**
	* Retarget a single bone transform, to apply right after extraction.
	*
	* @param	BoneTransform		BoneTransform to read/write from.
	* @param	SkeletonBoneIndex	Bone Index in USkeleton.
	* @param	BoneIndex			Bone Index in Bone Transform array.
	* @param	RequiredBones		BoneContainer
	*/
	ENGINE_API void RetargetBoneTransform(FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive) const;

	/**
	* Get Bone Transform of the additive animation for the Time given, relative to Parent for all RequiredBones
	*
	* @param	OutAnimationPoseData	[out] Output pose data
	* @param	ExtractionContext		Extraction Context (position, looping, root motion, etc.)
	*/
	ENGINE_API void GetBonePose_Additive(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const;

	/**
	* Get Bone Transform of the base (reference) pose of the additive animation for the Time given, relative to Parent for all RequiredBones
	*
	* @param	OutAnimationPoseData	[out] Output pose data
	* @param	ExtractionContext	Extraction Context (position, looping, root motion, etc.)
	*/
	ENGINE_API void GetAdditiveBasePose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const;

	/**
	 * Get Bone Transform of the Time given, relative to Parent for the Track Given
	 *
	 * @param	OutAtom					[out] Output bone transform.
	 * @param	BoneIndex				Index of bone to evaluate.
	 * @param	ExtractionContext		Extraction context containing time and interpolation info.
	 * @param	bUseRawData		If true, use raw animation data instead of compressed data.
	 */
	ENGINE_API void GetBoneTransform(FTransform& OutAtom, FSkeletonPoseBoneIndex BoneIndex, const FAnimExtractContext& ExtractionContext, bool bUseRawData) const;
	// End Transform related functions

	// Begin Memory related functions

	/** @return	estimate uncompressed raw size. This is *not* the real raw size. 
				Here we estimate what it would be with no trivial compression. */
#if WITH_EDITOR
	ENGINE_API int64 GetUncompressedRawSize() const;

	/**
	 * @return		The approximate size of raw animation data.
	 */
	ENGINE_API int64 GetApproxRawSize() const;

	/**
	 * @return		The approximate size of raw bone animation data.
	 */
	ENGINE_API int64 GetApproxBoneRawSize() const;

	/**
	 * @return		The approximate size of raw curve animation data.
	 */
	ENGINE_API int64 GetApproxCurveRawSize() const;
#endif // WITH_EDITOR

	/**
	 * @return		The approximate size of compressed animation data for only bones.
	 */
	ENGINE_API int32 GetApproxBoneCompressedSize() const;
	
	/**
	 * @return		The approximate size of compressed animation data.
	 */
	ENGINE_API int32 GetApproxCompressedSize() const;
protected:
	/**
	 * @return		The approximate size of compressed animation data for only bones.
	 */
	ENGINE_API int32 GetApproxBoneCompressedSize_Lockless() const;
	
	/**
	 * @return		The approximate size of compressed animation data.
	 */
	ENGINE_API int32 GetApproxCompressedSize_Lockless() const;

	void EvaluateCurveData_Lockless(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const;	
	float EvaluateCurveData_Lockless(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const;	
	FTransform ExtractRootTrackTransform_Lockless(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const;

	ENGINE_API void GetBoneTransform_Lockless(FTransform& OutAtom, FSkeletonPoseBoneIndex BoneIndex, const FAnimExtractContext& ExtractionContext, bool bForceUseRawData) const;	

#if WITH_EDITOR
	ENGINE_API bool ShouldPerformStripping(const bool bPerformFrameStripping, const bool bPerformStrippingOnOddFramedAnims) const;
#endif // WITH_EDITOR


protected:
    // Write the compressed data to the supplied FArchive
    ENGINE_API void SerializeCompressedData(FArchive& Ar, bool bDDCData);
	ENGINE_API void SerializeCompressedData(FArchive& Ar, FCompressedAnimSequence& CompressedData);
#if WITH_EDITOR
	ENGINE_API virtual void OnAnimModelLoaded() override;
#endif
public:
	ENGINE_API bool IsCompressedDataValid() const;
	ENGINE_API bool IsBoneCompressedDataValid() const;	
	ENGINE_API bool IsCurveCompressedDataValid() const;
	// End Memory related functions
#if WITH_EDITOR
	/**
	 * Add Key to Transform Curves
	 */
	ENGINE_API void AddKeyToSequence(float Time, const FName& BoneName, const FTransform& AdditiveTransform);

	/**
	* Return true if compressed data is invalid or if it is not in sync with the skeleton
	*/
	ENGINE_API bool IsCompressedDataOutOfDate() const;

	/**
	 * Create Animation Sequence from Reference Pose of the Mesh
	 */
	ENGINE_API bool CreateAnimation(class USkeletalMesh* Mesh);
	/**
	 * Create Animation Sequence from the Mesh Component's current bone transform
	 */
	ENGINE_API bool CreateAnimation(class USkeletalMeshComponent* MeshComponent);
	/**
	 * Create Animation Sequence from the given animation
	 */
	ENGINE_API bool CreateAnimation(class UAnimSequence* Sequence);

	/** 
	 * Add validation check to see if it's being ready to play or not
	 */
	ENGINE_API virtual bool IsValidToPlay() const override;

	// Get a pointer to the data for a given Anim Notify
	ENGINE_API uint8* FindSyncMarkerPropertyData(int32 SyncMarkerIndex, FArrayProperty*& ArrayProperty);

	virtual int32 GetMarkerUpdateCounter() const { return MarkerDataUpdateCounter; }
#endif

	/** Sort the sync markers array by time, earliest first. */
	ENGINE_API void SortSyncMarkers();

#if WITH_EDITOR
	/** Remove all markers with the specified names */
	ENGINE_API bool RemoveSyncMarkers(const TArray<FName>& MarkersToRemove);

	/** Rename the markers with the specified name */
	ENGINE_API bool RenameSyncMarkers(FName InOldName, FName InNewName);
#endif

	// Advancing based on markers
	ENGINE_API float GetCurrentTimeFromMarkers(FMarkerPair& PrevMarker, FMarkerPair& NextMarker, float PositionBetweenMarkers) const;
	ENGINE_API virtual void AdvanceMarkerPhaseAsLeader(bool bLooping, float MoveDelta, const TArray<FName>& ValidMarkerNames, float& CurrentTime, FMarkerPair& PrevMarker, FMarkerPair& NextMarker, TArray<FPassedMarker>& MarkersPassed, const UMirrorDataTable* MirrorTable) const;
	ENGINE_API virtual void AdvanceMarkerPhaseAsFollower(const FMarkerTickContext& Context, float DeltaRemaining, bool bLooping, float& CurrentTime, FMarkerPair& PreviousMarker, FMarkerPair& NextMarker, const UMirrorDataTable* MirrorTable) const;
	ENGINE_API virtual void GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker) const;

	ENGINE_API virtual FMarkerSyncAnimPosition GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, const UMirrorDataTable* MirrorTable) const;
	ENGINE_API virtual void GetMarkerIndicesForPosition(const FMarkerSyncAnimPosition& SyncPosition, bool bLooping, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker, float& CurrentTime, const UMirrorDataTable* MirrorTable) const;
	
	ENGINE_API virtual float GetFirstMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition) const override;
	ENGINE_API virtual float GetNextMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const override;
	ENGINE_API virtual float GetPrevMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const override;

	// to support anim sequence base to all montages
	ENGINE_API virtual void EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock) override;
	ENGINE_API virtual bool GetEnableRootMotionSettingFromMontage() const override;

#if WITH_EDITOR
	virtual class UAnimSequence* GetAdditiveBasePose() const override 
	{ 
		if (IsValidAdditive())
		{
			return RefPoseSeq;
		}

		return nullptr;
	}

	// Is this animation valid for baking into additive
	ENGINE_API bool CanBakeAdditive() const;

	// Test whether at any point we will scale a bone to 0 (needed for validating additive anims)
	ENGINE_API bool DoesSequenceContainZeroScale() const;

	// Helper function to allow us to notify animations that depend on us that they need to update
	ENGINE_API void FlagDependentAnimationsAsRawDataOnly() const;

	// Helper function to allow us to update streaming animations that depend on us with our data when we are updated
	ENGINE_API void UpdateDependentStreamingAnimations() const;

	/** Resets Bone Animation, Curve data and Notify tracks **/
	ENGINE_API void ResetAnimation();
#endif

private:
	/**
	* Get Bone Transform of the animation for the Time given, relative to Parent for all RequiredBones
	* This return mesh rotation only additive pose
	*
	* @param	OutAnimationPoseData	[out] Output pose data
	* @param	ExtractionContext		Extraction Context (position, looping, root motion, etc.)
	*/
	ENGINE_API void GetBonePose_AdditiveMeshRotationOnly(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const;

protected:
	/** Returns whether or not evaluation of the raw (source) animation data is possible according to whether or not the (editor only) data has been stripped */
	ENGINE_API virtual bool CanEvaluateRawAnimationData() const;

private:
#if WITH_EDITOR
	/**
	 * Remap Tracks to New Skeleton
	 */ 
	ENGINE_API virtual void RemapTracksToNewSkeleton( USkeleton* NewSkeleton, bool bConvertSpaces ) override;

	/** Retargeting functions */
	ENGINE_API int32 GetSpaceBasedAnimationData(TArray< TArray<FTransform> > & AnimationDataInComponentSpace) const;
#endif

public:
	/** Refresh sync marker data*/
	ENGINE_API void RefreshSyncMarkerDataFromAuthored();

	/** Take a set of marker positions and validates them against a requested start position, updating them as desired */
	ENGINE_API void ValidateCurrentPosition(const FMarkerSyncAnimPosition& Position, bool bPlayingForwards, bool bLooping, float&CurrentTime, FMarkerPair& PreviousMarker, FMarkerPair& NextMarker, const UMirrorDataTable* MirrorTable = nullptr) const;

public:
	/** Authored Sync markers */
	UPROPERTY()
	TArray<FAnimSyncMarker>		AuthoredSyncMarkers;

	/** List of Unique marker names in this animation sequence */
	TArray<FName>				UniqueMarkerNames;

public:
	ENGINE_API void EvaluateAttributes(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, bool bUseRawData) const;	
protected:
#if WITH_EDITOR
	ENGINE_API void SynchronousAnimatedBoneAttributesCompression();
	ENGINE_API void MoveAttributesToModel();

	// Begin UAnimSequenceBase virtual overrides
	ENGINE_API virtual void OnModelModified(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload) override;
	ENGINE_API virtual void PopulateModel() override;
	// End UAnimSequenceBase virtual overrides

	ENGINE_API void EnsureValidRawDataGuid();
	ENGINE_API void CalculateNumberOfSampledKeys();
	ENGINE_API void DeleteBoneAnimationData();
	ENGINE_API void DeleteDeprecatedRawAnimationData();
public:
	ENGINE_API void DeleteNotifyTrackData();
	// Resets the bone and curve compression settings to the project default 
	ENGINE_API void ResetCompressionSettings();
#endif // WITH_EDITOR

protected:
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Animation")
	FPerPlatformFrameRate PlatformTargetFrameRate;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FFrameRate TargetFrameRate;	

	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Animation", Transient, DuplicateTransient)
	int32 NumberOfSampledKeys;

	UPROPERTY(VisibleAnywhere, Category = "Animation", Transient, DuplicateTransient)
	int32 NumberOfSampledFrames;

	bool bBlockCompressionRequests;
private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.0, "PerBoneCustomAttributeData has been deprecated see UAnimDataModel::AnimatedBoneAttributes")
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category=CustomAttributes)
	TArray<FCustomAttributePerBoneData> PerBoneCustomAttributeData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

protected:
	UPROPERTY()
	TMap<FAnimationAttributeIdentifier, FAttributeCurve> AttributeCurves;

#if WITH_EDITOR
	ENGINE_API FIoHash CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform) const;
	FString CreateDerivedDataKeyString(const ITargetPlatform* TargetPlatform) const;
	/** Archive-based key generation — writes key data directly to archive without allocating an intermediate FString. Opt-in via a.AnimSequence.UseBinaryDDCKey. */
	bool CreateDerivedDataKey(const ITargetPlatform* TargetPlatform, FArchive& Output) const;

	ENGINE_API FIoHash BeginCacheDerivedData(const ITargetPlatform* TargetPlatform);
	ENGINE_API bool PollCacheDerivedData(const FIoHash& KeyHash) const;
	ENGINE_API void EndCacheDerivedData(const FIoHash& KeyHash);

	mutable FCompressedAnimSequence* CurrentPlatformData = nullptr; 
	TMap<FIoHash, TUniquePtr<FCompressedAnimSequence>> DataByPlatformKeyHash;
	TMap<FIoHash, TPimplPtr<UE::Anim::FAnimationSequenceAsyncCacheTask>> CacheTasksByKeyHash;
	mutable UE::FSharedRecursiveMutex SharedCompressedDataMutex;
	mutable UE::FMutex CacheTasksMutex;

	// Cached FIoHash keys stored according to ITargetPlatform type-hash
	mutable TMap<uint32, FIoHash> PlatformHashToKeyHash;
	mutable FRWLock HashCacheLock;
	// Whether or not compressed data should be cleared in case of residency releasing (cook-time behaviour only)
	TAtomic<bool> bShouldClearCompressedData = false;
	
	// Bi-directional multi-maps keeping track of required (target-platform specific) compressed data, stored by hash and provided identifier from API usage
	TMultiMap<uint32, FIoHash> ResidencyReferencerHashes;
	TMultiMap<FIoHash, uint32> PlatformHashToReferencers;

	mutable FRWLock ResidencyLock;

	ENGINE_API bool TryCancelAsyncTasks();
	ENGINE_API bool WaitForAsyncTasks(float TimeLimitSeconds);
	ENGINE_API void FinishAsyncTasks();
	ENGINE_API void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority);
	ENGINE_API bool IsAsyncTaskComplete() const;
	ENGINE_API bool IsCompiling() const;
	ENGINE_API bool RequiresResidency(const FIoHash& InKeyHash) const;

	ENGINE_API void ValidateCompressionSettings();

	const FCompressedAnimSequence& GetPlatformCompressedData(const ITargetPlatform* InTargetPlatform) const;
#endif // WITH_EDITOR

	// Whether or not RawData should be sampled during pose extraction, either because compressed data is not available or through user directive
	ENGINE_API bool ShouldUseRawDataForPoseExtraction(const FBoneContainer& RequiredBones, const FAnimExtractContext& ExtractionContext) const;
	ENGINE_API bool ShouldUseRawDataForPoseExtraction_Lockless(const FBoneContainer& RequiredBones, const FAnimExtractContext& ExtractionContext) const;
	ENGINE_API bool ShouldUseRawDataForPoseExtraction_Lockless(const FAnimExtractContext& ExtractionContext) const;

	// Clears all currently cached compressed data (all platform data)
	ENGINE_API void ClearAllCompressionData();
	// Clears compressed data, if currently cached, for specified hash
	ENGINE_API void ClearCompressionData(const FIoHash& InKeyHash);
	// Whether or not compressed data, for specified hash, is currently cached
	ENGINE_API bool HasCompressedDataForHash(const FIoHash& InKeyHash) const;

private:	
	// Returns compressed animation data for specified Hash or TargetPlatform (not thread-safe see FCompressedAnimationDataReadScope and FCompressedAnimationResidencyScope)
	ENGINE_API const FCompressedAnimSequence& GetPlatformCompressedData(const FAnimExtractContext& AnimExtractContext) const;
	ENGINE_API FCompressedAnimSequence& GetPlatformCompressedData(const FAnimExtractContext& AnimExtractContext);
	ENGINE_API FCompressedAnimSequence& GetPlatformCompressedData(const ITargetPlatform* InTargetPlatform);
	
	// Returns compressed animation data for the current platform (not thread-safe in editor-builds see FCompressedAnimationDataReadScope and FCompressedAnimationResidencyScope)
	ENGINE_API const FCompressedAnimSequence& GetCompressedData_Internal() const;
	// Returns compressed animation data for the current platform (not thread-safe in editor-builds see FCompressedAnimationDataReadScope and FCompressedAnimationResidencyScope)
	ENGINE_API FCompressedAnimSequence& GetCompressedData_Internal();

public:	
	ENGINE_API void BeginCacheDerivedDataForCurrentPlatform();
	ENGINE_API void CacheDerivedDataForCurrentPlatform();
#if WITH_EDITOR
	ENGINE_API void WaitOnExistingCompression(const bool bWantResults=true);
	
	// Returns hash identifying compressed animation data with the specified target platform its settings (these can overlap between different platforms due to the settings matching)
	ENGINE_API FIoHash GetDerivedDataKeyHash(const ITargetPlatform* TargetPlatform) const;
	
	// Whether or not compressed data, for specified platform, is currently cached
	ENGINE_API bool HasCompressedDataForPlatform(const ITargetPlatform* InPlatform) const;	
	// Synchronous caching of compressed animation data for provided target platform
	ENGINE_API void CacheDerivedDataForPlatform(const ITargetPlatform* TargetPlatform);

	// Whether or not animation sequence can be compressed (dependent on loading state)
	ENGINE_API bool CanBeCompressed() const;
	
	// Whether or not compressed animation data has been requested to stay resident until released 
	ENGINE_API bool HasResidency(const ITargetPlatform* InPlatform) const;
	ENGINE_API bool HasResidency(uint32 InReferencerHash) const;
	// Fetches and keeps resident the compressed animation data for the provided target platform (until released)
	ENGINE_API void RequestResidency(const ITargetPlatform* InPlatform, uint32 InReferencerHash);
	// Releases previously requested residency of compressed animation data
	ENGINE_API void ReleaseResidency(const ITargetPlatform* InPlatform, uint32 InReferencerHash);
	
	ENGINE_API FFrameRate GetTargetSamplingFrameRate(const ITargetPlatform* InPlatform) const;
#endif

public:
	// Provides access to an instance of compressed animation data within the lifetime of FScopedCompressedAnimSequence, should _only_ be used on the stack due to risk of deadlocking
	struct FScopedCompressedAnimSequence
	{
		FScopedCompressedAnimSequence() = delete;
		FScopedCompressedAnimSequence(const FScopedCompressedAnimSequence&) = delete;
		FScopedCompressedAnimSequence& operator=(const FScopedCompressedAnimSequence&) = delete;		
		FScopedCompressedAnimSequence(FScopedCompressedAnimSequence&&) = delete;
		FScopedCompressedAnimSequence& operator=(FScopedCompressedAnimSequence&&) = delete;
		
		const FCompressedAnimSequence& operator->() const { return *CompressedData; }		
		const FCompressedAnimSequence& Get() const { return *CompressedData; }
#if WITH_EDITOR
		~FScopedCompressedAnimSequence()
		{
			SharedLock.Reset();
		}
#endif // WITH_EDITOR
		
	protected:
		FScopedCompressedAnimSequence(const UAnimSequence* InAnimSequence, const FCompressedAnimSequence* InCompressedData) :
			CompressedData(InCompressedData)
		{
			check(CompressedData && InAnimSequence);			
#if WITH_EDITOR
			SharedLock = MakeUnique<UE::TSharedLock<UE::FSharedRecursiveMutex>>(InAnimSequence->SharedCompressedDataMutex);
#endif // WITH_EDITOR
		}

		const FCompressedAnimSequence* CompressedData = nullptr;
#if WITH_EDITOR
		TUniquePtr<UE::TSharedLock<UE::FSharedRecursiveMutex>> SharedLock;
#endif // WITH_EDITOR
		friend class UAnimSequence;
	};
	
	// Returns scoped version of the compressed AnimSequence, only valid to read when FScopedConstCompressedAnimSequence is in scope (this internally locks CompressedDataLock)
	ENGINE_API FScopedCompressedAnimSequence GetCompressedData() const;	
	ENGINE_API FScopedCompressedAnimSequence GetCompressedData(const FAnimExtractContext& AnimExtractContext) const;
	
#if WITH_EDITOR	
	ENGINE_API FScopedCompressedAnimSequence GetCompressedData(const ITargetPlatform* InTargetPlatform) const;	
protected:
	/** Helper struct to enter/leave compressed data lock in read-mode */
	struct FCompressedAnimationDataReadScope
	{
		FCompressedAnimationDataReadScope(const UAnimSequence* InAnimSequence)
		{
			check(InAnimSequence);
			SharedLock = MakeUnique<UE::TSharedLock<UE::FSharedRecursiveMutex>>(InAnimSequence->SharedCompressedDataMutex);
		}
		TUniquePtr<UE::TSharedLock<UE::FSharedRecursiveMutex>> SharedLock;
	};
	
	/** Helper struct to enter/leave compressed data lock in write-mode */
	struct FCompressedAnimationDataWriteScope
	{
		FCompressedAnimationDataWriteScope(const UAnimSequence* InAnimSequence) : AnimSequence(InAnimSequence)
		{
			check(InAnimSequence);
			InAnimSequence->SharedCompressedDataMutex.Lock();
		}
		~FCompressedAnimationDataWriteScope()
		{
			AnimSequence->SharedCompressedDataMutex.Unlock();
		}

		const UAnimSequence* AnimSequence = nullptr;
	};
#endif // WITH_EDITOR

public:
	friend class UAnimationAsset;
	friend struct FScopedAnimSequenceRawDataCache;
	friend class UAnimationBlueprintLibrary;
	friend class UAnimBoneCompressionSettings;
	friend class FCustomAttributeCustomization;
	friend class FAnimSequenceDeterminismHelper;
	friend class FAnimSequenceTestBase;
	friend struct UE::Anim::Compression::FScopedCompressionGuard;
	friend class FAnimDataControllerTestBase;
	friend class UE::Anim::FAnimSequenceCompilingManager;
	friend UE::UAF::FDecompressionTools;
	friend class FAnimSequenceDetails;
	friend class FCompressedAnimationDataNodeBuilder;
	friend class UAnimBank;
	friend class FAnimBankBuildAsyncCacheTask;
	friend class UAnimSequenceTransformProviderData;
	friend class FAnimSequenceTransformProviderBuildAsyncCacheTask;
	friend struct FAnimCurveBufferAccess;
	friend class UE::Anim::FAnimationSequenceAsyncCacheTask;
};

#ifndef UE_COMPRESSED_DATA_WRITE_SCOPE
#if WITH_EDITOR
	#define UE_COMPRESSED_DATA_WRITE_SCOPE(sequence) \
	UAnimSequence::FCompressedAnimationDataWriteScope Scope(sequence);
#else
	#define UE_COMPRESSED_DATA_WRITE_SCOPE(sequence)
#endif
#endif

#ifndef UE_COMPRESSED_DATA_READ_SCOPE
#if WITH_EDITOR
	#define UE_COMPRESSED_DATA_READ_SCOPE(sequence) \
	UAnimSequence::FCompressedAnimationDataReadScope Scope(sequence);
#else
	#define UE_COMPRESSED_DATA_READ_SCOPE(sequence)
#endif
#endif
