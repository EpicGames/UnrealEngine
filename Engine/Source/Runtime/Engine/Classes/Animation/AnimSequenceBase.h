// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Abstract base class of animation sequence that can be played and evaluated to produce a pose.
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/AnimData/AnimDataModelNotifyCollector.h"
#include "Animation/AnimData/IAnimationDataController.h"

#include "AnimSequenceBase.generated.h"

UENUM()
enum ETypeAdvanceAnim : int
{
	ETAA_Default,
	ETAA_Finished,
	ETAA_Looped
};

struct FAnimationPoseData;
struct FAnimDataModelNotifPayload;
class UAnimDataModel;
class IAnimationDataModel;
enum class EAnimDataModelNotifyType : uint8;

UCLASS(abstract, BlueprintType, MinimalAPI)
class UAnimSequenceBase : public UAnimationAsset
{
	GENERATED_UCLASS_BODY()

public:
	/** Animation notifies, sorted by time (earliest notification first). */
	UPROPERTY()
	TArray<struct FAnimNotifyEvent> Notifies;

protected:
	/** Length (in seconds) of this AnimSequence if played back with a speed of 1.0. */
	UE_DEPRECATED(5.0, "Public access to SequenceLength is deprecated, use GetPlayLength or UAnimDataController::SetPlayLength instead")
	UPROPERTY(Category=Length, AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly)
	float SequenceLength;

	/**
	 * Raw uncompressed float curve data
	 */
	UE_DEPRECATED(5.0, "Public access to RawCurveData is deprecated, see UAnimDataModel for source data or use GetCurveData for runtime instead")
	UPROPERTY()
	struct FRawCurveTracks RawCurveData;
	
public:
	/** Number for tweaking playback rate of this animation globally. */
	UPROPERTY(EditAnywhere, Category=Animation)
	float RateScale;
	
	/** 
	 * The default looping behavior of this animation.
	 * Asset players can override this
	 */
	UPROPERTY(EditAnywhere, Category=Animation)
	bool bLoop;
#if WITH_EDITORONLY_DATA
	// if you change Notifies array, this will need to be rebuilt
	UPROPERTY()
	TArray<FAnimNotifyTrack> AnimNotifyTracks;
#endif // WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

	/** Returns the total play length of the montage, if played back with a speed of 1.0. */
	ENGINE_API virtual float GetPlayLength() const override;

	/** Sort the Notifies array by time, earliest first. */
	ENGINE_API void SortNotifies();	

	/** Remove the notifies specified */
	ENGINE_API bool RemoveNotifies(const TArray<FName>& NotifiesToRemove);
	
	/** Remove all notifies */
	ENGINE_API void RemoveNotifies();

#if WITH_EDITOR
	/** Renames all named notifies with InOldName to InNewName */
	ENGINE_API void RenameNotifies(FName InOldName, FName InNewName);
#endif
	
	/**
	* Retrieves AnimNotifies given a StartTime and a DeltaTime.
	* Time will be advanced and support looping if bAllowLooping is true.
	* Supports playing backwards (DeltaTime<0).
	* Returns notifies between StartTime (exclusive) and StartTime+DeltaTime (inclusive)
	*/
	ENGINE_API void GetAnimNotifies(const float& StartTime, const float& DeltaTime, FAnimNotifyContext& NotifyContext) const;

	/**
	* Retrieves AnimNotifies between two time positions. ]PreviousPosition, CurrentPosition]
	* Between PreviousPosition (exclusive) and CurrentPosition (inclusive).
	* Supports playing backwards (CurrentPosition<PreviousPosition).
	* Only supports contiguous range, does NOT support looping and wrapping over.
	*/
	ENGINE_API virtual void GetAnimNotifiesFromDeltaPositions(const float& PreviousPosition, const float & CurrentPosition, FAnimNotifyContext& NotifyContext) const;

	/** Evaluate curve data to Instance at the time of CurrentTime **/
	ENGINE_API virtual void EvaluateCurveData(FBlendedCurve& OutCurve, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const;

	ENGINE_API virtual float EvaluateCurveData(FName CurveName, const FAnimExtractContext& AnimExtractContext, bool bForceUseRawData = false) const;
	
	ENGINE_API virtual const FRawCurveTracks& GetCurveData() const;
		
	ENGINE_API virtual bool HasCurveData(FName CurveName, bool bForceUseRawData = false) const;

	/** Return the total number of keys sampled for this animation, including the T0 key **/
	ENGINE_API virtual int32 GetNumberOfSampledKeys() const;

	/** Return rate at which the animation is sampled **/
	ENGINE_API virtual FFrameRate GetSamplingFrameRate() const;

	/** Get the time at the given frame */
	ENGINE_API virtual float GetTimeAtFrame(const int32 Frame) const;

#if WITH_EDITOR
	/** Get the frame number for the provided time */
	ENGINE_API virtual int32 GetFrameAtTime(const float Time) const;

	// @todo document
	ENGINE_API void InitializeNotifyTrack();

	/** Fix up any notifies that are positioned beyond the end of the sequence */
	ENGINE_API void ClampNotifiesAtEndOfSequence();

	/** Calculates what (if any) offset should be applied to the trigger time of a notify given its display time */ 
	ENGINE_API virtual EAnimEventTriggerOffsets::Type CalculateOffsetForNotify(float NotifyDisplayTime) const;

	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	// Get a pointer to the data for a given Anim Notify
	ENGINE_API uint8* FindNotifyPropertyData(int32 NotifyIndex, FArrayProperty*& ArrayProperty);

	// Get a pointer to the data for a given array property item
	ENGINE_API uint8* FindArrayProperty(const TCHAR* PropName, FArrayProperty*& ArrayProperty, int32 ArrayIndex);

protected:
	ENGINE_API virtual void RefreshParentAssetData() override;
#endif	//WITH_EDITORONLY_DATA
public: 
	// update cache data (notify tracks, sync markers)
	ENGINE_API virtual void RefreshCacheData();

	//~ Begin UAnimationAsset Interface
	ENGINE_API virtual void TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const override;
	//~ End UAnimationAsset Interface
	
	ENGINE_API void TickByMarkerAsFollower(FMarkerTickRecord &Instance, FMarkerTickContext &MarkerContext, float& CurrentTime, float& OutPreviousTime, const float MoveDelta, const bool bLooping, const UMirrorDataTable* MirrorTable = nullptr) const;
	ENGINE_API void TickByMarkerAsLeader(FMarkerTickRecord& Instance, FMarkerTickContext& MarkerContext, float& CurrentTime, float& OutPreviousTime, const float MoveDelta, const bool bLooping, const UMirrorDataTable* MirrorTable = nullptr) const;

	/**
	* Get Bone Transform of the Time given, relative to Parent for all RequiredBones
	* This returns different transform based on additive or not. Or what kind of additive.
	*
	* @param	OutPose				Pose object to fill
	* @param	OutCurve			Curves to fill
	* @param	ExtractionContext	Extraction Context (position, looping, root motion, etc.)
	*/	
	virtual void GetAnimationPose(FAnimationPoseData& OutPoseData, const FAnimExtractContext& ExtractionContext) const
		PURE_VIRTUAL(UAnimSequenceBase::GetAnimationPose, );
	
	ENGINE_API virtual void HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const;

	virtual bool HasRootMotion() const { return false; }

	// Extract Root Motion transform from the animation
	virtual FTransform ExtractRootMotion(const FAnimExtractContext& ExtractionContext) const { return {}; }

	// Extract Root Motion transform from a contiguous position range (no looping)
	virtual FTransform ExtractRootMotionFromRange(double StartTime, double EndTime, const FAnimExtractContext& ExtractionContext) const { return {}; }

	// Extract the transform from the root track for the given animation position
	virtual FTransform ExtractRootTrackTransform(const FAnimExtractContext& ExtractionContext, const FBoneContainer* RequiredBones) const { return {}; }

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	virtual void AdvanceMarkerPhaseAsLeader(bool bLooping, float MoveDelta, const TArray<FName>& ValidMarkerNames, float& CurrentTime, FMarkerPair& PrevMarker, FMarkerPair& NextMarker, TArray<FPassedMarker>& MarkersPassed, const UMirrorDataTable* MirrorTable) const { check(false); /*Should never call this (either missing override or calling on unsupported asset */ }

	virtual void AdvanceMarkerPhaseAsFollower(const FMarkerTickContext& Context, float DeltaRemaining, bool bLooping, float& CurrentTime, FMarkerPair& PreviousMarker, FMarkerPair& NextMarker, const UMirrorDataTable* MirrorTable) const { check(false); /*Should never call this (either missing override or calling on unsupported asset */ }
	
	virtual void GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker) const { check(false); /*Should never call this (either missing override or calling on unsupported asset */ }

	virtual FMarkerSyncAnimPosition GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, const UMirrorDataTable* MirrorTable) const { check(false); return FMarkerSyncAnimPosition(); /*Should never call this (either missing override or calling on unsupported asset */ }

	virtual void GetMarkerIndicesForPosition(const FMarkerSyncAnimPosition& SyncPosition, bool bLooping, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker, float& CurrentTime, const UMirrorDataTable* MirrorTable ) const { check(false); /*Should never call this (either missing override or calling on unsupported asset */ }
	
	virtual float GetFirstMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition) const { return 0.f; }
	virtual float GetNextMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const { return 0.f; }
	virtual float GetPrevMatchingPosFromMarkerSyncPos(const FMarkerSyncAnimPosition& InMarkerSyncGroupPosition, const float& StartingPosition) const { return 0.f; }

	// default implementation, no additive
	virtual EAdditiveAnimationType GetAdditiveAnimType() const { return AAT_None; }
	virtual bool CanBeUsedInComposition() const { return true;  }

	// to support anim sequence base to montage
	virtual void EnableRootMotionSettingFromMontage(bool bInEnableRootMotion, const ERootMotionRootLock::Type InRootMotionRootLock) {};
	virtual bool GetEnableRootMotionSettingFromMontage() const { return false; }

#if WITH_EDITOR
private:
	DECLARE_MULTICAST_DELEGATE( FOnNotifyChangedMulticaster );
	FOnNotifyChangedMulticaster OnNotifyChanged;

public:
	typedef FOnNotifyChangedMulticaster::FDelegate FOnNotifyChanged;

	/** Registers a delegate to be called after notification has changed*/
	ENGINE_API void RegisterOnNotifyChanged(const FOnNotifyChanged& Delegate);
	ENGINE_API void UnregisterOnNotifyChanged(FDelegateUserObject Unregister);
	virtual bool IsValidToPlay() const { return true; }
	// ideally this would be animsequcnebase, but we might have some issue with that. For now, just allow AnimSequence
	virtual class UAnimSequence* GetAdditiveBasePose() const { return nullptr; }
#endif

	// return true if anim notify is available 
	ENGINE_API virtual bool IsNotifyAvailable() const;

#if WITH_EDITOR
	ENGINE_API void OnEndLoadPackage(const FEndLoadPackageContext& Context);
	ENGINE_API virtual void OnAnimModelLoaded();
public:
	/** Returns the IAnimationDataModel object embedded in this UAnimSequenceBase */
	ENGINE_API IAnimationDataModel* GetDataModel() const;

	/** Returns the IAnimationDataModel as a script-interface, provides access to UObject and Interface */
	ENGINE_API TScriptInterface<IAnimationDataModel> GetDataModelInterface() const;

	/** Returns the transient UAnimDataController set to operate on DataModel */
	ENGINE_API IAnimationDataController& GetController();
protected:
	/** Populates the UAnimDataModel object according to any pre-existing data. (overrides expect to populate the model according to their data) */
	ENGINE_API virtual void PopulateModel();
	ENGINE_API virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> ExistingDataModel);

	/** Callback registered to UAnimDatModel::GetModifiedEvent for the embedded object */
	ENGINE_API virtual void OnModelModified(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload);

	/** Validates that DataModel contains a valid UAnimDataModel object */
	ENGINE_API void ValidateModel() const;
	
	/** Binds to DataModel its modification delegate */
	ENGINE_API void BindToModelModificationEvent();

	/** Replaces the current DataModel, if any, with the provided one */
	ENGINE_API void CopyDataModel(const TScriptInterface<IAnimationDataModel>& ModelToDuplicate);
private:
	/** Creates a new UAnimDataModel instance and sets DataModel accordingly */
	ENGINE_API void CreateModel();

public:
	ENGINE_API bool ShouldDataModelBeValid() const;
	bool IsDataModelValid() const
	{
		if(ShouldDataModelBeValid())
		{
			return DataModelInterface != nullptr;
		}

		return false;
	}	
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:
	UE_DEPRECATED(5.1, "DataModel has been converted to DataModelInterface")
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Model")
	TObjectPtr<UAnimDataModel> DataModel;

	/** IAnimationDataModel instance containing (source) animation data */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Animation Model")
	TScriptInterface<IAnimationDataModel> DataModelInterface;

	/** Flag set whenever the data-model is initially populated (during upgrade path) */
	bool bPopulatingDataModel;

	/** UAnimDataController instance set to operate on DataModel */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, DuplicateTransient, Category = "Animation Model")
	TScriptInterface<IAnimationDataController> Controller;
	
	/** Helper object that keeps track of any controller brackets, and all unique notify types that are broadcasted during it */
	UE::Anim::FAnimDataModelNotifyCollector NotifyCollector;
#endif // WITH_EDITORONLY_DATA

};
