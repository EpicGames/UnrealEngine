// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtrTemplates.h"
#include "AnimData/AnimDataNotifications.h"
#include "BoneContainer.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"

class UAnimSequence;
class IAnimationDataModel;
class UAnimDataController;
class IAnimationDataController;
class FName;
struct FCompactPose;
struct FRawCurveTracks;
struct FRawAnimSequenceTrack;
struct FBoneAnimationTrack;
struct FRawAnimSequenceTrack;
struct FAnimationPoseData;
class IAnimationDataController;
struct FVirtualBoneCompactPoseData;

enum class EAnimInterpolationType : uint8;

namespace UE {

namespace Anim {

#if WITH_EDITOR
	/**
	* Populates a FAnimationPoseData according to the data stored within a IAnimationDataModel provided a time to sample.
	*
	* @param	Model				Model to sample from
	* @param	OutPoseData			[out] Pose data to be populated using the model
	* @param	Time				Time value to sample at
	* @param	InterpolationType	Method to be used to interpolate between different sample points
	* @param	RetargetSource		Name of the retarget source to retrieve from the skeleton
	* @param	RetargetTransforms	Base transforms to use when retargetting the pose
	*/
	ENGINE_API void BuildPoseFromModel(const IAnimationDataModel* Model, FAnimationPoseData& OutPoseData, const double Time, const EAnimInterpolationType& InterpolationType, const FName& RetargetSource, const TArray<FTransform>& RetargetTransforms);

	/**
	* Populates float curves according to the data stored within a IAnimationDataModel provided a time to sample.
	*
	* @param	Model				Model to sample from
	* @param	OutCurves			[out] Curves to be populated using the model
	* @param	Time				Time value to sample at
	*/
	ENGINE_API void EvaluateFloatCurvesFromModel(const IAnimationDataModel* Model, FBlendedCurve& OutCurves, double Time);

	/**
	* Populates transform curves according to the data stored within a IAnimationDataModel provided a time to sample.
	*
	* @param	Model				Model to sample from
	* @param	OutCurves			[out] Curves which were sampled from the model, key-ed with their equivalent name
	* @param	Time				Time value to sample at
	* @param	BlendWeight			Blending weight to apply when sampling
	*/
	ENGINE_API void EvaluateTransformCurvesFromModel(const IAnimationDataModel* Model, TMap<FName, FTransform>& OutCurves, double Time, float BlendWeight);

	/**
	* Copies over any individual curve from FRawCurveTracks to a IAnimationDataModel instance targeted by the provided controller.
	*
	* @param	CurveData			Container of the curves to be copied over
	* @param	Skeleton			Skeleton to use for verifying and or add the curve names
	* @param	Controller			Controller to use for adding curve data to a model
	*/
	ENGINE_API void CopyCurveDataToModel(const FRawCurveTracks& CurveData, const USkeleton* Skeleton, IAnimationDataController& Controller);

	/**
	* Copies over any individual curve from FRawCurveTracks to a IAnimationDataModel instance targeted by the provided controller.
	*
	* @param	SourceAnimSeq				Animation Sequence to copy the notifies from
	* @param	DestAnimSeq					Animation Sequence to copy the notifies to
	* @param	bShowDialogs				Whether or not to show any user-facing dialogs for confirmation of the copy
	* @param	bDeleteExistingNotifies		Whether or not to delete all notifies found on the destination sequence
	*
	* @return Whether or not the copy was succesful
	*/
	ENGINE_API bool CopyNotifies(const UAnimSequenceBase* SourceAnimSeq, UAnimSequenceBase* DestAnimSeq, bool bShowDialogs = true, bool bDeleteExistingNotifies = false);

	namespace AnimationData
	{
		/**
		* Adds an additional animated frame to match with the first animated frame, attempting to make it loop
		*
		* @param	InSequence		Animation Sequence to add the animated frame to
		*
		* @return Whether or not the operation was succesful
		*/
		ENGINE_API bool AddLoopingInterpolation(UAnimSequence* InSequence);

		/**
		* Trim a specific window from the animation data
		*
		* @param	InSequence		Animation Sequence to add the animated frame to
		* @param	TrimRange		Range of _frame numbers_ to remove
		*
		* @return Whether or not the operation was succesful
		*/
		ENGINE_API bool Trim(UAnimSequence* InSequence, TRange<FFrameNumber> TrimRange);

		/**
		* Insert duplicate key(s) for all tracks for the provided Animation Sequence
		*
		* @param	InSequence			Animation Sequence to add the duplicated animation frames
		* @param	StartKeyIndex		Key index after which the duplicate keys should be inserted
		* @param	NumDuplicates		Number of duplicated keys to insert
		* @param	SourceKeyIndex		Specific key index to use as a source for the duplicated keys, otherwise uses key at StartKeyIndex
		*/
		ENGINE_API void DuplicateKeys(UAnimSequence* InSequence, int32 StartKeyIndex, int32 NumDuplicates, int32 SourceKeyIndex = INDEX_NONE);

		/**
		* Remove a number of keys for all tracks for the provided Animation Sequence
		*
		* @param	InSequence			Animation Sequence to remove the animated frames from
		* @param	StartKeyIndex		Key index at which the key removal should start
		* @param	NumKeysToRemove		Number of keys to remove
		*/
		ENGINE_API void RemoveKeys(UAnimSequence* InSequence, int32 StartKeyIndex, int32 NumKeysToRemove);

		/**
		* Finds the name of the first child track (bone) for the provided bone name
		*
		* @param	InSequence			Animation Sequence to remove the animated frames from
		* @param	Skeleton			Skeleton to use for retrieving child bone names
		* @param	BoneName			Name of the bone to find the child track index for
		*
		* @return The first child track name for the provided bone name, if none found will return NAME_None
		*/
		ENGINE_API FName FindFirstChildTrackName(const UAnimSequence* InSequence, const USkeleton* Skeleton, const FName& BoneName);

	}
#endif // WITH_EDITOR

	/**
	* Extract Bone Transform of the Time given, from the provided FRawAnimSequenceTrack
	*
	* @param	RawTrack		RawAnimationTrack it extracts bone transform from
	* @param	OutAtom			[out] Output bone transform.
	* @param	KeyIndex		Key index to retrieve from the track.
	*/
	ENGINE_API void ExtractBoneTransform(const struct FRawAnimSequenceTrack& RawTrack, FTransform& OutTransform, int32 KeyIndex);


	/**
	 * Mirrors transform based on the specified mirror table.
	 *
	 * @param	Transform		Transform to mirror
	 * @param	MirrorDataTable	Animation data mirroring data table.
	 * @returns Transform mirrored based on the mirror data table.
	 */
	ENGINE_API FTransform MirrorTransform(const FTransform& Transform, const UMirrorDataTable& MirrorDataTable);

	/**
	 * Extract relative root motion between start and end position of the animation asset. With optional looping if applicable.
	 * NOTE WELL: Our second time param is a delta time, not an end like like other variations of this API.
	 *
	 * @param	Animation		Animation asset to extract the root motion from.
	 * @param	MirrorDataTable	[optional] Animation data mirroring data table.
	 * @param	StartTime		Start Position of the animation.
	 * @param	DeltaTime		Delta time to Advance, can be negative.
	 * @param	bAllowLooping	True if we should accumulate and loop.
	 */
	ENGINE_API FTransform ExtractRootMotionFromAnimationAsset(const UAnimationAsset* Animation, const UMirrorDataTable* MirrorDataTable, float StartTime, float DeltaTime, bool bAllowLooping);

	/**
	 * Extract relative root motion between start and end position of the animation asset.
	 *
	 * @param	Animation		Animation asset to extract the root motion from.
	 * @param	MirrorDataTable	[optional] Animation data mirroring data table.
	 * @param	StartPosition	Start Position of the animation.
	 * @param	EndPosition		End Position of the animation.
	 */
	ENGINE_API FTransform ExtractRootMotionFromAnimationAsset(const UAnimationAsset* Animation, const UMirrorDataTable* MirrorDataTable, float StartPosition, float EndPosition);

	/**
	 * Extract absolute root transform at specific position in time from the animation asset.
	 * Note: this function applies skeleton reference pose the transform (similar to ExtractRootMotionFromAnimationAsset, but unlike UAnimSequence::ExtractRootTrackTransform()). 
	 *
	 * @param	Animation		Animation asset to extract the root motion from.
	 * @param	Position		Start Position of the animation.
	 */
	ENGINE_API FTransform ExtractRootTransformFromAnimationAsset(const UAnimationAsset* Animation, float Position);

	namespace Compression
	{
		/**
		* Compress the provided tracks using two methods.
		* 1. Checking for uniform key data according to the provided error metrics
		* 2. Setting number of scale keys to 0 if it is uniform, equal to FVector::OneVector and matches the Skeleton reference bone pose(s)
		*
		* @param	RawAnimationData	Set of tracks to apply the reduction to
		* @param	NumberOfKeys		Expected number of keys for each component within the tracks
		* @param	ErrorName			Identifier for outer callsite when failing to reduce the data
		* @param	MaxPosDiff			Maximum positional delta when determining whether or not two positional keys are identical
		* @param	MaxAngleDiff		Maximum rotational delta when determining whether or not two rotational keys are identical
		*
		* @return Whether or not the operation was succesful
		*/
		ENGINE_API bool CompressAnimationDataTracks(const USkeleton* Skeleton, const TArray<FTrackToSkeletonMap>& TrackToSkeleton,TArray<FRawAnimSequenceTrack>& RawAnimationData, int32 NumberOfKeys, FName ErrorName, float MaxPosDiff = 0.0001f, float MaxAngleDiff = 0.0003f, float MaxScaleDiff = 0.0001f);

		/**
		* Compress the provided track by checking for uniform key data according to the provided error metrics
		*
		* @param	RawTrack			Individual track to apply the reduction to
		* @param	NumberOfKeys		Expected number of keys for each component within the tracks
		* @param	ErrorName			Identifier for outer callsite when failing to reduce the data
		* @param	MaxPosDiff			Maximum positional delta when determining whether or not two positional keys are identical
		* @param	MaxAngleDiff		Maximum rotational delta when determining whether or not two rotational keys are identical
		* @param	MaxScaleDiff		Maximum scale delta when determining whether or not two scale keys are identical
		*
		* @return Whether or not the operation was succesful
		*/
		ENGINE_API bool CompressRawAnimSequenceTrack(FRawAnimSequenceTrack& RawTrack, int32 NumberOfKeys, FName ErrorName, float MaxPosDiff, float MaxAngleDiff, float MaxScaleDiff);

		/**
		* Sanitize the provided track by snapping small scale values to 0, and normalizing any rotational keys
		*
		* @param	RawTrack			Individual track to sanitize
		*/
		ENGINE_API void SanitizeRawAnimSequenceTrack(FRawAnimSequenceTrack& RawTrack);

#if WITH_EDITOR
		// RAII helper to temporarily block Animation compression requests for specified AnimationSequence
		struct FScopedCompressionGuard
		{
			FScopedCompressionGuard() = delete;
			ENGINE_API FScopedCompressionGuard(UAnimSequence* InAnimSequence);
			ENGINE_API ~FScopedCompressionGuard();
		protected:
			TObjectPtr<UAnimSequence> AnimSequence;
		};
#endif // WITH_EDITOR
	}

	namespace DataModel { struct FEvaluationContext; }
	namespace Retargeting
	{
		struct FRetargetTracking
		{
			const FCompactPoseBoneIndex PoseBoneIndex;
			const int32 SkeletonBoneIndex;

			FRetargetTracking(const FCompactPoseBoneIndex InPoseBoneIndex, const int32 InSkeletonBoneIndex) : PoseBoneIndex(InPoseBoneIndex), SkeletonBoneIndex(InSkeletonBoneIndex) {}
		};

		struct FRetargetingScope
		{
			ENGINE_API FRetargetingScope(const USkeleton* InSourceSkeleton, FCompactPose& ToRetargetPose, const DataModel::FEvaluationContext& InEvaluationContext);
			ENGINE_API ~FRetargetingScope();
			
			ENGINE_API void AddTrackedBone(FCompactPoseBoneIndex CompactBoneIndex, int32 SkeletonBoneIndex) const;
		private:
			const USkeleton* SourceSkeleton;
			FCompactPose& RetargetPose;
			const DataModel::FEvaluationContext& EvaluationContext;
			TArray<FRetargetTracking>& RetargetTracking;
			bool bShouldRetarget;
		};
	}

	struct FBuildRawPoseScratchArea : public TThreadSingleton<FBuildRawPoseScratchArea>
	{
		TArray<Retargeting::FRetargetTracking> RetargetTracking;
		TArray<FVirtualBoneCompactPoseData> VirtualBoneCompactPoseData;
	};

} // namespace Anim	

} // namespace UE
