// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/TransformProviderData.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "RenderCommandFence.h"
#include "SkinningDefinitions.h"
#include "SkeletalRenderPublic.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AnimTrackPool.h"
#include "Engine/DataAsset.h"

#if WITH_EDITOR
#include "IO/IoHash.h"
#endif

#include "AnimSequenceTransformProviderData.generated.h"

class ITargetPlatform;
class USkinnedAsset;
class UAnimBankData;
class UAnimSequence;
class UBlendSpace;
class FAnimSequenceTransformProviderBuildAsyncCacheTask;
class UAnimSequenceTransformProviderData;
class FAnimSequenceTransformProviderProxy;
class IAnimSequenceTransformProvider;
class FDataValidationContext;

UENUM(BlueprintType)
enum class EAnimSequenceTrackLoopMode : uint8
{
	// Looping is forced for the current track.
	Loop,

	// Clamp is forced for the current track.
	Clamp
};

UENUM(BlueprintType)
enum class EAnimSequenceTrackMode : uint8
{
	AutoPlay,
	Manual,
	BlendSpace,
};

USTRUCT(BlueprintType)
struct FAnimSequenceTrackPosition
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Animation)
	float SourcePosition = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = Animation)
	float TargetPosition = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = Animation)
	float BlendWeight = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = Animation)
	float BlendTime = 0.0f;

	bool IsBlending() const
	{
		return BlendWeight > 0.0f && BlendWeight < 1.0f;
	}
};

UENUM(BlueprintType)
enum class EAnimSequenceTransformProviderLayerBlendMode : uint8
{
	// Replaces the accumulated result. Weight controls blend factor.
	Override,

	// Adds a weighted delta on top of the accumulated result.
	Additive,
};

USTRUCT(BlueprintType)
struct FAnimSequenceTransformProviderLayer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Layer)
	EAnimSequenceTransformProviderLayerBlendMode BlendMode = EAnimSequenceTransformProviderLayerBlendMode::Override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Layer, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Weight = 1.0f;

	// Name of a blend profile on the skeleton to use as a per-bone weight mask. Empty means all bones at full weight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Layer)
	FName BoneMaskProfileName;

	// Blend rotation in global space instead of local space.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Layer)
	bool bMeshSpaceRotation = false;
};

USTRUCT(BlueprintType)
struct FAnimSequenceTrackAutoPlayData
{
	GENERATED_USTRUCT_BODY()

	// The sequence index to play.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	int32 SequenceIndex = 0;

	// The starting time position to begin play.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float Position = 0.0f;

	// The rate multiplier to play the animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float PlayRate = 1.0f;

	// The duration of the blend transition in seconds. Use 0 for an instance switch.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (ClampMin = "0.0"))
	float BlendTime = 0.0f;

	// The mode to control looping behavior.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	EAnimSequenceTrackLoopMode LoopMode = EAnimSequenceTrackLoopMode::Loop;
};

USTRUCT(BlueprintType)
struct FAnimSequenceTrackManualData
{
	GENERATED_USTRUCT_BODY()

	// The sequence index to play.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	int32 SequenceIndex = 0;

	// The time position to sample in the sequence.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float Position = 0.0f;

	// The duration of the blend transition in seconds. Use 0 for an instant switch.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (ClampMin = "0.0"))
	float BlendTime = 0.0f;

	// The mode to control looping behavior for when the position is not within the sequence bounds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	EAnimSequenceTrackLoopMode LoopMode = EAnimSequenceTrackLoopMode::Loop;
};

USTRUCT(BlueprintType)
struct FAnimSequenceTrackBlendSpaceData
{
	GENERATED_USTRUCT_BODY()

	// Index into the ASTP's BlendSpaces array.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	int32 BlendSpaceIndex = 0;

	// The 2D blend position.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	FVector2f BlendPosition = FVector2f::ZeroVector;

	// The starting time position to begin play.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float Position = 0.0f;

	// The rate multiplier to play the animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float PlayRate = 1.0f;

	// The duration of the blend transition in seconds. Use 0 for an instant switch.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (ClampMin = "0.0"))
	float BlendTime = 0.0f;

	// The mode to control looping behavior.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	EAnimSequenceTrackLoopMode LoopMode = EAnimSequenceTrackLoopMode::Loop;
};

USTRUCT(BlueprintType)
struct FAnimSequenceTransformProviderSequence
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class UAnimSequence> Sequence;

	// The default rate multiplier to play the animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float PlayRate = 1.0f;

	// The default position to start playing the animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float Position = 0.0f;
};

USTRUCT(BlueprintType)
struct FAnimSequenceTransformProviderBlendSpace
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<UBlendSpace> BlendSpace;
};

class FAnimSequenceTransformProviderProxy;
class UAnimSequenceTransformProviderDataInstance;

enum class EAnimSequenceTrackPatchFlags : uint8
{
	None         = 0,
	PlayRate     = 1 << 0,
	LoopMode     = 1 << 1,
	All          = 0x3
};
ENUM_CLASS_FLAGS(EAnimSequenceTrackPatchFlags);

class FAnimSequenceTrackPackedData
{
public:
	/////////////////////////////////////////////////////////////////////////////////////////////////
	// AutoPlay

	static FAnimSequenceTrackPackedData Pack(FAnimSequenceTrackAutoPlayData AutoPlayData, double WorldTime)
	{
		FAnimSequenceTrackPackedData Data;
		Data.SequenceIndex = AutoPlayData.SequenceIndex;
		Data.bAutoPlay = 1;
		Data.LoopMode = static_cast<uint32>(AutoPlayData.LoopMode);
		Data.PatchFlags = static_cast<uint32>(EAnimSequenceTrackPatchFlags::All);
		Data.Auto.ReferenceTimestamp = GetTimestamp(WorldTime, AutoPlayData.Position, AutoPlayData.PlayRate);
		Data.Auto.Position = AutoPlayData.Position;
		Data.Auto.PlayRate = FFloat16(AutoPlayData.PlayRate);
		Data.BlendTime = FFloat16(AutoPlayData.BlendTime);
		return Data;
	}

	bool IsAutoPlay() const
	{
		return bAutoPlay;
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////
	// Manual

	static FAnimSequenceTrackPackedData Pack(FAnimSequenceTrackManualData ManualData, float SequenceLength)
	{
		FAnimSequenceTrackPackedData Data;
		Data.SequenceIndex = ManualData.SequenceIndex;
		Data.bAutoPlay = 0;
		Data.LoopMode = static_cast<uint32>(ManualData.LoopMode);
		Data.PatchFlags = static_cast<uint32>(EAnimSequenceTrackPatchFlags::All);
		Data.Manual.Position = WrapPosition(ManualData.LoopMode, ManualData.Position, SequenceLength);
		Data.BlendTime = FFloat16(ManualData.BlendTime);
		return Data;
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////
	// BlendSpace

	static FAnimSequenceTrackPackedData Pack(const FAnimSequenceTrackBlendSpaceData& BlendSpaceData, double WorldTime)
	{
		FAnimSequenceTrackPackedData Data;
		Data.bBlendSpace = 1;
		Data.bAutoPlay = 0;
		Data.LoopMode = static_cast<uint32>(BlendSpaceData.LoopMode);
		Data.PatchFlags = static_cast<uint32>(EAnimSequenceTrackPatchFlags::All);
		Data.BlendSpace.ReferenceTimestamp = GetTimestamp(WorldTime, BlendSpaceData.Position, BlendSpaceData.PlayRate);
		Data.BlendSpace.Position = BlendSpaceData.Position;
		Data.BlendSpace.PlayRate = FFloat16(BlendSpaceData.PlayRate);
		Data.BlendSpace.BlendPosition[0] = FFloat16(BlendSpaceData.BlendPosition.X);
		Data.BlendSpace.BlendPosition[1] = FFloat16(BlendSpaceData.BlendPosition.Y);
		Data.BlendSpace.Samples[0] = 0;
		Data.BlendSpace.Samples[1] = 0;
		Data.BlendSpace.Samples[2] = 0;
		Data.BlendSpace.BlendSpaceIndex = static_cast<uint8>(BlendSpaceData.BlendSpaceIndex);
		Data.BlendTime = FFloat16(BlendSpaceData.BlendTime);
		return Data;
	}

	bool IsBlendSpace() const
	{
		return bBlendSpace;
	}

	FVector2f GetBlendSpacePosition() const
	{
		return FVector2f(BlendSpace.BlendPosition[0].GetFloat(), BlendSpace.BlendPosition[1].GetFloat());
	}

	void SetBlendSpacePosition(FVector2f InPosition)
	{
		BlendSpace.BlendPosition[0] = FFloat16(InPosition.X);
		BlendSpace.BlendPosition[1] = FFloat16(InPosition.Y);
	}

	void SetManualPosition(float InPosition)
	{
		Manual.Position = InPosition;
	}

	uint32 GetNumSamples() const
	{
		return bBlendSpace ? BlendSpace.NumSamples : 1;
	}

	void SetSampleData(uint32 SampleIndex, uint16 InSequenceIndex, FFloat16 Weight)
	{
		BlendSpace.Samples[SampleIndex] = (uint32(InSequenceIndex) & 0xFFFF) | (uint32(Weight.Encoded) << 16);
	}

	uint16 GetSampleSequenceIndex(uint32 SampleIndex) const
	{
		return BlendSpace.Samples[SampleIndex] & 0xFFFF;
	}

	FFloat16 GetSampleWeight(uint32 SampleIndex) const
	{
		FFloat16 Result;
		Result.Encoded = BlendSpace.Samples[SampleIndex] >> 16;
		return Result;
	}

	void SetNumSamples(uint32 InNumSamples)
	{
		BlendSpace.NumSamples = static_cast<uint8>(InNumSamples);
	}

	uint8 GetBlendSpaceIndex() const
	{
		return BlendSpace.BlendSpaceIndex;
	}

	bool IsManual() const
	{
		return !bAutoPlay && !bBlendSpace;
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////

	int32 GetSequenceIndex() const
	{
		return bBlendSpace ? INDEX_NONE : static_cast<int32>(SequenceIndex);
	}

	// Returns the raw unwrapped time position. Caller must fmod against sequence play length.
	double GetTimePosition(double WorldTime) const
	{
		if (IsManual())
		{
			return static_cast<double>(Manual.Position);
		}

		if (IsBlendSpace())
		{
			return (WorldTime - BlendSpace.ReferenceTimestamp) * BlendSpace.PlayRate.GetFloat();
		}

		return (WorldTime - Auto.ReferenceTimestamp) * Auto.PlayRate.GetFloat();
	}

	double GetReferenceTimestamp() const
	{
		check(!IsManual());
		return bBlendSpace ? BlendSpace.ReferenceTimestamp : Auto.ReferenceTimestamp;
	}

	float GetManualPosition() const
	{
		return Manual.Position;
	}

	void SetLoopMode(EAnimSequenceTrackLoopMode NewLoopMode)
	{
		LoopMode = static_cast<uint32>(NewLoopMode);
		PatchFlags |= (uint32)EAnimSequenceTrackPatchFlags::LoopMode;
	}

	EAnimSequenceTrackLoopMode GetLoopMode() const
	{
		return static_cast<EAnimSequenceTrackLoopMode>(LoopMode);
	}

	float GetBlendTime() const
	{
		return BlendTime.GetFloat();
	}

	void ResetPatchFlags()
	{
		PatchFlags = 0;
	}

	EAnimSequenceTrackPatchFlags GetPatchFlags() const
	{
		return static_cast<EAnimSequenceTrackPatchFlags>(PatchFlags);
	}

	void SetPlayRate(float NewPlayRate, double WorldTime)
	{
		check(IsAutoPlay() || IsBlendSpace());

		if (IsBlendSpace())
		{
			const double Position = (WorldTime - BlendSpace.ReferenceTimestamp) * BlendSpace.PlayRate.GetFloat();
			BlendSpace.ReferenceTimestamp = GetTimestamp(WorldTime, static_cast<float>(Position), NewPlayRate);
			BlendSpace.PlayRate = FFloat16(NewPlayRate);
		}
		else
		{
			const double Position = (WorldTime - Auto.ReferenceTimestamp) * Auto.PlayRate.GetFloat();
			Auto.ReferenceTimestamp = GetTimestamp(WorldTime, static_cast<float>(Position), NewPlayRate);
			Auto.PlayRate = FFloat16(NewPlayRate);
		}

		PatchFlags |= (uint32)EAnimSequenceTrackPatchFlags::PlayRate;
	}

	float GetPlayRate() const
	{
		if (IsBlendSpace())
		{
			return BlendSpace.PlayRate.GetFloat();
		}
		return IsAutoPlay() ? Auto.PlayRate.GetFloat() : 0.0f;
	}

	ENGINE_API void Serialize(FArchive& Ar);

	void PostLoad(double WorldTime)
	{
		if (IsBlendSpace())
		{
			BlendSpace.ReferenceTimestamp = GetTimestamp(WorldTime, BlendSpace.Position, BlendSpace.PlayRate.GetFloat());
		}
		else if (IsAutoPlay())
		{
			Auto.ReferenceTimestamp = GetTimestamp(WorldTime, Auto.Position, Auto.PlayRate.GetFloat());
		}
	}

	void SetLayerWeight(float Weight)
	{
		LayerWeight = FFloat16(Weight);
	}

	float GetLayerWeight() const
	{
		return LayerWeight.GetFloat();
	}

	void SetData(const FAnimSequenceTrackPackedData& RHS)
	{
		const FFloat16 PrevLayerWeight = LayerWeight;
		*this = RHS;
		LayerWeight = PrevLayerWeight;
	}

	static float WrapPosition(EAnimSequenceTrackLoopMode LoopMode, double Position, double SequenceLength)
	{
		if (LoopMode == EAnimSequenceTrackLoopMode::Loop)
		{
			if (SequenceLength > 0.0)
			{
				Position = FMath::Fmod(Position, SequenceLength);

				if (Position < 0.0)
				{
					Position += SequenceLength;
				}
			}
			else
			{
				Position = 0.0;
			}
		}
		else
		{
			Position = FMath::Clamp(Position, 0.0, SequenceLength);
		}

		return static_cast<float>(Position);
	}

private:
	static double GetTimestamp(double WorldTime, float Position, float PlayRate)
	{
		if (FMath::Abs(PlayRate) > UE_SMALL_NUMBER)
		{
			return WorldTime - static_cast<double>(Position / PlayRate);
		}
		return WorldTime;
	}

	uint32 SequenceIndex        : 20 = 0;
	uint32 LoopMode             : 1 = 0;
	uint32 bAutoPlay            : 1 = 0;
	uint32 bBlendSpace          : 1 = 0;
	uint32 PatchFlags           : 9 = 0;

	union
	{
		struct
		{
			double ReferenceTimestamp;
			float Position;
			FFloat16 PlayRate;
		} Auto;

		struct
		{
			float Position;
		} Manual;

		struct
		{
			double ReferenceTimestamp;
			float Position;
			uint32 Samples[3];
			FFloat16 PlayRate;
			FFloat16 BlendPosition[2];
			uint8 NumSamples;
			uint8 BlendSpaceIndex;
		} BlendSpace = {};
	};

	FFloat16 BlendTime = FFloat16(0.0f);
	FFloat16 LayerWeight = FFloat16(1.0f);

	friend class FAnimSequenceTransformProvider;
};

inline FArchive& operator <<(FArchive& Ar, FAnimSequenceTrackPackedData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

using FAnimSequenceTrackPool = TAnimTrackPool<FAnimSequenceTrackPackedData>;

class FAnimSequenceTrackRenderData
{
public:
	FAnimSequenceTrackRenderData() = default;
	FAnimSequenceTrackRenderData(const FAnimSequenceTrackRenderData&) = default;
	FAnimSequenceTrackRenderData& operator=(const FAnimSequenceTrackRenderData&) = default;

	FAnimSequenceTrackRenderData(const FAnimSequenceTrackPackedData& Data)
	{
		*this = Data;
	}

	FAnimSequenceTrackRenderData& operator=(const FAnimSequenceTrackPackedData& PackedData)
	{
		Source = PackedData;
		Target = PackedData;
		BlendWeight = FFloat16(0.0f);
		return *this;
	}

	bool IsBlending() const
	{
		return BlendWeight.Encoded != 0;
	}

	float GetBlendWeight(bool bPrevious = false) const
	{
		return bPrevious ? PreviousBlendWeight.GetFloat() : BlendWeight.GetFloat();
	}

	uint32 GetTotalSampleCount(EPreviousBoneTransformUpdateMode UpdateMode = EPreviousBoneTransformUpdateMode::None) const
	{
		const bool bUpdatePrevious = (UpdateMode == EPreviousBoneTransformUpdateMode::UpdatePrevious);
		const uint32 SourceSamples = Source.GetNumSamples();
		const uint32 TargetSamples = Target.GetNumSamples();
		const uint32 CurrentSamples = SourceSamples + ((BlendWeight.Encoded != 0) ? TargetSamples : 0);
		const uint32 PreviousSamples = SourceSamples + ((PreviousBlendWeight.Encoded != 0) ? TargetSamples : 0);

		return CurrentSamples + (bUpdatePrevious ? PreviousSamples : 0);
	}

	const FAnimSequenceTrackPackedData& GetSource() const
	{
		return Source;
	}

	const FAnimSequenceTrackPackedData& GetTarget() const
	{
		return Target;
	}

	ENGINE_API void Patch(const FAnimSequenceTrackPackedData& Request, double WorldTime, bool bWasActive);

	ENGINE_API void Tick(float DeltaTime, double WorldTime);

	bool HasPreviousPosition() const
	{
		return bHasPreviousPosition;
	}

	double GetPreviousTimePosition() const
	{
		return PreviousTimePosition;
	}

	FVector2f GetPreviousBlendSpacePosition() const
	{
		return FVector2f(PreviousBlendSpacePosition[0].GetFloat(), PreviousBlendSpacePosition[1].GetFloat());
	}

private:
	FAnimSequenceTrackPackedData Source;
	FAnimSequenceTrackPackedData Target;

	double PreviousTimePosition = 0.0;
	FFloat16 PreviousBlendSpacePosition[2] = { FFloat16(0.0f), FFloat16(0.0f) };
	FFloat16 PreviousBlendWeight = FFloat16(0.0f);
	FFloat16 BlendWeight = FFloat16(0.0f);
	bool bHasPreviousPosition = false;
};

using FAnimSequenceTrackRenderPool = TAnimTrackPool<FAnimSequenceTrackRenderData>;

UCLASS(BlueprintType, MinimalAPI)
class UAnimSequenceTransformProviderLayerStack : public UDataAsset
{
	GENERATED_BODY()

public:
	UAnimSequenceTransformProviderLayerStack()
	{
		Layers.Emplace();
	}

	const TArray<FAnimSequenceTransformProviderLayer>& GetLayers() const
	{
		return Layers;
	}

	void SetLayers(const TArray<FAnimSequenceTransformProviderLayer>& InLayers)
	{
		Layers = InLayers;
	}

	int32 GetNumLayers() const
	{
		return Layers.Num();
	}

#if WITH_EDITOR
	virtual FText GetDisplayNameText() const override
	{
		return NSLOCTEXT("AnimSequenceTransformProvider", "LayerStackDesc", "Animation Layer Stack");
	}

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layers, meta = (AllowPrivateAccess = "true"))
	TArray<FAnimSequenceTransformProviderLayer> Layers;
};

UCLASS(BlueprintType, MinimalAPI)
class UAnimSequenceTransformProviderSequenceList : public UDataAsset
{
	GENERATED_BODY()

public:
	const TArray<FAnimSequenceTransformProviderSequence>& GetSequences() const
	{
		return Sequences;
	}

	void SetSequences(const TArray<FAnimSequenceTransformProviderSequence>& InSequences)
	{
		Sequences = InSequences;
	}

#if WITH_EDITOR
	virtual FText GetDisplayNameText() const override
	{
		return NSLOCTEXT("AnimSequenceTransformProvider", "SequenceListDesc", "Animation Sequence List");
	}

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sequences, meta = (AllowPrivateAccess = "true"))
	TArray<FAnimSequenceTransformProviderSequence> Sequences;
};

UCLASS(BlueprintType, MinimalAPI)
class UAnimSequenceTransformProviderBlendSpaceList : public UDataAsset
{
	GENERATED_BODY()

public:
	const TArray<FAnimSequenceTransformProviderBlendSpace>& GetBlendSpaces() const
	{
		return BlendSpaces;
	}

	void SetBlendSpaces(const TArray<FAnimSequenceTransformProviderBlendSpace>& InBlendSpaces)
	{
		BlendSpaces = InBlendSpaces;
	}

#if WITH_EDITOR
	virtual FText GetDisplayNameText() const override
	{
		return NSLOCTEXT("AnimSequenceTransformProvider", "BlendSpaceListDesc", "Animation Blend Space List");
	}

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BlendSpaces, meta = (AllowPrivateAccess = "true"))
	TArray<FAnimSequenceTransformProviderBlendSpace> BlendSpaces;
};

struct FAnimSequenceTransformProviderRenderLayer
{
	FAnimSequenceTrackRenderPool Tracks;
	FAnimSequenceTransformProviderLayer Layer;
	float Weight = 1.0f;
};

class FAnimSequenceTransformProviderRenderData
{
public:
	ENGINE_API ~FAnimSequenceTransformProviderRenderData();

	TArrayView<FAnimSequenceTransformProviderRenderLayer> GetLayers()
	{
		return Layers;
	}

	TConstArrayView<FAnimSequenceTransformProviderRenderLayer> GetLayers() const
	{
		return Layers;
	}

	int32 GetNumLayers() const
	{
		return Layers.Num();
	}

	uint32 GetMeshSpaceMask() const
	{
		return MeshSpaceMask;
	}

	uint32 GetLayerModeMask() const
	{
		return LayerModeMask;
	}

	bool IsTrackActive(int32 TrackIndex, int32 LayerIndex) const
	{
		return (LayerIndex == 0 || Layers[LayerIndex].Weight > 0.0f) && Layers[LayerIndex].Tracks.IsActiveIndex(TrackIndex);
	}

	int32 GetNumActiveLayers(int32 TrackIndex) const
	{
		int32 Count = 0;
		for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); LayerIndex++)
		{
			Count += IsTrackActive(TrackIndex, LayerIndex) ? 1 : 0;
		}
		return Count;
	}

private:
	void Patch(TArray<FAnimSequenceTrackPool::FPatch>&& InPatches, TArray<FAnimSequenceTransformProviderRenderLayer>&& InLayers, double WorldTime);
	void InsertProxy(FInstancedSkinningSceneExtensionProxy* Proxy);
	void RemoveProxy(FInstancedSkinningSceneExtensionProxy* Proxy);

	TArray<FAnimSequenceTransformProviderRenderLayer> Layers;
	Experimental::TRobinHoodHashSet<FInstancedSkinningSceneExtensionProxy*> Proxies;
	uint32 MeshSpaceMask = 0;
	uint32 LayerModeMask = 0;

	friend FAnimSequenceTransformProviderProxy;
	friend UAnimSequenceTransformProviderDataInstance;
};

class FAnimSequenceTransformProviderProxy : public FTransformProviderRenderProxy
{
	friend class UAnimSequenceTransformProviderDataInstance;
	friend class UAnimSequenceTransformProviderData;

	ENGINE_API FAnimSequenceTransformProviderProxy(TConstArrayView<FAnimSequenceTransformProviderSequence> Sequences, FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy);
	ENGINE_API FAnimSequenceTransformProviderProxy(TConstArrayView<FAnimSequenceTransformProviderSequence> Sequences, FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy, FAnimSequenceTransformProviderRenderData& RenderData);
	ENGINE_API ~FAnimSequenceTransformProviderProxy();

public:
	virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;

	TConstArrayView<FAnimSequenceTransformProviderSequence> GetSequences() const
	{
		return Sequences;
	}

	FInstancedSkinningSceneExtensionProxy* GetSceneExtensionProxy() const
	{
		return SceneExtensionProxy;
	}

	FAnimSequenceTransformProviderRenderData* GetRenderData() const
	{
		return RenderData;
	}

	TConstArrayView<FAnimSequenceTransformProviderRenderLayer> GetLayers() const
	{
		return RenderData->GetLayers();
	}

	int32 GetNumLayers() const
	{
		return RenderData->GetNumLayers();
	}

private:
	TConstArrayView<FAnimSequenceTransformProviderSequence> Sequences;
	FAnimSequenceTransformProviderRenderData* RenderData = nullptr;
	FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy = nullptr;
	const USkinnedAsset* SkinnedAsset = nullptr;
	IAnimSequenceTransformProvider* AnimSequenceTransformProvider = nullptr;
	bool bOwnsRenderData = false;
};

class IAnimSequenceTransformProvider
{
public:
	virtual ~IAnimSequenceTransformProvider() = default;

	virtual void RegisterProxy(FAnimSequenceTransformProviderProxy* Proxy) {}
	virtual void UnregisterProxy(FAnimSequenceTransformProviderProxy* Proxy) {}
};

struct FAnimSequenceTransformProviderCachedData
{
	TArray<FBoxSphereBounds> SequenceBounds;
	FBoxSphereBounds ConservativeBounds = FBoxSphereBounds(ForceInit);
};

inline FArchive& operator<<(FArchive& Ar, FAnimSequenceTransformProviderCachedData& CachedData)
{
	Ar << CachedData.SequenceBounds;

	// Derive conservative bounds from sequence bounds on load.
	if (Ar.IsLoading())
	{
		CachedData.ConservativeBounds = FBoxSphereBounds(ForceInit);
		for (int32 Index = 0; Index < CachedData.SequenceBounds.Num(); Index++)
		{
			CachedData.ConservativeBounds = (Index == 0)
				? CachedData.SequenceBounds[Index]
				: CachedData.ConservativeBounds + CachedData.SequenceBounds[Index];
		}
	}
	return Ar;
}

UCLASS(config = Engine, hidecategories = Object, MinimalAPI, BlueprintType)
class UAnimSequenceTransformProviderData : public UTransformProviderData
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const override;
	virtual bool IsGpuOnly() const override { return true; }
	virtual const FGuid& GetTransformProviderID() const override;
	virtual uint32 GetUniqueAnimationCount() const override;
	virtual bool UsesSkeletonBatching() const override;
	virtual bool HasAnimationBounds() const override;
	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const override;
	virtual bool IsCompiling() const override;
	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const override;
	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const override;
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual UTransformProviderData* GetRootTransformProvider() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	virtual void ClearAllCachedCookedPlatformData() override;

	/** Try to cancel any pending async tasks. Returns true if there is no more async tasks pending, false otherwise. */
	bool TryCancelAsyncTasks();

	/** Returns false if there is currently an async task running */
	bool IsAsyncTaskComplete() const;

	/** Wait until all async tasks are complete, up to a time limit. Returns true if all tasks are completed. */
	bool WaitForAsyncTasks(float TimeLimitSeconds);
#endif

public:
	// Returns the skinned asset for the transform provider.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	USkinnedAsset* GetSkinnedAsset() const
	{
		return SkinnedAsset;
	}

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	const TArray<struct FAnimSequenceTransformProviderSequence>& GetSequences() const
	{
		return Sequences;
	}

	// Returns the play length for a sequence.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API float GetSequencePlayLength(int32 SequenceIndex) const;

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	const TArray<struct FAnimSequenceTransformProviderLayer>& GetLayers() const
	{
		return Layers;
	}

	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	const TArray<FAnimSequenceTransformProviderBlendSpace>& GetBlendSpaces() const
	{
		return BlendSpaces;
	}

#if WITH_EDITORONLY_DATA
	UAnimSequenceTransformProviderLayerStack* GetLayerStack() const
	{
		return LayerStack;
	}

	UAnimSequenceTransformProviderSequenceList* GetSequenceList() const
	{
		return SequenceList;
	}
#endif

	// Returns the number of layers. Always >= 1.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	int32 GetNumLayers() const
	{
		return GetLayers().Num();
	}

#if WITH_EDITOR
	// Creates a transform provider data asset from an AnimBankData asset.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence", meta = (DevelopmentOnly))
	static ENGINE_API UAnimSequenceTransformProviderData* CreateFromAnimBankData(UAnimBankData* InAnimBankData);

	// Creates a transform provider data asset from an AnimBank asset.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence", meta = (DevelopmentOnly))
	static ENGINE_API UAnimSequenceTransformProviderData* CreateFromAnimBank(UAnimBank* InAnimBank);
#endif

protected:
	bool IsValidFor(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const;

private:
#if WITH_EDITOR
	static void ForEachReferencingProvider(
		TFunctionRef<bool(const UAnimSequenceTransformProviderData*)> Predicate,
		TFunctionRef<void(UAnimSequenceTransformProviderData*)> Action);
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TransformProvider, meta = (AllowPrivateAccess = "true", ShowOnlyInnerProperties))
	TObjectPtr<USkinnedAsset> SkinnedAsset;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UAnimSequenceTransformProviderSequenceList> SequenceList;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true", EditCondition = "SequenceList == nullptr", EditConditionHides))
	TArray<struct FAnimSequenceTransformProviderSequence> Sequences;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UAnimSequenceTransformProviderLayerStack> LayerStack;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true", EditCondition = "LayerStack == nullptr", EditConditionHides))
	TArray<FAnimSequenceTransformProviderLayer> Layers;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UAnimSequenceTransformProviderBlendSpaceList> BlendSpaceList;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true", EditCondition = "BlendSpaceList == nullptr", EditConditionHides))
	TArray<FAnimSequenceTransformProviderBlendSpace> BlendSpaces;

	// Enable game-thread playback tracking for blend state queries via GetPosition.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TransformProvider, meta = (AllowPrivateAccess = "true"))
	bool bEnableBlendTracking = false;

	double GetWorldTime() const;

	FRenderCommandFence DestroyFence;

	FAnimSequenceTransformProviderCachedData CachedData;

#if WITH_EDITOR
	FIoHash CreateDerivedDataKeyHash(const ITargetPlatform* TargetPlatform);
	FIoHash BeginCacheDerivedData(const ITargetPlatform* TargetPlatform);
	FAnimSequenceTransformProviderCachedData* CacheDerivedData(const ITargetPlatform* TargetPlatform);
	bool PollCacheDerivedData(const FIoHash& KeyHash) const;
	bool PollCacheDerivedData() const;
	void EndCacheDerivedData(const FIoHash& KeyHash);
	void FinishCacheDerivedData();

	FIoHash DataKeyHash;
	TMap<FIoHash, TUniquePtr<FAnimSequenceTransformProviderCachedData>> DataByPlatformKeyHash;
	TMap<FIoHash, TPimplPtr<FAnimSequenceTransformProviderBuildAsyncCacheTask>> CacheTasksByKeyHash;
#endif

	friend class UAnimSequenceTransformProviderDataInstance;
	friend class FAnimSequenceTransformProviderCompilingManager;
	friend class UAnimSequenceTransformProviderLayerStack;
	friend class UAnimSequenceTransformProviderSequenceList;
	friend class UAnimSequenceTransformProviderBlendSpaceList;
};

UCLASS(config = Engine, hidecategories = Object, MinimalAPI, BlueprintType, Within = InstancedSkinnedMeshComponent, NotBlueprintable, NotPlaceable)
class UAnimSequenceTransformProviderDataInstance : public UAnimSequenceTransformProviderData
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const override;
	virtual uint32 GetUniqueAnimationCount() const override;
	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const override;
	virtual bool HasAnimationBounds() const;
	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const override;
	virtual bool IsCompiling() const override;
	virtual UTransformProviderData* GetRootTransformProvider() override;
	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const override;
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void SubmitChanges() override;
public:
	// Creates an instance from a provider data asset.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	static ENGINE_API UAnimSequenceTransformProviderDataInstance* CreateAnimSequenceTransformProviderDataInstance(UAnimSequenceTransformProviderData* ProviderData, UInstancedSkinnedMeshComponent* Owner);

	// Sets the per-track weight for a layer. Multiplied with the asset's layer weight.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API void SetLayerWeight(int32 TrackIndex, int32 LayerIndex, float Weight);

	// Returns the per-track weight for a layer, or 1.0 if invalid.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API float GetLayerWeight(int32 TrackIndex, int32 LayerIndex) const;

	// Allocates a track slot across all layer pools with no animation data.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API int32 AllocateTrack();

	// Deallocates a track across all layer pools.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool DeallocateTrack(int32 TrackIndex);

	// Sets auto play animation data on a track layer.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetAutoPlayData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackAutoPlayData& Data);

	// Sets manual animation data on a track layer.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetManualData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackManualData& Data);

	// Sets the blend space data for a track layer. The blend space index must reference a valid entry in the BlendSpaces array.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetBlendSpaceData(int32 TrackIndex, int32 LayerIndex, const FAnimSequenceTrackBlendSpaceData& Data);

	// Updates the blend space 2D position on a track layer without resetting the animation.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetBlendSpacePosition(int32 TrackIndex, int32 LayerIndex, FVector2f Position);

	// Updates the manual position on a track layer without resetting the animation.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetManualPosition(int32 TrackIndex, int32 LayerIndex, float Position);

	// Sets the play rate on a track layer. Works for auto-play and blend space modes.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetPlayRate(int32 TrackIndex, int32 LayerIndex, float PlayRate);

	// Returns the play rate for a track layer, or 0.0 for manual mode, or 1.0 if invalid.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API float GetPlayRate(int32 TrackIndex, int32 LayerIndex) const;

	// Sets the loop mode on a track layer.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API bool SetLoopMode(int32 TrackIndex, int32 LayerIndex, EAnimSequenceTrackLoopMode LoopMode);

	// Returns the loop mode for a track layer.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API EAnimSequenceTrackLoopMode GetLoopMode(int32 TrackIndex, int32 LayerIndex) const;

	// Returns the current playback position state for a track layer. Source/target blend info requires bEnableBlendTracking.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API FAnimSequenceTrackPosition GetPosition(int32 TrackIndex, int32 LayerIndex) const;

	// Returns the 2D blend space position for a track layer, or zero if not in blend space mode.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API FVector2f GetBlendSpacePosition(int32 TrackIndex, int32 LayerIndex) const;

	// Returns the sequence index for a track layer. Returns INDEX_NONE for blend space mode.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API int32 GetSequenceIndex(int32 TrackIndex, int32 LayerIndex) const;

	// Returns the current playback mode for a track layer.
	UFUNCTION(BlueprintCallable, Category = "TransformProviders|AnimSequence")
	ENGINE_API EAnimSequenceTrackMode GetTrackMode(int32 TrackIndex, int32 LayerIndex) const;

private:
	void MarkOwnerRenderStateDirty();
	void InitLayerPools(bool bMarkDirty = true);
	void SetParentProvider(UAnimSequenceTransformProviderData* ProviderData);
	bool IsValidTrackAndLayer(int32 TrackIndex, int32 LayerIndex) const;

	UPROPERTY()
	TObjectPtr<UAnimSequenceTransformProviderData> ParentProvider;

	UInstancedSkinnedMeshComponent* CachedOwnerComponent = nullptr;
	TArray<FAnimSequenceTrackPool> TrackPools;
	TUniquePtr<FAnimSequenceTransformProviderRenderData> RenderData{ new FAnimSequenceTransformProviderRenderData };
	FRenderCommandFence InstanceDestroyFence;
	UE::FMutex SubmitChangesMutex;

	class FPlaybackTracker;
	TPimplPtr<FPlaybackTracker> PlaybackTracker;

	// Per-blend-space resolution data (one per BlendSpaces entry)
	struct FBlendSpaceCache
	{
		TArray<uint16> SampleToSequenceIndex;
		bool bValid = false;
	};
	TArray<FBlendSpaceCache> BlendSpaceCaches;
	void BuildBlendSpaceCaches();
	void ResolveBlendSpaceSamples(FAnimSequenceTrackPackedData& Data) const;
	bool bTrackPoolsDirty = false;
	bool bPendingRenderStateDirty = false;
	bool bNeedsPostLoadInit = false;

	friend class UAnimSequenceTransformProviderData;
	friend class UAnimSequenceTransformProviderLayerStack;
};