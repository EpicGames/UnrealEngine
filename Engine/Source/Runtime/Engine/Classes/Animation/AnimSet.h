// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * This is a set of AnimSequences
 * All sequence have the same number of tracks, and they relate to the same bone names.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AnimSet.generated.h"

class UAnimSequence;
class USkeletalMesh;

/** This is a mapping table between each bone in a particular skeletal mesh and the tracks of this animation set. */
USTRUCT()
struct FAnimSetMeshLinkup
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Mapping table. Size must be same as size of SkelMesh reference skeleton. 
	 * No index should be more than the number of tracks in this AnimSet.
	 * -1 indicates no track for this bone - will use reference pose instead.
	 */
	UPROPERTY()
	TArray<int32> BoneToTrackTable;



		/** Reset this linkup and re-create between the provided skeletal mesh and anim set. */
		void BuildLinkup(USkeletalMesh* InSkelMesh, class UAnimSet* InAnimSet);
	
};

UCLASS(hidecategories=Object, MinimalAPI)
class UAnimSet : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	/** 
	 *	Indicates that only the rotation should be taken from the animation sequence and the translation should come from the USkeletalMesh ref pose. 
	 *	Note that the root bone always takes translation from the animation, even if this flag is set.
	 *	You can use the UseTranslationBoneNames array to specify other bones that should use translation with this flag set.
	 */
	UPROPERTY(EditAnywhere, Category=AnimSet)
	uint32 bAnimRotationOnly:1;

public:
	/** Bone name that each track relates to. TrackBoneName.Num() == Number of tracks. */
	UPROPERTY()
	TArray<FName> TrackBoneNames;

#if WITH_EDITORONLY_DATA
	/** Actual animation sequence information. */
	UPROPERTY()
	TArray<TObjectPtr<class UAnimSequence>> Sequences;

#endif // WITH_EDITORONLY_DATA
private:
	/** Non-serialised cache of linkups between different skeletal meshes and this AnimSet. */
	// Do not change private - they will go away
	UPROPERTY(transient)
	TArray<struct FAnimSetMeshLinkup> LinkupCache;

	/** 
	 *	Array of booleans that indicate whether or not to read the translation of a bone from animation or ref skeleton.
	 *	This is basically a cooked down version of UseTranslationBoneNames for speed.
	 *	Size matches the number of tracks.
	 */
	// Do not change private - they will go away
	UPROPERTY(transient)
	TArray<uint8> BoneUseAnimTranslation;

	/** Cooked down version of ForceMeshTranslationBoneNames */
	// Do not change private - they will go away
	UPROPERTY(transient)
	TArray<uint8> ForceUseMeshTranslation;

	/** Names of bones that should use translation from the animation, if bAnimRotationOnly is set. */
	// Do not change private - they will go away
	UPROPERTY(EditAnywhere, Category=AnimSet)
	TArray<FName> UseTranslationBoneNames;

	/** List of bones which are ALWAYS going to use their translation from the mesh and not the animation. */
	// Do not change private - they will go away
	UPROPERTY(EditAnywhere, Category=AnimSet)
	TArray<FName> ForceMeshTranslationBoneNames;

public:
	/** In the AnimSetEditor, when you switch to this AnimSet, it sees if this skeletal mesh is loaded and if so switches to it. */
	UPROPERTY()
	FName PreviewSkelMeshName;

	/** Holds the name of the skeletal mesh whose reference skeleton best matches the TrackBoneName array. */
	UPROPERTY()
	FName BestRatioSkelMeshName;

	/**
	 * Find a mesh linkup table (mapping of sequence tracks to bone indices) for a particular SkeletalMesh
	 * If one does not already exist, create it now.
	 *
	 * @param SkelMesh SkeletalMesh to look for linkup with.
	 *
	 * @return Index of Linkup between mesh and animation set.
	 */
	virtual int32 GetMeshLinkupIndex(class USkeletalMesh* SkelMesh);

public:
	/** Runtime built mapping table between SkeletalMeshes, and LinkupCache array indices. */
	// Do change private - they will go away
	TMap<FName,int32> SkelMesh2LinkupCache;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin UAnimSet Interface
	/**
	 * See if we can play sequences from this AnimSet on the provided USkeletalMesh.
	 * Returns true if there is a bone in SkelMesh for every track in the AnimSet,
	 * or there is a track of animation for every bone of the SkelMesh.
	 * 
	 * @param	SkelMesh	USkeletalMesh to compare the AnimSet against.
	 * @return				true if animation set can play on supplied USkeletalMesh, false if not.
	 */
	bool CanPlayOnSkeletalMesh(USkeletalMesh* SkelMesh) const;

	/** Get Ratio of how much that mesh fits that animation set */
	float GetSkeletalMeshMatchRatio(USkeletalMesh* SkelMesh) const;

	/**
	 * Returns the AnimSequence with the specified name in this set.
	 * 
	 * @param		SequenceName	Name of sequence to find.
	 * @return						Pointer to AnimSequence with desired name, or NULL if sequence was not found.
	 */
	UAnimSequence* FindAnimSequence(FName SequenceName);

	/**
	 * @return		The track index for the bone with the supplied name, or INDEX_NONE if no track exists for that bone.
	 */
	int32 FindTrackWithName(FName BoneName) const
	{
		return TrackBoneNames.Find( BoneName );
	}

	/**
	 * Clears all sequences and resets the TrackBoneNames table.
	 */
	void ResetAnimSet();

	/** 
	 * Properly remove an AnimSequence from an AnimSet, and updates references it might have.
	 * @return true if AnimSequence was properly removed, false if it wasn't found.
	 */
	bool RemoveAnimSequenceFromAnimSet(UAnimSequence* AnimSeq);

	/** Util that finds all AnimSets and flushes their LinkupCache, then calls InitAnimTree on all SkeletalMeshComponents. */
	static void ClearAllAnimSetLinkupCaches();

	friend struct FAnimSetMeshLinkup;
};



