// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This Instance only contains one AnimationAsset, and produce poses
 * Used by Preview in AnimGraph, Playing single animation in Kismet2 and etc
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Optional.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTypes.h"
#include "AnimSingleNodeInstance.generated.h"

struct FAnimInstanceProxy;

DECLARE_DYNAMIC_DELEGATE(FPostEvaluateAnimEvent);

UCLASS(transient, NotBlueprintable, MinimalAPI)
class UAnimSingleNodeInstance : public UAnimInstance
{
	GENERATED_UCLASS_BODY()

	// Disable compiler-generated deprecation warnings by implementing our own destructor
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~UAnimSingleNodeInstance() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Current Asset being played **/
	UPROPERTY(Transient)
	TObjectPtr<class UAnimationAsset> CurrentAsset;
	 
	UPROPERTY(Transient)
	FPostEvaluateAnimEvent PostEvaluateAnimEvent;

	//~ Begin UAnimInstance Interface
	ENGINE_API virtual void NativeInitializeAnimation() override;
	ENGINE_API virtual void NativePostEvaluateAnimation() override;
	ENGINE_API virtual void OnMontageInstanceStopped(FAnimMontageInstance& StoppedMontageInstance) override;

protected:
	ENGINE_API virtual void Montage_Advance(float DeltaTime) override;
	//~ End UAnimInstance Interface
public:
	UFUNCTION(BlueprintCallable, Category="Animation")
    ENGINE_API void SetMirrorDataTable(const UMirrorDataTable* MirrorDataTable);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API const UMirrorDataTable* GetMirrorDataTable();
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetLooping(bool bIsLooping);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetPlayRate(float InPlayRate);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetReverse(bool bInReverse);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetPosition(float InPosition, bool bFireNotifies=true);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetPositionWithPreviousTime(float InPosition, float InPreviousTime, bool bFireNotifies=true);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetBlendSpacePosition(const FVector& InPosition);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void SetPlaying(bool bIsPlaying);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API float GetLength();
	/* For AnimSequence specific **/
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void PlayAnim(bool bIsLooping=false, float InPlayRate=1.f, float InStartPosition=0.f);
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API void StopAnim();
	/** Set New Asset - calls InitializeAnimation, for now we need MeshComponent **/
	UFUNCTION(BlueprintCallable, Category="Animation")
	ENGINE_API virtual void SetAnimationAsset(UAnimationAsset* NewAsset, bool bIsLooping=true, float InPlayRate=1.f);
	/** Get the currently used asset */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API virtual UAnimationAsset* GetAnimationAsset() const;
	/** Set pose value */
 	UFUNCTION(BlueprintCallable, Category = "Animation")
 	ENGINE_API void SetPreviewCurveOverride(const FName& PoseName, float Value, bool bRemoveIfZero);

public:
	/** Gets the current state of any BlendSpace */
	ENGINE_API void GetBlendSpaceState(FVector& OutPosition, FVector& OutFilteredPosition) const;

	/** AnimSequence specific **/
	ENGINE_API void StepForward();
	ENGINE_API void StepBackward();

	/** custom evaluate pose **/
	ENGINE_API virtual void RestartMontage(UAnimMontage * Montage, FName FromSection = FName());
	ENGINE_API void SetMontageLoop(UAnimMontage* Montage, bool bIsLooping, FName StartingSection = FName());

	/** Set the montage slot to preview */
	ENGINE_API void SetMontagePreviewSlot(FName PreviewSlot);

	/** Updates montage weights based on a jump in time (as this wont be handled by SetPosition) */
	ENGINE_API void UpdateMontageWeightForTimeSkip(float TimeDifference);

	/** Updates the blendspace samples list in the case of our asset being a blendspace */
	ENGINE_API void UpdateBlendspaceSamples(FVector InBlendInput);

	/** Check whether we are currently playing */
	ENGINE_API bool IsPlaying() const;

	/** Check whether we are currently playing in reverse */
	ENGINE_API bool IsReverse() const;

	/** Check whether we are currently looping */
	ENGINE_API bool IsLooping() const;

	/** Get the current playback time */
	ENGINE_API float GetCurrentTime() const;

	/** Get the current play rate multiplier */
	ENGINE_API float GetPlayRate() const;

	/** Get the currently playing asset. Can return NULL */
	ENGINE_API UAnimationAsset* GetCurrentAsset();

	/** Get the last filter output */
	ENGINE_API FVector GetFilterLastOutput();

	/** Set animation interpolation type override. If not set, it will not override. */
	ENGINE_API void SetInterpolationOverride(TOptional<EAnimInterpolationType> InterpolationType);

	/** Get animation interpolation type override. If not set, it will not override. */
 	ENGINE_API TOptional<EAnimInterpolationType> GetInterpolationOverride() const;


protected:
	ENGINE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
};



