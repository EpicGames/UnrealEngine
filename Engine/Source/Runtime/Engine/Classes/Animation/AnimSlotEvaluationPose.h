// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimTypes.h"
#include "Animation/AttributesRuntime.h"
#include "BonePose.h"
#include "AnimSlotEvaluationPose.generated.h"

/** Helper struct for Slot node pose evaluation. */
USTRUCT()
struct FSlotEvaluationPose
{
	GENERATED_USTRUCT_BODY()

		/** Type of additive for pose */
		UPROPERTY()
		TEnumAsByte<EAdditiveAnimationType> AdditiveType;

	/** Weight of pose */
	UPROPERTY()
		float Weight;

	/*** ATTENTION *****/
	/* These Pose/Curve is stack allocator. You should not use it outside of stack. */
	FCompactPose Pose;
	FBlendedCurve Curve;
	UE::Anim::FStackAttributeContainer Attributes;

	FSlotEvaluationPose()
		: AdditiveType(AAT_None)
		, Weight(0.0f)
	{
	}

	FSlotEvaluationPose(float InWeight, EAdditiveAnimationType InAdditiveType)
		: AdditiveType(InAdditiveType)
		, Weight(InWeight)
	{
	}

	FSlotEvaluationPose(FSlotEvaluationPose&& InEvaluationPose)
		: AdditiveType(InEvaluationPose.AdditiveType)
		, Weight(InEvaluationPose.Weight)
	{
		Pose.MoveBonesFrom(InEvaluationPose.Pose);
		Curve.MoveFrom(InEvaluationPose.Curve);
		Attributes.MoveFrom(InEvaluationPose.Attributes);
	}

	FSlotEvaluationPose(const FSlotEvaluationPose& InEvaluationPose) = default;
	FSlotEvaluationPose& operator=(const FSlotEvaluationPose& InEvaluationPose) = default;
};
