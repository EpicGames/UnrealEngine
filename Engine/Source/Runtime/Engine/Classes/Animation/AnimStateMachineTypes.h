// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AlphaBlend.h"
#include "BlendProfile.h"
#include "AnimStateMachineTypes.generated.h"

class UCurveFloat;
class UAnimStateTransitionNode;

UENUM(BlueprintType)
enum class ETransitionRequestQueueMode : uint8
{
	Shared			UMETA(ToolTip = "Only one transition can handle this request"),
	Unique			UMETA(ToolTip = "Allows multiple transitions to handle the same request"),
};

UENUM(BlueprintType)
enum class ETransitionRequestOverwriteMode : uint8
{
	Append			UMETA(ToolTip = "This request is added whether or not another with the same name is already queued"),
	Ignore			UMETA(ToolTip = "This request is ignored if another request with the same name is already queued"),
	Overwrite		UMETA(ToolTip = "This request overwrites another request with the same name if one exists")
};

//@TODO: Document
UENUM()
namespace ETransitionBlendMode
{
	enum Type : int
	{
		TBM_Linear UMETA(DisplayName="Linear"),
		TBM_Cubic UMETA(DisplayName="Cubic")
	};
}

//@TODO: Document
UENUM()
namespace ETransitionLogicType
{
	enum Type : int
	{
		TLT_StandardBlend UMETA(DisplayName="Standard Blend", ToolTip="Blend smoothly from source state to destination state.\nBoth states update during the transition.\nFalls back to Inertialization on re-entry to an already active state when Fall Back to Inertialization is true."),
		TLT_Inertialization UMETA(DisplayName = "Inertialization", ToolTip="Use inertialization to extrapolate when blending between states.\nOnly one state is active at a time.\nRequires an Inertialization or Dead Blending node rootwards of this node in the graph."),
		TLT_Custom UMETA(DisplayName="Custom", ToolTip="Use a custom graph to define exactly how the blend works.")
	};
}

struct FTransitionEvent
{
	TArray<int32, TInlineAllocator<8>> ConsumedTransitions;
	double CreationTime;
	double TimeToLive;
	FName EventName;
	ETransitionRequestQueueMode QueueMode;
	ETransitionRequestOverwriteMode OverwriteMode;

	FTransitionEvent(const FName& InEventName, const double InTimeToLive, const ETransitionRequestQueueMode& InQueueMode, const ETransitionRequestOverwriteMode& InOverwriteMode)
		: TimeToLive(InTimeToLive)
		, EventName(InEventName)
		, QueueMode(InQueueMode)
		, OverwriteMode(InOverwriteMode)
	{
		CreationTime = FPlatformTime::Seconds();
	}

	bool IsValidRequest() const
	{
		return TimeToLive > 0.0;
	}

	double GetRemainingTime() const
	{
		return TimeToLive - (FPlatformTime::Seconds() - CreationTime);
	}

	bool HasExpired() const
	{
		return GetRemainingTime() <= 0.0;
	}

	bool ToBeConsumed() const
	{
		if (QueueMode == ETransitionRequestQueueMode::Shared && ConsumedTransitions.Num() > 0)
		{
			return true;
		}
		return false;
	}

	bool HasBeenHandled() const
	{
		return ConsumedTransitions.Num() > 0;
	}

	FString ToDebugString() const
	{
		FString HandledByString = *FString::JoinBy(ConsumedTransitions, TEXT(", "), [](const int32& TransitionIndex) { return FString::Printf(TEXT("%d"), TransitionIndex); });
		return FString::Printf(TEXT("%s (%.2fs) [Handled by: %s]"), *EventName.ToString(), GetRemainingTime(), *HandledByString);
	}
};

// This structure represents a baked transition rule inside a state
USTRUCT()
struct FAnimationTransitionRule
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName RuleToExecute;

	/** What RuleToExecute must return to take transition (for bidirectional transitions) */
	UPROPERTY()
	bool TransitionReturnVal;

	UPROPERTY()
	int32 TransitionIndex;

	FAnimationTransitionRule()
		: TransitionReturnVal(true)
		, TransitionIndex(INDEX_NONE)
	{}

	FAnimationTransitionRule(int32 InTransitionState)
		: TransitionIndex(InTransitionState)
	{
	}
};

// This is the base class that both baked states and transitions use
USTRUCT()
struct FAnimationStateBase
{
	GENERATED_USTRUCT_BODY()

	// The name of this state
	UPROPERTY()
	FName StateName;

	FAnimationStateBase()
	{}
};

//
USTRUCT()
struct FAnimationState : public FAnimationStateBase
{
	GENERATED_USTRUCT_BODY()

	// Set of legal transitions out of this state; already in priority order
	UPROPERTY()
	TArray<FAnimationTransitionRule> Transitions;

	// The root node index (into the AnimNodeProperties array of the UAnimBlueprintGeneratedClass)
	UPROPERTY()
	int32 StateRootNodeIndex;

	// The index of the notify to fire when this state is first entered (weight within the machine becomes non-zero)
	UPROPERTY()
	int32 StartNotify;

	// The index of the notify to fire when this state is finished exiting (weight within the machine becomes zero)
	UPROPERTY()
	int32 EndNotify;

	// The index of the notify to fire when this state is fully entered (weight within the machine becomes one)
	UPROPERTY()
	int32 FullyBlendedNotify;
	
	FAnimationState()
		: FAnimationStateBase()
		, StateRootNodeIndex(INDEX_NONE)
		, StartNotify(INDEX_NONE)
		, EndNotify(INDEX_NONE)
		, FullyBlendedNotify(INDEX_NONE)
	{}
};

// This represents a baked transition
USTRUCT()
struct FAnimationTransitionBetweenStates : public FAnimationStateBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UCurveFloat> CustomCurve;

	UPROPERTY()
	TObjectPtr<UBlendProfile> BlendProfile;
	
	// Transition-only: State being transitioned from
	UPROPERTY()
	int32 PreviousState;

	// Transition-only: State being transitioned to
	UPROPERTY()
	int32 NextState;

	UPROPERTY()
	float CrossfadeDuration;

	UPROPERTY()
	float MinTimeBeforeReentry;

	UPROPERTY()
	int32 StartNotify;

	UPROPERTY()
	int32 EndNotify;

	UPROPERTY()
	int32 InterruptNotify;

	UPROPERTY()
	EAlphaBlendOption BlendMode;

	UPROPERTY()
	TEnumAsByte<ETransitionLogicType::Type> LogicType;

	UPROPERTY()
	uint8 bAllowInertializationForSelfTransitions : 1;

#if WITH_EDITORONLY_DATA
	// This is only needed for the baking process, to denote which baked transitions need to reverse their prev/next state in the final step
	uint8 ReverseTransition : 1;
#endif

	FAnimationTransitionBetweenStates()
		: FAnimationStateBase()
		, CustomCurve(nullptr)
		, BlendProfile(nullptr)
		, PreviousState(INDEX_NONE)
		, NextState(INDEX_NONE)
		, CrossfadeDuration(-1.0f)
		, MinTimeBeforeReentry(-1.0f)
		, StartNotify(INDEX_NONE)
		, EndNotify(INDEX_NONE)
		, InterruptNotify(INDEX_NONE)
		, BlendMode(EAlphaBlendOption::CubicInOut)
		, LogicType(ETransitionLogicType::TLT_StandardBlend)
		, bAllowInertializationForSelfTransitions(false)
#if WITH_EDITOR
		, ReverseTransition(false)
#endif
	{}
};


USTRUCT()
struct FBakedStateExitTransition
{
	GENERATED_USTRUCT_BODY()

	// The node property index for this rule
	UPROPERTY()
	int32 CanTakeDelegateIndex;

	// The blend graph result node index
	UPROPERTY()
	int32 CustomResultNodeIndex;

	// The index into the machine table of transitions
	UPROPERTY()
	int32 TransitionIndex;

	// What the transition rule node needs to return to take this transition (for bidirectional transitions)
	UPROPERTY()
	bool bDesiredTransitionReturnValue;

	// Automatic Transition Rule based on animation remaining time.
	UPROPERTY()
	bool bAutomaticRemainingTimeRule;

	// If true, this transition rule is skipped when the state machine's update context is inactive (i.e. when blending out).
	UPROPERTY()
	bool bOnlyEvaluateWhenActive;

	// Automatic Transition Rule triggering time:
	//  < 0 means trigger the transition 'Crossfade Duration' seconds before the end of the asset player, so a standard blend would finish just as the asset player ends
	// >= 0 means trigger the transition 'Automatic Rule Trigger Time' seconds before the end of the asset player
	UPROPERTY()
	float AutomaticRuleTriggerTime;

	// Additional rule around SyncGroup requiring Valid Markers
	UPROPERTY()
	FName SyncGroupNameToRequireValidMarkersRule;

	UPROPERTY()
	TArray<int32> PoseEvaluatorLinks;

	FBakedStateExitTransition()
		: CanTakeDelegateIndex(INDEX_NONE)
		, CustomResultNodeIndex(INDEX_NONE)
		, TransitionIndex(INDEX_NONE)
		, bDesiredTransitionReturnValue(true)
		, bAutomaticRemainingTimeRule(false)
		, bOnlyEvaluateWhenActive(false)
		, AutomaticRuleTriggerTime(-1.f)
		, SyncGroupNameToRequireValidMarkersRule(NAME_None)
	{
	}
};


//
USTRUCT()
struct FBakedAnimationState
{
	GENERATED_USTRUCT_BODY()

	// Indices into the property array for player nodes in the state
	UPROPERTY()
	TArray<int32> PlayerNodeIndices;

	// Indices into the property array for layer nodes in the state
	UPROPERTY()
	TArray<int32> LayerNodeIndices;
	
	// Set of legal transitions out of this state; already in priority order
	UPROPERTY()
	TArray<FBakedStateExitTransition> Transitions;

	// The name of this state
	UPROPERTY()
	FName StateName;

	// The root node index (into the AnimNodeProperties array of the UAnimBlueprintGeneratedClass)
	UPROPERTY()
	int32 StateRootNodeIndex;

	UPROPERTY()
	int32 StartNotify;

	UPROPERTY()
	int32 EndNotify;

	UPROPERTY()
	int32 FullyBlendedNotify;
	
	UPROPERTY()
	int32 EntryRuleNodeIndex;

	// Whether or not this state will ALWAYS reset it's state on reentry, regardless of remaining weight
	UPROPERTY()
	bool bAlwaysResetOnEntry;

	UPROPERTY()
	bool bIsAConduit;

public:
	FBakedAnimationState()
		: StateRootNodeIndex(INDEX_NONE)
		, StartNotify(INDEX_NONE)
		, EndNotify(INDEX_NONE)
		, FullyBlendedNotify(INDEX_NONE)
		, EntryRuleNodeIndex(INDEX_NONE)
		, bAlwaysResetOnEntry(false)
		, bIsAConduit(false)
	{}
};

USTRUCT()
struct FBakedAnimationStateMachine
{
	GENERATED_USTRUCT_BODY()

	// Name of this machine (primarily for debugging purposes)
	UPROPERTY()
	FName MachineName;

	// Index of the initial state that the machine will start in
	UPROPERTY()
	int32 InitialState;

	// List of all states this machine can be in
	UPROPERTY()
	TArray<FBakedAnimationState> States;

	// List of all transitions between states
	UPROPERTY()
	TArray<FAnimationTransitionBetweenStates> Transitions;

	// Cached StatID for this state machine
	STAT(mutable TStatId StatID;)

public:
	FBakedAnimationStateMachine()
		: InitialState(INDEX_NONE)
	{}

	// Finds a state by name or INDEX_NONE if no such state exists
	ENGINE_API int32 FindStateIndex(const FName& StateName) const;

	// Find the index of a transition from StateNameFrom to StateNameTo
	ENGINE_API int32 FindTransitionIndex(const FName& InStateNameFrom, const FName& InStateNameTo) const;
	ENGINE_API int32 FindTransitionIndex(const int32 InStateIdxFrom, const int32 InStateIdxTo) const;

#if STATS
	/** Get the StatID for timing this state machine */
	inline TStatId GetStatID() const
	{
		if (!StatID.IsValidStat())
		{
			StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Anim>(MachineName);
		}
		return StatID;
	}
#endif // STATS
};

UCLASS()
class UAnimStateMachineTypes : public UObject
{
	GENERATED_UCLASS_BODY()
};

