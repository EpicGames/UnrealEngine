// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Animation that can be streamed instead of being loaded completely
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Serialization/BulkData.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimCompress.h"
#include "Animation/AnimCompressionTypes.h"
#include "Animation/AnimSequenceBase.h"
#include "AnimStreamable.generated.h"

class UAnimSequence;
class UAnimCompress;
struct FCompactPose;

class FAnimStreamableChunk
{
public:
	FAnimStreamableChunk() : StartTime(0.f), SequenceLength(0.f), CompressedAnimSequence(nullptr) {}
	~FAnimStreamableChunk()
	{
		if (CompressedAnimSequence)
		{
			delete CompressedAnimSequence;
			CompressedAnimSequence = nullptr;
		}
	}

	float StartTime;

	float SequenceLength;
	int32 NumFrames;

	// Compressed Data for this chunk (if nullptr then data needs to be loaded via BulkData)
	FCompressedAnimSequence* CompressedAnimSequence;

	// Bulk data if stored in the package.
	FByteBulkData BulkData;

	SIZE_T GetMemorySize() const
	{
		static const SIZE_T ClassSize = sizeof(FAnimStreamableChunk);
		SIZE_T CurrentSize = ClassSize;

		if (CompressedAnimSequence)
		{
			CurrentSize += CompressedAnimSequence->GetMemorySize();
		}
		return CurrentSize;
	}

	/** Serialization. */
	void Serialize(FArchive& Ar, UAnimStreamable* Owner, int32 ChunkIndex);
};

class FStreamableAnimPlatformData
{
public:
	TArray<FAnimStreamableChunk> Chunks;

	void Serialize(FArchive& Ar, class UAnimStreamable* Owner);

	void Reset()
	{
		Chunks.Reset();
	}

	SIZE_T GetMemorySize() const
	{
		SIZE_T ChunkSize = 0;
		for (const FAnimStreamableChunk& Chunk : Chunks)
		{
			ChunkSize += Chunk.GetMemorySize();
		}
		return sizeof(FStreamableAnimPlatformData) + ChunkSize;
	}
};


UCLASS(config=Engine, hidecategories=UObject, MinimalAPI, BlueprintType)
class UAnimStreamable : public UAnimSequenceBase
{
	GENERATED_UCLASS_BODY()

public:
	/** The number of keys expected within the individual animation tracks. */
	UPROPERTY(AssetRegistrySearchable)
	int32 NumberOfKeys;

	/** This defines how values between keys are calculated **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Animation)
	EAnimInterpolationType Interpolation;

	/** Base pose to use when retargeting */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Animation)
	FName RetargetSource;

	UPROPERTY()
	FFrameRate SamplingFrameRate;

#if WITH_EDITORONLY_DATA

	// Sequence the streamable was created from (used for reflecting changes to the source in editor)
	UPROPERTY()
	TObjectPtr<const UAnimSequence> SourceSequence;

	UPROPERTY()
	FGuid RawDataGuid;

	/** Number of raw frames in this sequence (not used by engine - just for informational purposes). */
	UE_DEPRECATED(5.0, "Num Frames is deprecated use NumberOfKeys instead")
	UPROPERTY()
	int32 NumFrames;

	/**
	 * Raw uncompressed keyframe data.
	 */
	UPROPERTY()
	TArray<struct FRawAnimSequenceTrack> RawAnimationData;

	/**
	 * In the future, maybe keeping RawAnimSequenceTrack + TrackMap as one would be good idea to avoid inconsistent array size
	 * TrackToSkeletonMapTable(i) should contains  track mapping data for RawAnimationData(i).
	 */
	UPROPERTY()
	TArray<struct FTrackToSkeletonMap> TrackToSkeletonMapTable;

	/**
	 * This is name of RawAnimationData tracks for editoronly - if we lose skeleton, we'll need relink them
	 */
	UPROPERTY()
	TArray<FName> AnimationTrackNames;

	// Editor can have multiple platforms loaded at once
	TMap<const ITargetPlatform*, FStreamableAnimPlatformData*> StreamableAnimPlatformData;

	FStreamableAnimPlatformData* RunningAnimPlatformData;
#else

	// Non editor only has one set of platform data
	FStreamableAnimPlatformData RunningAnimPlatformData;
#endif

	bool HasRunningPlatformData() const
	{
#if WITH_EDITOR
		return RunningAnimPlatformData != nullptr;
#else
		return true;
#endif
	}

	FStreamableAnimPlatformData& GetRunningPlatformData()
	{
#if WITH_EDITOR
		check(RunningAnimPlatformData);
		return *RunningAnimPlatformData;
#else
		return RunningAnimPlatformData;
#endif
	}

	const FStreamableAnimPlatformData& GetRunningPlatformData() const
	{
#if WITH_EDITOR
		check(RunningAnimPlatformData);
		return *RunningAnimPlatformData;
#else
		return RunningAnimPlatformData;
#endif
	}

	/** The bone compression settings used to compress bones in this sequence. */
	UPROPERTY(Category = Compression, EditAnywhere)
	TObjectPtr<class UAnimBoneCompressionSettings> BoneCompressionSettings;

	/** The curve compression settings used to compress curves in this sequence. */
	UPROPERTY(Category = Compression, EditAnywhere)
	TObjectPtr<class UAnimCurveCompressionSettings> CurveCompressionSettings;

	/** The settings used to control whether or not to use variable frame stripping and its amount*/
	UPROPERTY(Category = Compression, EditAnywhere)
	TObjectPtr<class UVariableFrameStrippingSettings> VariableFrameStrippingSettings;

	/** If this is on, it will allow extracting of root motion **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = RootMotion, meta = (DisplayName = "EnableRootMotion"))
	bool bEnableRootMotion;

	/** Root Bone will be locked to that position when extracting root motion.**/
	UPROPERTY(EditAnywhere, Category = RootMotion)
	TEnumAsByte<ERootMotionRootLock::Type> RootMotionRootLock;

	/** Force Root Bone Lock even if Root Motion is not enabled */
	UPROPERTY(EditAnywhere, Category = RootMotion)
	bool bForceRootLock;

	/** If this is on, it will use a normalized scale value for the root motion extracted: FVector(1.0, 1.0, 1.0) **/
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = RootMotion, meta = (DisplayName = "Use Normalized Root Motion Scale"))
	bool bUseNormalizedRootMotionScale;

	//~ Begin UObject Interface
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void FinishDestroy() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

	//~ Begin UAnimSequenceBase Interface
	ENGINE_API virtual void HandleAssetPlayerTickedInternal(FAnimAssetTickContext &Context, const float PreviousTime, const float MoveDelta, const FAnimTickRecord &Instance, struct FAnimNotifyQueue& NotifyQueue) const override;
	virtual void GetAnimationPose(FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext) const override;
	virtual int32 GetNumberOfSampledKeys() const override { return NumberOfKeys; }
	//~ End UAnimSequenceBase Interface

#if WITH_EDITOR
	ENGINE_API void InitFrom(const UAnimSequence* InSourceSequence);
#endif

	ENGINE_API FStreamableAnimPlatformData& GetStreamingAnimPlatformData(const ITargetPlatform* Platform);

	ENGINE_API float GetChunkSizeSeconds(const ITargetPlatform* Platform) const;

	private:

#if WITH_EDITOR
	ENGINE_API void RequestCompressedData(const  ITargetPlatform* Platform=nullptr);

	void UpdateRawData();

	FString GetBaseDDCKey(uint32 NumChunks, const ITargetPlatform* TargetPlatform) const;

	void RequestCompressedDataForChunk(const FString& ChunkDDCKey, FAnimStreamableChunk& Chunk, const int32 ChunkIndex, const uint32 FrameStart, const uint32 FrameEnd, const ITargetPlatform* TargetPlatform);
#endif

	bool bUseRawDataOnly;
	int32 GetChunkIndexForTime(const TArray<FAnimStreamableChunk>& Chunks, const float CurrentTime) const;
};

