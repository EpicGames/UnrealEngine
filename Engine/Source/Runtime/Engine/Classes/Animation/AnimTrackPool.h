// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/Archive.h"
#include "Experimental/ConcurrentLinearAllocator.h"

/**
 * An pool / allocator class that manages a pool of tracks of type DataType. Supports allocation / deallocation / updates of track data, as well as
 * a mechanism to build diffs into a Patch data structure which can be marshaled and applied to another instance for thread-safe game / render updates.
 * Track are pool allocated and so only a subset may be active at one time.
 */
template <typename DataType>
class TAnimTrackPool
{
public:
	TAnimTrackPool() = default;
	TAnimTrackPool(const TAnimTrackPool&) = default;
	TAnimTrackPool(TAnimTrackPool&&) = default;
	TAnimTrackPool& operator=(const TAnimTrackPool&) = default;
	TAnimTrackPool& operator=(TAnimTrackPool&&) = default;

	template<typename SourceTrackDataType>
	TAnimTrackPool& operator=(const TAnimTrackPool<SourceTrackDataType>& Other)
	{
		const int32 NumTracks = Other.GetNumTracks();

		Datas.SetNum(NumTracks);
		ActiveBits.Init(false, NumTracks);
		DirtyBits.Init(false, NumTracks);
		NumActiveTracks = Other.GetNumActiveTracks();

		for (int32 Index = 0; Index < NumTracks; ++Index)
		{
			if (Other.IsActiveIndex(Index))
			{
				Datas[Index] = Other.GetData(Index);
				ActiveBits[Index] = true;
			}
		}

		return *this;
	}

	class FPatch
	{
	public:
		int32 GetNumTracks() const
		{
			return NumTracks;
		}

		struct FActivateOp
		{
			FActivateOp(int32 InIndex, const DataType& InData)
				: Index(InIndex)
				, Data(InData)
			{}

			uint32 Index;
			DataType Data;
		};

		struct FDeactivateOp
		{
			FDeactivateOp(int32 InIndex)
				: Index(InIndex)
			{}

			uint32 Index;
		};

		int32 NumTracks = 0;
		int32 NumActiveTracks = 0;
		TArray<FActivateOp, FConcurrentLinearArrayAllocator> ActivateOps;
		TArray<FDeactivateOp, FConcurrentLinearArrayAllocator> DeactivateOps;
	};

	void Reserve(int32 Num)
	{
		Datas.Reserve(Num);
		ActiveBits.Reserve(Num);
		DirtyBits.Reserve(Num);
	}

	// Initialize all active tracks from source but custom initialize each track.
	template <typename InitFunctionType>
	void InitFrom(const TAnimTrackPool& Source, InitFunctionType InitFunction)
	{
		Datas.SetNum(Source.Datas.Num());
		ActiveBits = Source.ActiveBits;
		NumActiveTracks = Source.NumActiveTracks;
		DirtyBits = Source.ActiveBits;
		NumDirtyTracks = Source.NumActiveTracks;

		for (TConstSetBitIterator<> BitIt(ActiveBits); BitIt; ++BitIt)
		{
			Datas[BitIt.GetIndex()] = InitFunction(BitIt.GetIndex());
		}
	}

	int32 AllocateTrack(const DataType& Data)
	{
		const int32 Index = ActiveBits.FindAndSetFirstZeroBit();
		NumActiveTracks++;

		if (Index != INDEX_NONE)
		{
			SetDirty(Index);
			Datas[Index] = Data;
			return Index;
		}

		Datas.Emplace(Data);
		ActiveBits.Add(true);
		DirtyBits.Add(true);
		NumDirtyTracks++;

		return Datas.Num() - 1;
	}

	int32 AllocateTrackAt(int32 Index, const DataType& Data)
	{
		if (Index < 0)
		{
			return INDEX_NONE;
		}

		if (Index >= Datas.Num())
		{
			const int32 OldNum = Datas.Num();
			const int32 NewNum = Index + 1;
			Datas.SetNum(NewNum);
			ActiveBits.Add(false, NewNum - OldNum);
			DirtyBits.Add(false, NewNum - OldNum);
		}

		if (ActiveBits[Index])
		{
			return INDEX_NONE;
		}

		ActiveBits[Index] = true;
		NumActiveTracks++;
		SetDirty(Index);
		Datas[Index] = Data;
		return Index;
	}

	bool UpdateTrack(int32 Index, const DataType& Data)
	{
		if (IsActiveIndex(Index))
		{
			SetDirty(Index);
			Datas[Index].SetData(Data);
			return true;
		}
		return false;
	}

	bool DeallocateTrack(int32 Index)
	{
		if (IsActiveIndex(Index))
		{
			SetDirty(Index);
			Datas[Index] = {};
			ActiveBits[Index] = false;
			NumActiveTracks--;
			return true;
		}
		return false;
	}
	
	struct FDefaultDeactivate
	{
		void operator()(DataType&, int32 /*Index*/) const
		{
		}
	};
	
	struct FDefaultAssign
	{
		void operator()(DataType& Out, const DataType& In) const
		{
			Out = In;
		}
	};

	struct FDefaultReset
	{
		void operator()(DataType& InOut) const
		{
		}
	};

	template <typename ResetFunctionType = FDefaultReset>
	FPatch Finalize(ResetFunctionType ResetFunction = FDefaultReset())
	{
		FPatch Patch;
		Patch.NumTracks = Datas.Num();
		Patch.NumActiveTracks = NumActiveTracks;

		// Early-out when nothing is dirty: callers still get authoritative metadata
		// (NumTracks / NumActiveTracks) without the cost of empty Reserve allocations
		// or a SetAllClean memset.
		if (NumDirtyTracks == 0)
		{
			return Patch;
		}

		Patch.DeactivateOps.Reserve(NumDirtyTracks);
		Patch.ActivateOps.Reserve(NumDirtyTracks);

		for (TConstSetBitIterator<> BitIt(DirtyBits); BitIt; ++BitIt)
		{
			const uint32 Index = BitIt.GetIndex();
			if (ActiveBits[Index])
			{
				Patch.ActivateOps.Emplace(Index, Datas[Index]);
				ResetFunction(Datas[Index]);
			}
			else
			{
				Patch.DeactivateOps.Emplace(Index);
			}
		}
		SetAllClean();

		return Patch;
	}

	template <typename SourceTrackData, typename AssignFunctionType, typename DeactivateFunctionType = FDefaultDeactivate>
	void Patch(
		typename TAnimTrackPool<SourceTrackData>::FPatch&& Patch,
		AssignFunctionType AssignFunction,
		DeactivateFunctionType DeactivateFunction = FDefaultDeactivate{})
	{
		using FSourcePatch = typename TAnimTrackPool<SourceTrackData>::FPatch;

		Datas.SetNum(Patch.NumTracks);
		ActiveBits.SetNum(Patch.NumTracks, false);
		DirtyBits.SetNum(Patch.NumTracks, false);
		NumActiveTracks = Patch.NumActiveTracks;

		for (typename FSourcePatch::FDeactivateOp Op : Patch.DeactivateOps)
		{
			DeactivateFunction(Datas[Op.Index], int32(Op.Index));
			ActiveBits[Op.Index] = false;
			SetDirty(Op.Index);
		}

		for (const typename FSourcePatch::FActivateOp& Op : Patch.ActivateOps)
		{
			auto Bit = ActiveBits[Op.Index];
			AssignFunction(Datas[Op.Index], Op.Data, Bit, int32(Op.Index));
			SetDirty(Op.Index);
			Bit = true;
		}
	}

	void Serialize(FArchive& Ar)
	{
		Ar << NumActiveTracks;
		Ar << Datas;
		Ar << ActiveBits;

		if (Ar.IsLoading())
		{
			DirtyBits.Init(false, GetNumTracks());
			NumDirtyTracks = 0;
		}
	}

	int32 GetNumTracks() const
	{
		return Datas.Num();
	}

	int32 GetNumActiveTracks() const
	{
		return NumActiveTracks;
	}

	int32 GetNumDirtyTracks() const
	{
		return NumDirtyTracks;
	}

	template <typename LambdaType>
	void EnumerateDirtyTracks(LambdaType&& Lambda)
	{
		for (TConstSetBitIterator<> BitIt(DirtyBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}

	template <typename LambdaType>
	void EnumerateDirtyTracks(LambdaType&& Lambda) const
	{
		for (TConstSetBitIterator<> BitIt(DirtyBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}

	template <typename LambdaType>
	void EnumerateActiveTracks(LambdaType&& Lambda)
	{
		for (TConstSetBitIterator<> BitIt(ActiveBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}

	template <typename LambdaType>
	void EnumerateActiveTracks(LambdaType&& Lambda) const
	{
		for (TConstSetBitIterator<> BitIt(ActiveBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}
	
	template <typename LambdaType>
	void EnumerateActiveDirtyTracks(LambdaType&& Lambda)
	{
		for (TConstDualSetBitIterator<> BitIt(ActiveBits, DirtyBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}

	template <typename LambdaType>
	void EnumerateActiveDirtyTracks(LambdaType&& Lambda) const
	{
		for (TConstDualSetBitIterator<> BitIt(ActiveBits, DirtyBits); BitIt; ++BitIt)
		{
			Lambda(BitIt.GetIndex());
		}
	}

	bool IsValidIndex(int32 Index) const
	{
		return Datas.IsValidIndex(Index);
	}

	bool IsActiveIndex(int32 Index) const
	{
		return IsValidIndex(Index) && ActiveBits[Index];
	}

	bool IsActiveIndexChecked(int32 Index) const
	{
		return ActiveBits[Index];
	}

	bool IsDirtyIndex(int32 Index) const
	{
		return IsValidIndex(Index) && DirtyBits[Index];
	}

	bool IsDirtyIndexChecked(int32 Index) const
	{
		return DirtyBits[Index];
	}
	
	DataType& GetData(int32 Index)
	{
		check(IsActiveIndex(Index));
		return Datas[Index];
	}

	const DataType& GetData(int32 Index) const
	{
		check(IsActiveIndex(Index));
		return Datas[Index];
	}

	void SetAllDirty()
	{
		NumDirtyTracks = DirtyBits.Num();
		DirtyBits.Init(true, NumDirtyTracks);
	}

	void SetDirty(int32 Index)
	{
		FBitReference Bit = DirtyBits[Index];
		NumDirtyTracks += Bit ? 0 : 1;
		Bit = true;
	}

	bool SetClean(int32 Index)
	{
		FBitReference Bit = DirtyBits[Index];
		bool bWasDirty = Bit;
		NumDirtyTracks -= bWasDirty ? 1 : 0;
		Bit = false;
		return bWasDirty;
	}

	void SetAllClean()
	{
		if (NumDirtyTracks > 0)
		{
			NumDirtyTracks = 0;
			DirtyBits.Init(false, DirtyBits.Num());
		}
	}

	void Empty()
	{
		Datas.Empty();
		ActiveBits.Empty();
		DirtyBits.Empty();
		NumDirtyTracks = 0;
		NumActiveTracks = 0;
	}

protected:
	TArray<DataType> Datas;
	TBitArray<> ActiveBits;
	TBitArray<> DirtyBits;
	int32 NumActiveTracks = 0;
	int32 NumDirtyTracks = 0;
};