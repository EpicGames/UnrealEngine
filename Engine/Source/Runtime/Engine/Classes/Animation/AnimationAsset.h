// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Abstract base class of animation assets that can be played back and evaluated to produce a pose.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "AnimInterpFilter.h"
#include "AnimEnums.h"
#include "Animation/AnimTraceDefines.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#if WITH_EDITOR
#include "IO/IoHash.h"
#endif //  WITH_EDITOR
#include "AnimationAsset.generated.h"

class UAnimMetaData;
class UAnimMontage;
class UAssetMappingTable;
class UAssetUserData;
class USkeleton;
class UAnimSequenceBase;
class UBlendSpace;
class UPoseAsset;
class UMirrorDataTable;
class USkeletalMesh;
class ITargetPlatform;
struct FAnimationUpdateContext;
struct FAnimInstanceProxy;

namespace SmartName
{
typedef uint16 UID_Type;
}

namespace UE { namespace Anim {
class IAnimNotifyEventContextDataInterface;
}}

namespace MarkerIndexSpecialValues
{
	enum Type
	{
		Uninitialized = -2,
		AnimationBoundary = -1,
	};
};

struct FMarkerPair
{
	int32 MarkerIndex;
	float TimeToMarker;

	FMarkerPair() : MarkerIndex(MarkerIndexSpecialValues::Uninitialized) {}
	FMarkerPair(int32 InMarkerIndex, float InTimeToMarker) : MarkerIndex(InMarkerIndex), TimeToMarker(InTimeToMarker) {}

	void Reset() { MarkerIndex = MarkerIndexSpecialValues::Uninitialized; }
};

struct FMarkerTickRecord
{
	//Current Position in marker space, equivalent to TimeAccumulator
	FMarkerPair PreviousMarker;
	FMarkerPair NextMarker;

	bool IsValid(bool bLooping) const
	{
		int32 Threshold = bLooping ? MarkerIndexSpecialValues::AnimationBoundary : MarkerIndexSpecialValues::Uninitialized;
		return PreviousMarker.MarkerIndex > Threshold && NextMarker.MarkerIndex > Threshold;
	}

	void Reset() { PreviousMarker.Reset(); NextMarker.Reset(); }

	/** Debug output function*/
	FString ToString() const
	{
		return FString::Printf(TEXT("[PreviousMarker Index/Time %i/%.2f, NextMarker Index/Time %i/%.2f]"), PreviousMarker.MarkerIndex, PreviousMarker.TimeToMarker, NextMarker.MarkerIndex, NextMarker.TimeToMarker);
	}
};

/**
 * Used when sampling a given animation asset, this structure will contain the previous frame's
 * internal sample time alongside the 'effective' delta time leading into the current frame.
 * 
 * An 'effective' delta time represents a value that has undergone all side effects present in the
 * corresponding asset's TickAssetPlayer call including but not limited to syncing, play rate 
 * adjustment, looping, etc.
 * 
 * For montages Delta isn't always abs(CurrentPosition-PreviousPosition) because a Montage can jump or repeat or loop
 */
struct FDeltaTimeRecord
{
public:
	void Set(float InPrevious, float InDelta)
	{
		Previous = InPrevious;
		Delta = InDelta;
		bPreviousIsValid = true;
	}
	void SetPrevious(float InPrevious) { Previous = InPrevious; bPreviousIsValid = true; }
	float GetPrevious() const { return Previous; }
	bool IsPreviousValid() const { return bPreviousIsValid; }

	float Delta = 0.f;
	
	FDeltaTimeRecord() = default;
	explicit FDeltaTimeRecord(float InDeltaTime) : Delta(InDeltaTime), Previous(0.f) {}
private:
	float Previous = 0.f;
	bool  bPreviousIsValid = false; // Will be set to true when Previous has been set
};

/** Transform definition */
USTRUCT(BlueprintType)
struct FBlendSampleData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 SampleDataIndex;

	UPROPERTY()
	TObjectPtr<class UAnimSequence> Animation;

	UPROPERTY()
	float TotalWeight;

	// Rate of change of the Weight - used in smoothed BlendSpace blends
	UPROPERTY()
	float WeightRate;

	UPROPERTY()
	float Time;

	UPROPERTY()
	float PreviousTime;

	// We may merge multiple samples if they use the same animation
	// Calculate the combined sample play rate here
	UPROPERTY()
	float SamplePlayRate;

	FDeltaTimeRecord DeltaTimeRecord;

	FMarkerTickRecord MarkerTickRecord;

	// transient per-bone interpolation data
	TArray<float> PerBoneBlendData;

	// transient per-bone weight rate - only allocated when used
	TArray<float> PerBoneWeightRate;

	FBlendSampleData()
		: SampleDataIndex(0)
		, Animation(nullptr)
		, TotalWeight(0.f)
		, WeightRate(0.f)
		, Time(0.f)
		, PreviousTime(0.f)
		, SamplePlayRate(0.0f)
		, DeltaTimeRecord()
	{
	}

	FBlendSampleData(int32 Index)
		: SampleDataIndex(Index)
		, Animation(nullptr)
		, TotalWeight(0.f)
		, WeightRate(0.f)
		, Time(0.f)
		, PreviousTime(0.f)
		, SamplePlayRate(0.0f)
		, DeltaTimeRecord()
	{
	}

	bool operator==( const FBlendSampleData& Other ) const 
	{
		// if same position, it's same point
		return (Other.SampleDataIndex== SampleDataIndex);
	}
	void AddWeight(float Weight)
	{
		TotalWeight += Weight;
	}

	float GetClampedWeight() const
	{
		return FMath::Clamp<float>(TotalWeight, 0.f, 1.f);
	}

	static void ENGINE_API NormalizeDataWeight(TArray<FBlendSampleData>& SampleDataList);
	static void ENGINE_API NormalizeDataWeight(const TArrayView<FBlendSampleData*>& SampleDataList);
};

USTRUCT()
struct FBlendFilter
{
	GENERATED_USTRUCT_BODY()

	TArray<FFIRFilterTimeBased> FilterPerAxis;

	FBlendFilter()
	{
	}
	
	FVector GetFilterLastOutput() const
	{
		return FVector(
			FilterPerAxis.Num() > 0 ? FilterPerAxis[0].LastOutput : 0.0f,
			FilterPerAxis.Num() > 1 ? FilterPerAxis[1].LastOutput : 0.0f,
			FilterPerAxis.Num() > 2 ? FilterPerAxis[2].LastOutput : 0.0f);
	}
};

/*
 * Pose Curve container for extraction
 * This is used by pose anim node
 * Saves Name/PoseIndex/Value of the curve
 */
struct FPoseCurve
{
	// The name of the curve
	FName				Name;
	// PoseIndex of pose asset it's dealing with
	// used to extract pose value fast
	int32				PoseIndex;
	// Curve Value
	float				Value;

	FPoseCurve()
		: Name(NAME_None)
		, PoseIndex(INDEX_NONE)
		, Value(0.f)
	{}

	FPoseCurve(int32 InPoseIndex, FName InName, float InValue)
		: Name(InName)
		, PoseIndex(InPoseIndex)
		, Value(InValue)
	{}
};

/** Animation Extraction Context */
struct FAnimExtractContext
{
	/** Position in animation to extract pose from */
	double CurrentTime;

	/** Is root motion being extracted? */
	bool bExtractRootMotion;

	/** Delta time range required for root motion extraction **/
	FDeltaTimeRecord DeltaTimeRecord;

	/** Is the current animation asset marked as looping? **/
	bool bLooping;

	/** 
	 * Pose Curve Values to extract pose from pose assets. 
	 * This is used by pose asset extraction 
	 */
	TArray<FPoseCurve> PoseCurves;

	/**
	 * The BonesRequired array is a list of bool flags to determine
	 * if a bone is required to be retrieved. This is currently used
	 * by several animation nodes to optimize evaluation time.
	 */
	TArray<bool> BonesRequired;

	/** 
	 * The optional interpolation mode override.
	 * If not set, it will simply use the interpolation mode provided by the asset.
	 * One example where this could be used is if you want to force sampling the animation with Step interpolation
	 * even when the animation sequence asset is set to Linear interpolation.
	 */
	TOptional<EAnimInterpolationType> InterpolationOverride;

#if WITH_EDITOR
	bool bIgnoreRootLock;

	// Experimental
	bool bExtractWithRootMotionProvider;

	// Flag when set will enforce evaluation/sampling of, specific, compressed animation data, will assert if it is missing (user is responsible for ensuring the data was previously requested/resident)
	bool bEnforceCompressedDataSampling = false;
	// Used in combination with bForceSampleCompressedData, forcing a specific target platform hash with corresponding compressed data to sample (retrieved using UAnimSequence::GetDerivedDataKeyHash) 
	FIoHash TargetPlatformHash = FIoHash::Zero;
	// Used in combination with bForceSampleCompressedData, forcing a specific target platform its compressed data to sample
	const ITargetPlatform* TargetPlatform = nullptr;
#endif 
	
	FAnimExtractContext(double InCurrentTime = 0.0, bool InbExtractRootMotion = false, FDeltaTimeRecord InDeltaTimeRecord = {}, bool InbLooping = false)
		: CurrentTime(InCurrentTime)
		, bExtractRootMotion(InbExtractRootMotion)
		, DeltaTimeRecord(InDeltaTimeRecord)
		, bLooping(InbLooping)
		, PoseCurves()
		, BonesRequired()
		, InterpolationOverride()
#if WITH_EDITOR
		, bIgnoreRootLock(false)
		, bExtractWithRootMotionProvider(true)
#endif 
	{
	}

	bool IsBoneRequired(int32 BoneIndex) const
	{
		if (BoneIndex >= BonesRequired.Num())
		{
			return true;
		}

		return BonesRequired[BoneIndex];
	}
};

//Represent a current play position in an animation
//based on sync markers
USTRUCT(BlueprintType)
struct FMarkerSyncAnimPosition
{
	GENERATED_USTRUCT_BODY()

	/** The marker we have passed*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sync)
	FName PreviousMarkerName;

	/** The marker we are heading towards */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sync)
	FName NextMarkerName;

	/** Value between 0 and 1 representing where we are:
	0   we are at PreviousMarker
	1   we are at NextMarker
	0.5 we are half way between the two */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sync)
	float PositionBetweenMarkers;

	/** Is this a valid Marker Sync Position */
	bool IsValid() const { return (PreviousMarkerName != NAME_None && NextMarkerName != NAME_None); }

	FMarkerSyncAnimPosition()
		: PositionBetweenMarkers(0.0f)
	{}

	FMarkerSyncAnimPosition(const FName& InPrevMarkerName, const FName& InNextMarkerName, const float& InAlpha)
		: PreviousMarkerName(InPrevMarkerName)
		, NextMarkerName(InNextMarkerName)
		, PositionBetweenMarkers(InAlpha)
	{}

	/** Debug output function*/
	FString ToString() const
	{
		return FString::Printf(TEXT("[PreviousMarker %s, NextMarker %s] : %0.2f "), *PreviousMarkerName.ToString(), *NextMarkerName.ToString(), PositionBetweenMarkers);
	}
};

struct FPassedMarker
{
	FName PassedMarkerName;

	float DeltaTimeWhenPassed;
};

/**
* Information about an animation asset that needs to be ticked
*/
USTRUCT()
struct FAnimTickRecord
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<class UAnimationAsset> SourceAsset = nullptr;

	float* TimeAccumulator = nullptr;
	float PlayRateMultiplier = 1.0f;
	float EffectiveBlendWeight = 0.0f;
	float RootMotionWeightModifier = 1.0f;

	bool bLooping = false;
	bool bIsEvaluator = false;
	bool bRequestedInertialization = false;
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
	bool bIsExclusiveLeader = false;
	bool bActiveContext = true;

	const UMirrorDataTable* MirrorDataTable = nullptr;

	TSharedPtr<TArray<TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>>> ContextData;

	union
	{
		struct
		{
			FBlendFilter* BlendFilter;
			TArray<FBlendSampleData>* BlendSampleDataCache;
			int32  TriangulationIndex;
			float  BlendSpacePositionX;
			float  BlendSpacePositionY;
			bool   bTeleportToTime;
		} BlendSpace;

		struct
		{
			float CurrentPosition;  // montage doesn't use accumulator, but this
			TArray<FPassedMarker>* MarkersPassedThisTick;
		} Montage;
	};

	// Asset players (and other nodes) have ownership of their respective DeltaTimeRecord value/state,
	// while an asset's tick update will forward the time-line through the tick record
	FDeltaTimeRecord* DeltaTimeRecord = nullptr;

	// marker sync related data
	FMarkerTickRecord* MarkerTickRecord = nullptr;
	bool bCanUseMarkerSync = false;
	float LeaderScore = 0.0f;
	
#if ANIM_TRACE_ENABLED
	// Source proxy for deferred tracing - captures the originating anim instance proxy at tick record
	// creation time so that linked graph tick records are correctly attributed during the deferred trace loop.
	// Raw pointer is safe here: proxy lifetime is tied to its UAnimInstance which outlives the tick record.
	FAnimInstanceProxy* SourceProxy = nullptr;
#endif
	
	// Return the root motion weight for this tick record
	float GetRootMotionWeight() const { return EffectiveBlendWeight * RootMotionWeightModifier; }

private:
	void AllocateContextDataContainer();

public:
	FAnimTickRecord() = default;

	// Create a tick record for an anim sequence
	ENGINE_API FAnimTickRecord(UAnimSequenceBase* InSequence, bool bInLooping, float InPlayRate, bool bInIsEvaluator, float InFinalBlendWeight, float& InCurrentTime, FMarkerTickRecord& InMarkerTickRecord);

	// Create a tick record for a blendspace
	ENGINE_API FAnimTickRecord(
		UBlendSpace* InBlendSpace, const FVector& InBlendInput, TArray<FBlendSampleData>& InBlendSampleDataCache, FBlendFilter& InBlendFilter, bool bInLooping, 
		float InPlayRate, bool bShouldTeleportToTime, bool bInIsEvaluator, float InFinalBlendWeight, float& InCurrentTime, FMarkerTickRecord& InMarkerTickRecord);

	// Create a tick record for a montage
	ENGINE_API FAnimTickRecord(UAnimMontage* InMontage, float InCurrentPosition, float InWeight, TArray<FPassedMarker>& InMarkersPassedThisTick, FMarkerTickRecord& InMarkerTickRecord);

	// Create a tick record for a pose asset
	ENGINE_API FAnimTickRecord(UPoseAsset* InPoseAsset, float InFinalBlendWeight);

	// Gather any data from the current update context
	ENGINE_API void GatherContextData(const FAnimationUpdateContext& InContext);
	
	// Explicitly add typed context data to the tick record
	template<typename Type, typename... TArgs>
	void MakeContextData(TArgs&&... Args)
	{
		static_assert(TPointerIsConvertibleFromTo<Type, const UE::Anim::IAnimNotifyEventContextDataInterface>::Value, "'Type' template parameter to MakeContextData must be derived from IAnimNotifyEventContextDataInterface");
		if (!ContextData.IsValid())
		{
			AllocateContextDataContainer();
		}

		ContextData->Add(MakeUnique<Type>(Forward<TArgs>(Args)...));
	}

	/** This can be used with the Sort() function on a TArray of FAnimTickRecord to sort from higher leader score */
	bool operator <(const FAnimTickRecord& Other) const { return LeaderScore > Other.LeaderScore; }
};

class FMarkerTickContext
{
public:

	static const TArray<FName> DefaultMarkerNames;

	FMarkerTickContext(const TArray<FName>& ValidMarkerNames) 
		: ValidMarkers(&ValidMarkerNames) 
	{}

	FMarkerTickContext() 
		: ValidMarkers(&DefaultMarkerNames) 
	{}

	void SetMarkerSyncStartPosition(const FMarkerSyncAnimPosition& SyncPosition)
	{
		MarkerSyncStartPostion = SyncPosition;
	}

	void SetMarkerSyncEndPosition(const FMarkerSyncAnimPosition& SyncPosition)
	{
		MarkerSyncEndPostion = SyncPosition;
	}

	const FMarkerSyncAnimPosition& GetMarkerSyncStartPosition() const
	{
		return MarkerSyncStartPostion;
	}

	const FMarkerSyncAnimPosition& GetMarkerSyncEndPosition() const
	{
		return MarkerSyncEndPostion;
	}

	const TArray<FName>& GetValidMarkerNames() const
	{
		return *ValidMarkers;
	}

	bool IsMarkerSyncStartValid() const
	{
		return MarkerSyncStartPostion.IsValid();
	}

	bool IsMarkerSyncEndValid() const
	{
		// does it have proper end position
		return MarkerSyncEndPostion.IsValid();
	}

	TArray<FPassedMarker> MarkersPassedThisTick;

	/** Debug output function */
	FString  ToString() const
	{
		FString MarkerString;

		for (const auto& ValidMarker : *ValidMarkers)
		{
			MarkerString.Append(FString::Printf(TEXT("%s,"), *ValidMarker.ToString()));
		}

		return FString::Printf(TEXT(" - Sync Start Position : %s\n - Sync End Position : %s\n - Markers : %s"),
			*MarkerSyncStartPostion.ToString(), *MarkerSyncEndPostion.ToString(), *MarkerString);
	}
private:
	// Structure representing our sync position based on markers before tick
	// This is used to allow new animations to play from the right marker position
	FMarkerSyncAnimPosition MarkerSyncStartPostion;

	// Structure representing our sync position based on markers after tick
	FMarkerSyncAnimPosition MarkerSyncEndPostion;

	// Valid marker names for this sync group
	const TArray<FName>* ValidMarkers;
};


UENUM()
namespace EAnimGroupRole
{
	enum Type : int
	{
		/** This node can be the leader, as long as it has a higher blend weight than the previous best leader. */
		CanBeLeader,
		
		/** This node will always be a follower (unless there are only followers, in which case the first one ticked wins). */
		AlwaysFollower,

		/** This node will always be a leader (if more than one node is AlwaysLeader, the last one ticked wins). */
		AlwaysLeader,

		/** This node will be excluded from the sync group while blending in. Once blended in it will be the sync group leader until blended out*/
		TransitionLeader,

		/** This node will be excluded from the sync group while blending in. Once blended in it will be a follower until blended out*/
		TransitionFollower,

		/** This node will always be a leader. If it fails to be ticked as a leader it will be run as ungrouped asset player (EAnimSyncMethod::DoNotSync) .*/
		ExclusiveAlwaysLeader,
	};
}

// Deprecated - do not use
UENUM()
enum class EAnimSyncGroupScope : uint8
{
	// Sync only with animations in the current instance (either main or linked instance)
	Local,

	// Sync with all animations in the main and linked instances of this skeletal mesh component
	Component,
};

// How an asset will synchronize with other assets
UENUM()
enum class EAnimSyncMethod : uint8
{
	// Don't sync ever
	DoNotSync,

	// Use a named sync group
	SyncGroup,

	// Use the graph structure to provide a sync group to apply
	Graph
};

USTRUCT()
struct FAnimGroupInstance
{
	GENERATED_USTRUCT_BODY()

public:
	// The list of animation players in this group which are going to be evaluated this frame
	TArray<FAnimTickRecord> ActivePlayers;

	// The current group leader
	// @note : before ticking, this is invalid
	// after ticking, this should contain the real leader
	// during ticket, this list gets sorted by LeaderScore of AnimTickRecord,
	// and it starts from 0 index, but if that fails due to invalid position, it will go to the next available leader
	int32 GroupLeaderIndex;

	// Valid marker names for this sync group
	TArray<FName> ValidMarkers;

	// Can we use sync markers for ticking this sync group
	bool bCanUseMarkerSync;

	// This has latest Montage Leader Weight
	float MontageLeaderWeight;

	FMarkerTickContext MarkerTickContext;

	// Float in 0 - 1 range representing how far through an animation we were before ticking
	float PreviousAnimLengthRatio;

	// Float in 0 - 1 range representing how far through an animation we are
	float AnimLengthRatio;

public:
	FAnimGroupInstance()
		: GroupLeaderIndex(INDEX_NONE)
		, bCanUseMarkerSync(false)
		, MontageLeaderWeight(0.f)
		, PreviousAnimLengthRatio(0.f)
		, AnimLengthRatio(0.f)
	{
	}

	void Reset()
	{
		GroupLeaderIndex = INDEX_NONE;
		ActivePlayers.Reset();
		bCanUseMarkerSync = false;
		MontageLeaderWeight = 0.f;
		MarkerTickContext = FMarkerTickContext();
		PreviousAnimLengthRatio = 0.f;
		AnimLengthRatio = 0.f;
	}

	// Checks the last tick record in the ActivePlayers array to see if it's a better leader than the current candidate.
	// This should be called once for each record added to ActivePlayers, after the record is setup.
	ENGINE_API void TestTickRecordForLeadership(EAnimGroupRole::Type MembershipType);

	// Called after leader has been ticked and decided
	ENGINE_API void Finalize(const FAnimGroupInstance* PreviousGroup);

	// Called after all tick records have been added but before assets are actually ticked
	ENGINE_API void Prepare(const FAnimGroupInstance* PreviousGroup);
};

/** Utility struct to accumulate root motion. */
USTRUCT()
struct FRootMotionMovementParams
{
	GENERATED_USTRUCT_BODY()

private:
	ENGINE_API static FVector RootMotionScale;

public:
	
	UPROPERTY()
	bool bHasRootMotion;

	UPROPERTY()
	float BlendWeight;

private:
	UPROPERTY()
	FTransform RootMotionTransform;

public:
	FRootMotionMovementParams()
		: bHasRootMotion(false)
		, BlendWeight(0.f)
	{
	}

	// Copy/Move constructors and assignment operator added for deprecation support
	// Could be removed once RootMotionTransform is made private
	FRootMotionMovementParams(const FRootMotionMovementParams& Other)
		: bHasRootMotion(Other.bHasRootMotion)
		, BlendWeight(Other.BlendWeight)
	{
		RootMotionTransform = Other.RootMotionTransform;
	}

	FRootMotionMovementParams(const FRootMotionMovementParams&& Other)
		: bHasRootMotion(Other.bHasRootMotion)
		, BlendWeight(Other.BlendWeight)
	{
		RootMotionTransform = Other.RootMotionTransform;
	}

	FRootMotionMovementParams& operator=(const FRootMotionMovementParams& Other)
	{
		bHasRootMotion = Other.bHasRootMotion;
		BlendWeight = Other.BlendWeight;
		RootMotionTransform = Other.RootMotionTransform;
		return *this;
	}

	void Set(const FTransform& InTransform)
	{
		bHasRootMotion = true;
		RootMotionTransform = InTransform;
		RootMotionTransform.SetScale3D(RootMotionScale);
		BlendWeight = 1.f;
	}

	void Accumulate(const FTransform& InTransform)
	{
		if (!bHasRootMotion)
		{
			Set(InTransform);
		}
		else
		{
			RootMotionTransform = InTransform * RootMotionTransform;
			RootMotionTransform.SetScale3D(RootMotionScale);
		}
	}

	void Accumulate(const FRootMotionMovementParams& MovementParams)
	{
		if (MovementParams.bHasRootMotion)
		{
			Accumulate(MovementParams.RootMotionTransform);
		}
	}

	void AccumulateWithBlend(const FTransform& InTransform, float InBlendWeight)
	{
		const ScalarRegister VBlendWeight(InBlendWeight);
		if (bHasRootMotion)
		{
			RootMotionTransform.AccumulateWithShortestRotation(InTransform, VBlendWeight);
			RootMotionTransform.SetScale3D(RootMotionScale);
			BlendWeight += InBlendWeight;
		}
		else
		{
			Set(InTransform * VBlendWeight);
			BlendWeight = InBlendWeight;
		}
	}

	void AccumulateWithBlend(const FRootMotionMovementParams & MovementParams, float InBlendWeight)
	{
		if (MovementParams.bHasRootMotion)
		{
			AccumulateWithBlend(MovementParams.RootMotionTransform, InBlendWeight);
		}
	}

	void Clear()
	{
		bHasRootMotion = false;
		BlendWeight = 0.f;
	}

	void MakeUpToFullWeight()
	{
		float WeightLeft = FMath::Max(1.f - BlendWeight, 0.f);
		if (WeightLeft > UE_KINDA_SMALL_NUMBER)
		{
			AccumulateWithBlend(FTransform(), WeightLeft);
		}
		RootMotionTransform.NormalizeRotation();
	}

	FRootMotionMovementParams ConsumeRootMotion(float Alpha)
	{
		FTransform PartialRootMotion(FQuat::Slerp(FQuat::Identity, RootMotionTransform.GetRotation(), Alpha), RootMotionTransform.GetTranslation()*Alpha, RootMotionScale);

		// remove the part of the root motion we are applying now and leave the remainder in RootMotionTransform
		RootMotionTransform = RootMotionTransform.GetRelativeTransform(PartialRootMotion);

		FRootMotionMovementParams ReturnParams;
		ReturnParams.Set(PartialRootMotion);

		check(PartialRootMotion.IsRotationNormalized());
		check(RootMotionTransform.IsRotationNormalized());
		return ReturnParams;
	}

	const FTransform& GetRootMotionTransform() const { return RootMotionTransform; }
	void ScaleRootMotionTranslation(float TranslationScale) { RootMotionTransform.ScaleTranslation(TranslationScale); }
};

// This structure is used to either advance or synchronize animation players
struct FAnimAssetTickContext
{
public:
	FAnimAssetTickContext(float InDeltaTime, ERootMotionMode::Type InRootMotionMode, bool bInOnlyOneAnimationInGroup, const TArray<FName>& ValidMarkerNames)
		: RootMotionMode(InRootMotionMode)
		, MarkerTickContext(ValidMarkerNames)
		, DeltaTime(InDeltaTime)
		, LeaderDelta(0.f)
		, PreviousAnimLengthRatio(0.0f)
		, AnimLengthRatio(0.0f)
		, bIsMarkerPositionValid(ValidMarkerNames.Num() > 0)
		, bIsLeader(true)
		, bOnlyOneAnimationInGroup(bInOnlyOneAnimationInGroup)
		, bResyncToSyncGroup(false)
	{
	}

	FAnimAssetTickContext(float InDeltaTime, ERootMotionMode::Type InRootMotionMode, bool bInOnlyOneAnimationInGroup)
		: RootMotionMode(InRootMotionMode)
		, DeltaTime(InDeltaTime)
		, LeaderDelta(0.f)
		, PreviousAnimLengthRatio(0.0f)
		, AnimLengthRatio(0.0f)
		, bIsMarkerPositionValid(false)
		, bIsLeader(true)
		, bOnlyOneAnimationInGroup(bInOnlyOneAnimationInGroup)
		, bResyncToSyncGroup(false)
	{
	}

	// Are we the leader of our sync group (or ungrouped)?
	bool IsLeader() const
	{
		return bIsLeader;
	}

	bool IsFollower() const
	{
		return !bIsLeader;
	}

	// Return the delta time of the tick
	float GetDeltaTime() const
	{
		return DeltaTime;
	}

	void SetLeaderDelta(float InLeaderDelta)
	{
		LeaderDelta = InLeaderDelta;
	}

	float GetLeaderDelta() const
	{
		return LeaderDelta;
	}

	void SetPreviousAnimationPositionRatio(float NormalizedTime)
	{
		PreviousAnimLengthRatio = NormalizedTime;
	}

	void SetAnimationPositionRatio(float NormalizedTime)
	{
		AnimLengthRatio = NormalizedTime;
	}

	// Returns the previous synchronization point (normalized time)
	float GetPreviousAnimationPositionRatio() const
	{
		return PreviousAnimLengthRatio;
	}

	// Returns the synchronization point (normalized time)
	float GetAnimationPositionRatio() const
	{
		return AnimLengthRatio;
	}

	void InvalidateMarkerSync()
	{
		bIsMarkerPositionValid = false;
	}

	bool CanUseMarkerPosition() const
	{
		return bIsMarkerPositionValid;
	}

	void ConvertToFollower()
	{
		bIsLeader = false;
	}

	bool ShouldGenerateNotifies() const
	{
		return IsLeader();
	}

	bool IsSingleAnimationContext() const
	{
		return bOnlyOneAnimationInGroup;
	}

	void SetResyncToSyncGroup(bool bInResyncToSyncGroup)
	{
		bResyncToSyncGroup = bInResyncToSyncGroup;
	}

	// Should we resync to the sync group this tick (eg: when initializing or resuming from zero weight)?
	bool ShouldResyncToSyncGroup() const
	{
		return bResyncToSyncGroup;
	}

	//Root Motion accumulated from this tick context
	FRootMotionMovementParams RootMotionMovementParams;

	// The root motion mode of the owning AnimInstance
	ERootMotionMode::Type RootMotionMode;

	FMarkerTickContext MarkerTickContext;

private:
	float DeltaTime;

	float LeaderDelta;

	// Float in 0 - 1 range representing how far through an animation we were before ticking
	float PreviousAnimLengthRatio;

	// Float in 0 - 1 range representing how far through an animation we are
	float AnimLengthRatio;

	bool bIsMarkerPositionValid;

	bool bIsLeader;

	bool bOnlyOneAnimationInGroup;

	// True if the asset player being ticked should (re)synchronize to the sync group's time (eg: it was inactive and has now reactivated)
	bool bResyncToSyncGroup;
};

USTRUCT()
struct FAnimationGroupReference
{
	GENERATED_USTRUCT_BODY()
	
	// How this animation will synchronize with other animations. 
	UPROPERTY(EditAnywhere, Category=Settings)
	EAnimSyncMethod Method;

	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category=Settings, meta = (EditCondition = "Method == EAnimSyncMethod::SyncGroup"))
	FName GroupName;

	// The role this animation can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category=Settings, meta = (EditCondition = "Method == EAnimSyncMethod::SyncGroup"))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole;

	FAnimationGroupReference()
		: Method(EAnimSyncMethod::DoNotSync)
		, GroupName(NAME_None)
		, GroupRole(EAnimGroupRole::CanBeLeader)
	{
	}
};

UCLASS(abstract, BlueprintType, MinimalAPI)
class UAnimationAsset : public UObject, public IInterface_AssetUserData, public IInterface_PreviewMeshProvider
{
	GENERATED_BODY()

private:
	/** Pointer to the Skeleton this asset can be played on .	*/
	UPROPERTY(AssetRegistrySearchable, Category=Animation, VisibleAnywhere)
	TObjectPtr<class USkeleton> Skeleton;

	/** Skeleton guid. If changes, you need to remap info*/
	FGuid SkeletonGuid;

	/** Allow animations to track virtual bone info */
	FGuid SkeletonVirtualBoneGuid; 

	/** Meta data that can be saved with the asset 
	 * 
	 * You can query by GetMetaData function
	 */
	UPROPERTY(Category=MetaData, instanced, EditAnywhere)
	TArray<TObjectPtr<UAnimMetaData>> MetaData;

public:
	/* 
	 * Parent asset is used for AnimMontage when it derives all settings but remap animation asset. 
	 * For example, you can just use all parent's setting  for the montage, but only remap assets
	 * This isn't magic bullet unfortunately and it is consistent effort of keeping the data synced with parent
	 * If you add new property, please make sure those property has to be copied for children. 
	 * If it does, please add the copy in the function RefreshParentAssetData
	 * We'd like to extend this feature to BlendSpace in the future
	 */
#if WITH_EDITORONLY_DATA
	/** Parent Asset, if set, you won't be able to edit any data in here but just mapping table
	 * 
	 * During cooking, this data will be used to bake out to normal asset */
	UPROPERTY(AssetRegistrySearchable, Category=Animation, VisibleAnywhere)
	TObjectPtr<class UAnimationAsset> ParentAsset;

	/** 
	 * @todo : comment
	 */
	ENGINE_API void ValidateParentAsset();

	/**
	 * note this is transient as they're added as they're loaded
	 */
	UPROPERTY(transient)
	TArray<TObjectPtr<class UAnimationAsset>> ChildrenAssets;

	const UAssetMappingTable* GetAssetMappingTable() const
	{
		return AssetMappingTable;
	}
protected:
	/** Asset mapping table when ParentAsset is set */
	UPROPERTY(Category=Animation, VisibleAnywhere)
	TObjectPtr<class UAssetMappingTable> AssetMappingTable;
#endif // WITH_EDITORONLY_DATA

protected:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Animation)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

public:
	/** Advances the asset player instance 
	 * 
	 * @param Instance		AnimationTickRecord Instance - saves data to evaluate
	 * @param NotifyQueue	Queue for any notifies we create
	 * @param Context		The tick context (leader/follower, delta time, sync point, etc...)
	 */
	virtual void TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const {}

	UFUNCTION(BlueprintPure, Category = "Animation", meta=(BlueprintThreadSafe))
	virtual float GetPlayLength() const { return 0.f; };

	ENGINE_API void SetSkeleton(USkeleton* NewSkeleton);
	ENGINE_API virtual void PostLoad() override;

	/** Validate our stored data against our skeleton and update accordingly */
	ENGINE_API void ValidateSkeleton();

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Get available Metadata within the animation asset
	 */
	const TArray<UAnimMetaData*>& GetMetaData() const { return MetaData; }
	
	/** Returns the first metadata of the specified class */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API UAnimMetaData* FindMetaDataByClass(const TSubclassOf<UAnimMetaData> MetaDataClass) const;

	/** Templatized version of FindMetaDataByClass that handles casting for you */
	template<class T>
	T* FindMetaDataByClass() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, const UAnimMetaData>::Value, "'T' template parameter to FindMetaDataByClass must be derived from UAnimMetaData");

		return (T*)FindMetaDataByClass(T::StaticClass());
	}
	
	ENGINE_API void AddMetaData(UAnimMetaData* MetaDataInstance); 
	void EmptyMetaData() { MetaData.Empty(); }	
	ENGINE_API void RemoveMetaData(UAnimMetaData* MetaDataInstance);
	ENGINE_API void RemoveMetaData(TArrayView<UAnimMetaData*> MetaDataInstances);

	/** IInterface_PreviewMeshProvider interface */
	ENGINE_API virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override;
	ENGINE_API virtual USkeletalMesh* GetPreviewMesh(bool bFindIfNotSet = false) override;
	ENGINE_API virtual USkeletalMesh* GetPreviewMesh() const override;

#if WITH_EDITOR
	/** Sets or updates the preview skeletal mesh */
	UFUNCTION(BlueprintCallable, Category=Animation)
	void SetPreviewSkeletalMesh(USkeletalMesh* PreviewMesh) { SetPreviewMesh(PreviewMesh); }
	
	/** Replace Skeleton 
	 * 
	 * @param NewSkeleton	NewSkeleton to change to 
	 */
	ENGINE_API bool ReplaceSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces=false);

	virtual void OnSetSkeleton(USkeleton* NewSkeleton) {}

	// Helper function for GetAllAnimationSequencesReferred, it adds itself first and call GetAllAnimationSEquencesReferred
	ENGINE_API void HandleAnimReferenceCollection(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive);

	/** Retrieve all animations that are used by this asset 
	 * 
	 * @param (out)		AnimationSequences 
	 **/
	ENGINE_API virtual bool GetAllAnimationSequencesReferred(TArray<class UAnimationAsset*>& AnimationSequences, bool bRecursive = true);

public:
	/** Replace this assets references to other animations based on ReplacementMap 
	 * 
	 * @param ReplacementMap	Mapping of original asset to new asset
	 **/
	ENGINE_API virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap);

	virtual int32 GetMarkerUpdateCounter() const { return 0; }

	/** 
	 * Parent Asset related function. Used by editor
	 */
	ENGINE_API void SetParentAsset(UAnimationAsset* InParentAsset);
	bool HasParentAsset() const { return ParentAsset != nullptr; }
	ENGINE_API bool RemapAsset(UAnimationAsset* SourceAsset, UAnimationAsset* TargetAsset);
	// we have to update whenever we have anything loaded
	ENGINE_API void UpdateParentAsset();
protected:
	ENGINE_API virtual void RefreshParentAssetData();
#endif //WITH_EDITOR

public:
	/** Return a list of unique marker names for blending compatibility */
	virtual TArray<FName>* GetUniqueMarkerNames() { return nullptr; }

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/**
	* return true if this is valid additive animation
	* false otherwise
	*/
	virtual bool IsValidAdditive() const { return false; }

#if WITH_EDITORONLY_DATA
	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	/** The default skeletal mesh to use when previewing this asset - this only applies when you open Persona using this asset*/
	// @todo: note that this doesn't retarget right now
	UPROPERTY(duplicatetransient, EditAnywhere, Category = Animation)
	TObjectPtr<class UPoseAsset> PreviewPoseAsset;

private:
	/** The default skeletal mesh to use when previewing this asset - this only applies when you open Persona using this asset*/
	UPROPERTY(duplicatetransient, AssetRegistrySearchable)
	TSoftObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;
#endif //WITH_EDITORONLY_DATA

protected:
#if WITH_EDITOR
	ENGINE_API virtual void RemapTracksToNewSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces);
#endif // WITH_EDITOR

public:
	class USkeleton* GetSkeleton() const { return Skeleton; }

	FGuid GetSkeletonVirtualBoneGuid() const { return SkeletonVirtualBoneGuid; }
	void SetSkeletonVirtualBoneGuid(FGuid Guid) { SkeletonVirtualBoneGuid = Guid; }
	FGuid GetSkeletonGuid() const { return SkeletonGuid; }
};

