// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimationAsset.h"
#include "Animation/Skeleton.h"

#include "AnimationAssetExtensions.generated.h"

UCLASS()
class UAnimationAssetExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	/**
	* @param	InAsset		Animation Asset to retrieve the Skeleton for
	*
	* @return	The target USkeleton for the provided UAnimationAsset
	*/
	UFUNCTION(BlueprintPure, Category = AnimationAsset, meta = (ScriptMethod))
	static USkeleton* GetSkeleton(UAnimationAsset* InAsset)
	{
		if (InAsset)
		{
			return InAsset->GetSkeleton();
		}

		return nullptr;
	}
#endif // WITH_EDITOR
};