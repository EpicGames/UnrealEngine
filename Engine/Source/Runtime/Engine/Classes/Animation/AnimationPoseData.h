// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FCompactPose;
struct FBlendedCurve;
struct FPoseContext;
struct FSlotEvaluationPose;

namespace UE { namespace Anim { struct FStackAttributeContainer; } }

/** Structure used for passing around animation pose related data throughout the Animation Runtime */
struct FAnimationPoseData
{
	ENGINE_API FAnimationPoseData(FPoseContext& InPoseContext);
	ENGINE_API FAnimationPoseData(FSlotEvaluationPose& InSlotPoseContext);
	ENGINE_API FAnimationPoseData(FCompactPose& InPose, FBlendedCurve& InCurve, UE::Anim::FStackAttributeContainer& InAttributes);
	
	/** No default constructor, or assignment */
	FAnimationPoseData() = delete;
	FAnimationPoseData& operator=(FAnimationPoseData&& Other) = delete;

	/** Getters for the wrapped structures */
	ENGINE_API const FCompactPose& GetPose() const;
	ENGINE_API FCompactPose& GetPose();
	ENGINE_API const FBlendedCurve& GetCurve() const;
	ENGINE_API FBlendedCurve& GetCurve();
	ENGINE_API const UE::Anim::FStackAttributeContainer& GetAttributes() const;
	ENGINE_API UE::Anim::FStackAttributeContainer& GetAttributes();

protected:
	FCompactPose& Pose;
	FBlendedCurve& Curve;
	UE::Anim::FStackAttributeContainer& Attributes;
};
