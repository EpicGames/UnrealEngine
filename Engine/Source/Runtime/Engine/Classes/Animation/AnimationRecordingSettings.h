// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Curves/RichCurve.h"
#include "Misc/FrameRate.h"
#include "AnimationRecordingSettings.generated.h"

/** Settings describing how to record an animation */
USTRUCT()
struct FAnimationRecordingSettings
{
	GENERATED_BODY()

	/** 30Hz default sample frame rate */
	static ENGINE_API const FFrameRate DefaultSampleFrameRate;

	/** 1 minute default length */
	static ENGINE_API const float DefaultMaximumLength;

	/** Length used to specify unbounded */
	static ENGINE_API const float UnboundedMaximumLength;

	FAnimationRecordingSettings()
		: bRecordInWorldSpace(true)
		, bRemoveRootAnimation(true)
		, bAutoSaveAsset(false)
		, SampleFrameRate(DefaultSampleFrameRate)
		, Length((float)DefaultMaximumLength)
		, Interpolation(EAnimInterpolationType::Linear)
		, InterpMode(ERichCurveInterpMode::RCIM_Linear)
		, TangentMode(ERichCurveTangentMode::RCTM_Auto)
		, bCheckDeltaTimeAtBeginning(true)
		, bSetRetargetSourceAsset(false)
		, bRecordTransforms(true)
		, bRecordMorphTargets(true)
		, bRecordAttributeCurves(true)
		, bRecordMaterialCurves(true)
		, bSkipCurvesWithZeroValue(true)
		, bRemoveExcludedCurves(false)
		, bTransactRecording(true)
	{}
	
	/** Whether to record animation in world space, defaults to true */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRecordInWorldSpace;

	/** Whether to remove the root bone transform from the animation */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRemoveRootAnimation;

	/** Whether to auto-save asset when recording is completed. Defaults to false */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bAutoSaveAsset;

	/** Sample rate of the recorded animation */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FFrameRate SampleFrameRate;

	/** Maximum length of the animation recorded (in seconds). If zero the animation will keep on recording until stopped. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Length;

	/** This defines how values between keys are calculated for transforms.**/
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimInterpolationType Interpolation;

	/** Interpolation mode for the recorded keys. */
	UPROPERTY(EditAnywhere, Category = "Settings", DisplayName = "Interpolation Mode")
	TEnumAsByte<ERichCurveInterpMode> InterpMode;

	/** Tangent mode for the recorded keys. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TEnumAsByte<ERichCurveTangentMode> TangentMode;

	/** Whether to check DeltaTime at recording for pauses, turned off for TakeRecorder*/
	bool bCheckDeltaTimeAtBeginning;

	/** Whether or not to set retarget source asset */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bSetRetargetSourceAsset;


	/** Whether or not to record transforms */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRecordTransforms;

	/** Whether or not to record morph targets */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRecordMorphTargets;

	/** Whether or not to record parameter curves */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRecordAttributeCurves;

	/** Whether or not to record material curves */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRecordMaterialCurves;

	/** If enabled, we will skip any curves that have only zero values */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bSkipCurvesWithZeroValue;

	/** If enabled, curves in ExcludeAnimationNames that already exist in the target sequence will be removed at the start of recording */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRemoveExcludedCurves;

	/** Whether or not to transact recording changes */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bTransactRecording;

	/** Include only the animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> IncludeAnimationNames;

	/** Exclude all animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> ExcludeAnimationNames;
};
