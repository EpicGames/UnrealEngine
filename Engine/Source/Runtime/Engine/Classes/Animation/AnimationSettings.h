// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationSettings.h: Declares the AnimationSettings class.
=============================================================================*/

#pragma once

#include "Engine/DeveloperSettings.h"
#include "CustomAttributes.h"
#include "AnimationSettings.generated.h"

class UUserDefinedStruct;
struct FCustomAttributeSetting;
struct FMirrorFindReplaceExpression;

/**
 * Default animation settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Animation"), MinimalAPI)
class UAnimationSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/** Compression version for recompress commandlet, bump this to trigger full recompressed, otherwise only new imported animations will be recompressed */
	UPROPERTY(config, VisibleAnywhere, Category = Compression)
	int32 CompressCommandletVersion;

	/** List of bone names to treat with higher precision, in addition to any bones with sockets */
	UPROPERTY(config, EditAnywhere, Category = Compression)
	TArray<FString> KeyEndEffectorsMatchNameArray;

	/** If true, this will forcibly recompress every animation, this should not be checked in enabled */
	UPROPERTY(config, EditAnywhere, Category = Compression)
	bool ForceRecompression;

	/** If true and the existing compression error is greater than Alternative Compression Threshold, then any compression technique (even one that increases the size) with a lower error will be used until it falls below the threshold */
	UPROPERTY(config, EditAnywhere, Category = Compression)
	bool bForceBelowThreshold;

	/** If true, then the animation will be first recompressed with its current compressor if non-NULL, or with the global default compressor (specified in the engine ini) 
	* Also known as "Run Current Default Compressor"
	*/
	UPROPERTY(config, EditAnywhere, Category = Compression)
	bool bFirstRecompressUsingCurrentOrDefault;

	/** If true and the existing compression error is greater than Alternative Compression Threshold, then Alternative Compression Threshold will be effectively raised to the existing error level */
	UE_DEPRECATED(5.1, "This is being removed because it is unused")
	UPROPERTY(config /*, EditAnywhere, Category = Compression*/, meta = (DeprecatedProperty, DeprecationMessage = "No longer used."))
	bool bRaiseMaxErrorToExisting;

	/** If true, recompression will log performance information */
	UPROPERTY(config, EditAnywhere, Category = Performance)
	bool bEnablePerformanceLog;

	/** If true, animation track data will be stripped from dedicated server cooked data */
	UPROPERTY(config, EditAnywhere, Category = Performance)
	bool bStripAnimationDataOnDedicatedServer;

	/** If true, pre-4.19 behavior of zero-ticking animations during skeletal mesh init */
	UPROPERTY(config, EditAnywhere, Category = Performance)
	bool bTickAnimationOnSkeletalMeshInit;

	/** Names that identify bone animation attributes representing the individual components of a timecode and a subframe along with a take name.
	    These will be included in the list of bone custom attribute names to import. */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes, meta=(DisplayName="Bone Timecode Animation Attribute name settings"))
	FTimecodeCustomAttributeNameSettings BoneTimecodeCustomAttributeNameSettings;

	/** List of animation attribute names to import directly on their corresponding bone names. The meaning field allows to contextualize the attribute name and customize tooling for it. */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes, meta=(DisplayName="Bone Animation Attributes names"))
	TArray<FCustomAttributeSetting> BoneCustomAttributesNames;

	/** Gets the complete list of bone animation attribute names to consider for import.
	    This includes the designated timecode animation attributes as well as other bone animation attributes identified in the settings. */
	UFUNCTION(BlueprintPure, Category = AnimationAttributes)
	ENGINE_API TArray<FString> GetBoneCustomAttributeNamesToImport() const;

	/** List of bone names for which all animation attributes are directly imported on the bone. */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes, meta=(DisplayName="Bone names with Animation Attributes"))
	TArray<FString> BoneNamesWithCustomAttributes;

	/** Animation Attribute specific blend types (by name) */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes)
	TMap<FName, ECustomAttributeBlendType> AttributeBlendModes;

	/** Default Animation Attribute blend type */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes)
	ECustomAttributeBlendType DefaultAttributeBlendMode;

	/** Names to match against when importing FBX node transform curves as attributes (can use ? and * wildcards) */
	UPROPERTY(config, EditAnywhere, Category = AnimationAttributes)
	TArray<FString> TransformAttributeNames;

	/** Register user defined structs as animation attributes*/
	UPROPERTY(config, EditAnywhere, DisplayName="User Defined Struct Animation Attributes (Runtime only, Non-blendable)", Category = AnimationAttributes, meta=(AllowedClasses="/Script/Engine.UserDefinedStruct"))
	TArray<TSoftObjectPtr<UUserDefinedStruct>> UserDefinedStructAttributes;

	/** Find and Replace Expressions used for mirroring  */
	UPROPERTY(config, EditAnywhere, Category = Mirroring)
	TArray<FMirrorFindReplaceExpression> MirrorFindReplaceExpressions;

	/** When a mirror pair is not found map to self instead of none.  */
	UPROPERTY(config, EditAnywhere, Category = Mirroring)
	bool bOnMirrorPairFindReplaceFailedMapToSelf = true;

	/** When enabled, automatically validates the mirror data table when it is opened in the editor. Only shows a notification if issues are found. */
	UPROPERTY(config, EditAnywhere, Category = Mirroring)
	bool bValidateMirrorDataTableOnOpen = true;
	
	/** Project specific default frame-rate used when (re)initializing any animation based data */
	UPROPERTY(config, EditAnywhere, Category = AnimationData)
	FFrameRate DefaultFrameRate;

	/** Whether to enforce the project to only use entries from SupportedFrameRates for the animation assets, if disable will warn instead */
	UPROPERTY(config, EditAnywhere, Category = AnimationData)
	bool bEnforceSupportedFrameRates;
public:
	static UAnimationSettings * Get() { return CastChecked<UAnimationSettings>(UAnimationSettings::StaticClass()->GetDefaultObject()); }

	/** Returns the project specific default frame-rate */
	ENGINE_API const FFrameRate& GetDefaultFrameRate() const;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};
