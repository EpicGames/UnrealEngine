// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationTypes.generated.h"

/** A named float */
USTRUCT(BlueprintType)
struct FNamedFloat
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Float")
	float Value = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Float")
	FName Name;
};

/** A named float */
USTRUCT(BlueprintType)
struct FNamedVector
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Vector")
	FVector Value = FVector(0.f);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Vector")
	FName Name;
};

/** A named color */
USTRUCT(BlueprintType)
struct FNamedColor
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color")
	FColor Value = FColor(0);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color")
	FName Name;
};

/** A named transform */
USTRUCT(BlueprintType)
struct FNamedTransform
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transform")
	FTransform Value;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transform")
	FName Name;
};

/** A pose in local space (i.e. each transform is relative to its parent) */
USTRUCT(BlueprintType)
struct FLocalSpacePose
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	TArray<FTransform> Transforms;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Names")
	TArray<FName> Names;
};

/** A pose in component space (i.e. each transform is relative to the component's transform) */
USTRUCT(BlueprintType)
struct FComponentSpacePose
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	TArray<FTransform> Transforms;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Names")
	TArray<FName> Names;
};
