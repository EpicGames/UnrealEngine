// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * This is the definition for a AssetMappingTable that is used for mapping different animation assets
 * however right now this is skeleton agnostic - this is generic asset mapper
 * we could check the skeleton part in tools
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetMappingTable.generated.h"

class UAnimationAsset;

 /** This defines one asset mapping */
USTRUCT()
struct FAssetMapping
{
	GENERATED_USTRUCT_BODY()

	/** source asset **/
	UPROPERTY(EditAnywhere, Category = "FAssetMapping")
	TObjectPtr<class UAnimationAsset>			SourceAsset;

	/** source asset **/
	UPROPERTY(EditAnywhere, Category = "FAssetMapping")
	TObjectPtr<class UAnimationAsset>			TargetAsset;

	FAssetMapping()
		: SourceAsset(nullptr)
		, TargetAsset(nullptr)
	{
	}

	FAssetMapping(UAnimationAsset* InSourceAsset)
		: SourceAsset(InSourceAsset)
		, TargetAsset(nullptr)
	{
		checkSlow(SourceAsset);
	}

private:
	bool SetTargetAsset(UAnimationAsset* InTargetAsset);
	static bool IsValidMapping(UAnimationAsset* InSourceAsset, UAnimationAsset* InTargetAsset);
	bool IsValidMapping() const;

	friend class UAssetMappingTable;
};

/**
 *	UAssetMappingTable : that has AssetMappingTableging data 
 *		- used for retargeting
 *		- support to share different animations
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UAssetMappingTable : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	/** Mappin of asset between source and target **/
	UPROPERTY(VisibleAnywhere, Category = AssetMappingTable, EditFixedSize)
	TArray<FAssetMapping> MappedAssets;

public:
	/** Find Mapped Asset */
	int32 FindMappedAsset(const UAnimationAsset* NewAsset) const;
	void Clear();
	void RefreshAssetList(const TArray<UAnimationAsset*>& AnimAssets);

	bool RemapAsset(UAnimationAsset* SourceAsset, UAnimationAsset* TargetAsset);
	ENGINE_API UAnimationAsset* GetMappedAsset(UAnimationAsset* SourceAsset) const;
#if WITH_EDITOR
	bool GetAllAnimationSequencesReferred(TArray<class UAnimationAsset*>& AnimationSequences, bool bRecursive = true);
	void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap);
#endif // WITH_EDITOR
private:
	void RemovedUnusedSources();
};

