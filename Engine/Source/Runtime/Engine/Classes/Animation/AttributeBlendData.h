// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AttributesRuntime.h"


namespace UE
{
	namespace Anim
	{	
		struct FAttributeBlendData
		{
			friend struct Attributes;

		protected:
			static FAttributeBlendData PerContainerWeighted(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, SourceWeights, AttributeScriptStruct);
			}

			static FAttributeBlendData PerContainerPtrWeighted(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, SourceWeights, AttributeScriptStruct);
			}

			static FAttributeBlendData PerContainerRemappedWeighted(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, SourceWeights, SourceWeightsIndices, AttributeScriptStruct);
			}

			static FAttributeBlendData SingleContainerUniformWeighted(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, const UScriptStruct* AttributeScriptStruct)
			{
				ensure(SourceAttributes.Num() == 1);
				return FAttributeBlendData(SourceAttributes, InUniformWeight, AttributeScriptStruct);
			}

			static FAttributeBlendData SingleAdditiveContainerUniformWeighted(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, EAdditiveAnimationType AdditiveType, const UScriptStruct* AttributeScriptStruct)
			{
				ensure(SourceAttributes.Num() == 1);
				return FAttributeBlendData(SourceAttributes, InUniformWeight, AdditiveType, AttributeScriptStruct);
			}
			
			static FAttributeBlendData SingleAdditiveContainerWithOverrideWeights(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TMap<UE::Anim::FAttributeId, float>& InAttributeWeightOverrides, const float InUniformWeight, EAdditiveAnimationType AdditiveType, const UScriptStruct* AttributeScriptStruct)
			{
				ensure(SourceAttributes.Num() == 1);
				return FAttributeBlendData(SourceAttributes, InAttributeWeightOverrides, InUniformWeight, AdditiveType, AttributeScriptStruct);
			}

			static FAttributeBlendData PerBoneWeighted(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, InPerBoneBlendWeights, AttributeScriptStruct);
			}

			static FAttributeBlendData PerBoneBlendSamples(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, InPerBoneInterpolationIndices, InBlendSampleDataCache, AttributeScriptStruct);
			}

			static FAttributeBlendData PerBoneRemappedBlendSamples(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, TArrayView<const int32> InBlendSampleDataCacheIndices, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes, InPerBoneInterpolationIndices, InBlendSampleDataCache, InBlendSampleDataCacheIndices, AttributeScriptStruct);
			}

			static FAttributeBlendData PerBoneFilteredWeighted(const FStackAttributeContainer& BaseAttributes, const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(BaseAttributes, SourceAttributes, InPerBoneBlendWeights, AttributeScriptStruct);
			}

			static FAttributeBlendData PerBoneSingleContainerWeighted(const FStackAttributeContainer& SourceAttributes1, const FStackAttributeContainer& SourceAttributes2, const TArrayView<const float> WeightsOfSource2, const UScriptStruct* AttributeScriptStruct)
			{
				return FAttributeBlendData(SourceAttributes1, SourceAttributes2, WeightsOfSource2, AttributeScriptStruct);
			}
		private:
			// Blend constructor
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct);
						
			// Blend-by-ptr constructor
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TArrayView<const float> SourceWeights, const UScriptStruct* AttributeScriptStruct);

			// Blend remapped weights constructor
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, const UScriptStruct* AttributeScriptStruct);

			// Accumulate using a single weight
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, const UScriptStruct* AttributeScriptStruct);

			// Additive accumulate using a single weight
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const float InUniformWeight, EAdditiveAnimationType InAdditiveType, const UScriptStruct* AttributeScriptStruct);
			
			// Additive accumulate using a single weight plus optional per attribute override weights 
			ENGINE_API FAttributeBlendData(TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TMap<UE::Anim::FAttributeId, float>& InAttributeWeightOverrides, const float InUniformWeight, EAdditiveAnimationType InAdditiveType, const UScriptStruct* AttributeScriptStruct);

			// Blend using per-bone blend weights
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct);

			// Blend using BlendSample (per-bone) weight data 
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, const UScriptStruct* AttributeScriptStruct);

			// Blend using BlendSample (per-bone) remapped weight data 
			ENGINE_API FAttributeBlendData(const TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> InPerBoneInterpolationIndices, const TArrayView<const FBlendSampleData> InBlendSampleDataCache, TArrayView<const int32> InBlendSampleDataCacheIndices, const UScriptStruct* AttributeScriptStruct);

			// Blend (per-bone filtered) using (per-bone) weight data 
			ENGINE_API FAttributeBlendData(const FStackAttributeContainer& BaseAttributes, const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const FPerBoneBlendWeight> InPerBoneBlendWeights, const UScriptStruct* AttributeScriptStruct);

			// Blend using (per-bone) weight data for one of the two inputs
			ENGINE_API FAttributeBlendData(const FStackAttributeContainer& SourceAttributes1, const FStackAttributeContainer& SourceAttributes2, const TArrayView<const float> WeightsOfSource2, const UScriptStruct* AttributeScriptStruct);

			ENGINE_API FAttributeBlendData();
		private:
			ENGINE_API void ProcessAttributes(const FStackAttributeContainer& AttributeContainers, int32 SourceAttributesIndex, const UScriptStruct* AttributeScriptStruct);

			/** Retrieves the weight on a top-level container basis */
			ENGINE_API float GetContainerWeight(int32 ContainerIndex) const;

			/* Retrieves the weight on a per-bone level basis according to the attribute and bone indices */
			ENGINE_API float GetBoneWeight(int32 AttributeIndex, int32 BoneIndex) const;

			/** Tests for different weight basis */
			ENGINE_API bool HasBoneWeights() const;
			ENGINE_API bool HasContainerWeights() const;
		private:
			/** Structure containing overlapping attributes */
			struct FAttributeSet
			{
				/** Pointers to attribute values */
				TArray<const uint8*, FAnimStackAllocator> DataPtrs;

				/** Weight indices used to map to weight data */
				TArray<int32, FAnimStackAllocator> WeightIndices;
				
				/** Identifier of the attribute */
				const FAttributeId* Identifier;

				/** Highest weight value, and its weight index, that was processed */
				float HighestWeight;
				int32 HighestWeightedIndex;
			};

			/** Structure representing a unique (non-overlapping) attribute */
			struct FUniqueAttribute
			{
				/** Identifier of the attribute */
				const FAttributeId* Identifier;

				/** Weight index used to map to weight data */
				int32 WeightIndex;

				/** Pointer to attribute value */
				const uint8* DataPtr;
			};

			/** Processed unique and sets of attributes */
			TArray<FAttributeSet, FAnimStackAllocator> AttributeSets;
			TArray<FUniqueAttribute, FAnimStackAllocator> UniqueAttributes;

			/** Container level weight data */
			float UniformWeight;
			 /* Contains container of per-bone weights */
			TArrayView<const float> Weights;
			/* Contains container or BlendSampleDataCache remapping indices */
			TArrayView<const int32> WeightIndices; 
			/** Valid whenever performing an additive accumulate */
			EAdditiveAnimationType AdditiveType;

			/** Bone level weight data */
			TArrayView<const FPerBoneBlendWeight> PerBoneWeights;
			TArray<int32, FAnimStackAllocator> HighestBoneWeightedIndices;
			inline const TArrayView<const float>& GetBoneWeights() const { return Weights; }
			bool bPerBoneFilter = false;
			
			/** Blend sample weight data */
			TArrayView<const int32> PerBoneInterpolationIndices;
			TArrayView<const FBlendSampleData> BlendSampleDataCache;
			inline const TArrayView<const int32>& GetBlendSampleDataCacheIndices() const { return WeightIndices; }
			
			/* Contains optional per attribute weight overrides */
			TMap<UE::Anim::FAttributeId, float> AttributeWeightOverrides;
			
		public:
			template<typename AttributeType>
			struct TAttributeSetIterator
			{
				friend struct FAttributeBlendData;
			protected:
				TAttributeSetIterator(const FAttributeBlendData& InData, const FAttributeSet& InCollection) : Data(InData), Collection(InCollection), CurrentIndex(-1) {}
			public:
				/** Return the value for the currently indexed entry in the attribute set */
				const AttributeType& GetValue() const
				{
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.DataPtrs.IsValidIndex(CurrentIndex));
					return *(const AttributeType*)Collection.DataPtrs[CurrentIndex];
				}

				/** Returns (container level) weight value for the current attribute's container in the attribute set */
				const float GetWeight() const
				{
					check(Data.HasContainerWeights());
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.WeightIndices.IsValidIndex(CurrentIndex));
					return Data.GetContainerWeight(Collection.WeightIndices[CurrentIndex]);
				}

				/** Returns (bone level) weight value for the current attribute its bone and container */
				const float GetBoneWeight() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.WeightIndices.IsValidIndex(CurrentIndex));
					return Data.GetBoneWeight(Collection.WeightIndices[CurrentIndex], Collection.Identifier->GetIndex());
				}

				/** Returns highest (container level) weighted value for the attribute set */
				const AttributeType& GetHighestWeightedValue() const
				{
					check(Data.HasContainerWeights());
					check(Collection.HighestWeightedIndex < Collection.DataPtrs.Num() && Collection.DataPtrs.IsValidIndex(Collection.HighestWeightedIndex));
					return *(const AttributeType*)Collection.DataPtrs[Collection.HighestWeightedIndex];
				}

				/** Returns highest (bone level) weighted value for the attribute set */
				const AttributeType& GetHighestBoneWeightedValue() const
				{
					check(Data.HasBoneWeights());

					int32 HighestIndex = INDEX_NONE;
					float Weight = -1.f;
					for (const int32 Index : Collection.WeightIndices)
					{
						const float BoneWeight = Data.GetBoneWeight(Index, Collection.Identifier->GetIndex());
						if (BoneWeight > Weight)
						{
							Weight = BoneWeight;
							HighestIndex = Index;
						}
					}
					ensure(HighestIndex != INDEX_NONE);
					return *(const AttributeType*)Collection.DataPtrs[Collection.WeightIndices.IndexOfByKey(HighestIndex)];
				}

				/** Returns highest (bone level) weighted value, and its weight for the attribute set */
				void GetHighestBoneWeighted(const AttributeType*& OutAttributePtr, float& OutWeight) const
				{
					check(Data.HasBoneWeights());

					int32 HighestIndex = INDEX_NONE;
					float Weight = -1.f;
					for (const int32 Index : Collection.WeightIndices)
					{
						const float BoneWeight = Data.GetBoneWeight(Index, Collection.Identifier->GetIndex());
						if (BoneWeight > Weight)
						{
							Weight = BoneWeight;
							HighestIndex = Index;
						}
					}
					ensure(HighestIndex != INDEX_NONE);
					OutAttributePtr = (const AttributeType*)Collection.DataPtrs[Collection.WeightIndices.IndexOfByKey(HighestIndex)];
					OutWeight = Weight;
				}

				/** Returns the identifier for the current attribute set */
				const FAttributeId& GetIdentifier() const
				{
					return *Collection.Identifier;
				}

				EAdditiveAnimationType GetAdditiveType() const
				{
					return Data.AdditiveType;
				}

				bool IsFilteredBlend() const
				{
					return Data.bPerBoneFilter;
				}

				/** Cycle through to next entry in the attribute set, returns false if the end was reached */
				bool Next()
				{
					++CurrentIndex;
					return CurrentIndex < Collection.DataPtrs.Num();
				}

				int32 GetIndex() const
				{
					return CurrentIndex;
				}
			protected:
				/** Outer object that creates this */
				const FAttributeBlendData& Data;
				/** Attribute collection for current index */
				const FAttributeSet& Collection;
				int32 CurrentIndex;
			};

			struct TAttributeSetRawIterator
			{
				friend struct FAttributeBlendData;
			protected:
				TAttributeSetRawIterator(const FAttributeBlendData& InData, const FAttributeSet& InCollection) : Data(InData), Collection(InCollection), CurrentIndex(-1) {}
			public:
				/** Return the value for the currently indexed entry in the attribute set */
				const uint8* GetValuePtr() const
				{
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.DataPtrs.IsValidIndex(CurrentIndex));
					return Collection.DataPtrs[CurrentIndex];
				}

				/** Returns (container level) weight value for the current attribute's container in the attribute set */
				const float GetWeight() const
				{
					check(Data.HasContainerWeights());
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.WeightIndices.IsValidIndex(CurrentIndex));
					return Data.GetContainerWeight(Collection.WeightIndices[CurrentIndex]);
				}

				/** Returns (bone level) weight value for the current attribute its bone and container */
				const float GetBoneWeight() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < Collection.DataPtrs.Num() && Collection.WeightIndices.IsValidIndex(CurrentIndex));
					return Data.GetBoneWeight(Collection.WeightIndices[CurrentIndex], Collection.Identifier->GetIndex());
				}

				/** Returns highest (container level) weighted value for the attribute set */
				const uint8* GetHighestWeightedValue() const
				{
					check(Data.HasContainerWeights());
					check(Collection.HighestWeightedIndex < Collection.DataPtrs.Num() && Collection.DataPtrs.IsValidIndex(Collection.HighestWeightedIndex));
					return Collection.DataPtrs[Collection.HighestWeightedIndex];
				}

				/** Returns highest (bone level) weighted value for the attribute set */
				const uint8* GetHighestBoneWeightedValue() const
				{
					check(Data.HasBoneWeights());

					int32 HighestIndex = INDEX_NONE;
					float Weight = -1.f;
					for (const int32 Index : Collection.WeightIndices)
					{
						const float BoneWeight = Data.GetBoneWeight(Index, Collection.Identifier->GetIndex());
						if (BoneWeight > Weight)
						{
							Weight = BoneWeight;
							HighestIndex = Index;
						}
					}
					ensure(HighestIndex != INDEX_NONE);
					return Collection.DataPtrs[Collection.WeightIndices.IndexOfByKey(HighestIndex)];
				}

				/** Returns highest (bone level) weighted value, and its weight for the attribute set */
				void GetHighestBoneWeighted(const uint8* OutAttributePtr, float& OutWeight) const
				{
					check(Data.HasBoneWeights());

					int32 HighestIndex = INDEX_NONE;
					float Weight = -1.f;
					for (const int32 Index : Collection.WeightIndices)
					{
						const float BoneWeight = Data.GetBoneWeight(Index, Collection.Identifier->GetIndex());
						if (BoneWeight > Weight)
						{
							Weight = BoneWeight;
							HighestIndex = Index;
						}
					}
					ensure(HighestIndex != INDEX_NONE);
					OutAttributePtr = Collection.DataPtrs[Collection.WeightIndices.IndexOfByKey(HighestIndex)];
					OutWeight = Weight;
				}

				/** Returns the identifier for the current attribute set */
				const FAttributeId& GetIdentifier() const
				{
					return *Collection.Identifier;
				}

				EAdditiveAnimationType GetAdditiveType() const
				{
					return Data.AdditiveType;
				}

				bool IsFilteredBlend() const
				{
					return Data.bPerBoneFilter;
				}

				/** Cycle through to next entry in the attribute set, returns false if the end was reached */
				bool Next()
				{
					++CurrentIndex;
					return CurrentIndex < Collection.DataPtrs.Num();
				}

				int32 GetIndex() const
				{
					return CurrentIndex;
				}
			protected:
				/** Outer object that creates this */
				const FAttributeBlendData& Data;
				/** Attribute collection for current index */
				const FAttributeSet& Collection;
				int32 CurrentIndex;
			};

			template<typename AttributeType>
			struct TSingleIterator
			{
				friend struct FAttributeBlendData;
			protected:
				TSingleIterator(const FAttributeBlendData& InData, TArrayView<const FUniqueAttribute> InAttributes) : Data(InData), AttributesView(InAttributes), CurrentIndex(-1) {}
			public:
				/** Cycle through to next unique attribute, returns false if the end was reached */
				bool Next()
				{
					++CurrentIndex;
					return CurrentIndex < AttributesView.Num();
				}

				/** Return the value for the currently indexed unique attribute */
				const AttributeType& GetValue() const
				{
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					return *(const AttributeType*)AttributesView[CurrentIndex].DataPtr;
				}

				/** Returns (container level) weight value for the unique attribute its container */
				const float GetWeight() const
				{
					check(Data.HasContainerWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					
					return Data.GetContainerWeight(AttributesView[CurrentIndex].WeightIndex);
				}
				
				/** Returns (bone level) weight value for the unique attribute its bone and container */
				const float GetBoneWeight() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					
					return Data.GetBoneWeight(AttributesView[CurrentIndex].WeightIndex, AttributesView[CurrentIndex].Identifier->GetIndex());
				}

				/** Returns whether or not the unique attribute its (bone level) weight is the highest across the containers */
				bool IsHighestBoneWeighted() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					check(Data.HighestBoneWeightedIndices.IsValidIndex(AttributesView[CurrentIndex].Identifier->GetIndex()));

					return Data.HighestBoneWeightedIndices[AttributesView[CurrentIndex].Identifier->GetIndex()] == AttributesView[CurrentIndex].WeightIndex;
				}

				/** Returns the identifier for the current attribute set */
				const FAttributeId& GetIdentifier() const
				{
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));					
					return *AttributesView[CurrentIndex].Identifier;
				}

				EAdditiveAnimationType GetAdditiveType() const
				{
					return Data.AdditiveType;
				}
				
				bool IsFilteredBlend() const
				{
					return Data.bPerBoneFilter;
				}
				
				const float GetWeightOverride(const FAttributeId& InAttribute) const
				{
					const float* FoundWeight = Data.AttributeWeightOverrides.Find(InAttribute);
					if (FoundWeight)
					{
						return *FoundWeight; 
					}
					
					return -1.0f;
				}

			protected:
				/** Outer object that creates this */
				const FAttributeBlendData& Data;
				TArrayView<const FUniqueAttribute> AttributesView;
				int32 CurrentIndex;
			};

			struct TSingleRawIterator
			{
				friend struct FAttributeBlendData;
			protected:
				TSingleRawIterator(const FAttributeBlendData& InData, TArrayView<const FUniqueAttribute> InAttributes) : Data(InData), AttributesView(InAttributes), CurrentIndex(-1) {}
			public:
				/** Cycle through to next unique attribute, returns false if the end was reached */
				bool Next()
				{
					++CurrentIndex;
					return CurrentIndex < AttributesView.Num();
				}

				/** Return the value for the currently indexed unique attribute */
				const uint8* GetValuePtr() const
				{
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					return AttributesView[CurrentIndex].DataPtr;
				}

				/** Returns (container level) weight value for the unique attribute its container */
				const float GetWeight() const
				{
					check(Data.HasContainerWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					
					return Data.GetContainerWeight(AttributesView[CurrentIndex].WeightIndex);
				}
				
				/** Returns (bone level) weight value for the unique attribute its bone and container */
				const float GetBoneWeight() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					
					return Data.GetBoneWeight(AttributesView[CurrentIndex].WeightIndex, AttributesView[CurrentIndex].Identifier->GetIndex());
				}

				/** Returns whether or not the unique attribute its (bone level) weight is the highest across the containers */
				bool IsHighestBoneWeighted() const
				{
					check(Data.HasBoneWeights());
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));
					check(Data.HighestBoneWeightedIndices.IsValidIndex(AttributesView[CurrentIndex].Identifier->GetIndex()));

					return Data.HighestBoneWeightedIndices[AttributesView[CurrentIndex].Identifier->GetIndex()] == AttributesView[CurrentIndex].WeightIndex;
				}

				/** Returns the identifier for the current attribute set */
				const FAttributeId& GetIdentifier() const
				{
					check(CurrentIndex < AttributesView.Num() && AttributesView.IsValidIndex(CurrentIndex));					
					return *AttributesView[CurrentIndex].Identifier;
				}

				EAdditiveAnimationType GetAdditiveType() const
				{
					return Data.AdditiveType;
				}
				
				bool IsFilteredBlend() const
				{
					return Data.bPerBoneFilter;
				}
				
				const float GetWeightOverride(const FAttributeId& InAttribute) const
				{
					const float* FoundWeight = Data.AttributeWeightOverrides.Find(InAttribute);
					if (FoundWeight)
					{
						return *FoundWeight; 
					}
					
					return -1.0f;
				}

			protected:
				/** Outer object that creates this */
				const FAttributeBlendData& Data;
				TArrayView<const FUniqueAttribute> AttributesView;
				int32 CurrentIndex;
			};

		public:
			template<typename AttributeType>
			void ForEachAttributeSet(TFunctionRef<void(TAttributeSetIterator<AttributeType>&)> ForEachFunction) const
			{
				for (const FAttributeSet& Collection : AttributeSets)
				{
					TAttributeSetIterator<AttributeType> It(*this, Collection);
					ForEachFunction(It);
				}
			}

			template<typename AttributeType>
			void ForEachUniqueAttribute(TFunctionRef<void(TSingleIterator<AttributeType>&)> ForEachFunction) const
			{
				TSingleIterator<AttributeType> It(*this, UniqueAttributes);
				ForEachFunction(It);
			}
			
			void ForEachAttributeSet(TFunctionRef<void(TAttributeSetRawIterator&)> ForEachFunction) const
			{
				for (const FAttributeSet& Collection : AttributeSets)
				{
					TAttributeSetRawIterator It(*this, Collection);
					ForEachFunction(It);
				}
			}

			void ForEachUniqueAttribute(TFunctionRef<void(TSingleRawIterator&)> ForEachFunction) const
			{
				TSingleRawIterator It(*this, UniqueAttributes);
				ForEachFunction(It);
			}
		};
	}
}
