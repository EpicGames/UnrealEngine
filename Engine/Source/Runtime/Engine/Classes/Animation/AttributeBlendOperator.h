// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AttributeBlendData.h"
#include "Animation/AttributeTraits.h"
#include "Animation/IAttributeBlendOperator.h"
#include "Animation/CustomAttributes.h"
#include "Templates/EnableIf.h"

#include "HAL/Platform.h"
 
namespace UE
{
	namespace Anim
	{
		/** Default blend operator used for any registered attribute type, when no user-defined operator has been specified
			Using TEnableIf to select appropriate behaviour according to TAttributeTypeTraits<Type>::IsBlendable value for AttributeType template */
		template<typename AttributeType>
		class TAttributeBlendOperator : public IAttributeBlendOperator
		{
		public:
			/** Begin IAttributeBlendOperator overrides */
			virtual void Accumulate(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final { Accumulate_Internal<AttributeType>(BlendData, OutAttributes); }
			virtual void AccumulateWithOverrideWeights(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final { AccumulateWithOverrideWeights_Internal<AttributeType>(BlendData, OutAttributes); }
			virtual void Interpolate(const void* FromData, const void* ToData, float Alpha, void* InOutData) const final { Interpolate_Internal<AttributeType>(FromData, ToData, Alpha, InOutData); }
			virtual void Override(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final { Override_Internal<AttributeType>(BlendData, OutAttributes); }
			virtual void Blend(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final { Blend_Internal<AttributeType>(BlendData, OutAttributes); }
			virtual void BlendPerBone(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final { BlendPerBone_Internal<AttributeType>(BlendData, OutAttributes); }
			virtual void ConvertToAdditive(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAdditiveAttributes) const final { ConvertToAdditive_Internal<AttributeType>(BlendData, OutAdditiveAttributes);	}
			/** End IAttributeBlendOperator overrides */

		protected:
			/** Blend operation for blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type Blend_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachAttributeSet<AttributeType>([OutAttributes](typename FAttributeBlendData::template TAttributeSetIterator<AttributeType>& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);

					AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
					AttributeType& OutAttribute = (*OutAttributePtr);

					if (BlendType == ECustomAttributeBlendType::Override)
					{
						// Not iterating over each attribute, but just picking highest weighted value
						OutAttribute = It.GetHighestWeightedValue();
					}
					else
					{
						// Iterate over each attribute while accumulating final value according to provided weightings
						while (It.Next())
						{
							const AttributeType& Attribute = It.GetValue();
							const float AttributeWeight = It.GetWeight();

							if (It.GetIndex() == 0)
							{
								OutAttribute = Attribute;
								OutAttribute = Attribute.Multiply(AttributeWeight);
							}
							else
							{
								OutAttribute.Accumulate(Attribute, AttributeWeight, It.GetAdditiveType());
							}
						}

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});

				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const float AttributeWeight = It.GetWeight();
						const FAttributeId& Identifier = It.GetIdentifier();
						const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);

						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = (*OutAttributePtr);
						
						// Choose between weighted (blended) assignment vs override
						if (BlendType == ECustomAttributeBlendType::Override)
						{
							OutAttribute = Attribute;
						}
						else
						{
							OutAttribute.Interpolate(Attribute, AttributeWeight);
						}

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});				
			}

			/** Blend operation for non-blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type Blend_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{			
				BlendData.ForEachAttributeSet<AttributeType>([OutAttributes](typename FAttributeBlendData::template TAttributeSetIterator<AttributeType>& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);
					
					// Not iterating over each attribute, but just picking highest weighted value
					OutAttributes->Add<AttributeType>(Identifier, It.GetHighestWeightedValue());			
				});

				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const FAttributeId& Identifier = It.GetIdentifier();
						OutAttributes->Add<AttributeType>(Identifier, It.GetValue());
					}
				});
			}

			/* Per-bone weighted blend operation for blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type BlendPerBone_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachAttributeSet<AttributeType>([this, OutAttributes](typename FAttributeBlendData::template TAttributeSetIterator<AttributeType>& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);

					const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
					AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
					AttributeType& OutAttribute = *OutAttributePtr;
					
					if (BlendType == ECustomAttributeBlendType::Override)
					{
						// Not iterating over each attribute, but just picking highest weighted value
						OutAttribute = It.GetHighestBoneWeightedValue();
					}
					else
					{
						while (It.Next())
						{
							const float AttributeWeight = It.GetBoneWeight();							
							const AttributeType& Attribute = It.GetValue();
							if (FAnimWeight::IsFullWeight(AttributeWeight))
							{
								OutAttribute = Attribute;
								break;
							}
							else
							{
								if (It.GetIndex() == 0 && ExistingIndex == INDEX_NONE)
								{
									OutAttribute = Attribute.Multiply(AttributeWeight);
								}
								else
								{
									OutAttribute.Accumulate(Attribute, AttributeWeight, It.GetAdditiveType());
								}
							}
						}
						
						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});

				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);

						const float AttributeWeight = It.GetBoneWeight();
						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;

						const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);
						if (BlendType == ECustomAttributeBlendType::Override)
						{
							OutAttribute = It.GetValue();
						}
						else
						{
							const AttributeType& Attribute = It.GetValue();
							if (ExistingIndex == INDEX_NONE)
							{
								OutAttribute.Interpolate(Attribute, AttributeWeight);
							}
							else
							{
								OutAttribute.Accumulate(Attribute, AttributeWeight, It.GetAdditiveType());
							}
						}

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});
			}

			/* Per-bone weighted blend operation for non-blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type BlendPerBone_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachAttributeSet<AttributeType>([OutAttributes](typename FAttributeBlendData::template TAttributeSetIterator<AttributeType>& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					AttributeType* OutAttribute = OutAttributes->FindOrAdd<AttributeType>(Identifier);
					AttributeType::StaticStruct()->CopyScriptStruct(OutAttribute, &It.GetHighestBoneWeightedValue());
				});

				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
						const float BoneWeight = It.GetBoneWeight();

						if (FAnimWeight::IsRelevant(BoneWeight) || ExistingIndex == INDEX_NONE)
						{
							AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
							AttributeType& OutAttribute = *OutAttributePtr;
							const AttributeType& Attribute = It.GetValue();

							// 'Blend' value if the attribute did not yet exist, or based upon being the highest weighted attribute
							const bool bHighestWeight = It.IsHighestBoneWeighted();
							if (bHighestWeight || ExistingIndex == INDEX_NONE)
							{
								AttributeType::StaticStruct()->CopyScriptStruct(OutAttributePtr, &Attribute);
							}
						}
					}
				});				
			}
			
			/* Override operation for blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type Override_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();

						// Find or add as the attribute might already exist, so Add would fail
						AttributeType& OutAttribute = *OutAttributes->FindOrAdd<AttributeType>(Identifier);

						const float AttributeWeight = It.GetWeight();
						OutAttribute = Attribute.Multiply(AttributeWeight);

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});
			}

			/* Override operation for non-blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type Override_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();

						// Find or add as the attribute might already exist, so Add would fail
						AttributeType& OutAttribute = *OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType::StaticStruct()->CopyScriptStruct(&OutAttribute, &Attribute);
					}
				});
			}			

			/* Accumulate operation for blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type Accumulate_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;
						
						// Accumulate with weighted add
						const float AttributeWeight = It.GetWeight();
						if (ExistingIndex != INDEX_NONE)
						{						
							OutAttribute.Accumulate(Attribute, AttributeWeight, It.GetAdditiveType());
						}
						else
						{
							OutAttribute = Attribute.Multiply(AttributeWeight);							
						}

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});
			}

			/* Accumulate operation for non-blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type Accumulate_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{						
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;

						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						const float AttributeWeight = It.GetWeight();
						if (ExistingIndex == INDEX_NONE || AttributeWeight > 0.5f)
						{
							AttributeType::StaticStruct()->CopyScriptStruct(OutAttributePtr, &Attribute);
						}
					}
				});
			}
			
			/* Accumulate operation for blendable attribute types with optional per attribute weight overrides */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type AccumulateWithOverrideWeights_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;
						
						float AttributeWeight = It.GetWeightOverride(Identifier);
						if (AttributeWeight <= -1.0)
						{
							AttributeWeight = It.GetWeight();
						}
						
						// Accumulate with weighted add
						if (ExistingIndex != INDEX_NONE)
						{						
							OutAttribute.Accumulate(Attribute, AttributeWeight, It.GetAdditiveType());
						}
						else
						{
							OutAttribute = Attribute.Multiply(AttributeWeight);							
						}

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});
			}

			/* Accumulate operation for non-blendable attribute types with optional per attribute weight overrides */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type AccumulateWithOverrideWeights_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{						
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;
						
						float AttributeWeight = It.GetWeightOverride(Identifier);
						if (AttributeWeight <= -1.0)
						{
							AttributeWeight = It.GetWeight();
						}
						
						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						if (ExistingIndex == INDEX_NONE || AttributeWeight > 0.5f)
						{
							AttributeType::StaticStruct()->CopyScriptStruct(OutAttributePtr, &Attribute);
						}
					}
				});
			}

			/* Make additive operation for blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable, void>::Type ConvertToAdditive_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAdditiveAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAdditiveAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAdditiveAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAdditiveAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;

						OutAttribute.MakeAdditive(Attribute);

						if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
						{
							OutAttribute.Normalize();
						}
					}
				});
			}

			/* Make additive operation for non-blendable attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable, void>::Type ConvertToAdditive_Internal(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAdditiveAttributes) const
			{
				BlendData.ForEachUniqueAttribute<AttributeType>([OutAdditiveAttributes](typename FAttributeBlendData::template TSingleIterator<AttributeType>& It) -> void
				{
					while (It.Next())
					{
						const AttributeType& Attribute = It.GetValue();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAdditiveAttributes->IndexOfByKey<AttributeType>(Identifier);
						AttributeType* OutAttributePtr = OutAdditiveAttributes->FindOrAdd<AttributeType>(Identifier);
						AttributeType& OutAttribute = *OutAttributePtr;

						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						const float AttributeWeight = It.GetWeight();
						if (ExistingIndex == INDEX_NONE)
						{							
							AttributeType::StaticStruct()->CopyScriptStruct(OutAttributePtr, &Attribute);
						}
					}
				});
			}
			
			/* Interpolate operation for blendable; non-step interpolated attribute types */
			template <typename Type>
			inline typename TEnableIf<TAttributeTypeTraits<Type>::IsBlendable && !TAttributeTypeTraits<AttributeType>::StepInterpolate, void>::Type Interpolate_Internal(const void* FromData, const void* ToData, float Alpha, void* InOutData) const
			{
				const AttributeType& TypedFrom = *(const AttributeType*)FromData;
				const AttributeType& TypedTo = *(const AttributeType*)ToData;
				AttributeType& Output = *(AttributeType*)InOutData;				

				Output = TypedFrom;
				Output.Interpolate(TypedTo, Alpha);

				if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
				{
					Output.Normalize();
				}
			}

			/* Interpolate operation for non-blendable; step interpolated attribute types */
			template <typename Type>
			inline typename TEnableIf<!TAttributeTypeTraits<Type>::IsBlendable || TAttributeTypeTraits<AttributeType>::StepInterpolate, void>::Type Interpolate_Internal(const void* FromData, const void* ToData, float Alpha, void* InOutData) const
			{
				// Determine stepped value to copy
				const void* CopySource = Alpha > 0.5f ? ToData : FromData;
				// Using CopyScriptStruct as assignment operator might not be implemented for attribute type
				AttributeType::StaticStruct()->CopyScriptStruct(InOutData, CopySource);
			}
		};

		class FNonBlendableAttributeBlendOperator : public IAttributeBlendOperator
		{
		public:
			FNonBlendableAttributeBlendOperator(const UScriptStruct* InScriptStruct) : ScriptStructPtr(InScriptStruct) {}; 
			
			/** Begin IAttributeBlendOperator overrides */
			virtual void Accumulate(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final
			{
				BlendData.ForEachUniqueAttribute([&, OutAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{						
						const uint8* Attribute= It.GetValuePtr();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey(ScriptStructPtr.Get(), Identifier);
						uint8* OutAttributePtr = OutAttributes->FindOrAdd(ScriptStructPtr.Get(), Identifier);

						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						const float AttributeWeight = It.GetWeight();
						if (ExistingIndex == INDEX_NONE || AttributeWeight > 0.5f)
						{
							ScriptStructPtr.Get()->CopyScriptStruct(OutAttributePtr, Attribute);
						}
					}
				});
			};
			
			virtual void AccumulateWithOverrideWeights(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final
			{
				BlendData.ForEachUniqueAttribute([&, OutAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{						
						const uint8* Attribute= It.GetValuePtr();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey(ScriptStructPtr.Get(), Identifier);
						uint8* OutAttributePtr = OutAttributes->FindOrAdd(ScriptStructPtr.Get(), Identifier);

						float AttributeWeight = It.GetWeightOverride(Identifier);
						if (AttributeWeight <= -1.0)
						{
							AttributeWeight = It.GetWeight();
						}
						
						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						if (ExistingIndex == INDEX_NONE || AttributeWeight > 0.5f)
						{
							ScriptStructPtr.Get()->CopyScriptStruct(OutAttributePtr, Attribute);
						}
					}
				});
			};
			
			virtual void Interpolate(const void* FromData, const void* ToData, float Alpha, void* InOutData) const final
			{
				// Determine stepped value to copy
				const void* CopySource = Alpha > 0.5f ? ToData : FromData;
				// Using CopyScriptStruct as assignment operator might not be implemented for attribute type
				ScriptStructPtr.Get()->CopyScriptStruct(InOutData, CopySource);	
			};
			virtual void Override(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final
			{
				BlendData.ForEachUniqueAttribute([&, OutAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{
						const uint8* Attribute = It.GetValuePtr();
						const FAttributeId& Identifier = It.GetIdentifier();

						// Find or add as the attribute might already exist, so Add would fail
						uint8* OutAttribute = OutAttributes->FindOrAdd(ScriptStructPtr.Get(),Identifier);
						ScriptStructPtr.Get()->CopyScriptStruct(OutAttribute, Attribute);
					}
				});
			};
			virtual void Blend(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final
			{
				BlendData.ForEachAttributeSet([&, OutAttributes](FAttributeBlendData::TAttributeSetRawIterator& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					const ECustomAttributeBlendType BlendType = Attributes::GetAttributeBlendType(Identifier);
					
					// Not iterating over each attribute, but just picking highest weighted value
					uint8* OutAttribute = OutAttributes->Add(ScriptStructPtr.Get(), Identifier);

					ScriptStructPtr.Get()->CopyScriptStruct(OutAttribute, It.GetHighestWeightedValue());
				});

				BlendData.ForEachUniqueAttribute([&, OutAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{
						const FAttributeId& Identifier = It.GetIdentifier();
						
						uint8* OutAttribute = OutAttributes->Add(ScriptStructPtr.Get(), Identifier);
						ScriptStructPtr.Get()->CopyScriptStruct(OutAttribute, It.GetValuePtr());	
					}
				});
			};
			virtual void BlendPerBone(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const final
			{
				BlendData.ForEachAttributeSet([&, OutAttributes](FAttributeBlendData::TAttributeSetRawIterator& It) -> void
				{
					const FAttributeId& Identifier = It.GetIdentifier();
					uint8* OutAttribute = OutAttributes->FindOrAdd(ScriptStructPtr.Get(), Identifier);
					ScriptStructPtr.Get()->CopyScriptStruct(OutAttribute, It.GetHighestBoneWeightedValue());
				});

				BlendData.ForEachUniqueAttribute([&, OutAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAttributes->IndexOfByKey(ScriptStructPtr.Get() ,Identifier);
						const float BoneWeight = It.GetBoneWeight();

						if (FAnimWeight::IsRelevant(BoneWeight) || ExistingIndex == INDEX_NONE)
						{
							uint8* OutAttributePtr = OutAttributes->FindOrAdd(ScriptStructPtr.Get(), Identifier);
							const uint8* Attribute = It.GetValuePtr();

							// 'Blend' value if the attribute did not yet exist, or based upon being the highest weighted attribute
							const bool bHighestWeight = It.IsHighestBoneWeighted();
							if (bHighestWeight || ExistingIndex == INDEX_NONE)
							{
								ScriptStructPtr.Get()->CopyScriptStruct(OutAttributePtr, Attribute);
							}
						}
					}
				});
			};
			virtual void ConvertToAdditive(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAdditiveAttributes) const final
			{
				BlendData.ForEachUniqueAttribute([&, OutAdditiveAttributes](FAttributeBlendData::TSingleRawIterator& It) -> void
				{
					while (It.Next())
					{
						const uint8* Attribute = It.GetValuePtr();
						const FAttributeId& Identifier = It.GetIdentifier();
						const int32 ExistingIndex = OutAdditiveAttributes->IndexOfByKey(ScriptStructPtr.Get(), Identifier);
						uint8* OutAttributePtr = OutAdditiveAttributes->FindOrAdd(ScriptStructPtr.Get(),Identifier);

						// 'Accumulate' value if the attribute did not yet exist, or based upon being the highest weighted attribute
						const float AttributeWeight = It.GetWeight();
						if (ExistingIndex == INDEX_NONE)
						{							
							ScriptStructPtr.Get()->CopyScriptStruct(OutAttributePtr, &Attribute);
						}
					}
				});
			};

		protected:
			TWeakObjectPtr<const UScriptStruct> ScriptStructPtr;
		};
	}
}