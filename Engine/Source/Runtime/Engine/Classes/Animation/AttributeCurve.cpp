// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeCurve.h"

#include "Animation/AttributeTypes.h"
#include "Animation/IAttributeBlendOperator.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeCurve)

FAttributeCurve::FAttributeCurve(const FAttributeCurve& OtherCurve)
{
	Keys = OtherCurve.Keys;
	ScriptStructPath = OtherCurve.ScriptStructPath;
	ScriptStruct = OtherCurve.ScriptStruct;
	Operator = OtherCurve.Operator;
	bShouldInterpolate = OtherCurve.bShouldInterpolate;
}

void FAttributeCurve::SetScriptStruct(UScriptStruct* InScriptStruct)
{
	if (InScriptStruct && ScriptStruct != InScriptStruct && (Keys.Num() == 0))
	{
		ScriptStruct = InScriptStruct;
		ScriptStructPath = InScriptStruct;

		Operator = UE::Anim::AttributeTypes::GetTypeOperator(ScriptStruct);
		ensure(Operator);
		bShouldInterpolate = UE::Anim::AttributeTypes::CanInterpolateType(ScriptStruct);
	}
}

bool FAttributeCurve::CanEvaluate() const
{
	return ScriptStruct != nullptr && Keys.Num() > 0;
}

void FAttributeCurve::SetKeyTime(FKeyHandle KeyHandle, float NewTime)
{
	if (IsKeyHandleValid(KeyHandle))
	{
		const FAttributeKey OldKey = GetKey(KeyHandle);

		DeleteKey(KeyHandle);
		AddKey(NewTime, (void*)OldKey.Value.GetPtr<void>(), KeyHandle);
		
		// Copy all properties from old key, but then fix time to be the new time
		FAttributeKey& NewKey = GetKey(KeyHandle);
		NewKey = OldKey;
		NewKey.Time = NewTime;
	}
}

float FAttributeCurve::GetKeyTime(FKeyHandle KeyHandle) const
{
	if (!IsKeyHandleValid(KeyHandle))
	{
		return 0.f;
	}

	return GetKey(KeyHandle).Time;
}

void FAttributeCurve::EvaluateToPtr(const UScriptStruct* InScriptStruct, float Time, uint8* InOutDataPtr) const
{
	if (CanEvaluate() && InScriptStruct == ScriptStruct)
	{
		const void* DataPtr = Keys[0].Value.GetPtr<void>();

		if (bShouldInterpolate)
		{	
			const int32 NumKeys = Keys.Num();
			if (NumKeys == 0)
			{
				ensure(false);
				// If no keys in curve, return the Default value.
			}
			else if (NumKeys < 2 || (Time <= Keys[0].Time))
			{
				DataPtr = Keys[0].Value.GetPtr<void>();
			}
			else if (Time < Keys[NumKeys - 1].Time)
			{
				// perform a lower bound to get the second of the interpolation nodes
				int32 first = 1;
				int32 last = NumKeys - 1;
				int32 count = last - first;

				while (count > 0)
				{
					int32 step = count / 2;
					int32 middle = first + step;

					if (Time >= Keys[middle].Time)
					{
						first = middle + 1;
						count -= step + 1;
					}
					else
					{
						count = step;
					}
				}

				const FAttributeKey& Key = Keys[first - 1];
				const FAttributeKey& Key1 = Keys[first];

				const float Diff = Key1.Time - Key.Time;
				if (Diff > 0.f)
				{
					const float Alpha = (Time - Key.Time) / Diff;
					Operator->Interpolate(Key.Value.GetPtr<void>(), Key1.Value.GetPtr<void>(), Alpha, InOutDataPtr);
					return;
				}
				else
				{
					DataPtr = Key.Value.GetPtr<void>();
				}
			}
			else
			{
				// Key is beyon the last point in the curve.  Return it's value
				DataPtr = Keys[Keys.Num() - 1].Value.GetPtr<void>();
			}
		}
		else
		{
			if (Keys.Num() == 0 || (Time < Keys[0].Time))
			{
				// If no keys in curve, or bUseDefaultValueBeforeFirstKey is set and the time is before the first key, return the Default value.
			}
			else if (Keys.Num() < 2 || Time < Keys[0].Time)
			{
				// There is only one key or the time is before the first value. Return the first value
				DataPtr = Keys[0].Value.GetPtr<void>();
			}
			else if (Time < Keys[Keys.Num() - 1].Time)
			{
				// The key is in the range of Key[0] to Keys[Keys.Num()-1].  Find it by searching
				for (int32 i = 0; i < Keys.Num(); ++i)
				{
					if (Time < Keys[i].Time)
					{
						DataPtr = Keys[FMath::Max(0, i - 1)].Value.GetPtr<void>();
						break;
					}
				}
			}
			else
			{
				// Key is beyon the last point in the curve.  Return it's value
				DataPtr = Keys[Keys.Num() - 1].Value.GetPtr<void>();
			}
		}

		ScriptStruct->CopyScriptStruct(InOutDataPtr, DataPtr, 1);
	}
}

bool FAttributeCurve::HasAnyData() const
{
	return Keys.Num() != 0;
}

TArray<FAttributeKey>::TConstIterator FAttributeCurve::GetKeyIterator() const
{
	return Keys.CreateConstIterator();
}

FKeyHandle FAttributeCurve::AddKey(float InTime, const void* InValue, FKeyHandle InKeyHandle)
{
	int32 Index = 0;
	for (; Index < Keys.Num() && Keys[Index].Time < InTime; ++Index);

	FAttributeKey& NewKey = Keys.Insert_GetRef(FAttributeKey(InTime), Index);
	NewKey.Value.Allocate(ScriptStruct);
	ScriptStruct->CopyScriptStruct(NewKey.Value.GetPtr<void>(), InValue);
	

	KeyHandlesToIndices.Add(InKeyHandle, Index);

	return GetKeyHandle(Index);
}

void FAttributeCurve::DeleteKey(FKeyHandle InKeyHandle)
{
	int32 Index = GetIndex(InKeyHandle);

	Keys.RemoveAt(Index);

	KeyHandlesToIndices.Remove(InKeyHandle);
}

FKeyHandle FAttributeCurve::UpdateOrAddKey(float InTime, const void* InValue, float KeyTimeTolerance)
{
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		float KeyTime = Keys[KeyIndex].Time;

		if (FMath::IsNearlyEqual(KeyTime, InTime, KeyTimeTolerance))
		{
			ScriptStruct->CopyScriptStruct(Keys[KeyIndex].Value.GetPtr<void>(), InValue, 1);
			return GetKeyHandle(KeyIndex);
		}

		if (KeyTime > InTime)
		{
			// All the rest of the keys exist after the key we want to add
			// so there is no point in searching
			break;
		}
	}

	// A key wasnt found, add it now
	return AddKey(InTime, InValue);
}

FAttributeKey& FAttributeCurve::GetKey(FKeyHandle KeyHandle)
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

const FAttributeKey& FAttributeCurve::GetKey(FKeyHandle KeyHandle) const
{
	EnsureAllIndicesHaveHandles();
	return Keys[GetIndex(KeyHandle)];
}

FKeyHandle FAttributeCurve::FindKey(float KeyTime, float KeyTimeTolerance) const
{
	int32 Start = 0;
	int32 End = Keys.Num() - 1;

	// Binary search since the keys are in sorted order
	while (Start <= End)
	{
		int32 TestPos = Start + (End - Start) / 2;
		float TestKeyTime = Keys[TestPos].Time;

		if (FMath::IsNearlyEqual(TestKeyTime, KeyTime, KeyTimeTolerance))
		{
			return GetKeyHandle(TestPos);
		}
		else if (TestKeyTime < KeyTime)
		{
			Start = TestPos + 1;
		}
		else
		{
			End = TestPos - 1;
		}
	}

	return FKeyHandle::Invalid();
}

FKeyHandle FAttributeCurve::FindKeyBeforeOrAt(float KeyTime) const
{
	// If there are no keys or the time is before the first key return an invalid handle.
	if (Keys.Num() == 0 || KeyTime < Keys[0].Time)
	{
		return FKeyHandle::Invalid();
	}

	// If the time is after or at the last key return the last key.
	if (KeyTime >= Keys[Keys.Num() - 1].Time)
	{
		return GetKeyHandle(Keys.Num() - 1);
	}

	// Otherwise binary search to find the handle of the nearest key at or before the time.
	int32 Start = 0;
	int32 End = Keys.Num() - 1;
	int32 FoundIndex = -1;
	while (FoundIndex < 0)
	{
		int32 TestPos = (Start + End) / 2;
		float TestKeyTime = Keys[TestPos].Time;
		float NextTestKeyTime = Keys[TestPos + 1].Time;
		if (TestKeyTime <= KeyTime)
		{
			if (NextTestKeyTime > KeyTime)
			{
				FoundIndex = TestPos;
			}
			else
			{
				Start = TestPos + 1;
			}
		}
		else
		{
			End = TestPos;
		}
	}
	return GetKeyHandle(FoundIndex);
}

void FAttributeCurve::RemoveRedundantKeys()
{
	TSet<int32> KeyIndicesToRemove;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		if (KeyIndex + 2 < Keys.Num())
		{
			const FAttributeKey& CurrentKey = Keys[KeyIndex];
			const FAttributeKey& NextKeyOne = Keys[KeyIndex + 1];
			const FAttributeKey& NextKeyTwo = Keys[KeyIndex + 2];

			if (ScriptStruct->CompareScriptStruct(CurrentKey.Value.GetPtr<void>(), NextKeyOne.Value.GetPtr<void>(), 0)
				&& ScriptStruct->CompareScriptStruct(NextKeyOne.Value.GetPtr<void>(), NextKeyTwo.Value.GetPtr<void>(), 0))
			{
				KeyIndicesToRemove.Add(KeyIndex + 1);
			}
		}
	}

	if (KeyIndicesToRemove.Num())
	{
	    TArray<FAttributeKey> NewKeys;
		NewKeys.Reserve(Keys.Num() - KeyIndicesToRemove.Num());
	    for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	    {
		    if (!KeyIndicesToRemove.Contains(KeyIndex))
		    {
			    NewKeys.Add(Keys[KeyIndex]);
		    }
	    }
    
	    Swap(Keys, NewKeys);
	    KeyHandlesToIndices.Empty(Keys.Num());
	    KeyHandlesToIndices.SetKeyHandles(Keys.Num());
	}

	// If only two keys left and they are identical as well, remove the 2nd one.
	if (Keys.Num() == 2 && ScriptStruct->CompareScriptStruct(Keys[0].Value.GetPtr<void>(), Keys[1].Value.GetPtr<void>(), 0))
	{
		DeleteKey(GetKeyHandle(1));
	}
}

bool FAttributeCurve::Serialize(FArchive& Ar)
{
	Ar << Keys;
	Ar << ScriptStructPath;

	if (!ScriptStructPath.IsNull())
	{
		if (Ar.IsSaving())
		{
			ensure(ScriptStruct);
			for (FAttributeKey& Key : Keys)
			{
				ScriptStruct->SerializeItem(Ar, Key.Value.GetPtr<void>(), nullptr);
			}
		}
		else if (Ar.IsLoading())
		{
			ScriptStruct = Cast<UScriptStruct>(ScriptStructPath.ResolveObject());
			ensure(ScriptStruct);

			for (FAttributeKey& Key : Keys)
			{
				Key.Value.Allocate(ScriptStruct);
				ScriptStruct->SerializeItem(Ar, Key.Value.GetPtr<void>(), nullptr);
			}

			Operator = UE::Anim::AttributeTypes::GetTypeOperator(ScriptStruct);
			ensure(Operator);

			bShouldInterpolate = UE::Anim::AttributeTypes::CanInterpolateType(ScriptStruct);
		}
	}	

	return true;
}

void FAttributeCurve::Reset()
{
	Keys.Empty();
	KeyHandlesToIndices.Empty();
}

void FAttributeCurve::SetKeys(TArrayView<const float> InTimes, TArrayView<const void*> InValues)
{
	check(InTimes.Num() == InValues.Num());

	Reset();

	Keys.SetNum(InTimes.Num());
	KeyHandlesToIndices.SetKeyHandles(InTimes.Num());
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
	{
		Keys[KeyIndex].Time = InTimes[KeyIndex];
		Keys[KeyIndex].Value.Allocate(ScriptStruct);

		ScriptStruct->CopyScriptStruct(Keys[KeyIndex].Value.GetPtr<void>(), InValues[KeyIndex]);
	}
}

void FAttributeCurve::ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime)
{
	EnsureAllIndicesHaveHandles();

	// first readjust modified time keys
	float ModifiedDuration = OldEndTime - OldStartTime;

	if (bInsert)
	{
		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			if (CurrentTime >= OldStartTime)
			{
				CurrentTime += ModifiedDuration;
			}
		}
	}
	else
	{
		// since we only allow one key at a given time, we will just cache the value that needs to be saved
		// this is the key to be replaced when this section is gone
		bool bAddNewKey = false;
		FWrappedAttribute NewValue;
		NewValue.Allocate(ScriptStruct);

		TArray<int32> KeysToDelete;

		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			float& CurrentTime = Keys[KeyIndex].Time;
			// if this key exists between range of deleted
			// we'll evaluate the value at the "OldStartTime"
			// and re-add key, so that it keeps the previous value at the
			// start time
			// But that means if there are multiple keys, 
			// since we don't want multiple values in the same time
			// the last one will override the value
			if (CurrentTime >= OldStartTime && CurrentTime <= OldEndTime)
			{
				// get new value and add new key on one of OldStartTime, OldEndTime;
				// this is a bit complicated problem since we don't know if OldStartTime or OldEndTime is preferred. 
				// generall we use OldEndTime unless OldStartTime == 0.f
				// which means it's cut in the beginning. Otherwise it will always use the end time. 
				bAddNewKey = true;
				if (OldStartTime != 0.f)
				{	
					ScriptStruct->InitializeDefaultValue(NewValue.GetPtr<uint8>());
					EvaluateToPtr(ScriptStruct, OldStartTime, NewValue.GetPtr<uint8>());
				}
				else
				{
					ScriptStruct->InitializeDefaultValue(NewValue.GetPtr<uint8>());
					EvaluateToPtr(ScriptStruct, OldEndTime, NewValue.GetPtr<uint8>());
				}
				// remove this key, but later because it might change eval result
				KeysToDelete.Add(KeyIndex);
			}
			else if (CurrentTime > OldEndTime)
			{
				CurrentTime -= ModifiedDuration;
			}
		}

		if (bAddNewKey)
		{
			for (int32 KeyIndex = KeysToDelete.Num() - 1; KeyIndex >= 0; --KeyIndex)
			{
				const FKeyHandle* KeyHandle = KeyHandlesToIndices.FindKey(KeysToDelete[KeyIndex]);
				if (KeyHandle)
				{
					DeleteKey(*KeyHandle);
				}

			}

			UpdateOrAddKey(OldStartTime, (void*)NewValue.GetPtr<void>());
		}
	}

	// now remove all redundant key
	RemoveRedundantKeys();

	// now cull out all out of range 
	float MinTime, MaxTime;

	if (Keys.Num() == 0)
	{
		MinTime = 0.f;
		MaxTime = 0.f;
	}
	else
	{
		MinTime = Keys[0].Time;
		MaxTime = Keys[Keys.Num() - 1].Time;
	}

	bool bNeedToDeleteKey = false;

	// if there is key below min time, just add key at new min range, 
	if (MinTime < NewMinTimeRange)
	{
		FWrappedAttribute NewValue;
		NewValue.Allocate(ScriptStruct);
		EvaluateToPtr(ScriptStruct, NewMinTimeRange, NewValue.GetPtr<uint8>());

		UpdateOrAddKey(NewMinTimeRange, (void*)NewValue.GetPtr<void>());

		bNeedToDeleteKey = true;
	}

	// if there is key after max time, just add key at new max range, 
	if (MaxTime > NewMaxTimeRange)
	{
		FWrappedAttribute NewValue;
		NewValue.Allocate(ScriptStruct);
		EvaluateToPtr(ScriptStruct, NewMaxTimeRange, NewValue.GetPtr<uint8>());

		UpdateOrAddKey(NewMaxTimeRange, (void*)NewValue.GetPtr<void>());

		bNeedToDeleteKey = true;
	}

	// delete the keys outside of range
	if (bNeedToDeleteKey)
	{
		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			if (Keys[KeyIndex].Time < NewMinTimeRange || Keys[KeyIndex].Time > NewMaxTimeRange)
			{
				const FKeyHandle* KeyHandle = KeyHandlesToIndices.FindKey(KeyIndex);
				if (KeyHandle)
				{
					DeleteKey(*KeyHandle);
					--KeyIndex;
				}
			}
		}
	}
}

TArray<FAttributeKey> FAttributeCurve::GetCopyOfKeys() const
{
	return Keys;
}

const TArray<FAttributeKey>& FAttributeCurve::GetConstRefOfKeys() const
{
	return Keys;
}
