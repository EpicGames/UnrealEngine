// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "Serialization/Archive.h"

#include "Animation/WrappedAttribute.h"
#include "Animation/IAttributeBlendOperator.h"

#include "AttributeCurve.generated.h"

namespace UE { namespace Anim { class IAttributeBlendOperator; struct Attributes; } }
namespace UE::UAF { class FDecompressionTools; }

typedef UE::Anim::TWrappedAttribute<FDefaultAllocator> FWrappedAttribute;

USTRUCT()
struct FAttributeKey
{
	GENERATED_USTRUCT_BODY()
public:
	 
	FAttributeKey(float InTime = 0.f) : Time(InTime) {}

	/** The keyed time */
	UPROPERTY(EditAnywhere, Category = Key)
	float Time;

	template<typename AttributeType>
	const AttributeType& GetValue() const
	{
		return Value.GetRef<AttributeType>();
	}

	template<typename AttributeType>
	const AttributeType* GetValuePtr() const
	{
		return Value.GetPtr<AttributeType>();
	}

	
	friend FArchive& operator<<(FArchive& Ar, FAttributeKey& P)
	{
		Ar << P.Time;
		return Ar;
	}

protected:
	/** Value for this key, populated by FAttributeCurve during serialization */
	FWrappedAttribute Value;

	friend struct FAttributeCurve;
};

USTRUCT(BlueprintType)
struct FAttributeCurve : public FIndexedCurve
{
	GENERATED_USTRUCT_BODY()
public:
	FAttributeCurve() : ScriptStruct(nullptr), bShouldInterpolate(false), Operator(nullptr) {}
	FAttributeCurve(UScriptStruct* InScriptStruct) : ScriptStructPath(InScriptStruct), ScriptStruct(InScriptStruct), bShouldInterpolate(false), Operator(nullptr) {}

	ENGINE_API FAttributeCurve(const FAttributeCurve& OtherCurve);

	/** Virtual destructor. */
	virtual ~FAttributeCurve() = default;

	ENGINE_API bool Serialize(FArchive& Ar);

	/** Begin FIndexedCurve overrides */
	virtual int32 GetNumKeys() const override final { return Keys.Num(); }
	virtual FAttributeCurve* Duplicate() const final { return new FAttributeCurve(*this); }
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) override final;
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const override final;
	/** End FIndexedCurve overrides */
	
	/** Sets the underlying type for the curve, only possible when not containing any keys (see ::Reset) */
	ENGINE_API void SetScriptStruct(UScriptStruct* InScriptStruct);
	const UScriptStruct* GetScriptStruct() const { return ScriptStruct; }

	/** Whether or not the curve can be evaluated, based upon having a valid type and any keys */
	ENGINE_API bool CanEvaluate() const;

	/** Evaluate the curve keys into a temporary value container */
	template<typename AttributeType>
	AttributeType Evaluate(float Time) const
	{
		AttributeType EvaluatedValue;
		EvaluateToPtr(AttributeType::StaticStruct(), Time, (uint8*)&EvaluatedValue);
		return EvaluatedValue;
	}

	/** Check whether this curve has any data or not */
	ENGINE_API bool HasAnyData() const;

	/** Removes all key data */
	ENGINE_API void Reset();

	/** Const iterator for the keys, so the indices and handles stay valid */
	ENGINE_API TArray<FAttributeKey>::TConstIterator GetKeyIterator() const;

	/** Add a new typed key to the curve with the supplied Time and Value. */
	template<typename AttributeType>
	FKeyHandle AddTypedKey(float InTime, const AttributeType& InValue, FKeyHandle InKeyHandle = FKeyHandle())
	{
		check(AttributeType::StaticStruct() == ScriptStruct); 
		return AddKey(InTime, &InValue, InKeyHandle);
	}

	/** Remove the specified key from the curve.*/
	ENGINE_API void DeleteKey(FKeyHandle KeyHandle);

	/** Finds the key at InTime, and updates its typed value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	template<typename AttributeType>
	FKeyHandle UpdateOrAddTypedKey(float InTime, const AttributeType& InValue, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER)
	{
		check(AttributeType::StaticStruct() == ScriptStruct);
		return UpdateOrAddKey(InTime, &InValue, KeyTimeTolerance);
	}

	/** Finds the key at InTime, and updates its typed value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	FKeyHandle UpdateOrAddTypedKey(float InTime, const void* InValue, const UScriptStruct* ValueType, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER)
	{
		check(ValueType == ScriptStruct);
		return UpdateOrAddKey(InTime, InValue, KeyTimeTolerance);
	}
			
	/** Functions for getting keys based on handles */
	ENGINE_API FAttributeKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API const FAttributeKey& GetKey(FKeyHandle KeyHandle) const;

	/** Finds the key at KeyTime and returns its handle. If it can't find the key within the KeyTimeTolerance, it will return an invalid handle */
	ENGINE_API FKeyHandle FindKey(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Gets the handle for the last key which is at or before the time requested.  If there are no keys at or before the requested time, an invalid handle is returned. */
	ENGINE_API FKeyHandle FindKeyBeforeOrAt(float KeyTime) const;

	/** Tries to reduce the number of keys required for accurate evaluation (zero error threshold) */
	ENGINE_API void RemoveRedundantKeys();
	ENGINE_API void SetKeys(TArrayView<const float> InTimes, TArrayView<const void*> InValues);

	/** Populates OutKeys with typed value-ptrs */
	template<typename AttributeType>
	void GetTypedKeys(TArray<const AttributeType*>& OutKeys) const
	{
		for (const FAttributeKey& Key : Keys)
		{
			OutKeys.Add(Key.Value.GetPtr<AttributeType>());
		}
	}

	/** Return copy of contained key-data */
	ENGINE_API TArray<FAttributeKey> GetCopyOfKeys() const;
	ENGINE_API const TArray<FAttributeKey>& GetConstRefOfKeys() const;

	/** Used for adjusting the internal key-data when owning object its playlength changes */
	ENGINE_API void ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime);

protected:
	/** Evaluate the curve keys into the provided memory (should be appropriatedly sized) */
	ENGINE_API void EvaluateToPtr(const UScriptStruct* InScriptStruct, float Time, uint8* InOutDataPtr) const;

	/** Finds the key at InTime, and updates its typed value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	ENGINE_API FKeyHandle UpdateOrAddKey(float InTime, const void* InValue, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER);

	/** Add a new raw memory key (should be appropriately sized) to the curve with the supplied Time and Value. */
	ENGINE_API FKeyHandle AddKey(float InTime, const void* InValue, FKeyHandle InKeyHandle = FKeyHandle());
protected:
	/** The keys, ordered by time */
	UPROPERTY(EditAnywhere, Category = "Custom Attributes")
	TArray<FAttributeKey> Keys;	

	/* Path to UScriptStruct to be loaded */
	UPROPERTY(VisibleAnywhere, Category = "Custom Attributes")
	FSoftObjectPath ScriptStructPath;

	/* Transient UScriptStruct instance representing the underlying value type for the curve */
	UPROPERTY(EditAnywhere, Transient, Category = "Custom Attributes")
	TObjectPtr<UScriptStruct> ScriptStruct;

	/** Whether or not to interpolate between keys of ScripStruct type */
	UPROPERTY(EditAnywhere, Transient, Category = "Custom Attributes")
	bool bShouldInterpolate;

	/** Operator instanced used for interpolating between keys */
	const UE::Anim::IAttributeBlendOperator* Operator;

	friend class UAnimSequence;
	friend struct UE::Anim::Attributes;
	friend UE::UAF::FDecompressionTools;
};

template<>
struct TStructOpsTypeTraits<FAttributeCurve> : public TStructOpsTypeTraitsBase2<FAttributeCurve>
{
	enum
	{
		WithSerializer = true,
	};
};
