// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"

#include "UObject/NameTypes.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "Animation/AnimTypes.h"
#include "Animation/WrappedAttribute.h"
#include "BoneContainer.h"

#include "BoneIndices.h"

enum class ECustomAttributeBlendType : uint8;

namespace UE
{
	namespace Anim
	{
		extern ENGINE_API const FName BoneAttributeNamespace;

		/** Runtime identifier for an attribute */
		struct FAttributeId
		{
			FAttributeId(const FName& InName, const FCompactPoseBoneIndex& InCompactBoneIndex) : Namespace(BoneAttributeNamespace), Name(InName), Index(InCompactBoneIndex.GetInt()) {}
			FAttributeId(const FName& InName, int32 InIndex, const FName& InNamespace) : Namespace(InNamespace), Name(InName), Index(InIndex) {}
			
			bool operator==(const FAttributeId& Other) const
			{
				return Index == Other.Index && Name == Other.Name && Namespace == Other.Namespace;
			}
			
			friend inline uint32 GetTypeHash(const FAttributeId& AttributeId)
			{
				return HashCombine(GetTypeHash(AttributeId.Index), GetTypeHash(AttributeId.Name), GetTypeHash(AttributeId.Namespace));
			}

			int32 GetIndex() const { return Index; }
			const FName& GetName() const { return Name; }
			const FName& GetNamespace() const { return Namespace; }			
		protected:
			FName Namespace;
			FName Name;
			int32 Index;
		};

		namespace Conversion
		{
			template<typename BoneIndexTypeA, typename BoneIndexTypeB>
			BoneIndexTypeB MakeIndex(BoneIndexTypeA BoneIndex, const FBoneContainer& BoneContainer)
			{
				return BoneIndexTypeB();
			}

			template<>
			inline FMeshPoseBoneIndex MakeIndex(FCompactPoseBoneIndex BoneIndex, const FBoneContainer& BoneContainer)
			{
				// Guard against invalid compact-pose indexing
				const bool bValidBoneIndex = BoneContainer.GetBoneIndicesArray().IsValidIndex(BoneIndex.GetInt());
				return bValidBoneIndex ? BoneContainer.MakeMeshPoseIndex(BoneIndex) : FMeshPoseBoneIndex(INDEX_NONE);
			}
			
			template<>
			inline FCompactPoseBoneIndex MakeIndex(FMeshPoseBoneIndex BoneIndex, const FBoneContainer& BoneContainer)
			{
				return BoneContainer.MakeCompactPoseIndex(BoneIndex);
			}
		}

		/** Runtime container for Animation Attributes, providing a TMap like interface. Used in FStack*/
		template<class BoneIndexType, typename InAllocator>
		struct TAttributeContainer
		{
			/** Copies all contained data from another Container instance using another memory allocator */
			template <typename OtherAllocator>
			void CopyFrom(const TAttributeContainer<BoneIndexType, OtherAllocator>& Other)
			{
				AttributeIdentifiers = Other.AttributeIdentifiers;
				UniqueTypedBoneIndices = Other.UniqueTypedBoneIndices;
				UniqueTypes = Other.UniqueTypes;
				Values.Empty(Other.Values.Num());
				const int32 NumTypes = UniqueTypes.Num();
				for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
				{
					const TArray<TWrappedAttribute<OtherAllocator>>& Array = Other.Values[TypeIndex];
					TArray<TWrappedAttribute<InAllocator>>& ValuesArray = Values.AddDefaulted_GetRef();

					const TWeakObjectPtr<const UScriptStruct>& WeakScriptStruct = UniqueTypes[TypeIndex];
					for (const TWrappedAttribute<OtherAllocator>& StructOnScope : Array)
					{
						TWrappedAttribute<InAllocator>& StructData = ValuesArray.Add_GetRef(WeakScriptStruct.Get());
						WeakScriptStruct->CopyScriptStruct(StructData.template GetPtr<void>(), StructOnScope.template GetPtr<void>());
					}
				}
			}
					
		public:
			template<class OtherBoneIndexType, typename OtherAllocator>
			void CopyFrom(const TAttributeContainer<OtherBoneIndexType, OtherAllocator>& Other, const TMap<OtherBoneIndexType, BoneIndexType>& IndexMapping)
			{
				UniqueTypes = Other.UniqueTypes;
				AttributeIdentifiers.Empty(Other.AttributeIdentifiers.Num());
				UniqueTypedBoneIndices.Empty(Other.UniqueTypedBoneIndices.Num());
				Values.Empty(Other.Values.Num());
				
				const int32 NumTypes = Other.UniqueTypedBoneIndices.Num();
				for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
				{					
					const TWeakObjectPtr<const UScriptStruct>& WeakScriptStruct = UniqueTypes[TypeIndex];
					const TArray<FAttributeId>& OtherTypeAttributeIdentifiers = Other.AttributeIdentifiers[TypeIndex];
					const TArray<TWrappedAttribute<OtherAllocator>>& OtherValuesArray = Other.Values[TypeIndex];					

					// Populate an array entry for the type
					TArray<FAttributeId>& TypeAttributeIdentifiers = AttributeIdentifiers.AddDefaulted_GetRef();
					TArray<TWrappedAttribute<InAllocator>>& ValuesArray = Values.AddDefaulted_GetRef();
					TArray<int32>& TypeBoneIndices = UniqueTypedBoneIndices.AddDefaulted_GetRef();

					const int32 NumOtherAttributes = OtherTypeAttributeIdentifiers.Num();
					for (int32 AttributeIndex = 0; AttributeIndex < NumOtherAttributes; ++AttributeIndex)
					{
						const FAttributeId& Id = OtherTypeAttributeIdentifiers[AttributeIndex];
						const TWrappedAttribute<OtherAllocator>& Value = OtherValuesArray[AttributeIndex];

						// If the bone index was mapped successfully - add the remapped attribute
						if (const BoneIndexType* MyIndexPtr = IndexMapping.Find(OtherBoneIndexType(Id.GetIndex())))
						{
							FAttributeId MappedId(Id.GetName(), MyIndexPtr->GetInt(), Id.GetNamespace());
							TypeAttributeIdentifiers.Add(MappedId);
							TWrappedAttribute<InAllocator>& StructData = ValuesArray.Add_GetRef(WeakScriptStruct.Get());							
							WeakScriptStruct->CopyScriptStruct(StructData.template GetPtr<void>(), Value.template GetPtr<void>());
							
							TypeBoneIndices.AddUnique(MyIndexPtr->GetInt());
						}
					}	
				}
			}

			template<class OtherBoneIndexType, typename OtherAllocator>
			void CopyFrom(const TAttributeContainer<OtherBoneIndexType, OtherAllocator>& Other, const FBoneContainer& BoneContainer)
			{
				UniqueTypes = Other.UniqueTypes;
				AttributeIdentifiers.Empty(Other.AttributeIdentifiers.Num());
				UniqueTypedBoneIndices.Empty(Other.UniqueTypedBoneIndices.Num());
				Values.Empty(Other.Values.Num());
				
				const int32 NumTypes = Other.UniqueTypedBoneIndices.Num();
				for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
				{
					// Generate mapping for all contained unique bones to other container its index type
					TMap<int32, BoneIndexType> IndexMapping;
					const TArray<int32>& OtherTypeBoneIndices = Other.UniqueTypedBoneIndices[TypeIndex];
					for (int32 Index : OtherTypeBoneIndices)
					{
						OtherBoneIndexType OtherIndex(Index);						
						BoneIndexType MyIndex = Conversion::MakeIndex<OtherBoneIndexType, BoneIndexType>(OtherIndex, BoneContainer);
						if (MyIndex.IsValid())
						{
							IndexMapping.Add(Index, MyIndex);
						}			
					}
					
					const TWeakObjectPtr<const UScriptStruct>& WeakScriptStruct = UniqueTypes[TypeIndex];
					const TArray<FAttributeId>& OtherTypeAttributeIdentifiers = Other.AttributeIdentifiers[TypeIndex];
					const TArray<TWrappedAttribute<OtherAllocator>>& OtherValuesArray = Other.Values[TypeIndex];					

					// Populate an array entry for the type
					TArray<FAttributeId>& TypeAttributeIdentifiers = AttributeIdentifiers.AddDefaulted_GetRef();
					TArray<TWrappedAttribute<InAllocator>>& ValuesArray = Values.AddDefaulted_GetRef();
					TArray<int32>& TypeBoneIndices = UniqueTypedBoneIndices.AddDefaulted_GetRef();

					const int32 NumOtherAttributes = OtherTypeAttributeIdentifiers.Num();
					for (int32 AttributeIndex = 0; AttributeIndex < NumOtherAttributes; ++AttributeIndex)
					{
						const FAttributeId& Id = OtherTypeAttributeIdentifiers[AttributeIndex];
						const TWrappedAttribute<OtherAllocator>& Value = OtherValuesArray[AttributeIndex];

						// If the bone index was mapped successfully - add the remapped attribute
						if (const BoneIndexType* MyIndexPtr = IndexMapping.Find(Id.GetIndex()))
						{
							FAttributeId MappedId(Id.GetName(), MyIndexPtr->GetInt(), Id.GetNamespace());
							TypeAttributeIdentifiers.Add(MappedId);
							TWrappedAttribute<InAllocator>& StructData = ValuesArray.Add_GetRef(WeakScriptStruct.Get());							
							WeakScriptStruct->CopyScriptStruct(StructData.template GetPtr<void>(), Value.template GetPtr<void>());
							
							TypeBoneIndices.AddUnique(MyIndexPtr->GetInt());
						}
					}	
				}
			}

			/** Copies all contained data from another Container instance using the same memory allocator */
			void CopyFrom(const TAttributeContainer<BoneIndexType, InAllocator>& Other)
			{
				/** Ensure a copy to self is never performed */
				if (&Other != this)
				{
					AttributeIdentifiers = Other.AttributeIdentifiers;
					UniqueTypedBoneIndices = Other.UniqueTypedBoneIndices;
					UniqueTypes = Other.UniqueTypes;
					Values.Empty(Other.Values.Num());

					const int32 NumTypes = UniqueTypes.Num();
					for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
					{
						const TArray<TWrappedAttribute<InAllocator>>& Array = Other.Values[TypeIndex];
						TArray<TWrappedAttribute<InAllocator>>& ValuesArray = Values.AddDefaulted_GetRef();

						const TWeakObjectPtr<const UScriptStruct>& WeakScriptStruct = UniqueTypes[TypeIndex];
						for (const TWrappedAttribute<InAllocator>& StructOnScope : Array)
						{
							TWrappedAttribute<InAllocator>& StructData = ValuesArray.Add_GetRef(WeakScriptStruct.Get());
							WeakScriptStruct->CopyScriptStruct(StructData.template GetPtr<void>(), StructOnScope.template GetPtr<void>());
						}
					}
				}
			}

			/** Moves all contained data from another Container instance, once moved the other Container instance data is invalid */
			void MoveFrom(TAttributeContainer<BoneIndexType, InAllocator>& Other)
			{
				AttributeIdentifiers = MoveTemp(Other.AttributeIdentifiers);
				UniqueTypedBoneIndices = MoveTemp(Other.UniqueTypedBoneIndices);
				Values = MoveTemp(Other.Values);
				UniqueTypes = MoveTemp(Other.UniqueTypes);
			}

			/** Returns whether or not any this container contains any entries */
			bool ContainsData() const
			{
				return Values.Num() != 0;
			}

			/** Cleans out all contained entries and types */
			void Empty()
			{
				AttributeIdentifiers.Empty();
				UniqueTypedBoneIndices.Empty();
				Values.Empty();
				UniqueTypes.Empty();
			}

			bool operator!=(const TAttributeContainer<BoneIndexType, InAllocator>& Other)
			{
				/** Number of types should match */
				if (UniqueTypes.Num() == Other.UniqueTypes.Num())
				{
					for (int32 ThisTypeIndex = 0; ThisTypeIndex < UniqueTypes.Num(); ++ThisTypeIndex)
					{
						const TWeakObjectPtr<const UScriptStruct>& ThisType = UniqueTypes[ThisTypeIndex];

						const int32 OtherTypeIndex = Other.FindTypeIndex(ThisType.Get());

						/** Other should contain ThisType */
						if (OtherTypeIndex != INDEX_NONE)
						{ 
							/** Number of entries for type should match */
							if (Values[ThisTypeIndex].Num() == Other.Values[OtherTypeIndex].Num())
							{
								for (int32 ThisAttributeIndex = 0; ThisAttributeIndex < Values[ThisTypeIndex].Num(); ++ThisAttributeIndex)
								{
									const FAttributeId& ThisAttributeId = AttributeIdentifiers[ThisTypeIndex][ThisAttributeIndex];

									/** Other should contain ThisAttributeId */
									const int32 OtherAttributeIndex = Other.AttributeIdentifiers[OtherTypeIndex].IndexOfByKey(ThisAttributeId);
									if (OtherAttributeIndex != INDEX_NONE)
									{
										const TWrappedAttribute<InAllocator>& ThisAttributeValue = Values[ThisTypeIndex][ThisAttributeIndex];
										const TWrappedAttribute<InAllocator>& OtherAttributeValue = Other.Values[OtherTypeIndex][OtherAttributeIndex];

										/** Other value should match ThisAttributeValue */
										if (!ThisType->CompareScriptStruct(ThisAttributeValue.template GetPtr<void>(), OtherAttributeValue.template GetPtr<void>(), 0))
										{
											return true;
										}
									}
									else
									{
										return true;
									}
								}
							}
							else
							{
								return true;
							}
						}
						else
						{
							return true;
						}
					}
				}
				else
				{
					return true;
				}

				// If absolutely everything matches
				return false;
			}
			
			/**
			* Adds a new attribute type/value entry of the specified underlying AttributeType.
			*
			* @param	InAttributeId		Key to be used for the entry
			* @param	Attribute			Value for the new entry
			*
			* @return	The ptr to the added and populated entry, to be used for populating the data, nullptr if adding it failed
			*/
			template<typename AttributeType>
			AttributeType* Add(const FAttributeId& InAttributeId, const AttributeType& Attribute)
			{
				AttributeType* AddedPtr = (AttributeType*)Add(AttributeType::StaticStruct(), InAttributeId);
				if (AddedPtr)
				{
					AttributeType::StaticStruct()->CopyScriptStruct(AddedPtr, &Attribute);
				}

				return AddedPtr;
			}

			/**
			* Adds a new attribute type/value entry for the specified InScriptType.
			*
			* @param	InScriptStruct		UScriptStruct (type) for the entry
			* @param	InAttributeId		Key to be used for the entry
			*
			* @return	The ptr to the added entry, to be used for populating the data, nullptr if adding it failed
			*/
			uint8* Add(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindOrAddTypeIndex(InScriptStruct);

				TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];

				const int32 ExistingAttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

				// Should only add an attribute once
				if (ensure(ExistingAttributeIndex == INDEX_NONE))
				{
					int32 NewIndex = INDEX_NONE;

					AttributeIds.Add(InAttributeId);

					UniqueTypedBoneIndices[TypeIndex].AddUnique(InAttributeId.GetIndex());

					TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];
					NewIndex = TypedArray.Add(InScriptStruct);
					TWrappedAttribute<InAllocator>& StructData = TypedArray[NewIndex];
					InScriptStruct->InitializeStruct(StructData.template GetPtr<void>());

					// Ensure arrays match in size
					ensure(AttributeIds.Num() == TypedArray.Num());

					return StructData.template GetPtr<uint8>();
				}

				return nullptr;
			}

			/**
			* Adds, if not yet existing, a new attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key to be used for the entry
			*
			* @return	The ptr to the added/existing entry, to be used for populating the data, nullptr if adding it failed
			*/
			template<typename AttributeType>
			AttributeType* FindOrAdd(const FAttributeId& InAttributeId)
			{
				return (AttributeType*)FindOrAdd(AttributeType::StaticStruct(), InAttributeId);
			}

			/**
			* Adds, if not yet existing, a new attribute type/value entry for the specified InScriptType.
			*
			* @param	InScriptStruct		UScriptStruct (type) for the entry
			* @param	InAttributeId		Key to be used for the entry
			*
			* @return	The ptr to the added/existing entry, to be used for populating the data, nullptr if adding it failed
			*/
			uint8* FindOrAdd(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindOrAddTypeIndex(InScriptStruct);
				TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];

				int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

				ensure(Values.IsValidIndex(TypeIndex));
				TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

				// Should only add an attribute once
				if (AttributeIndex == INDEX_NONE)
				{
					AttributeIds.Add(InAttributeId);

					UniqueTypedBoneIndices[TypeIndex].AddUnique(InAttributeId.GetIndex());

					AttributeIndex = TypedArray.Add(InScriptStruct);
					TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];
					InScriptStruct->InitializeStruct(StructData.template GetPtr<void>());

					// Ensure arrays match in size
					ensure(AttributeIds.Num() == TypedArray.Num());

					return StructData.template GetPtr<uint8>();
				}

				ensure(TypedArray.IsValidIndex(AttributeIndex));
				TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];

				return StructData.template GetPtr<uint8>();
			}
			
			/**
			* Tries to find a attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	The ptr to the entry, to be used for modifying the data, nullptr if it was not found
			*/
			template<typename AttributeType>
			AttributeType* Find(const FAttributeId& InAttributeId)
			{
				return (AttributeType*)Find(AttributeType::StaticStruct(), InAttributeId);
			}

			/**
			* Tries to find a attribute type/value entry for the specified InScriptType.
			*
			* @param	InScriptStruct		UScriptStruct (type) for the entry
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	The ptr to the entry, to be used for modifying the data, nullptr if it was not found
			*/
			uint8* Find(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindTypeIndex(InScriptStruct);
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];

					const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);
					ensure(Values.IsValidIndex(TypeIndex));
					TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

					if (AttributeIndex != INDEX_NONE)
					{
						ensure(TypedArray.IsValidIndex(AttributeIndex));
						TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];
						return StructData.template GetPtr<uint8>();
					}
				}

				return nullptr;
			}

			/**
			* Tries to find a attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	The ptr to the constant entry, nullptr if it was not found
			*/
			template<typename AttributeType>
			const AttributeType* Find(const FAttributeId& InAttributeId) const
			{
				return (AttributeType*)Find(AttributeType::StaticStruct(), InAttributeId);
			}

			/**
			* Tries to find a attribute type/value entry for the specified InScriptType.
			*
			* @param	InScriptStruct		UScriptStruct (type) for the entry
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	The ptr to the constant entry, nullptr if it was not found
			*/
			const uint8* Find(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId) const
			{
				const int32 TypeIndex = FindTypeIndex(InScriptStruct);
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];
					const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

					ensure(Values.IsValidIndex(TypeIndex));
					const TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

					if (AttributeIndex != INDEX_NONE)
					{
						ensure(TypedArray.IsValidIndex(AttributeIndex));
						const TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];
						return StructData.template GetPtr<uint8>();
					}
				}

				return nullptr;
			}

			/**
			* Tries to find a attribute type/value entry of the specified AttributeType.
			* Asserts when the attribute was not found.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	Reference to the entry
			*/
			template<typename AttributeType>
			AttributeType& FindChecked(const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindTypeIndex(AttributeType::StaticStruct());
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];
					const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

					ensure(Values.IsValidIndex(TypeIndex));
					const TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

					if (AttributeIndex != INDEX_NONE)
					{
						ensure(TypedArray.IsValidIndex(AttributeIndex));
						TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];
						return *(AttributeType*)(TypedArray[AttributeIndex].template GetPtr<void>());
					}
					else
					{
						checkf(false, TEXT("Failed TAttributeContainer::FindChecked"));
					}
				}

				checkf(false, TEXT("Failed TAttributeContainer::FindChecked"));
				return AttributeType();
			}

			/**
			* Tries to find a attribute type/value entry of the specified AttributeType.
			* Asserts when the attribute was not found.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	Reference to the entry
			*/
			template<typename AttributeType>
			const AttributeType& FindChecked(const FAttributeId& InAttributeId) const
			{
				const int32 TypeIndex = FindTypeIndex(AttributeType::StaticStruct());
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];
					const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

					ensure(Values.IsValidIndex(TypeIndex));
					const TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

					if (AttributeIndex != INDEX_NONE)
					{
						ensure(TypedArray.IsValidIndex(AttributeIndex));
						const TWrappedAttribute<InAllocator>& StructData = TypedArray[AttributeIndex];
						return *(const AttributeType*)(TypedArray[AttributeIndex].template GetPtr<void>());
					}
				}

				checkf(false, TEXT("Failed TAttributeContainer::FindChecked"));
				return AttributeType();
			}

			/**
			* Tries to find and return the indedx of a attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	Index to the entry, INDEX_NONE if not found
			*/
			template<typename AttributeType>
			int32 IndexOfByKey(const FAttributeId& InAttributeId) const
			{
				const int32 TypeIndex = FindTypeIndex(AttributeType::StaticStruct());
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<int32>& UniqueBoneIndices = UniqueTypedBoneIndices[TypeIndex];

					// Early out if for this bone index no attributes are currently contained
					if (UniqueBoneIndices.Contains(InAttributeId.GetIndex()))
					{
						const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];
						const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);
						
						return AttributeIndex;
					}
				}

				return INDEX_NONE;
			}

			/**
			* Tries to find and return the indedx of a attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key to be used for seraching the entry
			*
			* @return	Index to the entry, INDEX_NONE if not found
			*/
			int32 IndexOfByKey(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId) const
			{
				const int32 TypeIndex = FindTypeIndex(InScriptStruct);
				if (TypeIndex != INDEX_NONE)
				{
					const TArray<int32>& UniqueBoneIndices = UniqueTypedBoneIndices[TypeIndex];

					// Early out if for this bone index no attributes are currently contained
					if (UniqueBoneIndices.Contains(InAttributeId.GetIndex()))
					{
						const TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];
						const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);
						
						return AttributeIndex;
					}
				}

				return INDEX_NONE;
			}
			
			/**
			* Removes, if existing, an attribute type/value entry of the specified AttributeType.
			*
			* @param	InAttributeId		Key of the entry to be removed
			*
			* @return	Whether or not the removal was succesful
			*/
			template<typename AttributeType>
			bool Remove(const FAttributeId& InAttributeId)
			{
				return Remove(AttributeType::StaticStruct(), InAttributeId);
			}
			
			/**
			* Removes, if existing, an attribute type/value entry for the specified InScriptType.
			*
			* @param	InAttributeId		Key of the entry to be removed
			*
			* @return	Whether or not the removal was succesful
			*/
			bool Remove(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindTypeIndex(InScriptStruct);
				if (TypeIndex != INDEX_NONE)
				{
					TArray<FAttributeId>& AttributeIds = AttributeIdentifiers[TypeIndex];

					const int32 AttributeIndex = AttributeIds.IndexOfByKey(InAttributeId);

					// Can only remove if it exists
					if (AttributeIndex != INDEX_NONE)
					{
						TArray<TWrappedAttribute<InAllocator>>& TypedArray = Values[TypeIndex];

						// If removing the last one, remove all data for the attribute type
						if (TypedArray.Num() == 1)
						{
							UniqueTypes.RemoveAtSwap(TypeIndex);
							UniqueTypedBoneIndices.RemoveAtSwap(TypeIndex);
							AttributeIdentifiers.RemoveAtSwap(TypeIndex);
							Values.RemoveAtSwap(TypeIndex);

							check(UniqueTypedBoneIndices.Num() == UniqueTypes.Num());
							check(AttributeIdentifiers.Num() == UniqueTypes.Num());
							check(Values.Num() == UniqueTypes.Num());
						}
						else
						{
							AttributeIds.RemoveAtSwap(AttributeIndex);
							TypedArray.RemoveAtSwap(AttributeIndex);
						}

						return true;
					}
				}
				

				return false;
			}


			/**
			* Removes all, if existing, an attribute type/value entries of the specified AttributeType.
			*
			* @return	Whether or not the removal was succesful
			*/
			template<typename AttributeType>
			bool RemoveAll()
			{
				return RemoveAll(AttributeType::StaticStruct());
			}

			/**
			* Removes all, if existing, an attribute type/value entries for the specified InScriptType.
			*
			* @return	Whether or not the removal was succesful
			*/
			bool RemoveAll(const UScriptStruct* InScriptStruct, const FAttributeId& InAttributeId)
			{
				const int32 TypeIndex = FindTypeIndex(InScriptStruct);
				if (TypeIndex != INDEX_NONE)
				{
					AttributeIdentifiers.RemoveAtSwap(TypeIndex);

					UniqueTypedBoneIndices.RemoveAtSwap(TypeIndex);
					AttributeIdentifiers.RemoveAtSwap(TypeIndex);
					Values.RemoveAtSwap(TypeIndex);

					ensure(UniqueTypedBoneIndices.Num() == UniqueTypes.Num());
					ensure(AttributeIdentifiers.Num() == UniqueTypes.Num());
					ensure(Values.Num() == UniqueTypes.Num());

					return true;
				}

				return false;
			}

			/*
			* @return The TypeIndex for the specified InScriptStruct type, INDEX_NONE if not found
			*/
			int32 FindTypeIndex(const UScriptStruct* InScriptStruct) const
			{
				return UniqueTypes.IndexOfByKey(InScriptStruct);
			}
			 
			/*
			* @return Array of all the contained keys for the provided TypeIndex
			*/
			const TArray<FAttributeId>& GetKeys(int32 TypeIndex) const { return AttributeIdentifiers[TypeIndex]; }

			/*
			* @return Array of all the contained values for the provided TypeIndex
			*/
			const TArray<TWrappedAttribute<InAllocator>>& GetValues(int32 TypeIndex) const { return Values[TypeIndex]; }

			/*
			* @return Array of all the contained attribute types (UScriptStruct)
			*/
			const TArray<TWeakObjectPtr<UScriptStruct>>& GetUniqueTypes() const { return UniqueTypes; }

			/*
			* Populate out parameter OutAttributeKeyNames for all contained attribute key names
			* 
			* @return bool, whether or not any attribute key names were present 
			*/
			template<typename ArrayAllocator>
			bool GetAllKeyNames(TArray<FName, ArrayAllocator>& OutAttributeKeyNames) const
			{
				if (!AttributeIdentifiers.IsEmpty())
				{
					for (const TArray<FAttributeId>& AttributeTypeEntry : AttributeIdentifiers)
					{
						for (const FAttributeId& AttributeId : AttributeTypeEntry)
						{
							OutAttributeKeyNames.AddUnique(AttributeId.GetName());
						}
					}

					return true;
				}

				return false;
			}

		/*
		* @return Unique bone indices for all contained entries of a specific attribute type
		*/
		const TArray<int32>& GetUniqueTypedBoneIndices(int32 TypeIndex) const { return UniqueTypedBoneIndices[TypeIndex]; }

		/*
		* @return total number of attributes
		*/
		int32 Num() const
		{
			int32 NumAttributes = 0;
			for (const TArray<FAttributeId>& AttributesPerType : AttributeIdentifiers)
			{
				NumAttributes += AttributesPerType.Num();
			}
				
			return NumAttributes;
		}
			
		protected:

			/*
			* @return Array of all the contained values for the provided TypeIndex
			*/
			TArray<TWrappedAttribute<InAllocator>>& GetValuesInternal(int32 TypeIndex)  { return Values[TypeIndex]; }

			/*
			* @return Array of all the contained keys for the provided TypeIndex
			*/
			TArray<FAttributeId>& GetKeysInternal(int32 TypeIndex) { return AttributeIdentifiers[TypeIndex]; }
			
			/** Find or add a new root-level entry for the provided attribute data type, returning the index into the arrays representing the type */
			int32 FindOrAddTypeIndex(const UScriptStruct* InScriptStruct)
			{
				const int32 OldNum = UniqueTypes.Num();
				// Technically we could have avoided using const cast here if we made UniqueTypes use const UScriptStruct*
				// but a bit of refactoring is required
				const int32 TypeIndex = UniqueTypes.AddUnique(const_cast<UScriptStruct*>(InScriptStruct));

				if (UniqueTypes.Num() > OldNum)
				{
					UniqueTypedBoneIndices.AddDefaulted();
					AttributeIdentifiers.AddDefaulted();
					Values.AddDefaulted();
					ensure(UniqueTypedBoneIndices.Num() == UniqueTypes.Num());
					ensure(AttributeIdentifiers.Num() == UniqueTypes.Num());
					ensure(Values.Num() == UniqueTypes.Num());
				}

				return TypeIndex;
			}
		protected:
			/** Unique bone indices for all contained entries of a specific attribute type */
			TArray<TArray<int32>> UniqueTypedBoneIndices;
			TArray<TArray<FAttributeId>> AttributeIdentifiers;
			TArray<TArray<TWrappedAttribute<InAllocator>>> Values;
			TArray<TWeakObjectPtr<UScriptStruct>> UniqueTypes;

			template <typename OtherBoneIndexType, typename OtherAllocator>
			friend struct TAttributeContainer;

			template<class OtherBoneIndexType, typename OtherAllocator>
			friend struct TAttributeContainerAccessor;
		};
	}
}
