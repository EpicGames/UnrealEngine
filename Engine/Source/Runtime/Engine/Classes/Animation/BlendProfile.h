// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BoneContainer.h"
#include "AnimationRuntime.h"
#include "BlendProfile.generated.h"

#define UE_API ENGINE_API

struct FAlphaBlend;
struct FCompactPose;
struct FBlendedCurve;
struct FSlotEvaluationPose;
namespace UE { namespace Anim { struct FStackAttributeContainer; } }

/** The mode in which the blend profile should be applied. */
UENUM()
enum class EBlendProfileMode : uint8
{
	// The bone's transition time is a factor based on the transition time. 
	// For example 0.5 means it takes half the time of the transition.
	// Values should be between 0 and 1. They will be clamped if they go out of this range.
	// A bone value of 0 means the bone will instantly transition into the target state.
	TimeFactor = 0,

	// The bone's transition weight is multiplied by this factor.
	// For example 2.0 means the bone's blend weight is twice as high as the transition's blend weight.
	// Values should typically be equal or greater than 1.0.
	// If you want certain bones to instantly transition into the target state
	// the Time Factor based method might be a better choice.
	WeightFactor,

	// Used for blend masks. Per bone alpha
	BlendMask UMETA(Hidden),
};

/** A single entry for a blend scale within a profile, mapping a bone to a blendscale */
USTRUCT()
struct FBlendProfileBoneEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=BoneSettings)
	FBoneReference BoneReference;

	UPROPERTY(EditAnywhere, Category=BoneSettings)
	float BlendScale = 0.f;
};

class IBlendProfileInterface : public IInterpolationIndexProvider
{
public:
	virtual ~IBlendProfileInterface() = default;

	virtual float GetBoneBlendScale(int32 InBoneIdx) const = 0;

	virtual int32 GetNumBlendEntries() const = 0;

	virtual TObjectPtr<USkeleton> GetSkeleton() const = 0;

	virtual EBlendProfileMode GetMode() const = 0;

	void UpdateBoneWeights(FBlendSampleData& InOutCurrentData, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse = false) const;
};

class UBlendProfile;

UINTERFACE(MinimalAPI, Blueprintable)
class UBlendProfileProviderInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface to be implemented by custom data structures that can create UBlendProfiles
 */
class IBlendProfileProviderInterface
{
	GENERATED_BODY()

public:
	// Convert the custom internal data struct into a regular blend profile
	virtual void ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const = 0;
};

/**
 * Helper class for storing either USkeleton blend profiles and custom blend profile implementations.
 * Has a custom details panel for switching between blend profile providers.
 */
USTRUCT()
struct FBlendProfileInterfaceWrapper
{
	GENERATED_BODY()

public:
	UE_API void UpdateCachedBlendProfile();

	const TObjectPtr<UBlendProfile> GetBlendProfile() const { return BlendProfile; }

	TObjectPtr<UObject> GetCustomProviderObject() const { return BlendProfileProvider.GetObject(); }

	UE_API void SetBlendProfileProvider(TObjectPtr<UObject> Provider, IBlendProfileProviderInterface* ProviderInterface, UObject* Outer);

	UE_API void SetSkeletonBlendProfile(TObjectPtr<UBlendProfile> InBlendProfile);

	bool UsesSkeletonBlendProfile() const { return BlendProfileProvider.GetObject() == nullptr; }

private:
	UPROPERTY()
	bool bIsSkeletonBlendProfile = true;

	// The object responsible for constructing the blend profile
	UPROPERTY()
	TScriptInterface<IBlendProfileProviderInterface> BlendProfileProvider;

	// The actual blend profile to use
	UPROPERTY()
	TObjectPtr<UBlendProfile> BlendProfile;
};

//////////////////////////////////////////////////////////////////////////

/** A blend profile is a set of per-bone scales that can be used in transitions and blend lists
 *  to tweak the weights of specific bones. The scales are applied to the normal weight for that bone
 */
UCLASS(MinimalAPI, BlueprintType)
class UBlendProfile : public UObject, public IBlendProfileInterface
{
public:

	GENERATED_BODY()

	ENGINE_API UBlendProfile();

	/** Get the number of entries in the profile (an entry is any blend scale that isn't 1.0f) */
	int32 GetNumBlendEntries() const { return ProfileEntries.Num(); }

	/** Set the blend scale for a specific bone 
	 *  @param InBoneIdx Index of the bone to set the blend scale of
	 *  @param InScale The scale to set the bone to
	 *  @param bRecurse Whether or not to set the scale on all children of this bone
	 *  @param bCreate Whether or not to create a blend profile entry if one does not exist for the specified bone
	 */
	ENGINE_API void SetBoneBlendScale(int32 InBoneIdx, float InScale, bool bRecurse = false, bool bCreate = false);

	/** Set the blend scale for a specific bone 
	 *  @param InBoneName Name of the bone to set the blend scale of
	 *  @param InScale The scale to set the bone to
	 *  @param bRecurse Whether or not to set the scale on all children of this bone
	 *  @param bCreate Whether or not to create a blend profile entry if one does not exist for the specified bone
	 */
	ENGINE_API void SetBoneBlendScale(const FName& InBoneName, float InScale, bool bRecurse = false, bool bCreate = false);

	/** Removes the entry for the specified bone index (does nothing if it doesn't exist) 
	 *  @param InBoneIdx Index of the bone to remove from this blend profile
	 */
	ENGINE_API void RemoveEntry(int32 InBoneIdx);

	/** Ensures the bone name of the specified entry matches the skeleton index (does nothing if it doesn't exist)
	 *  @param InBoneIdx Index of the bone to refresh
	 */
	ENGINE_API void RefreshBoneEntry(int32 InBoneIndex);

	/** Ensures the bone names match the skeleton indices by using the bone name as our lookup key. */
	ENGINE_API void RefreshBoneEntriesFromName();

	/** Removes entries with bone references to invalid bones */
	ENGINE_API void CleanupBoneEntries();

	TObjectPtr<USkeleton> GetSkeleton() const override { return OwningSkeleton; }

	/**
	 * Get the bone entry by entry index.
	 * @param[in] InEntryIdx The index to the bone entry in range [0, GetNumBlendEntries()-1].
	 * @return The bone entry containing the bone reference and blend scale.
	 **/
	ENGINE_API const FBlendProfileBoneEntry& GetEntry(const int32 InEntryIdx) const;

	/** Get the set blend scale for the specified bone, will return 1.0f if no entry was found (no scale)
	 *  @param InBoneIdx Index of the bone to retrieve
	 */
	ENGINE_API float GetBoneBlendScale(int32 InBoneIdx) const override;

	/** Get the set blend scale for the specified bone, will return 1.0f if no entry was found (no scale). The term bone factor and entry or bone scale refer to the same thing.
	 *  @param InBoneName Name of the bone to retrieve
	 */
	ENGINE_API float GetBoneBlendScale(const FName& InBoneName) const;
	
	/** Get the index of the entry for the specified bone
	 *  @param InBoneIdx Skeleton index of the bone
	 */
	ENGINE_API int32 GetEntryIndex(const FSkeletonPoseBoneIndex InBoneIdx) const;

	/** Get the index of the entry for the specified bone
	 *  @param InBoneIdx Index of the bone
	 */
	ENGINE_API int32 GetEntryIndex(const FName& BoneName) const;

	/** Get the blend scale stored in a specific entry. The term bone factor and entry scale refer to the same thing.
	 *  @param InEntryIdx Index of the entry to retrieve
	 */
	ENGINE_API float GetEntryBlendScale(const int32 InEntryIdx) const;

	/** Update all the bone weights for some provided FBlendSampleData.
	 *  This internally will iterate over all entries inside the InOutCurrentData::PerBoneBlendData parameter and call CalculateBoneWeight for each entry.
	 *  @param InOutCurrentData This is both input and output. The FBlendSampleData::PerBoneBlendData member will be updated with the weight values as calculated by CalculateBoneWeight for each entry.
	 *  @param BlendInfo Information about the blend. This contains things like the blend duration and current alpha. Not all blend profile modes might use this data.
	 *  @param BlendStartAlpha The linear alpha value, so not sampled from the curve, of where the blend started. This is mostly used when we are in the middle of a blend and suddenly reverse its direction.
	 *  This value basically should contain the alpha value of the blend at the point of reversal.
	 *  @param MainWeight The weight of the blend. This is used in the weight factor based mode, where this weight is multiplied by the bone factors.
	 *  @param bInverse Should we inverse the weights? This can be used for things like transition reversal. In most cases you would want it set to false.
	 */
	ENGINE_API void UpdateBoneWeights(FBlendSampleData& InOutCurrentData, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse = false);

	/** Calculate the blend weight for a given bone. This methoid basically defines how each blend profile mode works.
	 *  @param BoneFactor This is the per bone value setup in the blend profile editor. The impact of this value depends on what blend profile mode is used.
	 *  @param Mode The blend profile mode that should be used in combination with this bone factor.
	 *  @param BlendInfo Information about the blend. This contains things like the blend duration and current alpha. Not all blend profile modes might use this data.
	 *  @param BlendStartAlpha The linear alpha value, so not sampled from the curve, of where the blend started. This is mostly used when we are in the middle of a blend and suddenly reverse its direction.
	 *  This value basically should contain the alpha value of the blend at the point of reversal.
	 *  @param MainWeight The weight of the blend. This is used in the weight factor based mode, where this weight is multiplied by the bone factors.
	 *  @param bInverse Should we inverse the weights? This can be used for things like transition reversal. In most cases you would want it set to false.
	 */
	static ENGINE_API float CalculateBoneWeight(float BoneFactor, EBlendProfileMode Mode, const FAlphaBlend& BlendInfo, float BlendStartAlpha, float MainWeight, bool bInverse = false);

	/** Resize and fill an array of floats with the bone factor values. One for each bone inside the compact pose.
	 *  @param OutBoneBlendProfileFactors This array will be resized and filled with the factors for each bone in the compact pose, as setup in the blend profile editor.
	 *  @param BoneContainer The bone container which is used to extract how many bones are inside the compact pose and to figure out what factor value to place at what array element.
	 */
	ENGINE_API void FillBoneScalesArray(TArray<float>& OutBoneBlendProfileFactors, const FBoneContainer& BoneContainer) const;

	/** Fill an array of floats with the bone duration values. One for each bone in the skeleton pose.
	 * @param OutDurationPerBone Must be sized to the number bones in the skeleton pose. It will be filled with the durations of each bone as setup in the blend profile editor.
	 * @param Duration The duration of the blend.
	 */
	/** Fill an array of floats with the bone duration values. One for each bone in the skeleton pose.
	 * @param OutDurationPerBone Must be sized to the number bones in the skeleton pose. It will be filled with the durations of each bone as setup in the blend profile editor.
	 * @param Duration The duration of the blend.
	 * @param TargetSkeleton The target skeleton we are working on. If this is a nullptr, the owning skeleton of the blend profile is assumed. This can be used when using skeleton compatibility.
	 */
	ENGINE_API void FillSkeletonBoneDurationsArray(TCustomBoneIndexArrayView<float, FSkeletonPoseBoneIndex> OutDurationPerBone, float Duration, const USkeleton* TargetSkeleton) const;

	// IInterpolationIndexProvider
	ENGINE_API virtual int32 GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& BoneContainer, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;
	ENGINE_API virtual int32 GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;
	// End IInterpolationIndexProvider

	// UObject
	virtual bool IsSafeForRootSet() const override {return false;}
	ENGINE_API virtual void PostLoad() override;
	// End UObject

	// Default value of entries. Default values are not saved
	virtual float GetDefaultBlendScale() const { return IsBlendMask() ? 0.0f : 1.0f; }

	bool IsBlendMask() const { return Mode == EBlendProfileMode::BlendMask;  }

	EBlendProfileMode GetMode() const { return Mode; }

	/** Sets the skeleton this blend profile is used with */
	ENGINE_API void SetSkeleton(USkeleton* InSkeleton);

	// Empties the bone enties array
	void ClearEntries();

private:
	/** Set the blend scale for a single bone (ignore children) */
	ENGINE_API void SetSingleBoneBlendScale(int32 InBoneIdx, float InScale, bool bCreate = false);

public:
	// The skeleton that owns this profile
	UPROPERTY()
	TObjectPtr<USkeleton> OwningSkeleton;

	// List of blend scale entries
	UPROPERTY()
	TArray<FBlendProfileBoneEntry> ProfileEntries;

	// Blend Profile Mode. Read EBlendProfileMode for more details
	UPROPERTY()
	EBlendProfileMode Mode;
};

#undef UE_API
