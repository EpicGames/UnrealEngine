// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Blend Space. Contains functionality shared across all blend space objects
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Animation/AnimationAsset.h"
#include "AnimationRuntime.h"
#include "AnimNodeBase.h"
#include "Containers/ArrayView.h"
#include "Animation/BoneSocketReference.h"
#include "Animation/AnimTypes.h"
#include "BlendSpace.generated.h"

class UCachedAnalysisProperties;
class UBlendSpace;

/**
* The base class for properties to be used in analysis. Engine will inherit from this to define structures used for
* the functions it supports. User-defined functions will likely need their own analysis structures inheriting from
* this too.
*/
UCLASS(MinimalAPI, config=Engine)
class UAnalysisProperties : public UObject
{
	GENERATED_BODY()

public:
	virtual void InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache) {};
	virtual void MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace) {};

	/** Analysis function for this axis */
	UPROPERTY()
	FString Function = "None";
};

/** Interpolation data types. */
UENUM()
enum EBlendSpaceAxis : int
{
	BSA_None UMETA(DisplayName = "None"),
	BSA_X UMETA(DisplayName = "Horizontal (X) Axis"),
	BSA_Y UMETA(DisplayName = "Vertical (Y) Axis")
};

UENUM()
enum class EPreferredTriangulationDirection : uint8
{
	None UMETA(DisplayName = "None", ToolTip = "None"),
	Tangential UMETA(DisplayName = "Tangential", ToolTip = "When there is ambiguity, rectangles will be split so that the inserted edge tends to not point towards the origin"),
	Radial UMETA(DisplayName = "Radial", ToolTip = "When there is ambiguity, rectangles will be split so that the inserted edge tends to point towards the origin")
};

UENUM()
enum class EBlendSpacePerBoneBlendMode : uint8
{
	ManualPerBoneOverride UMETA(DisplayName = "Manual Per Bone Override", ToolTip = "Manually specify the bones and their smoothing interpolation times."),
	BlendProfile UMETA(DisplayName = "Blend Profile", ToolTip = "Use a blend profile to specify the bone smoothing interpolation times.")
};

USTRUCT()
struct FBlendSpaceBlendProfile
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName = "Blend Profile", UseAsBlendMask = true, EditConditionHides))
	TObjectPtr<UBlendProfile> BlendProfile;

	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName = "Weight Speed", ClampMin = "0"))
	float TargetWeightInterpolationSpeedPerSec = 0.0f;
};

USTRUCT()
struct FInterpolationParameter
{
	GENERATED_BODY()

	/**
	 * Smoothing Time used to move smoothly across the blendpsace from the current parameters to the target
	 * parameters. The different Smoothing Types will treat this in different ways, but in general a value of
	 * zero will disable all smoothing, and larger values will smooth more.
	 */
	UPROPERTY(EditAnywhere, DisplayName = "Smoothing Time", Category=Parameter, meta = (ClampMin = "0"))
	float InterpolationTime = 0.f;

	/**
	 * Damping ratio - only used when the type is set to SpringDamper. A value of 1 will move quickly and
	 * smoothly to the target, without overshooting. Values as low as 0 can be used to encourage some overshoot,
	 * and values around 0.7 can make pose transitions look more natural.
	 */
	UPROPERTY(EditAnywhere, Category=Parameter, meta = (ClampMin = "0", EditCondition = "InterpolationType == EFilterInterpolationType::BSIT_SpringDamper && InterpolationTime > 0"))
	float DampingRatio = 1.f;

	/**
	 * Maximum speed, in real units. For example, if this axis is degrees then you could use a value of 90 to
	 * limit the turn rate to 90 degrees per second. Only used when greater than zero and the type is
	 * set to SpringDamper or Exponential.
	 */
	UPROPERTY(EditAnywhere, Category=Parameter, meta = (ClampMin = "0"))
	float MaxSpeed = 0.f;

	/** Type of smoothing used for filtering the input value to decide how to get to target. */
	UPROPERTY(EditAnywhere, DisplayName = "Smoothing Type", Category=Parameter, meta = (EditCondition = "InterpolationTime > 0"))
	TEnumAsByte<EFilterInterpolationType> InterpolationType = EFilterInterpolationType::BSIT_SpringDamper;
};

USTRUCT()
struct FBlendParameter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, DisplayName = "Name", Category=BlendParameter)
	FString DisplayName;

	/** Minimum value for this axis range. */
	UPROPERTY(EditAnywhere, DisplayName = "Minimum Axis Value", Category=BlendParameter, meta=(NoResetToDefault))
	float Min;

	/** Maximum value for this axis range. */
	UPROPERTY(EditAnywhere, DisplayName = "Maximum Axis Value", Category=BlendParameter, meta=(NoResetToDefault))
	float Max;

	/** The number of grid divisions along this axis. */
	UPROPERTY(EditAnywhere, DisplayName = "Grid Divisions", Category=BlendParameter, meta=(UIMin="1", ClampMin="1"))
	int32 GridNum;

	/** If true then samples will always be snapped to the grid on this axis when added, moved, or the axes are changed. */
	UPROPERTY(EditAnywhere, DisplayName = "Snap to Grid", Category = BlendParameter)
	bool bSnapToGrid;

	/** If true then the input can go outside the min/max range and the blend space is treated as being cyclic on this axis. If false then input parameters are clamped to the min/max values on this axis. */
	UPROPERTY(EditAnywhere, DisplayName = "Wrap Input", Category = BlendParameter)
	bool bWrapInput;

	FBlendParameter()
		: DisplayName(TEXT("None"))
		, Min(0.f)
		, Max(100.f)
		, GridNum(4) // TODO when changing GridNum's default value, it breaks all grid samples ATM - provide way to rebuild grid samples during loading
		, bSnapToGrid(false)
		, bWrapInput(false)
	{
	}

	float GetRange() const
	{
		return Max-Min;
	}
	/** Return size of each grid. */
	float GetGridSize() const
	{
		return GetRange()/(float)GridNum;
	}
	
};

/** Sample data */
USTRUCT()
struct FBlendSample
{
	GENERATED_BODY()

	// For linked animations
	UPROPERTY(EditAnywhere, Category=BlendSample)
	TObjectPtr<class UAnimSequence> Animation;

	//blend 0->x, blend 1->y, blend 2->z

	UPROPERTY(EditAnywhere, Category=BlendSample)
	FVector SampleValue;
	
	UPROPERTY(EditAnywhere, Category = BlendSample, meta=(UIMin="0.01", UIMax="2.0", ClampMin="0.01", ClampMax="64.0"))
	float RateScale = 1.0f;

	UPROPERTY(EditAnywhere, Category = BlendSample)
	bool bMirror = false;
	
	UPROPERTY(EditAnywhere, Category = BlendSample, meta=(InlineEditConditionToggle))
	bool bUseSingleFrameForBlending = false;
	
	// Single Frame Blending: If selected, instead of using the entire animation sequence, only the specified single frame is used for sampling the animation
	UPROPERTY(EditAnywhere, Category = BlendSample, meta=(EditCondition="bUseSingleFrameForBlending", DisplayName="Use Single Frame", UIMin="0"))
	uint32 FrameIndexToSample = 0;

#if WITH_EDITORONLY_DATA
	// Whether or not this sample will be moved when the "analyse all" button is used. Note that, even if disabled,
	// it will still be available for individual sample analysis/moving
	UPROPERTY(EditAnywhere, Category = BlendSample, meta=(UIMin="0.01", UIMax="2.0", ClampMin="0.01", ClampMax="64.0"))
	uint8 bIncludeInAnalyseAll : 1;

	UPROPERTY(transient)
	uint8 bIsValid : 1;

	// Cache the samples marker data counter so that we can track if it changes and revalidate the blendspace
	int32 CachedMarkerDataUpdateCounter;

#endif // WITH_EDITORONLY_DATA

	FBlendSample()
		: Animation(nullptr)
		, SampleValue(0.f)
		, RateScale(1.0f)
#if WITH_EDITORONLY_DATA
		, bIncludeInAnalyseAll(true)
		, bIsValid(false)
		, CachedMarkerDataUpdateCounter(INDEX_NONE)
#endif // WITH_EDITORONLY_DATA
	{		
	}
	
	FBlendSample(class UAnimSequence* InAnim, FVector InValue, bool bInIsSnapped, bool bInIsValid) 
		: Animation(InAnim)
		, SampleValue(InValue)
		, RateScale(1.0f)
#if WITH_EDITORONLY_DATA
		, bIncludeInAnalyseAll(true)
		, bIsValid(bInIsValid)
		, CachedMarkerDataUpdateCounter(INDEX_NONE)
#endif // WITH_EDITORONLY_DATA
	{		 
	}
	
	bool operator==( const FBlendSample& Other ) const 
	{
		return (Other.Animation == Animation && Other.SampleValue == SampleValue && Other.bUseSingleFrameForBlending == bUseSingleFrameForBlending && bMirror == Other.bMirror && Other.FrameIndexToSample == FrameIndexToSample && FMath::IsNearlyEqual(Other.RateScale, RateScale));
	}

	ENGINE_API float GetSamplePlayLength() const;
};

/**
 * This is the runtime representation of a segment which stores its vertices (start and end) in normalized space.
 */
USTRUCT()
struct FBlendSpaceSegment
{
	GENERATED_BODY();

public:
	// Triangles have three vertices
	static const int32 NUM_VERTICES = 2;

	/** Indices into the samples */
	UPROPERTY()
	int32 SampleIndices[NUM_VERTICES] = { INDEX_NONE, INDEX_NONE };

	/** The vertices are in the normalized space - i.e. in the range 0-1. */
	UPROPERTY()
	float Vertices[NUM_VERTICES]  = { 0.f, 0.f };
};

USTRUCT()
struct FBlendSpaceTriangleEdgeInfo
{
	GENERATED_BODY();

public:
	/** Edge normal faces out */
	UPROPERTY()
	FVector2D Normal = FVector2D(0.f);

	UPROPERTY()
	int32 NeighbourTriangleIndex = INDEX_NONE;

	/**
	* IF there is no neighbor, then (a) we're on the perimeter and (b) these will be the indices of
	* triangles along the perimeter (next to the start and end of this edge, respectively) 
	*/
	UPROPERTY()
	int32 AdjacentPerimeterTriangleIndices[2] = { INDEX_NONE, INDEX_NONE };

	/**
	 * The vertex index of the associated AdjacentPerimeterTriangle such that the perimeter edge is
	 * from this vertex to the next.
	 */
	UPROPERTY()
	int32 AdjacentPerimeterVertexIndices[2]  = { INDEX_NONE, INDEX_NONE };
};

/**
* This is the runtime representation of a triangle. Each triangle stores its vertices etc in normalized space,
* with an index to the original samples.
 */
USTRUCT()
struct FBlendSpaceTriangle
{
	GENERATED_BODY();

public:
	// Triangles have three vertices
	static const int32 NUM_VERTICES = 3;

public:

	/** Indices into the samples */
	UPROPERTY(EditAnywhere, Category = EditorElement)
	int32 SampleIndices[NUM_VERTICES] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };

	/** The vertices are in the normalized space - i.e. in the range 0-1. */
	UPROPERTY(EditAnywhere, Category = EditorElement)
	FVector2D Vertices[NUM_VERTICES] = { FVector2D(0.f), FVector2D(0.f), FVector2D(0.f) };
	
	/** Info for the edge starting at the vertex index and going (anti-clockwise) to the next vertex */
	UPROPERTY(EditAnywhere, Category = EditorElement)
	FBlendSpaceTriangleEdgeInfo EdgeInfo[NUM_VERTICES];
};

USTRUCT()
struct FWeightedBlendSample
{
	GENERATED_BODY();

public:
	FWeightedBlendSample(int32 Index = INDEX_NONE, float Weight = 0) : SampleIndex(Index), SampleWeight(Weight) {}

public:
	UPROPERTY()
	int32 SampleIndex;

	UPROPERTY()
	float SampleWeight;
};

/**
* The runtime data used for interpolating. Note that only one of Segments/Triangles will be in use,
* depending on the dimensionality of the data.
*/
USTRUCT()
struct FBlendSpaceData
{
	GENERATED_BODY();
public:
	void GetSamples(
		TArray<FWeightedBlendSample>& OutWeightedSamples,
		const TArray<int32>&          InDimensionIndices,
		const FVector&                InSamplePosition,
		int32&                        InOutTriangulationIndex) const;

	void Empty()
	{
		Segments.Empty();
		Triangles.Empty();
	}

	bool IsEmpty() const 
	{
		return Segments.Num() == 0 && Triangles.Num() == 0;
	}
public:
	UPROPERTY()
	TArray<FBlendSpaceSegment> Segments;

	UPROPERTY()
	TArray<FBlendSpaceTriangle> Triangles;

private:
	void GetSamples1D(
		TArray<FWeightedBlendSample>& OutWeightedSamples,
		const TArray<int32>&          InDimensionIndices,
		const FVector&                InSamplePosition,
		int32&                        InOutSegmentIndex) const;

	void GetSamples2D(
		TArray<FWeightedBlendSample>& OutWeightedSamples,
		const TArray<int32>&          InDimensionIndices,
		const FVector&                InSamplePosition,
		int32&                        InOutTriangleIndex) const;
};

/**
 * Each elements in the grid
 */
USTRUCT()
struct FEditorElement
{
	GENERATED_BODY()

	// for now we only support triangles
	static const int32 MAX_VERTICES = 3;

	UPROPERTY(EditAnywhere, Category=EditorElement)
	int32 Indices[MAX_VERTICES];

	UPROPERTY(EditAnywhere, Category=EditorElement)
	float Weights[MAX_VERTICES];

	FEditorElement()
	{
		for (int32 ElementIndex = 0; ElementIndex < MAX_VERTICES; ElementIndex++)
		{
			Indices[ElementIndex] = INDEX_NONE;
			Weights[ElementIndex] = 0;
		}
	}
	
};

/** result of how much weight of the grid element **/
USTRUCT()
struct FGridBlendSample
{
	GENERATED_BODY()

	UPROPERTY()
	struct FEditorElement GridElement;

	UPROPERTY()
	float BlendWeight;

	FGridBlendSample()
		: BlendWeight(0)
	{
	}

};

USTRUCT()
struct FPerBoneInterpolation
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=FPerBoneInterpolation)
	FBoneReference BoneReference;

	/**
	* If greater than zero, this is the speed at which the sample weights are allowed to change for this specific bone.
	*
	* A speed of 1 means a sample weight can change from zero to one (or one to zero) in one second.
	* A speed of 2 means that this would take half a second.
	*
	* Smaller values mean slower adjustments of the sample weights, and thus more smoothing. However, a
	* value of zero disables this smoothing entirely.
	* 
	* If set, the value overrides the overall Sample Weight Speed which will no longer affect this bone.
	*/
	UPROPERTY(EditAnywhere, Category=FPerBoneInterpolation, meta=(DisplayName="Weight Speed"))
	float InterpolationSpeedPerSec;

	FPerBoneInterpolation()
		: InterpolationSpeedPerSec(6.f)
	{}

	void Initialize(const USkeleton* Skeleton)
	{
		BoneReference.Initialize(Skeleton);
	}
};

UENUM()
namespace ENotifyTriggerMode
{
	enum Type : int
	{
		AllAnimations UMETA(DisplayName="All Animations"),
		HighestWeightedAnimation UMETA(DisplayName="Highest Weighted Animation"),
		None,
	};
}

/**
 * Allows multiple animations to be blended between based on input parameters
 */
UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UBlendSpace : public UAnimationAsset, public IInterpolationIndexProvider
{
	GENERATED_UCLASS_BODY()
public:

	/** Required for accessing protected variable names */
	friend class FBlendSpaceDetails;
	friend class FBlendSampleDetails;
	friend class UAnimGraphNode_BlendSpaceGraphBase;
	friend class SBlendSpaceGridWidget;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

	//~ Begin UAnimationAsset Interface
	virtual void TickAssetPlayer(FAnimTickRecord& Instance, struct FAnimNotifyQueue& NotifyQueue, FAnimAssetTickContext& Context) const override;
	// this is used in editor only when used for transition getter
	// this doesn't mean max time. In Sequence, this is SequenceLength,
	// but for BlendSpace CurrentTime is normalized [0,1], so this is 1
	virtual float GetPlayLength() const override { return 1.f; }
	virtual TArray<FName>* GetUniqueMarkerNames() override;
	virtual bool IsValidAdditive() const override;
#if WITH_EDITOR
	virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;
	virtual int32 GetMarkerUpdateCounter() const;
	void    RuntimeValidateMarkerData();
#endif
	//~ End UAnimationAsset Interface
	
	ENGINE_API TConstArrayView<FAnimSyncMarker> GetAuthoredSyncMarkers() const;
	
	// Begin IInterpolationIndexProvider Overrides

	/**
	 * Sorts the PerBoneBlend data into a form that can be repeatedly used in GetPerBoneInterpolationIndex
	 */
	virtual TSharedPtr<IInterpolationIndexProvider::FPerBoneInterpolationData> GetPerBoneInterpolationData(const USkeleton* Skeleton) const override;

	/**
	* Get PerBoneInterpolationIndex for the input BoneIndex
	* If nothing found, return INDEX_NONE
	*/
	virtual int32 GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& RequiredBones, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;

	/**
	* Get PerBoneInterpolationIndex for the input BoneIndex
	* If nothing found, return INDEX_NONE
	*/
	virtual int32 GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;
	// End IInterpolationIndexProvider Overrides

	/** Returns whether or not the given additive animation type is compatible with the blendspace type */
	ENGINE_API virtual bool IsValidAdditiveType(EAdditiveAnimationType AdditiveType) const;

	/**
	 * BlendSpace Get Animation Pose function
	 */
	UE_DEPRECATED(4.26, "Use GetAnimationPose with other signature")
	ENGINE_API void GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, /*out*/ FCompactPose& OutPose, /*out*/ FBlendedCurve& OutCurve) const;
	
	ENGINE_API void GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, const FAnimExtractContext& ExtractionContext, /*out*/ FAnimationPoseData& OutAnimationPoseData) const;

	ENGINE_API void GetAnimationPose(TArray<FBlendSampleData>& BlendSampleDataCache, TArrayView<FPoseLink> InPoseLinks, const FAnimExtractContext& ExtractionContext, /*out*/ FPoseContext& Output) const;

	/** Accessor for blend parameter **/
	ENGINE_API const FBlendParameter& GetBlendParameter(const int32 Index) const;

	/** Get this blend spaces sample data */
	const TArray<struct FBlendSample>& GetBlendSamples() const { return SampleData; }

	/** Returns the Blend Sample at the given index, will assert on invalid indices */
	ENGINE_API const struct FBlendSample& GetBlendSample(const int32 SampleIndex) const;

	/**
	* Get Grid Samples from BlendInput
	* It will return all samples that has weight > KINDA_SMALL_NUMBER
	*
	* @param	BlendInput	BlendInput X, Y, Z corresponds to BlendParameters[0], [1], [2]
	* @param	InOutCachedTriangulationIndex	The previous index into triangulation/segmentation to warm-start the search. 
	* @param	bCombineAnimations	Will combine samples that point to the same animation. Useful when processing, but confusing when viewing
	*
	* @return	true if it has valid OutSampleDataList, false otherwise
	*/
	ENGINE_API bool GetSamplesFromBlendInput(const FVector &BlendInput, TArray<FBlendSampleData> & OutSampleDataList, int32& InOutCachedTriangulationIndex, bool bCombineAnimations) const;

	/** Utility function to calculate animation length from sample data list **/
	ENGINE_API float GetAnimationLengthFromSampleData(const TArray<FBlendSampleData>& SampleDataList) const;

	/** 
	 * Initialize BlendSpace filtering for runtime. Filtering supports multiple dimensions, defaulting to
	 * two (since we don't have 3D BlendSpaces yet) 
	**/
	ENGINE_API void InitializeFilter(FBlendFilter* Filter, int NumDimensions = 2) const;

	/** Update BlendSpace filtering parameters - values that don't require a full initialization **/
	ENGINE_API void UpdateFilterParams(FBlendFilter* Filter) const;

	/** Returns the blend input after clamping and/or wrapping */
	ENGINE_API FVector GetClampedAndWrappedBlendInput(const FVector& BlendInput) const;

	/** 
	 * Updates a cached set of blend samples according to internal parameters, blendspace position and a delta time. Used internally to GetAnimationPose().
	 * Note that this function does not perform any filtering internally.
 	 * @param	InBlendSpacePosition	The current position parameter of the blendspace
	 * @param	InOutSampleDataCache	The sample data cache. Previous frames samples are re-used in the case of target weight interpolation
	 * @param	InDeltaTime				The tick time for this update
	 */
	ENGINE_API bool UpdateBlendSamples(const FVector& InBlendSpacePosition, float InDeltaTime, TArray<FBlendSampleData>& InOutSampleDataCache, int32& InOutCachedTriangulationIndex) const;
	
	/**
	 * Resets a cached set of blend samples to match a given input time. All samples will be advanced using sync marker if possible, otherwise, their time will just be set match the input normalized time.
	 * 
	 * @param	InOutSampleDataCache			The sample data cache to use.
	 * @param	InNormalizedCurrentTime			The time to match when advancing samples. 
	 * @param	bLooping						If true, advance samples as a looping blend space would.
	 * @param	bMatchSyncPhases				If true, all follower samples will pass the same amount of markers the leader sample has passed to match its sync phase. Otherwise, followers samples will only match their next valid sync position.  
	 */
	ENGINE_API void ResetBlendSamples(TArray<FBlendSampleData>& InOutSampleDataCache, float InNormalizedCurrentTime, bool bLooping, bool bMatchSyncPhases = true) const;

	/**
	 * Allows the user to iterate through all the data samples available in the blend space.
	 * @param Func The function to run for each blend sample
	 */
	ENGINE_API void ForEachImmutableSample(const TFunctionRef<void(const FBlendSample&)> Func) const;
	
	/** Interpolate BlendInput based on Filter data **/
	ENGINE_API FVector FilterInput(FBlendFilter* Filter, const FVector& BlendInput, float DeltaTime) const;

	/** Computes the axis scale factor. See AxisToScaleAnimation. */
	ENGINE_API float ComputeAxisScaleFactor(const FVector& BlendSpacePosition, const FVector& FilteredBlendInput) const;

#if WITH_EDITOR	
	/** Validates sample data for blendspaces using the given animation sequence */
	ENGINE_API static void UpdateBlendSpacesUsingAnimSequence(UAnimSequenceBase* Sequence);

	/** Validates the contained data */
	ENGINE_API void ValidateSampleData();

	/** Add samples */
	ENGINE_API int32 AddSample(const FVector& SampleValue);
	ENGINE_API int32 AddSample(UAnimSequence* AnimationSequence, const FVector& SampleValue);

	ENGINE_API void ExpandRangeForSample(const FVector& SampleValue);
	
	/** edit samples */
	ENGINE_API bool	EditSampleValue(const int32 BlendSampleIndex, const FVector& NewValue);

	/** update animation on grid sample */
	ENGINE_API bool	ReplaceSampleAnimation(const int32 BlendSampleIndex, UAnimSequence* AnimationSequence);

	/** delete samples */
	ENGINE_API bool	DeleteSample(const int32 BlendSampleIndex);
	
	/** Get the number of sample points for this blend space */
	int32 GetNumberOfBlendSamples()  const { return SampleData.Num(); }

	/** Check whether or not the sample index is valid in combination with the stored sample data */
	ENGINE_API bool IsValidBlendSampleIndex(const int32 SampleIndex) const;

	/**
	* return GridSamples from this BlendSpace
	*
	* @param	OutGridElements
	*
	* @return	Number of OutGridElements
	*/
	ENGINE_API const TArray<FEditorElement>& GetGridSamples() const;

	/** Returns the sample position associated with the elements returned by GetGridSamples */
	ENGINE_API FVector GetGridPosition(int32 GridIndex) const;

	/** Returns the sample position associated with the coordinates */
	ENGINE_API FVector GetGridPosition(int32 GridX, int32 GridY) const;

	/**
	 * Returns the runtime triangulation etc data
	 */
	ENGINE_API const FBlendSpaceData& GetBlendSpaceData() const;

	/**
	 * Runs triangulation/segmentation to update our grid and BlendSpaceData structures
	 */
	ENGINE_API void ResampleData();

	/**
	 * Sets up BlendSpaceData based on Line elements
	 */
	ENGINE_API void SetBlendSpaceData(const TArray<FBlendSpaceSegment>& Segments);

	/** Validate that the given animation sequence and contained blendspace data */
	ENGINE_API bool ValidateAnimationSequence(const UAnimSequence* AnimationSequence) const;

	/** Check if the blend spaces contains samples whos additive type match that of the animation sequence */
	ENGINE_API bool DoesAnimationMatchExistingSamples(const UAnimSequence* AnimationSequence) const;
	
	/** Check if the the blendspace contains additive samples only */	
	ENGINE_API bool ShouldAnimationBeAdditive() const;

	/** Check if the animation sequence's skeleton is compatible with this blendspace */
	ENGINE_API bool IsAnimationCompatibleWithSkeleton(const UAnimSequence* AnimationSequence) const;

	/** Check if the animation sequence additive type is compatible with this blend space */
	ENGINE_API bool IsAnimationCompatible(const UAnimSequence* AnimationSequence) const;

	/** Validates supplied blend sample against current contents of blendspace */
	ENGINE_API bool ValidateSampleValue(const FVector& SampleValue, int32 OriginalIndex = INDEX_NONE) const;

	ENGINE_API bool IsSampleWithinBounds(const FVector &SampleValue) const;

	/** Check if given sample value isn't too close to existing sample point **/
	ENGINE_API bool IsTooCloseToExistingSamplePoint(const FVector& SampleValue, int32 OriginalIndex) const;
#endif

protected:
	/**
	* Get Grid Samples from BlendInput, From Input, it will populate OutGridSamples with the closest grid points.
	*
	* @param	BlendInput			BlendInput X, Y, Z corresponds to BlendParameters[0], [1], [2]
	* @param	OutBlendSamples		Populated with the samples nearest the BlendInput
	*
	*/
	void GetRawSamplesFromBlendInput(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const;

	/** Returns the axis which can be used to scale animation speed. **/
	virtual EBlendSpaceAxis GetAxisToScale() const { return AxisToScaleAnimation; }

	/** Initialize Per Bone Blend **/
	void InitializePerBoneBlend();
	
	UE_DEPRECATED(5.8, "Please use TickFollowerSamples() that takes in the blendspace's tick record.")
	void TickFollowerSamples(
		TArray<FBlendSampleData> &SampleDataList, const int32 HighestWeightIndex, FAnimAssetTickContext &Context, 
		bool bResetMarkerDataOnFollowers, bool bLooping, const UMirrorDataTable* MirrorDataTable = nullptr) const;

	/** Ticks the samples in SampleDataList apart from the HighestWeightIndex one. */
	void TickFollowerSamples(TArray<FBlendSampleData> &SampleDataList, const int32 HighestWeightIndex, FAnimAssetTickContext &Context, 
		bool bResetMarkerDataOnFollowers, bool bLooping, const FAnimTickRecord& BlendSpaceInstance) const;
	
	/** Returns the blend input clamped to the valid range, unless that axis has been set to wrap in which case no clamping is done **/
	FVector GetClampedBlendInput(const FVector& BlendInput) const;
	
	/** Translates BlendInput to grid space */
	FVector ConvertBlendInputToGridSpace(const FVector& BlendInput) const;

	/** Translates BlendInput to grid space */
	FVector GetNormalizedBlendInput(const FVector& BlendInput) const;

	/** Returns the grid element at Index or NULL if Index is not valid */
	const FEditorElement* GetGridSampleInternal(int32 Index) const;
	
	/** Utility function to interpolate weight of samples from OldSampleDataList to NewSampleDataList and copy back the interpolated result to FinalSampleDataList **/
	bool InterpolateWeightOfSampleData(float DeltaTime, const TArray<FBlendSampleData> & OldSampleDataList, const TArray<FBlendSampleData> & NewSampleDataList, TArray<FBlendSampleData> & FinalSampleDataList) const;

	/** Returns whether or not all animation set on the blend space samples match the given additive type */
	bool ContainsMatchingSamples(EAdditiveAnimationType AdditiveType) const;

	/** Checks if the given samples points overlap */
	bool IsSameSamplePoint(const FVector& SamplePointA, const FVector& SamplePointB) const;	

#if WITH_EDITOR
	bool ContainsNonAdditiveSamples() const;
	void UpdatePreviewBasePose();
	/** If around border, snap to the border to avoid empty hole of data that is not valid **/
	virtual void SnapSamplesToClosestGridPoint();
#endif // WITH_EDITOR

	ENGINE_API static bool IsSingleFrameBlendingIndexInBounds(const FBlendSample& BlendSample);
private:
	// Internal helper function for GetAnimationPose variants
	void GetAnimationPose_Internal(TArray<FBlendSampleData>& BlendSampleDataCache, TArrayView<FPoseLink> InPoseLinks, FAnimInstanceProxy* InProxy, bool bInExpectsAdditivePose, const FAnimExtractContext& ExtractionContext, /*out*/ FAnimationPoseData& OutAnimationPoseData) const;

	// Internal helper function for UpdateBlendSamples and TickAssetPlayer
	bool UpdateBlendSamples_Internal(const FVector& InBlendSpacePosition, float InDeltaTime, TArray<FBlendSampleData>& InOutOldSampleDataList, TArray<FBlendSampleData>& InOutSampleDataCache, int32& InOutCachedTriangulationIndex) const;	

	/** Gets the appropriate mirror data table for tick record's blend sample. */
	const UMirrorDataTable* GetEffectiveMirrorDataTableForSample(const FAnimTickRecord& InInstance, const FBlendSample& InBlendSample) const;
	
	/** Gets the appropriate mirror data table for a blend sample taking into account any upstream mirror requests */
	const UMirrorDataTable* GetEffectiveMirrorDataTableForSample(const FBlendSample& InBlendSample, const UMirrorDataTable* InUpStreamMirrorDataTable = nullptr) const;
	
public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property is deprecated. Please use/see bContainsRotationOffsetMeshSpaceSamples instead")
	bool bRotationBlendInMeshSpace_DEPRECATED;
#endif

	/** Indicates whether any samples have the flag to apply rotation offsets in mesh space */
	UPROPERTY()
	bool bContainsRotationOffsetMeshSpaceSamples;

	/** Input Smoothing parameters for each input axis */
	UPROPERTY(EditAnywhere, Category = InputInterpolation)
	FInterpolationParameter	InterpolationParam[3];

#if WITH_EDITORONLY_DATA
	/** 
	* Analysis properties for each axis. Note that these can be null. They will be created/set from 
	* FBlendSpaceDetails::HandleAnalysisFunctionChanged
	*/
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	TObjectPtr<UAnalysisProperties> AnalysisProperties[3];

	/** Cached properties used to initialize properties when newly created. */
	TObjectPtr<UCachedAnalysisProperties> CachedAnalysisProperties[3];
#endif

	/**
	* If greater than zero, this is the speed at which the sample weights are allowed to change.
	* 
	* A speed of 1 means a sample weight can change from zero to one (or one to zero) in one second.
	* A speed of 2 means that this would take half a second.
	* 
	* This allows the Blend Space to switch to new parameters without going through intermediate states, 
	* effectively blending between where it was and where the new target is. For example, imagine we have 
	* a blend space for locomotion, moving left, forward and right. Now if you interpolate the inputs of 
	* the blend space itself, from one extreme to the other, you will go from left, to forward, to right. 
	* As an alternative, by setting this Sample Weight Speed to a value higher than zero, it will go 
	* directly from left to right, without going through moving forward first.
	* 
	* Smaller values mean slower adjustments of the sample weights, and thus more smoothing. However, a 
	* value of zero disables this smoothing entirely.
	*/
	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName = "Weight Speed", ClampMin = "0"))
	float TargetWeightInterpolationSpeedPerSec = 0.0f;

	/**
	 * If set then this eases in/out the sample weight adjustments, using the speed to determine how much smoothing to apply.
	 */
	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName = "Smoothing"))
	bool bTargetWeightInterpolationEaseInOut = true;

	/**
	 * If set then blending is performed in mesh space if there are per-bone sample smoothing overrides.
	 * 
	 * Note that mesh space blending is significantly more expensive (slower) than normal blending when the 
	 * samples are regular animations (i.e. not additive animations that are already set to apply in mesh 
	 * space), and is typically only useful if you want some parts of the skeleton to achieve a pose 
	 * in mesh space faster or slower than others - for example to make the head move faster than the 
	 * body/arms when aiming, so the character looks at the target slightly before aiming at it.
	 * 
	 * Note also that blend space assets with additive/mesh space samples will always blend in mesh space, and 
	 * also that enabling this option with blend space graphs producing additive/mesh space samples may cause
	 * undesired results.
	 */
	UPROPERTY(EditAnywhere, Category = SampleSmoothing)
	bool bAllowMeshSpaceBlending = false;

	/** 
	* The default looping behavior of this blend space.
	* Asset players can override this
	*/
	UPROPERTY(EditAnywhere, Category=Animation)
	bool bLoop = true;

	/** Whether to allow marker based sync between the samples (it won't force sync if the markers don't exist) */
	UPROPERTY(EditAnywhere, Category = Animation)
	bool bAllowMarkerBasedSync = true;

	/** If true, all follower samples will pass the same amount of markers the leader sample has passed to match its sync phase. Otherwise, followers samples will only match their next valid sync position. */
	UPROPERTY(EditAnywhere, Category = Animation)
	bool bShouldMatchSyncPhases = false;

	/** Enable to use the old method for calculating the sample point's effective animation length.
	The legacy method was to calculate each sample's effective animation length and perform a weighted blend to calculate the animation length for the preview.
	This means the result is non-linear with respect to each sample's play rate as play rate is the inverse of animation length.
	The new method is to perform a weighted blend on the play rates directly and then derive the animation length.
	*/
	UPROPERTY(EditAnywhere, Category = Animation)
	bool bUseLegacySamplePointAnimationLengthCalculations = false;
	
#if WITH_EDITORONLY_DATA
	/** Preview Base pose for additive BlendSpace **/
	UPROPERTY(EditAnywhere, Category = AdditiveSettings)
	TObjectPtr<UAnimSequence> PreviewBasePose;
#endif // WITH_EDITORONLY_DATA

	/** This is the maximum length of any sample in the blendspace. **/
	UPROPERTY()
	float AnimLength;

	/** The current mode used by the BlendSpace to decide which animation notifies to fire. Valid options are:
	- AllAnimations: All notify events will fire
	- HighestWeightedAnimation: Notify events will only fire from the highest weighted animation
	- None: No notify events will fire from any animations
	*/
	UPROPERTY(EditAnywhere, Category = AnimationNotifies)
	TEnumAsByte<ENotifyTriggerMode::Type> NotifyTriggerMode;

	/** If true then interpolation is done via a grid at runtime. If false the interpolation uses the triangulation. */
	UPROPERTY(EditAnywhere, Category = InputInterpolation, meta = (DisplayName="Use Grid"))
	bool bInterpolateUsingGrid = false;

	/** Preferred edge direction when the triangulation has to make an arbitrary choice */
	UPROPERTY(EditAnywhere, Category = InputInterpolation)
	EPreferredTriangulationDirection PreferredTriangulationDirection = EPreferredTriangulationDirection::Tangential;

	// Used to mirror any blend samples with bMirror set to true
	UPROPERTY(Category=Mirroring, EditAnywhere)
	TObjectPtr<UMirrorDataTable> MirrorDataTable = nullptr;
	
protected:

	/**
	 * There are two ways to use per pone sample smoothing: Blend profiles and manually maintaining the per bone overrides.
	 */
	UPROPERTY(EditAnywhere, Category = SampleSmoothing)
	EBlendSpacePerBoneBlendMode PerBoneBlendMode = EBlendSpacePerBoneBlendMode::ManualPerBoneOverride;

	/**
	 * Per bone sample smoothing settings, which affect the specified bone and all its descendants in the skeleton.
	 * These act as overrides to the global sample smoothing speed, which means the global sample smoothing speed does
	 * not affect these bones. Note that they also override each other - so a per-bone setting on the chest will not
	 * affect the hand if there is a per-bone setting on the arm.
	 */
	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName="Per Bone Overrides", EditCondition = "PerBoneBlendMode == EBlendSpacePerBoneBlendMode::ManualPerBoneOverride", EditConditionHides))
	TArray<FPerBoneInterpolation> ManualPerBoneOverrides;

	/**
	 * Reference to a blend profile of the corresponding skeleton to be used for per bone smoothing in case the per bone blend mode is set to use a blend profile.
	 **/
	UPROPERTY(EditAnywhere, Category = SampleSmoothing, meta = (DisplayName = "Per Bone Overrides", EditCondition = "PerBoneBlendMode == EBlendSpacePerBoneBlendMode::BlendProfile", EditConditionHides))
	FBlendSpaceBlendProfile PerBoneBlendProfile;

	/**
	 * Stores the actual bone references and their smoothing interpolation speeds used by the blend space. This will be either filled by the manual per bone overrides or
	 * the blend profile, depending on the set per bone blend mode.
	 **/
	TArray<FPerBoneInterpolation> PerBoneBlendValues;

	/** Track index to get marker data from. Samples are tested for the suitability of marker based sync
	    during load and if we can use marker based sync we cache an index to a representative sample here */
	UPROPERTY()
	int32 SampleIndexWithMarkers = INDEX_NONE;

	/** Sample animation data */
	UPROPERTY(EditAnywhere, Category=BlendSamples)
	TArray<struct FBlendSample> SampleData;

	/** Grid samples, indexing scheme imposed by subclass */
	UPROPERTY()
	TArray<struct FEditorElement> GridSamples;

	/** Container for the runtime data, which could be line segments, triangulation or tetrahedrons */
	UPROPERTY()
	FBlendSpaceData BlendSpaceData;
	
	/** Blend Parameters for each axis. **/
	UPROPERTY(EditAnywhere, Category = BlendParametersTest)
	struct FBlendParameter BlendParameters[3];

	/**
	 * If you have input smoothing, this specifies the axis on which to scale the animation playback speed. E.g. for 
	 * locomotion animation, the speed axis will scale the animation speed in order to make up the difference 
	 * between the target and the result of blending the samples.
	 */
	UPROPERTY(EditAnywhere, Category = InputInterpolation)
	TEnumAsByte<EBlendSpaceAxis> AxisToScaleAnimation;

	/** Reset to reference pose. It does apply different refpose based on additive or not */
	void ResetToRefPose(FCompactPose& OutPose) const;

	/** The order in which to use the dimensions in the data - e.g. [1, 2] means a 2D blend using Y and Z */
	UPROPERTY()
	TArray<int32> DimensionIndices;

#if WITH_EDITOR
private:
	// Track whether we have updated markers so cached data can be updated
	int32 MarkerDataUpdateCounter;
protected:
	FVector PreviousAxisMinMaxValues[3];
	float   PreviousGridSpacings[3];
#endif	

private:

	void GetRawSamplesFromBlendInput1D(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const;

	void GetRawSamplesFromBlendInput2D(const FVector& BlendInput, TArray<FGridBlendSample, TInlineAllocator<4> >& OutBlendSamples) const;

	/** Fill up local GridElements from the grid elements that are created using the sorted points
	*	This will map back to original index for result
	*
	*  @param	SortedPointList		This is the pointlist that are used to create the given GridElements
	*								This list contains subsets of the points it originally requested for visualization and sorted
	*
	*/
	void FillupGridElements(const TArray<FEditorElement>& GridElements, const TArray<int32>& InDimensionIndices);

	void EmptyGridElements();

	void ClearBlendSpaceData();

	void SetBlendSpaceData(const TArray<FBlendSpaceTriangle>& Triangles);

	void ResampleData1D();
	void ResampleData2D();


	/** Get the Editor Element from Index
	*
	* @param	XIndex	Index of X
	* @param	YIndex	Index of Y
	*
	* @return	FEditorElement * return the grid data
	*/
	const FEditorElement* GetEditorElement(int32 XIndex, int32 YIndex) const;
};
