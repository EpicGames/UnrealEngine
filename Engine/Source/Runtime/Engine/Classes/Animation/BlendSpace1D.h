// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Blend Space 1D. Contains 1 axis blend 'space'
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/BlendSpace.h"
#include "BlendSpace1D.generated.h"

 /**
  * Allows multiple animations to be blended between based on input parameters
  */
UCLASS(config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UBlendSpace1D : public UBlendSpace
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bDisplayEditorVertically_DEPRECATED;
#endif

	/**
	 * If you have input smoothing, whether to scale the animation speed. E.g. for locomotion animation, 
	 * the speed axis will scale the animation speed in order to make up the difference between the target 
	 * and the result of blending the samples.
	 */
	UPROPERTY(EditAnywhere, Category = InputInterpolation)
	bool bScaleAnimation;

protected:
	//~ Begin UBlendSpace Interface
	virtual EBlendSpaceAxis GetAxisToScale() const override;
#if WITH_EDITOR
	virtual void SnapSamplesToClosestGridPoint() override;
#endif
	//~ End UBlendSpace Interface
};
