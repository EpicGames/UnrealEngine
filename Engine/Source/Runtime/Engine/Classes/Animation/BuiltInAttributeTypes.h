// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AttributeTraits.h"
#include "AnimationRuntime.h"

#include "Algo/Transform.h"

#include "BuiltInAttributeTypes.generated.h"

/** Attribute type supporting the legacy TVariant<float> atttributes */
USTRUCT(BlueprintType)
struct FFloatAnimationAttribute 
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FloatAnimationAttribute)
	float Value = 0.f;

	void Accumulate(const FFloatAnimationAttribute& Attribute, float Weight, EAdditiveAnimationType AdditiveType)
	{
		Value += Attribute.Value * Weight;
	}

	void MakeAdditive(const FFloatAnimationAttribute& BaseAttribute)
	{
		Value -= BaseAttribute.Value;
	}

	FFloatAnimationAttribute Multiply(float Weight) const
	{
		FFloatAnimationAttribute Out;
		Out.Value = Value * Weight;
		return Out;
	}

	void Interpolate(const FFloatAnimationAttribute& Attribute, float Alpha)
	{
		Value *= (1.f - Alpha);
		Value += (Attribute.Value * Alpha);
	}
};

/** Attribute type supporting the legacy TVariant<int32> atttributes */
USTRUCT(BlueprintType)
struct FIntegerAnimationAttribute
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntegerAnimationAttribute)
	int32 Value = 0;

	void Accumulate(const FIntegerAnimationAttribute& Attribute, float Weight, EAdditiveAnimationType AdditiveType)
	{
		Value += (int32)(Attribute.Value * Weight);
	}

	void MakeAdditive(const FIntegerAnimationAttribute& BaseAttribute)
	{
		Value -= BaseAttribute.Value;
	}

	FIntegerAnimationAttribute Multiply(float Weight) const
	{
		FIntegerAnimationAttribute Out;
		Out.Value = (int32)(Value * Weight);
		return Out;
	}

	void Interpolate(const FIntegerAnimationAttribute& Attribute, float Alpha)
	{
		Value = FMath::TruncToInt32(Value * (1.f - Alpha));
		Value += FMath::TruncToInt32(Attribute.Value * Alpha);
	}
};

/** Attribute type supporting the legacy TVariant<FString> attributes */
USTRUCT(BlueprintType)
struct FStringAnimationAttribute
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=StringAnimationAttribute)
	FString Value;
};

inline uint32 GetTypeHash(const FStringAnimationAttribute& Key)
{
	return GetTypeHash(Key.Value);
}

/** Attribute type supporting the legacy TVariant<FTransform> attributes */
USTRUCT(BlueprintType)
struct FTransformAnimationAttribute
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TransformAnimationAttribute)
	FTransform Value;

	void Accumulate(const FTransformAnimationAttribute& Attribute, float Weight, EAdditiveAnimationType AdditiveType)
	{
		//if (FAnimWeight::IsRelevant(Weight))
		{
			const ScalarRegister VBlendWeight(Weight);

			if (AdditiveType == AAT_None)
			{
				Value.AccumulateWithShortestRotation(Attribute.Value, VBlendWeight);
			}
			else
			{
				if (FAnimWeight::IsFullWeight(Weight))
				{
					Value.AccumulateWithAdditiveScale(Attribute.Value, VBlendWeight);
				}
				else
				{
					FTransform::BlendFromIdentityAndAccumulate(Value, Attribute.Value, VBlendWeight);
				}
			}
		}
	}

	void MakeAdditive(const FTransformAnimationAttribute& BaseAttribute)
	{
		FAnimationRuntime::ConvertTransformToAdditive(Value, BaseAttribute.Value);
	}

	void Normalize()
	{
		Value.NormalizeRotation();
	}
	
	FTransformAnimationAttribute Multiply(const float Weight) const
	{
		FTransformAnimationAttribute Out;

		const ScalarRegister VBlendWeight(Weight);
		Out.Value = Value * VBlendWeight;

		return Out;
	}

	void Interpolate(const FTransformAnimationAttribute& Attribute, float Alpha)
	{
		Value.BlendWith(Attribute.Value, Alpha);
	}
};


USTRUCT(BlueprintType)
struct FVectorAnimationAttribute
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VectorAnimationAttribute)
	FVector Value = FVector::ZeroVector;

	void Accumulate(const FVectorAnimationAttribute& Attribute, float Weight, EAdditiveAnimationType AdditiveType)
	{
		// if (FAnimWeight::IsRelevant(Weight))
		{
			Value += Attribute.Value * Weight;
		}
	}

	void MakeAdditive(const FVectorAnimationAttribute& BaseAttribute)
	{
		Value = Value - BaseAttribute.Value;
	}

	void Normalize()
	{
		Value.Normalize();
	}
	
	FVectorAnimationAttribute Multiply(const float Weight) const
	{
		FVectorAnimationAttribute Out;
		Out.Value = Value * Weight;

		return Out;
	}

	void Interpolate(const FVectorAnimationAttribute& Attribute, float Alpha)
	{
		Value = FMath::Lerp<FVector>(Value, Attribute.Value, Alpha);
	}
};

USTRUCT(BlueprintType)
struct FQuaternionAnimationAttribute
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=QuaternionAnimationAttribute)
	FQuat Value = FQuat::Identity;

	void Accumulate(const FQuaternionAnimationAttribute& Attribute, float Weight, EAdditiveAnimationType AdditiveType)
	{
		// if (FAnimWeight::IsRelevant(Weight))
		{
			if (AdditiveType == AAT_None)
			{
				const FQuat WeightedRotation = Attribute.Value * Weight;
				
				// From VectorAccumulateQuaternionShortestPath
				// Blend rotation
				//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
				//     const float Bias = (|A.B| >= 0 ? 1 : -1)
				//     return A + B * Bias;
				const FQuat::FReal DotResult = (Value | WeightedRotation);
				const FQuat::FReal Bias = FMath::FloatSelect(DotResult, FQuat::FReal(1.0f), FQuat::FReal(-1.0f));
				
				Value += WeightedRotation * Bias;
			}
			else
			{
				// Quaternion equivalent of FTransform::BlendFromIdentityAndAccumulate
				const FQuat WeightedRotation = FQuat::FastLerp(FQuat::Identity, Attribute.Value, Weight).GetNormalized();
				Value = WeightedRotation * Value;
			}
			
		}
	}

	void MakeAdditive(const FQuaternionAnimationAttribute& BaseAttribute)
	{
		Value = Value * BaseAttribute.Value.Inverse();
		Value.Normalize();
	}

	void Normalize()
	{
		Value.Normalize();
	}
	
	FQuaternionAnimationAttribute Multiply(const float Weight) const
	{
		FQuaternionAnimationAttribute Out;
		Out.Value = Value * Weight;

		return Out;
	}

	void Interpolate(const FQuaternionAnimationAttribute& Attribute, float Alpha)
	{
		Value = FQuat::FastLerp(Value, Attribute.Value, Alpha).GetNormalized();
	}
};

USTRUCT()
struct FNonBlendableQuaternionAnimationAttribute : public FQuaternionAnimationAttribute
{
	GENERATED_BODY()
};

USTRUCT()
struct FNonBlendableVectorAnimationAttribute : public FVectorAnimationAttribute
{
	GENERATED_BODY()
};

USTRUCT()
struct FNonBlendableTransformAnimationAttribute : public FTransformAnimationAttribute
{
	GENERATED_BODY()
};

USTRUCT()
struct FNonBlendableFloatAnimationAttribute : public FFloatAnimationAttribute
{
	GENERATED_BODY()
};

USTRUCT()
struct FNonBlendableIntegerAnimationAttribute : public FIntegerAnimationAttribute
{
	GENERATED_BODY()
};

namespace UE
{
	namespace Anim
	{
		/** Integer attribute is step-interpolated by default */
		template<>
		struct TAttributeTypeTraits<FIntegerAnimationAttribute> : public TAttributeTypeTraitsBase<FIntegerAnimationAttribute>
		{
			enum
			{
				StepInterpolate = true,
			};
		};

		/** String attribute is not blend-able by default */
		template<>
		struct TAttributeTypeTraits<FStringAnimationAttribute> : public TAttributeTypeTraitsBase<FStringAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};

		/** Transform attribute requires normalization */
		template<>
		struct TAttributeTypeTraits<FTransformAnimationAttribute> : public TAttributeTypeTraitsBase<FTransformAnimationAttribute>
		{
			enum
			{
				RequiresNormalization = true,
			};
		};

		/** Quaternion attribute requires normalization */
		template<>
		struct TAttributeTypeTraits<FQuaternionAnimationAttribute> : public TAttributeTypeTraitsBase<FQuaternionAnimationAttribute>
		{
			enum
			{
				RequiresNormalization = true,
			};
		};

		/** Non blendable types*/
		template<>
		struct TAttributeTypeTraits<FNonBlendableQuaternionAnimationAttribute> : public TAttributeTypeTraits<FQuaternionAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};

		/** Non blendable types*/
		template<>
		struct TAttributeTypeTraits<FNonBlendableVectorAnimationAttribute> : public TAttributeTypeTraits<FVectorAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};
		
		/** Non blendable types*/
		template<>
		struct TAttributeTypeTraits<FNonBlendableTransformAnimationAttribute> : public TAttributeTypeTraits<FTransformAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};
		
		template<>
		struct TAttributeTypeTraits<FNonBlendableFloatAnimationAttribute> : public TAttributeTypeTraits<FFloatAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};

		template<>
		struct TAttributeTypeTraits<FNonBlendableIntegerAnimationAttribute> : public TAttributeTypeTraits<FIntegerAnimationAttribute>
		{
			enum
			{
				IsBlendable = false,
			};
		};

#if WITH_EDITOR
		/** Helper functionality allowing the user to add an attribute with a typed value array */
		template<typename AttributeType, typename ValueType>
		bool AddTypedCustomAttribute(const FName& AttributeName, const FName& BoneName, UAnimSequenceBase* AnimSequenceBase, TArrayView<const float> Keys, TArrayView<const ValueType> Values, bool bShouldTransact = true)
		{
			const FAnimationAttributeIdentifier Identifier = UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(AnimSequenceBase, AttributeName, BoneName, AttributeType::StaticStruct());

			IAnimationDataController& Controller = AnimSequenceBase->GetController();
			if (AnimSequenceBase->GetDataModelInterface()->FindAttribute(Identifier) || Controller.AddAttribute(Identifier))
			{
				TArray<AttributeType> AttributeValues; 
				Algo::Transform(Values, AttributeValues, [](const ValueType& Value)
				{
					AttributeType Attribute;
					Attribute.Value = Value;
					return Attribute;
				});

				return Controller.SetTypedAttributeKeys<AttributeType>(Identifier, Keys, MakeArrayView(AttributeValues), bShouldTransact);
			}

			return false;
		}		
#endif // WITH_EDITOR
	}
}


UCLASS()
class UBuiltInAttributesExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = Attributes, meta = (ScriptMethod))
	static bool AddTransformAttribute(UAnimSequenceBase* AnimSequenceBase, const FName& AttributeName, const FName& BoneName, const TArray<float>& Keys, const TArray<FTransform>& Values)
	{
		return UE::Anim::AddTypedCustomAttribute<FTransformAnimationAttribute, FTransform>(AttributeName, BoneName, AnimSequenceBase, MakeArrayView(Keys), MakeArrayView(Values));
	}
#endif // WITH_EDITOR
};
