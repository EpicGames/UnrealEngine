// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"
#include "CurveSourceInterface.generated.h"

UINTERFACE(MinimalAPI)
class UCurveSourceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/** Name/value pair for retrieving curve values */
USTRUCT(BlueprintType)
struct FNamedCurveValue
{
	GENERATED_BODY()

	/** The name of the curve */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Curve")
	FName Name;

	/** The value of the curve */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Curve")
	float Value = 0.f;
};

/** A source for curves */
class ICurveSourceInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	/** The default binding, for clients to opt-in to */
	static ENGINE_API const FName DefaultBinding;

	/** 
	 * Get the name that this curve source can be bound to by.
	 * Clients of this curve source will use this name to identify this source.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Curves")
	ENGINE_API FName GetBindingName() const;

	/** Get the value for a specified curve */
	UFUNCTION(BlueprintNativeEvent, Category = "Curves")
	ENGINE_API float GetCurveValue(FName CurveName) const;

	/** Evaluate all curves that this source provides */
	UFUNCTION(BlueprintNativeEvent, Category = "Curves")
	ENGINE_API void GetCurves(TArray<FNamedCurveValue>& OutValues) const;
};
