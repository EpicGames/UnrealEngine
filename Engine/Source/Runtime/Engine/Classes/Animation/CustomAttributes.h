// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "Misc/Variant.h"
#include "Animation/AnimTypes.h"
#include "BoneIndices.h"
#include "Templates/Tuple.h"
#include "Curves/StringCurve.h"
#include "Curves/IntegralCurve.h"
#include "Curves/SimpleCurve.h"

#include "CustomAttributes.generated.h"

UENUM(Experimental)
enum class ECustomAttributeBlendType : uint8
{
	/** Overrides Custom attributes according to highest weighted pose */
	Override,
	/** Blends Custom attributes according to weights per pose */
	Blend
};

USTRUCT()
struct FCustomAttributeSetting
{
	GENERATED_BODY()

	/** Name of the custom attribute */
	UPROPERTY(EditAnywhere, Category = CustomAttributeSetting)
	FString Name;

	/** Optional property describing the meaning (or role) of the custom attribute, allowing to add context to an attribute */
	UPROPERTY(EditAnywhere, Category = CustomAttributeSetting)
	FString Meaning;
};

/**
 * Settings that identify the names of custom attributes that represent the individual components of a timecode and a subframe along with a take name.
 */
USTRUCT()
struct FTimecodeCustomAttributeNameSettings
{
	GENERATED_BODY()

	/** Name of the custom attribute representing the hour component of a timecode. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName HourAttributeName;

	/** Name of the custom attribute representing the minute component of a timecode. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName MinuteAttributeName;

	/** Name of the custom attribute representing the second component of a timecode. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName SecondAttributeName;

	/** Name of the custom attribute representing the frame component of a timecode. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName FrameAttributeName;

	/** Name of the custom attribute representing a subframe value. Though not strictly a component
		of a timecode, this attribute can be authored to identify samples in between timecodes. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName SubframeAttributeName;

	/** Name of the custom attribute representing the timecode rate. This may be different from
	    the animation or capture frame rate, for example when capturing "high" frame rate data
		at 120 frames per second but recording SMPTE timecode at 30 frames per second. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName RateAttributeName;

	/** Name of the custom attribute representing the name of a take. */
	UPROPERTY(EditAnywhere, Category = TimecodeCustomAttributeNameSettings)
	FName TakenameAttributeName;
};

struct UE_DEPRECATED(5.0, "FCustomAttribute has been deprecated") FCustomAttribute;
USTRUCT(Experimental)
struct FCustomAttribute
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** Name of this attribute */
	UPROPERTY(VisibleAnywhere, Category = CustomAttribute)
	FName Name;

	/** (FVariant) type contained by Values array */
	UPROPERTY(VisibleAnywhere, Category = CustomAttribute)
	int32 VariantType = 0;

	/** Time keys (should match number of Value entries) */
	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	TArray<float> Times;
		
	/** Value keys (should match number of Times entries) */
	TArray<FVariant> Values;

	bool Serialize(FArchive& Ar)
	{
		Ar << Name;
		Ar << VariantType;
		Ar << Times;
		Ar << Values;

		return true;
	}
#endif // WITH_EDITORONLY_DATA
};

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Custom serializer required for FVariant array
template<>
struct TStructOpsTypeTraits<FCustomAttribute> : public TStructOpsTypeTraitsBase2<FCustomAttribute>
{
	enum
	{
		WithSerializer = true
	};
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

/** Structure describing custom attributes for a single bone (index) */
struct UE_DEPRECATED(5.0, "FCustomAttributePerBoneData has been deprecated") FCustomAttributePerBoneData;

USTRUCT(Experimental)
struct FCustomAttributePerBoneData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	int32 BoneTreeIndex = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = CustomAttributeBoneData)
	TArray<FCustomAttribute> Attributes;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};

/** (Baked) string custom attribute, uses FStringCurve for evaluation instead of FVariant array */
struct UE_DEPRECATED(5.0, "FBakedStringCustomAttribute has been deprecated") FBakedStringCustomAttribute;

USTRUCT(Experimental)
struct FBakedStringCustomAttribute
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FName AttributeName;

	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FStringCurve StringCurve;
#endif // WITH_EDITORONLY_DATA
};

/** (Baked) int32 custom attribute, uses FIntegralCurve for evaluation instead of FVariant array */
struct UE_DEPRECATED(5.0, "FBakedIntegerCustomAttribute has been deprecated") FBakedIntegerCustomAttribute;

USTRUCT(Experimental)
struct FBakedIntegerCustomAttribute
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FName AttributeName;

	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FIntegralCurve IntCurve;
#endif // WITH_EDITORONLY_DATA
};

/** (Baked) float custom attribute, uses FSimpleCurve for evaluation instead of FVariant array */
struct UE_DEPRECATED(5.0, "FBakedFloatCustomAttribute has been deprecated") FBakedFloatCustomAttribute;

USTRUCT(Experimental)
struct FBakedFloatCustomAttribute
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FName AttributeName;

	UPROPERTY(VisibleAnywhere, Category = CustomAttributeBoneData)
	FSimpleCurve FloatCurve;
#endif // WITH_EDITORONLY_DATA
};

/** Structure describing baked custom attributes for a single bone (index) */
struct UE_DEPRECATED(5.0, "FBakedCustomAttributePerBoneData has been deprecated") FBakedCustomAttributePerBoneData;

USTRUCT(Experimental)
struct FBakedCustomAttributePerBoneData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 BoneTreeIndex = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = CustomAttributeBoneData)
	TArray<FBakedStringCustomAttribute> StringAttributes;

	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = CustomAttributeBoneData)
	TArray<FBakedIntegerCustomAttribute> IntAttributes;

	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = CustomAttributeBoneData)
	TArray<FBakedFloatCustomAttribute> FloatAttributes;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};
