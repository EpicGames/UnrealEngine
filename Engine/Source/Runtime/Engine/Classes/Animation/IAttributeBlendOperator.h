// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE
{
	namespace Anim
	{
		struct FAttributeBlendData;
		struct FStackAttributeContainer;

		/** Interface required to implement for user-defined blending behaviour of an animation attribute type. See TAttributeBlendOperator for an example implementation. */
		class IAttributeBlendOperator
		{
		public:
			virtual ~IAttributeBlendOperator() = default;

			/** Invoked when two or multiple sets of attribute container inputs are to be blended together*/
			virtual void Blend(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const = 0;

			/** Invoked when two or multiple sets of attribute container inputs are to be blended together, using individual bone weights */
			virtual void BlendPerBone(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const = 0;
			
			/** Invoked when an attribute container A is expected to override attributes in container B */
			virtual void Override(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const = 0;

			/** Invoked when an attribute container A is accumulated into container B */
			virtual void Accumulate(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const = 0;
			
			/** Invoked when an attribute container A is accumulated into container B with additional per attribute weight overrides */
			virtual void AccumulateWithOverrideWeights(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAttributes) const = 0;

			/** Invoked when an attribute container is supposed to be made additive with regards to container B */
			virtual void ConvertToAdditive(const FAttributeBlendData& BlendData, FStackAttributeContainer* OutAdditiveAttributes) const = 0;

			/** Invoked to interpolate between two individual attribute type values, according to the provided alpha */
			virtual void Interpolate(const void* FromData, const void* ToData, float Alpha, void* InOutData) const = 0;
		};
	}
}
