// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AlphaBlend.h"
#include "InputScaleBias.generated.h"

struct FAnimationUpdateContext;

// Input modifier with scaling and biasing
USTRUCT(BlueprintType)
struct FInputScaleBias
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	float Scale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	float Bias;

public:
	FInputScaleBias()
		: Scale(1.0f)
		, Bias(0.0f)
	{
	}

	ENGINE_API float ApplyTo(float Value) const;
#if WITH_EDITOR
	ENGINE_API FText GetFriendlyName(FText InFriendlyName) const;
#endif
};

USTRUCT(BlueprintType)
struct FInputRange
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Min;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Max;

public:
	FInputRange()
		: Min(0.0f)
		, Max(1.0f)
	{
	}

	FInputRange(const float InMin, const float InMax)
		: Min(InMin)
		, Max(InMax)
	{}

	FVector2f ToVector2f() const { return FVector2f(Min, Max); } 
	FVector2D ToVector2D() const { return (FVector2D)ToVector2f(); } 
	
	bool ClampValue(float& InOutValue) const 
	{ 
		const float OriginalValue = InOutValue;
		InOutValue = FMath::Clamp(InOutValue, Min, Max);
		return (OriginalValue != InOutValue);
	}
};

// Input modifier with remapping, scaling, biasing, clamping, and interpolation
USTRUCT(BlueprintType)
struct FInputScaleBiasClamp
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bMapRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bClampResult;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bInterpResult;

	mutable bool bInitialized;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bMapRange"))
	FInputRange InRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bMapRange"))
	FInputRange OutRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Scale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Bias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMax;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing;

	mutable float InterpolatedResult;

public:
	FInputScaleBiasClamp()
		: bMapRange(false)
		, bClampResult(false)
		, bInterpResult(false)
		, bInitialized(false)
		, Scale(1.0f)
		, Bias(0.0f)
		, ClampMin(0.f)
		, ClampMax(1.f)
		, InterpSpeedIncreasing(10.f)
		, InterpSpeedDecreasing(10.f)
		, InterpolatedResult(0.f)
	{
	}

	ENGINE_API float ApplyTo(float Value, float InDeltaTime) const;

	void Reinitialize() { bInitialized = false; }
#if WITH_EDITOR
	ENGINE_API FText GetFriendlyName(FText InFriendlyName) const;
#endif
};

// Input modifier with clamping and interpolation
USTRUCT(BlueprintType)
struct FInputClampConstants
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bClampResult = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bInterpResult = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMin = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMax = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing = 10.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing = 10.f;

#if WITH_EDITOR
	// Get a friendly name to display on a pin
	ENGINE_API FText GetFriendlyName(FText InFriendlyName) const;
#endif
};

// Mutable state struct to be used with FInputClampConstants
USTRUCT(BlueprintType)
struct FInputClampState
{
	GENERATED_BODY()

	// The interpolated result
	float InterpolatedResult = 0.f;

	// Whether this state is initialized
	bool bInitialized = false;

	// Apply scale, bias, and clamp to value
	ENGINE_API float ApplyTo(const FInputClampConstants& InConstants, float InValue, float InDeltaTime);

	void Reinitialize() { bInitialized = false; }
};

USTRUCT(BlueprintType)
struct FInputScaleBiasClampConstants
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bMapRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bClampResult;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bInterpResult;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bMapRange"))
	FInputRange InRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bMapRange"))
	FInputRange OutRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Scale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float Bias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bClampResult"))
	float ClampMax;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing;

public:
	FInputScaleBiasClampConstants()
		: bMapRange(false)
		, bClampResult(false)
		, bInterpResult(false)
		, Scale(1.0f)
		, Bias(0.0f)
		, ClampMin(0.f)
		, ClampMax(1.f)
		, InterpSpeedIncreasing(10.f)
		, InterpSpeedDecreasing(10.f)
	{
	}

#if WITH_EDITOR
	// Get a friendly name to display on a pin
	ENGINE_API FText GetFriendlyName(FText InFriendlyName) const;

	// Copy parameters from the legacy combined constants/state structure
	ENGINE_API void CopyFromLegacy(const FInputScaleBiasClamp& InLegacy);
#endif
};

// Mutable state struct to be used with FInputScaleBiasClampConstants
USTRUCT(BlueprintType)
struct FInputScaleBiasClampState
{
	GENERATED_BODY()

	// The interpolated result
	float InterpolatedResult;

	// Whether this state is initialized
	bool bInitialized;

public:
	FInputScaleBiasClampState()
		: InterpolatedResult(0.f)
		, bInitialized(false)
	{
	}

	// Apply scale, bias, and clamp to value
	ENGINE_API float ApplyTo(const FInputScaleBiasClampConstants& InConstants, float Value, float InDeltaTime);

	// Apply but dont modify InterpolatedResult
	ENGINE_API float ApplyTo(const FInputScaleBiasClampConstants& InConstants, float Value) const;

	void Reinitialize() { bInitialized = false; }
};

// AnimNodes using an Alpha can choose how it is driven.
UENUM()
enum class EAnimAlphaInputType : uint8
{
	Float	UMETA(DisplayName = "Float Value"),
	Bool	UMETA(DisplayName = "Bool Value"),
	Curve	UMETA(DisplayName = "Anim Curve Value")
};

USTRUCT(BlueprintType)
struct FInputAlphaBoolBlend
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	bool Serialize(FStructuredArchive::FSlot Slot);
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float BlendInTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float BlendOutTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EAlphaBlendOption BlendOption;

	UPROPERTY(Transient)
	bool bInitialized;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TObjectPtr<UCurveFloat> CustomCurve;

	UPROPERTY(Transient)
	FAlphaBlend AlphaBlend;

	FInputAlphaBoolBlend()
		: BlendInTime(0.0f)
		, BlendOutTime(0.0f)
		, BlendOption(UE::Anim::DefaultBlendOption)
		, bInitialized(false)
		, CustomCurve(nullptr)
	{}

	ENGINE_API float ApplyTo(bool bEnabled, float InDeltaTime);

	void Reinitialize() { bInitialized = false; }
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FInputAlphaBoolBlend> : public TStructOpsTypeTraitsBase2<FInputAlphaBoolBlend>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializer = true,
	};
};
#endif // WITH_EDITORONLY_DATA
