// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDeformer.generated.h"

class UMeshComponent;
class UMeshDeformerInstance;
class UMeshDeformerInstanceSettings;

/** 
 * Base class for mesh deformer assets.
 * Mesh deformers can be added to mesh components to implement flexible deformation systems.
 * A UMeshDeformer needs to implement creation of a UMeshDeformerInstance which will apply deformer actions and store deformer state.
 */
UCLASS(Abstract, MinimalAPI)
class UMeshDeformer : public UObject
{
	GENERATED_BODY()

public:
	/** Create persistent settings for the mesh deformer instance */ 
	virtual UMeshDeformerInstanceSettings* CreateSettingsInstance(
		UMeshComponent* InMeshComponent
		) PURE_VIRTUAL(, return nullptr;);
	
	/** Create an instance of the mesh deformer. */
	virtual UMeshDeformerInstance* CreateInstance(
		UMeshComponent* InMeshComponent,
		UMeshDeformerInstanceSettings* InSettings
		) PURE_VIRTUAL(, return nullptr;);
};
