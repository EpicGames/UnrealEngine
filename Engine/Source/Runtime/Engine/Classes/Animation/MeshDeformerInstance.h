// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA
#include "Animation/MeshDeformerGeometryReadback.h"
#endif // WITH_EDITORONLY_DATA

#include "MeshDeformerInstance.generated.h"

class FSceneInterface;
#if WITH_EDITORONLY_DATA
struct FMeshDescription;
#endif // WITH_EDITORONLY_DATA

enum class EMeshDeformerOutputBuffer : uint8
{
	None = 0,
	SkinnedMeshPosition = 1 << 0,
	SkinnedMeshTangents = 1 << 1,
	SkinnedMeshVertexColor = 1 << 2,
};

ENUM_CLASS_FLAGS(EMeshDeformerOutputBuffer);

/**
 * Base class for mesh deformers instance settings.
 * This contains the serialized user settings to apply to the UMeshDeformer.
 */
UCLASS(Abstract, MinimalAPI)
class UMeshDeformerInstanceSettings : public UObject
{
	GENERATED_BODY()
};


/** 
 * Base class for mesh deformers instances.
 * This contains the transient per instance state for a UMeshDeformer.
 */
UCLASS(Abstract, MinimalAPI)
class UMeshDeformerInstance : public UObject
{
	GENERATED_BODY()

public:


	/** Called to allocate any persistent render resources */
	virtual void AllocateResources() PURE_VIRTUAL(, );

	/** Called when persistent render resources should be released */
	virtual void ReleaseResources() PURE_VIRTUAL(, );

	/** Enumeration for workloads to EnqueueWork. */
	enum EWorkLoad
	{
		WorkLoad_Setup,
		WorkLoad_Trigger,
		WorkLoad_Update,
	};

	/** Enumeration for execution groups to EnqueueWork on. */
	enum EExectutionGroup
	{
		ExecutionGroup_Default,
		ExecutionGroup_Immediate,
		ExecutionGroup_EndOfFrameUpdate,
		ExecutionGroup_BeginInitViews,
	};

	/** Structure of inputs to EnqueueWork. */
	struct FEnqueueWorkDesc
	{
		FSceneInterface* Scene = nullptr;
		EWorkLoad WorkLoadType = WorkLoad_Update;
		EExectutionGroup ExecutionGroup = ExecutionGroup_Default;
		/** Name used for debugging and profiling markers. */
		FName OwnerName;
		/** Render thread delegate that will be executed if Enqueue fails at any stage. */
		FSimpleDelegate FallbackDelegate;
	};

	/** Enqueue the mesh deformer workload on a scene. */
	virtual void EnqueueWork(FEnqueueWorkDesc const& InDesc) PURE_VIRTUAL(, );
	
	/** Return the buffers that this deformer can potentially write to */
	virtual EMeshDeformerOutputBuffer GetOutputBuffers() const PURE_VIRTUAL(, return EMeshDeformerOutputBuffer::None; );

#if WITH_EDITORONLY_DATA
	/** Reads back the deformed geometry and generates a mesh description */
	virtual bool RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest) PURE_VIRTUAL(, return false; );
#endif // WITH_EDITORONLY_DATA
	
	/** Returns the specific instance that directly represents the source deformer, this is needed as a deformer may create intermediate instances that aren't
	 * necessarily user-facing.
	 */
	virtual UMeshDeformerInstance* GetInstanceForSourceDeformer() PURE_VIRTUAL(, return this; ); 
};
