// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "MeshDeformerProducer.generated.h"

class UMeshDeformerInstance;

/** Interface for all the UObject able to add directly mesh deformer instances to the manager */
UINTERFACE(MinimalAPI)
class UMeshDeformerProducer: public UInterface
{
	GENERATED_BODY()
};

class IMeshDeformerProducer
{
	GENERATED_BODY()

public:
	DECLARE_EVENT_OneParam(UMeshDeformerInstance, FMeshDeformerBeginDestroyEvent, class IMeshDeformerProducer*);
	virtual FMeshDeformerBeginDestroyEvent& OnBeginDestroy() = 0;
};

