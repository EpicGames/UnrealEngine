// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "BoneContainer.h"
#include "CustomBoneIndexArray.h"
#include "Containers/StaticArray.h"
#include "MirrorDataTable.generated.h"

/** Type referenced by a row in the mirror data table */
UENUM()
namespace EMirrorRowType
{
	enum Type : int
	{
		Bone,
		AnimationNotify,
		Curve,
		SyncMarker,
		Custom
	};
}


/** Find and Replace Method for FMirrorFindReplaceExpression. */
UENUM()
namespace EMirrorFindReplaceMethod
{
	enum Type : int
	{
		/** Only find and replace matching strings at the start of the name  */
		Prefix,
        /** Only find and replace matching strings at the end of the name  */
        Suffix,
        /** Use regular expressions for find and replace, including support for captures $1 - $10 */
        RegularExpression
    };
}

#if WITH_EDITORONLY_DATA

/** Controls which bones are updated when refreshing the mirror data table from the skeleton. */
UENUM()
enum class EMirrorTableBoneRefreshScope : uint8
{
	/** Refreshes all bones found in the associated skeleton. */
	FullSkeleton,

	/** Only refresh the bones explicitly listed in the Specific Bone List. Useful for partial mirrors or managing a large skeleton. */
	ExplicitBoneList,
};

#endif

/**  Base Mirror Table containing all data required by the animation mirroring system. */
USTRUCT()
struct FMirrorTableRow : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mirroring, meta=(EditCondition="bEnabled"))
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mirroring, meta=(EditCondition="bEnabled"))
	FName MirroredName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mirroring, meta=(EditCondition="bEnabled"))
	TEnumAsByte<EMirrorRowType::Type> MirrorEntryType = EMirrorRowType::Bone;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mirroring)
	bool bEnabled = true;
	
	FMirrorTableRow() = default;

	FMirrorTableRow(const FName& InName, const FName& InMirroredName, const TEnumAsByte<EMirrorRowType::Type>& InMirrorEntryType, bool bInEnabled)
		: Name(InName)
		, MirroredName(InMirroredName)
		, MirrorEntryType(InMirrorEntryType)
		, bEnabled(bInEnabled)
	{
	}

	ENGINE_API FMirrorTableRow(const FMirrorTableRow& Other);
	ENGINE_API FMirrorTableRow& operator=(FMirrorTableRow const& Other);
	ENGINE_API bool operator==(FMirrorTableRow const& Other) const;
	ENGINE_API bool operator!=(FMirrorTableRow const& Other) const;
	ENGINE_API bool operator<(FMirrorTableRow const& Other) const;
};


/** Find and Replace expressions used to generate mirror tables*/
USTRUCT()
struct FMirrorFindReplaceExpression
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Mirroring)
	FName FindExpression;

	UPROPERTY(EditAnywhere, Category = Mirroring)
	FName ReplaceExpression;

	UPROPERTY(EditAnywhere, Category = Mirroring)
	TEnumAsByte<EMirrorFindReplaceMethod::Type> FindReplaceMethod;

	FMirrorFindReplaceExpression() 
		: FindExpression(NAME_None)
		, ReplaceExpression(NAME_None)
		, FindReplaceMethod(EMirrorFindReplaceMethod::Prefix) {}

	FMirrorFindReplaceExpression(FName InFindExpression, FName InReplaceExpression, EMirrorFindReplaceMethod::Type Method)
		: FindExpression(InFindExpression)
		, ReplaceExpression(InReplaceExpression)
		, FindReplaceMethod(Method)
	{
	}
};

/**
 * Data table for mirroring bones, notifies, and curves. The mirroring table allows self mirroring with entries where the name and mirrored name are identical
 */
UCLASS(MinimalAPI, BlueprintType, hideCategories = (ImportOptions, ImportSource) /* AutoExpandCategories = "MirrorDataTable,ImportOptions"*/)
class UMirrorDataTable : public UDataTable
{
	GENERATED_BODY()

	friend class UMirrorDataTableFactory;

public:
	UMirrorDataTable(const FObjectInitializer& ObjectInitializer);

	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/**
	 * Apply the animation settings mirroring find and replace strings against the given name, returning
	 * the mirrored name or NAME_None if none of the find strings are found in the name. 
	 * 
	 * @param	InName		Name to map against animation settings mirroring find and replace 
	 * @return				The mirrored name or NAME_None
	 */
	ENGINE_API static FName GetSettingsMirrorName(FName InName); 

	/**
	 * Apply the provided find and replace strings against the given name, returning
	 * the mirrored name or NAME_None if none of the find strings are found in the name. 
	 * 
	 * @param	MirrorFindReplaceExpressions		Find and replace expressions.  The first matching expression will be returned
	 * @param	InName								Name to find and replace 
	 * @return										The mirrored name or NAME_None if none of the expressions match
	 */
	ENGINE_API static FName GetMirrorName(FName InName, const TArray<FMirrorFindReplaceExpression>& MirrorFindReplaceExpressions);

	/**
     * Create Mirror Bone Indices for the provided BoneContainer.  The CompactBonePoseMirrorBones provides an index map which can be used to mirror at runtime
	 *
	 * @param	BoneContainer					The Bone Container that the OutCompactPaseMirrorBones should match
	 * @param	MirrorBoneIndexes				Mirror bone indexes created for the ReferenceSkeleton used by the BoneContainer 
	 * @param	OutCompactPoseMirrorBones		An efficient representation of the bones to mirror which can be used at runtime
	 */
	ENGINE_API static void FillCompactPoseMirrorBones(const FBoneContainer& BoneContainer, const TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex>& MirrorBoneIndexes, TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex>& OutCompactPoseMirrorBones);


	/**
	 * Converts the mirror data table Name -> MirrorName map into an index map for the given ReferenceSkeleton
	 *
	 * @param	ReferenceSkeleton		The ReferenceSkeleton to compute the mirror index against
	 * @param	OutMirrorBoneIndexes	An array that provides the bone index of the mirror bone, or INDEX_NONE if the bone is not mirrored
	 */
	ENGINE_API void FillMirrorBoneIndexes(const USkeleton* Skeleton, TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex>& OutMirrorBoneIndexes) const;

	/**
	 * Populates two arrays with a mapping of compact pose mirror bones and reference rotations
	 *
	 * @param	BoneContainer					Structure which holds the required bones
	 * @param	OutCompactPoseMirrorBones		Output array mapping compact pose mirror bones
	 * @param	OutComponentSpaceRefRotations	Output array mapping reference rotations
	 */
	ENGINE_API void FillCompactPoseAndComponentRefRotations(
		const FBoneContainer& BoneContainer,
		TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex>& OutCompactPoseMirrorBones,
		TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex>& OutComponentSpaceRefRotations) const;

#if WITH_EDITOR

	/** Describes the sync relationship between the mirror data table and its associated skeleton. */
	enum class ESyncStatus : uint8
	{
		Unknown,   // Never been synced
		UpToDate,  // GUIDs match the current skeleton
		Stale,     // GUIDs no longer match the current skeleton
	};

	/** Options for updating the table from find/replace expressions. */
	struct FFindReplaceOptions
	{
		/** Controls which mutations are allowed during the find/replace refresh. */
		enum class EFlags : uint8
		{
			None = 0,						// Do not perform row mutations.
			UpdateExistingRows = 1 << 0,	// Only update rows that already exist in the mirror data table.
			AddMissingRows = 1 << 1,		// Add rows for skeleton entries that do not yet exist in the mirror data table.
			DisableStaleRows = 1 << 2,		// Disable existing skeleton-derived rows that were not found in current skeleton scan.
			DeleteStaleRows = 1 << 3,		// Deletes existing skeleton-derived rows that were not found in current skeleton scan.
		};
		
		/** Combination of operations to perform when applying find/replace expressions. Defaults to legacy behavior. */
		EFlags Flags = EFlags::AddMissingRows;
		
		/** Whether to show a completion notification once the operation is finished. */
		bool bShowNotification = false;
		
		/** Preset: Only add missing rows. */
		ENGINE_API static FFindReplaceOptions AddMissingOnly();

		/** Preset: Updates existing rows and disables stale rows. */
		ENGINE_API static FFindReplaceOptions UpdateExisting();

		/** Preset: Update existing rows, adds missing rows, and disables stale rows. */
		ENGINE_API static FFindReplaceOptions Sync();
		
		/** Sets whether to show a completion notification. */
		ENGINE_API FFindReplaceOptions& WithNotification(bool bInShowNotification = true);
	};
	
	/**
	 * Populates the table by running the MirrorFindReplaceExpressions on bone names in the Skeleton.  If the mirrored name is also found 
	 * on the Skeleton it is added to the table.
	 */
	UE_DEPRECATED(5.8, "Please use UpdateFromFindReplaceExpressions() instead.")
	ENGINE_API void FindReplaceMirroredNames();
	
	/** Updates the table from the current find/replace expressions and skeleton data. */
	ENGINE_API void UpdateFromFindReplaceExpressions(const FFindReplaceOptions& Options);
	
	/** Returns the sync status of the table relative to its associated skeleton. */
	ENGINE_API ESyncStatus GetSkeletonSyncStatus() const;

	/** Marks the cached skeleton data as stale. The next call to GetSkeletonSyncStatus() will return Stale until the table is refreshed from the skeleton. */
	ENGINE_API void InvalidateCachedSkeletonData();

#endif // WITH_EDITOR

	/**
	 * Evaluate the MirrorFindReplaceExpressions on InName and return the replaced value of the first entry that matches
	 *
	 * @param	InName		The input string to find & replace
	 * @return				The replaced result of the first MirrorFindReplaceExpression where the find pattern matched
	 */
	ENGINE_API FName FindReplace(FName InName) const;

	/**
	 * Finds the "best matching" mirrored bone across the specified axis. Priority is given to bones with the mirrored name,
	 * falling back to spatial proximity if no mirrored bone is found using the naming rules.
	 *
	 * When falling back to proximity, bones within the SearchThreshold distance are considered coincident and a fuzzy string
	 * comparison is used to find the most likely bone that matches the input bone.
	 *
	 * NOTE: The naming scheme assumes a mirror axis of X (Left/Right). Naming rules for other axes are not supported.
	 *
	 * @param	InBoneName		The input bone for which you want to find the mirrored equivalent
	 * @param	InRefSkeleton	The reference skeleton used to find bone names and their spatial relationships (in ref pose)
	 * @param	InMirrorAxis	The axis to cross when searching for a mirrored bone
	 * @param	SearchThreshold	The distance in Unreal units to consider when trying to "tie-break" coincident bones
	 * @return					The "best match" mirrored bone
	 */
	ENGINE_API static FName FindBestMirroredBone(
		const FName InBoneName,
		const FReferenceSkeleton& InRefSkeleton,
		EAxis::Type InMirrorAxis,
		const float SearchThreshold = 2.0f);

public:

	/** Used to compute the mirror pair of the data table entries. */
	UPROPERTY(EditAnywhere, Category = CreateTable)
	TArray<FMirrorFindReplaceExpression> MirrorFindReplaceExpressions;

	/** Axis to mirror across (X/Y/Z). Use to flip transforms/poses. */
	UPROPERTY(EditAnywhere, Category = Mirroring)
	TEnumAsByte<EAxis::Type> MirrorAxis;

	/** Used to determine if the root motion attribute should be mirrored when using this mirror data table. */
	UPROPERTY(EditAnywhere, Category = Mirroring)
	bool  bMirrorRootMotion = true;
	
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Skeleton)
	TObjectPtr<USkeleton> Skeleton; 

	// Index of the mirror bone for a given bone index in the reference skeleton, or INDEX_NONE if the bone is not mirrored
	TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex> BoneToMirrorBoneIndex;

	// Set of all the disabled mirror entries/pairs
	TSet<FName> DisabledEntries;
	
	// Map from animation curve to mirrored animation curve	
	TMap<FName, FName> CurveToMirrorCurveMap;
	
	// Map from animation notify to mirrored animation notify
	TMap<FName, FName> AnimNotifyToMirrorAnimNotifyMap;
	
	// Map from sync marker to mirrored sync marker 
	TMap<FName, FName> SyncToMirrorSyncMap;

#if WITH_EDITORONLY_DATA
	
	/** Determines which bones are updated when refreshing the table from the associated skeleton. Set to 'Explicit Bone List' to target only specific bones. */
	UPROPERTY(EditAnywhere, Category = Import)
	EMirrorTableBoneRefreshScope BoneScope = EMirrorTableBoneRefreshScope::FullSkeleton;

	/** The set of bones to refresh when Bone Scope is set to 'Explicit Bone List'. Managed via the bone picker in the Import panel. */
	UPROPERTY()
	TSet<FName> BoneScopeNameList;
	
#endif
	
protected: 

	// Fill BoneToMirrorBoneIndex, CurveMirrorSourceUIDArray, CurveMirrorTargetUIDArray and NotifyToMirrorNotifyIndex based on the Skeleton and Table Contents
	ENGINE_API void FillMirrorArrays();

private:
	
	/** Contains entry data for a given row type (bone/notify/sync marker/curve/custom). */
	struct FCategoryState
	{
		/** Existing that already exist in the table for this category. */
		TSet<FName> ExistingEntryNames;
		
		/** Entries processed from the current skeleton scan for this category. */
		TSet<FName> ProcessedEntryNames;
		
		/**
		 * Maps logical entry names to row names in the table.
		 * Row names must be unique across all categories, for example:
		 *   left_1 -> left_1:Bone
		 *   left_1 -> left_1:SyncMarker
		 */
		TMap<FName, FName> EntryNameToRowName;
		
		/** Entries with duplicate rows in the table for this category. */
		TSet<FName> DuplicateEntryNames;
	};
	
	/** Collects all the existing mirror entries and categorizes them by their mirror row type. */
	void InitCategoryEntryStates(TStaticArray<FCategoryState, 5> & OutStates) const;
	
	/** Generates a unique row name for a mirror data table entry. */
	FName MakeUniqueMirrorRowName(const FName EntryName, const EMirrorRowType::Type RowType) const;
	
#if WITH_EDITORONLY_DATA

	UPROPERTY()
	FGuid SkeletonHierarchyGuid;

	UPROPERTY()
	FGuid SkeletonVirtualBonesHierarchyGuid;

#endif
	
};

#if WITH_EDITOR
ENUM_CLASS_FLAGS(UMirrorDataTable::FFindReplaceOptions::EFlags);
#endif