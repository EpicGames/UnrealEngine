// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "PackedNormal.h"
#include "Rendering/MorphTargetVertexCodec.h"
#include "Serialization/MemoryArchive.h"

#include "MorphTarget.generated.h"

class USkeletalMesh;
class UStaticMesh;

/** Morph mesh vertex data used for rendering */
struct FMorphTargetDelta
{
	/** change in position */
	FVector3f			PositionDelta = FVector3f::ZeroVector;

	/** Tangent basis normal */
	FVector3f			TangentZDelta = FVector3f::ZeroVector;

	/** index of source vertex to apply deltas to */
	uint32			SourceIdx = std::numeric_limits<uint32>::max();

	/** pipe operator */
	friend FArchive& operator<<(FArchive& Ar, FMorphTargetDelta& V)
	{
		if ((Ar.UEVer() < VER_UE4_MORPHTARGET_CPU_TANGENTZDELTA_FORMATCHANGE) && Ar.IsLoading())
		{
			/** old format of change in tangent basis normal */
			FDeprecatedSerializedPackedNormal TangentZDelta_DEPRECATED;
			Ar << V.PositionDelta << TangentZDelta_DEPRECATED << V.SourceIdx;
			V.TangentZDelta = TangentZDelta_DEPRECATED;
		}
		else
		{
			Ar << V.PositionDelta << V.TangentZDelta << V.SourceIdx;
		}
		return Ar;
	}
};

#if !WITH_EDITOR
/** Compressed morph vertex deltas, used for cooked serialization. Only available in game. This matched the
 *  GPU compressed delta storage and has the same quantization behavior. The tolerance stored is the same
 *  value that was used when the deltas were compressed and should not be changed, or the decompression will
 *  result in erroneous values.
 */
struct FMorphTargetCompressedLODModel
{
	TArray<UE::MorphTargetVertexCodec::FDeltaBatchHeader> PackedDeltaHeaders;
	TArray<uint32> PackedDeltaData;
	float PositionPrecision = 0.0f;
	float TangentPrecision = 0.0f;
};
#endif


class FMorphTargetDeltaIterator
{
public:
	FMorphTargetDeltaIterator() = default;
	FMorphTargetDeltaIterator(const FMorphTargetDeltaIterator&) = default;
	FMorphTargetDeltaIterator(FMorphTargetDeltaIterator&&) = default;
	FMorphTargetDeltaIterator& operator=(const FMorphTargetDeltaIterator&) = default;

	// Support the minimal functionality required for a ranged-for.
	[[nodiscard]] const FMorphTargetDelta& operator*() const 
	{
		return CurrentDelta;
	}
	
	[[nodiscard]] const FMorphTargetDelta* operator->() const
	{
		return &CurrentDelta;
	}
	
	FMorphTargetDeltaIterator& operator++()
	{
		Advance();
		return *this;
	}

	[[nodiscard]] bool AtEnd() const
	{
		return Token == InvalidToken;
	}
	
private:
	static constexpr uint64 InvalidToken = std::numeric_limits<uint64>::max(); 
	friend class UMorphTarget;

	FMorphTargetDeltaIterator(
		TConstArrayView<FMorphTargetDelta> InRawDeltas
		) :
		Token(0),
		RawDeltas(InRawDeltas)
	{
		Advance();
	}
#if !WITH_EDITOR
	FMorphTargetDeltaIterator(
		const FMorphTargetCompressedLODModel& InCompressedDeltas 
		) :
		Token(0),
		CompressedDelta(&InCompressedDeltas)
	{
		if (!InCompressedDeltas.PackedDeltaHeaders.IsEmpty() && !InCompressedDeltas.PackedDeltaData.IsEmpty())
		{
			Advance();
		}
		else
		{
			Clear();
		}
	}
#endif
	
	void Advance()
	{
		if (!RawDeltas.IsEmpty())
		{
			if (Token < RawDeltas.Num())
			{
				CurrentDelta = RawDeltas[static_cast<int32>(Token++)];
			}
			else
			{
				Clear();
			}
		}
#if !WITH_EDITOR
		else if (!CompressedDelta || !IterativeDecode(Token, CompressedDelta->PackedDeltaHeaders, CompressedDelta->PackedDeltaData, CompressedDelta->PositionPrecision, CompressedDelta->TangentPrecision, CurrentDelta))
		{
			// If the decode failed, we're either at the end, or the data is invalid. Stop any further decoding.
			Clear();
		}
#endif
	}

	void Clear()
	{
		Token = InvalidToken;
		RawDeltas = {};
#if !WITH_EDITOR
		CompressedDelta = nullptr;
#endif
	}

	uint64 Token = InvalidToken;
	TConstArrayView<FMorphTargetDelta> RawDeltas;
#if !WITH_EDITOR
	const FMorphTargetCompressedLODModel* CompressedDelta = nullptr;
#endif
	FMorphTargetDelta CurrentDelta;
};


/**
* Mesh data for a single LOD model of a morph target
*/
struct FMorphTargetLODModel
{
	/** vertex data for a single LOD morph mesh */
	TArray<FMorphTargetDelta> Vertices;

	/** Number of elements in Vertices array. This property is set at runtime and is not serialized. */
	int32 NumVertices = 0;

	/** number of original verts in the base mesh */
	int32 NumBaseMeshVerts = 0;
	
	/** list of sections this morph is used */
	TArray<int32> SectionIndices;

	/** Is this LOD generated by reduction setting */
	bool bGeneratedByEngine = false;

	/** The source filename use to import this morph target. If source is empty this morph target was import with the LOD geometry.
	    Only defined in editor, since we don't need this information to spill into a cooked build. 
	    */
	FString SourceFilename;
	
	FMorphTargetLODModel() = default;

	/** pipe operator */
	friend FArchive& operator<<(FArchive& Ar, FMorphTargetLODModel& M);

	void Reset()
	{
		Vertices.Reset();
		NumVertices = 0;
		NumBaseMeshVerts = 0;
		SectionIndices.Reset();
		// since engine cleared it, we mark as engine generated
		// this makes it clear to clear up later
		bGeneratedByEngine = true;
		SourceFilename.Empty();
	}
};


#if WITH_EDITOR
/**
* Data to cache serialization results for async asset building
*/
struct FFinishBuildMorphTargetData
{
public:
	virtual ~FFinishBuildMorphTargetData() = default;
	
	/** Load morph target data */
	ENGINE_API virtual void LoadFromMemoryArchive(FMemoryArchive & Ar);
	
	/** Apply serialized data to skeletal mesh in game thread */
	ENGINE_API virtual void ApplyEditorData(USkeletalMesh * SkeletalMesh, bool bIsSerializeSaving) const;
	
protected:
	bool bApplyMorphTargetsData = false;
	TMap<FName, TArray<FMorphTargetLODModel>> MorphLODModelsPerTargetName;
	};
#endif

UCLASS(hidecategories=Object, MinimalAPI)
class UMorphTarget
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** USkeletalMesh that this vertex animation works on. */
	UPROPERTY(AssetRegistrySearchable)
	TObjectPtr<class USkeletalMesh> BaseSkelMesh;

	/** morph mesh vertex data for each LOD */
	virtual const TArray<FMorphTargetLODModel>& GetMorphLODModels() const { return MorphLODModels; }

	/** morph mesh vertex data for each LOD */
	virtual TArray<FMorphTargetLODModel>& GetMorphLODModels() { return MorphLODModels; }


#if !WITH_EDITOR
	/** compressed morph mesh vertex data for each LOD */
	virtual const TArray<FMorphTargetCompressedLODModel>& GetCompressedLODModels() const { return CompressedLODModels; }

	/** compressed morph mesh vertex data for each LOD */
	virtual TArray<FMorphTargetCompressedLODModel>& GetCompressedLODModels() { return CompressedLODModels; }
#endif

protected:
	/** morph mesh vertex data for each LOD */
	TArray<FMorphTargetLODModel>	MorphLODModels;
#if !WITH_EDITOR
	TArray<FMorphTargetCompressedLODModel> CompressedLODModels;
#endif
	
public:

#if WITH_EDITOR
	/** Get Morphtarget Delta array for the given input Index */
	UE_DEPRECATED(5.7, "Use the TConstArrayView version of GetMorphTargetDelta instead.")
	ENGINE_API virtual const FMorphTargetDelta* GetMorphTargetDelta(int32 LODIndex, int32& OutNumDeltas) const;

	/** Get the morph target vertex delta array for a given LOD. Only available in editor targets, since cooked builds
	 *  store the deltas internally in a compressed format. For non-editor builds, use GetDeltaIteratorForLOD */
	ENGINE_API virtual TConstArrayView<FMorphTargetDelta> GetMorphTargetDeltas(int32 LODIndex) const;
	
#endif

	/** Returns an iterator that runs over vertices, either the raw delta form or the cooked compressed form. Use this
	 *  function at runtime to avoid the overhead of keeping the uncompressed raw deltas in memory. */
	ENGINE_API virtual FMorphTargetDeltaIterator GetDeltaIteratorForLOD(int32 LODIndex) const;

	/** Returns the number of morph deltas for the given LOD. If 0, there is no morph for this LOD.
	 *  At runtime, this value may be non-zero, even if GetDeltaIteratorForLOD returns no morph deltas, since
	 *  the morphs might be tuned to only compress the deltas for the GPU, leaving no CPU deltas in memory.
	 *  In editor, this value will always match the number of entries returned by GetMorphTargetDeltas.
	 */
	ENGINE_API virtual int32 GetNumDeltasForLOD(int32 LODIndex) const;
	
	/** Returns true if the given LOD has morph data */
	ENGINE_API virtual bool HasDataForLOD(int32 LODIndex) const;
	
	/** return true if this morphtarget contains data for section within LOD */
	ENGINE_API virtual bool HasDataForSection(int32 LODIndex, int32 SectionIndex) const;
	
	/** return true if this morphtarget contains valid vertices */
	ENGINE_API virtual bool HasValidData() const;
	ENGINE_API virtual void EmptyMorphLODModels();

	/** return true if this morphtarget was custom imported from a file */
	ENGINE_API virtual bool IsCustomImported(int32 LODIndex) const;
	ENGINE_API virtual const FString& GetCustomImportedSourceFilename(int32 LODIndex) const;
	ENGINE_API virtual void SetCustomImportedSourceFilename(int32 LODIndex, const FString& InSourceFilename);

	/** return true if this morphtarget was generated by an engine tool */
	ENGINE_API virtual bool IsGeneratedByEngine(int32 LODIndex) const;
	ENGINE_API virtual void SetGeneratedByEngine(int32 LODIndex, bool bInGeneratedByEngine);

	/** Return true if this morph target uses engine built-in compression */
	virtual bool UsesBuiltinMorphTargetCompression() const { return true; }

#if WITH_EDITOR
	/** Populates the given morph target LOD model with the provided deltas */
	ENGINE_API virtual void PopulateDeltas(const TArray<FMorphTargetDelta>& Deltas, const int32 LODIndex, const TArray<struct FSkelMeshSection>& Sections, const bool bCompareNormal = false, const bool bGeneratedByReductionSetting = false, const float PositionThreshold = UE_THRESH_POINTS_ARE_NEAR);
	/** Remove empty LODModels */
	ENGINE_API virtual void RemoveEmptyMorphTargets();
	/** Factory function to define type of FinishBuildData needed*/
	ENGINE_API virtual TUniquePtr<FFinishBuildMorphTargetData> CreateFinishBuildMorphTargetData() const;
#endif // WITH_EDITOR

	//~ UObject interface

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass);
#endif
	ENGINE_API virtual void PostLoad() override;

	/** UObject does not support serialization via FMemoryArchive, so manually handle separately */
	ENGINE_API virtual void SerializeMemoryArchive(FMemoryArchive& Ar);
};
