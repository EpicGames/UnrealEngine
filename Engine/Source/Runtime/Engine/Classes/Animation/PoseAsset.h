// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Class of pose asset that can evaluate pose by weights
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/SmartName.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimCurveTypes.h"
#include "AnimData/IAnimationDataModel.h"
#include "PoseAsset.generated.h"

class UAnimSequence;
class USkeletalMeshComponent;
struct FLiveLinkCurveElement;
struct FReferenceSkeleton;
class FPoseAssetDetails;
class FAnimationRuntimeRetargetBoneTransformContext;

/** 
 * Pose data 
 * 
 * This is one pose data structure
 * This will let us blend poses quickly easily
 * All poses within this asset should contain same number of tracks, 
 * so that we can blend quickly
 */

USTRUCT()
struct FPoseData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	// source local space pose, this pose is always full pose
	// the size this array matches Tracks in the pose container
	UPROPERTY()
	TArray<FTransform>		SourceLocalSpacePose;

	// source curve data that is full value
	UPROPERTY()
	TArray<float>			SourceCurveData;
#endif // WITH_EDITORONLY_DATA

	// local space pose, # of array match with # of TrackToBufferIndex
	// it only saves the one with delta as base pose or ref pose if full pose
	UPROPERTY()
	TArray<FTransform>		LocalSpacePose;

	// # of array match with # of Curves in PoseDataContainer
	// curve data is not compressed
 	UPROPERTY()
 	TArray<float>			CurveData;
};

USTRUCT()
struct FPoseAssetInfluence
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 PoseIndex = INDEX_NONE;

	UPROPERTY()
	int32 BoneTransformIndex = INDEX_NONE;
};

USTRUCT()
struct FPoseAssetInfluences
{
	GENERATED_USTRUCT_BODY()

    UPROPERTY()
	TArray<FPoseAssetInfluence> Influences;
};

USTRUCT()
struct FPoseAssetInfluenceForPose
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 TrackIndex = INDEX_NONE;

	UPROPERTY()
	int32 BoneTransformIndex = INDEX_NONE;
};

/**
 * Intermediary Interface class to be implemented to extract information from a pose asset.
 * This is used for UPoseAsset::GetAnimationPose, for example.
 */
class FPoseAssetExtractionContext
{
public:
	FPoseAssetExtractionContext() {}
	virtual ~FPoseAssetExtractionContext() {}

	/**
	 * The number of pose curves available in this context
	 * @return The number of pose curves available in this context
	 */
	virtual int32 NumPoses() const = 0;

	/**
	 * Retrieves the name of the pose given an index.
	 * @param InPoseIndexInContext The index of the pose to return the name of
	 * @return The name of the pose given the index
	 */
	virtual const FName& GetPoseName(const int32& InPoseIndexInContext) const = 0;

	/**
	 * Retrieves the index of the pose in the pose asset given the index of the pose within the context.
	 * @param InPoseIndexInContext The index of the pose within the context
	 * @return The index of the pose in the pose asset
	 */
	virtual int32 GetPoseIndexInPoseAsset(const int32& InPoseIndexInContext) const = 0;

	/**
	 * Retrieves the weight / value of a pose in the context
	 * @param InPoseIndexInContext The index of the pose in the context
	 * @return The weight / value of a pose in the context
	 */
	virtual float GetPoseWeight(const int32& InPoseIndexInContext) const = 0;

	/**
	 * Retrieves the pose stored in the context as a FPoseCurve
	 * @param InPoseIndexInContext The index of the pose in the context
	 * @return The FPoseCurve representing the stored pose in the context
	 */
	virtual FPoseCurve GetPoseCurve(const int32& InPoseIndexInContext) const
	{
		return FPoseCurve(
			GetPoseIndexInPoseAsset(InPoseIndexInContext),
			GetPoseName(InPoseIndexInContext),
			GetPoseWeight(InPoseIndexInContext)
		);
	}

	/**
	 * Returns the number of curves stored in the context
	 * @return The number of curves stored in the context
	 */
	virtual int32 NumCurves() const = 0;

	/**
	 * Blends an existing blended curve onto this context given a weight
	 * @param InCurve The curve to blend onto this context
	 * @param InWeight The weight to use when blending
	 */
	virtual void BlendPoseCurve(const FBlendedCurve& InCurve, float InWeight) = 0;

	/**
	 * Returns the target skeleton this context is using
	 * @param bEvenIfUnreachable If true the skeleton will be returned using a most expensive lookup if the cached weakptr is invalid
	 * @return The target skeleton this context is using
	 */
	virtual const USkeleton* GetTargetSkeleton(bool bEvenIfUnreachable = false) const = 0;

	/**
	 * Returns true if a given bone is required to be computed.
	 * @param InOutputBoneIndex The bone index within the context
	 * @return True if a given bone is required to be computed.
	 */
	virtual bool IsOutputBoneRequired(const int32& InOutputBoneIndex) const = 0;

	/**
	 * Returns the bone index within the output pose of this context given the skeleton pose bone index
	 * @param InSkeletonBoneIndex The index of the bone in the skeleton's pose
	 * @return The bone index within the output pose
	 */
	virtual int32 GetOutputPoseBoneIndexFromSkeletonPoseBoneIndex(const int32& InSkeletonBoneIndex) const = 0;

	/**
	 * Returns the curve filter to apply when blending curves (or nullptr)
	 * @return The curve filter to apply when blending curves (or nullptr)
	 */
	virtual const UE::Anim::FCurveFilter* GetCurveFilter() const = 0;

	/**
	 * Resets the output pose to additive identity (with zero scale)
	 */
	virtual void ResetPoseToAdditiveIdentity() = 0;

	/**
	 * Resets the output pose to the reference pose
	 */
	virtual void ResetPoseToRefPose() = 0;

	/**
	 * Returns the transform within the output pose given a bone index
	 * @param InOutputBoneIndex The index of the bone within this context
	 * @return The transform within the output pose given a bone index
	 */
	virtual FTransform& GetOutputTransform(const int32& InOutputBoneIndex) = 0;

	/**
	 * Returns the context to use for FAnimationRuntime::RetargetBoneTransform
	 * @return The context to use for FAnimationRuntime::RetargetBoneTransform
	 */
	virtual const FAnimationRuntimeRetargetBoneTransformContext* GetRetargetBoneTransformContext() const = 0;
};

/**
* Pose data container
* 
* Contains animation and curve for all poses
*/
USTRUCT()
struct FPoseDataContainer
{
	GENERATED_USTRUCT_BODY()

public:
	/** For StructOpsTypeTraits */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);
	
private:
#if WITH_EDITORONLY_DATA
	// pose names - horizontal data
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TArray<FSmartName> PoseNames_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// pose names - horizontal data
	UPROPERTY()
	TArray<FName> PoseFNames;

	// Sorted curve name indices
	TArray<int32> SortedCurveIndices;
	
	// this is list of tracks - vertical data
	UPROPERTY()
	TArray<FName>							Tracks;

	// cache containting the skeleton indices for FName in Tracks array
	UPROPERTY(transient)
	TArray<int32>						TrackBoneIndices;

	UPROPERTY()
	TArray<FPoseAssetInfluences>		TrackPoseInfluenceIndices;
	
	// this is list of poses
	UPROPERTY()
	TArray<FPoseData>						Poses;
	
	
	// curve meta data # of Curve UIDs should match with Poses.CurveValues.Num
	UPROPERTY()
	TArray<FAnimCurveBase>					Curves;

	ENGINE_API void Reset();

	ENGINE_API FPoseData* FindPoseData(FName PoseName);
	ENGINE_API FPoseData* FindOrAddPoseData(FName PoseName);

	int32 GetNumPoses() const { return Poses.Num();  }
	bool Contains(FName PoseName) const { return PoseFNames.Contains(PoseName); }

	bool IsValid() const { return PoseFNames.Num() == Poses.Num() && Tracks.Num() == TrackBoneIndices.Num(); }
	ENGINE_API void GetPoseCurve(const FPoseData* PoseData, FBlendedCurve& OutCurve) const;
	ENGINE_API void BlendPoseCurve(const FPoseData* PoseData, FBlendedCurve& OutCurve, float Weight) const;
	ENGINE_API void BlendPoseCurve(const FPoseData* PoseData, FPoseAssetExtractionContext* InOutContext, float Weight) const;

	// we have to delete tracks if skeleton has modified
	// usually this may not be issue since once cooked, it should match
	ENGINE_API void DeleteTrack(int32 TrackIndex);
	
	// get default transform - it considers for retarget source if exists
	ENGINE_API FTransform GetDefaultTransform(const FName& InTrackName, USkeleton* InSkeleton, const TArray<FTransform>& RefPose) const;
	ENGINE_API FTransform GetDefaultTransform(int32 SkeletonIndex, const TArray<FTransform>& RefPose) const;

#if WITH_EDITOR
	ENGINE_API void AddOrUpdatePose(const FName& InPoseName, const TArray<FTransform>& InlocalSpacePose, const TArray<float>& InCurveData);
	ENGINE_API void RenamePose(FName OldPoseName, FName NewPoseName);
	ENGINE_API int32 DeletePose(FName PoseName);
	ENGINE_API bool DeleteCurve(FName CurveName);
	ENGINE_API bool InsertTrack(const FName& InTrackName, USkeleton* InSkeleton, const TArray<FTransform>& RefPose);
	
	ENGINE_API bool FillUpSkeletonPose(FPoseData* PoseData, const USkeleton* InSkeleton);
	ENGINE_API void RetrieveSourcePoseFromExistingPose(bool bAdditive, int32 InBasePoseIndex, const TArray<FTransform>& InBasePose, const TArray<float>& InBaseCurve);

	// editor features for full pose <-> additive pose
	ENGINE_API void ConvertToFullPose(USkeleton* InSkeleton, const TArray<FTransform>& RefPose);
	ENGINE_API void ConvertToAdditivePose(const TArray<FTransform>& InBasePose, const TArray<float>& InBaseCurve);
#endif // WITH_EDITOR

	ENGINE_API void RebuildCurveIndexTable();
	
	friend class UPoseAsset;
};

template<>
struct TStructOpsTypeTraits<FPoseDataContainer> : public TStructOpsTypeTraitsBase2<FPoseDataContainer>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

/**
 * Pose Asset that can be blended by weight of curves 
 */
UCLASS(MinimalAPI, BlueprintType)
class UPoseAsset : public UAnimationAsset
{
	GENERATED_UCLASS_BODY()

private:
	/** Animation Pose Data*/
	UPROPERTY()
	struct FPoseDataContainer PoseContainer;

	/** Whether or not Additive Pose or not - these are property that needs post process, so */
	UPROPERTY(Category = Additive, EditAnywhere)
	bool bAdditivePose;

	/** if -1, use ref pose */
	UPROPERTY()
	int32 BasePoseIndex;

public: 
	/** Base pose to use when retargeting */
	UPROPERTY(Category=Animation, EditAnywhere)
	FName RetargetSource;

#if WITH_EDITORONLY_DATA
	/** If RetargetSource is set to Default (None), this is asset for the base pose to use when retargeting. Transform data will be saved in RetargetSourceAssetReferencePose. */
	UE_DEPRECATED(5.5, "Direct access to RetargetSourceAsset has been deprecated. Please use members GetRetargetSourceAsset & SetRetargetSourceAsset instead.")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category=Animation, meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TSoftObjectPtr<USkeletalMesh> RetargetSourceAsset;
#endif

	/** When using RetargetSourceAsset, use the post stored here */
	UPROPERTY()
	TArray<FTransform> RetargetSourceAssetReferencePose;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Category=Source, EditAnywhere)
	TObjectPtr<UAnimSequence> SourceAnimation;

	/** GUID cached when the contained poses were last updated according to SourceAnimation - used to keep track of out-of-date/sync data*/ 
	UPROPERTY()	
	FGuid SourceAnimationRawDataGUID;
#endif // WITH_EDITORONLY_DATA

	/**
	* Get Animation Pose from one pose of PoseIndex and with PoseWeight
	* This returns OutPose and OutCurve of one pose of PoseIndex with PoseWeight
	*
	* @param	OutPose				Pose object to fill
	* @param	InOutCurve			Curves to fill
	* @param	PoseIndex			Index of Pose
	* @param	PoseWeight			Weight of pose
	*/
	UE_DEPRECATED(4.26, "Use GetAnimationPose with other signature")
	ENGINE_API bool GetAnimationPose(struct FCompactPose& OutPose, FBlendedCurve& OutCurve, const FAnimExtractContext& ExtractionContext) const;
	ENGINE_API bool GetAnimationPose(struct FAnimationPoseData& OutAnimationPoseData, const FAnimExtractContext& ExtractionContext, const bool bIgnoreSourceAnimationCurves = false, const bool bAllowNegativeWeights = false) const;
	ENGINE_API bool GetAnimationPose(FPoseAssetExtractionContext* InOutContext, const bool bIgnoreSourceAnimationCurves = false, const bool bAllowNegativeWeights = false) const;

	UE_DEPRECATED(4.26, "Use GetBaseAnimationPose with other signature")
	ENGINE_API void GetBaseAnimationPose(struct FCompactPose& OutPose, FBlendedCurve& OutCurve) const;
	ENGINE_API void GetBaseAnimationPose(struct FAnimationPoseData& OutAnimationPoseData) const;

	virtual bool HasRootMotion() const { return false; }
	virtual bool IsValidAdditive() const { return bAdditivePose; }

	// this is utility function that just cares by names to be used by live link
	// this isn't fast. Use it at your caution
	ENGINE_API void GetAnimationCurveOnly(TArray<FName>& InCurveNames, TArray<float>& InCurveValues, TArray<FName>& OutCurveNames, TArray<float>& OutCurveValues) const;

	//Begin UObject Interface
	virtual void PostLoad() override;
	virtual bool IsPostLoadThreadSafe() const override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
//End UObject Interface

public:
	ENGINE_API int32 GetNumPoses() const;
	ENGINE_API int32 GetNumCurves() const;
	ENGINE_API int32 GetNumTracks() const;

	ENGINE_API const TArray<FName>& GetPoseFNames() const;
	ENGINE_API const TArray<FName>& GetTrackNames() const;

	ENGINE_API const TArray<FName> GetCurveFNames() const;
	ENGINE_API const TArray<FAnimCurveBase>& GetCurveData() const;
	ENGINE_API const TArray<float>& GetCurveValues(const int32 PoseIndex) const;

#if WITH_EDITOR
	// Return full / absolute curve values. Will do conversion if PoseAsset is Additive. 
	ENGINE_API const TArray<float>& GetFullCurveValues(const int32 PoseIndex) const;
#endif
	// Return relative / additive curve values. Will give access to raw data and won't do conversion.
	// This is the same as GetCurveValues
	ENGINE_API const TArray<float>& GetRawCurveValues(const int32 PoseIndex) const;

	/** Find index of a track with a given bone name. Returns INDEX_NONE if not found. */
	ENGINE_API const int32 GetTrackIndexByName(const FName& InTrackName) const;

	/** 
	 *	Return value of a curve for a particular pose 
	 *	@return	Returns true if OutValue is valid, false if not
	 */
	ENGINE_API bool GetCurveValue(const int32 PoseIndex, const int32 CurveIndex, float& OutValue) const;

	ENGINE_API bool ContainsPose(const FName& InPoseName) const;

#if WITH_EDITOR
	/** Renames a specific pose */
	UFUNCTION(BlueprintCallable, Category=PoseAsset)
	void RenamePose(const FName& OriginalPoseName, const FName& NewPoseName);
	
	/** Returns the name of all contained poses */
	UFUNCTION(BlueprintPure, Category=PoseAsset)
	void GetPoseNames(TArray<FName>& PoseNames) const;

	/** Returns base pose name, only valid when additive, NAME_None indicates reference pose */
	UFUNCTION(BlueprintPure, Category=PoseAsset)
	FName GetBasePoseName() const;

	/** Set base pose index by name, NAME_None indicates reference pose - returns true if set successfully */
	UFUNCTION(BlueprintCallable, Category=PoseAsset)
    bool SetBasePoseName(const FName& NewBasePoseName);

	ENGINE_API FName AddPoseWithUniqueName(const USkeletalMeshComponent* MeshComponent);

	ENGINE_API void AddOrUpdatePose(const FName& PoseName, const USkeletalMeshComponent* MeshComponent, bool bUpdateCurves = true);

	ENGINE_API void AddReferencePose(const FName& PoseName, const FReferenceSkeleton& ReferenceSkeleton);

	ENGINE_API void CreatePoseFromAnimation(class UAnimSequence* AnimSequence, const TArray<FName>* InPoseNames = nullptr);

	/** Contained poses are re-generated from the provided Animation Sequence*/
	UFUNCTION(BlueprintCallable, Category=PoseAsset)
	ENGINE_API void UpdatePoseFromAnimation(class UAnimSequence* AnimSequence);

	// Begin AnimationAsset interface
	virtual bool GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets, bool bRecursive = true) override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& ReplacementMap) override;
	// End AnimationAsset interface

	ENGINE_API bool ModifyPoseName(FName OldPoseName, FName NewPoseName);

	// Rename poses or curves using the names supplied
	ENGINE_API void RenamePoseOrCurveName(const FName& InOriginalName, const FName& InNewName);

	// Remove poses or curves using the names supplied
	ENGINE_API void RemovePoseOrCurveNames(const TArray<FName>& InNamesToRemove);

	// editor interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// Return full (local space, non additive) pose. Will do conversion if PoseAsset is Additive. 
	ENGINE_API bool GetFullPose(int32 PoseIndex, TArray<FTransform>& OutTransforms) const;
	ENGINE_API const TArray<FTransform>& GetFullPose(int32 PoseIndex) const;
#endif

	// Return local (local space, additive) pose. Will give access to raw data and won't do conversion 
	ENGINE_API bool GetRawPose(int32 PoseIndex, TArray<FTransform>& OutTransforms) const;
	ENGINE_API const TArray<FTransform>& GetRawPose(int32 PoseIndex) const;

#if WITH_EDITOR
	// util to return transform of a bone from the pose asset in component space, by walking up tracks in pose asset */
	ENGINE_API FTransform GetComponentSpaceTransform(FName BoneName, const TArray<FTransform>& LocalTransforms) const;

	// util to return transform of a bone from the pose asset in local space */
	ENGINE_API const FTransform& GetLocalSpaceTransform(FName BoneName, int32 PoseIndex = 0) const;
	ENGINE_API const FTransform& GetLocalSpaceTransform(int32 TrackIndex, int32 PoseIndex = 0) const;

	ENGINE_API int32 DeletePoses(TArray<FName> PoseNamesToDelete);
	ENGINE_API int32 DeleteCurves(TArray<FName> CurveNamesToDelete);
	ENGINE_API bool ConvertSpace(bool bNewAdditivePose, int32 NewBasePoseInde);
	const FName GetPoseNameByIndex(int32 InBasePoseIndex) const { return PoseContainer.PoseFNames.IsValidIndex(InBasePoseIndex) ? PoseContainer.PoseFNames[InBasePoseIndex] : NAME_None; }
#endif // WITH_EDITOR

	int32 GetBasePoseIndex() const { return BasePoseIndex;  }
	ENGINE_API const int32 GetPoseIndexByName(const FName& InBasePoseName) const;
	ENGINE_API const int32 GetCurveIndexByName(const FName& InCurveName) const;

	ENGINE_API bool GetBasePoseTransform(TArray<FTransform>& OutBasePose, TArray<float>& OutBaseCurve) const;
	ENGINE_API bool GetTrackInfluencesForPose(const int32 PoseIndex, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const;
	ENGINE_API bool GetTrackInfluencesForPoses(const TArrayView<const FPoseCurve>& InPoses, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const;
	ENGINE_API bool ConvertToFullPose(const int32 PoseIndex, TArray<FTransform>& OutFullPose, TArray<FPoseAssetInfluenceForPose>& OutInfluences, TArray<float>& OutFullCurves) const;
	ENGINE_API bool ConvertToFullPose(const int32 PoseIndex, TArray<FTransform>& OutFullPose, TArray<FPoseAssetInfluenceForPose>& OutInfluences) const;
	ENGINE_API bool ConvertToFullPose(const int32 PoseIndex, TArray<float>& OutFullCurves) const;

#if WITH_EDITOR
private: 
	DECLARE_MULTICAST_DELEGATE(FOnPoseListChangedMulticaster)
	FOnPoseListChangedMulticaster OnPoseListChanged;

public:
	typedef FOnPoseListChangedMulticaster::FDelegate FOnPoseListChanged;

	/** Registers a delegate to be called after the preview animation has been changed */
	FDelegateHandle RegisterOnPoseListChanged(const FOnPoseListChanged& Delegate)
	{
		return OnPoseListChanged.Add(Delegate);
	}
	/** Unregisters a delegate to be called after the preview animation has been changed */
	void UnregisterOnPoseListChanged(FDelegateHandle Handle)
	{
		OnPoseListChanged.Remove(Handle);
	}

	ENGINE_API static FName GetUniquePoseName(UPoseAsset* PoseAsset);
	ENGINE_API FGuid GetSourceAnimationGuid() const;
	
protected:
	virtual void RemapTracksToNewSkeleton(USkeleton* NewSkeleton, bool bConvertSpaces) override;
private: 
	// this will do multiple things, it will add tracks and make sure it fix up all poses with it
	// use same as retarget source system we have for animation
	void CombineTracks(const TArray<FName>& NewTracks);

	void ConvertToFullPose();
	void ConvertToAdditivePose(int32 NewBasePoseIndex);
	void Reinitialize();

	// After any update to SourceLocalPoses, this does update runtime data
	void AddOrUpdatePose(const FName& PoseName, const TArray<FName>& TrackNames, const TArray<FTransform>& LocalTransform, const TArray<float>& CurveValues);
	void PostProcessData();
	void BreakAnimationSequenceGUIDComparison();
#endif // WITH_EDITOR	

private:
	void UpdateTrackBoneIndices();
	bool RemoveInvalidTracks();

public:
#if WITH_EDITOR
	// Assigns the passed skeletal mesh to the retarget source
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetRetargetSourceAsset(USkeletalMesh* InRetargetSourceAsset);

	// Resets the retarget source asset
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void ClearRetargetSourceAsset();

	// Returns the retarget source asset soft object pointer.
	UFUNCTION(BlueprintPure, Category = "Animation")
	ENGINE_API const TSoftObjectPtr<USkeletalMesh>& GetRetargetSourceAsset() const;

	// Update the retarget data pose from the source, if it exist, else clears the retarget data pose saved in RetargetSourceAssetReferencePose.
	// Warning : This function calls LoadSynchronous at the retarget source asset soft object pointer, so it can not be used at PostLoad
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void UpdateRetargetSourceAssetData();
#endif

	ENGINE_API const TArray<FTransform>& GetRetargetTransforms() const;
	ENGINE_API FName GetRetargetTransformsSourceName() const;

	friend class FPoseAssetDetails;
};
