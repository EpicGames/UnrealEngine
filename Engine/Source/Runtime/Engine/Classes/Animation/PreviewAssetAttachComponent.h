// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * This is a component used by USkeleton and USkeletalMesh 
 * to deal with attaching assets to sockets or bones
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "PreviewAssetAttachComponent.generated.h"

/** Preview items that are attached to the skeleton **/
USTRUCT()
struct FPreviewAttachedObjectPair
{
	GENERATED_USTRUCT_BODY()

private:
	/** the object to be attached */
	UPROPERTY()
	TSoftObjectPtr<class UObject> AttachedObject;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UObject> Object_DEPRECATED;
#endif

public:

	FPreviewAttachedObjectPair()
#if WITH_EDITORONLY_DATA
		: Object_DEPRECATED(NULL)
#endif
	{}

	/** The name of the attach point of the Object (for example a bone or socket name) */
	UPROPERTY()
	FName AttachedTo;

	void SaveAttachedObjectFromDeprecatedProperty()
	{
#if WITH_EDITORONLY_DATA
		if (Object_DEPRECATED)
		{
			AttachedObject = Object_DEPRECATED;
			Object_DEPRECATED = NULL;
		}
#endif
	}

	UObject* GetAttachedObject() const
	{
		return AttachedObject.LoadSynchronous();
	}

	void SetAttachedObject(UObject* InObject)
	{
		AttachedObject = InObject;
	}
};

// Iterators
typedef TIndexedContainerIterator<      TArray<FPreviewAttachedObjectPair>,       FPreviewAttachedObjectPair, int32> TIterator;
typedef TIndexedContainerIterator<const TArray<FPreviewAttachedObjectPair>, const FPreviewAttachedObjectPair, int32> TConstIterator;

/** Component which deals with attaching assets */
USTRUCT()
struct FPreviewAssetAttachContainer
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	 * Adds the given Name/Object to the PreviewAttachedObjects list. Allows us to recreate the attached objects later
	 *
	 * @param	AttachObject		The object that is being attached
	 * @param	AttachPointName		The place where the object is attached (bone or socket)
	 */
	ENGINE_API void AddAttachedObject( UObject* AttachObject, FName AttachPointName );

	/**
	* Adds the given Name/Object uniquely to the PreviewAttachedObjects list. Allows us to recreate the attached objects later
	*
	* @param	AttachObject		The object that is being attached
	* @param	AttachPointName		The place where the object is attached (bone or socket)
	*/
	ENGINE_API void AddUniqueAttachedObject(UObject* AttachObject, FName AttachPointName);

	/**
	 * Remove the given object from the attached list
	 *
	 * @param	ObjectToRemove		The object that is being removed
	 */
	ENGINE_API void RemoveAttachedObject( UObject* ObjectToRemove, FName AttachName );

	/**
	 * Grab the asset (if any) attached at the place given
	 *
	 * @param	AttachName		The name of the attach point (Bone name or socket name) to look for an asset
	 * @return					UObject or NULL if nothing is attached at that location.
	 */
	ENGINE_API UObject* GetAttachedObjectByAttachName( FName AttachName ) const;

	/**
	 * Clears all the preview attached objects
	 */
	ENGINE_API void ClearAllAttachedObjects();

	/**
	 * Returns the number of attached objects
	 */
	ENGINE_API int32 Num() const;

	/** 
	 * operator [] - pipes through to AttachedObjects
	 */
	ENGINE_API FPreviewAttachedObjectPair& operator[]( int32 i );

	/**
	* operator [] - pipes through to AttachedObjects
	*/
	ENGINE_API const FPreviewAttachedObjectPair& operator[](int32 i) const;

	/**
	 * Const iterator creator for AttachedObjects
	 */
	ENGINE_API TConstIterator CreateConstIterator() const;

	/**
	 * Iterator creator for AttachedObjects
	 */
	ENGINE_API TIterator CreateIterator();

	/** 
	 * RemoveAtSwap passthrough
	 */
	ENGINE_API void RemoveAtSwap( int32 Index, EAllowShrinking AllowShrinking = EAllowShrinking::Default);
	ENGINE_API void RemoveAtSwap( int32 Index, int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default);

	/**
	 * Helper function to fix up attached objects after property deprecation
	 */
	ENGINE_API void SaveAttachedObjectsFromDeprecatedProperties();

	/**
	 * Helper function to remove invalid attached object references
	 */
	ENGINE_API int32 ValidatePreviewAttachedObjects();

private:
	UPROPERTY()
	TArray<FPreviewAttachedObjectPair> AttachedObjects;
};
