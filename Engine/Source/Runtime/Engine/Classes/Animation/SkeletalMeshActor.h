// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "SkeletalMeshActor.generated.h"

class UAnimMontage;
class UAnimSequence;

/**
 * SkeletalMeshActor is an instance of a USkeletalMesh in the world.
 * Skeletal meshes are deformable meshes that can be animated and change their geometry at run-time.
 * Skeletal meshes dragged into the level from the Content Browser are automatically converted to StaticMeshActors.
 * 
 * @see https://docs.unrealengine.com/latest/INT/Engine/Content/Types/SkeletalMeshes/
 * @see USkeletalMesh
 */
UCLASS(ClassGroup=ISkeletalMeshes, Blueprintable, ComponentWrapperClass, ConversionRoot, meta=(ChildCanTick), MinimalAPI)
class ASkeletalMeshActor : public AActor
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

	/** Whether or not this actor should respond to anim notifies - CURRENTLY ONLY AFFECTS PlayParticleEffect NOTIFIES**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay)
	uint32 bShouldDoAnimNotifies:1;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint32 bWakeOnLevelStart_DEPRECATED:1;
#endif

private:
	UPROPERTY(Category = SkeletalMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Components|SkeletalMesh,Animation,Physics", AllowPrivateAccess = "true"))
	TObjectPtr<class USkeletalMeshComponent> SkeletalMeshComponent;
public:

	/** Used to replicate mesh to clients */
	UPROPERTY(replicatedUsing=OnRep_ReplicatedMesh, transient)
	TObjectPtr<class USkeletalMesh> ReplicatedMesh;

	/** Used to replicate physics asset to clients */
	UPROPERTY(replicatedUsing=OnRep_ReplicatedPhysAsset, transient)
	TObjectPtr<class UPhysicsAsset> ReplicatedPhysAsset;

	/** used to replicate the material in index 0 */
	UPROPERTY(replicatedUsing=OnRep_ReplicatedMaterial0)
	TObjectPtr<class UMaterialInterface> ReplicatedMaterial0;

	UPROPERTY(replicatedUsing=OnRep_ReplicatedMaterial1)
	TObjectPtr<class UMaterialInterface> ReplicatedMaterial1;

	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedMesh();

	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedPhysAsset();

	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedMaterial0();

	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedMaterial1();


	//~ Begin UObject Interface
protected:
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
public:
	//~ End UObject Interface

	//~ Begin AActor Interface
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const override;
	ENGINE_API virtual void EditorReplacedActor(AActor* OldActor) override;
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
#endif
	ENGINE_API virtual void PostInitializeComponents() override;
	//~ End AActor Interface

private:
	// currently actively playing montage
	TMap<FName, TWeakObjectPtr<class UAnimMontage>> CurrentlyPlayingMontages;

public:
	/** Returns SkeletalMeshComponent subobject **/
	class USkeletalMeshComponent* GetSkeletalMeshComponent() const { return SkeletalMeshComponent; }
};



