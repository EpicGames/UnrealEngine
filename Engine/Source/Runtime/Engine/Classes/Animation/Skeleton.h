// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * This is the definition for a skeleton, used to animate USkeletalMesh
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "HAL/CriticalSection.h"
#include "Misc/Guid.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "ReferenceSkeleton.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "Animation/SmartName.h"
#include "Engine/AssetUserData.h"
#include "HAL/CriticalSection.h"  // jira SOL-6812: Remove this (duplicate include).
#include "Interfaces/Interface_AssetUserData.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/ObjectKey.h"
#include "Skeleton.generated.h"

class UAnimCurveMetaData;
class UAnimSequence;
class UBlendProfile;
class URig;
class USkeletalMeshSocket;
class USkinnedAsset;
class FPackageReloadedEvent;
struct FAssetData;
enum class EPackageReloadPhase : uint8;
class USkeleton;
typedef SmartName::UID_Type SkeletonAnimCurveUID;
class USkeleton;
struct FSkeletonRemapping;
class FEditableSkeleton;

// Delegate used to control global skeleton compatibility
DECLARE_DELEGATE_RetVal(bool, FAreAllSkeletonsCompatible);

/** This is a mapping table between bone in a particular skeletal mesh and bone of this skeleton set. */
USTRUCT()
struct FSkeletonToMeshLinkup
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Mapping table. Size must be same as size of bone tree (not Mesh Ref Pose). 
	 * No index should be more than the number of bones in this skeleton
	 * -1 indicates no match for this bone - will be ignored.
	 */
	UPROPERTY()
	TArray<int32> SkeletonToMeshTable;

	/** 
	 * Mapping table. Size must be same as size of ref pose (not bone tree). 
	 * No index should be more than the number of bones in this skeletalmesh
	 * -1 indicates no match for this bone - will be ignored.
	 */
	UPROPERTY()
	TArray<int32> MeshToSkeletonTable;

};

/** Bone translation retargeting mode. */
UENUM()
namespace EBoneTranslationRetargetingMode
{
	enum Type : int
	{
		/** Use translation from animation data. */
		Animation,
		/** Use fixed translation from Skeleton. */
		Skeleton,
		/** Use Translation from animation, but scale length by Skeleton's proportions. */
		AnimationScaled,
		/** Use Translation from animation, but also play the difference from retargeting pose as an additive. */
		AnimationRelative,
		/** Apply delta orientation and scale from ref pose */
		OrientAndScale,
	};
}

/** Max error allowed when considering bone translations for 'Orient And Scale' retargeting. */
#define BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION (0.001f) 

/** Each Bone node in BoneTree */
USTRUCT()
struct FBoneNode
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	/** Name of bone, this is the search criteria to match with mesh bone. This will be NAME_None if deleted. */
	UPROPERTY()
	FName Name_DEPRECATED;

	/** Parent Index. -1 if not used. The root has 0 as its parent. Do not delete the element but set this to -1. If it is revived by other reason, fix up this link. */
	UPROPERTY()
	int32 ParentIndex_DEPRECATED;
#endif

	/** Retargeting Mode for Translation Component. */
	UPROPERTY(EditAnywhere, Category=BoneNode)
	TEnumAsByte<EBoneTranslationRetargetingMode::Type> TranslationRetargetingMode;

	FBoneNode()
		:
#if WITH_EDITORONLY_DATA
		ParentIndex_DEPRECATED(INDEX_NONE),
#endif
		TranslationRetargetingMode(EBoneTranslationRetargetingMode::Animation)
	{
	}

	FBoneNode(FName InBoneName, int32 InParentIndex)
		:
#if WITH_EDITORONLY_DATA
		Name_DEPRECATED(InBoneName),
		ParentIndex_DEPRECATED(InParentIndex),
#endif
		TranslationRetargetingMode(EBoneTranslationRetargetingMode::Animation)
	{
	}
};

/** This is a mapping table between bone in a particular skeletal mesh and bone of this skeleton set. */
USTRUCT()
struct FReferencePose
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName	PoseName;

	UPROPERTY()
	TArray<FTransform>	ReferencePose;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSoftObjectPtr<USkeletalMesh> SourceReferenceMesh;
#endif

	/**
	 * Serializes the bones
	 *
	 * @param Ar - The archive to serialize into.
	 * @param P - The FReferencePose to serialize
	 * @param Outer - The object containing this instance. Used to determine if we're loading cooked data.
	 */
	friend void SerializeReferencePose(FArchive& Ar, FReferencePose& P, UObject* Outer);
};

USTRUCT()
struct FBoneReductionSetting
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FName> BonesToRemove;

	bool Add(FName BoneName)
	{
		if ( BoneName!=NAME_None && !BonesToRemove.Contains(BoneName) )
		{
			BonesToRemove.Add(BoneName);
			return true;
		}

		return false;
	}

	void Remove(FName BoneName)
	{
		BonesToRemove.Remove(BoneName);
	}

	bool Contains(FName BoneName)
	{
		return (BonesToRemove.Contains(BoneName));
	}
};

USTRUCT()
struct FNameMapping
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName NodeName;

	UPROPERTY()
	FName BoneName;

	FNameMapping()
		: NodeName(NAME_None)
		, BoneName(NAME_None)
	{
	}

	FNameMapping(FName InNodeName)
		: NodeName(InNodeName)
		, BoneName(NAME_None)
	{
	}

	FNameMapping(FName InNodeName, FName InBoneName)
		: NodeName(InNodeName)
		, BoneName(InBoneName)
	{
	}
};

USTRUCT()
struct FAnimSlotGroup
{
	GENERATED_USTRUCT_BODY()

public:
	static ENGINE_API const FName DefaultGroupName;
	static ENGINE_API const FName DefaultSlotName;

	UPROPERTY()
	FName GroupName;

	UPROPERTY()
	TArray<FName> SlotNames;

	FAnimSlotGroup()
		: GroupName(DefaultGroupName)
	{
	}

	FAnimSlotGroup(FName InGroupName)
		: GroupName(InGroupName)
	{
	}
};

namespace VirtualBoneNameHelpers
{
	extern ENGINE_API const FString VirtualBonePrefix;

	ENGINE_API FString AddVirtualBonePrefix(const FString& InName);
	ENGINE_API FName RemoveVirtualBonePrefix(const FString& InName);
	ENGINE_API bool CheckVirtualBonePrefix(const FString& InName);
}

USTRUCT()
struct FVirtualBone
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY()
	FName SourceBoneName;

	UPROPERTY()
	FName TargetBoneName;

	UPROPERTY()
	FName VirtualBoneName;

	FVirtualBone() {}

	FVirtualBone(FName InSource, FName InTarget, FName InVirtual)
		: SourceBoneName(InSource)
		, TargetBoneName(InTarget)
		, VirtualBoneName(InVirtual)
	{
	}

	FVirtualBone(FName InSource, FName InTarget)
		: SourceBoneName(InSource)
		, TargetBoneName(InTarget)
	{
		FString VBNameString = VirtualBoneNameHelpers::AddVirtualBonePrefix(SourceBoneName.ToString() + TEXT("_") + TargetBoneName.ToString());
		VirtualBoneName = FName(*VBNameString);
	}
};


/**
 *	USkeleton : that links between mesh and animation
 *		- Bone hierarchy for animations
 *		- Bone/track linkup between mesh and animation
 *		- Retargetting related
 */
UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class USkeleton : public UObject, public IInterface_AssetUserData, public IInterface_PreviewMeshProvider
{
	friend class UAnimationBlueprintLibrary;
	friend class FSkeletonDetails;

	GENERATED_UCLASS_BODY()

protected:
	/** Skeleton bone tree - each contains name and parent index**/
	UPROPERTY(VisibleAnywhere, Category=Skeleton)
	TArray<struct FBoneNode> BoneTree;

#if WITH_EDITORONLY_DATA
	/** Reference skeleton poses in local space */
	UPROPERTY()
	TArray<FTransform> RefLocalPoses_DEPRECATED;

	// Preview axis to consider as "forward" for the skeleton. Only used for preview purposes.
	UPROPERTY(EditAnywhere, Category = Preview)
	TEnumAsByte<EAxis::Type> PreviewForwardAxis;
#endif

	/** Reference Skeleton */
	FReferenceSkeleton ReferenceSkeleton;

	/** Guid for skeleton */
	FGuid Guid;

	/** Guid for virtual bones.
	 *  Separate so that we don't have to dirty the original guid when only changing virtual bones */
	UPROPERTY()
	FGuid VirtualBoneGuid;

	/** Conversion function. Remove when VER_UE4_REFERENCE_SKELETON_REFACTOR is removed. */
	void ConvertToFReferenceSkeleton();

	/**
	*  Array of this skeletons virtual bones. These are new bones are links between two existing bones
	*  and are baked into all the skeletons animations
	*/
	UPROPERTY()
	TArray<FVirtualBone> VirtualBones;

	/**
	 * The list of compatible skeletons. This skeleton will be able to use animation data originating from skeletons within this array, such as animation sequences.
	 * This property is not bi-directional.
	 * 
	 * This is an array of TSoftObjectPtr in order to prevent all skeletons to be loaded, as we only want to load things on demand.
	 * As this is EditAnywhere and an array of TSoftObjectPtr, checking validity of pointers is needed.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CompatibleSkeletons)
	TArray<TSoftObjectPtr<USkeleton>> CompatibleSkeletons;

	/**
	 * Should we use the per bone translational retarget mode from the source (compatible) skeleton's instead of from this skeleton? On default this is disabled.
	 * Enabling this would allow you to have one shared set of animations. You would configure the retarget settings on the animation skeleton.
	 * Then every character that plays animations from this source skeleton will use the translational retarget settings from the source skeleton, which saves you from
	 * having to configure the retarget modes for every bone in every character as they can be setup just once now on the animation skeleton.
	 */
	UPROPERTY(EditAnywhere, Category = CompatibleSkeletons)
	bool bUseRetargetModesFromCompatibleSkeleton = false;

public:
	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	EAxis::Type GetPreviewForwardAxis() const { return PreviewForwardAxis; }
#endif
	ENGINE_API virtual void BeginDestroy() override;

	/** Accessor to Reference Skeleton to make data read only */
	const FReferenceSkeleton& GetReferenceSkeleton() const
	{
		return ReferenceSkeleton;
	}

	/** Accessor for the array of virtual bones on this skeleton */
	const TArray<FVirtualBone>& GetVirtualBones() const { return VirtualBones; }

	/** 
	 *	Array of named socket locations, set up in editor and used as a shortcut instead of specifying 
	 *	everything explicitly to AttachComponent in the SkeletalMeshComponent.
	 */
	UPROPERTY()
	TArray<TObjectPtr<class USkeletalMeshSocket>> Sockets;

	/** Serializable retarget sources for this skeleton **/
	TMap< FName, FReferencePose > AnimRetargetSources;

	UE_DEPRECATED(5.7, "AnimCurveUID is deprecated and no longer used.")
	typedef SmartName::UID_Type AnimCurveUID;

	ENGINE_API FCurveMetaData* GetCurveMetaData(FName CurveName);
	ENGINE_API const FCurveMetaData* GetCurveMetaData(FName CurveName) const;

	/**
	 * Iterate over all curve metadata entries, calling InFunction on each
	 * @param	InFunction	The function to call
	 **/
	ENGINE_API void ForEachCurveMetaData(TFunctionRef<void(FName, const FCurveMetaData&)> InFunction) const;
	
	/** @return the number of curve metadata entries **/
	ENGINE_API int32 GetNumCurveMetaData() const;

	/**
	 * Adds a curve metadata entry with the specified name
	 * @param	InCurveName			The name of the curve to find
	 * @param   bTransact           If true record a new transaction
	 * @return true if an entry was added, false if an entry already existed
	 */
	ENGINE_API bool AddCurveMetaData(FName CurveName, bool bTransact = true);

	/**
	 * Get an array of all curve metadata names
	 * @param	OutNames		The array to receive the metadata names 
	 */
	ENGINE_API void GetCurveMetaDataNames(TArray<FName>& OutNames) const;

#if WITH_EDITOR
	/**
	 * Renames a curve metadata entry. Metadata is preserved, but assigned to a different curve name.
	 * @param OldName	The name of an existing curve entry
	 * @param NewName	The name to change the entry to
	 * @return			true if the rename was successful (the old name was found and the new name didnt collide with an
	 *					existing entry)
	 */	
	ENGINE_API bool RenameCurveMetaData(FName OldName, FName NewName);

	/**
	 * Removes a curve metadata entry for the specified name.
	 * @param CurveName	The name of the curve to remove the metadata for
	 * @return true if the entry was successfully removed (i.e. it existed)
	 */
	ENGINE_API bool RemoveCurveMetaData(FName CurveName);

	/**
	 * Removes a group of curve metadata entries for the specified names.
	 * @param CurveNames	The names of the curves to remove the metadata for
	 * @return true if any of the entries were successfully removed (i.e. something changed)
	 */
	ENGINE_API bool RemoveCurveMetaData(TArrayView<FName> CurveNames);

	/**
	 * Gets the material flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return true if the flag has been set
	 */
	ENGINE_API bool GetCurveMetaDataMaterial(FName CurveName) const;

	/**
	 * Gets the morph target flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return true if the flag has been set
	 */
	ENGINE_API bool GetCurveMetaDataMorphTarget(FName CurveName) const;

	/**
	 * Set the material flag for a curve's metadata
	 * @param	CurveName			The name of the curve to set
	 * @param	bOverrideMaterial	Whether to set the material flag
	 */
	ENGINE_API void SetCurveMetaDataMaterial(FName CurveName, bool bOverrideMaterial);

	/**
	 * Set the morph target flag for a curve's metadata
	 * @param	CurveName				The name of the curve to set
	 * @param	bOverrideMorphTarget	Whether to set the morph target flag
	 */
	ENGINE_API void SetCurveMetaDataMorphTarget(FName CurveName, bool bOverrideMorphTarget);
#endif

	// this is called when you know both flags - called by post serialize and import
	ENGINE_API void AccumulateCurveMetaData(FName CurveName, bool bMaterialSet, bool bMorphtargetSet);

	ENGINE_API bool AddNewVirtualBone(const FName SourceBoneName, const FName TargetBoneName);

	ENGINE_API bool AddNewVirtualBone(const FName SourceBoneName, const FName TargetBoneName, FName& NewVirtualBoneName);

	ENGINE_API bool AddNewNamedVirtualBone(const FName SourceBoneName, const FName TargetBoneName, const FName NewVirtualBoneName);
	
	ENGINE_API void RemoveVirtualBones(const TArray<FName>& BonesToRemove);

	ENGINE_API void RenameVirtualBone(const FName OriginalBoneName, const FName NewBoneName);
	
	void HandleVirtualBoneChanges();

	// return version of AnimCurveUidVersion
	uint16 GetAnimCurveUidVersion() const;

	const TArray<TSoftObjectPtr<USkeleton>>& GetCompatibleSkeletons() const { return CompatibleSkeletons; }

	ENGINE_API bool GetUseRetargetModesFromCompatibleSkeleton() const;
	ENGINE_API void SetUseRetargetModesFromCompatibleSkeleton(bool bUse);

#if WITH_EDITOR
	// Get existing (seen) sync marker names for this Skeleton
	const TArray<FName>& GetExistingMarkerNames() const { return ExistingMarkerNames; }

	// Register a new sync marker name
	void RegisterMarkerName(FName MarkerName) { ExistingMarkerNames.AddUnique(MarkerName); ExistingMarkerNames.Sort(FNameLexicalLess()); }

	// Remove a sync marker name
	ENGINE_API bool RemoveMarkerName(FName MarkerName);

	// Rename a sync marker name
	ENGINE_API bool RenameMarkerName(FName InOldName, FName InNewName);
#endif

protected:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "SmartNames in Skeleton have been deprecated, use CurveMetaData instead.")
	UPROPERTY()
	FSmartNameContainer SmartNames_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//Cached marker sync marker names (stripped for non editor)
	TArray<FName> ExistingMarkerNames;

private:
	// Refresh skeleton metadata (updates bone indices for linked bone references)
	void RefreshSkeletonMetaData();

	// Returns the UAnimCurveMetaData from this skeleton's AssetUserData, creating one if it doesn't exist
	UAnimCurveMetaData* GetOrCreateCurveMetaDataObject();

public:
	//////////////////////////////////////////////////////////////////////////
	// Blend Profiles

	/** List of blend profiles available in this skeleton */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UBlendProfile>> BlendProfiles;

	/** Get the specified blend profile by name */
	UFUNCTION(BlueprintPure, Category = Skeleton, meta = (BlueprintThreadSafe))
	ENGINE_API UBlendProfile* GetBlendProfile(const FName& InProfileName);

	/** Create a new blend profile with the specified name */
	ENGINE_API UBlendProfile* CreateNewBlendProfile(const FName& InProfileName);

	/** Rename an existing blend profile with the specified name. Returns the pointer if success, nullptr on failure */
	ENGINE_API UBlendProfile* RenameBlendProfile(const FName& InProfileName, const FName& InNewProfileName);

	//////////////////////////////////////////////////////////////////////////

	/************************************************************************/
	/* Slot Groups */
	/************************************************************************/
private:
	// serialized slot groups and slot names.
	UPROPERTY()
	TArray<FAnimSlotGroup> SlotGroups;

	/** SlotName to GroupName TMap, only at runtime, not serialized. **/
	TMap<FName, FName> SlotToGroupNameMap;

	void BuildSlotToGroupMap(bool bInRemoveDuplicates = false);

public:
	ENGINE_API FAnimSlotGroup* FindAnimSlotGroup(const FName& InGroupName);
	ENGINE_API const FAnimSlotGroup* FindAnimSlotGroup(const FName& InGroupName) const;
	ENGINE_API const TArray<FAnimSlotGroup>& GetSlotGroups() const;
	ENGINE_API bool ContainsSlotName(const FName& InSlotName) const;

	/** Register a slot name. Return true if a slot was registered, false if it was already registered. */
	ENGINE_API bool RegisterSlotNode(const FName& InSlotName);

	ENGINE_API void SetSlotGroupName(const FName& InSlotName, const FName& InGroupName);
	/** Returns true if Group is added, false if it already exists */
	ENGINE_API bool AddSlotGroupName(const FName& InNewGroupName);
	ENGINE_API FName GetSlotGroupName(const FName& InSlotName) const;

	// Edits/removes slot group data
	// WARNING: Does not verify that the names aren't used anywhere - if it isn't checked
	// by the caller the names will be recreated when referencing assets load again.
	ENGINE_API void RemoveSlotName(const FName& InSlotName);
	ENGINE_API void RemoveSlotGroup(const FName& InSlotName);
	ENGINE_API void RenameSlotName(const FName& OldName, const FName& NewName);

#if WITH_EDITORONLY_DATA
private:
	/** The default skeletal mesh to use when previewing this skeleton */
	UPROPERTY(duplicatetransient, AssetRegistrySearchable)
	TSoftObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	/** The additional skeletal meshes to use when previewing this skeleton */
	UPROPERTY(duplicatetransient, AssetRegistrySearchable)
	TSoftObjectPtr<class UDataAsset> AdditionalPreviewSkeletalMeshes;

	/** rig property will be saved separately */
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

public:

	/** AnimNotifiers that has been created. Right now there is no delete step for this, but in the future we'll supply delete**/
	UPROPERTY()
	TArray<FName> AnimationNotifies;

	/* Attached assets component for this skeleton */
	UPROPERTY()
	FPreviewAssetAttachContainer PreviewAttachedAssetContainer;
#endif // WITH_EDITORONLY_DATA

private:
	DECLARE_MULTICAST_DELEGATE( FOnRetargetSourceChangedMulticaster )
	FOnRetargetSourceChangedMulticaster OnRetargetSourceChanged;

public:
	typedef FOnRetargetSourceChangedMulticaster::FDelegate FOnRetargetSourceChanged;

	/** Registers a delegate to be called after the preview animation has been changed */
	FDelegateHandle RegisterOnRetargetSourceChanged(const FOnRetargetSourceChanged& Delegate)
	{
		return OnRetargetSourceChanged.Add(Delegate);
	}

	const FGuid GetGuid() const
	{
		return Guid;
	}

	FGuid GetVirtualBoneGuid() const
	{
		return VirtualBoneGuid;
	}

	/** Unregisters a delegate to be called after the preview animation has been changed */
	void UnregisterOnRetargetSourceChanged(FDelegateHandle Handle)
	{
		OnRetargetSourceChanged.Remove(Handle);
	}

	void CallbackRetargetSourceChanged()
	{
		OnRetargetSourceChanged.Broadcast();
	}

	typedef TArray<FBoneNode> FBoneTreeType;

private:
	//Use this Lock everytime you change or access SkinnedAssetLinkupCache member.
	FTransactionallySafeRWLock SkinnedAssetLinkupCacheLock;

	/** Runtime built mapping table between SkinnedAssets and Mesh Linkup Data*/
	TMap<TObjectKey<USkinnedAsset>, TUniquePtr<FSkeletonToMeshLinkup>> SkinnedAssetLinkupCache;

public:
	/** A cached soft object pointer of this skeleton. This is done for performance reasons when searching for compatible skeletons when using IsCompatible(Skeleton). */
	UE_DEPRECATED(5.7, "CachedSoftObjectPtr is no longer used and has been deprecated.")
	TSoftObjectPtr<USkeleton> CachedSoftObjectPtr;

	/** IInterface_PreviewMeshProvider interface */
	virtual USkeletalMesh* GetPreviewMesh(bool bFindIfNotSet = false) override;
	virtual USkeletalMesh* GetPreviewMesh() const override;
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty=true);

#if WITH_EDITORONLY_DATA

	/*
	 * Collect animation notifies that are referenced in all animations that use this skeleton (uses the asset registry).
	 * @param	OutNotifies		All the notifies that were found
	 */
	ENGINE_API void CollectAnimationNotifies(TArray<FName>& OutNotifies) const;

	// Adds a new anim notify to the cached AnimationNotifies array.
	ENGINE_API void AddNewAnimationNotify(FName NewAnimNotifyName);

	// Removes an anim notify from the cached AnimationNotifies array.
	ENGINE_API void RemoveAnimationNotify(FName AnimNotifyName);

	// Renames an anim notify
	ENGINE_API void RenameAnimationNotify(FName OldAnimNotifyName, FName NewAnimNotifyName);

	ENGINE_API USkeletalMesh* GetAssetPreviewMesh(UObject* InAsset);

	/** Find the first compatible mesh for this skeleton */
	ENGINE_API USkeletalMesh* FindCompatibleMesh() const;

	/** Load any additional meshes we may have */
	ENGINE_API void LoadAdditionalPreviewSkeletalMeshes();

	/** Get the additional skeletal meshes we use when previewing this skeleton */
	ENGINE_API UDataAsset* GetAdditionalPreviewSkeletalMeshes() const;

	/** Set the additional skeletal meshes we use when previewing this skeleton */
	ENGINE_API void SetAdditionalPreviewSkeletalMeshes(UDataAsset* InPreviewCollectionAsset);

	/**
	 * Makes sure all attached objects are valid and removes any that aren't.
	 *
	 * @return		NumberOfBrokenAssets
	 */
	ENGINE_API int32 ValidatePreviewAttachedObjects();

	/**
	 * Get List of Child Bones of the ParentBoneIndex
	 *
	 * @param	Parent Bone Index
	 * @param	(out) List of Direct Children
	 */
	ENGINE_API int32 GetChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const;
	
	/**
	 * Check if animation content authored on the supplied skeleton may be played on this skeleton.
	 * Note that animations may not always be correct if they were not authored on this skeleton, and may require
	 * retargeting or other fixup.
	 */	
	ENGINE_API bool IsCompatibleForEditor(const USkeleton* InSkeleton) const;

	/**
	 * Check if this skeleton is compatible with a given other asset, if that is a skeleton.
	 */	
	ENGINE_API bool IsCompatibleForEditor(const FAssetData& AssetData, const TCHAR* InTag = TEXT("Skeleton")) const;

	/**
	 * Check if this skeleton is compatible with another skeleton asset that is identified by the string returned by AssetData(SkeletonPtr).GetExportTextName().
	 */
	ENGINE_API bool IsCompatibleForEditor(const FString& SkeletonAssetString) const;

	/** Wrapper for !IsCompatibleForEditor, used as a convenience function for binding to FOnShouldFilterAsset in asset pickers. */
	ENGINE_API bool ShouldFilterAsset(const FAssetData& InAssetData, const TCHAR* InTag = TEXT("Skeleton")) const;
	
	/** Get all skeleton assets that are compatible with this skeleton (not just the internal list, but also reciprocally and implicitly compatible skeletons) */
	ENGINE_API void GetCompatibleSkeletonAssets(TArray<FAssetData>& OutAssets) const;

	/** Get compatible assets given the asset's class and skeleton tag.*/
	ENGINE_API void GetCompatibleAssets(UClass* AssetClass, const TCHAR* InTag, TArray<FAssetData>& OutAssets) const;
#endif

	DECLARE_EVENT(USkeleton, FSmartNamesChangedEvent);

#if WITH_EDITORONLY_DATA
	/**
	 * Global compatibility delegate, used to override skeleton compatibility. If this returns true, no additional
	 * compatibility checks are made.
	 */
	ENGINE_API static FAreAllSkeletonsCompatible AreAllSkeletonsCompatibleDelegate;
#endif

public:
	UFUNCTION(BlueprintCallable, Category=Skeleton)
	ENGINE_API void AddCompatibleSkeleton(const USkeleton* SourceSkeleton);

	UFUNCTION(BlueprintCallable, Category = Skeleton, DisplayName = "AddCompatibleSkeleton")
	ENGINE_API void AddCompatibleSkeletonSoft(const TSoftObjectPtr<USkeleton>& SourceSkeleton);

	ENGINE_API void RemoveCompatibleSkeleton(const USkeleton* SourceSkeleton);
	ENGINE_API void RemoveCompatibleSkeleton(const TSoftObjectPtr<USkeleton>& SourceSkeleton);

	/** 
	 * Indexing naming convention
	 * 
	 * Since this code has indexing to very two distinct array but it can be confusing so I am making it consistency for naming
	 * 
	 * First index is SkeletalMesh->RefSkeleton index - I call this RefBoneIndex
	 * Second index is BoneTree index in USkeleton - I call this TreeBoneIndex
	 */

	/**
	 * Verify to see if we can match this skeleton with the provided SkinnedAsset.
	 * 
	 * Returns true 
	 *		- if bone hierarchy matches (at least needs to have matching parent) 
	 *		- and if parent chain matches - meaning if bone tree has A->B->C and if ref pose has A->C, it will fail
	 *		- and if more than 50 % of bones matches
	 *  
	 * @param	InSkinnedAsset	SkinnedAsset to compare the Skeleton against.
	 * @param   bDoParentChainCheck When true (the default) this method also compares if chains match with the parent. 
	 * 
	 * @return				true if animation set can play on supplied SkinnedAsset, false if not.
	 */
	ENGINE_API bool IsCompatibleMesh(const USkinnedAsset* InSkinnedAsset, bool bDoParentChainCheck=true) const;

	/** Clears all cache data **/
	ENGINE_API void ClearCacheData();

	/**
	 * Find a mesh linkup table (mapping of skeleton bone tree indices to refpose indices) for a particular SkinnedAsset
	 * If one does not already exist, create it now.
	 */
	ENGINE_API const FSkeletonToMeshLinkup& FindOrAddMeshLinkupData(const USkinnedAsset* InSkinnedAsset);

	/**
	 * Adds a new Mesh Linkup Table to the map  for a particular SkinnedAsset
	 *
	 * @param	InSkinnedAsset	: SkinnedAsset to build look up for
	 * @return	Const ref to the added FSkeletonToMeshLinkup unique ptr
	 */
	ENGINE_API const FSkeletonToMeshLinkup& AddMeshLinkupData(const USkinnedAsset* InSkinnedAsset);

	/** 
	 * Merge Bones (RequiredBones from InSkinnedAsset) to BoneTrees if not exists
	 * 
	 * Note that this bonetree can't ever clear up because doing so will corrupt all animation data that was imported based on this
	 * If nothing exists, it will build new bone tree 
	 * 
	 * @param InSkinnedAsset		: Mesh to build from. 
	 * @param RequiredRefBones		: RequiredBones are subset of list of bones (index to InSkinnedAsset->RefSkeleton)
									Most of cases, you don't like to add all bones to skeleton, so you'll have choice of cull out some
	 * 
	 * @return true if success
	 */
	ENGINE_API bool MergeBonesToBoneTree(const USkinnedAsset* InSkinnedAsset, const TArray<int32> &RequiredRefBones, bool bShowProgress = true);

	/** 
	 * Merge all Bones to BoneTrees if not exists
	 * 
	 * Note that this bonetree can't ever clear up because doing so will corrupt all animation data that was imported based on this
	 * If nothing exists, it will build new bone tree 
	 * 
	 * @param InSkinnedAsset		: Mesh to build from. 
	 * 
	 * @return true if success
	 */
	ENGINE_API bool MergeAllBonesToBoneTree(const USkinnedAsset* InSkinnedAsset, bool bShowProgress = true);

	/** 
	 * Merge has failed, then Recreate BoneTree
	 * 
	 * This will invalidate all animations that were linked before, but this is needed 
	 * 
	 * @param InSkinnedAsset		: Mesh to build from. 
	 * 
	 * @return true if success
	 */
	ENGINE_API bool RecreateBoneTree(USkinnedAsset* InSkinnedAsset);

	
	/**
	 * Get the local-space ref pose for the specified retarget source.
	 * @param	RetargetSource	The name of the retarget source to find
	 * @return the transforms for the retarget source reference pose. If the retarget source is not found, this returns the skeleton's reference pose.
	 */
	ENGINE_API const TArray<FTransform>& GetRefLocalPoses( FName RetargetSource = NAME_None ) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Find a retarget source for a particular mesh.
	 * @param	InSkinnedAsset	The skinned asset mesh to find a source for
	 * @return NAME_None if a retarget source was not found, or a valid name if it was
	 */
	ENGINE_API FName GetRetargetSourceForMesh(USkinnedAsset* InSkinnedAsset) const;

	/**
	 * Get all the retarget source names for this skeleton.
	 * @param	OutRetargetSources	The retarget source names array to be filled
	 */
	ENGINE_API void GetRetargetSources(TArray<FName>& OutRetargetSources) const;
#endif

	/** 
	 * Get Bone Tree Index from Reference Bone Index
	 * @param	InSkinnedAsset	SkinnedAsset for the ref bone idx
	 * @param	InRefBoneIdx	Reference Bone Index to look for - index of USkinnedAsset.RefSkeleton
	 * @return	Index of BoneTree Index
	 */
	ENGINE_API int32 GetSkeletonBoneIndexFromMeshBoneIndex(const USkinnedAsset* InSkinnedAsset, const int32 MeshBoneIndex);

	/** 
	 * Get Reference Bone Index from Bone Tree Index
	 * @param	InSkinnedAsset	SkinnedAsset for the ref bone idx
	 * @param	InBoneTreeIdx	Bone Tree Index to look for - index of USkeleton.BoneTree
	 * @return	Index of BoneTree Index
	 */
	ENGINE_API int32 GetMeshBoneIndexFromSkeletonBoneIndex(const USkinnedAsset* InSkinnedAsset, const int32 SkeletonBoneIndex);

	EBoneTranslationRetargetingMode::Type GetBoneTranslationRetargetingMode(const int32 BoneTreeIdx, bool bDisableRetargeting = false) const
	{
		if (!bDisableRetargeting && BoneTree.IsValidIndex(BoneTreeIdx))
		{
			return BoneTree[BoneTreeIdx].TranslationRetargetingMode;
		}
		return EBoneTranslationRetargetingMode::Animation;
	}

	/** 
	 * Rebuild Look up between SkelMesh to BoneTree - this should only get called when SkelMesh is re-imported or so, where the mapping may be no longer valid
	 *
	 * @param	InSkinnedAsset	: SkinnedAsset to build look up for
	 */
	ENGINE_API void RebuildLinkup(const USkinnedAsset* InSkinnedAsset);

	/**
	 * Remove Link up cache for the SkelMesh
	 *
	 * @param	InSkinnedAsset	: SkinnedAsset to remove linkup cache for 
	 */
	void RemoveLinkup(const USkinnedAsset* InSkinnedAsset);

	ENGINE_API void SetBoneTranslationRetargetingMode(const int32 BoneIndex, EBoneTranslationRetargetingMode::Type NewRetargetingMode, bool bChildrenToo=false);

	virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** 
	 * Create RefLocalPoses from InSkinnedAsset. Note InSkinnedAsset cannot be null and this function will assert if it is.
	 * 
	 * If bClearAll is false, it will overwrite ref pose of bones that are found in InSkelMesh
	 * If bClearAll is true, it will reset all Reference Poses 
	 * Note that this means it will remove transforms of extra bones that might not be found in this InSkinnedAsset
	 *
	 * @return true if successful. false if InSkinnedAsset wasn't compatible with the bone hierarchy
	 */
	ENGINE_API void UpdateReferencePoseFromMesh(const USkinnedAsset* InSkinnedAsset);

#if WITH_EDITORONLY_DATA
	/**
	 * Update Retarget Source with given name
	 *
	 * @param Name	Name of pose to update
	 */
	ENGINE_API void UpdateRetargetSource( const FName InName );
#endif
protected:
	/** 
	 * Check if Parent Chain Matches between BoneTree, and SkinnedAsset 
	 * Meaning if BoneTree has A->B->C (top to bottom) and if SkinnedAsset has A->C
	 * It will fail since it's missing B
	 * We ensure this chain matches to play animation properly
	 *
	 * @param StartBoneIndex	: BoneTreeIndex to start from in BoneTree 
	 * @param InSkinnedAsset	: InSkinnedAsset to compare
	 *
	 * @return true if matches till root. false if not. 
	 */
	bool DoesParentChainMatch(int32 StartBoneTreeIndex, const USkinnedAsset* InSkinnedAsset) const;

	/**
	 * Build Look up between SkinnedAsset to BoneTree
	 *
	 * @param	InSkinnedAsset			: SkinnedAsset to build look up for
	 * @param	FSkeletonToMeshLinkup	: Out mesh linkup data
	 */
	void BuildLinkupData(const USkinnedAsset* InSkinnedAsset, FSkeletonToMeshLinkup& NewMeshLinkup);

#if WITH_EDITORONLY_DATA
	/**
	 * Refresh All Retarget Sources
	 */
	void RefreshAllRetargetSources();
#endif
	/**
	 * Create Reference Skeleton From the given Mesh 
	 * 
	 * @param InSkinnedAsset	SkinnedAsset that this Skeleton is based on
	 * @param RequiredRefBones	List of required bones to create skeleton from
	 *
	 * @return true if successful
	 */
	bool CreateReferenceSkeletonFromMesh(const USkinnedAsset* InSkinnedAsset, const TArray<int32> & RequiredRefBones);

#if WITH_EDITOR
	DECLARE_TS_MULTICAST_DELEGATE( FOnSkeletonHierarchyChangedMulticaster );
	FOnSkeletonHierarchyChangedMulticaster OnSkeletonHierarchyChanged;

	DECLARE_TS_MULTICAST_DELEGATE( FOnReferenceSkeletonChangedMulticaster );
	FOnReferenceSkeletonChangedMulticaster OnReferenceSkeletonChanged;

	/** Call this when the skeleton has changed to fix dependent assets */
	ENGINE_API void HandleSkeletonHierarchyChange(bool bShowProgress = true);

	/** Call this when the reference skeleton has changed to fix dependent assets */
	ENGINE_API void HandleReferenceSkeletonChange() const;

public:
	typedef FOnSkeletonHierarchyChangedMulticaster::FDelegate FOnSkeletonHierarchyChanged;
	typedef FOnReferenceSkeletonChangedMulticaster::FDelegate FOnReferenceSkeletonChanged;

	/** Registers a delegate to be called after the skeleton hierarchy has changed */
	ENGINE_API void RegisterOnSkeletonHierarchyChanged(const FOnSkeletonHierarchyChanged& Delegate);
	ENGINE_API void UnregisterOnSkeletonHierarchyChanged(FDelegateUserObject Unregister);

	/** Registers a delegate to be called after the reference skeleton has changed */
	ENGINE_API void RegisterOnReferenceSkeletonChanged(const FOnReferenceSkeletonChanged& Delegate);
	ENGINE_API void UnregisterOnReferenceSkeletonChanged(FDelegateUserObject Unregister);

	/** Removes the supplied bones from the skeleton */
	ENGINE_API void RemoveBonesFromSkeleton(const TArray<FName>& BonesToRemove, bool bRemoveChildBones);

	// Asset registry information for animation notifies
	ENGINE_API static const FName AnimNotifyTag;
	ENGINE_API static const FString AnimNotifyTagDelimiter;

	// Asset registry information for animation sync markers
	ENGINE_API static const FName AnimSyncMarkerTag;
	ENGINE_API static const FString AnimSyncMarkerTagDelimiter;
	
	// Asset registry information for animation curves
	ENGINE_API static const FName CurveNameTag;
	ENGINE_API static const FString CurveTagDelimiter;

	// Asset registry information for animation attributes
	ENGINE_API static const FName AttributeTag;

	// Asset registry information for compatible skeletons
	ENGINE_API static const FName CompatibleSkeletonsNameTag;
	ENGINE_API static const FString CompatibleSkeletonsTagDelimiter;

#endif

public:
	ENGINE_API USkeletalMeshSocket* FindSocketAndIndex(FName InSocketName, int32& OutIndex) const;
	ENGINE_API USkeletalMeshSocket* FindSocket(FName InSocketName) const;

private:
	/** Regenerate new Guid */
	void RegenerateGuid();
	void RegenerateVirtualBoneGuid();

	// Handle skeletons being reloaded via the content browser
	static void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

public:
	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface
protected:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Skeleton)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Skeleton)
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;
#endif

	friend FReferenceSkeleton;
	friend struct FReferenceSkeletonModifier;
	friend class FEditableSkeleton;
};

