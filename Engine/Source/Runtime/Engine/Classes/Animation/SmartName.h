// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Animation/BoneReference.h"
#include "Animation/AnimTypes.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "Animation/AnimCurveMetadata.h"
#include "SmartName.generated.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

struct FSmartName;

// DEPRECATED - smart names and their mappings are no longer used
struct UE_DEPRECATED(5.8, "Smart names and their mappings are no longer used") FSmartNameMapping;
USTRUCT()
struct FSmartNameMapping
{
	friend struct FSmartNameMappingIterator;
	friend class USkeleton;

	GENERATED_USTRUCT_BODY();

	ENGINE_API FSmartNameMapping();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	~FSmartNameMapping() = default;
	FSmartNameMapping(const FSmartNameMapping&) = default;
	FSmartNameMapping(FSmartNameMapping&&) = default;
	FSmartNameMapping& operator=(const FSmartNameMapping&) = default;
	FSmartNameMapping& operator=(FSmartNameMapping&&) = default;

	UE_DEPRECATED(5.3, "FSmartNameMapping functions are no longer used.")
	ENGINE_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FSmartNameMapping& Elem);

private:
	// List of curve names, indexed by UID
	TArray<FName> CurveNameList;

#if !WITH_EDITOR
	// List of curve metadata, indexed by UID
	TArray<FCurveMetaData> CurveMetaDataList;
#endif

	TMap<FName, FCurveMetaData> CurveMetaDataMap;
};

// Struct for providing access to SmartNameMapping data within FSmartNameMapping::Iterate callback functions
struct UE_DEPRECATED(5.8, "Smart names and their mappings are no longer used") FSmartNameMappingIterator;
struct FSmartNameMappingIterator
{
	public:
		friend struct FSmartNameMapping;

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	FSmartNameMappingIterator() = default;
	~FSmartNameMappingIterator() = default;
	FSmartNameMappingIterator(const FSmartNameMappingIterator&) = default;
	FSmartNameMappingIterator(FSmartNameMappingIterator&&) = default;
	FSmartNameMappingIterator& operator=(const FSmartNameMappingIterator&) = default;
	FSmartNameMappingIterator& operator=(FSmartNameMappingIterator&&) = default;

	private:
		// This class struct should only be crated by FSmartNameMapping::Iterate
		FSmartNameMappingIterator(const FSmartNameMapping* InMapping, SmartName::UID_Type InIndex):
			Mapping(InMapping), Index(InIndex)
		{}

		const FSmartNameMapping* Mapping;
		SmartName::UID_Type Index;
};

struct UE_DEPRECATED(5.8, "Smart names and their mappings are no longer used") FSmartNameContainer;
USTRUCT()
struct FSmartNameContainer
{
	GENERATED_USTRUCT_BODY();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	FSmartNameContainer() = default;
	~FSmartNameContainer() = default;
	FSmartNameContainer(const FSmartNameContainer&) = default;
	FSmartNameContainer(FSmartNameContainer&&) = default;

	ENGINE_API void Serialize(FArchive& Ar, bool bIsTemplate);

	friend FArchive& operator<<(FArchive& Ar, FSmartNameContainer& Elem);

	/** Only restricted classes can access the protected interface */
	friend class USkeleton;
protected:
	ENGINE_API FSmartNameMapping* GetContainerInternal(const FName& ContainerName);
	ENGINE_API const FSmartNameMapping* GetContainerInternal(const FName& ContainerName) const;

private:
	TMap<FName, FSmartNameMapping> NameMappings;	// List of smartname mappings

#if WITH_EDITORONLY_DATA
	// Editor copy of the data we loaded, used to preserve determinism during cooking
	TMap<FName, FSmartNameMapping> LoadedNameMappings;
#endif

};

template<>
struct TStructOpsTypeTraits<FSmartNameContainer> : public TStructOpsTypeTraitsBase2<FSmartNameContainer>
{
	enum
	{
		WithCopy = false,
	};
};

struct UE_DEPRECATED(5.8, "Smart names and their mappings are no longer used") FSmartName;
USTRUCT()
struct FSmartName
{
	GENERATED_USTRUCT_BODY();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	~FSmartName() = default;
	FSmartName(const FSmartName&) = default;
	FSmartName(FSmartName&&) = default;
	FSmartName& operator=(const FSmartName&) = default;
	FSmartName& operator=(FSmartName&&) = default;

	// name
	UPROPERTY(VisibleAnywhere, Category=FSmartName)
	FName DisplayName;

	// SmartName::UID_Type - for faster access
	SmartName::UID_Type	UID;

	FSmartName()
		: DisplayName(NAME_None)
		, UID(SmartName::MaxUID)
	{}

	bool operator==(FSmartName const& Other) const
	{
		return (DisplayName == Other.DisplayName && UID == Other.UID);
	}

	bool operator!=(const FSmartName& Other) const
	{
		return !(*this == Other);
	}

	ENGINE_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FSmartName& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FSmartName> : public TStructOpsTypeTraitsBase2<FSmartName>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
