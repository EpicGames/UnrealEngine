// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Animation/AnimCurveTypes.h"
#include "TimeStretchCurve.generated.h"

UENUM()
enum class ETimeStretchCurveMapping : uint8
{
	T_Original = 0,
	T_TargetMin = 1,
	T_TargetMax = 2,
	MAX = 3
};

USTRUCT()
struct FTimeStretchCurveMarker
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = Animation)
	float Time[(uint8)ETimeStretchCurveMapping::MAX];

	UPROPERTY(VisibleAnywhere, Category = Animation)
	float Alpha;

	FTimeStretchCurveMarker()
		: Alpha(0.f)
	{
		Time[(uint8)ETimeStretchCurveMapping::T_Original] = 0.0f;
		Time[(uint8)ETimeStretchCurveMapping::T_TargetMin] = 0.0f;
		Time[(uint8)ETimeStretchCurveMapping::T_TargetMax] = 0.0f;
	}

	FTimeStretchCurveMarker(float InT_Original, float InAlpha)
		: Alpha(InAlpha)
	{
		Time[(uint8)ETimeStretchCurveMapping::T_Original] = InT_Original;
		Time[(uint8)ETimeStretchCurveMapping::T_TargetMin] = 0.0f;
		Time[(uint8)ETimeStretchCurveMapping::T_TargetMax] = 0.0f;
	}
}; 

/*
	= Time Stretch Curve =

	= What is it?

		The Time Stretch Curve is an optional float curve that montages can
		use to define where a montage is allowed to speed up or slow down.
		Let's say we have a montage of default play time T_Original.
		We now want that montage to play for a different T_Target play time
		Typically we would uniformly play rate the animation to reach that goal.

		The Time Stretch Curve allows doing the same thing but non uniformly,
		by defining which regions can be play rated more or less.

		The Curve values are normalized.
		So a Curve value of 0 means "don't play rate this".
		And a Curve value of 1 means "play rate this the most".
		Intermediate values will be weighted accordingly.

		Imagine the following scenario, you have a character attacking with a staff.
		The animation is authored with holds after striking.
		Let's say the character levels up over the course of the game, and
		their attacks are getting faster and faster (play time is shorter).

		By using a Time Stretch Curve, most of the time compression could happen
		during the holds. So the strikes look mostly unaffected.
		This allows using a single animation, and scaling it for very different
		desired play times.

	= How does it work?

		Given a Montage of length T_Original, and a float curve C.
		Curve C is sampled over N segments of fixed time 'SamplingTimeStep'.
		Each element, C_i is normalized. 0 <= C_i <= 1
		and 0 <= i <= N.

		We have Sum(SamplingTimeStep) = T_Original = N * SamplingTimeStep
		SamplingTimeStep = T_Original / N

		Given a different length T_Target,
		C remaps positions from T_Target to T_Original according to the following function:
		dTO = dT_i * U * (1 + S * C_i)
		where:
		Sum(dTO) = T_Original
		Sum(dT_i) = T_Target
		U = UniformPlayRate
		S = Curve Scale Factor
		C_i = sampled curve value, constant over the interval dTO

		dTO is fixed, based on the chosen 'SamplingTimeStep',
		but in practice we can combine consecutive segments that have the same C_i value.

		We would like U to be 1 (or -1) as much as possible.
		Meaning the Curve should do all the remapping whenever possible.
		U(niformPlayRate) should only come into play when C can't remap T_Target to T_Original on its own.
		This can happen when trying to speed up the animation,
		but the Curve is not enough to reach that time compression.
		In that event, uniform scaling kicks in.

		We call PR_i (or OverallPlayRate for Segment i)
		PR_i = U * (1 + S * C_i)
		dTO = dT_i * PR_i

		We also want PR_i > 0, that is it should never backtrack or pause during playback of animation.
		A Montage can still play in reverse with U < 0.

	= How is T_Target defined?

		When we play a Montage with a PlayRate of PR, we assume this means:
		T_Target = T_Original * PR
		So this system does not change the interface for playing a montage.

		If a curve is not defined, the mapping is defined as:
		dTO_i = dT_i * U

		If a a curve is defined:
		dTO_i = dT_i * U * (1 + S * C_i)

		We can see that no curve means S == 0.
		Also if we're not scaling the montage, T_Target == T_Original, we also have S == 0.

		So, this makes the curve completely optional. And it seamlessly integrates with the current montage interface.
		If you want playback time to be half, that means playing the montage with a play rate of 2.
		If there is a TimeStretchCurve, it will be used.
		But regardless or using a curve or not, play back time is guaranteed.

	= Finding U and S =

		Ideally, we could figure out what U and S are given a T_Target play time.
		Unfortunately, the math for this is very complex.

		We update the montage position like this:
		dTO_i = dT_i * U * (1 + C_i)

		We sum this over the N time segments:
		Sum(dTO_i) = Sum(dT_i * U * (1 + C_i))
		Sum(dTO_i) = Sum(dT_i) * U + Sum(dT_i * U * S * C_i))
		T_Original = T_Target * U + U * S * Sum(dT_i * C_i)

		Where:
		S = (T_Original - T_Target * U) / (U * Sum(dT_i * C_i))

		If we were able to get dT_i constant, we could pull it out and get:
		S = (T_Original - T_Target * U) / (U * dT * Sum(C_i))
		Where Sum(C_i) can be pre-computed.

		Unfortunately we don't have dT_i constant, and we can't,
		since it is variable, and dependent on what S and U are.

		So our approach instead is to precompute lower and upper bounds for this curve.
		We cache the results for dT_i for S = 100.f and S = -1.f + 0.01f
		This gives us data for T_Target_Min and T_Target_Max.
		Within these bounds, we can approximate dT_i, and also Sum(dT_i * C_i) by linear interpolation.
		Outside of these bounds, we rely on U to get us to our desired T_Target play back time.

		'ConditionallyUpdateTimeStretchCurveCachedData' takes care of figuring out U and S
		based on a given T_Target play back time.

	= 'target' and 'original' space

		At run time, we generate a set of markers in what we call 'target' and 'original' space.
		'original' space means in the original play time the montage was authored in.
		So that maps to actual frames of animation and actual position in the montage.

		'target' space is the same set of markers, but mapped in play back time.
		That is the actual time it takes to play back this montage.

		Taking our play rate equation from above, it is:
		dT_Original = dT_Target * U * (1 + S * C_i)

		We see that montage position = playback time * play rate.

		Once we have mapped our markers in both 'target' and 'original' space,
		we can easily go from one to the other, since time moves linearly between markers.
		Since between markers the play rate is defined as constant values:
		PR_i = U * (1.f + S * C_i).
		And we know that C_i is constant between two markers.

		So if we know between which markers we are in one space, we can switch to the other space instantly,
		as our relative position between these markers will be the same.

		So by jumping between these spaces, we can quickly go from a montage position to its playback time.
		We can increase the playback time by the current's frame deltatime,
		and jump back to the corresponding frame of animation.
		That's in a nutshell how this system works.
*/

USTRUCT()
struct FTimeStretchCurve
{
	GENERATED_USTRUCT_BODY();

	friend struct FTimeStretchCurveInstance;

public:
	FTimeStretchCurve()
		: SamplingRate(60.f)
		, CurveValueMinPrecision(0.01f)
	{
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_Original] = 0.0f;
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMin] = 0.0f;
		Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::T_TargetMax] = 0.0f;
	}

	ENGINE_API bool IsValid() const;
	ENGINE_API void Reset();
	ENGINE_API void BakeFromFloatCurve(const FFloatCurve& TimeStretchCurve, float InSequenceLength);

private:
	/**
		Desired Sampling rate of above curve.
		This will be rounded off so we sample the whole curve
		with a fixed time step.
	*/
	UPROPERTY(EditAnywhere, Category = TimeStretchCurve)
	float SamplingRate;

	/**
		Minimum delta allowed between consecutive sampled segments.
		below this value, segments will be combined
		to optimize number of markers.
	*/
	UPROPERTY(EditAnywhere, Category = TimeStretchCurve)
	float CurveValueMinPrecision;

	/** Optimized list of markers. */
	UPROPERTY(VisibleAnywhere, Category = TimeStretchCurve)
	TArray<FTimeStretchCurveMarker> Markers;

	/** Cached Sum(dT_i * C_i) */
	UPROPERTY(VisibleAnywhere, Category = TimeStretchCurve)
	float Sum_dT_i_by_C_i[(uint8)ETimeStretchCurveMapping::MAX];
};

USTRUCT()
struct FTimeStretchCurveInstance
{
	GENERATED_USTRUCT_BODY();

public:
	FTimeStretchCurveInstance()
		: bHasValidData(false)
	{}

	ENGINE_API void InitializeFromPlayRate(float InPlayRate, const FTimeStretchCurve& TimeStretchCurve);
	bool HasValidData() const { return bHasValidData; }

	/**
		Updates InOutMarkerIndex as needed based on 'InPosition' in 'InMarkerPositions'
		So that InOutMarkerIndex satisfies 'IsValidMarkerForPosition'
	*/
	ENGINE_API void UpdateMarkerIndexForPosition(int32& InOutMarkerIndex, float InPosition, const TArray<float>& InMarkerPositions) const;

	/** Validates that the supplied marker index correctly bookends supplied position. */
	ENGINE_API bool IsValidMarkerForPosition(int32 InMarkerIndex, float InPosition, const TArray<float>& InMarkerPositions) const;

	/** Validates that the supplied marker positions bookend supplied position. */
	ENGINE_API bool AreValidMarkerBookendsForPosition(float InPosition, float InP_CurrMarker, float InP_NextMarker) const;

	/**
		Find marker index that bookends supplied position, using supplied markers,
		doing a binary search to find match. Most steps taken are Log2(N).
	*/
	ENGINE_API int32 BinarySearchMarkerIndex(float InPosition, const TArray<float>& InMarkerPositions) const;

	/**
		Converts a Position from Original Space to Target Space.
		This requires a Marker Index that satisfies 'IsValidMarkerForPosition'
		for the supplied position.
	*/
	ENGINE_API float Convert_P_Original_To_Target(int32 InMarkerIndex, float In_P_Original) const;

	/**
		Converts a Position from Target Space to Original Space.
		This requires a Marker Index that satisfies 'IsValidMarkerForPosition'
		for the supplied position.
	*/
	ENGINE_API float Convert_P_Target_To_Original(int32 InMarkerIndex, float In_P_Target) const;

	/** Make sure In_P_Target stays in valid marker range. */
	ENGINE_API float Clamp_P_Target(float In_P_Target) const;

	/** Read access to markers in original space. */
	const TArray<float>& GetMarkers_Original() const { return P_Marker_Original; }
	
	/** Read access to markers in target space. */
	const TArray<float>& GetMarkers_Target() const { return P_Marker_Target; }

	/** Get original play back duration */
	float Get_T_Original() const { return T_Original; }

	/** Get target play back duration */
	float Get_T_Target() const { return T_Target; }

private:
	UPROPERTY(Transient)
	bool bHasValidData;

	float T_Original;
	float T_Target;

	TArray<float> P_Marker_Original;
	TArray<float> P_Marker_Target;
};
