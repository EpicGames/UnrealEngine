// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/PerPlatformProperties.h"
#include "VariableFrameStrippingSettings.generated.h"

#if WITH_EDITORONLY_DATA
namespace UE::Anim::Compression { struct FAnimDDCKeyArgs; }
#endif
/*
* This is a wrapper for the Variable frame stripping Codec.
* It allows for the mass changing of settings on animation sequences in an editor accessible way.
*/
UCLASS(hidecategories = Object, MinimalAPI)
class UVariableFrameStrippingSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	* Enables the change from standard 1/2 frame stripping to stripping a higher amount of frames per frame kept
	*/
	UPROPERTY(Category = Compression, EditAnywhere)
	FPerPlatformBool UseVariableFrameStripping;

	/**
	* The number of Frames that are stripped down to one.
	* Allows for overrides of that multiplier.
	* FrameStrippingRate == 1 would strip no frames, Therefore this is clamped to 2.
	*/
	UPROPERTY(Category = Compression, EditAnywhere, meta = (ClampMin = "2"))
	FPerPlatformInt FrameStrippingRate;


#if WITH_EDITORONLY_DATA
	/** Generates a DDC key that takes into account the current settings, selected codec, input anim sequence and TargetPlatform */
	ENGINE_API void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar);

#endif
};
