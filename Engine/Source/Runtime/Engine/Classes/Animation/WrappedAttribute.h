// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"

namespace UE
{
	namespace Anim 
	{
		/** Helper struct to wrap and templated operate raw memory */
		template<typename InAllocator>
		struct TWrappedAttribute
		{
			TWrappedAttribute() {}

			/** Construct with allocates memory buffer according to type size*/
			TWrappedAttribute(const UScriptStruct* InStruct)
			{
				Allocate(InStruct);
			}
						
			/** Returns typed ptr to memory */
			template <typename Type>
			inline typename TEnableIf<TIsFundamentalType<Type>::Value, Type*>::Type GetPtr()
			{
				return (Type*)StructMemory.GetData();
			}

			template <typename Type>
			inline typename TEnableIf<!TIsFundamentalType<Type>::Value, Type*>::Type GetPtr()
			{
				const UScriptStruct* ScriptStruct = Type::StaticStruct();
				check(ScriptStruct && ScriptStruct->GetStructureSize() == StructMemory.Num());
				return (Type*)StructMemory.GetData();
			}

			template<typename Type>
			inline Type& GetRef() { return *GetPtr<Type>(); }

			/** Returns typed const ptr to memory */
			template <typename Type>
			inline typename TEnableIf<TIsFundamentalType<Type>::Value, const Type*>::Type GetPtr() const
			{
				return (const Type*)StructMemory.GetData();
			}

			template <typename Type>
			inline typename TEnableIf<!TIsFundamentalType<Type>::Value, const Type*>::Type GetPtr() const
			{
				const UScriptStruct* ScriptStruct = Type::StaticStruct();
				check(ScriptStruct && ScriptStruct->GetStructureSize() == StructMemory.Num());
				return (const Type*)StructMemory.GetData();
			}

			/** Returns typed const reference to memory */
			template<typename Type>
			inline const Type& GetRef() const { return *GetPtr<Type>(); }

			/** Allocated memory buffer according to type size */
			template<typename AttributeType>
			inline void Allocate()
			{
				Allocate(AttributeType::StaticStruct());
			}

			/** Allocated memory buffer according to type size */
			inline void Allocate(const UScriptStruct* InStruct)
			{
				check(InStruct);
				if (InStruct)
				{
					const int32 StructureSize = InStruct->GetStructureSize();
					ensure(StructureSize > 0);

					StructMemory.SetNum(StructureSize);
					InStruct->InitializeStruct(GetPtr<void>());
				}
			}
		protected:
			TArray<uint8, InAllocator> StructMemory;
		};
	}
}