// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "AtmosphericFog.generated.h"

/** 
 *	A placeable fog actor that simulates atmospheric light scattering  
 *	@see https://docs.unrealengine.com/latest/INT/Engine/Actors/FogEffects/AtmosphericFog/index.html
 */
UCLASS(showcategories=(Movement, Rendering, Transformation, DataLayers, "Input|MouseInput", "Input|TouchInput"), ClassGroup=Fog, hidecategories=(Info,Object,Input), MinimalAPI, notplaceable)
class UE_DEPRECATED(4.26, "Please use the SkyAtmosphere actor instead.") AAtmosphericFog : public AInfo
{
	GENERATED_UCLASS_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	/** Main fog component */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Atmosphere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UAtmosphericFogComponent> AtmosphericFogComponent;

#if WITH_EDITORONLY_DATA
	/** Arrow component to indicate default sun rotation */
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
#endif

public:
	/** Returns AtmosphericFogComponent subobject **/
	ENGINE_API class UAtmosphericFogComponent* GetAtmosphericFogComponent() { return AtmosphericFogComponent; }

#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	ENGINE_API class UArrowComponent* GetArrowComponent() { return ArrowComponent; }
#endif
};





