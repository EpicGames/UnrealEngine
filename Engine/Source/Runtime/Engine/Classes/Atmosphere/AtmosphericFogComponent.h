// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "Components/SkyAtmosphereComponent.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Serialization/BulkData.h"
#endif

#include "AtmosphericFogComponent.generated.h"

/**
 *	Used to create fogging effects such as clouds.
 */
UCLASS(ClassGroup=Rendering, collapsecategories, hidecategories=(Object, Mobility, Activation, "Components|Activation"), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI, notplaceable)
class UE_DEPRECATED(4.26, "Please use the SkyAtmosphere component instead.") UAtmosphericFogComponent : public USkyAtmosphereComponent
{
	GENERATED_UCLASS_BODY()

	~UAtmosphericFogComponent();

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDefaultBrightness(float NewBrightness) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDefaultLightColor(FLinearColor NewLightColor) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetSunMultiplier(float NewSunMultiplier) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetFogMultiplier(float NewFogMultiplier) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDensityMultiplier(float NewDensityMultiplier) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDensityOffset(float NewDensityOffset) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDistanceScale(float NewDistanceScale) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetAltitudeScale(float NewAltitudeScale) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetStartDistance(float NewStartDistance) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetDistanceOffset(float NewDistanceOffset) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void DisableSunDisk(bool NewSunDisk) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void DisableGroundScattering(bool NewGroundScattering) {}

	/** Deprecated */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|AtmosphericFog")
	ENGINE_API void SetPrecomputeParams(float DensityHeight, int32 MaxScatteringOrder, int32 InscatterAltitudeSampleNum) {}

protected:

public:

protected:

public:
	
	//~ Begin UObject Interface. 
	virtual bool IsPostLoadThreadSafe() const override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface;

private:

};
