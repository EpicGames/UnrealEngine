// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameterControllerInterface.h"
#include "Containers/Array.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"

#include "ActorSoundParameterInterface.generated.h"


// Forward Declarations
class AActor;
class UActorComponent;

/** Interface used to allow an actor to automatically populate any sounds with parameters */
class IActorSoundParameterInterface : public IInterface
{
	GENERATED_BODY()

public:
	// Overrides logic for gathering AudioParameters to set by default when an AudioComponent/ActiveSound plays with a given actor as its Owner.
	UFUNCTION(BlueprintNativeEvent, Category = "Audio|Parameters", meta = (DisplayName = "Get Actor Audio Parameters"))
	ENGINE_API void GetActorSoundParams(TArray<FAudioParameter>& Params) const;
	virtual void GetActorSoundParams_Implementation(TArray<FAudioParameter>& Params) const = 0;
};

UINTERFACE(BlueprintType, MinimalAPI)
class UActorSoundParameterInterface : public UInterface
{
	GENERATED_BODY()

public:

	static ENGINE_API void Fill(const AActor* OwningActor, TArray<FAudioParameter>& OutParams);

private:

	static void GetImplementers(const AActor* InActor, TArray<const AActor*>& OutActors, TArray<const UActorComponent*>& OutComponents);
};
