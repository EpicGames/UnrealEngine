// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Engine/Scene.h"

#include "CameraActor.generated.h"

/** 
 * A CameraActor is a camera viewpoint that can be placed in a level.
 */
UCLASS(ClassGroup=Common, hideCategories=(Input, Rendering), showcategories=("Input|MouseInput", "Input|TouchInput"), Blueprintable, MinimalAPI)
class ACameraActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:

	/** Specifies which player controller, if any, should automatically use this Camera when the controller is active. */
	UPROPERTY(Category="AutoPlayerActivation", EditAnywhere)
	TEnumAsByte<EAutoReceiveInput::Type> AutoActivateForPlayer;

private:

	/** The camera component for this camera */
	UPROPERTY(Category = CameraActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UCameraComponent> CameraComponent;

	UPROPERTY(Category = CameraActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USceneComponent> SceneComponent;
public:

	/** Returns index of the player for whom we auto-activate, or INDEX_NONE (-1) if disabled. */
	UFUNCTION(BlueprintCallable, Category="AutoPlayerActivation")
	ENGINE_API int32 GetAutoActivatePlayerIndex() const;

private:

	UPROPERTY()
	uint32 bConstrainAspectRatio_DEPRECATED:1;

	UPROPERTY()
	float AspectRatio_DEPRECATED;

	UPROPERTY()
	float FOVAngle_DEPRECATED;

	UPROPERTY()
	float PostProcessBlendWeight_DEPRECATED;

	UPROPERTY()
	struct FPostProcessSettings PostProcessSettings_DEPRECATED;

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	ENGINE_API virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph) override;
#endif

	ENGINE_API virtual class USceneComponent* GetDefaultAttachComponent() const override;
	//~ End UObject Interface

protected:
	//~ Begin AActor Interface
	ENGINE_API virtual void BeginPlay() override;
	//~ End AActor Interface

public:
	/** Returns CameraComponent subobject **/
	class UCameraComponent* GetCameraComponent() const { return CameraComponent; }

	/** 
	 * Called to notify that this camera was cut to, so it can update things like interpolation if necessary.
	 * Typically called by the camera component.
	 */
	virtual void NotifyCameraCut() {};

};
