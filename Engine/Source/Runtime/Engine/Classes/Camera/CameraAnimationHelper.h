// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FMinimalViewInfo;

/**
 * Helper structure to store a camera transform.
 */
struct FCameraAnimationHelperOffset
{
	FVector Location;
	FRotator Rotation;
};

/**
 * Helper class for applying animations to a camera.
 */
class FCameraAnimationHelper
{
public:

	/**
	 * Apply an offset to a camera transform in camera space and get the resulting world-space info.
	 *
	 * @param InPOV		The current camera transform
	 * @param InOffset	The offset to apply
	 * @param OutResult	The world-space transform of the offset camera
	 */
	static ENGINE_API void ApplyOffset(const FMinimalViewInfo& InPOV, const FCameraAnimationHelperOffset& InOffset, FVector& OutLocation, FRotator& OutRotation);

	/**
	 * Apply an offset to a camera transform in the given space, and get the resulting world-space info.
	 *
	 * @param UserPlaySpaceMatrix The coordinate system in which to apply the offset
	 * @param InPOV		The current camera transform
	 * @param InOffset	The offset to apply
	 * @param OutResult	The world-space transform of the offset camera
	 */
	static ENGINE_API void ApplyOffset(const FMatrix& UserPlaySpaceMatrix, const FMinimalViewInfo& InPOV, const FCameraAnimationHelperOffset& InOffset, FVector& OutLocation, FRotator& OutRotation);
};

