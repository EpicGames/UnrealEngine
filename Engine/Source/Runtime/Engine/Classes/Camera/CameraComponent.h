// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Scene.h"
#include "Camera/CameraTypes.h"
#include "CameraComponent.generated.h"

class UStaticMesh;

/** Allows a custom SceneProxy to interact with camera specific show flags. */
UCLASS(MinimalAPI)
class UCameraProxyMeshComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
};

/**
  * Represents a camera viewpoint and settings, such as projection type, field of view, and post-process overrides.
  * The default behavior for an actor used as the camera view target is to look for an attached camera component and use its location, rotation, and settings.
  */
UCLASS(HideCategories=(Mobility, Rendering, LOD), Blueprintable, ClassGroup=Camera, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UCameraComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** 
	 * The horizontal field of view (in degrees) in perspective mode (ignored in Orthographic mode)
	 *
	 * If the aspect ratio axis constraint (from ULocalPlayer, ALevelSequenceActor, etc.) is set to maintain vertical FOV, the AspectRatio
	 * property will be used to convert this property's value to a vertical FOV.
	 *
	 */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (UIMin = "5.0", UIMax = "170", ClampMin = "0.001", ClampMax = "360.0", Units = deg))
	float FieldOfView;
	UFUNCTION(BlueprintCallable, Category = Camera)
	virtual void SetFieldOfView(float InFieldOfView) { FieldOfView = InFieldOfView; }

	/**
	 * The horizontal field of view (in degrees) used for primitives tagged as "IsFirstPerson".
	 */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (UIMin = "5.0", UIMax = "170", ClampMin = "0.001", ClampMax = "360.0", Units = deg, EditCondition = "bEnableFirstPersonFieldOfView"))
	float FirstPersonFieldOfView;
	UFUNCTION(BlueprintCallable, Category = Camera)
	virtual void SetFirstPersonFieldOfView(float InFirstPersonFieldOfView) { FirstPersonFieldOfView = InFirstPersonFieldOfView; }

	/**
	 * The scale to apply to primitives tagged as "IsFirstPerson". This is used to scale down primitives towards the camera such that they are small enough not to intersect with the scene.
	 */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (UIMin = "0.001", UIMax = "1.0", ClampMin = "0.001", ClampMax = "1.0", EditCondition = "bEnableFirstPersonScale"))
	float FirstPersonScale;
	UFUNCTION(BlueprintCallable, Category = Camera)
	virtual void SetFirstPersonScale(float InFirstPersonScale) { FirstPersonScale = InFirstPersonScale; }

	/** The desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings)
	float OrthoWidth;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetOrthoWidth(float InOrthoWidth) { OrthoWidth = InOrthoWidth; }

	/** Automatically determine a min/max Near/Far clip plane position depending on OrthoWidth value*/
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings)
	bool bAutoCalculateOrthoPlanes;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetAutoCalculateOrthoPlanes(bool bAutoCalculate) { bAutoCalculateOrthoPlanes = bAutoCalculate; }

	/** Manually adjusts the planes of this camera, maintaining the distance between them. Positive moves out to the farplane, negative towards the near plane */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (EditCondition = "bAutoCalculateOrthoPlanes"))
	float AutoPlaneShift;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetAutoPlaneShift(float InAutoPlaneShift) { AutoPlaneShift = InAutoPlaneShift; }

	/** The near plane distance of the orthographic view (in world units) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (EditCondition = "!bAutoCalculateOrthoPlanes"))
	float OrthoNearClipPlane;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetOrthoNearClipPlane(float InOrthoNearClipPlane) { OrthoNearClipPlane = InOrthoNearClipPlane; }

	/** The far plane distance of the orthographic view (in world units) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (EditCondition = "!bAutoCalculateOrthoPlanes"))
	float OrthoFarClipPlane;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetOrthoFarClipPlane(float InOrthoFarClipPlane) { OrthoFarClipPlane = InOrthoFarClipPlane; }

	/** Adjusts the near/far planes and the view origin of the current camera automatically to avoid clipping and light artefacting*/
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings)
	bool bUpdateOrthoPlanes;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetUpdateOrthoPlanes(bool bInUpdateOrthoPlanes) { bUpdateOrthoPlanes = bInUpdateOrthoPlanes; }

	/** If UpdateOrthoPlanes is enabled, this setting will use the cameras current height to compensate the distance to the general view (as a pseudo distance to view target when one isn't present) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (EditCondition = "bUpdateOrthoPlanes"))
	bool bUseCameraHeightAsViewTarget;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetUseCameraHeightAsViewTarget(bool bInUseCameraHeightAsViewTarget) { bUseCameraHeightAsViewTarget = bInUseCameraHeightAsViewTarget; }

	/** Aspect Ratio (Width/Height) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings, meta = (ClampMin = "0.1", ClampMax = "100.0", EditCondition = "bConstrainAspectRatio"))
	float AspectRatio;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetAspectRatio(float InAspectRatio) { AspectRatio = InAspectRatio; }

	/** Override for the default aspect ratio axis constraint defined on the local player */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions, meta = (EditCondition = "!bConstrainAspectRatio && bOverrideAspectRatioAxisConstraint"))
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetAspectRatioAxisConstraint(EAspectRatioAxisConstraint InAspectRatioAxisConstraint) { AspectRatioAxisConstraint = InAspectRatioAxisConstraint; }

	/** If bConstrainAspectRatio is true, black bars will be added if the destination view has a different aspect ratio than this camera requested. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bConstrainAspectRatio : 1;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetConstraintAspectRatio(bool bInConstrainAspectRatio) { bConstrainAspectRatio = bInConstrainAspectRatio; }

	/** Whether to override the default aspect ratio axis constraint defined on the local player */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bOverrideAspectRatioAxisConstraint : 1;

	// If true, account for the field of view angle when computing which level of detail to use for meshes.
	UPROPERTY(Interp, EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = CameraOptions)
	uint8 bUseFieldOfViewForLOD : 1;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetUseFieldOfViewForLOD(bool bInUseFieldOfViewForLOD) { bUseFieldOfViewForLOD = bInUseFieldOfViewForLOD; }
	
	/** Amount to increase the view frustum by, from 0.0 for no increase to 1.0 for 100% increase */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions, meta = (UIMin="0.0", ClampMin="0.0", UIMax="1.0", ClampMax="1.0"))
	float Overscan;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetOverscan(float InOverscan) { Overscan = InOverscan; }

	/**
	 * Experimental: Amount to increase each side of the view frustum by, from 0.0 for no increase to 1.0 for 100% increase.
	 * By convention, X is the left overscan, Y is the right overscan, Z is the top overscan, and W is the bottom overscan. Stacks with uniform Overscan.
	 * Not currently exposed to the editor or blueprints, intended for internal use for now
	 */
	UPROPERTY(meta = (UIMin="0.0", ClampMin="0.0", UIMax="1.0", ClampMax="1.0"))
	FVector4f AsymmetricOverscan = FVector4f::Zero();
	UFUNCTION()
	void SetAsymmetricOverscan(const FVector4f& InAsymmetricOverscan) { AsymmetricOverscan = InAsymmetricOverscan; }
	
	/**
	 * Indicates that the resolution should be scaled by the overscan amount so that the original view frustum remains the same resolution.
	 * Note that when enabled, increasing overscan will result in increased rendering workload, potentially decreasing performance as resolution increases
	 */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	bool bScaleResolutionWithOverscan;

	/**
	 * Sets whether to scale the resolution by the amount of overscan so that the original view frustum remains the same resolution.
	 * Note that when enabled, increasing overscan will result in increased rendering workload, potentially decreasing performance as resolution increases
	 */
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetScaleResolutionWithOverscan(bool bInScaleResolutionWithOverscan) { bScaleResolutionWithOverscan = bInScaleResolutionWithOverscan; }

	/**
	 * Indicates that the overscanned pixels should be cropped at the end of the rendering pipeline, allowing the overscanned pixels to be used in post process effects
	 * that need extra pixels beyond the view frustum (e.g. lens distortion) without having to render those pixels to the screen. When bScaleResolutionWithOverscan is enabled,
	 * the cropped image will have the same resolution as the original non-overscanned image, but when disabled, the cropped image will be a lower resolution.
	 */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	bool bCropOverscan;

	/**
	 * Sets whether to crop the overscanned pixels at the end of the rendering pipeline, allowing the overscanned pixels to be used in post process effects
	 * that need extra pixels beyond the view frustum (e.g. lens distortion) without having to render those pixels to the screen. When bScaleResolutionWithOverscan is enabled,
	 * the cropped image will have the same resolution as the original non-overscanned image, but when disabled, the cropped image will be a lower resolution.
	 */
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetCropOverscan(bool bInCropOverscan) { bCropOverscan = bInCropOverscan; }

#if WITH_EDITOR
	// Returns the filmback text used for burnins on preview viewports
	UFUNCTION()
	ENGINE_API virtual FText GetFilmbackText() const;
#endif

#if WITH_EDITORONLY_DATA
	// The Frustum visibility flag for draw frustum component initialization
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	bool bDrawFrustumAllowed = true;

	/** If the camera mesh is visible in game. Only relevant when running editor builds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	uint8 bCameraMeshHiddenInGame : 1;
#endif

	/** True if the camera's orientation and position should be locked to the HMD */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bLockToHmd : 1;

	/**
	 * If this camera component is placed on a pawn, should it use the view/control rotation of the pawn where possible?
	 * @see APawn::GetViewRotation()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bUsePawnControlRotation : 1;

	/** True if the first person field of view should be used for primitives tagged as "IsFirstPerson". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bEnableFirstPersonFieldOfView : 1;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetEnableFirstPersonFieldOfView(bool bInEnableFirstPersonFieldOfView) { bEnableFirstPersonFieldOfView = bInEnableFirstPersonFieldOfView; }

	/** True if the first person scale should be used for primitives tagged as "IsFirstPerson". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraOptions)
	uint8 bEnableFirstPersonScale : 1;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetEnableFirstPersonScale(bool bInEnableFirstPersonScale) { bEnableFirstPersonScale = bInEnableFirstPersonScale; }

protected:
	/** True to enable the additive view offset, for adjusting the view without moving the component. */
	uint8 bUseAdditiveOffset : 1;

public:
	// The type of camera
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = CameraSettings)
	TEnumAsByte<ECameraProjectionMode::Type> ProjectionMode;
	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetProjectionMode(ECameraProjectionMode::Type InProjectionMode) { ProjectionMode = InProjectionMode; }

	UFUNCTION(BlueprintCallable, Category = Camera)
	void SetPostProcessBlendWeight(float InPostProcessBlendWeight) { PostProcessBlendWeight = InPostProcessBlendWeight; }

	// UActorComponent interface
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
#if WITH_EDITOR
	ENGINE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void CheckForErrors() override;
#endif
	// End of UActorComponent interface

	// USceneComponent interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
#endif 
	// End of USceneComponent interface

	// UObject interface
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#endif
	// End of UObject interface

protected:
	ENGINE_API bool IsXRHeadTrackedCamera() const;
	ENGINE_API virtual void HandleXRCamera(float DeltaTime);

public:
	/**
	 * Returns camera's Point of View.
	 * Called by Camera class. Subclass and postprocess to add any effects.
	 */
	UFUNCTION(BlueprintCallable, Category = Camera)
	ENGINE_API virtual void GetCameraView(float DeltaTime, FMinimalViewInfo& DesiredView);

	/** Adds an Blendable (implements IBlendableInterface) to the array of Blendables (if it doesn't exist) and update the weight */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	void AddOrUpdateBlendable(TScriptInterface<IBlendableInterface> InBlendableObject, float InWeight = 1.0f) { PostProcessSettings.AddBlendable(InBlendableObject, InWeight); }

	/** Removes a blendable. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	void RemoveBlendable(TScriptInterface<IBlendableInterface> InBlendableObject) { PostProcessSettings.RemoveBlendable(InBlendableObject); }

#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void SetCameraMesh(UStaticMesh* Mesh);
#endif 

protected:

#if WITH_EDITORONLY_DATA
	// The frustum component used to show visually where the camera field of view is
	TObjectPtr<class UDrawFrustumComponent> DrawFrustum;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Camera)
	TObjectPtr<class UStaticMesh> CameraMesh;

	// The camera mesh to show visually where the camera is placed
	TObjectPtr<class UStaticMeshComponent> ProxyMeshComponent;

	ENGINE_API virtual void ResetProxyMeshTransform();

	/** Ensure the proxy mesh is in the correct place */
	ENGINE_API void UpdateProxyMeshTransform();

	/* Update draw frustum values */
	ENGINE_API void UpdateDrawFrustum();

#endif	// WITH_EDITORONLY_DATA

	/**
	* Internal function for updating the camera mesh visibility in PIE
	*/
	UFUNCTION(BlueprintSetter)
	ENGINE_API void OnCameraMeshHiddenChanged();

	/** An optional extra transform to adjust the final view without moving the component, in the camera's local space */
	FTransform AdditiveOffset;

public:

	/** Indicates if PostProcessSettings should be used when using this Camera to view through. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = PostProcess, meta = (UIMin = "0.0", UIMax = "1.0"))
	float PostProcessBlendWeight;

protected:
	/** An optional extra FOV offset to adjust the final view without modifying the component */
	float AdditiveFOVOffset;

	/** Optional extra PostProcessing blends stored for this camera. They are not applied automatically in GetCameraView. */
	TArray<FPostProcessSettings> ExtraPostProcessBlends;
	TArray<float> ExtraPostProcessBlendWeights;

public:

	/** Applies the given additive offset, preserving any existing offset */
	ENGINE_API void AddAdditiveOffset(FTransform const& Transform, float FOV);
	/** Removes any additive offset. */
	ENGINE_API void ClearAdditiveOffset();
	/** Get the additive offset */
	ENGINE_API void GetAdditiveOffset(FTransform& OutAdditiveOffset, float& OutAdditiveFOVOffset) const;

	/** Stores a given PP and weight to be later applied when the final PP is computed. Used for things like in-editor camera animation preview. */
	ENGINE_API void AddExtraPostProcessBlend(FPostProcessSettings const& PPSettings, float PPBlendWeight);
	/** Removes any extra PP blends. */
	ENGINE_API void ClearExtraPostProcessBlends();
	/** Returns any extra PP blends that were stored. */
	ENGINE_API void GetExtraPostProcessBlends(TArray<FPostProcessSettings>& OutSettings, TArray<float>& OutWeights) const;

	/** 
	 * Can be called from external code to notify that this camera was cut to, so it can update 
	 * things like interpolation if necessary.
	 */
	ENGINE_API virtual void NotifyCameraCut();

public:

	/** Post process settings to use for this camera. Don't forget to check the properties you want to override */
	UPROPERTY(Interp, BlueprintReadWrite, Category = PostProcess, meta=(ShowOnlyInnerProperties))
	struct FPostProcessSettings PostProcessSettings;

#if WITH_EDITORONLY_DATA
	// Refreshes the visual components to match the component state
	ENGINE_API virtual void RefreshVisualRepresentation();

	ENGINE_API void OverrideFrustumColor(FColor OverrideColor);
	ENGINE_API void RestoreFrustumColor();

public:
	/** DEPRECATED: use bUsePawnControlRotation instead */
	UPROPERTY()
	uint32 bUseControllerViewRotation_DEPRECATED:1;
#endif	// WITH_EDITORONLY_DATA
};
