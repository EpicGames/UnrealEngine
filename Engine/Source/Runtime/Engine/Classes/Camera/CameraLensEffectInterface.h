// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"
#include "Math/Transform.h"
#include "Templates/SubclassOf.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "CameraLensEffectInterface.generated.h"

class AActor;
class APlayerCameraManager;
class UFXSystemComponent;

UINTERFACE(meta=(CannotImplementInterfaceInBlueprint), MinimalAPI, BlueprintType)
class UCameraLensEffectInterface : public UInterface
{
	GENERATED_BODY()
};

class ICameraLensEffectInterface
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "CameraLensEffect")
	ENGINE_API virtual TArray<UFXSystemComponent*> GetParticleComponents() const;

	UFUNCTION(BlueprintCallable, Category = "CameraLensEffect")
	ENGINE_API virtual UFXSystemComponent* GetPrimaryParticleComponent() const;

	virtual const FTransform& GetRelativeTransform() const = 0;
	virtual float GetBaseFOV() const = 0;
	virtual bool ShouldAllowMultipleInstances() const = 0;
	virtual bool ResetWhenTriggered() const = 0;

	/** If this type needs to handle being pooled, override this function */
	virtual void NotifyWillBePooled() {};

	/** Is this emitter functionally equivalent to the class OtherEmitter? */
	virtual bool ShouldTreatEmitterAsSame(TSubclassOf<AActor> OtherEmitter) const = 0;

	/** Called when being added to the player camera manager. Typically happens before PreInitializeComponents (when handled from PlayerCameraManager). */
	virtual void RegisterCamera(APlayerCameraManager* CameraManager) = 0;
	
	/** Called when this emitter is re-triggered, for bAllowMultipleInstances=false emitters. */
	virtual void NotifyRetriggered() = 0;

	/** This will actually activate the lens Effect.  We want this separated from PostInitializeComponents so we can cache these emitters **/
	virtual void ActivateLensEffect() = 0;
	
	/** Deactivtes the particle system. If bDestroyOnSystemFinish is true, actor will die after particles are all dead. */
	virtual void DeactivateLensEffect() = 0;
	
	/** Given updated camera information, adjust this effect to display appropriately. */
	ENGINE_API virtual void UpdateLocation(const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg);

	virtual void AdjustBaseFOV(float NewFOV) = 0;

	static ENGINE_API FTransform GetAttachedEmitterTransform(const AActor* Emitter, const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg);

	/** Returns true if any associated particle system is set to looping */
	virtual bool IsLooping() const = 0;
};

//~ Engineering Note: You can validate assets to see if they have any TSubclassOf<AEmitterCameraLensEffectBase> [or other lens effect bases] and issue warnings where they do! Suggest this type instead!
/** Wrapper type for validation that the specfied subclass in fact does implement the desired interface! */
USTRUCT(BlueprintType)
struct FCameraLensInterfaceClassSupport
{
	GENERATED_BODY()

	/** The class to spawn/reference. Must implement CameraLensEffectInterface! */
	UPROPERTY(EditAnywhere, Category="Lens Effect", meta=(MustImplement= "/Script/Engine.CameraLensEffectInterface"))
	TSubclassOf<AActor> Class;
};

UENUM()
enum class EInterfaceValidResult : uint8 
{
	Valid,
	Invalid,
};

template <typename T>
class TScriptInterface;

UCLASS()
class UCameraLensEffectInterfaceClassSupportLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Returns the class represented by this lens effect wrapper... */
	UFUNCTION(BlueprintPure, Category="Lens Effect", meta=(BlueprintAutocast, CompactNodeTitle=".", Keywords="class, get, toclass, getclass, spawn, object"))
	static TSubclassOf<AActor> GetInterfaceClass(const FCameraLensInterfaceClassSupport& CameraLens);

	/** Check whether or not the interface class is valid */
	UFUNCTION(BlueprintCallable, Category="Lens Effect", meta=(ExpandEnumAsExecs=Result, Keywords = "valid, camera, real, lens", DisplayName="Is Valid Camera Lens Class"))
	static void IsInterfaceClassValid(const FCameraLensInterfaceClassSupport& CameraLens, EInterfaceValidResult& Result);
	
	/** Evaluate the live interface to see if it is a valid reference. */
	UFUNCTION(BlueprintCallable, Category="Lens Effect", meta=(ExpandEnumAsExecs=Result, Keywords = "valid, camera, real, lens", DisplayName="Is Valid Camera Lens"))
	static void IsInterfaceValid(const TScriptInterface<ICameraLensEffectInterface>& CameraLens, EInterfaceValidResult& Result);
	
	/** 
	 * Set the represented class of the passed in variable. Note: Check the tooltips on the individual pins.
	 * You cannot bypass the validation by connecting a wires to this node!!
	 * 
	 * @param Class MUST implement CameraLensEffectInterface - when connecting variables to the input, take care that the input class does in fact implement the interface.
	 * @param Var The wrapper (for validation purposes) of the lens effect class.
	 */
	UFUNCTION(BlueprintCallable, Category="Lens Effect", meta=(ExpandEnumAsExecs=Result))
	static void SetInterfaceClass(UPARAM(meta = (MustImplement = "CameraLensEffectInterface")) TSubclassOf<AActor> Class, UPARAM(Ref) FCameraLensInterfaceClassSupport& Var, EInterfaceValidResult& Result);
};
