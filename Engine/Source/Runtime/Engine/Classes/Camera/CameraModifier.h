// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Engine/Scene.h"
#include "CameraModifier.generated.h"

class AActor;
class APlayerCameraManager;
class FDebugDisplayInfo;

//=============================================================================
/**
 * A CameraModifier is a base class for objects that may adjust the final camera properties after
 * being computed by the APlayerCameraManager (@see ModifyCamera). A CameraModifier
 * can be stateful, and is associated uniquely with a specific APlayerCameraManager.
 */
UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UCameraModifier : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** If true, enables certain debug visualization features. */
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bDebug : 1;

	/** If true, no other modifiers of same priority allowed. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = CameraModifier)
	uint32 bExclusive : 1;

protected:
	/** If true, do not apply this modifier to the camera. */
	uint32 bDisabled:1;

	/** If true, this modifier will disable itself when finished interpolating out. */
	uint32 bPendingDisable:1;

public:
	/** Priority value that determines the order in which modifiers are applied. 0 = highest priority, 255 = lowest. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = CameraModifier)
	uint8 Priority;	
	
protected:
	/** Camera this object is associated with. */
	UPROPERTY(transient, BlueprintReadOnly, Category = CameraModifier)
	TObjectPtr<class APlayerCameraManager> CameraOwner;

	/** When blending in, alpha proceeds from 0 to 1 over this time */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = CameraModifier)
	float AlphaInTime;

	/** When blending out, alpha proceeds from 1 to 0 over this time */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = CameraModifier)
	float AlphaOutTime;

	/** Current blend alpha. */
	UPROPERTY(transient, BlueprintReadOnly, Category = CameraModifier)
	float Alpha;

	UFUNCTION()
	ENGINE_API void OnCameraOwnerDestroyed(AActor* InOwner);

protected:
	/** @return Returns the ideal blend alpha for this modifier. Interpolation will seek this value. */
	ENGINE_API virtual float GetTargetAlpha();

public:

	/**
	 * Allows camera modifiers to output debug text during "showdebug camera"
	 * @param Canvas - The canvas to write debug info on.
	 * @param DebugDisplay - Meta data about the current debug display.
	 * @param YL - Vertical spacing.
	 * @param YPos - Current vertical space to start writing.
	 */
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);

	/** 
	 * Allows any custom initialization. Called immediately after creation.
	 * @param Camera - The camera this modifier should be associated with.
	 */
	ENGINE_API virtual void AddedToCamera( APlayerCameraManager* Camera );
	
	/**
	 * Directly modifies variables in the owning camera
	 * @param	DeltaTime	Change in time since last update
	 * @param	InOutPOV	Current Point of View, to be updated.
	 * @return	bool		True if should STOP looping the chain, false otherwise
	 */
	ENGINE_API virtual bool ModifyCamera(float DeltaTime, struct FMinimalViewInfo& InOutPOV);

	/** 
	 * Called per tick that the modifier is active to allow Blueprinted modifiers to modify the camera's transform. 
	 * Scaling by Alpha happens after this in code, so no need to deal with that in the blueprint.
	 * @param	DeltaTime	Change in time since last update
	 * @param	ViewLocation		The current camera location.
	 * @param	ViewRotation		The current camera rotation.
	 * @param	FOV					The current camera fov.
	 * @param	NewViewLocation		(out) The modified camera location.
	 * @param	NewViewRotation		(out) The modified camera rotation.
	 * @param	NewFOV				(out) The modified camera FOV.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic)
	ENGINE_API void BlueprintModifyCamera(float DeltaTime, FVector ViewLocation, FRotator ViewRotation, float FOV, FVector& NewViewLocation, FRotator& NewViewRotation, float& NewFOV);

	/**
	 * Called per tick that the modifier is active to allow Blueprinted modifiers to modify the camera's postprocess effects.
	 * Scaling by Alpha happens after this in code, so no need to deal with that in the blueprint.
	 * @param	DeltaTime				Change in time since last update
	 * @param	PostProcessBlendWeight	(out) Blend weight applied to the entire postprocess structure.
	 * @param	PostProcessSettings		(out) Post process structure defining what settings and values to override.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic)
	ENGINE_API void BlueprintModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings);

	/** @return Returns true if modifier is disabled, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = CameraModifier)
	ENGINE_API virtual bool IsDisabled() const;
	
	/** @return Returns true if modifier is pending disable, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = CameraModifier)
	ENGINE_API virtual bool IsPendingDisable() const;
	
	/** @return Returns the actor the camera is currently viewing. */
	UFUNCTION(BlueprintCallable, Category = CameraModifier)
	ENGINE_API virtual AActor* GetViewTarget() const;

	/** 
	 *  Disables this modifier.
	 *  @param  bImmediate  - true to disable with no blend out, false (default) to allow blend out
	 */
	UFUNCTION(BlueprintCallable, Category=CameraModifier)
	ENGINE_API virtual void DisableModifier(bool bImmediate = false);

	/** Enables this modifier. */
	UFUNCTION(BlueprintCallable, Category = CameraModifier)
	ENGINE_API virtual void EnableModifier();

	/** Toggled disabled/enabled state of this modifier. */
	ENGINE_API virtual void ToggleModifier();
	
	/**
	 * Called to give modifiers a chance to adjust view rotation updates before they are applied.
	 *
	 * Default just returns ViewRotation unchanged
	 * @param ViewTarget - Current view target.
	 * @param DeltaTime - Frame time in seconds.
	 * @param OutViewRotation - In/out. The view rotation to modify.
	 * @param OutDeltaRot - In/out. How much the rotation changed this frame.
	 * @return Return true to prevent subsequent (lower priority) modifiers to further adjust rotation, false otherwise.
	 */
	ENGINE_API virtual bool ProcessViewRotation(class AActor* ViewTarget, float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot);

	/**
	 * Responsible for updating alpha blend value.
	 *
	 * @param	Camera		- Camera that is being updated
	 * @param	DeltaTime	- Amount of time since last update
	 */
	ENGINE_API virtual void UpdateAlpha(float DeltaTime);

	/** @return Returns the appropriate world context for this object. */
	ENGINE_API UWorld* GetWorld() const;

protected:
	/** Allows modifying the camera in native code. */
	ENGINE_API virtual void ModifyCamera(float DeltaTime, FVector ViewLocation, FRotator ViewRotation, float FOV, FVector& NewViewLocation, FRotator& NewViewRotation, float& NewFOV);

	/** Allows modifying the post process in native code. */
	ENGINE_API virtual void ModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings);
};
