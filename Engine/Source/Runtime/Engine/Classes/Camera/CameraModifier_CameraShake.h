// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Camera modifier that provides support for code-based oscillating camera shakes.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Camera/CameraTypes.h"
#include "Camera/CameraModifier.h"
#include "CameraModifier_CameraShake.generated.h"

class UCameraShakeBase;
class UCameraShakeSourceComponent;

/**
 * List of reusable camera shake instances that we store in a recycling pool.
 */
USTRUCT()
struct FPooledCameraShakes
{
	GENERATED_BODY()
		
	UPROPERTY()
	TArray<TObjectPtr<UCameraShakeBase>> PooledShakes;
};

/**
 * Information about an active camera shake.
 */
USTRUCT()
struct FActiveCameraShakeInfo
{
	GENERATED_BODY()

	FActiveCameraShakeInfo() 
		: ShakeInstance(nullptr)
		, ShakeSource(nullptr)
		, bIsCustomInitialized(false)
	{}

	/** The camera shake instance currently running */
	UPROPERTY()
	TObjectPtr<UCameraShakeBase> ShakeInstance;

	/** An optional source that the shake is running from (otherwise it's global and un-attenuated) */
	UPROPERTY()
	TWeakObjectPtr<const UCameraShakeSourceComponent> ShakeSource;

	/** Whether the shake was initialized with some custom callback code */
	UPROPERTY()
	bool bIsCustomInitialized;

#if UE_ENABLE_DEBUG_DRAWING
	/** Index of the debug graph display data for this shake */
	int32 DebugDataIndex = INDEX_NONE;
#endif
};

#if UE_ENABLE_DEBUG_DRAWING
/**
 * A data point for the debug graph display.
 */
struct FCameraShakeDebugDataPoint
{
	/** Time since start of the shake */
	float AccumulatedTime;
	/** Location movement of the shake for this frame */
	FVector DeltaLocation;
	/** Rotation movement of the shake for this frame */
	FRotator DeltaRotation;
};
#endif

#if UE_ENABLE_DEBUG_DRAWING
/** 
 * Information about active or recently ended camera shakes that we want to show in the 
 * debug graph display. Shakes stay alive a little longer in this struct (compared to the
 * ActiveShakes list) because after a shake ended we want to keep displaying its debug graph
 * for a little while instead of having it disappear immediately. We also draw again in
 * the same graph if the shake comes back soon enough.
 */
struct FCameraShakeDebugData
{
	/** The class of the shake that is/was running */
	TSubclassOf<UCameraShakeBase> ShakeClass;
	/** The name of the shake that is/was running */
	FString ShakeInstanceName;
	/** Data points for the debug graph display */
	TArray<FCameraShakeDebugDataPoint> DataPoints;
	/** How long ago did the shake end */
	float TimeInactive = 0.f;
	/** Is the shake active */
	bool bIsInactive = false;
};
#endif

DECLARE_DELEGATE_OneParam(FOnInitializeCameraShake, UCameraShakeBase*);

/** Parameter struct for adding new camera shakes to UCameraModifier_CameraShake */
struct FAddCameraShakeParams
{
	/** Global scale to use for the new camera shake */
	float Scale;
	/** The transform space to use for the new camera shake */
	ECameraShakePlaySpace PlaySpace;
	/** User space to use when PlaySpace is UserDefined */
	FRotator UserPlaySpaceRot;

	/** The source to use for dynamically attenuating the new camera shake */
	const UCameraShakeSourceComponent* SourceComponent;

	/**
	 * A custom callback to initialize the new camera shake.
	 *
	 * Note that when this is set, the given camera shake will not be recycled. It will be discarded
	 * completely upon finishing.
	 */
	FOnInitializeCameraShake Initializer;

	/**
	 * An optional override for the duration of the camera shake.
	 */
	TOptional<float> DurationOverride;

	FAddCameraShakeParams()
		: Scale(1.f), PlaySpace(ECameraShakePlaySpace::CameraLocal), UserPlaySpaceRot(FRotator::ZeroRotator), SourceComponent(nullptr)
	{}
	FAddCameraShakeParams(float InScale, ECameraShakePlaySpace InPlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator InUserPlaySpaceRot = FRotator::ZeroRotator, const UCameraShakeSourceComponent* InSourceComponent = nullptr)
		: Scale(InScale), PlaySpace(InPlaySpace), UserPlaySpaceRot(InUserPlaySpaceRot), SourceComponent(InSourceComponent)
	{}
};

//~=============================================================================
/**
 * A UCameraModifier_CameraShake is a camera modifier that can apply a UCameraShakeBase to 
 * the owning camera.
 */
UCLASS(config=Camera, MinimalAPI)
class UCameraModifier_CameraShake : public UCameraModifier
{
	GENERATED_BODY()

public:
	ENGINE_API UCameraModifier_CameraShake(const FObjectInitializer& ObjectInitializer);

	ENGINE_API virtual void BeginDestroy() override;

	/** 
	 * Adds a new active screen shake to be applied. 
	 * @param NewShake - The class of camera shake to instantiate.
	 * @param Params - The parameters for the new camera shake.
	 */
	ENGINE_API virtual UCameraShakeBase* AddCameraShake(TSubclassOf<UCameraShakeBase> NewShake, const FAddCameraShakeParams& Params);

	/**
	 * Returns a list of currently active camera shakes.
	 * @param ActiveCameraShakes - The array to fill up with shake information.
	 */
	ENGINE_API virtual void GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const;
	
	/**
	 * Stops and removes the camera shake of the given class from the camera.
	 * @param Shake - the camera shake class to remove.
	 * @param bImmediately		If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	ENGINE_API virtual void RemoveCameraShake(UCameraShakeBase* ShakeInst, bool bImmediately = true);

	/**
	 * Stops and removes all camera shakes of the given class from the camera. 
	 * @param bImmediately		If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	ENGINE_API virtual void RemoveAllCameraShakesOfClass(TSubclassOf<UCameraShakeBase> ShakeClass, bool bImmediately = true);

	/**
	 * Stops and removes all camera shakes originating from the given source.
	 * @param SourceComponent   The shake source.
	 * @param bImmediately      If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	ENGINE_API virtual void RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent, bool bImmediately = true);

	/**
	 * Stops and removes all camera shakes of the given class originating from the given source.
	 * @param ShakeClasss       The camera shake class to remove.
	 * @param SourceComponent   The shake source.
	 * @param bImmediately      If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	ENGINE_API virtual void RemoveAllCameraShakesOfClassFromSource(TSubclassOf<UCameraShakeBase> ShakeClass, const UCameraShakeSourceComponent* SourceComponent, bool bImmediately = true);

	/** 
	 * Stops and removes all camera shakes from the camera. 
	 * @param bImmediately		If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	ENGINE_API virtual void RemoveAllCameraShakes(bool bImmediately = true);
	
	//~ Begin UCameraModifer Interface
	ENGINE_API virtual bool ModifyCamera(float DeltaTime, struct FMinimalViewInfo& InOutPOV) override;
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	//~ End UCameraModifer Interface

protected:

	/** List of active CameraShake instances */
	UPROPERTY()
	TArray<FActiveCameraShakeInfo> ActiveShakes;

	UPROPERTY()
	TMap<TSubclassOf<UCameraShakeBase>, FPooledCameraShakes> ExpiredPooledShakesMap;

	ENGINE_API void SaveShakeInExpiredPoolIfPossible(const FActiveCameraShakeInfo& ShakeInfo);
	ENGINE_API void SaveShakeInExpiredPool(UCameraShakeBase* ShakeInst);
	ENGINE_API UCameraShakeBase* ReclaimShakeFromExpiredPool(TSubclassOf<UCameraShakeBase> CameraShakeClass);

	/** Scaling factor applied to all camera shakes in when in splitscreen mode. Normally used to reduce shaking, since shakes feel more intense in a smaller viewport. */
	UPROPERTY(EditAnywhere, Category = CameraModifier_CameraShake)
	float SplitScreenShakeScale;

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API void AddCameraShakeDebugData(FActiveCameraShakeInfo& ShakeInfo);
	ENGINE_API void RemoveCameraShakeDebugData(const FActiveCameraShakeInfo& ShakeInfo);
	ENGINE_API void DisplayDebugGraphs(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay);

	/** List of debug data for camera shakes */
	TArray<FCameraShakeDebugData> DebugShakes;
	/** Whether we are currently recording camera shake debug info */
	bool bRecordDebugData;
#endif

private:
	void OnPreGarbageCollect();
	void RemoveInvalidObjectsFromExpiredPool();
};

