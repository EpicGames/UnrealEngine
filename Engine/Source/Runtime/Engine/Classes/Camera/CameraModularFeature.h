// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Camera/CameraModifier.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"

/**
* A modular feature interface for cameras.
*/
class ICameraModularFeature : public IModularFeature
{
public:
	/**
	 * Gets the name of the camera module
	 */
	static ENGINE_API FName GetModularFeatureName();

	/**
	 * Get the list of registered default camera modifiers.
	 *
	 * The implementation should just add its modifier classes to the provided array, leaving any previously gathered
	 * modifiers from other modular feature implementers.
	 *
	 * @param ModifierClasses  The list of registered camera modifier classes.
	 */
	virtual void GetDefaultModifiers(TArray<TSubclassOf<UCameraModifier>>& ModifierClasses) const = 0;
};

