// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UWorld;
class APlayerCameraManager;
class ICameraPhotography;
struct FMinimalViewInfo;
struct FPostProcessSettings;

/**
 * Free-camera photography manager
 */
class APlayerCameraManager;

class FCameraPhotographyManager
{
public:
	/** Get (& possibly create) singleton FCameraPhotography */
	ENGINE_API static FCameraPhotographyManager& Get();
	/** Destroy current FCameraPhotography (if any); recreated by next Get() */
	ENGINE_API static void Destroy();
	/** @return Returns false if definitely unavailable at compile-time or run-time */
	ENGINE_API static bool IsSupported(UWorld* InWorld);

	/** Modify input camera according to cumulative free-camera transforms (if any).
	* Safe to call this even if IsSupported()==false, in which case it will leave camera
	* unchanged and return false.
	* @param InOutPOV - camera info to modify
	* @param PCMgr - player camera manager (non-NULL)
	* @return Returns whether camera was cut/non-contiguous/teleported */
	ENGINE_API bool UpdateCamera(FMinimalViewInfo& InOutPOV, APlayerCameraManager* PCMgr);

	/** Modify input postprocessing settings according to Photography requirements.
	* Safe to call this even if IsSupported()==false, in which case it will do nothing.
	* @param InOutPostProcessingSettings - the FPostProcessSettings to modify */
	ENGINE_API void UpdatePostProcessing(FPostProcessSettings& InOutPostProcessingSettings);

	/** Starts a photography session */
	ENGINE_API void StartSession();

	/** Stops a photography session */
	ENGINE_API void StopSession();

	/** Show or hide controls in the photography UI; see documentation for UAnselFunctionLibrary::SetUIControlVisibility */
	ENGINE_API void SetUIControlVisibility(uint8 UIControlTarget, bool bIsVisible);

	ENGINE_API void DefaultConstrainCamera(const FVector NewCameraLocation, const FVector PreviousCameraLocation, const FVector OriginalCameraLocation, FVector& OutCameraLocation, APlayerCameraManager* PCMgr);

protected:
	FCameraPhotographyManager();
	~FCameraPhotographyManager();
	static class FCameraPhotographyManager* Singleton;

	TSharedPtr<ICameraPhotography> ActiveImpl;
};


