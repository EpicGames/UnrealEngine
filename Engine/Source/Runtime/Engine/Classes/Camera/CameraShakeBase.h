// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Camera/CameraTypes.h"
#include "Engine/Scene.h"

#include "CameraShakeBase.generated.h"

class APlayerCameraManager;
class UCameraShakePattern;

/**
 * Parameters for starting a camera shake pattern.
 */
USTRUCT(BlueprintType)
struct FCameraShakePatternStartParams
{
	GENERATED_BODY()
	
	/** Whether the camera shake is restarting while playing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	bool bIsRestarting = false;

	/** Whether the camera shake's duration is overriden (see DurationOverride) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	bool bOverrideDuration = false;

	/** An optional override for the camera shake's duration */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake, meta=(EditCondition="bOverrideDuration"))
	float DurationOverride = 0.f;
};

/**
 * Parameters for updating a camera shake pattern.
 */
USTRUCT(BlueprintType)
struct FCameraShakePatternUpdateParams
{
	GENERATED_BODY()

	FCameraShakePatternUpdateParams()
	{}

	FCameraShakePatternUpdateParams(const FMinimalViewInfo& InPOV)
		: POV(InPOV)
	{}

	/** The time elapsed since last update */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float DeltaTime = 0.f;

	/** The base scale for this shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float ShakeScale = 1.f;
	/** The dynamic scale being passed down from the camera manger for the next update */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float DynamicScale = 1.f;

	/** The current view that this camera shake should modify */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	FMinimalViewInfo POV;

	/** The total scale to apply to the camera shake during the current update. Equals ShakeScale * DynamicScale */
	float GetTotalScale() const
	{
		return FMath::Max(ShakeScale * DynamicScale, 0.f);
	}
};

/**
 * Parameters for scrubbing a camera shake.
 */
USTRUCT(BlueprintType)
struct FCameraShakePatternScrubParams
{
	GENERATED_BODY()

	FCameraShakePatternScrubParams()
	{}

	FCameraShakePatternScrubParams(const FMinimalViewInfo& InPOV)
		: POV(InPOV)
	{}

	/** Convert this to an update parameter struct where the delta time is from 0 to the AbsoluteTime. */
	ENGINE_API FCameraShakePatternUpdateParams ToUpdateParams() const;

	/** The time to scrub to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float AbsoluteTime = 0.f;

	/** The base scale for this shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float ShakeScale = 1.f;
	/** The dynamic scale being passed down from the camera manger for the next update */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	float DynamicScale = 1.f;

	/** The current view that this camera shake should modify */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	FMinimalViewInfo POV;

	/** The total scale to apply to the camera shake during the current update. Equals ShakeScale * DynamicScale */
	float GetTotalScale() const
	{
		return FMath::Max(ShakeScale * DynamicScale, 0.f);
	}
};

/**
 * Flags that camera shake patterns can return to change base-class behaviour.
 */
UENUM()
enum class ECameraShakePatternUpdateResultFlags : uint8
{
	/** Apply the result location, rotation, and field of view as absolute values, instead of additive values. */
	ApplyAsAbsolute = 1 << 0,
	/** Do not apply scaling (dynamic scale, blending weight, shake scale), meaning that this will be done in the sub-class. Implied when ApplyAsAbsolute is set. */
	SkipAutoScale = 1 << 1,
	/** Do not re-orient the result based on the play-space. Implied when ApplyAsAbsolute is set. */
	SkipAutoPlaySpace = 1 << 2,

	/** Default flags: the sub-class is returning local, additive offsets, and lets the base class take care of the rest. */
	Default = 0
};
ENUM_CLASS_FLAGS(ECameraShakePatternUpdateResultFlags);

/**
 * The result of a camera shake pattern update.
 */
USTRUCT(BlueprintType)
struct FCameraShakePatternUpdateResult
{
	GENERATED_BODY()

	FCameraShakePatternUpdateResult()
		: Location(FVector::ZeroVector)
		, Rotation(FRotator::ZeroRotator)
		, FOV(0.f)
		, PostProcessBlendWeight(0.f)
		, Flags(ECameraShakePatternUpdateResultFlags::Default)
	{}

	/** Location offset for the view, or new absolute location if ApplyAsAbsolute flag is set */
	FVector Location;
	/** Rotation offset for the view, or new absolute rotation if ApplyAsAbsolute flag is set */
	FRotator Rotation;
	/** Field-of-view offset for the view, or new absolute field-of-view if ApplyAsAbsolute flag is set */
	float FOV;

	/** Post process settings, applied if PostProcessBlendWeight is above 0 */
	FPostProcessSettings PostProcessSettings;
	/** Blend weight for post process settings */
	float PostProcessBlendWeight;

	/** Flags for how the base class should handle the result */
	ECameraShakePatternUpdateResultFlags Flags;

	/** Apply the given scale to the result (only if it is "relative") */
	ENGINE_API void ApplyScale(float InScale);
};

/**
 * Parameters for stopping a camera shake.
 */
USTRUCT(BlueprintType)
struct FCameraShakePatternStopParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraShake)
	bool bImmediately = false;
};

// Redirections from old names.
#if !defined(UE_LEGACY_CAMERA_SHAKE_PATTERN_TYPES)
#define UE_LEGACY_CAMERA_SHAKE_PATTERN_TYPES 0
#endif

#if UE_LEGACY_CAMERA_SHAKE_PATTERN_TYPES
using ECameraShakeUpdateResultFlags = ECameraShakePatternUpdateResultFlags;
using FCameraShakeStartParams = FCameraShakePatternStartParams;
using FCameraShakeUpdateParams = FCameraShakePatternUpdateParams;
using FCameraShakeScrubParams = FCameraShakePatternScrubParams;
using FCameraShakeStopParams = FCameraShakePatternStopParams;
using FCameraShakeUpdateResult = FCameraShakePatternUpdateResult;
#endif // UE_LEGACY_CAMERA_SHAKE_PATTERN_TYPES

/**
 * Camera shake duration type.
 */
UENUM()
enum class ECameraShakeDurationType : uint8
{
	/** Camera shake has a fixed duration */
	Fixed,
	/** Camera shake is playing indefinitely, until explicitly stopped */
	Infinite,
	/** Camera shake has custom/dynamic duration */
	Custom
};

/**
 * Camera shake duration.
 */
USTRUCT(BlueprintType)
struct FCameraShakeDuration
{
	GENERATED_BODY()

	/** Returns an infinite shake duration */
	static FCameraShakeDuration Infinite() { return FCameraShakeDuration { 0.f, ECameraShakeDurationType::Infinite }; }
	/** Returns a custom shake duration */
	static FCameraShakeDuration Custom() { return FCameraShakeDuration { 0.f, ECameraShakeDurationType::Custom }; }
	/** Returns a custom shake duration with a hint for how long the shake might be */
	static FCameraShakeDuration Custom(float DurationHint) { return FCameraShakeDuration{ DurationHint, ECameraShakeDurationType::Custom }; }

	/** Creates a new shake duration */
	FCameraShakeDuration() : Duration(0.f), Type(ECameraShakeDurationType::Fixed) {}
	/** Creates a new shake duration */
	FCameraShakeDuration(float InDuration, ECameraShakeDurationType InType = ECameraShakeDurationType::Fixed) : Duration(InDuration), Type(InType) {}
	
	/** Returns the duration type */
	ECameraShakeDurationType GetDurationType() const { return Type; }
	/** Returns whether this duration is a fixed time */
	bool IsFixed() const { return Type == ECameraShakeDurationType::Fixed; }
	/** Returns whether this duration is infinite */
	bool IsInfinite() const { return Type == ECameraShakeDurationType::Infinite; }
	/** Returns whether this duration is custom */
	bool IsCustom() const { return Type == ECameraShakeDurationType::Custom; }
	/** Returns whether this duration is custom, but with a valid effective duration hint */
	bool IsCustomWithHint() const { return IsCustom() && Duration > 0.f; }

	/** When the duration is fixed, return the duration time */
	float Get() const { check(Type == ECameraShakeDurationType::Fixed || Type == ECameraShakeDurationType::Custom); return Duration; }

private:
	UPROPERTY()
	float Duration;

	UPROPERTY()
	ECameraShakeDurationType Type;
};

/**
 * Information about a camera shake class.
 */
USTRUCT(BlueprintType)
struct FCameraShakeInfo
{
	GENERATED_BODY()

	/** The duration of the camera shake */
	UPROPERTY()
	FCameraShakeDuration Duration;

	/** How much blending-in the camera shake should have */
	UPROPERTY()
	float BlendIn = 0.f;

	/** How much blending-out the camera shake should have */
	UPROPERTY()
	float BlendOut = 0.f;
};

/**
 * Parameter structure for appling scale and playspace to a camera shake result.
 */
struct FCameraShakeApplyResultParams
{
	/** The scale to apply to the result */
	float Scale = 1.f;
	/** The play space to modify the result by */
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;
	/** The custom space to use when PlaySpace is UserDefined */
	FMatrix UserPlaySpaceMatrix;

	/** Optional camera manager for queuing up blended post-process settings */
	TWeakObjectPtr<APlayerCameraManager> CameraManager;
};

/**
 * Transitive state of a shake or shake pattern.
 */
struct FCameraShakeState
{
	/**
	 * Create a new camera shake state
	 */
	ENGINE_API FCameraShakeState();

	/**
	 * Initialize the state with a shake's info and start playing.
	 */
	ENGINE_API void Start(const FCameraShakeInfo& InShakeInfo);

	/**
	 * Initialize the state with a shake's info and start playing.
	 */
	ENGINE_API void Start(const FCameraShakeInfo& InShakeInfo, TOptional<float> InDurationOverride);

	/**
	 * Initialize the state with a shake's info and start playing.
	 */
	ENGINE_API void Start(const UCameraShakePattern* InShakePattern);
	
	/**
	 * Initialize the state with a shake's info and start playing.
	 */
	ENGINE_API void Start(const UCameraShakePattern* InShakePattern, const FCameraShakePatternStartParams& InParams);

	/**
	 * Updates the state with a delta time.
	 *
	 * If the state isn't managed (i.e. it doesn't have any fixed duration information), this doesn't
	 * do anything and returns 1.
	 *
	 * If the state is managed, this updates the internal state of this class and auto-computes when
	 * the shake has ended, along with a blending weight if there is any blending in/out.
	 *
	 * @param DeltaTime The elapsed time since last update
	 * @return The evaluated blending weight (if any) for the new time
	 */
	ENGINE_API float Update(float DeltaTime);

	/**
	 * Scrub the state to the given absolute time.
	 *
	 * If the state isn't managed (i.e. it doesn't have any fixed duration information), this doesn't
	 * do anything and returns 1.
	 *
	 * If the state is managed, this updates the internal state of this class and auto-computes when
	 * the shake has ended, along with a blending weight if there is any blending in/out.
	 *
	 * @param AbsoluteTime The time to scrub to
	 * @return The evaluated blending weight (if any) for the scrub time
	 */
	ENGINE_API float Scrub(float AbsoluteTime);

	/**
	 * Marks the shake has having been stopped.
	 *
	 * This renders the shake inactive (if we need to stop immediately), or starts the shake's blend-out,
	 * if any (if we don't stop immediately). If no duration or blending information is available (i.e. if
	 * the shake duration is "Custom"), stopping non-immediately does nothing: the sub-class is expected
	 * to handle it.
	 */
	ENGINE_API void Stop(bool bImmediately);

	/** Returns whether the shake is playing */
	bool IsPlaying() const { return bIsPlaying; }

	/** Returns whether the shake is blending in */
	bool IsBlendingIn() const { return bIsBlendingIn; }

	/** Returns whether the shake is blending out */
	bool IsBlendingOut() const { return bIsBlendingOut; }

	/** Returns the elapsed time of the shake's current run */
	float GetElapsedTime() const { return ElapsedTime; }

	/** Returns the current time into the blend in (only valid if IsBlendingIn() returns true) */
	float GetCurrentBlendInTime() const { return CurrentBlendInTime; }

	/** Returns the current time into the blend in (only valid if IsBlendingOut() returns true) */
	float GetCurrentBlendOutTime() const { return CurrentBlendOutTime; }

	/** Returns the current shake info */
	const FCameraShakeInfo& GetShakeInfo() const { return ShakeInfo; }

	/** Helper method to get GetShakeInfo().Duration.IsFixed() */
	bool HasDuration() const { return ShakeInfo.Duration.IsFixed(); }

	/** Helper method to get GetShakeInfo().Duration.Get() */
	float GetDuration() const { return ShakeInfo.Duration.Get(); }

	/** Helper method to get GetShakeInfo().Duration.IsInifnite() */
	bool IsInfinite() const { return ShakeInfo.Duration.IsInfinite(); }

private:

	ENGINE_API void InitializePlaying();

private:

	// Information about the shake/shake pattern we're managing
	FCameraShakeInfo ShakeInfo;

	// Running state
	float ElapsedTime;

	float CurrentBlendInTime;
	float CurrentBlendOutTime;

	bool bIsBlendingIn : 1;
	bool bIsBlendingOut : 1;

	bool bIsPlaying : 1;

	// Cached values for blending information
	bool bHasBlendIn : 1;
	bool bHasBlendOut : 1;
};

/**
 * Parameter struct for starting a camera shake.
 */
struct FCameraShakeBaseStartParams
{
	/** The parent camera manager */
	TObjectPtr<APlayerCameraManager> CameraManager;

	/** The scale for playing the shake */
	float Scale = 1.f;

	/** The coordinate system in which to play the shake */
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;

	/** A custom rotation, only used if PlaySpace is UserDefined */
	FRotator UserPlaySpaceRot = FRotator::ZeroRotator;

	/** An optional override for the camera shake's duration */
	TOptional<float> DurationOverride;
};

/**
 * Base class for a camera shake. A camera shake contains a root shake "pattern" which is
 * the object that contains the actual logic driving how the camera is shaken. Keeping the two
 * separate makes it possible to completely change how a shake works without having to create
 * a completely different asset.
 */
UCLASS(Abstract, Blueprintable, EditInlineNew, MinimalAPI)
class UCameraShakeBase : public UObject
{
	GENERATED_BODY()

public:

	/** Create a new instance of a camera shake */
	ENGINE_API UCameraShakeBase(const FObjectInitializer& ObjectInitializer);
	
public:

	/**
	 * Gets the duration of this camera shake in seconds.
	 *
	 * The value could be 0 or negative if the shake uses the oscillator, meaning, respectively,
	 * no oscillation, or indefinite oscillation.
	 */
	ENGINE_API FCameraShakeDuration GetCameraShakeDuration() const;

	/**
	 * Gets the duration of this camera shake's blend in and out.
	 *
	 * The values could be 0 or negative if there's no blend in and/or out.
	 */
	ENGINE_API void GetCameraShakeBlendTimes(float& OutBlendIn, float& OutBlendOut) const;

	/**
	 * Gets the default duration for camera shakes of the given class.
	 *
	 * @param CameraShakeClass    The class of camera shake
	 * @param OutDuration         Will store the default duration of the given camera shake class, if possible
	 * @return                    Whether a valid default duration was found
	 */
	static bool GetCameraShakeDuration(TSubclassOf<UCameraShakeBase> CameraShakeClass, FCameraShakeDuration& OutDuration)
	{
		if (CameraShakeClass)
		{
			if (const UCameraShakeBase* CDO = CameraShakeClass->GetDefaultObject<UCameraShakeBase>())
			{
				OutDuration = CDO->GetCameraShakeDuration();
				return true;
			}
		}
		return false;
	}

	/**
	 * Gets the default blend in/out durations for camera shakes of the given class.
	 *
	 * @param CameraShakeClass    The class of camera shake
	 * @param OutBlendIn          Will store the default blend-in time of the given camera shake class, if possible
	 * @param OutBlendOut         Will store the default blend-out time of the given camera shake class, if possible
	 * @return                    Whether valid default blend in/out times were found
	 */
	static bool GetCameraShakeBlendTimes(TSubclassOf<UCameraShakeBase> CameraShakeClass, float& OutBlendIn, float& OutBlendOut)
	{
		if (CameraShakeClass)
		{
			if (const UCameraShakeBase* CDO = CameraShakeClass->GetDefaultObject<UCameraShakeBase>())
			{
				CDO->GetCameraShakeBlendTimes(OutBlendIn, OutBlendOut);
				return true;
			}
		}
		return false;
	}

	/** Uses the given result parameters to apply the given result to the given input view info */
	static ENGINE_API void ApplyResult(const FCameraShakeApplyResultParams& ApplyParams, const FCameraShakePatternUpdateResult& InResult, FMinimalViewInfo& InOutPOV);

	/** Applies all the appropriate auto-scaling to the current shake offset (only if the result is "relative") */
	static ENGINE_API void ApplyScale(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& InOutResult);

	/** Applies the given scale to the current shake offset (only if the result is "relative") */
	static ENGINE_API void ApplyScale(float Scale, FCameraShakePatternUpdateResult& InOutResult);

	/** Applies any appropriate system-wide limits */
	static ENGINE_API void ApplyLimits(const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& InOutResult);

	/**
	 * Modifies the current shake offset to be oriented in the current shake's play space (only if the result is "relative")
	 *
	 * Note that this modifies the result and makes it "absolute".
	 */
	static ENGINE_API void ApplyPlaySpace(ECameraShakePlaySpace PlaySpace, FMatrix UserPlaySpaceMatrix, const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& InOutResult);

public:

	/** 
	 *  If true to only allow a single instance of this shake class to play at any given time.
	 *  Subsequent attempts to play this shake will simply restart the timer.
	 */
	UPROPERTY(EditAnywhere, Category=CameraShake)
	bool bSingleInstance;

	/** The overall scale to apply to the shake. Only valid when the shake is active. */
	UPROPERTY(transient, BlueprintReadWrite, Category=CameraShake)
	float ShakeScale;

	/** Gets the root pattern of this camera shake */
	UFUNCTION(BlueprintPure, Category="CameraShake")
	UCameraShakePattern* GetRootShakePattern() const { return RootShakePattern; }

	/** Sets the root pattern of this camera shake */
	UFUNCTION(BlueprintCallable, Category="CameraShake")
	ENGINE_API void SetRootShakePattern(UCameraShakePattern* InPattern);

	/** Creates a new pattern of the given type and sets it as the root one on this shake */
	template<typename ShakePatternType>
	ShakePatternType* ChangeRootShakePattern()
	{
		ShakePatternType* ShakePattern = NewObject<ShakePatternType>(this);
		SetRootShakePattern(ShakePattern);
		return ShakePattern;
	}

public:

	/** Gets some infromation about this specific camera shake */
	ENGINE_API void GetShakeInfo(FCameraShakeInfo& OutInfo) const;

	/**
	 * Returns whether this shake is active.
	 *
	 * A camera shake is active between the calls to StartShake and TeardownShake.
	 */
	bool IsActive() const { return bIsActive; }

	/** Starts this camera shake with the given parameters */
	ENGINE_API void StartShake(APlayerCameraManager* Camera, float Scale, ECameraShakePlaySpace InPlaySpace, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/** Starts this camera shake with the given parameters */
	ENGINE_API void StartShake(const FCameraShakeBaseStartParams& Params);

	/** Returns whether this camera shake is finished */
	ENGINE_API bool IsFinished() const;

	/** Updates this camera shake and applies its effect to the given view */
	ENGINE_API void UpdateAndApplyCameraShake(float DeltaTime, float Alpha, FMinimalViewInfo& InOutPOV);

	/** Scrubs this camera shake to the given time and applies its effect to the given view */
	ENGINE_API void ScrubAndApplyCameraShake(float AbsoluteTime, float Alpha, FMinimalViewInfo& InOutPOV);

	/** Stops this camera shake */
	ENGINE_API void StopShake(bool bImmediately = true);

	/** Tears down this camera shake before destruction or recycling */
	ENGINE_API void TeardownShake();

public:

	/** Gets the current camera manager. Will be null if the shake isn't active. */
	APlayerCameraManager* GetCameraManager() const { return CameraManager; }

	/** Returns the current play space. The value is irrelevant if the shake isn't active. */
	ECameraShakePlaySpace GetPlaySpace() const { return PlaySpace; }
	/** Returns the current play space matrix. The value is irrelevant if the shake isn't active, or if its play space isn't UserDefined. */
	const FMatrix& GetUserPlaySpaceMatrix() const { return UserPlaySpaceMatrix; }
	/** Sets the current play space matrix. This method has no effect if the shake isn't active, or if its play space isn't UserDefined. */
	void SetUserPlaySpaceMatrix(const FMatrix& InMatrix) { UserPlaySpaceMatrix = InMatrix; }

protected:

	/**
	 * Modifies the current shake offset to be oriented in the current shake's play space (only if the result is "relative")
	 *
	 * Note that this modifies the result and makes it "absolute".
	 */
	ENGINE_API void ApplyPlaySpace(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& InOutResult) const;

private:

	/** The root pattern for this camera shake */
	UPROPERTY(EditAnywhere, Instanced, Category=CameraShakePattern)
	TObjectPtr<UCameraShakePattern> RootShakePattern;

	/** The camera manager owning this camera shake. Only valid when the shake is active. */
	UPROPERTY(transient)
	TObjectPtr<APlayerCameraManager> CameraManager;

	/** What space to play the shake in before applying to the camera. Only valid when the shake is active. */
	ECameraShakePlaySpace PlaySpace;

	/** Matrix defining a custom play space, used when PlaySpace is UserDefined. Only valid when the shake is active. */
	FMatrix UserPlaySpaceMatrix;

	/** Whether this shake is active, which is true between StartShake and TeardownShake */
	bool bIsActive;
};

/**
 * A shake "pattern" defines how a camera should be effectively shaken. Examples of shake patterns
 * are sinewave oscillation, perlin noise, or FBX animation.
 *
 */
UCLASS(Abstract, EditInlineNew, MinimalAPI)
class UCameraShakePattern : public UObject
{
	GENERATED_BODY()

public:

	/** Constructor for a shake pattern */
	ENGINE_API UCameraShakePattern(const FObjectInitializer& ObjectInitializer);

	/** Gets information about this shake pattern */
	ENGINE_API void GetShakePatternInfo(FCameraShakeInfo& OutInfo) const;
	/** Called when the shake pattern starts */
	ENGINE_API void StartShakePattern(const FCameraShakePatternStartParams& Params);
	/** Updates the shake pattern, which should add its generated offset to the given result */
	ENGINE_API void UpdateShakePattern(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult);
	/** Scrubs the shake pattern to the given time, and apply the generated offset to the given result */
	ENGINE_API void ScrubShakePattern(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult);
	/** Returns whether this shake pattern is finished */
	ENGINE_API bool IsFinished() const;
	/** Called when the shake pattern is manually stopped */
	ENGINE_API void StopShakePattern(const FCameraShakePatternStopParams& Params);
	/** Call when the shake pattern is discard, either after naturally finishing or being stopped manually */
	ENGINE_API void TeardownShakePattern();

protected:

	/** Gets the shake pattern's parent shake */
	ENGINE_API UCameraShakeBase* GetShakeInstance() const;

	/** Gets the shake pattern's parent shake */
	template<typename InstanceType>
	InstanceType* GetShakeInstance() const { return Cast<InstanceType>(GetShakeInstance()); }

private:

	// UCameraShakePattern interface
	virtual void GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const {}
	virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) {}
	virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) {}
	virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) {}
	virtual bool IsFinishedImpl() const { return true; }
	virtual void StopShakePatternImpl(const FCameraShakePatternStopParams& Params) {}
	virtual void TeardownShakePatternImpl()  {}
};

