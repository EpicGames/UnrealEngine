// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"

#include "CameraShakeSourceActor.generated.h"


class UCameraShakeSourceComponent;
class USceneComponent;
class UWorld;


UCLASS(MinimalAPI)
class ACameraShakeSourceActor : public AActor
{
	GENERATED_BODY()

public:
	ENGINE_API ACameraShakeSourceActor(const FObjectInitializer& ObjectInitializer);

	UCameraShakeSourceComponent* GetCameraShakeSourceComponent() const { return CameraShakeSourceComponent; }

private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = CameraShakeSourceActor, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UCameraShakeSourceComponent> CameraShakeSourceComponent;
};
