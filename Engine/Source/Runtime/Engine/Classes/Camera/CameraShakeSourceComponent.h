// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Camera/CameraTypes.h"
#include "Components/SceneComponent.h"
#include "UObject/ObjectMacros.h"
#include "CameraShakeSourceComponent.generated.h"

class UCameraShakeBase;
class UTexture2D;

UENUM(BlueprintType)
enum class ECameraShakeAttenuation : uint8
{
	Linear,
	Quadratic
};

struct FCameraShakeSourceComponentStartParams
{
	/** The type of camera shake to create */
	TSubclassOf<UCameraShakeBase> ShakeClass;

	/* The scale for playing the shake */
	float Scale = 1.f;

	/** The coordinate system in which to play the shake */
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;

	/** A custom rotation, only used if PlaySpace is UserDefined */
	FRotator UserPlaySpaceRot = FRotator::ZeroRotator;

	/** An optional override for the camera shake's duration */
	TOptional<float> DurationOverride;
};

UCLASS(Blueprintable, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UCameraShakeSourceComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	ENGINE_API UCameraShakeSourceComponent(const FObjectInitializer& ObjectInitializer);

	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	ENGINE_API virtual void OnRegister() override;

#if WITH_EDITOR
    ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	void UpdateEditorSpriteTexture();

#if WITH_EDITOR
	TSubclassOf<UCameraShakeBase> PreviousCameraShake;
#endif

public:
	/** Starts a new camera shake originating from this source, and apply it on all player controllers */
	ENGINE_API void StartCameraShake(const FCameraShakeSourceComponentStartParams& Params);

public:
	/** The attenuation profile for how camera shakes' intensity falls off with distance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	ECameraShakeAttenuation Attenuation;

	/** Under this distance from the source, the camera shakes are at full intensity */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	float InnerAttenuationRadius;

	/** Outside of this distance from the source, the camera shakes don't apply at all */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	float OuterAttenuationRadius;

	/** The camera shake class to use for this camera shake source actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraShake)
	TSubclassOf<UCameraShakeBase> CameraShake;

	/** Whether to auto start when created */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CameraShake)
	bool bAutoStart;

#if WITH_EDITORONLY_DATA
	/** Sprite to display in the editor. */
	UPROPERTY(transient)
	TObjectPtr<UTexture2D> EditorSpriteTexture;

	/** Sprite scaling for display in the editor. */
	UPROPERTY(transient)
	float EditorSpriteTextureScale;
#endif

public:
	UFUNCTION(BlueprintCallable, Category = CameraShake)
	ENGINE_API void Start();

	/** Starts a new camera shake originating from this source, and apply it on all player controllers */
	UFUNCTION(BlueprintCallable, Category = CameraShake)
	ENGINE_API void StartCameraShake(TSubclassOf<UCameraShakeBase> InCameraShake, float Scale=1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/** Stops a camera shake originating from this source */
	UFUNCTION(BlueprintCallable, Category = CameraShake)
	ENGINE_API void StopAllCameraShakesOfType(TSubclassOf<UCameraShakeBase> InCameraShake, bool bImmediately = true);

	/** Stops all currently active camera shakes that are originating from this source from all player controllers */
	UFUNCTION(BlueprintCallable, Category = CameraShake)
	ENGINE_API void StopAllCameraShakes(bool bImmediately = true);

	/** Computes an attenuation factor from this source */
	UFUNCTION(BlueprintCallable, Category = CameraShake)
	ENGINE_API float GetAttenuationFactor(const FVector& Location) const;
};
