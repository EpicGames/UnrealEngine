// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "CameraStackTypes.generated.h"

class APlayerCameraManager;

// NOTE:
// This code is work in progress - do not use these types until this comment is removed.

USTRUCT()
struct FDummySpacerCameraTypes
{
	GENERATED_USTRUCT_BODY()
};


//@TODO: Document
UENUM()
namespace ECameraAlphaBlendMode
{
	enum Type : int
	{
		CABM_Linear UMETA(DisplayName="Linear"),
		CABM_Cubic UMETA(DisplayName="Cubic")
	};
}

// Used to update camera entries in the camera stack
struct FCameraUpdateContext
{
private:
	float TrueCurrentWeight;
	float NonDebugCurrentWeight;
	float DeltaTime;
	APlayerCameraManager* Camera;

public:
	FCameraUpdateContext(APlayerCameraManager* InCamera, float InDeltaTime)
		: TrueCurrentWeight(1.0f)
		, NonDebugCurrentWeight(1.0f)
		, DeltaTime(InDeltaTime)
		, Camera(InCamera)
	{
	}

	FCameraUpdateContext FractionalWeight(float Multiplier, bool bFromDebugNode) const
	{
		FCameraUpdateContext Result(Camera, DeltaTime);
		Result.TrueCurrentWeight = TrueCurrentWeight * Multiplier;
		Result.NonDebugCurrentWeight = NonDebugCurrentWeight * (bFromDebugNode ? 1.0f : Multiplier);

		return Result;
	}

	// Returns the final blend weight contribution for this stage
	float GetTrueWeight() const { return TrueCurrentWeight; }

	// Returns the final blend weight contribution for this stage (ignoring debug cameras)
	float GetNonDebugWeight() const { return NonDebugCurrentWeight; }

	// Returns the delta time for this update, in seconds
	float GetDeltaTime() const { return DeltaTime; }

	// Returns the camera stack
	//@TODO: Hack to work around current view target construction...
	APlayerCameraManager* GetCameraManager() const { return Camera; }
};
