// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
#include "Engine/Scene.h"
#include "CameraTypes.generated.h"

class UCameraShakeSourceComponent;

//@TODO: Document
UENUM()
namespace ECameraProjectionMode
{
	enum Type : int
	{
		Perspective,
		Orthographic
	};
}

UENUM()
enum class ECameraShakePlaySpace : uint8
{
	/** This anim is applied in camera space. */
	CameraLocal,
	/** This anim is applied in world space. */
	World,
	/** This anim is applied in a user-specified space (defined by UserPlaySpaceMatrix). */
	UserDefined,
};

USTRUCT(BlueprintType)
struct FMinimalViewInfo
{
	GENERATED_USTRUCT_BODY()

	/** Location */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	FVector Location;

	/** Rotation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	FRotator Rotation;

	/** The horizontal field of view (in degrees) in perspective mode (ignored in orthographic mode). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float FOV;

	/** The originally desired horizontal field of view before any adjustments to account for different aspect ratios */
	UPROPERTY(Transient)
	float DesiredFOV;

	/** The horizontal field of view (in degrees) used for primitives tagged as "IsFirstPerson". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	float FirstPersonFOV;

	/** The scale to apply to primitives tagged as "IsFirstPerson". This is used to scale down primitives towards the camera such that they are small enough not to intersect with the scene. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	float FirstPersonScale;

	/** The desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float OrthoWidth;

	/** Option for the Ortho camera to automatically calculated the near/far plane */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	bool bAutoCalculateOrthoPlanes;

	/** Manually adjusts the planes of this camera, maintaining the distance between them. Positive moves out to the farplane, negative towards the near plane */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	float AutoPlaneShift;

	/** Adjusts the near/far planes and the view origin of the current camera automatically to avoid clipping and light artefacting*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	bool bUpdateOrthoPlanes;

	/** If UpdateOrthoPlanes is enabled, this setting will use the cameras current height to compensate the distance to the general view (as a pseudo distance to view target when one isn't present) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	bool bUseCameraHeightAsViewTarget;

	/** The near plane distance of the orthographic view (in world units) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category=Camera)
	float OrthoNearClipPlane;

	/** The far plane distance of the orthographic view (in world units) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category=Camera)
	float OrthoFarClipPlane;

	/** The near plane distance of the perspective view (in world units). Set to a negative value to use the default global value of GNearClippingPlane */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category=Camera)
	float PerspectiveNearClipPlane;

	// Aspect Ratio (Width/Height)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float AspectRatio;

	// Aspect ratio axis constraint override
	TOptional<EAspectRatioAxisConstraint> AspectRatioAxisConstraint;

	// If bConstrainAspectRatio is true, black bars will be added if the destination view has a different aspect ratio than this camera requested.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	uint32 bConstrainAspectRatio:1; 

	// If bUseFirstPersonParameters is true, FirstPersonFOV and FirstPersonScale should be applied to primitives tagged as "IsFirstPerson".
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Camera)
	uint32 bUseFirstPersonParameters : 1;

	// If true, account for the field of view angle when computing which level of detail to use for meshes.
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=CameraSettings)
	uint32 bUseFieldOfViewForLOD:1;

	// The type of camera
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	TEnumAsByte<ECameraProjectionMode::Type> ProjectionMode;

	/** Indicates if PostProcessSettings should be applied. */
	UPROPERTY(BlueprintReadWrite, Category = Camera)
	float PostProcessBlendWeight;

	/** Post-process settings to use if PostProcessBlendWeight is non-zero. */
	UPROPERTY(BlueprintReadWrite, Category = Camera)
	struct FPostProcessSettings PostProcessSettings;

	/** Off-axis / off-center projection offset as proportion of screen dimensions */
	UPROPERTY(Transient, VisibleInstanceOnly, BlueprintReadWrite, Category = Camera)
	FVector2D OffCenterProjectionOffset;

	/** Optional transform to be considered as this view's previous transform */
	TOptional<FTransform> PreviousViewTransform;

	/** Resolution fraction that scales with the amount of overscan added to the view */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float OverscanResolutionFraction;
	
	/** The fraction between 0.0 and 1.0 of the view to crop to during the final post process upscale, with 1.0 meaning no crop */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera, meta=(ClampMin=0.0, UIMin=0.0, ClampMax=1.0, UIMax=1.0))
	float CropFraction;

	/**
	 * Experimental: The fraction for each edge between 0.0 and 1.0 of the view to crop to during the final post process upscale, with 1.0 meaning no crop.
	 * By convention, X is the left edge, Y is the right edge, Z is the top edge, and W is the bottom edge. Stacks with uniform CropFraction.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera, meta=(ClampMin=0.0, UIMin=0.0, ClampMax=1.0, UIMax=1.0))
	FVector4f AsymmetricCropFraction;
	
private:
	/** The amount of overscan that has been applied to the view's frustum, with 0.0 meaning no overscan and 1.0 meaning 100% overscan */
	float Overscan;

	/**
	 * Experimental: The amount of asymmetric overscan that has been applied to the view's frustum, with 0.0 meaning no overscan and 1.0 meaning 100% overscan.
	 * By convention, X is the left overscan, Y is the right overscan, Z is the top overscan, and W is the bottom overscan. Stacks with uniform Overscan.
	 */
	FVector4f AsymmetricOverscan;
	
	// Only used for Ortho camera auto plane calculations, tells the Near plane of the extra distance that needs to be added.
	FVector CameraToViewTarget;

public:

	FMinimalViewInfo()
		: Location(ForceInit)
		, Rotation(ForceInit)
		, FOV(90.0f)
		, DesiredFOV(90.0f)
		, FirstPersonFOV(90.0f)
		, FirstPersonScale(1.0f)
		, OrthoWidth(512.0f)
		, bAutoCalculateOrthoPlanes(true)
		, AutoPlaneShift(0.0f)
		, bUpdateOrthoPlanes(false)
		, bUseCameraHeightAsViewTarget(false)
		, OrthoNearClipPlane(0.0f)
		, OrthoFarClipPlane(UE_OLD_WORLD_MAX)
		, PerspectiveNearClipPlane(-1.0f)
		, AspectRatio(1.33333333f)
		, bConstrainAspectRatio(false)
		, bUseFirstPersonParameters(false)
		, bUseFieldOfViewForLOD(true)
		, ProjectionMode(ECameraProjectionMode::Perspective)
		, PostProcessBlendWeight(0.0f)
		, OffCenterProjectionOffset(ForceInitToZero)
		, OverscanResolutionFraction(1.0f)
		, CropFraction(1.0f)
		, AsymmetricCropFraction(FVector4f::One())
		, Overscan(0.0f)
		, AsymmetricOverscan(FVector4f::Zero())
		, CameraToViewTarget(FVector::ZeroVector)
	{
	}

	// Is this equivalent to the other one?
	ENGINE_API bool Equals(const FMinimalViewInfo& OtherInfo) const;

	// Blends view information
	// Note: booleans are orred together, instead of blending
	ENGINE_API void BlendViewInfo(FMinimalViewInfo& OtherInfo, float OtherWeight);

	/** Applies weighting to this view, in order to be blended with another one. Equals to this *= Weight. */
	ENGINE_API void ApplyBlendWeight(const float& Weight);

	/** Combines this view with another one which will be weighted. Equals to this += OtherView * Weight. */
	ENGINE_API void AddWeightedViewInfo(const FMinimalViewInfo& OtherView, const float& Weight);

	/** Calculates the projection matrix using this view info's aspect ratio (regardless of bConstrainAspectRatio) */
	ENGINE_API FMatrix CalculateProjectionMatrix() const;

	/** Calculates the projection matrix (and potentially a constrained view rectangle) given a FMinimalViewInfo and partially configured projection data (must have the view rect already set) */
	ENGINE_API static void CalculateProjectionMatrixGivenView(FMinimalViewInfo& ViewInfo, TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint, class FViewport* Viewport, struct FSceneViewProjectionData& InOutProjectionData);
	/** Calculates the projection matrix (and potentially a constrained view rectangle) given a FMinimalViewInfo and partially configured projection data (must have the view rect already set). ConstrainedViewRectangle is only used if the ViewInfo.bConstrainAspectRatio is set. */
	ENGINE_API static void CalculateProjectionMatrixGivenViewRectangle(FMinimalViewInfo& ViewInfo, TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint, const FIntRect& ConstrainedViewRectangle, FSceneViewProjectionData& InOutProjectionData);

	/** The near plane distance of the perspective view (in world units). Returns the value of PerspectiveNearClipPlane if positive, and GNearClippingPlane otherwise */
	inline float GetFinalPerspectiveNearClipPlane() const
	{
		return PerspectiveNearClipPlane > 0.0f ? PerspectiveNearClipPlane : GNearClippingPlane;
	}

	/** Automatically calculates the Near/Far plane values for an Ortho camera */
	ENGINE_API bool AutoCalculateOrthoPlanes(FSceneViewProjectionData& InOutProjectionData);

	/** Sets the camera distance from view target for AutoCalculateOrthoPlanes */
	inline void SetCameraToViewTarget(const FVector ActorLocation)
	{
		CameraToViewTarget = ActorLocation - Location;
	}

	/**
	 * Transforms a world space location into "first person space". This function mirrors the morphing that is applied to first person primitives
	 * when they are rendered on the GPU, so it can be used for spawning objects (e.g. projectiles or ejected shell casings) relative to the morphed
	 * first person geometry on screen.
	 * Setting bIgnoreFirstPersonScale to true only applies the field of view morphing and is useful for cases where a full size projectile is spawned in front
	 * of the first person weapon. By ignoring the first person scale for the spawn location, the spawned full-size projectile will be spawned a bit further away from the camera,
	 * but its on-screen size will look correct.
	 */
	ENGINE_API FVector TransformWorldToFirstPerson(const FVector& WorldPosition, bool bIgnoreFirstPersonScale) const;

	/**
	 * Transforms a first person space location back into world space.
	 * bScaleWasIgnoredOnInput should be set to true if the first person position was calculated with TransformWorldToFirstPerson() and bIgnoreFirstPersonScale set to true.
	 */
	ENGINE_API FVector TransformFirstPersonToWorld(const FVector& FirstPersonPosition, bool bScaleWasIgnoredOnInput) const;

	/**
	 * Correction factor to apply to the first person transform used on primitives tagged as "IsFirstPerson" to achieve a first person specific field of view.
	 * It is computed as tan(SceneFOVRadians * 0.5) / tan(FirstPersonFOVRadians * 0.5).
	 */
	ENGINE_API float CalculateFirstPersonFOVCorrectionFactor() const;

	/**
	 * Apply overscan to the view info, which scales the field of view and ortho width to simulate expanding the view frustum.
	 * 
	 * @param InOverscan - The amount of overscan to apply, from 0.0 meaning no overscan to 1.0 meaning 100% overscan
	 * @param bScaleResolutionWithOverscan - Indicates that the view's resolution should be scaled with the amount of overscan, so that the original frustum remains the same resolution
	 * @param bCropOverscan - Indicates that the view should be cropped during the final post process pass to remove the overscanned pixels
	 */
	ENGINE_API void ApplyOverscan(float InOverscan, bool bScaleResolutionWithOverscan = false, bool bCropOverscan = false);

	/**
	 * Experimental: Apply asymmetric overscan to the view info, which scales the field of view, ortho width, aspect ratio, and off-center projection to
	 * simulate expanding the view frustum asymmetrically.
	 * 
	 * @param InAsymmetricOverscan - The amount of asymmetric overscan to apply, from 0.0 meaning no overscan to 1.0 meaning 100% overscan. By convention,
	 *								 X is the left overscan, Y is the right overscan, Z is the top overscan, and W is the bottom overscan.
	 * @param bScaleResolutionWithOverscan - Indicates that the view's resolution should be scaled with the amount of overscan, so that the original frustum remains the same resolution
	 * @param bCropOverscan - Indicates that the view should be cropped during the final post process pass to remove the overscanned pixels
	 */
	ENGINE_API void ApplyAsymmetricOverscan(const FVector4f& InAsymmetricOverscan, bool bScaleResolutionWithOverscan = false, bool bCropOverscan = false);
	
	/** Gets the total amount of overscan that has been applied to the view's frustum, with 0.0 meaning no overscan and 1.0 meaning 100% overscan */
	float GetOverscan() const { return Overscan; }
	
	/**
	 * Removes all overscan (uniform and asymmetric) from the view info.
	 */
	ENGINE_API void ClearOverscan();
};
