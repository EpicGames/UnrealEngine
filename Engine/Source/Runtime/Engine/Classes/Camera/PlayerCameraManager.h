// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Camera/CameraTypes.h"
#include "UObject/ScriptInterface.h"
#include "PlayerCameraManager.generated.h"

class AEmitterCameraLensEffectBase;
class APlayerController;
class FDebugDisplayInfo;
class UCameraModifier;
class UCameraShakeBase;
class UCameraShakeSourceComponent;
class ICameraLensEffectInterface;
struct FAddCameraShakeParams;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAudioFadeChangeSignature, bool, bFadeOut, float, FadeTime);

/** 
 * Options that define how to blend when changing view targets. 
 * @see FViewTargetTransitionParams, SetViewTarget 
 */
UENUM()
enum EViewTargetBlendFunction : int
{
	/** Camera does a simple linear interpolation. */
	VTBlend_Linear,
	/** Camera has a slight ease in and ease out, but amount of ease cannot be tweaked. */
	VTBlend_Cubic,
	/** Camera immediately accelerates, but smoothly decelerates into the target.  Ease amount controlled by BlendExp. */
	VTBlend_EaseIn,
	/** Camera smoothly accelerates, but does not decelerate into the target.  Ease amount controlled by BlendExp. */
	VTBlend_EaseOut,
	/** Camera smoothly accelerates and decelerates.  Ease amount controlled by BlendExp. */
	VTBlend_EaseInOut,
	/** The game's camera system has already performed the blending. Engine should not blend at all */
	VTBlend_PreBlended,
	VTBlend_MAX,
};

UENUM()
enum EViewTargetBlendOrder : int
{
	VTBlendOrder_Base,
	VTBlendOrder_Override
};

/** 
 * Cached camera POV info, stored as optimization so we only
 * need to do a full camera update once per tick.
 */
USTRUCT()
struct FCameraCacheEntry
{
	GENERATED_USTRUCT_BODY()
public:

	/** World time this entry was created. */
	UPROPERTY()
	float TimeStamp;

	/** Camera POV to cache. */
	UPROPERTY()
	FMinimalViewInfo POV;

	FCameraCacheEntry()
		: TimeStamp(0.f)
	{}
};

/** A ViewTarget is the primary actor the camera is associated with. */
USTRUCT(BlueprintType)
struct FTViewTarget
{
	GENERATED_USTRUCT_BODY()
public:

	/** Target Actor used to compute POV */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TViewTarget)
	TObjectPtr<class AActor> Target;

	/** Computed point of view */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TViewTarget)
	struct FMinimalViewInfo POV;

protected:
	/** PlayerState (used to follow same player through pawn transitions, etc., when spectating) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TViewTarget)
	TObjectPtr<class APlayerState> PlayerState;

public:
	class APlayerState* GetPlayerState() const { return PlayerState; }
	
	ENGINE_API void SetNewTarget(AActor* NewTarget);

	ENGINE_API class APawn* GetTargetPawn() const;

	ENGINE_API bool Equal(const FTViewTarget& OtherTarget) const;

	FTViewTarget()
		: Target(nullptr)
		, PlayerState(nullptr)
	{}

	/** Make sure ViewTarget is valid */
	ENGINE_API void CheckViewTarget(APlayerController* OwningController);
};

/** A set of parameters to describe how to transition between view targets. */
USTRUCT(BlueprintType)
struct FViewTargetTransitionParams
{
	GENERATED_USTRUCT_BODY()
public:

	/** Total duration of blend to pending view target. 0 means no blending. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ViewTargetTransitionParams)
	float BlendTime;

	/** Function to apply to the blend parameter. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ViewTargetTransitionParams)
	TEnumAsByte<enum EViewTargetBlendFunction> BlendFunction;

	/** Exponent, used by certain blend functions to control the shape of the curve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ViewTargetTransitionParams)
	float BlendExp;

	/** 
	 * If true, lock outgoing viewtarget to last frame's camera POV for the remainder of the blend.
	 * This is useful if you plan to teleport the old viewtarget, but don't want to affect the blend. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ViewTargetTransitionParams)
	uint32 bLockOutgoing:1;

	FViewTargetTransitionParams()
		: BlendTime(0.f)
		, BlendFunction(VTBlend_Cubic)
		, BlendExp(2.f)
		, bLockOutgoing(false)
	{}

	/** For a given linear blend value (blend percentage), return the final blend alpha with the requested function applied */
	float GetBlendAlpha(const float& TimePct) const
	{
		switch (BlendFunction)
		{
		case VTBlend_Linear: return FMath::Lerp(0.f, 1.f, TimePct); 
		case VTBlend_Cubic:	return FMath::CubicInterp(0.f, 0.f, 1.f, 0.f, TimePct); 
		case VTBlend_EaseInOut: return FMath::InterpEaseInOut(0.f, 1.f, TimePct, BlendExp); 
		case VTBlend_EaseIn: return FMath::Lerp(0.f, 1.f, FMath::Pow(TimePct, BlendExp)); 
		case VTBlend_EaseOut: return FMath::Lerp(0.f, 1.f, FMath::Pow(TimePct, (FMath::IsNearlyZero(BlendExp) ? 1.f : (1.f / BlendExp))));
		case VTBlend_PreBlended:
		case VTBlend_MAX:
		default:
			break;
		}

		return 1.f;
	}
};


//~=============================================================================
/**
 * A PlayerCameraManager is responsible for managing the camera for a particular
 * player. It defines the final view properties used by other systems (e.g. the renderer),
 * meaning you can think of it as your virtual eyeball in the world. It can compute the 
 * final camera properties directly, or it can arbitrate/blend between other objects or 
 * actors that influence the camera (e.g. blending from one CameraActor to another).
 * 
 * The PlayerCameraManagers primary external responsibility is to reliably respond to
 * various Get*() functions, such as GetCameraViewPoint. Most everything else is
 * implementation detail and overrideable by user projects.
 * 
 * By default, a PlayerCameraManager maintains a "view target", which is the primary actor
 * the camera is associated with. It can also apply various "post" effects to the final 
 * view state, such as camera animations, shakes, post-process effects or special 
 * effects such as dirt on the lens.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Camera/
 */
UCLASS(notplaceable, transient, BlueprintType, Blueprintable, Config=Engine, MinimalAPI)
class APlayerCameraManager : public AActor
{
	GENERATED_UCLASS_BODY()

	/** PlayerController that owns this Camera actor */
	UPROPERTY(transient)
	TObjectPtr<class APlayerController> PCOwner;

private:
	/** Dummy component we can use to attach things to the camera. */
	UPROPERTY(Category = PlayerCameraManager, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USceneComponent> TransformComponent;

public:
	/** Usable to define different camera behaviors. A few simple styles are implemented by default. */
	FName CameraStyle;

	/** FOV to use by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float DefaultFOV;

	float GetLockedFOV() const { return LockedFOV; }

protected:
	/** Value to lock FOV to, in degrees. Ignored if <= 0, utilized if > 0. */
	float LockedFOV;

public:
	/** The default desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float DefaultOrthoWidth;

protected:
	/** Value OrthoWidth is locked at, if > 0. Ignored if <= 0. */
	float LockedOrthoWidth;

public:
	/** Default aspect ratio. Most of the time the value from a camera component will be used instead. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float DefaultAspectRatio;

	/** Color to fade to (when bEnableFading == true). */
	FLinearColor FadeColor;

	/** Amount of fading to apply (when bEnableFading == true). */
	float FadeAmount;

	/** Allows control over scaling individual color channels in the final image (when bEnableColorScaling == true). */
	FVector ColorScale;

	/** Desired color scale which ColorScale will interpolate to (when bEnableColorScaling and bEnableColorScaleInterp == true) */
	FVector DesiredColorScale;

	/** Color scale value at start of interpolation (when bEnableColorScaling and bEnableColorScaleInterp == true) */
	FVector OriginalColorScale;

	/** Total time for color scale interpolation to complete (when bEnableColorScaling and bEnableColorScaleInterp == true) */
	float ColorScaleInterpDuration;

	/** Time at which interpolation started (when bEnableColorScaling and bEnableColorScaleInterp == true) */
	float ColorScaleInterpStartTime;

	/** Current ViewTarget */
	UPROPERTY(transient)
	struct FTViewTarget ViewTarget;

	/** Pending view target for blending */
	UPROPERTY(transient)
	struct FTViewTarget PendingViewTarget;

	/** Time remaining in viewtarget blend. */
	float BlendTimeToGo;

	/** Current view target transition blend parameters. */
	struct FViewTargetTransitionParams BlendParams;

public:
	/** Fires when ViewTarget is set to PendingViewTarget */
	DECLARE_EVENT(APlayerCameraManager, FOnBlendComplete)
	FOnBlendComplete& OnBlendComplete() const { return OnBlendCompleteEvent; }
private:
	mutable FOnBlendComplete OnBlendCompleteEvent;
private:
	/** Cached camera properties. */
	UPROPERTY(transient)
	struct FCameraCacheEntry CameraCachePrivate;

	/** Cached camera properties, one frame old. */
	UPROPERTY(transient)
	struct FCameraCacheEntry LastFrameCameraCachePrivate;

protected:
	/** List of active camera modifier instances that have a chance to update the final camera POV */
	UPROPERTY(transient)
	TArray<TObjectPtr<UCameraModifier>> ModifierList;

public:
	/**
	* Runs the given function on a local copy of the ModifierList of UCameraModifier, with the possibility of early exit
	* Most easily used with a lambda as follows:
	* ForEachCameraModifier([](UCameraModifier* Proxy) -> bool
	* {
	*     return continueLoop ? true : false;
	* });
	*/
	ENGINE_API void ForEachCameraModifier(TFunctionRef<bool(UCameraModifier*)> Fn);

	/** List of modifiers to create by default for this camera */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = CameraModifier)
	TArray< TSubclassOf<UCameraModifier> > DefaultModifiers;

	/** Distance to place free camera from view target (used in certain CameraStyles) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Debug)
	float FreeCamDistance;

	/** Offset to Z free camera position (used in certain CameraStyles) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Debug)
	FVector FreeCamOffset;

	/** Offset to view target (used in certain CameraStyles) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	FVector ViewTargetOffset;

	/** If bound, broadcast on fade start (with fade time) instead of manually altering audio device's primary volume directly */
	UPROPERTY(BlueprintAssignable, Transient)
	FOnAudioFadeChangeSignature OnAudioFadeChangeEvent;

	/** Current camera fade alpha range, where X = starting alpha and Y = final alpha (when bEnableFading == true) */
	FVector2D FadeAlpha;

	/** Total duration of the camera fade (when bEnableFading == true) */
	float FadeTime;

	/** Time remaining in camera fade (when bEnableFading == true) */
	float FadeTimeRemaining;

	/** Sets value of CameraCachePrivate.POV */
	ENGINE_API virtual void SetCameraCachePOV(const FMinimalViewInfo& InPOV);
	
	/** Sets value of LastFrameCameraCachePrivate.POV */
	ENGINE_API virtual void SetLastFrameCameraCachePOV(const FMinimalViewInfo& InPOV);

	/** Gets value of CameraCachePrivate.POV */
	ENGINE_API virtual const FMinimalViewInfo& GetCameraCacheView() const;

	/** Gets value of LastFrameCameraCachePrivate.POV */
	ENGINE_API virtual const FMinimalViewInfo& GetLastFrameCameraCacheView() const;

	/** Get value of CameraCachePrivate.Time  */
	float GetCameraCacheTime() const { return CameraCachePrivate.TimeStamp; }

	/** Get value of LastFrameCameraCachePrivate.Time  */
	float GetLastFrameCameraCacheTime() const { return LastFrameCameraCachePrivate.TimeStamp; }

protected:
	/** Get value of CameraCachePrivate.Time  */
	void SetCameraCacheTime(float InTime) { CameraCachePrivate.TimeStamp = InTime; }

	/** Get value of LastFrameCameraCachePrivate.Time  */
	void SetLastFrameCameraCacheTime(float InTime) { LastFrameCameraCachePrivate.TimeStamp = InTime; }

	/** Gets cached camera location and rotation, possibly modified to assist replication */
	ENGINE_API virtual void GetCameraCacheLocationAndRotationForServer(FVector& OutLocation, FRotator& OutRotation) const;

	/** Gets last frame's cached camera location and rotation, possibly modified to assist replication */
	ENGINE_API virtual void GetLastFrameCameraCacheLocationAndRotationForServer(FVector& OutLocation, FRotator& OutRotation) const;

	// "Lens" effects (e.g. blood, dirt on camera)
	/** CameraBlood emitter attached to this camera */
	UPROPERTY(transient)
	TArray<TScriptInterface<class ICameraLensEffectInterface>> CameraLensEffects;

	/////////////////////
	// Camera Modifiers
	/////////////////////

	/** Cached ref to modifier for code-driven screen shakes */
	UPROPERTY(transient)
	TObjectPtr<class UCameraModifier_CameraShake> CachedCameraShakeMod;


	/** Internal list of active post process effects. Parallel array to PostProcessBlendCacheWeights. */
	UPROPERTY(transient)
	TArray<struct FPostProcessSettings> PostProcessBlendCache;

	/** Internal list of weights for active post process effects. Parallel array to PostProcessBlendCache. */
	TArray<float> PostProcessBlendCacheWeights;

	/** Internal list of how to blend post process effects. */
	TArray<EViewTargetBlendOrder> PostProcessBlendCacheOrders;

public:
	/** Adds a postprocess effect at the given weight. */
	ENGINE_API void AddCachedPPBlend(struct FPostProcessSettings& PPSettings, float BlendWeight, EViewTargetBlendOrder BlendOrder = VTBlendOrder_Base);

	/** Returns active post process info. */
	ENGINE_API void GetCachedPostProcessBlends(TArray<struct FPostProcessSettings> const*& OutPPSettings, TArray<float> const*& OutBlendWeights) const;

	/** Returns active post process info. */
	ENGINE_API void GetCachedPostProcessBlends(TArray<struct FPostProcessSettings> const*& OutPPSettings, TArray<float> const*& OutBlendWeights, TArray<EViewTargetBlendOrder> const*& OutBlendOrders) const;

protected:
	/** Removes all postprocess effects. */
	ENGINE_API void ClearCachedPPBlends();

protected:
	/** Internal. Receives the output of individual camera animations. */
	UPROPERTY(transient)
	TObjectPtr<class ACameraActor> AnimCameraActor;


	////////////////////////
	// Flags
	////////////////////////

public:
	/** True when this camera should use an orthographic perspective instead of FOV */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlayerCameraManager)
	uint32 bIsOrthographic : 1;

	/** True when this camera should automatically calculated the Near+Far planes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlayerCameraManager, meta = (EditCondition = "bIsOrthographic"))
	uint32 bAutoCalculateOrthoPlanes : 1;

	/** Manually adjusts the planes of this camera, maintaining the distance between them. Positive moves out to the farplane, negative towards the near plane */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlayerCameraManager, meta = (EditCondition = "bIsOrthographic && bAutoCalculateOrthoPlanes"))
	float AutoPlaneShift;

	/** Adjusts the near/far planes and the view origin of the current camera automatically to avoid clipping and light artefacting*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlayerCameraManager, meta = (EditCondition = "bIsOrthographic"))
	uint32 bUpdateOrthoPlanes : 1;

	/** If UpdateOrthoPlanes is enabled, this setting will use the cameras current height to compensate the distance to the general view (as a pseudo distance to view target when one isn't present) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlayerCameraManager, meta = (EditCondition = "bIsOrthographic && bUpdateOrthoPlanes"))
	uint32 bUseCameraHeightAsViewTarget : 1;

	/** True if black bars should be added if the destination view has a different aspect ratio (only used when a view target doesn't specify whether or not to constrain the aspect ratio; most of the time the value from a camera component is used instead) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	uint32 bDefaultConstrainAspectRatio : 1;

	/** True if we should apply FadeColor/FadeAmount to the screen. */
	uint32 bEnableFading : 1;

	/** True to apply fading of audio alongside the video. */
	uint32 bFadeAudio : 1;

	/** True to turn on scaling of color channels in final image using ColorScale property. */
	uint32 bEnableColorScaling : 1;

	/** True to smoothly interpolate color scale values when they change. */
	uint32 bEnableColorScaleInterp : 1;

	/** True if clients are handling setting their own viewtarget and the server should not replicate it. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	uint32 bClientSimulatingViewTarget : 1;

	/** True if server will use camera positions replicated from the client instead of calculating them locally. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlayerCameraManager)
	uint32 bUseClientSideCameraUpdates : 1;

	/** For debugging. If true, replicate the client side camera position but don't use it, and draw the positions on the server */
	uint32 bDebugClientSideCamera : 1;

	/** If true, send a camera update to the server on next update. */
	uint32 bShouldSendClientSideCameraUpdate : 1;

	/** 
	 * True if we did a camera cut this frame. Automatically reset to false every frame.
	 * This flag affects various things in the renderer (such as whether to use the occlusion queries from last frame, and motion blur).
	 */
	UPROPERTY(Transient, BlueprintReadOnly, Category=PlayerCameraManager)
	uint32 bGameCameraCutThisFrame : 1;

protected:
	/** True if camera fade hold at it's final value when finished. */
	uint32 bHoldFadeWhenFinished : 1;

	uint32 bAutoAnimateFade : 1;

	/** By default camera modifiers are not applied to stock debug cameras (e.g. CameraStyle == "FreeCam"). Setting to true will always apply modifiers. */
	uint32 bAlwaysApplyModifiers : 1;

public:
	/** Minimum view pitch, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewPitchMin;

	/** Maximum view pitch, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewPitchMax;

	/** Minimum view yaw, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewYawMin;

	/** Maximum view yaw, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewYawMax;

	/** Minimum view roll, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewRollMin;

	/** Maximum view roll, in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerCameraManager)
	float ViewRollMax;

	/**
	* Implementable blueprint hook to allow a PlayerCameraManager subclass to
	* constrain or otherwise modify the camera during free-camera photography.
	* For example, a blueprint may wish to limit the distance from the camera's
	* original point, or forbid the camera from passing through walls.
	* NewCameraLocation contains the proposed new camera location.
	* PreviousCameraLocation contains the camera location in the previous frame.
	* OriginalCameraLocation contains the camera location before the game was put
	* into photography mode.
	* Return ResultCameraLocation as modified according to your constraints.
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic, Category = "Photography")
	ENGINE_API void PhotographyCameraModify(const FVector NewCameraLocation, const FVector PreviousCameraLocation, const FVector OriginalCameraLocation, FVector& ResultCameraLocation);

	/**
	* Event triggered upon entering Photography mode (before pausing, if
	* r.Photography.AutoPause is 1).
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic, Category = "Photography")
	ENGINE_API void OnPhotographySessionStart();

	/**
	* Event triggered upon leaving Photography mode (after unpausing, if
	* r.Photography.AutoPause is 1).
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic, Category = "Photography")
	ENGINE_API void OnPhotographySessionEnd();

	/**
	* Event triggered upon the start of a multi-part photograph capture (i.e. a
	* stereoscopic or 360-degree shot).  This is an ideal time to turn off
	* rendering effects that tile badly (UI, subtitles, vignette, very aggressive
	* bloom, etc; most of these are automatically disabled when
	* r.Photography.AutoPostprocess is 1).
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic, Category = "Photography")
	ENGINE_API void OnPhotographyMultiPartCaptureStart();

	/**
	* Event triggered upon the end of a multi-part photograph capture, when manual
	* free-roaming photographic camera control is about to be returned to the user.
	* Here you may re-enable whatever was turned off within
	* OnPhotographyMultiPartCaptureStart.
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic, Category = "Photography")
	ENGINE_API void OnPhotographyMultiPartCaptureEnd();

	/** 
	 * Blueprint hook to allow blueprints to override existing camera behavior or implement custom cameras.
	 * If this function returns true, we will use the given returned values and skip further calculations to determine
	 * final camera POV. 
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic)
	ENGINE_API bool BlueprintUpdateCamera(AActor* CameraTarget, FVector& NewCameraLocation, FRotator& NewCameraRotation, float& NewCameraFOV);

	/** Returns the PlayerController that owns this camera. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API virtual APlayerController* GetOwningPlayerController() const;

	ENGINE_API virtual void AssignViewTarget(AActor* NewTarget, FTViewTarget& VT, struct FViewTargetTransitionParams TransitionParams=FViewTargetTransitionParams());

	friend struct FTViewTarget;
public:
	/** Returns the current ViewTarget. */
	ENGINE_API AActor* GetViewTarget() const;

	/** Returns the ViewTarget if it is an APawn, or nullptr otherwise */
	ENGINE_API class APawn* GetViewTargetPawn() const;

	//~ Begin AActor Interface
	ENGINE_API virtual bool ShouldTickIfViewportsOnly() const override;
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual FRemoteServerId GetPreferredRemoteServer() const override;
#endif
	//~ End AActor Interface

	/** Static.  Plays an in-world camera shake that affects all nearby players, with radial distance-based attenuation.
	 * @param InWorld - World context.
	 * @param Shake - Camera shake asset to use.
	 * @param Epicenter - Location to place the effect in world space
	 * @param InnerRadius - Cameras inside this radius get the full intensity shake.
	 * @param OuterRadius - Cameras outside this radius are not affected.
	 * @param Falloff - Exponent that describes the shake intensity falloff curve between InnerRadius and OuterRadius. 1.0 is linear.
	 * @param bOrientShakeTowardsEpicenter - Changes the rotation of shake to point towards epicenter instead of forward. Useful for things like directional hits.
	 */
	static ENGINE_API void PlayWorldCameraShake(UWorld* InWorld, TSubclassOf<UCameraShakeBase> Shake, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff, bool bOrientShakeTowardsEpicenter = false);

protected:
	/** 
	 * Internal. Calculates shake scale for a particular camera.
	 * @return Returns the intensity scalar in the range [0..1] for a shake originating at Epicenter. 
	 */
	static ENGINE_API float CalcRadialShakeScale(class APlayerCameraManager* Cam, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff);

private:
	/**
	 * Used to tell how long in seconds its been since ServerUpdateCamera was called
	 */
	float TimeSinceLastServerUpdateCamera;

	/**
	 * Timeout in seconds used to determine when to force a call to ServerUpdateCamera
	 */
	UPROPERTY(Config)
	float ServerUpdateCameraTimeout;

public:

	/**
	 * Performs per-tick camera update. Called once per tick after all other actors have been ticked.
	 * Non-local players replicate the POV if bUseClientSideCameraUpdates is true.
	 */
	ENGINE_API virtual void UpdateCamera(float DeltaTime);

	/**
	* Performs a photography camera tick even when the camera wouldn't normally be ticking.
	*/
	ENGINE_API virtual void UpdateCameraPhotographyOnly();

	/** 
	 * Creates and initializes a new camera modifier of the specified class. 
	 * @param ModifierClass - The class of camera modifier to create.
	 * @return Returns the newly created camera modifier.
	 */
	UFUNCTION(BlueprintCallable, Category = "Game|Player", meta = (DeterminesOutputType = "ModifierClass"))
	ENGINE_API virtual UCameraModifier* AddNewCameraModifier(TSubclassOf<UCameraModifier> ModifierClass);

	/** 
	 * Returns camera modifier for this camera of the given class, if it exists. 
	 * Exact class match only. If there are multiple modifiers of the same class, the first one is returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "Game|Player", meta = (DeterminesOutputType = "ModifierClass"))
	ENGINE_API virtual UCameraModifier* FindCameraModifierByClass(TSubclassOf<UCameraModifier> ModifierClass);

	/** 
	 * Removes the given camera modifier from this camera (if it's on the camera in the first place) and discards it. 
	 * @return True if successfully removed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Game|Player")
	ENGINE_API virtual bool RemoveCameraModifier(UCameraModifier* ModifierToRemove);

protected:
	/** Internal. Places the given modifier in the ModifierList at the appropriate priority. */
	ENGINE_API virtual bool AddCameraModifierToList(UCameraModifier* NewModifier);

	ENGINE_API virtual void CleanUpAnimCamera(const bool bDestroy);

public:	
	/**
	 * Applies the current set of camera modifiers to the given camera POV.
	 * @param	DeltaTime	Time in seconds since last update
	 * @param	InOutPOV	Point of View
	 */
	ENGINE_API virtual void ApplyCameraModifiers(float DeltaTime, FMinimalViewInfo& InOutPOV);
	
	/**
	 * Initialize this PlayerCameraManager for the given associated PlayerController.
	 * @param PC	PlayerController associated with this Camera.
	 */
	ENGINE_API virtual void InitializeFor(class APlayerController* PC);
	
	/** Returns the camera's current full FOV angle, in degrees. */
	UFUNCTION(BlueprintCallable, Category = "Camera")
	ENGINE_API virtual float GetFOVAngle() const;
	
	/** 
	 * Locks the FOV to the given value.  Unlock with UnlockFOV.
	 * @param NewFOV - New full FOV angle to use, in degrees.
	 */
	ENGINE_API virtual void SetFOV(float NewFOV);

	/** Unlocks the FOV. */
	ENGINE_API virtual void UnlockFOV();

	/** Returns true if this camera is using an orthographic perspective. */
	ENGINE_API virtual bool IsOrthographic() const;

	/** Returns the current orthographic width for the camera. */
	ENGINE_API virtual float GetOrthoWidth() const;

	/** 
	 * Sets and locks the current orthographic width for the camera. Unlock with UnlockOrthoWidth. 
	 * Only used if IsOrthographic returns true.
	 * @param OrthoWidth - New orthographic width.
	 */
	ENGINE_API virtual void SetOrthoWidth(float OrthoWidth);

	/** Unlocks OrthoWidth value */
	ENGINE_API virtual void UnlockOrthoWidth();

	/**
	 * Primary function to retrieve Camera's actual view point.
	 * Consider calling PlayerController::GetPlayerViewPoint() instead.
	 *
	 * @param	OutCamLoc	Returned camera location
	 * @param	OutCamRot	Returned camera rotation
	 */
	ENGINE_API virtual void GetCameraViewPoint(FVector& OutCamLoc, FRotator& OutCamRot) const;
	
	/** Returns camera's current rotation. */
	UFUNCTION(BlueprintCallable, Category = "Camera", meta=(Keywords="View Direction"))
	ENGINE_API virtual FRotator GetCameraRotation() const;

	/** Returns camera's current location. */
	UFUNCTION(BlueprintCallable, Category = "Camera", meta=(Keywords="View Position"))
	ENGINE_API virtual FVector GetCameraLocation() const;
	
	/** 
	 * Sets the new desired color scale, enables color scaling, and enables color scale interpolation. 
	 * @param NewColorScale - new color scale to use
	 * @param InterpTime - duration of the interpolation from old to new, in seconds.
	 */
	ENGINE_API virtual void SetDesiredColorScale(FVector NewColorScale, float InterpTime);
	
protected:
	/** Internal function conditionally called from UpdateCamera to do the actual work of updating the camera. */
	ENGINE_API virtual void DoUpdateCamera(float DeltaTime);

	/** Updates the photography camera. Return true if a cut occurred */
	ENGINE_API virtual bool UpdatePhotographyCamera(FMinimalViewInfo& NewPOV);

public:
	/** Allows the photography system to override postprocessing */
	ENGINE_API virtual void UpdatePhotographyPostProcessing(FPostProcessSettings& InOutPostProcessing);

protected:
	/** Whether or not we allow photography mode */
	ENGINE_API virtual bool AllowPhotographyMode() const;
	/** Internal. Applies appropriate audio fading to the audio system. */
	ENGINE_API virtual void ApplyAudioFade();
	ENGINE_API virtual void StopAudioFade();
	
	/**
	 * Internal helper to blend two viewtargets.
	 *
	 * @param	A		Source view target
	 * @param	B		destination view target
	 * @param	Alpha	% of blend from A to B.
	 */
	ENGINE_API FPOV BlendViewTargets(const FTViewTarget& A, const FTViewTarget& B, float Alpha);
	
public:
	/** Caches given final POV info for efficient access from other game code. */
	ENGINE_API void FillCameraCache(const FMinimalViewInfo& NewInfo);
	
protected:
	/**
	 * Calculates an updated POV for the given viewtarget.
	 * @param	OutVT		ViewTarget to update.
	 * @param	DeltaTime	Delta Time since last camera update (in seconds).
	 */
	ENGINE_API virtual void UpdateViewTarget(FTViewTarget& OutVT, float DeltaTime);

	/** Update any attached camera lens effects **/
	ENGINE_API virtual void UpdateCameraLensEffects( const FTViewTarget& OutVT );

public:
	/** 
	 * Sets a new ViewTarget.
	 * @param NewViewTarget - New viewtarget actor.
	 * @param TransitionParams - Optional parameters to define the interpolation from the old viewtarget to the new. Transition will be instant by default.
	 */
	ENGINE_API virtual void SetViewTarget(class AActor* NewViewTarget, FViewTargetTransitionParams TransitionParams = FViewTargetTransitionParams());
	
	/** 
	 * Called to give PlayerCameraManager a chance to adjust view rotation updates before they are applied. 
	 * e.g. The base implementation enforces view rotation limits using LimitViewPitch, et al. 
	 * @param DeltaTime - Frame time in seconds.
	 * @param OutViewRotation - In/out. The view rotation to modify.
	 * @param OutDeltaRot - In/out. How much the rotation changed this frame.
	 */
	ENGINE_API virtual void ProcessViewRotation(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot);

	//
	// Camera Lens Effects
	//
	
	/** Returns first instance of a lens effect of the given class. */
	ENGINE_API virtual TScriptInterface<class ICameraLensEffectInterface> FindGenericCameraLensEffect(UPARAM(meta=(MustImplement = "CameraLensEffectInterface")) TSubclassOf<AActor> LensEffectEmitterClass);
	
	/** 
	 * Creates a camera lens effect of the given class on this camera. 
	 * @param LensEffectEmitterClass - Class of lens effect emitter to create.
	 * @return Returns the new emitter actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera")
	ENGINE_API virtual TScriptInterface<class ICameraLensEffectInterface> AddGenericCameraLensEffect(UPARAM(meta=(MustImplement = "CameraLensEffectInterface")) TSubclassOf<AActor> LensEffectEmitterClass);
	
	/** 
	 * Removes the given lens effect from the camera. 
	 * @param Emitter - the emitter actor to remove from the camera
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera")
	ENGINE_API virtual void RemoveGenericCameraLensEffect(TScriptInterface<class ICameraLensEffectInterface> Emitter);
	
	/** Removes all camera lens effects. */
	UFUNCTION(BlueprintCallable, Category = "Camera")
	ENGINE_API virtual void ClearCameraLensEffects();

	//
	// Camera Shakes.
	//

	/** 
	 * Plays a camera shake on this camera.
	 * @param Shake - The class of camera shake to play.
	 * @param Scale - Scalar defining how "intense" to play the shake. 1.0 is normal (as authored).
	 * @param PlaySpace - Which coordinate system to play the shake in (affects oscillations and camera anims)
	 * @param UserPlaySpaceRot - Coordinate system to play shake when PlaySpace == CAPS_UserDefined.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual UCameraShakeBase* StartCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, float Scale=1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/** 
	 * Plays a camera shake on this camera.
	 * @param Shake - The class of camera shake to play.
	 * @param SourceComponent - The source from which the camera shake originates.
	 * @param Scale - Applies an additional constant scale on top of the dynamic scale computed with the distance to the source
	 * @param PlaySpace - Which coordinate system to play the shake in (affects oscillations and camera anims)
	 * @param UserPlaySpaceRot - Coordinate system to play shake when PlaySpace == CAPS_UserDefined.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual UCameraShakeBase* StartCameraShakeFromSource(TSubclassOf<UCameraShakeBase> ShakeClass, UCameraShakeSourceComponent* SourceComponent, float Scale=1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/** Immediately stops the given shake instance and invalidates it. */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual void StopCameraShake(UCameraShakeBase* ShakeInstance, bool bImmediately = true);

	/** Stops playing all shakes of the given class. */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual void StopAllInstancesOfCameraShake(TSubclassOf<UCameraShakeBase> Shake, bool bImmediately = true);

	/** Stops all active camera shakes on this camera. */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual void StopAllCameraShakes(bool bImmediately = true);

	/** Stops playing all shakes of the given class originating from the given source. */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual void StopAllInstancesOfCameraShakeFromSource(TSubclassOf<UCameraShakeBase> Shake, UCameraShakeSourceComponent* SourceComponent, bool bImmediately = true);

	/** Stops playing all shakes originating from the given source. */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	ENGINE_API virtual void StopAllCameraShakesFromSource(UCameraShakeSourceComponent* SourceComponent, bool bImmediately = true);

	/**
	 * Plays a camera shake on this camera.
	 * @param Shake - The class of camera shake to play.
	 * @param Params - The parameters to pass to the camera shake camera modifier.
	 */
	ENGINE_API virtual UCameraShakeBase* StartCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, const FAddCameraShakeParams& Params);

	//
	//  Camera fades.
	//

	/** 
	 * Does a camera fade to/from a solid color.  Animates automatically.
	 * @param FromAlpha - Alpha at which to begin the fade. Range [0..1], where 0 is fully transparent and 1 is fully opaque solid color.
	 * @param ToAlpha - Alpha at which to finish the fade.
	 * @param Duration - How long the fade should take, in seconds.
	 * @param Color - Color to fade to/from.
	 * @param bShouldFadeAudio - True to fade audio volume along with the alpha of the solid color.
	 * @param bHoldWhenFinished - True for fade to hold at the ToAlpha until explicitly stopped (e.g. with StopCameraFade)
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Fades")
	ENGINE_API virtual void StartCameraFade(float FromAlpha, float ToAlpha, float Duration, FLinearColor Color, bool bShouldFadeAudio = false, bool bHoldWhenFinished = false);

	/** 
	 * Stops camera fading.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Fades")
	ENGINE_API virtual void StopCameraFade();

	/** 
	 * Turns on camera fading at the given opacity. Does not auto-animate, allowing user to animate themselves.
	 * Call StopCameraFade to turn fading back off.
	*/
	UFUNCTION(BlueprintCallable, Category = "Camera Fades")
	ENGINE_API virtual void SetManualCameraFade(float InFadeAmount, FLinearColor Color, bool bInFadeAudio);


	/** Sets the bGameCameraCutThisFrame flag to true (indicating we did a camera cut this frame; useful for game code to call, e.g., when performing a teleport that should be seamless) */
	UFUNCTION(BlueprintCallable, Category = "Camera")
	virtual void SetGameCameraCutThisFrame() { bGameCameraCutThisFrame = true; }

public:
	/** 
	 * Limit the player's view pitch. 
	 * @param ViewRotation - ViewRotation to modify. Both input and output.
	 * @param InViewPitchMin - Minimum view pitch, in degrees.
	 * @param InViewPitchMax - Maximum view pitch, in degrees.
	 */
	ENGINE_API virtual void LimitViewPitch(FRotator& ViewRotation, float InViewPitchMin, float InViewPitchMax);

	/** 
	 * Limit the player's view roll. 
	 * @param ViewRotation - ViewRotation to modify. Both input and output.
	 * @param InViewRollMin - Minimum view roll, in degrees.
	 * @param InViewRollMax - Maximum view roll, in degrees.
	 */
	ENGINE_API virtual void LimitViewRoll(FRotator& ViewRotation, float InViewRollMin, float InViewRollMax);

	/** 
	 * Limit the player's view yaw. 
	 * @param ViewRotation - ViewRotation to modify. Both input and output.
	 * @param InViewYawMin - Minimum view yaw, in degrees.
	 * @param InViewYawMax - Maximum view yaw, in degrees.
	 */
	ENGINE_API virtual void LimitViewYaw(FRotator& ViewRotation, float InViewYawMin, float InViewYawMax);

protected:
	/**
	 * Does the actual work to UpdateViewTarget. This is called from UpdateViewTarget under normal 
	 * circumstances (target is not a camera actor and no debug cameras are active)
	 * Provides a way for subclasses to override behavior without copy-pasting the special case code.
	 *
	 * @param	OutVT		ViewTarget to use.
	 * @param	DeltaTime	Delta Time since last camera update (in seconds).
	 */
	ENGINE_API virtual void UpdateViewTargetInternal(FTViewTarget& OutVT, float DeltaTime);

	// ClientSide camera updates prevents DoUpdateCamera from swapping PendingViewTarget in when the blend is complete, just use a timer to swap
	UFUNCTION()
	ENGINE_API void SwapPendingViewTargetWhenUsingClientSideCameraUpdates();

	FTimerHandle SwapPendingViewTargetWhenUsingClientSideCameraUpdatesTimerHandle;

private:
	// Buried to prevent use; use GetCameraRotation instead
	FRotator GetActorRotation() const { return Super::GetActorRotation(); }

	// Buried to prevent use; use GetCameraLocation instead
	FVector GetActorLocation() const { return Super::GetActorLocation(); }

public:
	/** Returns TransformComponent subobject */
	class USceneComponent* GetTransformComponent() const { return TransformComponent; }
};
