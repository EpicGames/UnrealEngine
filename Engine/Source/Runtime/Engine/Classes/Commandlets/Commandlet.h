// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Commandlet (command-line applet) class.
 *
 * Commandlets are executed from the ucc.exe command line utility, using the
 * following syntax:
 *
 *     yourgame.exe package_name.commandlet_class_name [parm=value]...
 *
 * for example:
 *
 *     yourgame.exe Core.HelloWorldCommandlet
 *     yourgame.exe UnrealEd.CookCommandlet
 *
 * As a convenience, if a user tries to run a commandlet and the exact
 * name they type isn't found, then ucc.exe appends the text "commandlet"
 * onto the name and tries again.  Therefore, the following shortcuts
 * perform identically to the above:
 *
 *     yourgame.exe Core.HelloWorld
 *     yourgame.exe UnrealEd.Make
 *
 * Commandlets are executed in a "raw" environment, in which the game isn't
 * loaded, the client code isn't loaded, no levels are loaded, and no actors exist.
 * 
 * To disable shader compiling during the run of the commandlet add "-NoShaderCompile" to the commandline.
 * This would be added as data member setting except that the shader compiler is initialized before a commandlet is created so it cannot be queried soon enough.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Commandlet.generated.h"

UCLASS(abstract, transient, MinimalAPI)
class UCommandlet : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Description of the commandlet's purpose */
	UPROPERTY()
	FString HelpDescription;

	/** Usage template to show for "ucc help" */
	UPROPERTY()
	FString HelpUsage;

	/** Hyperlink for more info */
	UPROPERTY()
	FString HelpWebLink;

	/** The name of the parameter the commandlet takes */
	UPROPERTY()
	TArray<FString> HelpParamNames;

	/** The description of the parameter */
	UPROPERTY()
	TArray<FString> HelpParamDescriptions;

	/**
	 * Whether to load objects required in server, client, and editor context.  If IsEditor is set to false, then a
	 * UGameEngine (or whatever the value of /Script/Engine.Engine.GameEngine is) will be created for the commandlet instead
	 * of a UEditorEngine (or /Script/Engine.Engine.EditorEngine), unless the commandlet overrides the CreateCustomEngine method.
	 */
	UPROPERTY()
	uint32 IsServer:1;

	UPROPERTY()
	uint32 IsClient:1;

	UPROPERTY()
	uint32 IsEditor:1;

	/** Whether to redirect standard log to the console */
	UPROPERTY()
	uint32 LogToConsole:1;

	/** Whether to show standard error and warning count on exit */
	UPROPERTY()
	uint32 ShowErrorCount:1;
  
	/** Whether to show cooking progress on tick */
	UPROPERTY()
	uint32 ShowProgress:1;

	/** Whether to exit the process as soon as the commandlet completes, skipping the engine shutdown */
	UPROPERTY()
	uint32 FastExit:1;

	/** Whether to use the commandlet result as the exit code, even if errors were logged during the whole engine execution */
	UPROPERTY()
	uint32 UseCommandletResultAsExitCode: 1;

	/**
	 * Entry point for your commandlet
	 *
	 * @param Params the string containing the parameters for the commandlet
	 */
	virtual int32 Main(const FString& Params) { return 0; }

	/**
	 * Parses a string into tokens, separating switches (beginning with - or /) from
	 * other parameters
	 *
	 * @param	CmdLine		the string to parse
	 * @param	Tokens		[out] filled with all parameters found in the string
	 * @param	Switches	[out] filled with all switches found in the string
	 *
	 * @return	@todo document
	 */
	static void ParseCommandLine( const TCHAR* CmdLine, TArray<FString>& Tokens, TArray<FString>& Switches )
	{
		FString NextToken;
		while ( FParse::Token(CmdLine, NextToken, false) )
		{
			if ( **NextToken == TCHAR('-') )
			{
				Switches.Add(NextToken.Mid(1));
			}
			else
			{
				Tokens.Add(MoveTemp(NextToken));
			}
		}
	}

	/**
	 * Parses a string into tokens, separating switches (beginning with - or /) from
	 * other parameters
	 *
	 * @param	CmdLine		the string to parse
	 * @param	Tokens		[out] filled with all parameters found in the string
	 * @param	Switches	[out] filled with all switches found in the string
	 * @param	Params		[out] filled with all switches found in the string with the format key=value
	 *
	 * @return	@todo document
	 */
	static void ParseCommandLine( const TCHAR* CmdLine, TArray<FString>& Tokens, TArray<FString>& Switches, TMap<FString, FString>& Params )
	{
		// Turn -foo -bar=1 into [Foo, Bar=1]
		ParseCommandLine(CmdLine, Tokens, Switches);

		for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
		{
			FString& Switch = Switches[SwitchIdx];
			TArray<FString> SplitSwitch;

			// Remove Bar=1 from the switch list and put it in params as {Bar,1}.
			// Note: Handle nested equality such as Bar="Key=Value"
			int32 AssignmentIndex = 0;
			if (Switch.FindChar(TEXT('='), AssignmentIndex))
			{
				Params.Add(Switch.Left(AssignmentIndex), Switch.RightChop(AssignmentIndex+1).TrimQuotes());
				Switches.RemoveAt(SwitchIdx);
			}
		}
	}

#if WITH_EDITOR
	/**
	 * Apply the switched to internal values if they exists.
	 *
	 * @param	Switches	The switches to apply to internal properties.
	 *
	 * @return	True if all values were properly applied.
	 */
	static ENGINE_API bool ApplyCommandLineSwitches( UObject* Object, const TArray<FString>& Switches );
#endif //  WITH_EDITOR

	/**
	 * Allows commandlets to override the default behavior and create a custom engine class for the commandlet. If
	 * the commandlet implements this function, it should fully initialize the UEngine object as well.  Commandlets
	 * should indicate that they have implemented this function by assigning the custom UEngine to GEngine.
	 */
	virtual void CreateCustomEngine(const FString& Params) {}
};

namespace CommandletHelpers
{
	/**
	 * Simulate an engine frame tick.
	 * Can be used by commandlets to tick various subsystems. If running with -AllowCommandletRendering, this will
	 * also tick rendering for the provided world scene(s).
	 */
	ENGINE_API void TickEngine(class UWorld* InWorld = nullptr, double InDeltaTime = 0.0);
}