// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "PluginCommandlet.generated.h"

UCLASS()
class UPluginCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	/** Parsed commandline tokens */
	TArray<FString> CmdLineTokens;

	/** Parsed commandline switches */
	TArray<FString> CmdLineSwitches;


	virtual int32 Main(const FString& Params) override;
};
