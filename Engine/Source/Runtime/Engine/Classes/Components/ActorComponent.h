// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/ScriptMacros.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/StructOnScope.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PropertyPairsMap.h"
#include "ComponentInstanceDataCache.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/Timeout.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Physics/Experimental/AsyncPhysicsStateProcessorInterface.h"
#include "ActorComponent.generated.h"

struct FTypedElementHandle;

class AActor;
class IRepChangedPropertyTracker;
class UActorComponent;
class UAssetUserData;
class ULevel;
class UWorld;
class UPrimitiveComponent;
struct FSimpleMemberReference;
class IPrimitiveComponent;
class UEngineReplicationBridge;

namespace UE::Net
{
	struct FReplicationSystemUtil;
}

ENGINE_API extern int32 GEnableDeferredPhysicsCreation;

class FRegisterComponentContext
{
public:
	using FPrimitiveBatch = TArray<UPrimitiveComponent*, FConcurrentLinearArrayAllocator>;

	ENGINE_API FRegisterComponentContext(UWorld* InWorld);
	ENGINE_API ~FRegisterComponentContext();

	bool IsInitialized() const
	{
		return World != nullptr;
	}
	void Initialize(UWorld* InWorld)
	{
		checkf(!IsInitialized(), TEXT("Context is already initialized"));
		World = InWorld;
	}

	FRegisterComponentContext& SetIncrementalUpdateTimeout(const UE::FTimeout& InTimeout);
	FRegisterComponentContext& SetIncrementalUpdateAddPrimitiveBatchSize(int32 Value);
	FRegisterComponentContext& SetIncrementalUpdateNumComponentsToUpdate(int32 Value);

	const UE::FTimeout& GetIncrementalUpdateTimeout() const { return Timeout; }
	int32 GetIncrementalUpdateAddPrimitiveBatchSize() const { return IncrementalUpdateAddPrimitiveBatchSize; }
	int32 GetIncrementalUpdateNumComponentsToUpdate() const { return IncrementalUpdateNumComponentsToUpdate; }

	void AddPrimitive(UPrimitiveComponent* PrimitiveComponent);
	void AddSendRenderDynamicData(UPrimitiveComponent* PrimitiveComponent);

	ENGINE_API static void SendRenderDynamicData(FRegisterComponentContext* Context, UPrimitiveComponent* PrimitiveComponent);

	UE_DEPRECATED(5.5, "This function is not used anymore")
	int32 Count() const { return AddPrimitiveBatches.Num(); }

	UE_DEPRECATED(5.8, "AsyncRegisterLevelContext has been removed.")
	FRegisterComponentContext& SetAsyncRegisterLevelContext(struct FAsyncRegisterLevelContext* Value);

	ENGINE_API void Process();

private:
	bool OnIncrementalRegisterComponentsDone();

	UWorld* World = nullptr;
	UE::FTimeout Timeout;
	int32	IncrementalUpdateAddPrimitiveBatchSize;
	int32	IncrementalUpdateNumComponentsToUpdate;
	FPrimitiveBatch AddPrimitiveBatches;
	FPrimitiveBatch SendRenderDynamicDataPrimitives;

	friend class ULevel;
};

#if WITH_EDITOR
class SWidget;
struct FMinimalViewInfo;
#endif

/** Information about how to update transform when something is moved */
enum class EUpdateTransformFlags : int32
{
	/** Default options */
	None = 0x0,
	/** Don't update the underlying physics */
	SkipPhysicsUpdate = 0x1,		
	/** The update is coming as a result of the parent updating (i.e. not called directly) */
	PropagateFromParent = 0x2,		
	/** Only update child transform if attached to parent via a socket */
	OnlyUpdateIfUsingSocket = 0x4	
};

constexpr inline EUpdateTransformFlags operator|(EUpdateTransformFlags Left, EUpdateTransformFlags Right)
{
	return static_cast<EUpdateTransformFlags> ( static_cast<int32> (Left) | static_cast<int32> (Right) );
}

constexpr inline EUpdateTransformFlags operator&(EUpdateTransformFlags Left, EUpdateTransformFlags Right)
{
	return static_cast<EUpdateTransformFlags> (static_cast<int32> (Left) & static_cast<int32> (Right));
}

constexpr inline bool operator !(EUpdateTransformFlags Value)
{
	return Value == EUpdateTransformFlags::None;
}

constexpr inline EUpdateTransformFlags operator ~(EUpdateTransformFlags Value)
{
	return static_cast<EUpdateTransformFlags>(~static_cast<int32>(Value));
}

/** Converts legacy bool into the SkipPhysicsUpdate bitflag */
inline EUpdateTransformFlags SkipPhysicsToEnum(bool bSkipPhysics){ return bSkipPhysics ? EUpdateTransformFlags::SkipPhysicsUpdate : EUpdateTransformFlags::None; }

class FSceneInterface;
extern ENGINE_API void UpdateAllPrimitiveSceneInfosForSingleComponent(UActorComponent* InComponent, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfosForBatching = nullptr);
extern ENGINE_API void UpdateAllPrimitiveSceneInfosForSingleComponentInterface(IPrimitiveComponent* InComponent, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfosForBatching = nullptr);
extern ENGINE_API void UpdateAllPrimitiveSceneInfosForScenes(TSet<FSceneInterface*> ScenesToUpdateAllPrimitiveSceneInfos);

class UActorComponent;

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FActorComponentActivatedSignature, UActorComponent, OnComponentActivated, UActorComponent*, Component, bool, bReset);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FActorComponentDeactivateSignature, UActorComponent, OnComponentDeactivated, UActorComponent*, Component);

DECLARE_MULTICAST_DELEGATE_OneParam(FActorComponentGlobalCreatePhysicsSignature, UActorComponent*);
DECLARE_MULTICAST_DELEGATE_OneParam(FActorComponentGlobalDestroyPhysicsSignature, UActorComponent*);

/**
 * ActorComponent is the base class for components that define reusable behavior that can be added to different types of Actors.
 * ActorComponents that have a transform are known as SceneComponents and those that can be rendered are PrimitiveComponents.
 *
 * @see [ActorComponent](https://docs.unrealengine.com/latest/INT/Programming/UnrealArchitecture/Actors/Components/index.html#actorcomponents)
 * @see USceneComponent
 * @see UPrimitiveComponent
 */
UCLASS(DefaultToInstanced, BlueprintType, abstract, meta=(ShortTooltip="An ActorComponent is a reusable component that can be added to any actor."), config=Engine, MinimalAPI)
class UActorComponent : public UObject, public IInterface_AssetUserData, public IAsyncPhysicsStateProcessor
{
	GENERATED_BODY()

public:
	/** Create component physics state global delegate.*/
	static ENGINE_API FActorComponentGlobalCreatePhysicsSignature GlobalCreatePhysicsDelegate;
	/** Destroy component physics state global delegate.*/
	static ENGINE_API FActorComponentGlobalDestroyPhysicsSignature GlobalDestroyPhysicsDelegate;

	/**
	 * Default UObject constructor that takes an optional ObjectInitializer.
	 */
	ENGINE_API UActorComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Main tick function for the Component */
	UPROPERTY(EditDefaultsOnly, Category="ComponentTick")
	struct FActorComponentTickFunction PrimaryComponentTick;

	/** Array of tags that can be used for grouping and categorizing. Can also be accessed from scripting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Tags)
	TArray<FName> ComponentTags;

protected:
	/** Array of user data stored with the component */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = AssetUserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of user data stored with the component */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = AssetUserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;
#endif


private:
	/** Used for fast removal of end of frame update */
	int32 MarkedForEndOfFrameUpdateArrayIndex : 28;

	/** Tracks whether the component has been added to one of the world's end of frame update lists */
	uint8 MarkedForEndOfFrameUpdateState : 2;

	/** Tracks whether the component supports an early end of frame update. */
	uint8 bReadyForEarlyEndOfFrameUpdate : 1;

	/** Tracks whether the component is currently being updated or is scheduled for an async update. */
	uint8 bRenderStateUpdating : 1;

	/** Populated when the component is created and tracks the often used order of creation on a per archetype/per actor basis */
	UPROPERTY()
	int32 UCSSerializationIndex;

protected:

	enum class EComponentRegistrationState
	{
		/** Component is not in any of the listed states, code should rely on bRegistered to determine if component is registered or not. */
		None,
		/** Component is pre-registering (not registered) */
		PreRegistering,
		/** Component is pre-registered (not registered) */
		PreRegistered,
		/** Component is pre-unregistering (registered) */
		PreUnregistering,
		/** Component is pre-unregistered (registered) */
		PreUnregistered
	};

	/** Indicates if this ActorComponent is currently pre-registering or pre-registered with a scene. */
	EComponentRegistrationState RegistrationState;

	/** 
	 *  Indicates if this ActorComponent is currently registered with a scene. 
	 */
	uint8 bRegistered:1;

	/** If the render state is currently created for this component */
	uint8 bRenderStateCreated:1;

	/**
	 * Render state is being recreated for this component.  Useful if a component wants to preserve certain render state across recreate -- set before a
	 * call to DestroyRenderState_Concurrent and cleared after the corresponding call to CreateRenderState_Concurrent.  By design, only set for re-creation
	 * due to MarkRenderStateDirty, not RecreateRenderStateContext variations.  The latter are used for editor operations, file loads, or bulk setting
	 * changes, which are assumed to potentially change render data in a way that could make preserved render state incompatible.
	 */
	uint8 bRenderStateRecreating:1;

	/** If the physics state is currently created for this component */
	uint8 bPhysicsStateCreated:1;

	/** If the physics state was created while component was pre-registering, notification to GameThread is deferred next time CreatePhysicsState is called. */
	uint8 bCallAsyncPhysicsStateCreatedDelegate:1;

	/** Is this component safe to ID over the network by name?  */
	UPROPERTY()
	uint8 bNetAddressable:1;

	/**
	* When true the replication system will only replicate the registered subobjects list
	* When false the replication system will instead call the virtual ReplicateSubObjects() function where the subobjects need to be manually replicated.
	*/
	UPROPERTY(Config, EditDefaultsOnly, BlueprintReadOnly, Category=Replication, AdvancedDisplay)
	uint8 bReplicateUsingRegisteredSubObjectList : 1;

private:
	/** Is this component currently replicating? Should the network code consider it for replication? Owning Actor must be replicating first! */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Replicated, Category=ComponentReplication,meta=(DisplayName = "Component Replicates", AllowPrivateAccess = "true"))
	uint8 bReplicates:1;

	/** Is this component in need of its whole state being sent to the renderer? */
	uint8 bRenderStateDirty:1;

	/** Is this component's transform in need of sending to the renderer? */
	uint8 bRenderTransformDirty:1;

	/** Is this component's dynamic data in need of sending to the renderer? */
	uint8 bRenderDynamicDataDirty:1;

	/** Is this component's instanced data in need of sending to the renderer? */
	uint8 bRenderInstancesDirty:1;

	/** Used to ensure that any subclass of UActorComponent that overrides PostRename calls up to the Super to make OwnedComponents arrays get updated correctly */
	uint8 bRoutedPostRename:1;

public:
	/** Does this component automatically register with its owner */
	uint8 bAutoRegister:1;

protected:
	/** Check whether the component class allows reregistration during ReregisterAllComponents */
	uint8 bAllowReregistration:1;

public:
	/** Should this component be ticked in the editor */
	uint8 bTickInEditor:1;

	/** If true, this component never needs a render update. */
	uint8 bNeverNeedsRenderUpdate:1;

	/** Can we tick this concurrently on other threads? */
	uint8 bAllowConcurrentTick:1;

	/** Can this component be destroyed (via K2_DestroyComponent) by any parent */
	uint8 bAllowAnyoneToDestroyMe:1;

#if WITH_EDITORONLY_DATA
	/** @deprecated Replaced by CreationMethod */
	UPROPERTY()
	uint8 bCreatedByConstructionScript_DEPRECATED:1;

	/** @deprecated Replaced by CreationMethod */
	UPROPERTY()
	uint8 bInstanceComponent_DEPRECATED:1;
#endif

	/** Whether the component is activated at creation or must be explicitly activated. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Activation)
	uint8 bAutoActivate:1;

private:
	/** Whether the component is currently active. */
	UPROPERTY(transient, ReplicatedUsing=OnRep_IsActive)
	uint8 bIsActive:1;

public:

	/** True if this component can be modified when it was inherited from a parent actor class */
	UPROPERTY(EditDefaultsOnly, Category="Variable")
	uint8 bEditableWhenInherited:1;

	/** Cached navigation relevancy flag for collision updates */
	uint8 bNavigationRelevant : 1;

protected:
	/** Whether this component can potentially influence navigation */
	UPROPERTY(EditAnywhere, Category = Navigation, AdvancedDisplay, config)
	uint8 bCanEverAffectNavigation : 1;

public:
	/** If true, we call the virtual InitializeComponent */
	uint8 bWantsInitializeComponent:1;

	/** If true, the component will be excluded from non-editor builds */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Cooking)
	uint8 bIsEditorOnly:1;

#if WITH_EDITORONLY_DATA
private:
	/** True if this component is only used for visualization, usually a sprite or text */
	UPROPERTY()
	uint8 bIsVisualizationComponent : 1;

	/** Marks this component pending kill once PostLoad occurs. Used to clean up old native default subobjects that were removed from code */
	UPROPERTY()
	uint8 bNeedsUCSSerializationIndexEvaluted : 1;
#endif

private:
	/** Indicates that OnCreatedComponent has been called, but OnDestroyedComponent has not yet */
	uint8 bHasBeenCreated:1;

	/** Indicates that InitializeComponent has been called, but UninitializeComponent has not yet */
	uint8 bHasBeenInitialized:1;

	/** Indicates that ReadyForReplication has been called  */
	uint8 bIsReadyForReplication:1;

	/** Indicates that BeginPlay has been called, but EndPlay has not yet */
	uint8 bHasBegunPlay:1;

	/** Indicates the the destruction process has begun for this component to avoid recursion */
	uint8 bIsBeingDestroyed:1;

	/** Whether we've tried to register tick functions. Reset when they are unregistered. */
	uint8 bTickFunctionsRegistered:1;

	/** If non-null, physics state creation/destruction is done asynchronously by FPhysScene_Chaos::ProcessAsyncPhysicsStateJobs. */
	FPhysScene* PhysicsStateAsyncCreationScene;
	
	/** Cached pointer to owning actor */
	mutable AActor* OwnerPrivate;

	/** 
	 * Pointer to the world that this component is currently registered with. 
	 * This is only non-NULL when the component is registered.
	 */
	UWorld* WorldPrivate;

#if WITH_EDITOR
	/** During undo/redo it isn't safe to cache owner */
	uint8 bCanUseCachedOwner:1;

	/** Marks this component pending kill once PostLoad occurs. Used to clean up old native default subobjects that were removed from code */
	uint8 bMarkPendingKillOnPostLoad : 1;
#endif

	/** True if this component was owned by a net startup actor during level load. */
	uint8 bIsNetStartupComponent : 1;

	/** Tracks whether the component has been added to the world's pre end of frame sync list */
	uint8 bMarkedForPreEndOfFrameSync : 1;

	/**
	* Whether this component will receive a call to AsyncPhysicsTickComponent (in code) and ReceiveAsyncPhysicsTick (a Blueprint implementable event).
    * These functions will execute on the game thread and are allowed to read and write both Physics Thread (PT) and Game Thread (GT) Data .
	* Performance Warning: This will cause both threads to halt and force a full sync of data between PT and GT. The PT will block until the GT is done executing all AsyncPhysicsTick functions.
	*/
	uint8 bAsyncPhysicsTickEnabled : 1;

	enum struct EPhysicsStateAsyncCreationState : uint8
	{
		None,
		Creating,
		Destroying
	};

	/** Whether the component is creating/destroying its physics state asynchronously (or not) */
	EPhysicsStateAsyncCreationState PhysicsStateAsyncCreationState : 2;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	uint8 bComponentTickEnabled : 1 = false;

	// When set to true the component will remove itself from Iris in EndPlay(). If set to false it's
	// assumed that the component will be responsible for removing itself from Iris.
	uint8 bCallStopReplicationInEndPlay : 1 = true;
#endif

public:

	/** Describes how a component instance will be created */
	UPROPERTY()
	EComponentCreationMethod CreationMethod;

public:
	/** Returns the UCS serialization index. This can be an expensive operation in the editor if you are dealing with a component that was saved before this information was present and it needs to be calculated. */
	int32 GetUCSSerializationIndex() const
	{
#if WITH_EDITORONLY_DATA
		if (bNeedsUCSSerializationIndexEvaluted)
		{
			const_cast<UActorComponent*>(this)->DetermineUCSSerializationIndexForLegacyComponent();
		}
#endif

		return UCSSerializationIndex;
	}

private:
	/** Calculate the UCS serialization index for a component that was saved before we started saving this data */
	ENGINE_API void DetermineUCSSerializationIndexForLegacyComponent();

#if WITH_EDITOR
	/**
	 * Creates an uninitialized component descriptor from this component. Meant to be called on the class CDO.
	 */
	ENGINE_API virtual TUniquePtr<class FWorldPartitionComponentDesc> CreateClassComponentDesc() const;

public:
	/**
	 * Creates an initialized component descriptor from this component.
	 */
	ENGINE_API TUniquePtr<class FWorldPartitionComponentDesc> CreateComponentDesc() const;

	/**
	 * Creates an uninitialized component descriptor from a specific class.
	 */
	static ENGINE_API TUniquePtr<class FWorldPartitionComponentDesc> StaticCreateClassComponentDesc(const TSubclassOf<UActorComponent>& ComponentClass);
#endif

public:

	/** Whether asynchronous physics state creation is running or not. */
	bool IsAsyncCreatePhysicsStateRunning() const
	{
		return PhysicsStateAsyncCreationScene && (PhysicsStateAsyncCreationState == EPhysicsStateAsyncCreationState::Creating);
	}

	/** Whether asynchronous physics state destruction is running or not. */
	bool IsAsyncDestroyPhysicsStateRunning() const
	{
		return PhysicsStateAsyncCreationScene && (PhysicsStateAsyncCreationState == EPhysicsStateAsyncCreationState::Destroying);
	}

	/** Tracks whether the component has been added to one of the world's end of frame update lists */
	uint32 GetMarkedForEndOfFrameUpdateState() const { return MarkedForEndOfFrameUpdateState; }

	/** Tracks whether the component has been added to one of the world's end of frame update lists */
	uint32 GetMarkedForPreEndOfFrameSync() const { return bMarkedForPreEndOfFrameSync; }

	/** Initializes the list of properties that are modified by the UserConstructionScript */
	ENGINE_API void DetermineUCSModifiedProperties();

	/** Returns the list of properties that are modified by the UserConstructionScript */
	ENGINE_API void GetUCSModifiedProperties(TSet<const FProperty*>& ModifiedProperties) const;

	/** Removes specified properties from the list of UCS-modified properties */
	ENGINE_API void RemoveUCSModifiedProperties(const TArray<FProperty*>& Properties);

	/** Clears the component's UCS modified properties */
	ENGINE_API void ClearUCSModifiedProperties();

	/** True if this component can be modified when it was inherited from a parent actor class */
	ENGINE_API bool IsEditableWhenInherited() const;

	/** Indicates that OnCreatedComponent has been called, but OnDestroyedComponent has not yet */
	bool HasBeenCreated() const { return bHasBeenCreated; }

	/** Indicates that InitializeComponent has been called, but UninitializeComponent has not yet */
	bool HasBeenInitialized() const { return bHasBeenInitialized; }

	/** Indicates that ReadyForReplication has been called */
	bool IsReadyForReplication() const { return bIsReadyForReplication; }

	/** Indicates that BeginPlay has been called, but EndPlay has not yet */
	bool HasBegunPlay() const { return bHasBegunPlay; }

	/**
	 * Returns whether the component is in the process of being destroyed.
	 */
	UFUNCTION(BlueprintCallable, Category="Components", meta=(DisplayName="Is Component Being Destroyed"))
	bool IsBeingDestroyed() const
	{
		return bIsBeingDestroyed;
	}

	/** Returns true if instances of this component are created by either the user or simple construction script */
	ENGINE_API bool IsCreatedByConstructionScript() const;

	/** Handles replication of active state, handles ticking by default but should be overridden as needed */
	UFUNCTION()
	ENGINE_API virtual void OnRep_IsActive();

private:
	ENGINE_API AActor* GetActorOwnerNoninline() const;

public:
	/** Follow the Outer chain to get the  AActor  that 'Owns' this component */
	UFUNCTION(BlueprintCallable, Category="Components", meta=(Keywords = "Actor Owning Parent"))
	AActor* GetOwner() const;

	/** Templated version of GetOwner(), will return nullptr if cast fails */
	template< class T >
	T* GetOwner() const
	{
		return Cast<T>(GetOwner());
	}

	/** Getter for the cached world pointer, will return null if the component is not actually spawned in a level */
	virtual UWorld* GetWorld() const override final { return (WorldPrivate ? WorldPrivate : GetWorld_Uncached()); }

	/** See if this component contains the supplied tag */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API bool ComponentHasTag(FName Tag) const;


	// Activation System

	/** Called when the component has been activated, with parameter indicating if it was from a reset */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FActorComponentActivatedSignature OnComponentActivated;

	/** Called when the component has been deactivated */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FActorComponentDeactivateSignature OnComponentDeactivated;

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TArray<FSimpleMemberReference> UCSModifiedProperties_DEPRECATED;
#endif

	static ENGINE_API FTransactionallySafeRWLock AllUCSModifiedPropertiesLock;
	static ENGINE_API TMap<UActorComponent*, TArray<FSimpleMemberReference>> AllUCSModifiedProperties;

public:
	/**
	 * Activates the SceneComponent, should be overridden by native child classes.
	 * @param bReset - Whether the activation should happen even if ShouldActivate returns false.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void Activate(bool bReset=false);
	
	/**
	 * Deactivates the SceneComponent.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void Deactivate();

	/**
	 * Sets whether the component is active or not
	 * @param bNewActive - The new active state of the component
	 * @param bReset - Whether the activation should happen even if ShouldActivate returns false.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void SetActive(bool bNewActive, bool bReset=false);

	/**
	 * Toggles the active state of the component
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void ToggleActive();

	/**
	 * Returns whether the component is active or not
	 * @return - The active state of the component.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation", meta=(UnsafeDuringActorConstruction="true"))
	bool IsActive() const { return bIsActive; }

	/**
	 * Sets whether the component should be auto activate or not. Only safe during construction scripts.
	 * @param bNewAutoActivate - The new auto activate state of the component
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation")
	ENGINE_API virtual void SetAutoActivate(bool bNewAutoActivate);

	/** Sets whether this component can tick when paused. */
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API void SetTickableWhenPaused(bool bTickableWhenPaused);

	/** Create any physics engine information for this component */
	ENGINE_API void CreatePhysicsState(bool bAllowDeferral = false);

	/** Shut down any physics engine structure for this component */
	ENGINE_API void DestroyPhysicsState();


	// Networking

	/** This signifies the component can be ID'd by name over the network. This only needs to be called by engine code when constructing blueprint components. */
	ENGINE_API void SetNetAddressable();

	/** Enable or disable replication. This is the equivalent of RemoteRole for actors (only a bool is required for components) */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API void SetIsReplicated(bool ShouldReplicate);

	/** Returns whether replication is enabled or not. */
	inline bool GetIsReplicated() const
	{
		return bReplicates;
	}

	/** 
	* Allows a component to replicate other subobject on the actor.
	* Must return true if any data gets serialized into the bunch.
	* This method is used only when bReplicateUsingRegisteredSubObjectList is false.
	* Otherwise this function is not called and only the ReplicatedSubObjects list is used.
	*/
	ENGINE_API virtual bool ReplicateSubobjects(class UActorChannel *Channel, class FOutBunch *Bunch, FReplicationFlags *RepFlags);

	/** 
	* Returns if this component is replicating subobjects via the registration list or via the virtual ReplicateSubObjects method.
	* Note: the owning actor of this component must also have it's bReplicateUsingRegisteredSubObjectList flag set to true.
	*/
	bool IsUsingRegisteredSubObjectList() const { return bReplicateUsingRegisteredSubObjectList; }

	/** Returns the replication condition for the component. Only works if the actor owning the component has bReplicateUsingRegisteredSubObjectList enabled. */
	virtual ELifetimeCondition GetReplicationCondition() const { return GetIsReplicated() ? COND_None : COND_Never;  }

	/** Called on the component right before replication occurs */
	virtual void PreReplication(IRepChangedPropertyTracker & ChangedPropertyTracker) {}

	/** Returns true if this type of component can ever replicate, override to disable the default behavior */
	ENGINE_API virtual bool GetComponentClassCanReplicate() const;

	/**
	* Register a SubObject that will get replicated along with the actor component.
	* The subobject needs to be manually removed from the list before it gets deleted.
	* @param SubObject The subobject to replicate
	* @param NetCondition Optional condition to select which type of connection we will replicate the object to.
	*/
	ENGINE_API void AddReplicatedSubObject(UObject* SubObject, ELifetimeCondition NetCondition = COND_None);

	/**
	* Unregister a SubObject to stop replicating it's properties to clients.
	* This does not remove or delete it from connections where it was already replicated.
	* By default a replicated subobject gets deleted on clients when the original pointer on the authority gets invalidated.
	* If you want to immediately remove it from client use the DestroyReplicatedSubObjectOnRemotePeers or TearOffReplicatedSubObject functions instead of this one.
	* @param SubObject The SubObject to remove
	*/
	ENGINE_API void RemoveReplicatedSubObject(UObject* SubObject);

	/**
	* Stop replicating a subobject and tell actor channels to delete the replica of this subobject next time the Actor gets replicated
	* Note it is up to the caller to delete the local object on the authority.
	* If you are using the legacy subobject replication method (ReplicateSubObjects() aka bReplicateUsingRegisteredSubObjectList=false) make sure the
	* subobject doesn't get replicated there either.
	* @param SubObject THe SubObject to delete
	*/
	ENGINE_API void DestroyReplicatedSubObjectOnRemotePeers(UObject* SubObject);

	/**
	* Stop replicating a subobject and tell actor channels who spawned a replica of this subobject to release ownership over it.
	* This means that on the remote connection the network engine will stop holding a reference to the subobject and it's up to other systems
	* to keep that reference active or the subobject will get garbage collected.
	* If you are using the legacy subobject replication method (ReplicateSubObjects() aka bReplicateUsingRegisteredSubObjectList=false) make sure the
	* subobject doesn't get replicated there either.
	* @param SubObject The SubObject to tear off
	*/
	ENGINE_API void TearOffReplicatedSubObjectOnRemotePeers(UObject* SubObject);

	/**
	* Tells if the object is registered to be replicated by this actor component.
	*/
	ENGINE_API bool IsReplicatedSubObjectRegistered(const UObject* SubObject) const;

#if WITH_EDITORONLY_DATA
	/** Returns whether this component is an editor-only object or not */
	virtual bool IsEditorOnly() const override { return bIsEditorOnly; }

	/** Called during component creation to mark this component as editor only */
	virtual void MarkAsEditorOnlySubobject() override
	{
		bIsEditorOnly = true;
		// A bit sketchy, but is best for backwards compatibility as the vast majority of editor only components were for visualization, 
		// so for the very few where visualization is not the purpose, it can be cleared after the subobject is created
		bIsVisualizationComponent = true; 
	}

	/** Returns true if this component is only used for visualization, usually a sprite or text */
	bool IsVisualizationComponent() const { return bIsVisualizationComponent; }

	/** Sets if this component is only used for visualization */
	void SetIsVisualizationComponent(const bool bInIsVisualizationComponent)
	{
		bIsVisualizationComponent = bInIsVisualizationComponent;
		if (bIsVisualizationComponent)
		{
			bIsEditorOnly = true;
		}
	}
#endif

	/** Returns true if we are replicating and this client is not authoritative */
	bool IsNetSimulating() const;

	/** Get the network role of the Owner, or ROLE_None if there is no owner. */
	ENGINE_API ENetRole GetOwnerRole() const;

	/**
	 * Get the network mode (dedicated server, client, standalone, etc) for this component.
	 * @see IsNetMode()
	 */
	ENetMode GetNetMode() const;

	/**
	* Test whether net mode is the given mode.
	* In optimized non-editor builds this can be more efficient than GetNetMode()
	* because it can check the static build flags without considering PIE.
	*/
	bool IsNetMode(ENetMode Mode) const;

	/** Returns true if this component was owned by a net startup actor during level load. */
	bool IsNetStartupComponent() const { return bIsNetStartupComponent; }

	/** This should only be called by the engine in ULevel::InitializeNetworkActors to initialize bIsNetStartupComponent. */
	void SetIsNetStartupComponent(const bool bInIsNetStartupComponent) { bIsNetStartupComponent = bInIsNetStartupComponent; }

	/** Allows components to handle an EOF update happening mid tick. Can be used to block on in-flight async tasks etc. This should ensure the the component's tick is complete so that it's render update is correct. */
	virtual void OnEndOfFrameUpdateDuringTick() {}

	/** Allows components to wait on outstanding tasks prior to sending EOF update data. Executed on Game Thread and may await tasks. */
	virtual void OnPreEndOfFrameSync() {}

	/** Register all replication fragments */
	ENGINE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
	
	struct FOnReplicationStartedParams
	{
	};

	/** Called when we start replicating this component with iris, Do not be call explicitly. */
	ENGINE_API virtual void OnReplicationStartedForIris(const FOnReplicationStartedParams&);

	struct FOnStopReplicationParams
	{
	};
	/** Called before we end replicating this component with ris, Do not be call explicitly. */
	ENGINE_API virtual void OnStopReplicationForIris(const FOnStopReplicationParams&);

	/** Called when we want to start replicating this component, should not be called explicitly.*/
	UE_DEPRECATED(5.7, "This is being deprecated as part of iris beta. Override OnReplicationStartedForIris instead.")
	virtual void BeginReplication() {}

	/** Tell component to end replication, should not be called explicitly. */
	UE_DEPRECATED(5.7, "This is being deprecated as part of iris beta. Override OnStopReplicationForIris instead.")
	virtual void EndReplication() {}

private:

	/** If WorldPrivate isn't set this will determine the world from outers */
	ENGINE_API UWorld* GetWorld_Uncached() const;

	/** Private version without inlining that does *not* check Dedicated server build flags (which should already have been done). */
	ENGINE_API ENetMode InternalGetNetMode() const;

	/** Calls OnUnregister, DestroyRenderState_Concurrent and OnDestroyPhysicsState. */
	ENGINE_API void ExecuteUnregisterEvents();

	/** Calls OnRegister, CreateRenderState_Concurrent and OnCreatePhysicsState. */
	ENGINE_API void ExecuteRegisterEvents(FRegisterComponentContext* Context = nullptr);

	/** Utility function for each of the PostEditChange variations to call for the same behavior */
	ENGINE_API void ConsolidatedPostEditChange(const FPropertyChangedEvent& PropertyChangedEvent);

protected:
	/** Return true if this component is in a state where it can be activated normally. */
	ENGINE_API virtual bool ShouldActivate() const;

	/** Returns whether this component should create its physics state asynchronously. */
	ENGINE_API virtual bool ShouldAsyncCreatePhysicsState(UWorld* WorldContext) const;

	/** Returns whether this component should destroy its physics state asynchronously. */
	ENGINE_API virtual bool ShouldAsyncDestroyPhysicsState() const;

	/** Called by PreRegisterComponentWithWorld when the component is being pre-registered. */
	ENGINE_API virtual void OnPreRegister();

	/** Called once the component is pre-registered. */
	ENGINE_API virtual void OnPreRegistered();

	/** Called by PreUnregisterComponentFromWorld when the component is being pre-unregistered. */
	ENGINE_API virtual void OnPreUnregister();

	/** Called once the component is pre-unregistered. */
	ENGINE_API virtual void OnPreUnregistered();

	/**
	 * Called when a component is registered, after Scene is set, but before CreateRenderState_Concurrent or OnCreatePhysicsState are called.
	 */
	ENGINE_API virtual void OnRegister();

	/**
	 * Called when a component is unregistered. Called after DestroyRenderState_Concurrent and OnDestroyPhysicsState are called.
	 */
	ENGINE_API virtual void OnUnregister();

	/** Precache all PSOs which can be used by the actor component */
	virtual void PrecachePSOs() {}

	/** Return true if CreateRenderState() should be called */
	virtual bool ShouldCreateRenderState() const 
	{
		return false;
	}

	/** 
	 * Used to create any rendering thread information for this component
	 * @warning This is called concurrently on multiple threads (but never the same component concurrently)
	 */
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context);

	/** 
	 * Called to send a transform update for this component to the rendering thread
	 * @warning This is called concurrently on multiple threads (but never the same component concurrently)
	 */
	ENGINE_API virtual void SendRenderTransform_Concurrent();

	/** Called to send dynamic data for this component to the rendering thread */
	ENGINE_API virtual void SendRenderDynamicData_Concurrent();

	/** Called to send instance data for this component to the rendering thread */
	ENGINE_API virtual void SendRenderInstanceData_Concurrent();

	/** 
	 * Used to shut down any rendering thread structure for this component
	 * @warning This is called concurrently on multiple threads (but never the same component concurrently)
	 */
	ENGINE_API virtual void DestroyRenderState_Concurrent();

	/** Used to create any physics engine information for this component */
	ENGINE_API virtual void OnCreatePhysicsState();

	/** Used to shut down and physics engine structure for this component */
	ENGINE_API virtual void OnDestroyPhysicsState();

	/** Return true if CreatePhysicsState() should be called.
	    Ideally CreatePhysicsState() should always succeed if this returns true, but this isn't currently the case */
	virtual bool ShouldCreatePhysicsState() const {return false;}

	/** Used to check that DestroyPhysicsState() is working correctly */
	virtual bool HasValidPhysicsState() const { return false; }

	/**
	 * Virtual call chain to register all tick functions
	 * @param bRegister - true to register, false, to unregister
	 */
	ENGINE_API virtual void RegisterComponentTickFunctions(bool bRegister);

public:

	/** Returns whether PreRegisterComponentWithWorld should be called for this component when its Level is being added to the world. */
	ENGINE_API virtual bool ShouldIncrementalPreRegister(UWorld* WorldContext) const;

	/** Returns whether PreUnregisterComponentFromWorld should be called for this component when its Level is being removed from the world. */
	ENGINE_API virtual bool ShouldIncrementalPreUnregister() const;

	//~ Begin IAsyncPhysicsStateProcessor
	virtual bool AllowsAsyncPhysicsStateCreation() const
	{
		return false;
	}
	virtual bool AllowsAsyncPhysicsStateDestruction() const
	{
		return false;
	}
	ENGINE_API virtual bool IsAsyncPhysicsStateCreated() const override final;
	ENGINE_API virtual UObject* GetAsyncPhysicsStateObject() const override final;
	ENGINE_API virtual bool OnAsyncCreatePhysicsState(const UE::FTimeout& Timeout) override;
	ENGINE_API virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override { return true; }
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
	//~ End IAsyncPhysicsStateProcessor
	/**
	 * Initializes the component.  Occurs at level startup or actor spawn. This is before BeginPlay (Actor or Component).  
	 * All Components in the level will be Initialized on load before any Actor/Component gets BeginPlay
	 * Requires component to be registered, and bWantsInitializeComponent to be true.
	 */
	ENGINE_API virtual void InitializeComponent();

	/**
	 * ReadyForReplication gets called on replicated components when their owning actor is officially ready for replication.
	 * Called after InitializeComponent but before BeginPlay. From there it will only be set false when the component is destroyed.
	 * This is where you want to register your replicated subobjects if you already possess some. 
	 * A component can get replicated before HasBegunPlay() is true if inside a tick or in BeginPlay() an RPC is called on it.
	 * Requires component to be registered, initialized and set to replicate.
	 */
	ENGINE_API virtual void ReadyForReplication();

	/**
	 * Begins Play for the component. 
	 * Called when the owning Actor begins play or when the component is created if the Actor has already begun play.
	 * Actor BeginPlay normally happens right after PostInitializeComponents but can be delayed for networked or child actors.
	 * Requires component to be registered and initialized.
	 */
	ENGINE_API virtual void BeginPlay();

	/** 
	 * Blueprint implementable event for when the component is beginning play, called before its owning actor's BeginPlay
	 * or when the component is dynamically created if the Actor has already BegunPlay. 
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Begin Play"))
	ENGINE_API void ReceiveBeginPlay();

	/**
	 * Ends gameplay for this component.
	 * Called from AActor::EndPlay only if bHasBegunPlay is true
	 */
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason);

	/**
	 * Handle this component being Uninitialized.
	 * Called from AActor::EndPlay only if bHasBeenInitialized is true
	 */
	ENGINE_API virtual void UninitializeComponent();

	/** Blueprint implementable event for when the component ends play, generally via destruction or its Actor's EndPlay. */
	UFUNCTION(BlueprintImplementableEvent, meta=(Keywords = "delete", DisplayName = "End Play"))
	ENGINE_API void ReceiveEndPlay(EEndPlayReason::Type EndPlayReason);
	
	/**
	 * When called, will call the virtual call chain to register all of the tick functions
	 * Do not override this function or make it virtual
	 * @param bRegister - true to register, false, to unregister
	 */
	ENGINE_API void RegisterAllComponentTickFunctions(bool bRegister);

	/**
	 * Function called every frame on this ActorComponent. Override this function to implement custom logic to be executed every frame.
	 * Only executes if the component is registered, and also PrimaryComponentTick.bCanEverTick must be set to true.
	 *	
	 * @param DeltaTime - The time since the last tick.
	 * @param TickType - The kind of tick this is, for example, are we paused, or 'simulating' in the editor
	 * @param ThisTickFunction - Internal tick function struct that caused this to run
	 */
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction);

	/**
	 * Override this function to implement custom logic to be executed every physics step.
	 * Only executes if the component is registered, and also bAsyncPhysicsTick must be set to true.
	 *	
	 * @param DeltaTime - The sim time associated with each step
	 * @param SimTime - This is the total sim time since the sim began
	 */
	virtual void AsyncPhysicsTickComponent(float DeltaTime, float SimTime) { ReceiveAsyncPhysicsTick(DeltaTime, SimTime); }
	
	/** 
	 * Set up a tick function for a component in the standard way. 
	 * Tick after the actor. Don't tick if the actor is static, or if the actor is a template or if this is a "NeverTick" component.
	 * I tick while paused if only if my owner does.
	 * @param	TickFunction - structure holding the specific tick function
	 * @return  true if this component met the criteria for actually being ticked.
	 */
	ENGINE_API bool SetupActorComponentTickFunction(struct FTickFunction* TickFunction);

	/** 
	 * Set this component's tick functions to be enabled or disabled. Only has an effect if the function is registered
	 * 
	 * @param	bEnabled - Whether it should be enabled or not
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API virtual void SetComponentTickEnabled(bool bEnabled);

	/** 
	 * Spawns a task on GameThread that will call SetComponentTickEnabled
	 * @param	bEnabled - Whether it should be enabled or not
	 */
	ENGINE_API virtual void SetComponentTickEnabledAsync(bool bEnabled);
	
	/** 
	 * Returns whether this component has tick enabled or not
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API virtual bool IsComponentTickEnabled() const;

	/** 
	* Sets the tick interval for this component's primary tick function. Does not enable the tick interval. Takes effect on next tick.
	* @param TickInterval	The duration between ticks for this component's primary tick function
	*/
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API void SetComponentTickInterval(float TickInterval);

	/**
	* Sets the tick interval for this component's primary tick function. Does not enable the tick interval. Takes effect imediately.
	* @param TickInterval	The duration between ticks for this component's primary tick function
	*/
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API void SetComponentTickIntervalAndCooldown(float TickInterval);

	/** Returns the tick interval for this component's primary tick function, which is the frequency in seconds at which it will be executed */
	UFUNCTION(BlueprintCallable, Category="Components|Tick")
	ENGINE_API float GetComponentTickInterval() const;

	/**
	 * Pre-registers a component with a specific world. Only called on components that return ShouldIncrementalPreRegister().
	 * Will initialize the necessary before calling OnPreRegister().
	 * @param InWorld - The world to pre-register the component with.
	 */
	ENGINE_API void PreRegisterComponentWithWorld(UWorld* InWorld);

	/**
	 * Pre-unregisters a component from a specific world. Only called on components that return ShouldIncrementalPreUnregister().
	 */
	ENGINE_API void PreUnregisterComponentFromWorld();

	/**
	 * Registers a component with a specific world, which creates any visual/physical state
	 * @param InWorld - The world to register the component with.
	 */
	ENGINE_API void RegisterComponentWithWorld(UWorld* InWorld, FRegisterComponentContext* Context = nullptr);

	/** Overridable check for a component to indicate to its Owner that it should prevent the Actor from auto destroying when finished */
	virtual bool IsReadyForOwnerToAutoDestroy() const { return true; }

	/** Returns whether the component's owner is selected in the editor */
	ENGINE_API bool IsOwnerSelected() const;

	/** Is this component's transform in need of sending to the renderer? */
	inline bool IsRenderTransformDirty() const { return bRenderTransformDirty; }

	/** Is this component's instance data in need of sending to the renderer? */
	inline bool IsRenderInstancesDirty() const { return bRenderInstancesDirty; }

	/** Is this component in need of its whole state being sent to the renderer? */
	inline bool IsRenderStateDirty() const { return bRenderStateDirty; }

	/** Invalidate lighting cache with default options. */
	void InvalidateLightingCache()
	{
		InvalidateLightingCacheDetailed(true, false);
	}

	/**
	 * Called when this actor component has moved, allowing it to discard statically cached lighting information.
	 */
	virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) {}

#if WITH_EDITOR
	/**
	 * Function that gets called from within Map_Check to allow this actor component to check itself
	 * for any potential errors and register them with map check dialog.
	 * @note Derived class implementations should call up to their parents.
	 */
	ENGINE_API virtual void CheckForErrors();

	/** 
	 * Supplies the editor with a view specific to this component (think a view 
	 * from a camera components POV, a render, etc.). Used for PIP preview windows.
	 * @return True if the component overrides this, and fills out the view info output.
	 */
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) { return false; }

	/**
	 * If this component is set up to provide a preview window in editor (see GetEditorPreviewInfo), 
	 * you can use this to customize the preview (to be something other than a world viewport).
	 * If this returns an empty pointer, then the preview will default to a viewport using the FMinimalViewInfo
	 * data from GetEditorPreviewInfo().
	 */
	virtual TSharedPtr<SWidget> GetCustomEditorPreviewWidget() { return TSharedPtr<SWidget>(); }

	/**
	 * Return the custom HLODBuilder class that should be used to generate HLODs for components of this type.
	 * Allows the HLOD system to include whatever is needed to represent a component at a distance.
	 * For example, this is currently used to build custom HLODs for landscape streaming proxies. 
	 * This could be used to generate fake lights, represent custom FX at a distance, or even to insert
	 * externally generated HLODs.
	 */
	virtual TSubclassOf<class UHLODBuilder> GetCustomHLODBuilderClass() const { return nullptr; }

	/**
	 * Add properties to the component owner's actor desc.
	 */
	virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const {}
#endif // WITH_EDITOR

	/**
	 * Uses the bRenderStateDirty/bRenderTransformDirty/bRenderInstancesDirty to perform any necessary work on this component.
	 * Do not call this directly, call MarkRenderStateDirty, MarkRenderDynamicDataDirty,  MarkRenderInstancesDataDirty,
	 *
	 * @warning This is called concurrently on multiple threads (but never the same component concurrently)
	 */
	ENGINE_API void DoDeferredRenderUpdates_Concurrent();

	/** Recalculate the value of our component to world transform */
	virtual void UpdateComponentToWorld(EUpdateTransformFlags UpdateTransformFlags = EUpdateTransformFlags::None, ETeleportType Teleport = ETeleportType::None){}

	/** Mark the render state as dirty - will be sent to the render thread at the end of the frame. */
	ENGINE_API void MarkRenderStateDirty();

	/** Indicate that dynamic data for this component needs to be sent at the end of the frame. */
	ENGINE_API void MarkRenderDynamicDataDirty();

	/** Marks the transform as dirty - will be sent to the render thread at the end of the frame*/
	ENGINE_API void MarkRenderTransformDirty();

	/** Marks the instances as dirty - changes to instance transforms/custom data will be sent to the render thread at the end of the frame*/
	ENGINE_API void MarkRenderInstancesDirty();

	/** If we belong to a world, mark this for a deferred update, otherwise do it now. If bReadyForEarlyUpdate is true we may launch an async update ASAP. */
	ENGINE_API void MarkForNeededEndOfFrameUpdate(bool bReadyForEarlyUpdate = false);

	/** If we belong to a world, mark this for a deferred update, otherwise do it now. */
	ENGINE_API void MarkForNeededEndOfFrameRecreate();

	inline bool IsReadyForEarlyEndOfFrameUpdate() const
	{
		return bReadyForEarlyEndOfFrameUpdate;
	}

	/** If we belong to a world, clear the request to do a deferred update. */
	void ClearNeedEndOfFrameUpdate();

	/** return true if this component requires end of frame updates to happen from the game thread. */
	ENGINE_API virtual bool RequiresGameThreadEndOfFrameUpdates() const;

	/** return true if this component requires end of frame recreates to happen from the game thread. */
	ENGINE_API virtual bool RequiresGameThreadEndOfFrameRecreate() const;

	/** return true if this component needs to sync to tasks before end of frame updates are executed */
	ENGINE_API virtual bool RequiresPreEndOfFrameSync() const;

	/** 
	 * Recreate the render state right away. Generally you always want to call MarkRenderStateDirty instead. 
	 * @warning This is called concurrently on multiple threads (but never the same component concurrently)
	 */
	ENGINE_API void RecreateRenderState_Concurrent();

	/** Recreate the physics state right way. */
	ENGINE_API void RecreatePhysicsState();

	/** Returns true if the render 'state' (e.g. scene proxy) is created for this component */
	bool IsRenderStateCreated() const
	{
		return bRenderStateCreated;
	}

	/** Returns true if the render 'state' is being recreated for this component (see additional comments on variable above) */
	bool IsRenderStateRecreating() const
	{
		return bRenderStateRecreating;
	}

	/** Returns true if the physics 'state' (e.g. physx bodies) are created for this component */
	bool IsPhysicsStateCreated() const
	{
		return bPhysicsStateCreated;
	}

	bool IsRenderStateUpdating() const
	{
		return bRenderStateUpdating;
	}

	/** Returns the rendering scene associated with this component */
	ENGINE_API class FSceneInterface* GetScene() const;

	/** Returns the physics scene associated with this component */
	ENGINE_API FPhysScene* GetPhysicsScene() const;

	/** Return the ULevel that this Component is part of. */
	ENGINE_API ULevel* GetComponentLevel() const;

	/** Returns true if this actor is contained by TestLevel. */
	ENGINE_API bool ComponentIsInLevel(const class ULevel *TestLevel) const;

	/** See if this component is in the persistent level */
	ENGINE_API bool ComponentIsInPersistentLevel(bool bIncludeLevelStreamingPersistent) const;

	/** Called on each component when the Actor's visibility state changes */
	virtual void OnActorVisibilityChanged() { MarkRenderStateDirty(); }

	/** Called on each component when the Actor's bEnableCollisionChanged flag changes */
	virtual void OnActorEnableCollisionChanged() {}

	/** 
	 * Returns a readable name for this component, including the asset name if applicable 
	 * By default this appends a space plus AdditionalStatObject()
	 */
	ENGINE_API virtual FString GetReadableName() const;

	/** Give a readable name for this component, including asset name if applicable */
	virtual UObject const* AdditionalStatObject() const
	{
		return nullptr;
	}

	/** Called before we throw away components during RerunConstructionScripts, to cache any data we wish to persist across that operation */
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const;

	/** Called after ApplyToComponent has run. */
	ENGINE_API virtual void PostApplyToComponent();

	/**
	 * Get the logical child elements of this component, if any.
	 * @see UTypedElementHierarchyInterface.
	 */
	virtual void GetComponentChildElements(TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) {}

	//~ Begin UObject Interface.
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool NeedsLoadForClient() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
	ENGINE_API virtual bool NeedsLoadForEditorGame() const override;
	ENGINE_API virtual bool IsNameStableForNetworking() const override;
	ENGINE_API virtual bool IsSupportedForNetworking() const override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual int32 GetFunctionCallspace( UFunction* Function, FFrame* Stack ) override;
	ENGINE_API virtual bool CallRemoteFunction( UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack ) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif
	ENGINE_API virtual bool Rename( const TCHAR* NewName=NULL, UObject* NewOuter=NULL, ERenameFlags Flags=REN_None ) override;
	ENGINE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#if WITH_EDITOR
	ENGINE_API virtual bool Modify( bool bAlwaysMarkDirty = true ) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty( FPropertyChangedChainEvent& PropertyChangedEvent ) override;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual bool IsSelectedInEditor() const override;
	virtual void SetPackageExternal(bool bExternal, bool bShouldDirty) {}
	virtual FBox GetStreamingBounds() const { return FBox(ForceInit); }
	virtual FBox GetStreamingBoundsEditor() const { return GetStreamingBounds(); }
	virtual bool ForceActorNonSpatiallyLoaded() const { return false; }
	virtual bool ForceActorNoDataLayers() const { return false; }
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	//~ Begin IInterface_ActorSubobject Interface
	// We're using the same API as IInterface_ActorSubobject, but not using it directly as a size and performance optimization
	ENGINE_API void OnCreatedFromReplication();
	ENGINE_API void OnDestroyedFromReplication();
	//~ End IInterface_ActorSubobject Interface

	/** See if the owning Actor is currently running the UCS */
	ENGINE_API bool IsOwnerRunningUserConstructionScript() const;

	/** See if this component is currently pre-registering */
	inline bool IsPreRegistering() const
	{
		return RegistrationState == EComponentRegistrationState::PreRegistering;
	}

	/** See if this component is currently pre-registered */
	inline bool IsPreRegistered() const
	{
		return RegistrationState == EComponentRegistrationState::PreRegistered;
	}

	/** See if this component is currently pre-unregistering */
	inline bool IsPreUnregistering() const
	{
		return RegistrationState == EComponentRegistrationState::PreUnregistering;
	}

	/** See if this component is currently pre-unregistered */
	inline bool IsPreUnregistered() const
	{
		return RegistrationState == EComponentRegistrationState::PreUnregistered;
	}

	/** See if this component is currently registered */
	inline bool IsRegistered() const { return bRegistered; }

	/** Check whether the component class allows reregistration during ReregisterAllComponents */
	inline bool AllowReregistration() const { return bAllowReregistration; }

	/** Register this component, creating any rendering/physics state. Will also add itself to the outer Actor's Components array, if not already present. */
	ENGINE_API void RegisterComponent();

	/** Unregister this component, destroying any rendering/physics state. */
	ENGINE_API void UnregisterComponent();

	/** Unregister the component, remove it from its outer Actor's Components array and mark for pending kill. */
	ENGINE_API virtual void DestroyComponent(bool bPromoteChildren = false);

	/** Called when a component is created (not loaded). This can happen in the editor or during gameplay */
	ENGINE_API virtual void OnComponentCreated();

	/** 
	 * Called when a component is destroyed
	 * 
	 * @param	bDestroyingHierarchy  - True if the entire component hierarchy is being torn down, allows avoiding expensive operations
	 */
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy);

	/**
	 * Unregister and mark for pending kill a component.  This may not be used to destroy a component that is owned by an actor unless the owning actor is calling the function.
	 */
	UFUNCTION(BlueprintCallable, Category="Components", meta=(Keywords = "Delete", HidePin="Object", DefaultToSelf="Object", DisplayName = "Destroy Component", ScriptName = "DestroyComponent"))
	ENGINE_API void K2_DestroyComponent(UObject* Object);

	/** Unregisters and immediately re-registers component. */
	ENGINE_API void ReregisterComponent();

	/** Changes the ticking group for this component */
	UFUNCTION(BlueprintCallable, Category="Components|Tick", meta=(Keywords = "dependency"))
	ENGINE_API void SetTickGroup(ETickingGroup NewTickGroup);

	/** Make this component tick after PrerequisiteActor */
	UFUNCTION(BlueprintCallable, Category="Components|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void AddTickPrerequisiteActor(AActor* PrerequisiteActor);

	/** Make this component tick after PrerequisiteComponent. */
	UFUNCTION(BlueprintCallable, Category="Components|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void AddTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent);

	/** Remove tick dependency on PrerequisiteActor. */
	UFUNCTION(BlueprintCallable, Category="Components|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void RemoveTickPrerequisiteActor(AActor* PrerequisiteActor);

	/** Remove tick dependency on PrerequisiteComponent. */
	UFUNCTION(BlueprintCallable, Category="Components|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void RemoveTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent);

	/** Event called every frame if tick is enabled */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Tick"))
	ENGINE_API void ReceiveTick(float DeltaSeconds);

	/** Event called every async physics tick if bAsyncPhysicsTickEnabled is true */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Async Physics Tick"))
	ENGINE_API void ReceiveAsyncPhysicsTick(float DeltaSeconds, float SimSeconds);
	
	/** 
	 *  Called by owner actor on position shifting
	 *  Component should update all relevant data structures to reflect new actor location
	 *
	 * @param InWorldOffset	 Offset vector the actor shifted by
	 * @param bWorldShift	 Whether this call is part of whole world shifting
	 */
	virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) {};

	/** Can this component potentially influence navigation */
	bool CanEverAffectNavigation() const;

	/**
	 * Sets value of bCanEverAffectNavigation flag and update navigation octree if needed.
	 * @note This flag influences the required updates when setting other properties associated
	 * to the navigation data (e.g., transform, primitive, mesh, etc.).
	 * So when setting it to 'true' it is more efficient to set the other properties first,
	 * otherwise ('false') it is more efficient to set this one first, then the other properties.
	 */
	ENGINE_API void SetCanEverAffectNavigation(bool bRelevant);

	/** Override to specify that a component is relevant to the navigation system */
	virtual bool IsNavigationRelevant() const { return false; }

	/** Override to specify that a component is relevant to the HLOD generation. */
	virtual bool IsHLODRelevant() const { return false; }

#if WITH_EDITOR
	/** Override to provide a custom set of components to be used in place of the current component for the HLOD generation. */
	virtual TArray<UActorComponent*> GetHLODProxyComponents() const { return TArray<UActorComponent*>(); }

	/** Override to allow a component subclass to compute an hash for the purpose of HLOD generation. */
	virtual void ComputeHLODHash(class FHLODHashBuilder& HashBuilder) const	{}
#endif

	/** Suffix used to identify template component instances */
	static ENGINE_API const FString ComponentTemplateNameSuffix;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMarkRenderStateDirty, UActorComponent&);

	/** Called When render state is marked dirty */
	static ENGINE_API FOnMarkRenderStateDirty MarkRenderStateDirtyEvent;

protected:
	/** Utility method used by subclasses to test basic conditions a component must fulfill to be considered HLOD relevant. */
	static ENGINE_API bool CanBeHLODRelevant(const UActorComponent* ActorComponent);

	/** Makes sure navigation system has up to date information regarding component's navigation relevancy 
	 *	and if it can affect navigation at all 
	 *	@param bForceUpdate by default updating navigation system will take place only if the component has
	 *		already been registered. Setting bForceUpdate to true overrides that check */
	ENGINE_API void HandleCanEverAffectNavigationChange(bool bForceUpdate = false);

	/**
	* Sets bAsyncPhysicsTickEnabled which determines whether this component will receive a call to AsyncPhysicsTickComponent (in code) and ReceiveAsyncPhysicsTick (a Blueprint implementable event).
	* These functions will execute on the game thread and are allowed to read and write both Physics Thread (PT) and Game Thread (GT) Data .
	* Performance Warning: This will cause both threads to halt and force a full sync of data between PT and GT. The PT will block until the GT is done executing all AsyncPhysicsTick functions.
	*/
	ENGINE_API void SetAsyncPhysicsTickEnabled(bool bEnabled);

	/** Defers the call to SetAsyncPhysicsTickEnabled(false) to the next async tick. This allows you to remove the
	 *  async physics tick callback within the async physics tick safely. */
	ENGINE_API void DeferRemoveAsyncPhysicsTick();

private:
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** This is the old name of the tick function. We just want to avoid mistakes with an attempt to override this */
	virtual void Tick( float DeltaTime ) final { check(0); }
#endif

	ENGINE_API void ClearNeedEndOfFrameUpdate_Internal();

	ENGINE_API void RegisterAsyncPhysicsTickEnabled(bool bRegister);

	friend struct FMarkComponentEndOfFrameUpdateState;
	friend struct FSetUCSSerializationIndex;
	friend class FActorComponentDetails;
	friend class FComponentReregisterContextBase;
	friend class FComponentRecreateRenderStateContext;
	friend struct FActorComponentTickFunction;
#if WITH_EDITOR
	friend struct SkeletalMeshUCSModifiedDataCleanupHelper;
#endif

	//~ Begin Methods for Replicated Members.
protected:

	/**
	 * Sets the value of bReplicates without causing other side effects to this instance.
	 * This should only be called during component construction.
	 *
	 * This method exists only to allow code to directly modify bReplicates to maintain existing
	 * behavior.
	 */
	ENGINE_API void SetIsReplicatedByDefault(const bool bNewReplicates);

	/**
	 * Gets the property name for bReplicates.
	 *
	 * This exists so subclasses don't need to have direct access to the bReplicates property so it
	 * can be made private later.
	 */
	static const FName GetReplicatesPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UActorComponent, bReplicates);
	}

public:

	/**
	 * Sets the value of bIsActive without causing other side effects to this instance.
	 *
	 * Activate, Deactivate, and SetActive are preferred in most cases because they respect virtual behavior.
	 */
	ENGINE_API void SetActiveFlag(const bool bNewIsActive);

	//~ End Methods for Replicated Members.

protected:

	/** Convenience method for testing whether or not our owner is still being constructed / initialized. */
	ENGINE_API bool OwnerNeedsInitialization() const;

	/** Convenience method for testing whether or not we are still be constructed / initialized. */
	ENGINE_API bool NeedsInitialization() const;
};

//////////////////////////////////////////////////////////////////////////
// UActorComponent inlines

inline bool UActorComponent::CanEverAffectNavigation() const
{
	return bCanEverAffectNavigation;
}

inline bool UActorComponent::IsNetSimulating() const
{
	return GetIsReplicated() && GetOwnerRole() != ROLE_Authority;
}

inline ENetMode UActorComponent::GetNetMode() const
{
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (IsRunningDedicatedServer())
	{
		return NM_DedicatedServer;
	}

	return InternalGetNetMode();
}

inline bool UActorComponent::IsNetMode(ENetMode Mode) const
{
#if UE_EDITOR
	// Editor builds are special because of PIE, which can run a dedicated server without the app running with -server.
	return GetNetMode() == Mode;
#else
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (Mode == NM_DedicatedServer)
	{
		return IsRunningDedicatedServer();
	}
	else
	{
		return !IsRunningDedicatedServer() && (InternalGetNetMode() == Mode);
	}
#endif // UE_EDITOR
}

inline void UActorComponent::ClearNeedEndOfFrameUpdate()
{
	if (MarkedForEndOfFrameUpdateState != 0)
	{
		ClearNeedEndOfFrameUpdate_Internal();
	}
}

inline AActor* UActorComponent::GetOwner() const
{
#if WITH_EDITOR
	// During undo/redo the cached owner is unreliable so just used GetTypedOuter
	if (bCanUseCachedOwner)
	{
		checkSlow(OwnerPrivate == GetActorOwnerNoninline()); // verify cached value is correct
		return OwnerPrivate;
	}
	else
	{
		return GetActorOwnerNoninline();
	}
#else
	checkSlow(OwnerPrivate == GetActorOwnerNoninline()); // verify cached value is correct
	return OwnerPrivate;
#endif
}

#if WITH_EDITOR
/** Callback for editor component selection. This must be in engine instead of editor for UActorComponent::IsSelectedInEditor to work */
extern ENGINE_API TFunction<bool(const UActorComponent*)> GIsComponentSelectedInEditor;
#endif
