// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ComponentInterfaces.h"

class FActorPrimitiveComponentInterface : public IPrimitiveComponent
{
public:
	bool IsRenderStateCreated() const override;
	bool IsRenderStateDirty() const override;
	bool ShouldCreateRenderState() const override;
	bool IsRegistered() const override;
	bool IsUnreachable() const override;
	bool IsStaticMobility() const override;
	bool IsMipStreamingForced() const override;

	UWorld* GetWorld() const override;
	FSceneInterface* GetScene() const override;
	FPrimitiveSceneProxy* GetSceneProxy() const override;
	void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const;
	void MarkRenderStateDirty() override;
	void DestroyRenderState() override;
	void CreateRenderState(FRegisterComponentContext* Context) override;
	void PrecachePSOs() override;

	FString GetName() const override;
	FString GetFullName() const override;
	FTransform GetTransform() const override;
	const FBoxSphereBounds& GetBounds() const override;
	float GetLastRenderTimeOnScreen() const override;
	void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const override;

	UObject*	GetUObject() override;
	const UObject*	GetUObject() const override;
	UObject* GetOwner() const override;
	FString GetOwnerName() const override;
	FPrimitiveSceneProxy* CreateSceneProxy() override;	

	FRenderAssetOwnerStreamingState& GetStreamingState() const override;
	ULevel* GetComponentLevel() const override;
	IPrimitiveComponent* GetLODParentPrimitive() const override;
	float GetMinDrawDistance() const override;
	float GetStreamingScale() const override;

	void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex) override;
	UStreamableRenderAsset* GetStreamableNaniteAsset() const override;
	void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;

#if WITH_EDITOR
	HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) override;
#endif
	HHitProxy* CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
};
