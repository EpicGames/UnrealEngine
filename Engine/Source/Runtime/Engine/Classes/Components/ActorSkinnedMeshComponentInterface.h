// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Components/ComponentInterfaces.h"

class FActorSkinnedMeshComponentInterface : public ISkinnedMeshComponent
{
public:
#if WITH_EDITOR
	virtual void PostAssetCompilation() override;
	virtual void PostSkeletalHierarchyChange() override;
#endif
	virtual USkinnedAsset* GetSkinnedAsset() const override;
	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() override;
};
