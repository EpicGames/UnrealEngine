// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Components/ComponentInterfaces.h"

class FActorStaticMeshComponentInterface : public IStaticMeshComponent
{
public:
#if WITH_EDITOR
	virtual void OnMeshRebuild(bool bRenderDataChanged) override;
	virtual void PreStaticMeshCompilation() override;
	virtual void PostStaticMeshCompilation() override;
#endif
	virtual UStaticMesh* GetStaticMesh() const override;
	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() override;
};
