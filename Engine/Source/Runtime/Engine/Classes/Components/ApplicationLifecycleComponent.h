// Copyright Epic Games, Inc. All Rights Reserved.
// ApplicationLifecycleComponent.:  See FCoreDelegates for details

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "Misc/CoreDelegates.h"
#include "ApplicationLifecycleComponent.generated.h"

// A parallel enum to the temperature change severity enum in CoreDelegates
// Note if you change this, then you must change the one in CoreDelegates
UENUM(BlueprintType)
enum class ETemperatureSeverityType : uint8
{
	Unknown,
	Good,
	Bad,
	Serious,
	Critical,

	NumSeverities,
};
static_assert((int)ETemperatureSeverityType::NumSeverities == (int)FCoreDelegates::ETemperatureSeverity::NumSeverities, "TemperatureSeverity enums are out of sync");

/** Component to handle receiving notifications from the OS about application state (activated, suspended, termination, etc). */
UCLASS(ClassGroup=Utility, HideCategories=(Activation, "Components|Activation", Collision), meta=(BlueprintSpawnableComponent), MinimalAPI)
class UApplicationLifecycleComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FApplicationLifetimeDelegate);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTemperatureChangeDelegate , ETemperatureSeverityType, Severity);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLowPowerModeDelegate, bool, bInLowPowerMode);

	// This is called when the application is about to be deactivated (e.g., due to a phone call or SMS or the sleep button). 
	// The game should be paused if possible, etc... 
	UPROPERTY(BlueprintAssignable)
	FApplicationLifetimeDelegate ApplicationWillDeactivateDelegate;  
	
	// Called when the application has been reactivated (reverse any processing done in the Deactivate delegate)
	UPROPERTY(BlueprintAssignable)
	FApplicationLifetimeDelegate ApplicationHasReactivatedDelegate; 
	
	// This is called when the application is being backgrounded (e.g., due to switching  
	// to another app or closing it via the home button)  
	// The game should release shared resources, save state, etc..., since it can be  
	// terminated from the background state without any further warning.  
	UPROPERTY(BlueprintAssignable)	
	FApplicationLifetimeDelegate ApplicationWillEnterBackgroundDelegate; // for instance, hitting the home button
	
	// Called when the application is returning to the foreground (reverse any processing done in the EnterBackground delegate)
	UPROPERTY(BlueprintAssignable)
	FApplicationLifetimeDelegate ApplicationHasEnteredForegroundDelegate; 
	
	// This *may* be called when the application is getting terminated by the OS.  
	// There is no guarantee that this will ever be called on a mobile device,  
	// save state when ApplicationWillEnterBackgroundDelegate is called instead.  
	UPROPERTY(BlueprintAssignable)
	FApplicationLifetimeDelegate ApplicationWillTerminateDelegate;

	// Called when the OS is running low on resources and asks the application to free up any cached resources, drop graphics quality etc.
	UPROPERTY(BlueprintAssignable)
	FApplicationLifetimeDelegate ApplicationShouldUnloadResourcesDelegate;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FApplicationStartupArgumentsDelegate, const TArray<FString>&, StartupArguments);

	// Called with arguments passed to the application on statup, perhaps meta data passed on by another application which launched this one.
	UPROPERTY(BlueprintAssignable)
	FApplicationStartupArgumentsDelegate ApplicationReceivedStartupArgumentsDelegate;

	// Called when temperature level has changed, and receives the severity 
	UPROPERTY(BlueprintAssignable)
	FOnTemperatureChangeDelegate OnTemperatureChangeDelegate;

	// Called when we are in low power mode
	UPROPERTY(BlueprintAssignable)
	FOnLowPowerModeDelegate OnLowPowerModeDelegate;

public:
	ENGINE_API void OnRegister() override;
	ENGINE_API void OnUnregister() override;

private:
	/** Native handlers that get registered with the actual FCoreDelegates, and then proceed to broadcast to the delegates above */
	void ApplicationWillDeactivateDelegate_Handler() { ApplicationWillDeactivateDelegate.Broadcast(); }
	void ApplicationHasReactivatedDelegate_Handler() { ApplicationHasReactivatedDelegate.Broadcast(); }
	void ApplicationWillEnterBackgroundDelegate_Handler() { ApplicationWillEnterBackgroundDelegate.Broadcast(); }
	void ApplicationHasEnteredForegroundDelegate_Handler() { ApplicationHasEnteredForegroundDelegate.Broadcast(); }
	void ApplicationWillTerminateDelegate_Handler() { ApplicationWillTerminateDelegate.Broadcast(); }
	void ApplicationShouldUnloadResourcesDelegate_Handler() { ApplicationShouldUnloadResourcesDelegate.Broadcast(); }
	void ApplicationReceivedStartupArgumentsDelegate_Handler(const TArray<FString>& StartupArguments) { ApplicationReceivedStartupArgumentsDelegate.Broadcast(StartupArguments); }
	void OnTemperatureChangeDelegate_Handler(FCoreDelegates::ETemperatureSeverity Severity) { OnTemperatureChangeDelegate.Broadcast((ETemperatureSeverityType)Severity); }
	void OnLowPowerModeDelegate_Handler(bool bInLowerPowerMode) { OnLowPowerModeDelegate.Broadcast(bInLowerPowerMode); }
};



