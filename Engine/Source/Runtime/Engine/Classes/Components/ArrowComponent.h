// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "ArrowComponent.generated.h"

class FPrimitiveSceneProxy;
struct FConvexVolume;
struct FEngineShowFlags;

/** 
 * A simple arrow rendered using lines. Useful for indicating which way an object is facing.
 */
UCLASS(ClassGroup=Utility, hidecategories=(Object,LOD,Physics,Lighting,TextureStreaming,Activation,"Components|Activation",Collision), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UArrowComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Color to draw arrow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetArrowFColor, Category=ArrowComponent)
	FColor ArrowColor;

	/** Relative size to scale drawn arrow by */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetArrowSize, Category=ArrowComponent)
	float ArrowSize;

	/** Total length of drawn arrow including head */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetArrowLength, Category=ArrowComponent)
	float ArrowLength;

	/** The size on screen to limit this arrow to (in screen space) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetScreenSize, Category = ArrowComponent)
	float ScreenSize;

	/** Set to limit the screen size of this arrow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetIsScreenSizeScaled, Category=ArrowComponent)
	uint8 bIsScreenSizeScaled : 1;

	/** If true, don't show the arrow when EngineShowFlags.BillboardSprites is disabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetTreatAsASprite, Category=ArrowComponent)
	uint8 bTreatAsASprite : 1;

#if WITH_EDITORONLY_DATA
	/** Sprite category that the arrow component belongs to, if being treated as a sprite. Value serves as a key into the localization file. */
	UPROPERTY()
	FName SpriteCategoryName_DEPRECATED;

	/** Sprite category information regarding the arrow component, if being treated as a sprite. */
	UPROPERTY()
	struct FSpriteCategoryInfo SpriteInfo;

	/** If true, this arrow component is attached to a light actor */
	UPROPERTY()
	uint32 bLightAttachment:1;

	/** Whether to use in-editor arrow scaling (i.e. to be affected by the global arrow scale) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetUseInEditorScaling, Category=ArrowComponent)
	bool bUseInEditorScaling;	
#endif // WITH_EDITORONLY_DATA
	/** Updates the arrow's colour, and tells it to refresh */
	UFUNCTION(BlueprintCallable, DisplayName="Set Arrow Color (Linear Color)", Category="Components|Arrow")
	ENGINE_API virtual void SetArrowColor(FLinearColor NewColor);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetArrowFColor(FColor NewColor);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetArrowSize(float NewSize);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetArrowLength(float NewLength);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetScreenSize(float NewScreenSize);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetIsScreenSizeScaled(bool bNewValue);

	UFUNCTION(BlueprintSetter, Category=ArrowComponent)
	ENGINE_API void SetTreatAsASprite(bool bNewValue);

	UFUNCTION(BlueprintSetter, Category = ArrowComponent)
	ENGINE_API void SetUseInEditorScaling(bool bNewValue);

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#if WITH_EDITOR
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

#if WITH_EDITORONLY_DATA
	/** Set the scale that we use when rendering in-editor */
	static ENGINE_API void SetEditorScale(float InEditorScale);

	/** The scale we use when rendering in-editor */
	static ENGINE_API float EditorScale;
#endif
};



