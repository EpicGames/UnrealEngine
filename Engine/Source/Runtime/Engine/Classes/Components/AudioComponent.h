// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Audio.h"
#include "Audio/SoundParameterControllerInterface.h"
#include "Components/SceneComponent.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "IAudioParameterTransmitter.h"
#include "Math/RandomStream.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Quartz/AudioMixerQuantizedCommands.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Sound/QuartzSubscription.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundModulationDestination.h"
#include "Sound/SoundSubmixSend.h"
#include "Sound/SoundSourceBusSend.h"
#include "UObject/ObjectMacros.h"

#include "AudioComponent.generated.h"


// Forward Declarations
class FAudioDevice;
class ISourceBufferListener;
class UAudioComponent;
class USoundBase;
class USoundClass;
class USoundConcurrency;
class USoundEffectSourcePresetChain;
class USoundWave;
struct FAudioComponentParam;
using FSharedISourceBufferListenerPtr = TSharedPtr<ISourceBufferListener, ESPMode::ThreadSafe>;


// Enum describing the audio component play state
UENUM(BlueprintType)
enum class EAudioComponentPlayState : uint8
{
	// If the sound is playing (i.e. not fading in, not fading out, not paused)
	Playing,

	// If the sound is not playing
	Stopped, 

	// If the sound is playing but paused
	Paused,

	// If the sound is playing and fading in
	FadingIn,

	// If the sound is playing and fading out
	FadingOut,

	Count UMETA(Hidden)
};


/** called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAudioFinished);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioFinishedNative, UAudioComponent*);

/** Called when subtitles are sent to the SubtitleManager.  Set this delegate if you want to hijack the subtitles for other purposes */
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnQueueSubtitles, const TArray<struct FSubtitleCue>&, Subtitles, float, CueDuration);

/** Called when sound's PlayState changes. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAudioPlayStateChanged, EAudioComponentPlayState, PlayState);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioPlayStateChangedNative, const UAudioComponent*, EAudioComponentPlayState);

/** Called when sound becomes virtualized or realized (resumes playback from virtualization). */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAudioVirtualizationChanged, bool, bIsVirtualized);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioVirtualizationChangedNative, const UAudioComponent*, bool);

/** Called as a sound plays on the audio component to allow BP to perform actions based on playback percentage.
* Computed as samples played divided by total samples, taking into account pitch.
* Not currently implemented on all platforms.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAudioPlaybackPercent, const USoundWave*, PlayingSoundWave, const float, PlaybackPercent);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAudioPlaybackPercentNative, const UAudioComponent*, const USoundWave*, const float);

/**
* Called while a sound plays and returns the sound's envelope value (using an envelope follower in the audio renderer).
* This only works in the audio mixer.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAudioSingleEnvelopeValue, const class USoundWave*, PlayingSoundWave, const float, EnvelopeValue);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAudioSingleEnvelopeValueNative, const UAudioComponent*, const USoundWave*, const float);

/**
* Called while a sound plays and returns the sound's average and max envelope value (using an envelope follower in the audio renderer per wave instance).
* This only works in the audio mixer.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnAudioMultiEnvelopeValue, const float, AverageEnvelopeValue, const float, MaxEnvelope, const int32, NumWaveInstances);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnAudioMultiEnvelopeValueNative, const UAudioComponent*, const float, const float, const int32);



/** Type of fade to use when adjusting the audio component's volume. */
UENUM(BlueprintType)
enum class EAudioFaderCurve : uint8
{
	// Linear Fade
	Linear,

	// Logarithmic Fade
	Logarithmic,

	// S-Curve, Sinusoidal Fade
	SCurve UMETA(DisplayName = "Sin (S-Curve)"),

	// Equal Power, Sinusoidal Fade
	Sin UMETA(DisplayName = "Sin (Equal Power)"),

	Count UMETA(Hidden)
};

/**
 * Legacy struct used for storing named parameter for a given AudioComponent.
 */
USTRUCT()
struct UE_DEPRECATED(5.0, "FAudioComponentParam has been deprecated, use FAudioParameter") FAudioComponentParam : public FAudioParameter
{
	GENERATED_BODY()

	// DEPRECATED
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AudioComponentParam)
	TObjectPtr<USoundWave> SoundWaveParam = nullptr;
};

/**
 *	Convenience class to get audio parameters set on an active sound's playback
 */
UCLASS(BlueprintType, MinimalAPI)
class UInitialActiveSoundParams : public UObject
{
	GENERATED_UCLASS_BODY()

	// Collection of parameters to be sent to the active sound
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	TArray<FAudioParameter> AudioParams;

	void Reset(int32 ReserveSize = 0)
	{
		AudioParams.Reset(ReserveSize);
	}
};

/**
 * AudioComponent is used to play a Sound
 *
 * @see https://docs.unrealengine.com/WorkingWithAudio/Overview
 * @see USoundBase
 */
UCLASS(ClassGroup=(Audio, Common), HideCategories=(Object, ActorComponent, Physics, Rendering, Mobility, LOD), ShowCategories=Trigger, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UAudioComponent : public USceneComponent, public ISoundParameterControllerInterface, public FQuartzTickableObject
{
	GENERATED_UCLASS_BODY()

	/** The sound to be played */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound)
	TObjectPtr<USoundBase> Sound;

	/** Array of parameters for this AudioComponent. Changes to this array directly will
	  * not be forwarded to the sound if the component is actively playing, and will be superseded
	  * by parameters set via the actor interface if set, or the instance parameters.
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Parameters, meta = (DisplayAfter = "bDisableParameterUpdatesWhilePlaying"))
	TArray<FAudioParameter> DefaultParameters;

	/** Array of transient parameters for this AudioComponent instance. Not serialized and can be set by code or BP.
	  * Changes to this array directly will not be forwarded to the sound if the component is actively playing.
	  * This should be done via the 'SetParameterX' calls implemented by the ISoundParameterControllerInterface.
	  * Instance parameter values superseded the parameters set by the actor interface & the component's default
	  * parameters.
	  */
	UPROPERTY(Transient)
	TArray<FAudioParameter> InstanceParameters;

	/** SoundClass that overrides that set on the referenced SoundBase when component is played. */
	UPROPERTY(EditAnywhere, Category = Sound, AdvancedDisplay)
	TObjectPtr<USoundClass> SoundClassOverride;

	/** Auto destroy this component on completion */
	UPROPERTY()
	uint8 bAutoDestroy:1;

	/** Stop sound when owner is destroyed */
	UPROPERTY()
	uint8 bStopWhenOwnerDestroyed:1;

	/** Whether the wave instances should remain active if they're dropped by the prioritization code. Useful for e.g. vehicle sounds that shouldn't cut out. */
	UPROPERTY()
	uint8 bShouldRemainActiveIfDropped:1;

	/** Overrides spatialization enablement in either the attenuation asset or on this audio component's attenuation settings override. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	uint8 bAllowSpatialization:1;

	/** Allows defining attenuation settings directly on this audio component without using an attenuation settings asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetOverrideAttenuation, Category = Attenuation)
	uint8 bOverrideAttenuation:1;

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetOverrideAttenuation(bool bInOverrideAttenuation);

	/** Whether or not to override the sound's subtitle priority. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Subtitles, meta = (InlineEditConditionToggle))
	uint8 bOverrideSubtitlePriority:1;

	/** Whether or not this sound plays when the game is paused in the UI */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, AdvancedDisplay)
	uint8 bIsUISound : 1;

	/** Whether or not to apply a low-pass filter to the sound that plays in this audio component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (InlineEditConditionToggle, DisplayAfter = "PitchMultiplier"))
	uint8 bEnableLowPassFilter : 1;
	
	/** Whether or not to apply a low-pass filter to the sound that plays in this audio component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (InlineEditConditionToggle, DisplayAfter = "LowPassFilterFrequency"))
	uint8 bEnableHighPassFilter : 1;

	/** Whether or not to override the priority of the given sound with the value provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (InlineEditConditionToggle, DisplayAfter = "HighPassFilterFrequency"))
	uint8 bOverridePriority:1;

	/** If true, subtitles in the sound data will be ignored. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Subtitles)
	uint8 bSuppressSubtitles:1;

	/** If true, the Audio Component will play multiple sound instances at once. Switching sounds or calling play while already playing
	  * will not stop already active instances. Virtualization for all played sounds will be disabled. Disabling while sound(s) are playing
	  * will not take effect until the AudioComponent is stopped and restarted. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (DisplayName = "Play Multiple Instances", DisplayAfter = "Priority"))
	uint8 bCanPlayMultipleInstances:1;

	/** If true, the Audio Component will ignore parameter updates for already-playing sound(s). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Parameters)
	uint8 bDisableParameterUpdatesWhilePlaying : 1;

	/** Whether this audio component is previewing a sound */
	uint8 bPreviewComponent:1;

	/** If true, this sound will not be stopped when flushing the audio device. */
	uint8 bIgnoreForFlushing:1;

	/** Whether to artificially prioritize the component to play */
	uint8 bAlwaysPlay:1;

	/** Whether or not this audio component is a music clip */
	uint8 bIsMusic:1;

	/** Whether or not the audio component should be excluded from reverb EQ processing */
	uint8 bReverb:1;

	/** Whether or not this sound class forces sounds to the center channel */
	uint8 bCenterChannelOnly:1;

	/** Whether or not this sound is a preview sound */
	uint8 bIsPreviewSound:1;

#if WITH_EDITORONLY_DATA
	/** Whether or not this sound uses submix sends if it is a preview sound */
	uint8 bIsDryPreviewPlayback : 1;
#endif //WITH_EDITORONLY_DATA

	/** Whether or not this audio component has been paused */
	uint8 bIsPaused:1;

	/** Whether or not this audio component's sound is virtualized */
	uint8 bIsVirtualized:1;

	/** Whether or not fade out was triggered. */
	uint8 bIsFadingOut:1;

	/**
	* True if we should automatically attach to AutoAttachParent when Played, and detach from our parent when playback is completed.
	* This overrides any current attachment that may be present at the time of activation (deferring initial attachment until activation, if AutoAttachParent is null).
	* If enabled, this AudioComponent's WorldLocation will no longer be reliable when not currently playing audio, and any attach children will also be
	* detached/attached along with it.
	* When enabled, detachment occurs regardless of whether AutoAttachParent is assigned, and the relative transform from the time of activation is restored.
	* This also disables attachment on dedicated servers, where we don't actually activate even if bAutoActivate is true.
	* @see AutoAttachParent, AutoAttachSocketName, AutoAttachLocationType
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Attachment)
	uint8 bAutoManageAttachment:1;

private:
	/** Did we auto attach during activation? Used to determine if we should restore the relative transform during detachment. */
	uint8 bDidAutoAttach : 1;

public:
	/** The specific audio device to play this component on */
	uint32 AudioDeviceID;

	/** Configurable, serialized ID for audio plugins */
	UPROPERTY()
	FName AudioComponentUserID;

	/** The lower bound to use when randomly determining a pitch multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Randomization|Pitch", meta = (DisplayName = "Pitch (Min)"))
	float PitchModulationMin;

	/** The upper bound to use when randomly determining a pitch multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Randomization|Pitch", meta = (DisplayName = "Pitch (Max)"))
	float PitchModulationMax;

	/** The lower bound to use when randomly determining a volume multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Randomization|Volume", meta = (DisplayName = "Volume (Min)"))
	float VolumeModulationMin;

	/** The upper bound to use when randomly determining a volume multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Randomization|Volume", meta = (DisplayName = "Volume (Max)"))
	float VolumeModulationMax;

	/** A volume multiplier to apply to sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (DisplayAfter = "Sound"))
	float VolumeMultiplier;

	/** The attack time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the 
	 *  envelope value of sounds played with this audio component.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Analysis, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerAttackTime;

	/** The release time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the
	 *  envelope value of sounds played with this audio component.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Analysis, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerReleaseTime;

	/** If enabled, overrides the priority of the selected sound with the value provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bOverridePriority", DisplayAfter = "HighPassFilterFrequency"))
	float Priority;

	/** Used by the subtitle manager to prioritize subtitles wave instances spawned by this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Subtitles, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bOverrideSubtitlePriority", DisplayAfter = "SuppressSubtitles"))
	float SubtitlePriority;

	/** The chain of Source Effects to apply to the sounds playing on the Audio Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (DisplayAfter = "CanPlayMultipleInstances"))
	TObjectPtr<USoundEffectSourcePresetChain> SourceEffectChain;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float VolumeWeightedPriorityScale_DEPRECATED;

	UPROPERTY()
	float HighFrequencyGainMultiplier_DEPRECATED;
#endif

	/** A pitch multiplier to apply to sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (DisplayAfter = "VolumeMultiplier"))
	float PitchMultiplier;

	/** If enabled, the frequency of the Lowpass Filter (in Hz) to apply to this voice. A frequency of 0.0 is the device sample rate and will bypass the filter. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableLowPassFilter", DisplayAfter = "PitchMultiplier"))
	float LowPassFilterFrequency;
	
	/** If enabled, the frequency of the Highpass Filter (in Hz) to apply to this voice. A frequency of 0.0 is the device sample rate and will bypass the filter. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableHighPassFilter", DisplayAfter = "LowPassFilterFrequency"))
	float HighPassFilterFrequency;

	/** A count of how many times we've started playing */
	int32 ActiveCount;

	/** If bOverrideSettings is false, the asset to use to determine attenuation properties for sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation, BlueprintSetter = SetAttenuationSettings, meta = (EditCondition = "!bOverrideAttenuation", DisplayAfter = "bOverrideAttenuation", EditConditionHides))
	TObjectPtr<USoundAttenuation> AttenuationSettings;

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetAttenuationSettings(USoundAttenuation* InAttenuationSettings);

	/** If bOverrideSettings is true, the attenuation properties to use for sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation, BlueprintSetter = SetAttenuationOverrides, meta = (EditCondition = "bOverrideAttenuation", DisplayAfter = "bOverrideAttenuation", EditConditionHides))
	struct FSoundAttenuationSettings AttenuationOverrides;

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetAttenuationOverrides(const FSoundAttenuationSettings& InAttenuationOverrides);

	/** What sound concurrency to use for sounds generated by this audio component */
	UPROPERTY()
	TObjectPtr<USoundConcurrency> ConcurrencySettings_DEPRECATED;

	/** What sound concurrency rules to use for sounds generated by this audio component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency)
	TSet<TObjectPtr<USoundConcurrency>> ConcurrencySet;

	/** The interval at which to re-test occlusion (in seconds).  Values < 0.f will only check occlusion once when we start playback */
	UPROPERTY(EditAnywhere, config, Category = Occlusion)
	float OcclusionCheckInterval;

	/** What time the audio component was told to play. Used to compute audio component state. */
	float TimeAudioComponentPlayed;

	/** How much time the audio component was told to fade in. */
	float FadeInTimeDuration;

	/**
	 * Options for how we handle our location when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachLocationRule;

	/**
	 * Options for how we handle our rotation when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachRotationRule;

	/**
	 * Options for how we handle our scale when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachScaleRule;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modulation)
	FSoundModulationDefaultRoutingSettings ModulationRouting;

	/** This function returns the Targeted Audio Component's current Play State.
	  * Playing, if the sound is currently playing.
	  * Stopped, if the sound is stopped.
	  * Paused, if the sound is currently playing, but paused.
	  * Fading In, if the sound is in the process of Fading In.
	  * Fading Out, if the sound is in the process of Fading Out.
	  */
	UPROPERTY(BlueprintAssignable)
	FOnAudioPlayStateChanged OnAudioPlayStateChanged;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioPlayStateChangedNative OnAudioPlayStateChangedNative;

	/** Called when virtualization state changes */
	UPROPERTY(BlueprintAssignable)
	FOnAudioVirtualizationChanged OnAudioVirtualizationChanged;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioVirtualizationChangedNative OnAudioVirtualizationChangedNative;

	/** Called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
	UPROPERTY(BlueprintAssignable)
	FOnAudioFinished OnAudioFinished;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioFinishedNative OnAudioFinishedNative;

	/** Called as a sound plays on the audio component to allow BP to perform actions based on playback percentage.
	 *  Computed as samples played divided by total samples, taking into account pitch.
	 *  Not currently implemented on all platforms.
	*/
	UPROPERTY(BlueprintAssignable)
	FOnAudioPlaybackPercent OnAudioPlaybackPercent;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioPlaybackPercentNative OnAudioPlaybackPercentNative;

	UPROPERTY(BlueprintAssignable)
	FOnAudioSingleEnvelopeValue OnAudioSingleEnvelopeValue;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioSingleEnvelopeValueNative OnAudioSingleEnvelopeValueNative;

	UPROPERTY(BlueprintAssignable)
	FOnAudioMultiEnvelopeValue OnAudioMultiEnvelopeValue;

	/** Shadow delegate for non UObject subscribers */
	FOnAudioMultiEnvelopeValueNative OnAudioMultiEnvelopeValueNative;

	/** Called when subtitles are sent to the SubtitleManager.  Set this delegate if you want to hijack the subtitles for other purposes */
	UPROPERTY()
	FOnQueueSubtitles OnQueueSubtitles;

	// Set what sound is played by this component
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void SetSound(USoundBase* NewSound);

	/**
	 * This function allows designers to call Play on an Audio Component instance while applying a volume curve over time. 
	 * Parameters allow designers to indicate the duration of the fade, the curve shape, and the start time if seeking into the sound.
	 *
	 * @param FadeInDuration How long it should take to reach the FadeVolumeLevel
	 * @param FadeVolumeLevel The percentage of the AudioComponents's calculated volume to fade to
	 * @param FadeCurve The curve to use when interpolating between the old and new volume
	 */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio", meta=(AdvancedDisplay = 1))
	ENGINE_API virtual void FadeIn(float FadeInDuration, float FadeVolumeLevel = 1.0f, float StartTime = 0.0f, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear);

	/**
	 * This function allows designers to call a delayed Stop on an Audio Component instance while applying a
	 * volume curve over time. Parameters allow designers to indicate the duration of the fade and the curve shape.
	 *
	 * @param FadeOutDuration how long it should take to reach the FadeVolumeLevel
	 * @param FadeVolumeLevel the percentage of the AudioComponents's calculated volume in which to fade to
	 * @param FadeCurve The curve to use when interpolating between the old and new volume
	 */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio", meta = (AdvancedDisplay = 1))
	ENGINE_API virtual	void FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear);

	/** Begins playing the targeted Audio Component's sound at the designated Start Time, seeking into a sound.
	 * @param StartTime The offset, in seconds, to begin reading the sound at
	 */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API virtual void Play(float StartTime = 0.0f);

	/** Start a sound playing on an audio component on a given quantization boundary with the handle to an existing clock */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true", Keywords = "play", AutoCreateRefTerm = "InDelegate"))
	ENGINE_API virtual void PlayQuantized(
		  const UObject* WorldContextObject
		, UPARAM(ref) UQuartzClockHandle*& InClockHandle
		, UPARAM(ref) FQuartzQuantizationBoundary& InQuantizationBoundary
		, const FOnQuartzCommandEventBP& InDelegate
		, float InStartTime = 0.f
		, float InFadeInDuration = 0.f
		, float InFadeVolumeLevel = 1.f
		, EAudioFaderCurve InFadeCurve = EAudioFaderCurve::Linear
	);

	// Sets a named Boolean
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Boolean Parameter"), Category = "Audio|Parameter")
	virtual void SetBoolParameter(FName InName, bool InBool) override
	{
		return ISoundParameterControllerInterface::SetBoolParameter(InName, InBool);
	}

	// Sets a named Int32
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Integer Parameter"), Category = "Audio|Parameter")
	virtual void SetIntParameter(FName InName, int32 InInt) override
	{
		return ISoundParameterControllerInterface::SetIntParameter(InName, InInt);
	}

	// Sets a named Float
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Float Parameter"), Category = "Audio|Parameter")
	virtual void SetFloatParameter(FName InName, float InFloat) override
	{
		return ISoundParameterControllerInterface::SetFloatParameter(InName, InFloat);
	}

	ENGINE_API virtual void ResetParameters() override;


	static ENGINE_API uint64 AudioComponentIDCounter;
	static ENGINE_API TMap<uint64, UAudioComponent*> AudioIDToComponentMap;
	static ENGINE_API FTransactionallySafeCriticalSection AudioIDToComponentMapLock;

private:
	// Data to hold pending quartz commands 
	struct FAudioComponentPendingQuartzCommandData
	{
		FQuartzQuantizationBoundary AnticipatoryBoundary;
		FOnQuartzCommandEventBP Delegate;
		float StartTime{ 0.0f };
		float FadeDuration{ 0.0f };
		float FadeVolume{ 0.0f };
		EAudioFaderCurve FadeCurve{ EAudioFaderCurve::Linear };
		uint32 CommandID{ (uint32)INDEX_NONE };
		TWeakObjectPtr<UQuartzClockHandle> ClockHandle;
		bool bHasBeenStoppedWhileQueued{ false };
	};

	TArray<FAudioComponentPendingQuartzCommandData> PendingQuartzCommandData;

public:
	//For if this is being played through a sound queued through Quartz
	ENGINE_API virtual void PlayQueuedQuantizedInternal(const UObject* WorldContextObject, FAudioComponentCommandInfo InCommandInfo);

	/** Stop an audio component's sound, issue any delegates if needed */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API virtual void Stop();

	/** Cues request to stop sound after the provided delay (in seconds), stopping immediately if delay is zero or negative */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void StopDelayed(float DelayTime);

	/** Pause an audio component playing its sound cue, issue any delegates if needed */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetPaused(bool bPause);

	/** Returns TRUE if the targeted Audio Component’s sound is playing.
	 *  Doesn't indicate if the sound is paused or fading in/out. Use GetPlayState() to get the full play state.
	 */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API virtual bool IsPlaying() const override;

	/** Returns if the sound is virtualized. */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API bool IsVirtualized() const;

	/** Returns the enumerated play states of the audio component. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API EAudioComponentPlayState GetPlayState() const;

	/** This function allows designers to trigger an adjustment to the sound instance’s playback Volume with options for smoothly applying a curve over time.
	 * @param AdjustVolumeDuration The length of time in which to interpolate between the initial volume and the new volume.
	 * @param AdjustVolumeLevel The new volume to set the Audio Component to.
	 * @param FadeCurve The curve used when interpolating between the old and new volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear);

	/** Sets the parameter matching the name indicated to the provided Wave. Provided for convenience/backward compatibility
	 * with SoundCues (The parameter interface supports any object and is up to the system querying it to determine whether
	 * it is a valid type).
	 * @param InName The name of the parameter to assign the wave to.
	 * @param InWave The wave value to set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Parameter")
	ENGINE_API void SetWaveParameter(FName InName, USoundWave* InWave);

	/** Set a new volume multiplier */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void SetVolumeMultiplier(float NewVolumeMultiplier);

	/** Set a new pitch multiplier */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void SetPitchMultiplier(float NewPitchMultiplier);

	/** Set whether sounds generated by this audio component should be considered UI sounds */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void SetUISound(bool bInUISound);

	/** This function is used to modify the Attenuation Settings on the targeted Audio Component instance. It is worth noting that Attenuation Settings are only passed to new Active Sounds on start, so modified Attenuation data should be set before sound playback. */
	UFUNCTION(BlueprintCallable, Category="Audio|Components|Audio")
	ENGINE_API void AdjustAttenuation(const FSoundAttenuationSettings& InAttenuationSettings);

	/** Allows designers to target a specific Audio Component instance’s sound set the send level (volume of sound copied) to the indicated Submix.
	 * @param Submix The Submix to send the signal to.
	 * @param SendLevel The scalar used to alter the volume of the copied signal.*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetSubmixSend(USoundSubmixBase* Submix, float SendLevel);

	/** Allows designers to target a specific Audio Component instance’s sound and set the send level (volume of sound copied)
	 *  to the indicated Source Bus. If the Source Bus is not already part of the sound’s sends, the reference will be added to
	 *  this instance’s Override sends. This particular send occurs before the Source Effect processing chain.
	 * @param SoundSourceBus The Bus to send the signal to.
	 * @param SourceBusSendLevel The scalar used to alter the volume of the copied signal.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetSourceBusSendPreEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel);

	/** Allows designers to target a specific Audio Component instance’s sound and set the send level (volume of sound copied)
	 *  to the indicated Source Bus. If the Source Bus is not already part of the sound’s sends, the reference will be added to
	 *  this instance’s Override sends. This particular send occurs after the Source Effect processing chain.
	 * @param SoundSourceBus The Bus to send the signal to
	 * @param SourceBusSendLevel The scalar used to alter the volume of the copied signal
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetSourceBusSendPostEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel);

	/** Allows designers to target a specific Audio Component instance’s sound and set the send level (volume of sound copied)
	 *  to the indicated Source Bus. If the Source Bus is not already part of the sound’s sends, the reference will be added to
	 *  this instance’s Override sends. This particular send occurs after attenuation processing.
	 * @param SoundSourceBus The Bus to send the signal to
	 * @param SourceBusSendLevel The scalar used to alter the volume of the copied signal
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetSourceBusSendPostAttenuation(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel);

	/** Sets how much audio the sound should send to the given Audio Bus (PRE Source Effects).
	 *  if the Bus Send doesn't already exist, it will be added to the overrides on the active sound. 
	 * @param AudioBus The Bus to send the signal to
	 * @param AudioBusSendLevel The scalar used to alter the volume of the copied signal
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetAudioBusSendPreEffect(UAudioBus* AudioBus, float AudioBusSendLevel);

	/** Sets how much audio the sound should send to the given Audio Bus (POST Source Effects).
	 *  if the Audio Bus Send doesn't already exist, it will be added to the overrides on the active sound. 
	 * @param AudioBus The Bus to send the signal to
	 * @param AudioBusSendLevel The scalar used to alter the volume of the copied signal
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetAudioBusSendPostEffect(UAudioBus* AudioBus, float AudioBusSendLevel);

	/** Sets how much audio the sound should send to the given Audio Bus (POST Attenuation).
	 *  if the Audio Bus Send doesn't already exist, it will be added to the overrides on the active sound. 
	 * @param AudioBus The Bus to send the signal to
	 * @param AudioBusSendLevel The scalar used to alter the volume of the copied signal
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetAudioBusSendPostAttenuation(UAudioBus* AudioBus, float AudioBusSendLevel);

	/** When set to TRUE, enables an additional Low Pass Filter Frequency to be calculated in with the
	 *  sound instance’s LPF total, allowing designers to set filter settings for the targeted Audio Component’s
	 *  sound instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetLowPassFilterEnabled(bool InLowPassFilterEnabled);
	
	/** When set to TRUE, enables an additional High Pass Filter Frequency to be calculated in with the
	 *  sound instance’s HPF total, allowing designers to set filter settings for the targeted Audio Component’s
	 *  sound instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetHighPassFilterEnabled(bool InHighPassFilterEnabled);

	/** Sets a cutoff frequency, in Hz, for the targeted Audio Component’s sound’s Low Pass Filter calculation.
	 *  The lowest cutoff frequency from all of the sound instance’s possible LPF calculations wins.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetLowPassFilterFrequency(float InLowPassFilterFrequency);
	
	/** Sets a cutoff frequency, in Hz, for the targeted Audio Component’s sound’s High Pass Filter calculation.
	 *  The highest cutoff frequency from all of the sound instance’s possible HPF calculations wins.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetHighPassFilterFrequency(float InHighPassFilterFrequency);

	/** Sets whether or not to output the audio to bus only. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API void SetOutputToBusOnly(bool bInOutputToBusOnly);

	/** Queries if the sound wave playing in this audio component has cooked FFT data, returns FALSE if none found.  */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API bool HasCookedFFTData() const;

	/** Queries whether or not the targeted Audio Component instance’s sound has Amplitude Envelope Data, returns FALSE if none found. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API bool HasCookedAmplitudeEnvelopeData() const;

	/**
	* Retrieves the current-time cooked spectral data of the sounds playing on the audio component.
	* Spectral data is averaged and interpolated for all playing sounds on this audio component.
	* Returns true if there is data and the audio component is playing.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API bool GetCookedFFTData(const TArray<float>& FrequenciesToGet, TArray<FSoundWaveSpectralData>& OutSoundWaveSpectralData);

	/**
	* Retrieves the current-time cooked spectral data of the sounds playing on the audio component.
	* Spectral data is not averaged or interpolated. Instead an array of data with all playing sound waves with cooked data is returned.
	* Returns true if there is data and the audio component is playing.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	ENGINE_API bool GetCookedFFTDataForAllPlayingSounds(TArray<FSoundWaveSpectralDataPerSound>& OutSoundWaveSpectralData);

	/**
	 * Retrieves Cooked Amplitude Envelope Data at the current playback time. If there are multiple
	 * SoundWaves playing, data is interpolated and averaged across all playing sound waves.
	 * Returns FALSE if no data was found.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName = "Get Cooked Amplitude Envelope Data")
	ENGINE_API bool GetCookedEnvelopeData(float& OutEnvelopeData);

	/**
	* Retrieves the current-time amplitude envelope data of the sounds playing on the audio component.
	* Envelope data is not averaged or interpolated. Instead an array of data with all playing sound waves with cooked data is returned.
	* Returns true if there is data and the audio component is playing.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName="Get Cooked Amplitude Envelope Data For All Playing Sounds")
	ENGINE_API bool GetCookedEnvelopeDataForAllPlayingSounds(TArray<FSoundWaveEnvelopeDataPerSound>& OutEnvelopeData);

	/**
	* Sets the routing for one of the given Audio component's Modulation Destinations. This will overwrite the current settings.
	* To Add modulators without removing previously set ones, use AddModulators.
	* @param Modulators The set of modulators to apply to the given destination on the component.
	* @param Destination The destination to assign the modulators to.
	* @param RoutingMethod The routing method to use for the given modulator.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName = "Set Modulation Routing")
	ENGINE_API void SetModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination, const EModulationRouting RoutingMethod = EModulationRouting::Inherit);

	/**
	* Adds the given set of Modulators to the modulators currently set on the Audio Component. This performs a logical Union, so duplicate modulators will not be added.
	* This function automatically sets the destination's routing method to Union, overriding whatever was set before.
	* To completely overwrite existing Modulation settings, use SetModulationRouting.
	* @param Modulators The set of modulators to add to the given destination on the component.
	* @param Destination The destination to add the modulators to.
	* @param bShouldModulationRoutingBeUpdated Whether we should get the ActiveSound to update the modulator on its next update call
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName = "Add Modulation Routing")
	ENGINE_API void AddModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination);

	/**
	* Removes the given set of Modulators to the modulators currently set on the Audio Component. This performs a logical subtraction of the sets, so modulators that are not in the given set will stay.
	* To completely overwrite current Modulation settings, use SetModulationRouting.
	* @param Modulators The set of modulators to remove from the given destination on the component.
	* @param Destination The destination to remove the modulators from.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName = "Remove Modulation Routing")
	ENGINE_API void RemoveModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination);

	/**
	* Gets the set of currently active modulators for a given Modulation Destination.
	* @param Destination The Destination to retrieve the Modulators from.
	* @return The set of of Modulators applied to this component for the given Destination.
	*/
	UFUNCTION(BlueprintPure, Category = "Audio|Components|Audio", DisplayName = "Get Modulators")
	ENGINE_API UPARAM(DisplayName = "Modulators") TSet<USoundModulatorBase*> GetModulators(const EModulationDestination Destination);

	static ENGINE_API void PlaybackCompleted(uint64 AudioComponentID, bool bFailedToStart);


	bool GetDisableParameterUpdatesWhilePlaying() const override { return static_cast<bool>(bDisableParameterUpdatesWhilePlaying); }

private:
	/** Called by the ActiveSound to inform the component that playback is finished */
	ENGINE_API void PlaybackCompleted(bool bFailedToStart);

	/** Whether or not the sound is audible. */
	ENGINE_API bool IsInAudibleRange(float* OutMaxDistance) const;

	ENGINE_API void SetBusSendEffectInternal(USoundSourceBus* InSourceBus, UAudioBus* InAudioBus, float SendLevel, EBusSendType InBusSendType);

	ENGINE_API void BroadcastPlayState();

	/** Returns the owning world's "AudioTime" - affected by world pause, but not time dilation.  If no world exists, returns the application time */
	ENGINE_API float GetAudioTimeSeconds() const;

public:
	/** Set when the sound is finished with initial fading in */
	ENGINE_API void SetFadeInComplete();

	/** Sets whether or not sound instance is virtualized */
	ENGINE_API void SetIsVirtualized(bool bInIsVirtualized);

	/** Sets Source Buffer Listener */
	ENGINE_API void SetSourceBufferListener(const FSharedISourceBufferListenerPtr& InSourceBufferListener, bool bShouldZeroBufferAfter);
	
	/** Gets  Source Buffer Listener */
	const FSharedISourceBufferListenerPtr& GetSourceBufferListener() const { return SourceBufferListener; }

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

	//~ Begin USceneComponent Interface
	ENGINE_API virtual void Activate(bool bReset=false) override;
	ENGINE_API virtual void Deactivate() override;
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None) override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface

	//~ Begin ActorComponent Interface.
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual const UObject* AdditionalStatObject() const override;
	ENGINE_API virtual bool IsReadyForOwnerToAutoDestroy() const override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End ActorComponent Interface.

	ENGINE_API void AdjustVolumeInternal(float AdjustVolumeDuration, float AdjustVolumeLevel, bool bIsFadeOut, EAudioFaderCurve FadeCurve);

	/** Returns a pointer to the attenuation settings to be used (if any) for this audio component dependent on the SoundAttenuation asset or overrides set. */
	ENGINE_API const FSoundAttenuationSettings* GetAttenuationSettingsToApply() const;
	ENGINE_API const TObjectPtr<USoundAttenuation> GetAttenuationSettingsAsset() const;

	/** Retrieves Attenuation Settings data on the targeted Audio Component. Returns FALSE if no settings were found. 
	 *  Because the Attenuation Settings data structure is copied, FALSE returns will return default values. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", meta = (DisplayName = "Get Attenuation Settings To Apply", ScriptName="GetAttenuationSettingsToApply"))
	ENGINE_API bool BP_GetAttenuationSettingsToApply(FSoundAttenuationSettings& OutAttenuationSettings);

	/** Collects the various attenuation shapes that may be applied to the sound played by the audio component for visualization
	 * in the editor or via the in game debug visualization. 
	 */
	ENGINE_API void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const;

	uint64 GetAudioComponentID() const { return AudioComponentID; }

	FName GetAudioComponentUserID() const { return AudioComponentUserID; }

	static ENGINE_API UAudioComponent* GetAudioComponentFromID(uint64 AudioComponentID);

	// Sets the audio thread playback time as used by the active sound playing this audio component
	// Will be set if the audio component is using baked FFT or envelope following data so as to be able to feed that data to BP based on playback time
	ENGINE_API void SetPlaybackTimes(const TMap<uint32, float>& InSoundWavePlaybackTimes);

	ENGINE_API void SetSourceEffectChain(USoundEffectSourcePresetChain* InSourceEffectChain);

	/** SoundParameterControllerInterface Implementation */
	ENGINE_API FAudioDevice* GetAudioDevice() const override;
	TArray<FAudioParameter>& GetInstanceParameters() override { return InstanceParameters; }
	uint64 GetInstanceOwnerID() const override { return AudioComponentID; }
	uint32 GetLastPlayOrder() const { return LastSoundPlayOrder; }
	USoundBase* GetSound() override { return Sound; }

	ENGINE_API virtual FName GetFNameForStatID() const override;

public:

	/**
	 * Component we automatically attach to when activated, if bAutoManageAttachment is true.
	 * If null during registration, we assign the existing AttachParent and defer attachment until we activate.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	TWeakObjectPtr<USceneComponent> AutoAttachParent;

	/**
	 * Socket we automatically attach to on the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	FName AutoAttachSocketName;

	struct PlayInternalRequestData
	{
		// start time
		float StartTime = 0.0f;

		// fade data
		float FadeInDuration = 0.0f;
		float FadeVolumeLevel = 1.0f;
		EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear;

		// Quantized event data
		Audio::FQuartzQuantizedRequestData QuantizedRequestData;
	};
	
private:

	uint64 AudioComponentID;
	uint32 LastSoundPlayOrder = 0;

	float RetriggerTimeSinceLastUpdate;
	float RetriggerUpdateInterval;

	/** Saved relative transform before auto attachment. Used during detachment to restore the transform if we had automatically attached. */
	FVector SavedAutoAttachRelativeLocation;
	FRotator SavedAutoAttachRelativeRotation;
	FVector SavedAutoAttachRelativeScale3D;

	struct FSoundWavePlaybackTimeData
	{
		USoundWave* SoundWave;
		float PlaybackTime;

		// Cached indices to boost searching cooked data indices
		uint32 LastEnvelopeCookedIndex;
		uint32 LastFFTCookedIndex;

		FSoundWavePlaybackTimeData()
			: SoundWave(nullptr)
			, PlaybackTime(0.0f)
			, LastEnvelopeCookedIndex(INDEX_NONE)
			, LastFFTCookedIndex(INDEX_NONE)
		{}

		FSoundWavePlaybackTimeData(USoundWave* InSoundWave)
			: SoundWave(InSoundWave)
			, PlaybackTime(0.0f)
			, LastEnvelopeCookedIndex(INDEX_NONE)
			, LastFFTCookedIndex(INDEX_NONE)
		{}
	};
	// The current playback times of sound waves in this audio component
	TMap<uint32, FSoundWavePlaybackTimeData> SoundWavePlaybackTimes;

	/** Restore relative transform from auto attachment and optionally detach from parent (regardless of whether it was an auto attachment). */
	ENGINE_API void CancelAutoAttachment(bool bDetachFromParent, const UWorld* MyWorld);
	
	/** Source Buffer Listener. */
	FSharedISourceBufferListenerPtr SourceBufferListener;
	bool bShouldSourceBufferListenerZeroBuffer = false;

	/** Pending submix and bus sends. */
	TArray<FSoundSubmixSendInfo> PendingSubmixSends;

	struct FPendingSourceBusSendInfo
	{
		EBusSendType BusSendType = EBusSendType::PreEffect;
		FSoundSourceBusSendInfo BusSendInfo;
	};

	TArray<FPendingSourceBusSendInfo> PendingBusSends;

protected:

	/** Utility function called by Play and FadeIn to start a sound playing. */
	ENGINE_API void PlayInternal(const PlayInternalRequestData& InPlayRequestData, USoundBase * InSoundOverride = nullptr);

#if WITH_EDITORONLY_DATA
	/** Utility function that updates which texture is displayed on the sprite dependent on the properties of the Audio Component. */
	ENGINE_API void UpdateSpriteTexture();
#endif

	// Used for processing queue commands
	//~ Begin FQuartzTickableObject
	ENGINE_API virtual void ProcessCommand(const Audio::FQuartzQueueCommandData& InQueueCommandData) override;
	//~ End FQuartzTickableObject

	FRandomStream RandomStream;
};
