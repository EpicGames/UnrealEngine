// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "BillboardComponent.generated.h"

class FPrimitiveSceneProxy;
struct FConvexVolume;
struct FEngineShowFlags;

/** 
 * A 2d texture that will be rendered always facing the camera.
 */
UCLASS(ClassGroup=Rendering, collapsecategories, hidecategories=(Object,Activation,"Components|Activation",Physics,Collision,Lighting,Mesh,PhysicsVolume), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UBillboardComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	TObjectPtr<class UTexture2D> Sprite;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	uint32 bIsScreenSizeScaled:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	float ScreenSize;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	float U;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	float UL;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	float V;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	float VL;

	/** The billboard is not rendered where texture opacity < OpacityMaskRefVal */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Sprite)
	float OpacityMaskRefVal;

#if WITH_EDITORONLY_DATA
	/** Sprite category that the component belongs to. Value serves as a key into the localization file. */
	UPROPERTY()
	FName SpriteCategoryName_DEPRECATED;

	/** Sprite category information regarding the component */
	UPROPERTY()
	struct FSpriteCategoryInfo SpriteInfo;

	/** Whether to use in-editor arrow scaling (i.e. to be affected by the global arrow scale) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sprite)
	bool bUseInEditorScaling;

	UPROPERTY()
	bool bShowLockedLocation;
#endif // WITH_EDITORONLY_DATA
	/** Change the sprite texture used by this component */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Sprite")
	ENGINE_API virtual void SetSprite(class UTexture2D* NewSprite);

	/** Change the sprite's UVs */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Sprite")
	ENGINE_API virtual void SetUV(int32 NewU, int32 NewUL, int32 NewV, int32 NewVL);

	/** Change the sprite texture and the UV's used by this component */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Sprite")
	ENGINE_API virtual void SetSpriteAndUV(class UTexture2D* NewSprite, int32 NewU, int32 NewUL, int32 NewV, int32 NewVL);

	/** Changed the opacity masked used by this component */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Sprite")
	ENGINE_API void SetOpacityMaskRefVal(float RefVal);

	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	ENGINE_API virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;
#if WITH_EDITOR
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
	//~ End UPrimitiveComponent Interface

#if WITH_EDITORONLY_DATA
	/** Set the scale that we use when rendering in-editor */
	static ENGINE_API void SetEditorScale(float InEditorScale);

	/** The scale we use when rendering in-editor */
	static ENGINE_API float EditorScale;
#endif
};



