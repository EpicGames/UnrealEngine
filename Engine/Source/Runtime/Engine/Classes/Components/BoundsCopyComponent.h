// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorComponent.h"
#include "GameFramework/Actor.h"
#include "BoundsCopyComponent.generated.h"

/** Component used to copy the bounds of another Actor. */
UCLASS(ClassGroup = Rendering, HideCategories = (Activation, AssetUserData, Collision, Cooking, Tags), MinimalAPI)
class UBoundsCopyComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	/** Actor to copy the bounds from to set up the transform. */
	UPROPERTY(EditAnywhere, Category = TransformFromBounds, meta = (DisplayName = "Source Actor", DisplayPriority = 3))
	TSoftObjectPtr<AActor> BoundsSourceActor = nullptr;

	/** If true, the source actor's bounds will include its colliding components bounds. */
	UPROPERTY(EditAnywhere, Category = TransformFromBounds, AdvancedDisplay, meta = (EditCondition = "BoundsSourceActor != nullptr"))
	bool bUseCollidingComponentsForSourceBounds = false;

	/** If true, the actor's scale will be changed so that after adjustment, its own bounds match the source bounds.*/
	UPROPERTY(EditAnywhere, Category = TransformFromBounds, AdvancedDisplay, meta = (EditCondition = "BoundsSourceActor != nullptr"))
	bool bKeepOwnBoundsScale = false;

	/** If true, the actor's own bounds will include its colliding components bounds. */
	UPROPERTY(EditAnywhere, Category = TransformFromBounds, AdvancedDisplay, meta = (EditCondition = "bKeepOwnBoundsScale && BoundsSourceActor != nullptr"))
	bool bUseCollidingComponentsForOwnBounds = false;

	/** Transform to apply to final result.*/
	UPROPERTY()
	FTransform PostTransform = FTransform::Identity;

	UPROPERTY()
	bool bCopyXBounds = true;

	UPROPERTY()
	bool bCopyYBounds = true;

	UPROPERTY()
	bool bCopyZBounds = true;

	virtual bool IsEditorOnly() const override { return true; }

public:
#if WITH_EDITOR
	/** Copy the rotation from BoundsSourceActor to this component */
	UFUNCTION(BlueprintCallable, Category = TransformFromBounds, meta = (DisplayName = "Copy Rotation"))
	ENGINE_API void SetRotation();

	/** Set this component transform to include the BoundsSourceActor bounds */
	UFUNCTION(BlueprintCallable, Category = TransformFromBounds, meta = (DisplayName = "Copy Bounds"))
	ENGINE_API void SetTransformToBounds();
#endif
};
