// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ShapeComponent.h"
#include "ShowFlags.h"
#include "BoxComponent.generated.h"

class FPrimitiveSceneProxy;

/** 
 * A box generally used for simple collision. Bounds are rendered as lines in the editor.
 */
UCLASS(ClassGroup="Collision", hidecategories=(Object,LOD,Lighting,TextureStreaming), editinlinenew, meta=(DisplayName="Box Collision", BlueprintSpawnableComponent), MinimalAPI)
class UBoxComponent : public UShapeComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** The extents (radii dimensions) of the box **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category=Shape)
	FVector BoxExtent;

#if WITH_EDITOR
	/** List of all show flags this box component visualizer should respect. */
	FEngineShowFlags ShowFlags;
#endif // WITH_EDITOR

public:
	/** 
	 * Change the box extent size. This is the unscaled size, before component scale is applied.
	 * @param	InBoxExtent: new extent (radius) for the box.
	 * @param	bUpdateOverlaps: if true and this shape is registered and collides, updates touching array for owner actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Box")
	ENGINE_API void SetBoxExtent(FVector InBoxExtent, bool bUpdateOverlaps=true);

	// @return the box extent, scaled by the component scale.
	UFUNCTION(BlueprintCallable, Category="Components|Box")
	FVector GetScaledBoxExtent() const;

	// @return the box extent, ignoring component scale.
	UFUNCTION(BlueprintCallable, Category="Components|Box")
	FVector GetUnscaledBoxExtent() const;

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual bool IsZeroExtent() const override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual struct FCollisionShape GetCollisionShape(float Inflation = 0.0f) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface

	//~ Begin UShapeComponent Interface
	ENGINE_API virtual void UpdateBodySetup() override;
	//~ End UShapeComponent Interface

	// Sets the box extents without triggering a render or physics update.
	inline void InitBoxExtent(const FVector& InBoxExtent) { BoxExtent = InBoxExtent; }

#if WITH_EDITOR
	FEngineShowFlags GetShowFlags() const { return ShowFlags; }
	ENGINE_API void SetShowFlags(const FEngineShowFlags& InShowFlags);
#endif // WITH_EDITOR
};


// ----------------- INLINES ---------------

inline FVector UBoxComponent::GetScaledBoxExtent() const
{
	return BoxExtent * GetComponentTransform().GetScale3D();
}

inline FVector UBoxComponent::GetUnscaledBoxExtent() const
{
	return BoxExtent;
}
