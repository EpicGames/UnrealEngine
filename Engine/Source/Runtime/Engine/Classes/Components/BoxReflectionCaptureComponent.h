// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ReflectionCaptureComponent.h"
#include "BoxReflectionCaptureComponent.generated.h"

	// -> will be exported to EngineDecalClasses.h
UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent), MinimalAPI)
class UBoxReflectionCaptureComponent : public UReflectionCaptureComponent
{
	GENERATED_UCLASS_BODY()

	/** Adjust capture transition distance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReflectionCapture, meta=(UIMin = "1", UIMax = "1000"))
	float BoxTransitionDistance;

	UPROPERTY()
	TObjectPtr<class UBoxComponent> PreviewInfluenceBox;

	UPROPERTY()
	TObjectPtr<class UBoxComponent> PreviewCaptureBox;

public:
	virtual void UpdatePreviewShape() override;
	virtual float GetInfluenceBoundingRadius() const override;

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface
};

