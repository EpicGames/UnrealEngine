// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "BrushComponent.generated.h"

class FPrimitiveSceneProxy;
class UMaterialInterface;
struct FConvexVolume;
struct FEngineShowFlags;

/** 
 *	A brush component defines a shape that can be modified within the editor. They are used both as part of BSP building, and for volumes. 
 *	@see https://docs.unrealengine.com/latest/INT/Engine/Actors/Volumes
 *	@see https://docs.unrealengine.com/latest/INT/Engine/Actors/Brushes
 */
UCLASS(editinlinenew, MinimalAPI, hidecategories=(Physics, Lighting, LOD, Rendering, TextureStreaming, Transform, Activation, "Components|Activation"), showcategories=(Mobility, "Rendering|Material"))
class UBrushComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UModel> Brush;

	/** Description of collision */
	UPROPERTY()
	TObjectPtr<class UBodySetup> BrushBodySetup;

#if WITH_EDITORONLY_DATA
	/** Local space translation */
	UPROPERTY()
	FVector PrePivot_DEPRECATED;
#endif

	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	void ConditionalRebuildAlteredBSP() const;
#endif //WITH_EDITOR

	//~ Begin USceneComponent Interface
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual bool ShouldCollideWhenPlacing() const override { return true; }
	//~ End USceneComponent Interface

public:

	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual class UBodySetup* GetBodySetup() override { return BrushBodySetup; };
	virtual void GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false ) const override;
	virtual ESceneDepthPriorityGroup GetStaticDepthPriorityGroup() const override;
	virtual bool IsEditorOnly() const override;

	virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;
#if WITH_EDITOR
	virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	//~ End UPrimitiveComponent Interface.

	/** Return true if the brush appears to have inverted polys */
	ENGINE_API bool HasInvertedPolys() const;

	/** If the transform mirroring no longer reflects the body setup, request its recalculation */
	ENGINE_API void RequestUpdateBrushCollision();
#endif

	/** Create the AggGeom collection-of-convex-primitives from the Brush UModel data. */
	ENGINE_API void BuildSimpleBrushCollision();
};


