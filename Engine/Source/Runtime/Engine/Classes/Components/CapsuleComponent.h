// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ShapeComponent.h"
#include "CapsuleComponent.generated.h"

class FPrimitiveSceneProxy;

/** 
 * A capsule generally used for simple collision. Bounds are rendered as lines in the editor.
 */
UCLASS(ClassGroup="Collision", editinlinenew, hidecategories=(Object,LOD,Lighting,TextureStreaming), meta=(DisplayName="Capsule Collision", BlueprintSpawnableComponent), MinimalAPI)
class UCapsuleComponent : public UShapeComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** 
	 *	Half-height, from center of capsule to the end of top or bottom hemisphere.  
	 *	This cannot be less than CapsuleRadius.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category=Shape, meta=(ClampMin="0", UIMin="0"))
	float CapsuleHalfHeight;

	/** 
	 *	Radius of cap hemispheres and center cylinder. 
	 *	This cannot be more than CapsuleHalfHeight.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category=Shape, meta=(ClampMin="0", UIMin="0"))
	float CapsuleRadius;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float CapsuleHeight_DEPRECATED;
#endif

public:
	/** 
	 * Change the capsule size. This is the unscaled size, before component scale is applied.
	 * @param	InRadius : radius of end-cap hemispheres and center cylinder.
	 * @param	InHalfHeight : half-height, from capsule center to end of top or bottom hemisphere. 
	 * @param	bUpdateOverlaps: if true and this shape is registered and collides, updates touching array for owner actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	ENGINE_API void SetCapsuleSize(float InRadius, float InHalfHeight, bool bUpdateOverlaps=true);

	/**
	 * Set the capsule radius. This is the unscaled radius, before component scale is applied.
	 * If this capsule collides, updates touching array for owner actor.
	 * @param	Radius : radius of end-cap hemispheres and center cylinder.
	 * @param	bUpdateOverlaps: if true and this shape is registered and collides, updates touching array for owner actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void SetCapsuleRadius(float Radius, bool bUpdateOverlaps=true);
	
	/**
	 * Set the capsule half-height. This is the unscaled half-height, before component scale is applied.
	 * If this capsule collides, updates touching array for owner actor.
	 * @param	HalfHeight : half-height, from capsule center to end of top or bottom hemisphere. 
	 * @param	bUpdateOverlaps: if true and this shape is registered and collides, updates touching array for owner actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void SetCapsuleHalfHeight(float HalfHeight, bool bUpdateOverlaps=true);

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#endif
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual void CalcBoundingCylinder(float& CylinderRadius, float& CylinderHalfHeight) const override;
	//~ End USceneComponent Interface

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual bool IsZeroExtent() const override;
	ENGINE_API virtual struct FCollisionShape GetCollisionShape(float Inflation = 0.0f) const override;
	ENGINE_API virtual bool AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UShapeComponent Interface
	ENGINE_API virtual void UpdateBodySetup() override;
	//~ End UShapeComponent Interface

	/**
	 * Returns the capsule radius scaled by the component scale.
	 * @return The capsule radius scaled by the component scale.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetScaledCapsuleRadius() const;

	/**
	 * Returns the capsule half-height scaled by the component scale. This includes both the cylinder and hemisphere cap.
	 * @return The capsule half-height scaled by the component scale.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetScaledCapsuleHalfHeight() const;

	/** 
	* Returns the capsule half-height minus radius (to exclude the hemisphere), scaled by the component scale.
	* From the center of the capsule this is the vertical distance along the straight cylindrical portion to the point just before the curve of top hemisphere begins.
	* @return The capsule half-height minus radius, scaled by the component scale.
	*/
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetScaledCapsuleHalfHeight_WithoutHemisphere() const;
	
	/**
	 * Returns the capsule radius and half-height scaled by the component scale. Half-height includes the hemisphere end cap.
	 * @param OutRadius Radius of the capsule, scaled by the component scale.
	 * @param OutHalfHeight Half-height of the capsule, scaled by the component scale. Includes the hemisphere end cap.
	 * @return The capsule radius and half-height scaled by the component scale.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void GetScaledCapsuleSize(float& OutRadius, float& OutHalfHeight) const;

	/**
	 * Returns the capsule radius and half-height scaled by the component scale. Half-height excludes the hemisphere end cap.
	 * @param OutRadius Radius of the capsule, ignoring component scaling.
	 * @param OutHalfHeightWithoutHemisphere Half-height of the capsule, scaled by the component scale. Excludes the hemisphere end cap.
	 * @return The capsule radius and half-height scaled by the component scale.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void GetScaledCapsuleSize_WithoutHemisphere(float& OutRadius, float& OutHalfHeightWithoutHemisphere) const;

	/**
	 * Returns the capsule radius, ignoring component scaling.
	 * @return the capsule radius, ignoring component scaling.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetUnscaledCapsuleRadius() const;

	/**
	 * Returns the capsule half-height, ignoring component scaling. This includes the hemisphere end cap.
	 * @return The capsule radius, ignoring component scaling.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetUnscaledCapsuleHalfHeight() const;
	
	/** 
	* Returns the capsule half-height minus radius (to exclude the hemisphere), ignoring component scaling. This excludes the hemisphere end cap.
	* From the center of the capsule this is the vertical distance along the straight cylindrical portion to the point just before the curve of top hemisphere begins.
	* @return The capsule half-height minus radius, ignoring component scaling.
	*/
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetUnscaledCapsuleHalfHeight_WithoutHemisphere() const;
	
	/**
	 * Returns the capsule radius and half-height scaled by the component scale. Half-height includes the hemisphere end cap.
	 * @param OutRadius Radius of the capsule, scaled by the component scale.
	 * @param OutHalfHeight Half-height of the capsule, scaled by the component scale. Includes the hemisphere end cap.
	 * @return The capsule radius and half-height scaled by the component scale.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void GetUnscaledCapsuleSize(float& OutRadius, float& OutHalfHeight) const;

	/**
	 * Returns the capsule radius and half-height, ignoring component scaling. Half-height excludes the hemisphere end cap.
	 * @param OutRadius Radius of the capsule, ignoring component scaling.
	 * @param OutHalfHeightWithoutHemisphere Half-height of the capsule, scaled by the component scale. Excludes the hemisphere end cap.
	 * @return The capsule radius and half-height (excluding hemisphere end cap), ignoring component scaling.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	void GetUnscaledCapsuleSize_WithoutHemisphere(float& OutRadius, float& OutHalfHeightWithoutHemisphere) const;

	/**
	 * Get the scale used by this shape. This is a uniform scale that is the minimum of any non-uniform scaling.
	 * @return the scale used by this shape.
	 */ 
	UFUNCTION(BlueprintCallable, Category="Components|Capsule")
	float GetShapeScale() const;

	// Sets the capsule size without triggering a render or physics update. This is the preferred method when initializing a component in a class constructor.
	inline void InitCapsuleSize(float InRadius, float InHalfHeight)
	{
		CapsuleRadius = FMath::Max(0.f, InRadius);
		CapsuleHalfHeight = FMath::Max3(0.f, InHalfHeight, InRadius);
	}
};


// ----------------- INLINES ---------------

inline void UCapsuleComponent::SetCapsuleRadius(float Radius, bool bUpdateOverlaps)
{
	SetCapsuleSize(Radius, GetUnscaledCapsuleHalfHeight(), bUpdateOverlaps);
}

inline void UCapsuleComponent::SetCapsuleHalfHeight(float HalfHeight, bool bUpdateOverlaps)
{
	SetCapsuleSize(GetUnscaledCapsuleRadius(), HalfHeight, bUpdateOverlaps);
}

inline float UCapsuleComponent::GetScaledCapsuleRadius() const
{
	const FVector& ComponentScale = GetComponentTransform().GetScale3D();
	return CapsuleRadius * UE_REAL_TO_FLOAT(ComponentScale.X < ComponentScale.Y ? ComponentScale.X : ComponentScale.Y);
}

inline float UCapsuleComponent::GetScaledCapsuleHalfHeight() const
{
	return CapsuleHalfHeight * UE_REAL_TO_FLOAT(GetComponentTransform().GetScale3D().Z);
}

inline float UCapsuleComponent::GetScaledCapsuleHalfHeight_WithoutHemisphere() const
{
	return GetScaledCapsuleHalfHeight() - GetScaledCapsuleRadius();
}

inline void UCapsuleComponent::GetScaledCapsuleSize(float& OutRadius, float& OutHalfHeight) const
{
	OutRadius = GetScaledCapsuleRadius();
	OutHalfHeight = GetScaledCapsuleHalfHeight();
}

inline void UCapsuleComponent::GetScaledCapsuleSize_WithoutHemisphere(float& OutRadius, float& OutHalfHeightWithoutHemisphere) const
{
	OutRadius = GetScaledCapsuleRadius();
	OutHalfHeightWithoutHemisphere = GetScaledCapsuleHalfHeight_WithoutHemisphere();
}


inline float UCapsuleComponent::GetUnscaledCapsuleRadius() const
{
	return CapsuleRadius;
}

inline float UCapsuleComponent::GetUnscaledCapsuleHalfHeight() const
{
	return CapsuleHalfHeight;
}

inline float UCapsuleComponent::GetUnscaledCapsuleHalfHeight_WithoutHemisphere() const
{
	return CapsuleHalfHeight - CapsuleRadius;
}

inline void UCapsuleComponent::GetUnscaledCapsuleSize(float& OutRadius, float& OutHalfHeight) const
{
	OutRadius = CapsuleRadius;
	OutHalfHeight = CapsuleHalfHeight;
}

inline void UCapsuleComponent::GetUnscaledCapsuleSize_WithoutHemisphere(float& OutRadius, float& OutHalfHeightWithoutHemisphere) const
{
	OutRadius = CapsuleRadius;
	OutHalfHeightWithoutHemisphere = CapsuleHalfHeight - CapsuleRadius;
}

inline float UCapsuleComponent::GetShapeScale() const
{
	return GetComponentTransform().GetMinimumAxisScale();
}

