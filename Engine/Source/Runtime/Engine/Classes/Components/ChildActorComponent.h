// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Components/SceneComponent.h"

#include "ChildActorComponent.generated.h"

class AActor;

USTRUCT()
struct FChildActorAttachedActorInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<AActor> Actor;
	UPROPERTY()
	FName SocketName;
	UPROPERTY()
	FTransform RelativeTransform;
};

USTRUCT()
struct FChildActorComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FChildActorComponentInstanceData() = default;
	ENGINE_API FChildActorComponentInstanceData(const class UChildActorComponent* Component);

	virtual ~FChildActorComponentInstanceData() = default;

	ENGINE_API virtual bool ContainsData() const override;

	ENGINE_API virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	// The class of the child actor when the instance data cache was stored
	UPROPERTY()
	TSubclassOf<AActor> ChildActorClass;

	// The name of the spawned child actor so it (attempts to) remain constant across construction script reruns
	UPROPERTY()
	FName ChildActorName;

	UPROPERTY()
	TArray<FChildActorAttachedActorInfo> AttachedActors;

#if WITH_EDITOR
	/** Keep track of the child actor GUID to reuse it when reinstancing */
	FGuid ChildActorGUID;
#endif

	// The saved properties for the ChildActor itself
	TSharedPtr<FActorInstanceData> ActorInstanceData;

	// The component instance data cache for the ChildActor spawned by this component
	TSharedPtr<FComponentInstanceDataCache> ComponentInstanceData;
};

#if WITH_EDITORONLY_DATA
UENUM()
enum class EChildActorComponentTreeViewVisualizationMode : uint8
{
	/** Use the editor's default setting. */
	UseDefault UMETA(Hidden),
	/** Show only the outer component as a single component node. */
	ComponentOnly,
	/** Include the child actor hierarchy attached to the outer component as the root node. */
	ComponentWithChildActor,
	/** Show only as a child actor hierarchy (i.e. do not show the outer component node as the root). */
	ChildActorOnly,
	/** Do not display the actor in the tree view. */
	Hidden,
};
#endif

/** A component that spawns an Actor when registered, and destroys it when unregistered.*/
UCLASS(ClassGroup=Utility, hidecategories=(Object,LOD,Physics,Lighting,TextureStreaming,Activation,"Components|Activation",Collision), meta=(BlueprintSpawnableComponent), MinimalAPI)
class UChildActorComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/**
	 * Sets the class to use for the child actor. 
	 * If called on a template component (owned by a CDO), the properties of any existing child actor template will be copied as best possible to the template. 
	 * If called on a component instance in a world (and the class is changing), the created ChildActor will use the class defaults as template.
	 * @param InClass The Actor subclass to spawn as a child actor
	 */
	UFUNCTION(BlueprintCallable, Category=ChildActorComponent)
	void SetChildActorClass(TSubclassOf<AActor> InClass)
	{
		SetChildActorClass(InClass, nullptr);
	}

	/**
	 * Sets then class to use for the child actor providing an optional Actor to use as the template.
	 * If called on a template component (owned by a CDO) and NewChildActorTemplate is not null, the new child actor template will be created using the supplied Actor as template.
	 * If called on a template component and NewChildActorTemplate is null, the properties of any existing child actor template will be copied as best possible to the template.
	 * If called on a component instance in a world with NewChildActorTemplate not null, then if registered a new child actor will be created using the supplied Actor as template, 
	 *    otherwise if not registered it will ensure. If the class also changed, then future ChildActors created by this component the class defaults will be used.
	 * If called on a component instance in a world with NewChildActorTemplate null and the class is changing, the created ChildActor will use the class defaults as template.
	 * @param InClass                 The Actor subclass to spawn as a child actor
	 * @param NewChildActorTemplate   An Actor to use as the template when spawning a child actor using this component (per the rules listed above)
	 */
	ENGINE_API void SetChildActorClass(TSubclassOf<AActor> InClass, AActor* NewChildActorTemplate);

	TSubclassOf<AActor> GetChildActorClass() const { return ChildActorClass; }

	/**
	 * Sets whether or not the owner of the child actor will be set to the owner of this component 
	 * @param bInSetChildActorOwner Will the owner of this component be set as the owner of the child actor
	 */
	void SetChildActorOwnerOnCreation(bool bInSetChildActorOwner) { bSetOwner = bInSetChildActorOwner; }

	friend class FChildActorComponentDetails;

private:
	/** The class of Actor to spawn */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ChildActorComponent, meta=(OnlyPlaceable, AllowPrivateAccess="true", ForceRebuildProperty="ChildActorTemplate"))
	TSubclassOf<AActor>	ChildActorClass;

	/** The actor that we spawned and own */
	UPROPERTY(Replicated, BlueprintReadOnly, ReplicatedUsing=OnRep_ChildActor, Category=ChildActorComponent, TextExportTransient, DuplicateTransient, meta=(AllowPrivateAccess="true"))
	TObjectPtr<AActor>	ChildActor;

	/** Property to point to the template child actor for details panel purposes */
	UPROPERTY(VisibleDefaultsOnly, DuplicateTransient, Category=ChildActorComponent, meta=(ShowInnerProperties))
	TObjectPtr<AActor> ChildActorTemplate;

	/** We try to keep the child actor's name as best we can, so we store it off here when destroying */
	FName ChildActorName;

	/** Detect when the parent actor is renamed, in which case we can't preserve the child actor's name */
	UObject* ActorOuter;

	/** Cached copy of the instance data when the ChildActor is destroyed to be available when needed */
	mutable FChildActorComponentInstanceData* CachedInstanceData;

#if WITH_EDITORONLY_DATA
	/** Indicates how this component will be visualized for editing in a tree view. Users can change this setting per instance via the context menu in the Blueprint/SCS editor. */
	UPROPERTY()
	EChildActorComponentTreeViewVisualizationMode EditorTreeViewVisualizationMode;
#endif

	/**
	 * Should the spawned actor be marked as transient?
	 * @note The spawned actor will also be marked transient if this component or its owner actor are transient, regardless of the state of this flag.
	 */
	UPROPERTY(EditDefaultsOnly, Category=ChildActorComponent)
	uint8 bChildActorIsTransient:1;

	/**
	 * Should the owner of this component be set as the owner of the child actor?
	 */
	UPROPERTY(EditDefaultsOnly, Category=ChildActorComponent)
	uint8 bSetOwner:1;

	/** Flag indicating that when the component is registered that the child actor should be recreated */
	uint8 bNeedsRecreate:1;

	/** When true, does not modify ChildActorName during spawn such as removing _UAID_ */
	uint8 bChildActorNameIsExact:1;

	void DeleteCachedInstanceData();

#if WITH_EDITOR
	ENGINE_API virtual void SetPackageExternal(bool bExternal, bool bShouldDirty) override;
#endif

public:

	ENGINE_API static bool ExperimentalAllowPerInstanceChildActorProperties();

	//~ Begin Object Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual void PostRepNotifies() override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End Object Interface.

	//~ Begin ActorComponent Interface.
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual bool IsHLODRelevant() const override;
#if WITH_EDITOR
	ENGINE_API virtual TSubclassOf<class UHLODBuilder> GetCustomHLODBuilderClass() const override;
#endif
	//~ End ActorComponent Interface.

	/** Apply the component instance data to the child actor component */
	ENGINE_API void ApplyComponentInstanceData(FChildActorComponentInstanceData* ComponentInstanceData, const ECacheApplyPhase CacheApplyPhase);

	/** Create the child actor */
	ENGINE_API virtual void CreateChildActor(TFunction<void(AActor*)> CustomizerFunc = nullptr);

	DECLARE_EVENT_OneParam(UChildActorComponent, FOnChildActorCreated, AActor*);
	FOnChildActorCreated& OnChildActorCreated() { return OnChildActorCreatedDelegate; }

	AActor* GetChildActor() const { return ChildActor; }
	AActor* GetChildActorTemplate() const { return ChildActorTemplate; }
	ENGINE_API AActor* GetSpawnableChildActorTemplate() const;

	FName GetChildActorName() const { return ChildActorName; }
	ENGINE_API void SetChildActorName(const FName InName);

	/** When true, does not modify ChildActorName during spawn such as removing _UAID_ */
	ENGINE_API void SetChildActorNameIsExact(bool bInExact);

	/** Kill any currently present child actor */
	ENGINE_API void DestroyChildActor();

#if WITH_EDITOR
	EChildActorComponentTreeViewVisualizationMode GetEditorTreeViewVisualizationMode() const
	{
		return EditorTreeViewVisualizationMode;
	}

	ENGINE_API void SetEditorTreeViewVisualizationMode(EChildActorComponentTreeViewVisualizationMode InMode);
#endif

	/** Register all replication fragments */
	ENGINE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;


private:
	ENGINE_API bool IsChildActorReplicated() const;

	ENGINE_API bool IsLevelBeingRemoved() const;

	UFUNCTION()
	void OnRep_ChildActor();

	void RegisterChildActorDestroyedDelegate();

	UFUNCTION()
	ENGINE_API void OnChildActorDestroyed(AActor* Actor);

	FOnChildActorCreated OnChildActorCreatedDelegate;
};

struct FActorParentComponentSetter
{
	static void Set(AActor* ChildActor, UChildActorComponent* ParentComponent);
private:
	friend UChildActorComponent;
};
