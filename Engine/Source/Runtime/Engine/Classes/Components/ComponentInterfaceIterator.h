// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/ComponentInterfaces.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

template< class T > 
class TComponentInterfaceIterator
{
public:

	enum EEndTagType
	{
		EndTag
	};

	explicit TComponentInterfaceIterator(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, bool bIncludeDerivedClasses = true, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
		: Index(-1)
	{
		TArray<UObject*>	ObjectArray;

		for (TComponentInterfaceImplementation<T>& Implementation : T::GetImplementers())
		{
			if (Implementation.Class)
			{
				if (Implementation.Resolver)
				{
					ObjectArray.Reset();
					GetObjectsOfClass(Implementation.Class, ObjectArray, bIncludeDerivedClasses, AdditionalExclusionFlags, GetObjectIteratorDefaultInternalExclusionFlags(InInternalExclusionFlags));
					Interfaces.Reserve(ObjectArray.Num() + Interfaces.Num());
					for (UObject* Object : ObjectArray)
					{
						T* Interface = Implementation.Resolver(Object);
						check(Interface);
						Interfaces.Add(Interface);
					}
				}

				if (Implementation.CollectionResolver)
				{
					ObjectArray.Reset();
					GetObjectsOfClass(Implementation.Class, ObjectArray, bIncludeDerivedClasses, AdditionalExclusionFlags, GetObjectIteratorDefaultInternalExclusionFlags(InInternalExclusionFlags));
					for (UObject* Object : ObjectArray)
					{
						TArray<T*> ContainerInterfaces = Implementation.CollectionResolver(Object);
						Interfaces.Reserve(ContainerInterfaces.Num() + Interfaces.Num());
						for (T* ContainerInterface : ContainerInterfaces)
						{
							T* Interface = ContainerInterface;
							check(Interface);
							Interfaces.Add(Interface);
						}
					}
				}
			}
		}

		Index = 0;
	}

	TComponentInterfaceIterator(EEndTagType, const TComponentInterfaceIterator<T>& InIter )
		: Index(InIter.Interfaces.Num())
	{
	}

	inline void operator++(int)
	{		
		Index++;
	}	

	inline void operator++()
	{		
		Index++;
	}

	/** Conversion to "bool" returning true if the iterator is valid. */
	inline explicit operator bool() const
	{ 
		return Interfaces.IsValidIndex(Index); 
	}

	/** Conversion to "bool" returning true if the iterator is valid. */
	inline bool operator !() const 
	{
		return !(bool)*this;
	}

	inline T* operator*() const
	{
		if (Interfaces.IsValidIndex(Index))
		{
			return  Interfaces[Index];
		}
		else
		{
			return nullptr;
		}
	}

	inline T* operator->() const
	{
		if (Interfaces.IsValidIndex(Index))
		{
			return  Interfaces[Index];
		}
		else
		{
			return nullptr;
		}
	}

	inline bool operator==(const TComponentInterfaceIterator& Rhs) const { return Index == Rhs.Index; }
	inline bool operator!=(const TComponentInterfaceIterator& Rhs) const { return Index != Rhs.Index; }

protected:
	/** Resolved results for all the Results from the GetObjectsOfClass queries */
	TArray<T*>	Interfaces;
	
	/** index of the current element in the object array */
	int32 Index;
};

template <typename T>
struct TComponentInterfaceRange
{
	TComponentInterfaceRange()
	{
	}

	friend TComponentInterfaceIterator<T> begin(const TComponentInterfaceRange& Range) { return Range.Begin; }
	friend TComponentInterfaceIterator<T> end  (const TComponentInterfaceRange& Range) { return TComponentInterfaceIterator<T>(TComponentInterfaceIterator<T>::EndTag, Range.Begin); }

	TComponentInterfaceIterator<T> Begin;
};
