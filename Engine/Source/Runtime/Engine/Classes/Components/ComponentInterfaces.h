// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "PrimitiveComponentId.h"
#include "Templates/RefCounting.h"
#include "Templates/Function.h"
#include "UObject/Object.h"
#include "Misc/Iteration.h"

class ULevel;
class UWorld;
class FSceneInterface;
class FRegisterComponentContext;
class UStaticMesh;
class UStreamableRenderAsset;
class USkinnedAsset;
class FPrimitiveSceneProxy;
class UMaterialInterface;
class HHitProxy;
class FStreamingTextureLevelContext;
struct FRenderAssetOwnerStreamingState;
struct FStreamingRenderAssetPrimitiveInfo;

struct FPrimitiveLODStats
{
	int32 LODIndex = 0;
	uint32 Sections = 1;
	uint32 Triangles = 0;
	bool bIsOptionalLOD = false;
	bool bIsAvailable = true;
	SIZE_T TotalResourceSize = 0;
	TArray<uint16> MaterialIndices;
	
	FPrimitiveLODStats(int32 InLOD) :
		LODIndex(InLOD)
	{
	}

	FPrimitiveLODStats(const FPrimitiveLODStats& Other) = default;
	FPrimitiveLODStats(FPrimitiveLODStats&& Other) = default;

	FPrimitiveLODStats& operator=(const FPrimitiveLODStats& RHS) = default;
	FPrimitiveLODStats& operator=(FPrimitiveLODStats&& RHS) = default;

	inline int32 GetDrawCount() const
	{
		return Sections * MaterialIndices.Num();
	}
};

/** 
* Structure used to report some primitive stats in debugging tools
*/
struct FPrimitiveStats
{
	TArray<FPrimitiveLODStats> LODStats;
};

// A ComponentInterface Provider bridges an owner/container/implementer object to a ComponentInterface, many providers are allowed to exist for the same interface instance
template<typename T>
struct TComponentInterfaceProvider
{
	TComponentInterfaceProvider(UClass* InClass, TFunction<T* (UObject*)> InProvider)
		: Class(InClass)
		, Provider(InProvider)
	{}

	TComponentInterfaceProvider(UClass* InClass, TFunction<TArray<T*>(UObject*)> InCollectionProvider)
		: Class(InClass)
		, CollectionProvider(InCollectionProvider)
	{}


	UClass* Class = nullptr;
	TFunction<T* (UObject*)> Provider;
	TFunction<TArray<T*> (UObject*)> CollectionProvider;

	static ENGINE_API T* Provides(UObject* SourceObject);
	static ENGINE_API void Provides(UObject* SourceObject, TArray<T*>& OutResults);
	static void Provides(UObject* SourceObject, TFunctionRef<EIteration (T*)>);
};

// A ComponentInterface Implementation is used to enumerate the various existing ComponentInterfaces, only one Implementation per interface instance is allowed
template<typename T>
struct TComponentInterfaceImplementation
{
	TComponentInterfaceImplementation(UClass* InClass, TFunction<T* (UObject*)> InResolver)
		: Class(InClass)
		, Resolver(InResolver)
	{}

	TComponentInterfaceImplementation(UClass* InClass, TFunction<TArray<T*>(UObject*)> InCollectionResolver)
		: Class(InClass)
		, CollectionResolver(InCollectionResolver)
	{}

	UClass*	Class = nullptr;
	TFunction<T*(UObject*)> Resolver;
	TFunction<TArray<T*> (UObject*)> CollectionResolver;
};
typedef TComponentInterfaceImplementation<void> FComponentInterfaceImplementation;

template<class T>
class TComponentInterfaceIterator;
	

template<class T> 
class TComponentInterfaceBase
{
public:
	static ENGINE_API void AddImplementer(const TComponentInterfaceImplementation<T>& Implementer);
	static ENGINE_API void RemoveImplementer(const UClass* ImplementerClass);
	static ENGINE_API void AddProvider(const TComponentInterfaceProvider<T>& Provider);
	static ENGINE_API void RemoveProvider(const UClass* ProviderClass);

protected:
	
	static ENGINE_API TArray<TComponentInterfaceImplementation<T>>& GetImplementers();

	friend TComponentInterfaceIterator<T>;
	friend TComponentInterfaceProvider<T>;
};

class IPrimitiveComponent : public TComponentInterfaceBase<IPrimitiveComponent>
{
public:
	virtual bool IsRenderStateCreated() const = 0;
	virtual bool IsRenderStateDirty() const = 0;	
	virtual bool ShouldCreateRenderState() const = 0;
	virtual bool IsRegistered() const = 0;
	virtual bool IsUnreachable() const = 0;
	virtual bool IsStaticMobility() const = 0;
	virtual bool IsMipStreamingForced() const = 0;

	virtual UWorld* GetWorld() const = 0;
	virtual FSceneInterface* GetScene() const = 0;
	virtual FPrimitiveSceneProxy* GetSceneProxy() const = 0;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const = 0;		
	virtual void MarkRenderStateDirty() = 0;
	virtual void DestroyRenderState() = 0;
	virtual void CreateRenderState(FRegisterComponentContext* Context) = 0;	
	virtual FString GetName() const = 0;
	virtual FString GetFullName() const = 0;
	virtual FTransform GetTransform() const = 0;
	virtual const FBoxSphereBounds& GetBounds() const = 0;
	virtual float GetLastRenderTimeOnScreen() const = 0;
	virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const = 0;
	virtual UObject* GetUObject() = 0;
	virtual const UObject*	GetUObject() const = 0;
	virtual void PrecachePSOs() = 0;

	// helper to obtain typed UObjects 
	template<class T> 
	inline const T* GetUObject() const { return Cast<T>(GetUObject()); }

	template<class T> 
	inline T* GetUObject() { return Cast<T>(GetUObject()); }

	virtual UObject* GetOwner() const = 0;

	// helper to have typed owners
	template<class T> 
	inline T* GetOwner() { return Cast<T>(GetOwner()); }

	virtual FString GetOwnerName() const = 0;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() = 0;

	// Streaming utilities
	virtual FRenderAssetOwnerStreamingState& GetStreamingState() const = 0;
	virtual ULevel* GetComponentLevel() const = 0;
	virtual IPrimitiveComponent* GetLODParentPrimitive() const = 0;
	virtual float GetMinDrawDistance() const = 0;
	virtual float GetStreamingScale() const = 0;

	virtual void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex) = 0;

	// Get the streamable Nanite asset if one exists for this type. Nanite streaming only supports one asset per primitive, so this isn't an array.
	// Nanite assets are managed differently from all other assets, so we return a pointer directly instead of using FStreamingRenderAssetPrimitiveInfo.
	virtual UStreamableRenderAsset* GetStreamableNaniteAsset() const = 0;

	UE_DEPRECATED(5.8, "Override FPrimitiveSceneProxy::GetStreamableRenderAssetInfo or IPrimitiveComponent::GetStreamingRenderAssetInfo instead.")
	virtual void GetStreamableRenderAssetInfo(TArray<FStreamingRenderAssetPrimitiveInfo>& StreamableRenderAssets) const {}

	// Get all streamable render assets for the given level context. This method is for the legacy static/dynamic streaming managers.
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const = 0;

#if WITH_EDITOR
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) = 0;
#endif
	virtual HHitProxy* CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) = 0;
};

class IStaticMeshComponent : public TComponentInterfaceBase<IStaticMeshComponent>
{
public:
#if WITH_EDITOR
	virtual void OnMeshRebuild(bool bRenderDataChanged) = 0;
	virtual void PreStaticMeshCompilation() = 0;
	virtual void PostStaticMeshCompilation() = 0;
#endif
	virtual UStaticMesh* GetStaticMesh() const = 0;

	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() = 0;
	inline const IPrimitiveComponent* GetPrimitiveComponentInterface() const
	{
		// use the non-const version and return it as a const object to avoid duplicating the code in implementers
		return (const_cast<IStaticMeshComponent*>(this))->GetPrimitiveComponentInterface();
	}
};

class ISkinnedMeshComponent : public TComponentInterfaceBase<ISkinnedMeshComponent>
{
public:
#if WITH_EDITOR
	virtual void PostAssetCompilation() = 0;
	virtual void PostSkeletalHierarchyChange() = 0;
#endif
	virtual USkinnedAsset* GetSkinnedAsset() const = 0;

	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() = 0;
	inline const IPrimitiveComponent* GetPrimitiveComponentInterface() const
	{
		// use the non-const version and return it as a const object to avoid duplicating the code in implementers
		return (const_cast<ISkinnedMeshComponent*>(this))->GetPrimitiveComponentInterface();
	}
};


#pragma region HelperMacros

// These macros are intended to allow implementing an interface with same memory footprint/performance as inheriting from 
// the abstract base class, but without mixing the interface methods with the class methods. 
//
// It declares a member for the interface and utility functions in the host class to get the host ptr from the interface instance. 
// And thus allow declaration of the interface inside a class without requiring the interface to host a back ptr to it's owner. 
// 
// 
// Example...
// 
//		class FActorFooComponentInterface : IFooComponent
//		{
//			// Nothing but the overrides
//			virtual void OverrideSomething() override;
//		};
//		
//		class UFooComponent
//		{
//			UE_DECLARE_COMPONENT_INTERFACE(FooComponent, FActor);
//		}
// 
//	    void FActorFooComponentInterface::OverrideSomething()
//		{
//			UFooComponent::GetFooComponent(this)->OverrideSomethingImplementer();
//		}
//

#define UE_DECLARE_COMPONENT_INTERFACE_INTERNAL(ThisType, ComponentInterfaceType, InterfaceType, InterfaceMember, InterfaceName)\
	public:\
		ComponentInterfaceType* Get##InterfaceName##Interface()\
		{\
			return &InterfaceMember;\
		}\
		const ComponentInterfaceType* Get##InterfaceName##Interface() const\
		{\
			return &InterfaceMember;\
		}\
	protected:\
		InterfaceType InterfaceMember;\
		\
		static uintptr_t GetAddress(const InterfaceType* InImpl)\
		{\
			return reinterpret_cast<uintptr_t>(InImpl) - offsetof(ThisType, InterfaceMember);\
		}\
		static ThisType* Get##InterfaceName(InterfaceType* InImpl)\
		{\
			return reinterpret_cast<ThisType*>(GetAddress(InImpl));\
		}\
		static const ThisType* Get##InterfaceName(const InterfaceType* InImpl)\
		{\
			return reinterpret_cast<const ThisType*>(GetAddress(InImpl));\
		}\
		friend class InterfaceType;

#define UE_DECLARE_COMPONENT_INTERFACE(ComponentName, BasePrefix)\
	UE_DECLARE_COMPONENT_INTERFACE_INTERNAL(U##ComponentName, I##ComponentName , BasePrefix##ComponentName##Interface, ComponentName##Interface, ComponentName)

#define UE_DECLARE_COMPONENT_ACTOR_INTERFACE(ComponentName)\
	UE_DECLARE_COMPONENT_INTERFACE(ComponentName, FActor)

#pragma endregion