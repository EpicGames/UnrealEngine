// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "PSOPrecacheComponent.h"

#include "DecalComponent.generated.h"

class FDeferredDecalProxy;
class UMaterialInterface;

/** 
 * A material that is rendered onto the surface of a mesh. A kind of 'bumper sticker' for a model.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Actors/DecalActor
 * @see UDecalActor
 */
UCLASS(Blueprintable, editinlinenew, hidecategories=(Collision, Object, Physics, SceneComponent, Activation, "Components|Activation", Mobility), ClassGroup=Rendering, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UDecalComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** Decal material. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Decal)
	TObjectPtr<class UMaterialInterface> DecalMaterial;

public:
	/** 
	 * Controls the order in which decal elements are rendered.  Higher values draw later (on top). 
	 * Setting many different sort orders on many different decals prevents sorting by state and can reduce performance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	int32 SortOrder;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	float FadeScreenSize;

	/**
	* Time in seconds to wait before beginning to fade out the decal. Set fade duration and start delay to 0 to make persistent.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	float FadeStartDelay;

	/**
	* Time in seconds for the decal to fade out. Set fade duration and start delay to 0 to make persistent. Only fades in active simulation or game.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	float FadeDuration;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	float FadeInDuration;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	float FadeInStartDelay;

	/**
	* Automatically destroys the owning actor after fully fading out.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	uint8 bDestroyOwnerAfterFade : 1;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API float GetFadeStartDelay() const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API float GetFadeDuration() const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API float GetFadeInStartDelay() const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API float GetFadeInDuration() const;

	/** Decal size in local space (does not include the component scale), technically redundant but there for convenience */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal, meta=(AllowPreserveRatio = "true"))
	FVector DecalSize;

	/** Decal color, can be accessed using the material Decal Color node. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Decal)
	FLinearColor DecalColor = FLinearColor::White;

	/**
	* Sets the decal's fade start time, duration and if the owning actor should be destroyed after the decal is fully faded out.
	* The default value of 0 for FadeStartDelay and FadeDuration makes the decal persistent. See DecalLifetimeOpacity material 
	* node to control the look of "fading out."
	*
	* @param StartDelay - Time in seconds to wait before beginning to fade out the decal.
	* @param Duration - Time in second for the decal to fade out.
	* @param DestroyOwnerAfterFade - Should the owning actor automatically be destroyed after it is completely faded out.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API void SetFadeOut(float StartDelay, float Duration, bool DestroyOwnerAfterFade = true);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API void SetFadeIn(float StartDelay, float Duration);

	/** Set the FadeScreenSize for this decal component */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API void SetFadeScreenSize(float NewFadeScreenSize);

	/** Sets the sort order for the decal component. Higher values draw later (on top). This will force the decal to reattach */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API void SetSortOrder(int32 Value);

	/** Sets the decal color. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Decal")
	ENGINE_API void SetDecalColor(const FLinearColor& Color);

	/** setting decal material on decal component. This will force the decal to reattach */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal")
	ENGINE_API void SetDecalMaterial(class UMaterialInterface* NewDecalMaterial);

	/** Accessor for decal material */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal")
	ENGINE_API class UMaterialInterface* GetDecalMaterial() const;

	/** Utility to allocate a new Dynamic Material Instance, set its parent to the currently applied material, and assign it */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal")
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateDynamicMaterialInstance();

public:

	/** The decal proxy. */
	FDeferredDecalProxy* SceneProxy;

	/**
	 * Pushes new selection state to the render thread primitive proxy
	 */
	UE_DEPRECATED(5.4, "This method has no effect, calls to this method may be safely removed.")
	ENGINE_API void PushSelectionToProxy();

protected:
	/** Handle for efficient management of DestroyDecalComponent timer */
	FTimerHandle TimerHandle_DestroyDecalComponent;

	/** Called when the life span of the decal has been exceeded */
	ENGINE_API virtual void LifeSpanCallback();
		
	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;

public:
		
	/** Get the PSO precaching data */
	FPSOPrecacheComponentData& GetPSOPrecacheComponentData() { return PSOPrecacheComponentData; }

	ENGINE_API void SetLifeSpan(const float LifeSpan);

	/**
	 * Retrieves the materials used in this component
	 *
	 * @param OutMaterials	The list of used materials.
	 */
	ENGINE_API virtual void GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false ) const;
	
	ENGINE_API virtual FDeferredDecalProxy* CreateSceneProxy();
	virtual int32 GetNumMaterials() const
	{
		return 1; // DecalMaterial
	}

	virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const
	{
		return (ElementIndex == 0) ? DecalMaterial : NULL;
	}
	virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial)
	{
		if (ElementIndex == 0)
		{
			SetDecalMaterial(InMaterial);
		}
	}

	
	//~ Begin UActorComponent Interface
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual const UObject* AdditionalStatObject() const override;
	ENGINE_API virtual void PrecachePSOs() override;
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
#endif
	//~ End UActorComponent Interface
	
	//~ Begin UObject Interface. 
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface

	FTransform GetTransformIncludingDecalSize() const
	{
		FTransform Ret = GetComponentToWorld();
		Ret.SetScale3D(Ret.GetScale3D() * DecalSize);

		return Ret;
	}
};

