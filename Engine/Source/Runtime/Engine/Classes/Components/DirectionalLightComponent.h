// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/LightComponent.h"

#include "DirectionalLightComponent.generated.h"

class FLightSceneProxy;

/**
 * A light component that has parallel rays. Will provide a uniform lighting across any affected surface (eg. The Sun). This will affect all objects in the defined light-mass importance volume.
 */
UCLASS(Blueprintable, ClassGroup=Lights, hidecategories=(Object, LightProfiles), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UDirectionalLightComponent : public ULightComponent
{
	GENERATED_UCLASS_BODY()

	/**
	* Controls the depth bias scaling across cascades. This allows to mitigage the shadow acne difference on shadow cascades transition.
	* A value of 1 scales shadow bias based on each cascade size (Default).
	* A value of 0 scales shadow bias uniformly accross all cacascade.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = Light, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float ShadowCascadeBiasDistribution;

	/** Whether to occlude fog and atmosphere inscattering with screenspace blurred occlusion from this light. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=LightShafts, meta=(DisplayName = "Light Shaft Occlusion"))
	uint32 bEnableLightShaftOcclusion:1;

	/** 
	 * Controls how dark the occlusion masking is, a value of 1 results in no darkening term.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=LightShafts, meta=(UIMin = "0", UIMax = "1"))
	float OcclusionMaskDarkness;

	/** Everything closer to the camera than this distance will occlude light shafts. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=LightShafts, meta=(UIMin = "0", UIMax = "500000"))
	float OcclusionDepthRange;

	/** 
	 * Can be used to make light shafts come from somewhere other than the light's actual direction. 
	 * This will only be used when non-zero.  It does not have to be normalized.
	 */
	UPROPERTY(EditAnywhere, interp, AdvancedDisplay, BlueprintReadOnly, Category=LightShafts)
	FVector LightShaftOverrideDirection;

	UPROPERTY()
	float WholeSceneDynamicShadowRadius_DEPRECATED;

	/** 
	 * How far Cascaded Shadow Map dynamic shadows will cover for a movable light, measured from the camera.
	 * A value of 0 disables the dynamic shadow.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "20000", DisplayName = "Dynamic Shadow Distance MovableLight"))
	float DynamicShadowDistanceMovableLight;

	/** 
	 * How far Cascaded Shadow Map dynamic shadows will cover for a stationary light, measured from the camera.
	 * A value of 0 disables the dynamic shadow.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "20000", DisplayName = "Dynamic Shadow Distance StationaryLight"))
	float DynamicShadowDistanceStationaryLight;

	/** 
	 * Number of cascades to split the view frustum into for the whole scene dynamic shadow.  
	 * More cascades result in better shadow resolution, but adds significant rendering cost.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "4", DisplayName = "Num Dynamic Shadow Cascades"))
	int32 DynamicShadowCascades;

	/** 
	 * Controls whether the cascades are distributed closer to the camera (larger exponent) or further from the camera (smaller exponent).
	 * An exponent of 1 means that cascade transitions will happen at a distance proportional to their resolution.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "1", UIMax = "4", DisplayName = "Distribution Exponent"))
	float CascadeDistributionExponent;

	/** 
	 * Proportion of the fade region between cascades.
	 * Pixels within the fade region of two cascades have their shadows blended to avoid hard transitions between quality levels.
	 * A value of zero eliminates the fade region, creating hard transitions.
	 * Higher values increase the size of the fade region, creating a more gradual transition between cascades.
	 * The value is expressed as a percentage proportion (i.e. 0.1 = 10% overlap).
	 * Ideal values are the smallest possible which still hide the transition.
	 * An increased fade region size causes an increase in shadow rendering cost.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "0.3", DisplayName = "Transition Fraction"))
	float CascadeTransitionFraction;

	/** 
	 * Controls the size of the fade out region at the far extent of the dynamic shadow's influence.  
	 * This is specified as a fraction of DynamicShadowDistance. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "1.0", DisplayName = "Distance Fadeout Fraction"))
	float ShadowDistanceFadeoutFraction;

	/** 
	 * Stationary lights only: Whether to use per-object inset shadows for movable components, even though cascaded shadow maps are enabled.
	 * This allows dynamic objects to have a shadow even when they are outside of the cascaded shadow map, which is important when DynamicShadowDistanceStationaryLight is small.
	 * If DynamicShadowDistanceStationaryLight is large (currently > 8000), this will be forced off.
	 * Disabling this can reduce shadowing cost significantly with many movable objects.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=CascadedShadowMaps, DisplayName = "Inset Shadows For Movable Objects")
	uint32 bUseInsetShadowsForMovableObjects : 1;

	/** 0: no Far Shadow Cascades, otherwise the number of cascades between DynamicShadowDistance and FarShadowDistance that are covered by Far Shadow Cascades. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "4"), DisplayName = "Far Shadow Cascade Count")
	int32 FarShadowCascadeCount;

	/** 
	 * Distance at which the far shadow cascade should end.  Far shadows will cover the range between 'Dynamic Shadow Distance' and this distance. 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=CascadedShadowMaps, meta=(UIMin = "0", UIMax = "800000"), DisplayName = "Far Shadow Distance")
	float FarShadowDistance;

	/** 
	 * Distance at which the ray traced shadow cascade should end.  Distance field shadows will cover the range between 'Dynamic Shadow Distance' this distance. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldShadows, meta=(UIMin = "0", UIMax = "100000"), DisplayName = "DistanceField Shadow Distance")
	float DistanceFieldShadowDistance;

	/**
	 * Forward lighting priority for the single directional light that will be used for forward shading, translucent, single layer water and volumetric fog.
	 * When two lights have equal priorities, the selection will be based on their overall brightness as a fallback.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Light, meta = (UIMin = "0", ClampMin = "0"))
	int32 ForwardShadingPriority;

	/** 
	 * Angle subtended by light source in degrees (also known as angular diameter).
	 * Defaults to 0.5357 which is the angle for our sun.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0", UIMax = "5"), DisplayName = "Source Angle")
	float LightSourceAngle;

	/** 
	 * Angle subtended by soft light source in degrees.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0", UIMax = "5"), DisplayName = "Source Soft Angle")
	float LightSourceSoftAngle;

	/**
	 * Shadow source angle factor, relative to the light source angle.
	 * Defaults to 1.0 to coincide with light source angle.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = RayTracing, meta = (UIMin = "0", UIMax = "5"), DisplayName = "Shadow Source Angle Factor")
	float ShadowSourceAngleFactor;

	/** Determines how far shadows can be cast, in world units.  Larger values increase the shadowing cost. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=DistanceFieldShadows, meta=(UIMin = "1000", UIMax = "100000"), DisplayName = "DistanceField Trace Distance")
	float TraceDistance;

	UPROPERTY()
	uint32 bUsedAsAtmosphereSunLight_DEPRECATED : 1;

	/**
	 * Whether the directional light can interact with the atmosphere, cloud and generate a visual disk. All of which compose the visual sky.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category= AtmosphereAndCloud, meta=(DisplayName = "Atmosphere Sun Light", ShouldShowInViewport = true))
	uint32 bAtmosphereSunLight : 1;

	/**
	 * Two atmosphere lights are supported. For instance: a sun and a moon, or two suns.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = AtmosphereAndCloud, meta = (DisplayName = "Atmosphere Sun Light Index", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax= "1"))
	int32 AtmosphereSunLightIndex;

	/**
	 * A color multiplied with the sun disk luminance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (DisplayName = "Atmosphere Sun Disk Color Scale"))
	FLinearColor AtmosphereSunDiskColorScale;

	/**
	 * Whether to apply atmosphere transmittance per pixel on opaque meshes, instead of using the light global transmittance. Note: VolumetricCloud per pixel transmittance option is selectable on the VolumetricCloud component itself.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay)
	uint32 bPerPixelAtmosphereTransmittance : 1;

	/**
	 * Whether the light should cast any shadows from opaque meshes onto clouds. This is disabled when 'Atmosphere Sun Light Index' is set to 1.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud)
	uint32 bCastShadowsOnClouds : 1;
	/**
	 * Whether the light should cast any shadows from opaque meshes onto the atmosphere.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud)
	uint32 bCastShadowsOnAtmosphere : 1;
	/**
	 * Whether the light should cast any shadows from clouds onto the atmosphere and other scene elements.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud)
	uint32 bCastCloudShadows : 1;
	/**
	 * The overall strength of the cloud shadow, higher value will block more light.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", SliderExponent = 1.0))
	float CloudShadowStrength;
	/**
	 * The strength of the shadow on atmosphere. Disabled when 0.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", SliderExponent = 1.0))
	float CloudShadowOnAtmosphereStrength;
	/**
	 * The strength of the shadow on opaque and transparent meshes. Disabled when 0.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", SliderExponent = 1.0))
	float CloudShadowOnSurfaceStrength;
	/**
	 * The bias applied to the shadow front depth of the volumetric cloud shadow map.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "-1", UIMax = "1"))
	float CloudShadowDepthBias;
	/**
	 * The world space radius of the cloud shadow map around the camera in kilometers.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "1", ClampMin = "1"))
	float CloudShadowExtent;
	/**
	 * Scale the cloud shadow map resolution, base resolution is 512. The resolution is still clamped to 'r.VolumetricCloud.ShadowMap.MaxResolution'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "0.25", UIMax = "8", ClampMin = "0.25", SliderExponent = 1.0))
	float CloudShadowMapResolutionScale;
	/**
	 * Scale the shadow map tracing sample count.
	 * The sample count resolution is still clamped according to scalability setting to 'r.VolumetricCloud.ShadowMap.RaySampleMaxCount'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, AdvancedDisplay, meta = (UIMin = "0.25", UIMax = "8", ClampMin = "0.25", SliderExponent = 1.0))
	float CloudShadowRaySampleCountScale;

	/**
	 * Scales the lights contribution when scattered in cloud participating media. This can help counter balance the fact that our multiple scattering solution is only an approximation.
	 * Setting this to 0 on the secondary atmospheric light will optimise the cloud tracing to only use a single light.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = AtmosphereAndCloud, meta = (HideAlphaChannel))
	FLinearColor CloudScatteredLuminanceScale;

	/** The Lightmass settings for this object. */
	UPROPERTY(EditAnywhere, Category=Light, meta=(ShowOnlyInnerProperties))
	struct FLightmassDirectionalLightSettings LightmassSettings;

	/**
	* Whether the light should cast modulated shadows from dynamic objects (mobile only).  Also requires Cast Shadows to be set to True.
	**/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (EditCondition = "Mobility == EComponentMobility::Stationary"))
	uint32 bCastModulatedShadows : 1;

	/**
	* Color to modulate against the scene color when rendering modulated shadows. (mobile only)
	**/
	UPROPERTY(BlueprintReadOnly, interp, Category = Light, meta = (HideAlphaChannel), AdvancedDisplay)
	FColor ModulatedShadowColor;
	
	/**
	 * Control the amount of shadow occlusion. A value of 0 means no occlusion, thus no shadow.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category = Light, meta = (HideAlphaChannel, UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"), AdvancedDisplay)
	float ShadowAmount;

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetDynamicShadowDistanceMovableLight(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetDynamicShadowDistanceStationaryLight(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetDynamicShadowCascades(int32 NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetCascadeDistributionExponent(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetCascadeTransitionFraction(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetShadowDistanceFadeoutFraction(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetShadowCascadeBiasDistribution(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetEnableLightShaftOcclusion(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetOcclusionMaskDarkness(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetOcclusionDepthRange(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetLightShaftOverrideDirection(FVector NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetLightSourceAngle(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetLightSourceSoftAngle(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetShadowSourceAngleFactor(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetShadowAmount(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetAtmosphereSunLight(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetAtmosphereSunLightIndex(int32 NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetAtmosphereSunDiskColorScale(FLinearColor NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetForwardShadingPriority(int32 NewValue);

	//~ Begin ULightComponent Interface
	ENGINE_API virtual FVector4 GetLightPosition() const override;
	ENGINE_API virtual ELightComponentType GetLightType() const override;
	virtual FLightmassLightSettings GetLightmassSettings() const override
	{
		return LightmassSettings;
	}

	ENGINE_API virtual float GetUniformPenumbraSize() const override;

	ENGINE_API virtual FLightSceneProxy* CreateSceneProxy() const override;
	virtual bool IsUsedAsAtmosphereSunLight() const override
	{
		return bAtmosphereSunLight; 
	}
	virtual uint8 GetAtmosphereSunLightIndex() const override
	{
		return static_cast<uint8>(AtmosphereSunLightIndex);
	}
	virtual FLinearColor GetAtmosphereSunDiskColorScale() const override
	{
		return AtmosphereSunDiskColorScale;
	}
	ENGINE_API virtual ELightUnits GetLightUnits() const;
	//~ End ULightComponent Interface

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual bool ForceActorNonSpatiallyLoaded() const override { return true; }
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ Begin UObject Interface

	ENGINE_API virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
};



