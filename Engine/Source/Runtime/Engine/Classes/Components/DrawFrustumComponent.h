// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "DrawFrustumComponent.generated.h"

class FPrimitiveSceneProxy;

/**
 *	Utility component for drawing a view frustum. Origin is at the component location, frustum points down position X axis.
 */ 

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDrawFrustumComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	// Enable or disable frustum visualization for this camera
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DrawFrustumComponent)
	bool bFrustumEnabled = true;

	/** Color to draw the wireframe frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	FColor FrustumColor;

	/** Angle of longest dimension of view shape. 
	  * If the angle is 0 then an orthographic projection is used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	float FrustumAngle;

	/** Ratio of horizontal size over vertical size. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	float FrustumAspectRatio;

	/** Distance from origin to start drawing the frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	float FrustumStartDist;

	/** Distance from origin to stop drawing the frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	float FrustumEndDist;

	/** optional texture to show on the near plane */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DrawFrustumComponent)
	TObjectPtr<class UTexture> Texture;


	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End UPrimitiveComponent Interface.

	virtual bool GetIgnoreBoundsForEditorFocus() const override { return true; }
};



