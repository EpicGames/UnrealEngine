// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SphereComponent.h"
#include "DrawSphereComponent.generated.h"

struct FConvexVolume;
struct FEngineShowFlags;

/** 
 * A sphere generally used for simple collision. Bounds are rendered as lines in the editor.
 */
UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDrawSphereComponent : public USphereComponent
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
};



