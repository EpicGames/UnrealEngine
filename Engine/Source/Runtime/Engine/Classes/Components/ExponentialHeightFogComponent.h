// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "StateStream/ExponentialHeightFogStateStreamHandle.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "Rendering/ExponentialHeightFogData.h"
#include "ExponentialHeightFogComponent.generated.h"

/**
 *	Used to create fogging effects such as clouds but with a density that is related to the height of the fog.
 */
UCLASS(ClassGroup=Rendering, collapsecategories, hidecategories=(Object, Mobility), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UExponentialHeightFogComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Global density factor. */
	UPROPERTY(BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta=(UIMin = "0", UIMax = ".05"))
	float FogDensity;

	/**
	 * Height density factor, controls how the density increases as height decreases.
	 * Smaller values make the visible transition larger.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category = ExponentialHeightFogComponent, meta = (UIMin = "0.001", UIMax = "2"))
	float FogHeightFalloff;

	/** Settings for the second fog. Setting the density of this to 0 means it doesn't have any influence. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ExponentialHeightFogComponent)
	FExponentialHeightFogData SecondFogData;

	UPROPERTY()
	FLinearColor FogInscatteringColor_DEPRECATED;

	/**
	 * Note: when r.SupportExpFogMatchesVolumetricFog = 1, this value is ignored and the volumetric fog Emissive is used instead.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta = (DisplayName = "Fog Inscattering Color"))
	FLinearColor FogInscatteringLuminance;

	/** Color used to modulate the SkyAtmosphere component contribution to the non directional component of the fog. Only effective when r.SupportSkyAtmosphereAffectsHeightFog>0 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ExponentialHeightFogComponent)
	FLinearColor SkyAtmosphereAmbientContributionColorScale;

	/** 
	 * Cubemap that can be specified for fog color, which is useful to make distant, heavily fogged scene elements match the sky.
	 * When the cubemap is specified, FogInscatteringColor is ignored and Directional inscattering is disabled. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InscatteringTexture)
	TObjectPtr<class UTextureCube> InscatteringColorCubemap;

	/** When >0 and when r.SupportsSkyLightCaptureAffectsHeightFogInScattering is true, this will make the sky light capture inject scattered light into the height fog. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = InscatteringTexture, meta = (DisplayName = "Sky Light Capture In-Scattering Strength", UIMin = "0", UIMax = "1", ClampMin = "0"))
	float SkyLightCaptureAffectsHeightFogStrength;

	/** This is the roughness at which the sky light capture cubemap will be sample to simulate scattering dispersion due to a phase function. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = InscatteringTexture, meta = (DisplayName = "Sky Light Capture In-Scattering Roughness", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float SkyLightCaptureAffectsHeightFogRoughness;

	/** Angle to rotate the InscatteringColorCubemap around the Z axis. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InscatteringTexture, meta=(UIMin = "0", UIMax = "360"))
	float InscatteringColorCubemapAngle;

	/** Tint color used when InscatteringColorCubemap is specified, for quick edits without having to reimport InscatteringColorCubemap. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InscatteringTexture)
	FLinearColor InscatteringTextureTint;

	/** Distance at which InscatteringColorCubemap should be used directly for the Inscattering Color. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InscatteringTexture, meta=(UIMin = "1000", UIMax = "1000000"))
	float FullyDirectionalInscatteringColorDistance;

	/** Distance at which only the average color of InscatteringColorCubemap should be used as Inscattering Color. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InscatteringTexture, meta=(UIMin = "1000", UIMax = "1000000"))
	float NonDirectionalInscatteringColorDistance;

	/** 
	 * Controls the size of the directional inscattering cone, which is used to approximate inscattering from a directional light.  
	 * Note: 
	 *   - there must be a directional light with bUsedAsAtmosphereSunLight enabled for DirectionalInscattering to be used.
	 *   - When r.SupportExpFogMatchesVolumetricFog = 1, this value is ignored and the volumetric fog Scattering Distribution is used instead.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=DirectionalInscattering, meta=(UIMin = "2", UIMax = "64"))
	float DirectionalInscatteringExponent;

	/** 
	 * Controls the start distance from the viewer of the directional inscattering, which is used to approximate inscattering from a directional light. 
	 * Note: 
	 *   - There must be a directional light with bUsedAsAtmosphereSunLight enabled for DirectionalInscattering to be used.
	 *   - When r.SupportExpFogMatchesVolumetricFog = 1, this value is ignored.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=DirectionalInscattering)
	float DirectionalInscatteringStartDistance;

	UPROPERTY()
	FLinearColor DirectionalInscatteringColor_DEPRECATED;

	/** 
	 * Controls the color of the directional inscattering, which is used to approximate inscattering from a directional light. 
	 * Note:
	 *   - there must be a directional light with bUsedAsAtmosphereSunLight enabled for DirectionalInscattering to be used.
	 *   - When r.SupportExpFogMatchesVolumetricFog = 1, this value is ignored.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=DirectionalInscattering, meta = (DisplayName = "Directional Inscattering Color"))
	FLinearColor DirectionalInscatteringLuminance;

	/** 
	 * Maximum opacity of the fog.  
	 * A value of 1 means the fog can become fully opaque at a distance and replace scene color completely,
	 * A value of 0 means the fog color will not be factored in at all.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta=(UIMin = "0", UIMax = "1"))
	float FogMaxOpacity;

	/** Distance from the camera that the fog will start, in world units. */
	UPROPERTY(BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta=(UIMin = "0", UIMax = "5000"))
	float StartDistance;

	/** Distance from the camera, on the horizontal XY plane, that the fog will end integrating the lighting and transmittance. Disabled when 0. */
	UPROPERTY(BlueprintReadOnly, interp, Category = ExponentialHeightFogComponent, meta = (UIMin = "0", UIMax = "500000"))
	float EndDistance;

	/** Scene elements past this distance will not have fog applied.  This is useful for excluding skyboxes which already have fog baked in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ExponentialHeightFogComponent, meta=(UIMin = "100000", UIMax = "20000000"))
	float FogCutoffDistance;

	/** 
	 * Whether to enable Volumetric fog.  Scalability settings control the resolution of the fog simulation. 
	 * Note that Volumetric fog currently does not support StartDistance, FogMaxOpacity and FogCutoffDistance.
	 * Volumetric fog also can't match exponential height fog in general as exponential height fog has non-physical behavior.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Volumetric Fog"))
	bool bEnableVolumetricFog;

	/** 
	 * Controls the scattering phase function - how much incoming light scatters in various directions.
	 * A distribution value of 0 scatters equally in all directions, while .9 scatters predominantly in the light direction.  
	 * In order to have visible volumetric fog light shafts from the side, the distribution will need to be closer to 0.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Scattering Distribution", UIMin = "-.9", UIMax = ".9"))
	float VolumetricFogScatteringDistribution;

	/** 
	 * The height fog particle reflectiveness used by volumetric fog. 
	 * Water particles in air have an albedo near white, while dust has slightly darker value.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Albedo"))
	FColor VolumetricFogAlbedo;

	/** 
	 * Light emitted by height fog.  This is a density so more light is emitted the further you are looking through the fog.
	 * In most cases skylight is a better choice, however right now volumetric fog does not support precomputed lighting, 
	 * So stationary skylights are unshadowed and static skylights don't affect volumetric fog at all.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Emissive"))
	FLinearColor VolumetricFogEmissive;

	/** Scales the height fog particle extinction amount used by volumetric fog.  Values larger than 1 cause fog particles everywhere absorb more light. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Extinction Scale", UIMin = ".1", UIMax = "10"))
	float VolumetricFogExtinctionScale;

	/** 
	 * Distance over which volumetric fog should be computed, after the start distance.  Larger values extend the effect into the distance but expose under-sampling artifacts in details.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "View Distance", UIMin = "1000", UIMax = "10000"))
	float VolumetricFogDistance;

	/** 
	 * Distance from the camera that the volumetric fog will start, in world units. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category= VolumetricFog, meta=(DisplayName = "Start Distance", UIMin = "0", UIMax = "5000"))
	float VolumetricFogStartDistance;

	/** 
	 * Distance over which volumetric fog will fade in from the start distance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Near Fade In Distance", UIMin = "0", UIMax = "1000"))
	float VolumetricFogNearFadeInDistance;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, meta=(DisplayName = "Static Lighting Scattering Intensity", UIMin = "0", UIMax = "10"))
	float VolumetricFogStaticLightingScatteringIntensity;

	/** 
	 * Whether to use FogInscatteringColor for the Sky Light volumetric scattering color and DirectionalInscatteringColor for the Directional Light scattering color. 
	 * Make sure your directional light has 'Atmosphere Sun Light' enabled!
	 * Enabling this allows Volumetric fog to better match Height fog in the distance, but produces non-physical volumetric lighting that may not match surface lighting.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VolumetricFog, AdvancedDisplay)
	bool bOverrideLightColorsWithFogInscatteringColors;

	/**
	 * Enable FSSS (Fog Screen Space Scattering) to approximate multiple scattering.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fog Screen Space Scattering - EXPERIMENTAL", AdvancedDisplay, meta = (DisplayName = "Enable Fog Screen Space Scattering"))
	bool bEnableFSSS;

	/**
	 * The amount of scene color injected into the fog screen space scattering technique.
	 * When 0, only the fog luminance will be scattered.
	 * When 1, fog and scene luminance will be scattered.
	 * When above 1, more scene color can be injected to increase the brightness of the multiple scattering effect, even when the fog is not too intense. We cannot scatter more than the scene color however.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fog Screen Space Scattering - EXPERIMENTAL", AdvancedDisplay, meta = (DisplayName = "Scene Color Scattering Amount Scale", UIMin = "0", UIMax = "1", ClampMin = "0"))
	float FSSSSceneColorScatteringAmountScale;

	/**
	 * Apply a power over the amount of scattered scene luminance after it is evaluated using Scene Color Scattering Amount Scale.
	 * A value smaller than 1 can show more scattering where the fog is less intense.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fog Screen Space Scattering - EXPERIMENTAL", AdvancedDisplay, meta = (DisplayName = "Scene Color Scattering Amount Power", UIMin = "0.01", UIMax = "2", ClampMin = "0"))
	float FSSSSceneColorScatteringAmountPower;

	/**
	 * The scale of the spread for the fog multiple scattering. This basically scales mip level sampled from the blurred screen space FSSS texture.
	 * Increasing will make scene and thicker fog look fuzzier.
	 * But too high values can cause artifacts such as halos around depth discontinuities.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fog Screen Space Scattering - EXPERIMENTAL", AdvancedDisplay, meta = (DisplayName = "Spread Scale", UIMin = "0", UIMax = "1", ClampMin = "0"))
	float FSSSSpreadScale;

	/**
	 * When generating the the blurred screen space FSSS texture, this controls how much of the blurred lower mip levels is injected back into the sharper higher mip levels.
	 * A higher value will result in a blurrier result. But when = 1, multiple scattering will be a single average color of the FSSS texture.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fog Screen Space Scattering - EXPERIMENTAL", AdvancedDisplay, meta = (DisplayName = "Blur Control", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float FSSSBlurControl;


	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, Interp)
	uint8 bHoldout : 1;

	/** If true, this component will be rendered in the main pass (basepass, transparency) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bRenderInMainPass : 1;

	/** If true, this component will be visible in reflection captures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInReflectionCaptures : 1;

	/** If true, this component will be visible in real-time sky light reflection captures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInRealTimeSkyCaptures : 1;


public:
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFogDensity(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetSecondFogDensity(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFogInscatteringColor(FLinearColor Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetInscatteringColorCubemap(UTextureCube* Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetInscatteringColorCubemapAngle(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFullyDirectionalInscatteringColorDistance(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetNonDirectionalInscatteringColorDistance(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetInscatteringTextureTint(FLinearColor Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetDirectionalInscatteringExponent(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetDirectionalInscatteringStartDistance(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetDirectionalInscatteringColor(FLinearColor Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetSecondFogHeightOffset(float Value);
	
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFogHeightFalloff(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetSecondFogHeightFalloff(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFogMaxOpacity(float Value);
	
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetStartDistance(float Value);
	
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetEndDistance(float Value);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|ExponentialHeightFog")
	ENGINE_API void SetFogCutoffDistance(float Value);
	
	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFog(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogScatteringDistribution(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogExtinctionScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogAlbedo(FColor NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogEmissive(FLinearColor NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogDistance(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogStartDistance(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|VolumetricFog")
	ENGINE_API void SetVolumetricFogNearFadeInDistance(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|VolumetricFog")
	ENGINE_API void SetSecondFogData(FExponentialHeightFogData NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHoldout(bool bNewHoldout);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRenderInMainPass(bool bValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|DirectionalInscattering")
	ENGINE_API void SetSkyAtmosphereAmbientContributionColorScale(FLinearColor NewValue);

protected:
	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void OnUnregister() override;
	//~ End UActorComponent Interface.

	ENGINE_API void AddFogIfNeeded();

	#if WITH_STATE_STREAM_ACTOR
	FExponentialHeightFogHandle Handle;
	#endif

public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
};


