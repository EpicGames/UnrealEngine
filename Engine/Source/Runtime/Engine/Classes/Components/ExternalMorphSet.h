// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Rendering/MorphTargetVertexInfoBuffers.h"

/** An external morph target set. External morph targets are managed by systems outside of the skinned meshes. */
struct FExternalMorphSet
{
	/** A name for this set, useful for debugging. */
	FName Name = FName("Unknown");

	/** The GPU compressed morph buffers. */
	FMorphTargetVertexInfoBuffers MorphBuffers;
};

/** The weight data for a specific external morph set. */
struct FExternalMorphSetWeights
{
	/** Update the number of active morph targets. */
	ENGINE_API void UpdateNumActiveMorphTargets();

	/** Set all weights to 0. Optionally set the NumActiveMorphTargets to zero as well. */
	ENGINE_API void ZeroWeights(bool bZeroNumActiveMorphTargets=true);

	/** The debug name. */
	FName Name = FName("Unknown ExternalMorphSetWeights");

	/** The weights, which can also be negative and go beyond 1.0 or -1.0. */
	TArray<float> Weights;

	/** The number of active morph targets. */
	int32 NumActiveMorphTargets = 0;

	/** The treshold used to determine if a morph target is active or not. Any weight equal to or above this value is seen as active morph target. */
	float ActiveWeightThreshold = 0.001f;
};

/** The morph target weight data for all external morph target sets. */
struct FExternalMorphWeightData
{
	/** Update the number of active morph targets for all sets. */
	ENGINE_API void UpdateNumActiveMorphTargets();

	/** Reset the morph target sets. */
	void Reset() { MorphSets.Reset(); NumActiveMorphTargets = 0; }

	/** Check if we have active morph targets or not. */
	bool HasActiveMorphs() const { return (NumActiveMorphTargets > 0); }

	/** The map with a collection of morph sets. Each set can contains multiple morph targets. */
	TMap<int32, FExternalMorphSetWeights> MorphSets;

	/** The number of active morph targets. */
	int32 NumActiveMorphTargets = 0;
};
