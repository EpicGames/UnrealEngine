// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "GameFramework/ForceFeedbackAttenuation.h"
#include "Components/SceneComponent.h"
#include "Tickable.h"
#include "UObject/GCObject.h"
#include "ForceFeedbackComponent.generated.h"

struct FDisplayDebugManager;
struct FForceFeedbackValues;
class UForceFeedbackEffect;

class FForceFeedbackManager : public FTickableGameObject, FGCObject
{
private:

	FForceFeedbackManager(UWorld* InWorld)
		: World(InWorld)
	{
	}

public:

	static FForceFeedbackManager* Get(UWorld* World, bool bCreateIfMissing = false);

	void AddActiveComponent(UForceFeedbackComponent* ForceFeedbackComponent);
	void RemoveActiveComponent(UForceFeedbackComponent* ForceFeedbackComponent);

	void Update(FVector Location, FForceFeedbackValues& Values, const FPlatformUserId UserId) const;

	void DrawDebug(const FVector Location, FDisplayDebugManager& DisplayDebugManager, const FPlatformUserId UserId) const;

private:
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override;
	virtual UWorld* GetTickableGameObjectWorld() const override;
	virtual void Tick( float DeltaTime ) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;

	TObjectPtr<UWorld> World;
	TArray<TObjectPtr<UForceFeedbackComponent>> ActiveForceFeedbackComponents;

	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	static TArray<FForceFeedbackManager*> PerWorldForceFeedbackManagers;
	static FDelegateHandle OnWorldCleanupHandle;
};

/** called when we finish playing forcefeedback effect, either because it played to completion or because a Stop() call turned it off early */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FOnForceFeedbackFinished, UForceFeedbackComponent*, ForceFeedbackComponent);

/**
 * ForceFeedbackComponent allows placing a rumble effect in to the world and having it apply to player characters who come near it
 */
UCLASS(ClassGroup=(Utility), hidecategories=(Object, ActorComponent, Physics, Rendering, Mobility, LOD), ShowCategories=Trigger, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UForceFeedbackComponent : public USceneComponent
{
	GENERATED_BODY()

public:

	ENGINE_API UForceFeedbackComponent(const FObjectInitializer& ObjectInitializer);

	/** The feedback effect to be played */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ForceFeedback)
	TObjectPtr<UForceFeedbackEffect> ForceFeedbackEffect;

	/** Auto destroy this component on completion */
	UPROPERTY()
	uint8 bAutoDestroy:1;

	/** Stop effect when owner is destroyed */
	UPROPERTY()
	uint8 bStopWhenOwnerDestroyed:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ForceFeedback)
	uint8 bLooping:1;

	/** Should the playback of the forcefeedback pattern ignore time dilation and use the app's delta time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ForceFeedback)
	uint8 bIgnoreTimeDilation:1;

	/** Should the Attenuation Settings asset be used (false) or should the properties set directly on the component be used for attenuation properties */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attenuation)
	uint8 bOverrideAttenuation:1;

	/** The intensity multiplier to apply to effects generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ForceFeedback)
	float IntensityMultiplier;

	/** If bOverrideSettings is false, the asset to use to determine attenuation properties for effects generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attenuation, meta=(EditCondition="!bOverrideAttenuation"))
	TObjectPtr<class UForceFeedbackAttenuation> AttenuationSettings;

	/** If bOverrideSettings is true, the attenuation properties to use for effects generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attenuation, meta=(EditCondition="bOverrideAttenuation"))
	struct FForceFeedbackAttenuationSettings AttenuationOverrides;

	/** called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
	UPROPERTY(BlueprintAssignable)
	FOnForceFeedbackFinished OnForceFeedbackFinished;

	/** Set what force feedback effect is played by this component */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback")
	ENGINE_API void SetForceFeedbackEffect( UForceFeedbackEffect* NewForceFeedbackEffect);

	/** Start a feedback effect playing */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback")
	ENGINE_API virtual void Play(float StartTime = 0.f);

	/** Stop playing the feedback effect */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback")
	ENGINE_API virtual void Stop();

	/** Set a new intensity multiplier */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback")
	ENGINE_API void SetIntensityMultiplier(float NewIntensityMultiplier);

	/** Modify the attenuation settings of the component */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback")
	ENGINE_API void AdjustAttenuation(const FForceFeedbackAttenuationSettings& InAttenuationSettings);

	//~ Begin UObject Interface.
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin USceneComponent Interface
	ENGINE_API virtual void Activate(bool bReset=false) override;
	ENGINE_API virtual void Deactivate() override;
	//~ End USceneComponent Interface

	//~ Begin ActorComponent Interface.
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void OnRegister() override;
#endif
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual const UObject* AdditionalStatObject() const override;
	ENGINE_API virtual bool IsReadyForOwnerToAutoDestroy() const override;
	//~ End ActorComponent Interface.

	/** Returns a pointer to the attenuation settings to be used (if any) for this audio component dependent on the ForceFeedbackEffectAttenuation asset or overrides set. */
	ENGINE_API const FForceFeedbackAttenuationSettings* GetAttenuationSettingsToApply() const;

	UFUNCTION(BlueprintCallable, Category = "ForceFeedback", meta = (DisplayName = "Get Attenuation Settings To Apply", ScriptName="GetAttenuationSettingsToApply"))
	ENGINE_API bool BP_GetAttenuationSettingsToApply(FForceFeedbackAttenuationSettings& OutAttenuationSettings) const;

	/** Collects the various attenuation shapes that may be applied to the effect played by the component for visualization in the editor. */
	ENGINE_API void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const;

private:
	
	float PlayTime;

#if WITH_EDITORONLY_DATA
	/** Utility function that updates which texture is displayed on the sprite dependent on the properties of the Audio Component. */
	ENGINE_API void UpdateSpriteTexture();
#endif

	ENGINE_API bool Advance(float DeltaTime);
	ENGINE_API void Update(FVector Location, FForceFeedbackValues& Values, const FPlatformUserId UserId) const;
	ENGINE_API void StopInternal(bool bRemoveFromManager = true);

	friend FForceFeedbackManager;
};



