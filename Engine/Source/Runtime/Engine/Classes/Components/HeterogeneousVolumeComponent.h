// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "EngineDefines.h"
#include "GameFramework/Info.h"
#include "Misc/Guid.h"
#include "RenderResource.h"

#include "HeterogeneousVolumeComponent.generated.h"

class USparseVolumeTexture;

/**
 * A component that represents a heterogeneous volume.
 */
UCLASS(Blueprintable, ClassGroup = (Rendering, Common), hidecategories = (Object, Activation, "Components|Activation"), ShowCategories = (Mobility), editinlinenew, meta = (BlueprintSpawnableComponent),MinimalAPI)
class UHeterogeneousVolumeComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Volume)
	FIntVector VolumeResolution;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Animation, AdvancedDisplay)
	FTransform FrameTransform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (EditCondition = "bPlaying == false"))
	float Frame;

	UPROPERTY(EditAnywhere, Category = Animation)
	float FrameRate;

	UPROPERTY(EditAnywhere, Category = Animation)
	float StartFrame;

	UPROPERTY(EditAnywhere, Category = Animation)
	float EndFrame;

	UPROPERTY(EditAnywhere, Category = Animation)
	uint32 bPlaying : 1;

	UPROPERTY(EditAnywhere, Category = Animation)
	uint32 bLooping : 1;

	// Bias to apply to the calculated mip level to stream at. This property essentially influences the distance from the
	// volume at which certain mip levels are no longer requested. Higher values result in earlier mip level transitions.
	UPROPERTY(EditAnywhere, Category = SparseVolumeTextureStreaming, AdvancedDisplay)
	float StreamingMipBias;

	UPROPERTY(EditAnywhere, Category = SparseVolumeTextureStreaming)
	uint32 bIssueBlockingRequests : 1;

	UPROPERTY(EditAnywhere, Category = Volume)
	uint32 bPivotAtCentroid : 1;

	UPROPERTY(EditAnywhere, Category = Lighting)
	float StepFactor;

	UPROPERTY(EditAnywhere, Category = Lighting)
	float ShadowStepFactor;

	UPROPERTY(EditAnywhere, Category = Lighting)
	float ShadowBiasFactor;

	UPROPERTY(EditAnywhere, Category = Lighting)
	float LightingDownsampleFactor;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstanceDynamic;

	UFUNCTION(BlueprintCallable, Category = "Volume")
	ENGINE_API void SetVolumeResolution(FIntVector NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetFrame(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetFrameRate(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetStartFrame(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetEndFrame(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetPlaying(bool NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void SetLooping(bool NewValue);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void Play();

	UFUNCTION(BlueprintCallable, Category = "SparseVolumeTextureStreaming")
	ENGINE_API void SetStreamingMipBias(int32 NewValue);

	~UHeterogeneousVolumeComponent() = default;

public:
	//~ Begin USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostReinitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool ShouldRenderSelected() const override { return true; }
	virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	virtual int32 GetNumMaterials() const override { return 1; }
	virtual int32 GetMaterialIndex(FName MaterialSlotName) const override;
	virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* Material) override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	//~ End UMeshComponent Interface.

private:
	const USparseVolumeTexture* PreviousSVT;

	static USparseVolumeTexture* GetSparseVolumeTexture(UMaterialInterface* MaterialInterface, int32 ParameterIndex, FName* OutParamName = nullptr);
	static UMaterialInstanceDynamic* CreateOrCastToMID(UMaterialInterface* MaterialInterface);
	void OnSparseVolumeTextureChanged(const USparseVolumeTexture* SparseVolumeTexture);
	UMaterialInterface* GetHeterogeneousVolumeMaterial() const; // Gets the UMaterialInterface* returned by GetMaterial(0), but returns nullptr if the material is incompatible with HeterogeneousVolumes.
};

/**
 * A placeable actor that represents a heterogeneous volume.
 */
UCLASS(showcategories = (Movement, Rendering, Transformation, DataLayers, "Input|MouseInput", "Input|TouchInput"), ClassGroup = Fog, hidecategories = (Info, Object, Input), MinimalAPI)
class AHeterogeneousVolume : public AInfo
{
	GENERATED_UCLASS_BODY()

private:

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Volume, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UHeterogeneousVolumeComponent> HeterogeneousVolumeComponent;

#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer()  const override { return true; }
#endif

};
