// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Async/AsyncWork.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "StaticMeshResources.h"

#include "HierarchicalInstancedStaticMeshComponent.generated.h"

class FStaticLightingTextureMapping_InstancedStaticMesh;


UENUM()
enum class EHISMViewRelevanceType : uint8
{
	Grass,
	Foliage,
	HISM
};

// Due to BulkSerialize we can't edit the struct, so we must deprecated this one and create a new one
USTRUCT()
struct FClusterNode_DEPRECATED
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector3f BoundMin;
	UPROPERTY()
	int32 FirstChild;
	UPROPERTY()
	FVector3f BoundMax;
	UPROPERTY()
	int32 LastChild;
	UPROPERTY()
	int32 FirstInstance;
	UPROPERTY()
	int32 LastInstance;

	FClusterNode_DEPRECATED()
		: BoundMin(MAX_flt, MAX_flt, MAX_flt)
		, FirstChild(-1)
		, BoundMax(-MAX_flt, -MAX_flt, -MAX_flt)
		, LastChild(-1)
		, FirstInstance(-1)
		, LastInstance(-1)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FClusterNode_DEPRECATED& NodeData)
	{
		// @warning BulkSerialize: FClusterNode is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << NodeData.BoundMin;
		Ar << NodeData.FirstChild;
		Ar << NodeData.BoundMax;
		Ar << NodeData.LastChild;
		Ar << NodeData.FirstInstance;
		Ar << NodeData.LastInstance;

		return Ar;
	}
};


USTRUCT()
struct FClusterNode
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector3f BoundMin;
	UPROPERTY()
	int32 FirstChild;
	UPROPERTY()
	FVector3f BoundMax;
	UPROPERTY()
	int32 LastChild;
	UPROPERTY()
	int32 FirstInstance;
	UPROPERTY()
	int32 LastInstance;

	UPROPERTY()
	FVector3f MinInstanceScale;
	UPROPERTY()
	FVector3f MaxInstanceScale;

	FClusterNode()
		: BoundMin(MAX_flt, MAX_flt, MAX_flt)
		, FirstChild(-1)
		, BoundMax(-MAX_flt, -MAX_flt, -MAX_flt)
		, LastChild(-1)
		, FirstInstance(-1)
		, LastInstance(-1)
		, MinInstanceScale(MAX_flt)
		, MaxInstanceScale(-MAX_flt)
	{
	}

	FClusterNode(const FClusterNode_DEPRECATED& OldNode)
		: BoundMin(OldNode.BoundMin)
		, FirstChild(OldNode.FirstChild)
		, BoundMax(OldNode.BoundMax)
		, LastChild(OldNode.LastChild)
		, FirstInstance(OldNode.FirstChild)
		, LastInstance(OldNode.LastInstance)
		, MinInstanceScale(MAX_flt)
		, MaxInstanceScale(-MAX_flt)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FClusterNode& NodeData)
	{
		// @warning BulkSerialize: FClusterNode is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << NodeData.BoundMin;
		Ar << NodeData.FirstChild;
		Ar << NodeData.BoundMax;
		Ar << NodeData.LastChild;
		Ar << NodeData.FirstInstance;
		Ar << NodeData.LastInstance;
		Ar << NodeData.MinInstanceScale;
		Ar << NodeData.MaxInstanceScale;
		
		return Ar;
	}
};

UCLASS(ClassGroup=Rendering, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UHierarchicalInstancedStaticMeshComponent : public UInstancedStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

	ENGINE_API ~UHierarchicalInstancedStaticMeshComponent();

	TSharedPtr<TArray<FClusterNode>, ESPMode::ThreadSafe> ClusterTreePtr;

	// If true then we allow a translated space when building the cluster tree.
	// This can help for implementations (foliage) where we can have instances with offsets to large for single float precision.
	UPROPERTY()
	uint32 bUseTranslatedInstanceSpace : 1;

	// Origin of the translated space used when building the cluster tree.
	UPROPERTY()
	FVector TranslatedInstanceSpaceOrigin;

	// Table for remapping instances from cluster tree to PerInstanceSMData order
	UPROPERTY()
	TArray<int32> SortedInstances;

	// The number of instances in the ClusterTree. Subsequent instances will always be rendered.
	UPROPERTY()
	int32 NumBuiltInstances;

	// Normally equal to NumBuiltInstances, but can be lower if density scaling is in effect
	int32 NumBuiltRenderInstances;

	// Bounding box of any built instances (cached from the ClusterTree)
	UPROPERTY()
	FBox BuiltInstanceBounds;

	// Bounding box of any unbuilt instances
	UPROPERTY()
	FBox UnbuiltInstanceBounds;

	// Bounds of each individual unbuilt instance, used for LOD calculation
	UPROPERTY()
	TArray<FBox> UnbuiltInstanceBoundsList;

	// Enable for detail meshes that don't really affect the game. Disable for anything important.
	// Typically, this will be enabled for small meshes without collision (e.g. grass) and disabled for large meshes with collision (e.g. trees)
	UPROPERTY()
	uint32 bEnableDensityScaling:1;

	// Current value of density scaling applied to this component
	float CurrentDensityScaling;

	// The number of nodes in the occlusion layer
	UPROPERTY()
	int32 OcclusionLayerNumNodes;

	// The last mesh bounds that was cache
	UPROPERTY()
	FBoxSphereBounds CacheMeshExtendedBounds;

	// Instances to render (including removed one until the build is complete)
	UPROPERTY()
	int32 InstanceCountToRender;

	bool bIsAsyncBuilding : 1;
	bool bIsOutOfDate : 1;
	bool bConcurrentChanges : 1;
	bool bAutoRebuildTreeOnInstanceChanges : 1;

#if WITH_EDITOR
	// in Editor mode we might disable the density scaling for edition
	bool bCanEnableDensityScaling : 1;
#endif

public:
	struct FClusterTree
	{
		TArray<FClusterNode> Nodes;
		TArray<int32> SortedInstances;
		TArray<int32> InstanceReorderTable;
		int32 OutOcclusionLayerNum = 0;

		bool PrintLevel(int32 NodeIndex, int32 Level, int32 CurrentLevel, int32 Parent);
	};

	class FClusterBuilder
	{
	public:
		ENGINE_API FClusterBuilder(TArray<FMatrix> InTransforms, TArray<float> InCustomDataFloats, int32 InNumCustomDataFloats, const FBox& InInstBox, int32 InMaxInstancesPerLeaf, float InDensityScaling, int32 InInstancingRandomSeed, bool InGenerateInstanceScalingRange);
		ENGINE_API void BuildTreeAndBufferAsync(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
		ENGINE_API void BuildTreeAndBuffer();
		ENGINE_API void BuildTree();

	protected:
		void Split(int32 InNum);
		void Split(int32 Start, int32 End);
		void BuildInstanceBuffer();
		void Init();

	public:
		TUniquePtr<FClusterTree> Result;
		TUniquePtr<FStaticMeshInstanceData> BuiltInstanceData;

	protected:
		int32 OriginalNum;
		int32 Num;
		FBox InstBox;
		int32 BranchingFactor;
		int32 InternalNodeBranchingFactor;
		int32 OcclusionLayerTarget;
		int32 MaxInstancesPerLeaf;
		int32 NumRoots;

		int32 InstancingRandomSeed;
		float DensityScaling;
		bool GenerateInstanceScalingRange;

		TArray<int32> SortIndex;
		TArray<FVector> SortPoints;
		TArray<FMatrix> Transforms;
		TArray<float> CustomDataFloats;
		int32 NumCustomDataFloats;

		struct FRunPair
		{
			int32 Start;
			int32 Num;

			FRunPair(int32 InStart, int32 InNum)
				: Start(InStart)
				, Num(InNum)
			{
			}

			bool operator< (const FRunPair& Other) const
			{
				return Start < Other.Start;
			}
		};
		TArray<FRunPair> Clusters;

		struct FSortPair
		{
			float d;
			int32 Index;

			bool operator< (const FSortPair& Other) const
			{
				return d < Other.d;
			}
		};
		TArray<FSortPair> SortPairs;
	};

	// Apply the results of the async build
	ENGINE_API void ApplyBuildTreeAsync(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent, TSharedRef<FClusterBuilder, ESPMode::ThreadSafe> Builder, double StartTime);

	//Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& BoundTransform) const override;
	
#if WITH_EDITOR
	ENGINE_API virtual void PostStaticMeshCompilation() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	// UInstancedStaticMesh interface
	ENGINE_API virtual int32 AddInstance(const FTransform& InstanceTransform, bool bWorldSpace = false) override;
	ENGINE_API virtual TArray<int32> AddInstances(const TArray<FTransform>& InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace = false, bool bUpdateNavigation = true) override;
	ENGINE_API virtual bool RemoveInstance(int32 InstanceIndex) override;
	ENGINE_API virtual bool RemoveInstances(const TArray<int32>& InstancesToRemove) override;
	ENGINE_API virtual bool RemoveInstances(const TArray<int32>& InstancesToRemove, bool bInstanceArrayAlreadySortedInReverseOrder) override;
	ENGINE_API virtual bool UpdateInstanceTransform(int32 InstanceIndex, const FTransform& NewInstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;
	ENGINE_API virtual bool SetCustomDataValue(int32 InstanceIndex, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty = false) override;
	ENGINE_API virtual bool SetCustomData(int32 InstanceIndex, TArrayView<const float> InCustomData, bool bMarkRenderStateDirty = false) override;
	ENGINE_API virtual bool BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace=false, bool bMarkRenderStateDirty=false, bool bTeleport=false) override;
	ENGINE_API virtual bool BatchUpdateInstancesTransforms(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport) override;

	ENGINE_API virtual bool BatchUpdateInstancesTransform(int32 StartInstanceIndex, int32 NumInstances, const FTransform& NewInstancesTransform, bool bWorldSpace=false, bool bMarkRenderStateDirty=false, bool bTeleport=false) override;
	ENGINE_API virtual bool BatchUpdateInstancesData(int32 StartInstanceIndex, int32 NumInstances, FInstancedStaticMeshInstanceData* StartInstanceData, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;

	ENGINE_API virtual void ClearInstances() override;
	ENGINE_API virtual TArray<int32> GetInstancesOverlappingSphere(const FVector& Center, float Radius, bool bSphereInWorldSpace = true) const override;
	ENGINE_API virtual TArray<int32> GetInstancesOverlappingBox(const FBox& Box, bool bBoxInWorldSpace = true) const override;
	ENGINE_API virtual void PreAllocateInstancesMemory(int32 AddedInstanceCount) override;
	virtual bool SupportsRemoveSwap() const override { return true; }

	/** Get the number of instances that overlap a given sphere */
	ENGINE_API int32 GetOverlappingSphereCount(const FSphere& Sphere) const;
	/** Get the number of instances that overlap a given box */
	ENGINE_API int32 GetOverlappingBoxCount(const FBox& Box) const;
	/** Get the transforms of instances inside the provided box */
	ENGINE_API void GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const;

	ENGINE_API bool BuildTreeIfOutdated(bool Async, bool ForceUpdate);
	bool IsAsyncBuilding() const { return bIsAsyncBuilding; }
	bool IsTreeFullyBuilt() const { return !bIsOutOfDate; }
	ENGINE_API void GetTree(TArray<FClusterNode>& OutClusterTree) const;
	ENGINE_API FVector GetAverageScale() const;

	/** Heuristic for the number of leaves in the tree **/
	ENGINE_API int32 DesiredInstancesPerLeaf();

	ENGINE_API virtual void ApplyComponentInstanceData(struct FInstancedStaticMeshComponentInstanceData* InstancedMeshData) override;
	
	// Number of instances in the render-side instance buffer
	virtual int32 GetNumRenderInstances() const { return SortedInstances.Num(); }

	/** Will apply current density scaling, if enabled **/
	ENGINE_API void UpdateDensityScaling();

	ENGINE_API virtual void PropagateLightingScenarioChange() override;

	EHISMViewRelevanceType GetViewRelevanceType() const { return ViewRelevanceType; }

protected:
	UE_DEPRECATED(5.7, "Please use BuildComponentInstanceData with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual void BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData) override;
	ENGINE_API virtual void BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData) override;


	ENGINE_API virtual void BuildTree();
	ENGINE_API virtual void BuildTreeAsync();
	ENGINE_API void ApplyBuildTree(FClusterBuilder& Builder, const bool bWasAsyncBuild);
	ENGINE_API void ApplyEmpty();
	ENGINE_API void SetPerInstanceLightMapAndEditorData(FStaticMeshInstanceData& PerInstanceData, const TArray<TRefCountPtr<HHitProxy>>& HitProxies);

	ENGINE_API FVector CalcTranslatedInstanceSpaceOrigin() const;
	ENGINE_API void GetInstanceTransforms(TArray<FMatrix>& InstanceTransforms, FVector const& Offset) const;

	/** Removes specified instances */ 
	ENGINE_API void RemoveInstancesInternal(TConstArrayView<int32> InstanceIndices);
	
	/** Gets and approximate number of verts for each LOD to generate heuristics **/
	ENGINE_API int32 GetVertsForLOD(int32 LODIndex);
	/** Average number of instances per leaf **/
	ENGINE_API float ActualInstancesPerLeaf();
	/** For testing, prints some stats after any kind of build **/
	ENGINE_API void PostBuildStats();

	ENGINE_API virtual void OnPostLoadPerInstanceData() override;

	ENGINE_API virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;

	virtual FVector GetTranslatedInstanceSpaceOrigin() const override { return TranslatedInstanceSpaceOrigin; }

	ENGINE_API static FBox GetClusterTreeBounds(TArray<FClusterNode> const& InClusterTree, const FVector& InOffset);

	FGraphEventArray BuildTreeAsyncTasks;
	EHISMViewRelevanceType ViewRelevanceType = EHISMViewRelevanceType::HISM;

private:
	/** hidden Implementation of BatchUpdateInstancesTransforms - it is shared by the TArray and TArrayView version of public API */
	ENGINE_API bool BatchUpdateInstancesTransformsInternal(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport);

protected:
	friend FStaticLightingTextureMapping_InstancedStaticMesh;
	friend FInstancedLightMap2D;
	friend FInstancedShadowMap2D;
	friend FClusterBuilder;
};

