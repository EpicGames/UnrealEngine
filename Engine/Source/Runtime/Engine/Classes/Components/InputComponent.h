// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Components/ActorComponent.h"
#include "Framework/Commands/InputChord.h"
#include "InputComponent.generated.h"

class UPlayerInput;

/** Utility delegate class to allow binding to either a C++ function or a blueprint script delegate */
template<class DelegateType, class DynamicDelegateType>
struct TInputUnifiedDelegate
{
	TInputUnifiedDelegate() {};
	TInputUnifiedDelegate(DelegateType D) : FuncDelegate(MakeShared<DelegateType>(MoveTemp(D))) {};
	TInputUnifiedDelegate(DynamicDelegateType D) : FuncDynDelegate(MakeShared<DynamicDelegateType>(MoveTemp(D))) {};

	/** Returns if either the native or dynamic delegate is bound */
	inline bool IsBound() const
	{
		return ((FuncDelegate.IsValid() && FuncDelegate->IsBound()) || (FuncDynDelegate.IsValid() && FuncDynDelegate->IsBound()));
	}

	/** Returns the UObject that this delegate is bound to. */
	inline UObject* GetUObject() const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				return FuncDelegate->GetUObject();
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				return FuncDynDelegate->GetUObject();
			}
		}
		
		return nullptr;
	}
	
	/** Returns if either the native or dynamic delegate is bound to an object */
	inline bool IsBoundToObject(void const* Object) const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				return FuncDelegate->IsBoundToObject(Object);
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				return FuncDynDelegate->IsBoundToObject(Object);
			}
		}

		return false;
	}

	/** Binds a native delegate and unbinds any bound dynamic delegate */
	template< class UserClass >
	inline void BindDelegate(UserClass* Object, typename DelegateType::template TMethodPtr< UserClass > Func)
	{
		FuncDynDelegate.Reset();;
		FuncDelegate = MakeShared<DelegateType>(DelegateType::CreateUObject(Object, Func));
	}

	/** Binds a dynamic delegate and unbinds any bound native delegate */
	inline void BindDelegate(UObject* Object, const FName FuncName)
	{
		FuncDelegate.Reset();
		FuncDynDelegate = MakeShared<DynamicDelegateType>();
		FuncDynDelegate->BindUFunction(Object, FuncName);
	}

	/** Returns a reference to the native delegate and unbinds any bound dynamic delegate */
	DelegateType& GetDelegateForManualSet()
	{
		FuncDynDelegate.Reset();
		FuncDelegate = MakeShared<DelegateType>();
		return *FuncDelegate;
	}

	/** Unbinds any bound delegates */
	inline void Unbind()
	{
		FuncDelegate.Reset();
		FuncDynDelegate.Reset();
	}

	/** Returns a const reference to the Function Delegate. */
	inline const DelegateType& GetDelegate() const
	{
		return (FuncDelegate.IsValid() ? *FuncDelegate : UnboundDelegate);
	}

	/** Returns a const reference to the Dynamic Function Delegate. */
	inline const DynamicDelegateType& GetDynamicDelegate() const
	{
		return (FuncDynDelegate.IsValid() ? *FuncDynDelegate : UnboundDynamicDelegate);
	}

protected:
	/** Holds the delegate to call. */
	TSharedPtr<DelegateType> FuncDelegate;
	/** Holds the dynamic delegate to call. */
	TSharedPtr<DynamicDelegateType> FuncDynDelegate;

	static const DelegateType UnboundDelegate;
	static const DynamicDelegateType UnboundDynamicDelegate;
};

template<class DelegateType, class DynamicDelegateType>
const DelegateType TInputUnifiedDelegate<DelegateType, DynamicDelegateType>::UnboundDelegate;

template<class DelegateType, class DynamicDelegateType>
const DynamicDelegateType TInputUnifiedDelegate<DelegateType, DynamicDelegateType>::UnboundDynamicDelegate;


/** Base class for the different binding types. */
struct FInputBinding
{
	/** Whether the binding should consume the input or allow it to pass to another component */
	uint8 bConsumeInput:1;

	/** Whether the binding should execute while paused */
	uint8 bExecuteWhenPaused:1;

	FInputBinding()
		: bConsumeInput(true)
		, bExecuteWhenPaused(false)
	{}
};

/** Delegate signature for action events. */
DECLARE_DELEGATE( FInputActionHandlerSignature );
DECLARE_DELEGATE_OneParam( FInputActionHandlerWithKeySignature, FKey );
DECLARE_DYNAMIC_DELEGATE_OneParam( FInputActionHandlerDynamicSignature, FKey, Key );

struct FInputActionUnifiedDelegate
{
	FInputActionUnifiedDelegate() : BoundDelegateType(EBoundDelegate::Unbound) {};
	FInputActionUnifiedDelegate(FInputActionHandlerSignature D) : FuncDelegate(MakeShared<FInputActionHandlerSignature>(MoveTemp(D))), BoundDelegateType(EBoundDelegate::Delegate) {};
	FInputActionUnifiedDelegate(FInputActionHandlerWithKeySignature D) : FuncDelegateWithKey(MakeShared<FInputActionHandlerWithKeySignature>(MoveTemp(D))), BoundDelegateType(EBoundDelegate::DelegateWithKey) {};
	FInputActionUnifiedDelegate(FInputActionHandlerDynamicSignature D) : FuncDynDelegate(MakeShared<FInputActionHandlerDynamicSignature>(MoveTemp(D))), BoundDelegateType(EBoundDelegate::DynamicDelegate) {};

	/** Returns if either the native or dynamic delegate is bound */
	inline bool IsBound() const
	{
		switch (BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			return FuncDelegate->IsBound();

		case EBoundDelegate::DelegateWithKey:
			return FuncDelegateWithKey->IsBound();

		case EBoundDelegate::DynamicDelegate:
			return FuncDynDelegate->IsBound();
		}

		return false;
	}

	/** Returns if either the native or dynamic delegate is bound to an object */
	inline bool IsBoundToObject(FDelegateUserObjectConst Object) const
	{
		switch (BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			return (FuncDelegate->IsBound() && FuncDelegate->IsBoundToObject(Object));

		case EBoundDelegate::DelegateWithKey:
			return (FuncDelegateWithKey->IsBound() && FuncDelegateWithKey->IsBoundToObject(Object));

		case EBoundDelegate::DynamicDelegate:
			return (FuncDynDelegate->IsBound() && FuncDynDelegate->IsBoundToObject(Object));
		}

		return false;
	}

	/** Returns the UObject bound to either the native or dynamic delegate (if either is a UFunction or UObject delegate) */
	inline const UObject* GetUObject() const
	{
		switch (BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			return FuncDelegate->GetUObject();

		case EBoundDelegate::DelegateWithKey:
			return FuncDelegateWithKey->GetUObject();

		case EBoundDelegate::DynamicDelegate:
			return FuncDynDelegate->GetUObject();
		}

		return nullptr;
	}

	/** Returns the object bound to either the native or dynamic delegate as a raw, untyped pointer. If you're looking for a bound UObject, prefer GetUObject(). */
	inline const void* GetObject() const
	{
		switch (BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			return FuncDelegate->GetObjectForTimerManager();

		case EBoundDelegate::DelegateWithKey:
			return FuncDelegateWithKey->GetObjectForTimerManager();

		case EBoundDelegate::DynamicDelegate:
			return FuncDynDelegate->GetUObject();
		}

		return nullptr;
	}

	/** Binds a native delegate and unbinds any bound dynamic delegate */
	template< class UserClass >
	inline void BindDelegate(UserClass* Object, typename FInputActionHandlerSignature::template TMethodPtr< UserClass > Func)
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::Delegate;
		FuncDelegate = MakeShared<FInputActionHandlerSignature>(FInputActionHandlerSignature::CreateUObject(Object, Func));
	}

	template< class UserClass >
	inline void BindDelegate(UserClass* Object, typename FInputActionHandlerWithKeySignature::template TMethodPtr< UserClass > Func)
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::DelegateWithKey;
		FuncDelegateWithKey = MakeShared<FInputActionHandlerWithKeySignature>(FInputActionHandlerWithKeySignature::CreateUObject(Object, Func));
	}

	template< class DelegateType, class UserClass, typename... VarTypes >
	inline void BindDelegate(UserClass* Object, typename DelegateType::template TMethodPtr< UserClass > Func, VarTypes... Vars)
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::Delegate;
		FuncDelegate = MakeShared<FInputActionHandlerSignature>(FInputActionHandlerSignature::CreateUObject(Object, Func, Vars...));
	}

	/** Binds a dynamic delegate and unbinds any bound native delegate */
	inline void BindDelegate(UObject* Object, const FName FuncName)
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::DynamicDelegate;
		FuncDynDelegate = MakeShared<FInputActionHandlerDynamicSignature>();
		FuncDynDelegate->BindUFunction(Object, FuncName);
	}

	/** Returns a reference to the native delegate and unbinds any bound dynamic delegate */
	FInputActionHandlerSignature& GetDelegateForManualSet()
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::Delegate;
		FuncDelegate = MakeShared<FInputActionHandlerSignature>();
		return *FuncDelegate;
	}

	/** Returns a reference to the native delegate and unbinds any bound dynamic delegate */
	FInputActionHandlerWithKeySignature& GetDelegateWithKeyForManualSet()
	{
		Unbind();
		BoundDelegateType = EBoundDelegate::DelegateWithKey;
		FuncDelegateWithKey = MakeShared<FInputActionHandlerWithKeySignature>();
		return *FuncDelegateWithKey;
	}

	/** Unbinds any bound delegates */
	inline void Unbind()
	{
		switch(BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			FuncDelegate->Unbind();
			break;

		case EBoundDelegate::DelegateWithKey:
			FuncDelegateWithKey->Unbind();
			break;

		case EBoundDelegate::DynamicDelegate:
			FuncDynDelegate->Unbind();
			break;
		}
		BoundDelegateType = EBoundDelegate::Unbound;
	}

	/** Execute function for the action unified delegate. */
	inline void Execute(const FKey Key) const
	{
		switch(BoundDelegateType)
		{
		case EBoundDelegate::Delegate:
			if (FuncDelegate->IsBound())
			{
				FuncDelegate->Execute();
			}
			break;

		case EBoundDelegate::DelegateWithKey:
			if (FuncDelegateWithKey->IsBound())
			{
				FuncDelegateWithKey->Execute(Key);
			}
			break;

		case EBoundDelegate::DynamicDelegate:
			if (FuncDynDelegate->IsBound())
			{
				FuncDynDelegate->Execute(Key);
			}
			break;
		}
	}
private:
	/** Holds the delegate to call. */
	TSharedPtr<FInputActionHandlerSignature> FuncDelegate;
	/** Holds the delegate that wants to know the key to call. */
	TSharedPtr<FInputActionHandlerWithKeySignature> FuncDelegateWithKey;
	/** Holds the dynamic delegate to call. */
	TSharedPtr<FInputActionHandlerDynamicSignature> FuncDynDelegate;

	enum class EBoundDelegate : uint8
	{
		Unbound,
		Delegate,
		DelegateWithKey,
		DynamicDelegate
	};

	EBoundDelegate BoundDelegateType;
};

/** Binds a delegate to an action. */
struct FInputActionBinding : public FInputBinding
{
private:
	/** Whether the binding is part of a paired (both pressed and released events bound) action */
	uint8 bPaired:1;

public:
	/** Key event to bind it to, e.g. pressed, released, double click */
	TEnumAsByte<EInputEvent> KeyEvent;

private:
	/** Friendly name of action, e.g "jump" */
	FName ActionName;

	/** The handle for this binding action */
	int32 Handle;

public:
	/** The delegate bound to the action */
	FInputActionUnifiedDelegate ActionDelegate;

	FInputActionBinding()
		: FInputBinding()
		, bPaired(false)
		, KeyEvent(EInputEvent::IE_Pressed)
		, ActionName(NAME_None)
		, Handle(INDEX_NONE)
	{ }

	FInputActionBinding(const FName InActionName, const  EInputEvent InKeyEvent)
		: FInputBinding()
		, bPaired(false)
		, KeyEvent(InKeyEvent)
		, ActionName(InActionName)
		, Handle(INDEX_NONE)
	{ }

	FName GetActionName() const { return ActionName; }
	bool IsPaired() const { return bPaired; }
	int32 GetHandle() const { return Handle; }

	bool operator==(const FInputActionBinding& Rhs)
	{
		return (IsValid() && GetHandle() == Rhs.GetHandle());
	}

	/** Indicates GenerateNewHandle was called */
	bool IsValid() { return (Handle != INDEX_NONE); }

	void GenerateNewHandle();

	friend class UInputComponent;
};

/** Binds a delegate to a key chord. */
struct FInputKeyBinding : public FInputBinding
{
	/** Key event to bind it to (e.g. pressed, released, double click) */
	TEnumAsByte<EInputEvent> KeyEvent;

	/** Input Chord to bind to */
	FInputChord Chord;

	/** The delegate bound to the key chord */
	FInputActionUnifiedDelegate KeyDelegate;

	FInputKeyBinding()
		: FInputBinding()
		, KeyEvent(EInputEvent::IE_Pressed)
	{ }

	FInputKeyBinding(const FInputChord InChord, const EInputEvent InKeyEvent)
		: FInputBinding()
		, KeyEvent(InKeyEvent)
		, Chord(InChord)
	{ }
};


/** 
 * Delegate signature for touch handlers. 
 * @FingerIndex: Which finger touched
 * @Location: The 2D screen location that was touched
 */
DECLARE_DELEGATE_TwoParams( FInputTouchHandlerSignature, ETouchIndex::Type, FVector );
DECLARE_DYNAMIC_DELEGATE_TwoParams( FInputTouchHandlerDynamicSignature, ETouchIndex::Type, FingerIndex, FVector, Location );

/** Unified delegate specialization for Touch events. */
struct FInputTouchUnifiedDelegate : public TInputUnifiedDelegate<FInputTouchHandlerSignature, FInputTouchHandlerDynamicSignature>
{
	/** Execute function for the touch unified delegate. */
	inline void Execute(const ETouchIndex::Type FingerIndex, const FVector Location) const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				FuncDelegate->Execute(FingerIndex, Location);
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				FuncDynDelegate->Execute(FingerIndex, Location);
			}
		}
	}
};

/** Binds a delegate to touch input. */
struct FInputTouchBinding : public FInputBinding
{
	/** Key event to bind it to (e.g. pressed, released, double click) */
	TEnumAsByte<EInputEvent> KeyEvent;

	/** The delegate bound to the touch events */
	FInputTouchUnifiedDelegate TouchDelegate;

	FInputTouchBinding()
		: FInputBinding()
		, KeyEvent(EInputEvent::IE_Pressed)
	{ }

	FInputTouchBinding(const enum EInputEvent InKeyEvent)
		: FInputBinding()
		, KeyEvent(InKeyEvent)
	{ }
};


/** 
 * Delegate signature for axis handlers. 
 * @AxisValue: "Value" to pass to the axis.  This value will be the device-dependent, so a mouse will report absolute change since the last update, 
 *		a joystick will report total displacement from the center, etc.  It is up to the handler to interpret this data as it sees fit, i.e. treating 
 *		joystick values as a rate of change would require scaling by frametime to get an absolute delta.
 */
DECLARE_DELEGATE_OneParam( FInputAxisHandlerSignature, float );
DECLARE_DYNAMIC_DELEGATE_OneParam( FInputAxisHandlerDynamicSignature, float, AxisValue );

/** Unified delegate specialization for float axis events. */
struct FInputAxisUnifiedDelegate : public TInputUnifiedDelegate<FInputAxisHandlerSignature, FInputAxisHandlerDynamicSignature>
{
	/** Execute function for the axis unified delegate. */
	inline void Execute(const float AxisValue) const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				FuncDelegate->Execute(AxisValue);
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				FuncDynDelegate->Execute(AxisValue);
			}
		}
	}
};


/** Binds a delegate to an axis mapping. */
struct FInputAxisBinding : public FInputBinding
{
	/** The axis mapping being bound to. */
	FName AxisName;

	/** 
	 * The delegate bound to the axis. 
	 * It will be called each frame that the input component is in the input stack 
	 * regardless of whether the value is non-zero or has changed.
	 */
	FInputAxisUnifiedDelegate AxisDelegate;

	/** 
	 * The value of the axis as calculated during the most recent UPlayerInput::ProcessInputStack
	 * if the InputComponent was in the stack, otherwise all values should be 0.
	 */
	float AxisValue;

	FInputAxisBinding()
		: FInputBinding()
		, AxisName(NAME_None)
		, AxisValue(0.f)
	{ }

	FInputAxisBinding(const FName InAxisName)
		: FInputBinding()
		, AxisName(InAxisName)
		, AxisValue(0.f)
	{ }
};


/** Binds a delegate to a raw float axis mapping. */
struct FInputAxisKeyBinding : public FInputBinding
{
	/** 
	* The value of the axis as calculated during the most recent UPlayerInput::ProcessInputStack
	* if the InputComponent containing the binding was in the stack, otherwise the value will be 0.
	*/
	float AxisValue;

	/** The axis being bound to. */
	FKey AxisKey;

	/** 
	 * The delegate bound to the axis. 
	 * It will be called each frame that the input component is in the input stack 
	 * regardless of whether the value is non-zero or has changed.
	 */
	FInputAxisUnifiedDelegate AxisDelegate;

	FInputAxisKeyBinding()
		: FInputBinding()
		, AxisValue(0.f)
	{ }

	FInputAxisKeyBinding(const FKey InAxisKey)
		: FInputBinding()
		, AxisValue(0.f)
		, AxisKey(InAxisKey)
	{
		ensure(AxisKey.IsAxis1D());
	}
};


/**
* Delegate signature for vector axis handlers.
* @AxisValue: "Value" to pass to the axis.
*/
DECLARE_DELEGATE_OneParam(FInputVectorAxisHandlerSignature, FVector);
DECLARE_DYNAMIC_DELEGATE_OneParam(FInputVectorAxisHandlerDynamicSignature, FVector, AxisValue);

/** Unified delegate specialization for vector axis events. */
struct FInputVectorAxisUnifiedDelegate : public TInputUnifiedDelegate<FInputVectorAxisHandlerSignature, FInputVectorAxisHandlerDynamicSignature>
{
	/** Execute function for the axis unified delegate. */
	inline void Execute(const FVector AxisValue) const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				FuncDelegate->Execute(AxisValue);
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				FuncDynDelegate->Execute(AxisValue);
			}
		}
	}
};

/** Binds a delegate to a raw vector axis mapping. */
struct FInputVectorAxisBinding : public FInputBinding
{
	/** 
	* The value of the axis as calculated during the most recent UPlayerInput::ProcessInputStack
	* if the InputComponent containing the binding was in the stack, otherwise the value will be (0,0,0).
	*/
	FVector AxisValue;

	/** The axis being bound to. */
	FKey AxisKey;

	/** 
	 * The delegate bound to the axis. 
	 * It will be called each frame that the input component is in the input stack 
	 * regardless of whether the value is non-zero or has changed.
	 */
	FInputVectorAxisUnifiedDelegate AxisDelegate;

	FInputVectorAxisBinding()
		: FInputBinding()
	{ }

	FInputVectorAxisBinding(const FKey InAxisKey)
		: FInputBinding()
		, AxisKey(InAxisKey)
	{
		ensure(AxisKey.IsAxis2D() || AxisKey.IsAxis3D());
	}
};


/** 
 * Delegate signature for gesture handlers. 
 * @Value: "Value" to pass to the axis.  Note that by convention this is assumed to be a framerate-independent "delta" value, i.e. absolute change for this frame
 *				so the handler need not scale by frametime.
 */
DECLARE_DELEGATE_OneParam( FInputGestureHandlerSignature, float );
DECLARE_DYNAMIC_DELEGATE_OneParam( FInputGestureHandlerDynamicSignature, float, Value );

/** Unified delegate specialization for gestureevents. */
struct FInputGestureUnifiedDelegate : public TInputUnifiedDelegate<FInputGestureHandlerSignature, FInputGestureHandlerDynamicSignature>
{
	/** Execute function for the gesture unified delegate. */
	inline void Execute(const float Value) const
	{
		if (FuncDelegate.IsValid())
		{
			if (FuncDelegate->IsBound())
			{
				FuncDelegate->Execute(Value);
			}
		}
		else if (FuncDynDelegate.IsValid())
		{
			if (FuncDynDelegate->IsBound())
			{
				FuncDynDelegate->Execute(Value);
			}
		}
	}
};


/** Binds a gesture to a function. */
struct FInputGestureBinding : public FInputBinding
{
	/** Value parameter, meaning is dependent on the gesture. */
	float GestureValue;

	/** The gesture being bound to. */
	FKey GestureKey;

	/** The delegate bound to the gesture events */
	FInputGestureUnifiedDelegate GestureDelegate;

	FInputGestureBinding()
		: FInputBinding()
		, GestureValue(0.f)
	{ }

	FInputGestureBinding(const FKey InGestureKey)
		: FInputBinding()
		, GestureValue(0.f)
		, GestureKey(InGestureKey)
	{ }
};



UENUM()
namespace EControllerAnalogStick
{
	enum Type : int
	{
		CAS_LeftStick,
		CAS_RightStick,
		CAS_MAX
	};
}

/**
* Struct that exists to store runtime cache to make key to action lookups faster.
*/
USTRUCT()
struct FCachedKeyToActionInfo
{
	GENERATED_BODY()

	/** Which PlayerInput object this has been built for */
	UPROPERTY()
	TWeakObjectPtr<UPlayerInput> PlayerInput;

	/** What index of the player input's key mappings was the map built for. */
	uint32 KeyMapBuiltForIndex;

	/** Reverse lookup map to speed up evaluation of action bindings. Will be rebuilt via ConditionalBuildKeyMap when action bindings or key mappings are changed. */
	TMap<FKey, TArray<TSharedPtr<FInputActionBinding>>> KeyToActionMap;

	/** Keep the AnyKey to action map separately as we don't want to have query the map to find it every time. */
	TArray<TSharedPtr<FInputActionBinding>> AnyKeyToActionMap;

	FCachedKeyToActionInfo()
		: PlayerInput(nullptr)
		, KeyMapBuiltForIndex(0)
	{
	}
};

/**
 * Implement an Actor component for input bindings.
 *
 * An Input Component is a transient component that enables an Actor to bind various forms of input events to delegate functions.  
 * Input components are processed from a stack managed by the PlayerController and processed by the PlayerInput.
 * Each binding can consume the input event preventing other components on the input stack from processing the input.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
UCLASS(NotBlueprintable, transient, config=Input, hidecategories=(Activation, "Components|Activation"), MinimalAPI)
class UInputComponent
	: public UActorComponent
{
	GENERATED_UCLASS_BODY()

	/** The collection of key bindings. */
	TArray<FInputKeyBinding> KeyBindings;

	/** The collection of touch bindings. */
	TArray<FInputTouchBinding> TouchBindings;

	/** The collection of axis bindings. */
	TArray<FInputAxisBinding> AxisBindings;

	/** The collection of axis key bindings. */
	TArray<FInputAxisKeyBinding> AxisKeyBindings;

	/** The collection of vector axis bindings. */
	TArray<FInputVectorAxisBinding> VectorAxisBindings;

	/** The collection of gesture bindings. */
	TArray<FInputGestureBinding> GestureBindings;

private:
	/** Holds the collection of action bindings. */
	TArray<TSharedPtr<FInputActionBinding>> ActionBindings;

	UPROPERTY(Transient, DuplicateTransient)
	TArray<FCachedKeyToActionInfo> CachedKeyToActionInfo;

	UFUNCTION()
	ENGINE_API void OnInputOwnerEndPlayed(AActor* InOwner, EEndPlayReason::Type EndPlayReason);

public:
	/** The priority of this input component when pushed in to the stack. */
	int32 Priority;

	/** Whether any components lower on the input stack should be allowed to receive input. */
	uint8 bBlockInput:1;

	/** Clears any input callback delegates from the given UObject */
	ENGINE_API virtual void ClearBindingsForObject(UObject* InOwner);
	
	ENGINE_API void ConditionalBuildKeyMap(UPlayerInput* PlayerInput);

	/**
	 * Gets the current value of the axis with the specified name.
	 *
	 * @param AxisName The name of the axis.
	 * @return Axis value.
	 * @see GetAxisKeyValue, GetVectorAxisValue
	 */
	ENGINE_API float GetAxisValue( const FName AxisName ) const;

	/**
	 * Gets the current value of the axis with the specified key.
	 *
	 * @param AxisKey The key of the axis.
	 * @return Axis value.
	 * @see GetAxisKeyValue, GetVectorAxisValue
	 */
	ENGINE_API float GetAxisKeyValue( const FKey AxisKey ) const;

	/**
	 * Gets the current vector value of the axis with the specified key.
	 *
	 * @param AxisKey The key of the axis.
	 * @return Axis value.
	 * @see GetAxisValue, GetAxisKeyValue
	 */
	ENGINE_API FVector GetVectorAxisValue( const FKey AxisKey ) const;

	/**
	 * Checks whether this component has any input bindings.
	 *
	 * @return true if any bindings are set, false otherwise.
	 */
	ENGINE_API virtual bool HasBindings() const;


	/**
	 * Adds the specified action binding.
	 *
	 * @param Binding The binding to add.
	 * @return The last binding in the list.
	 * @see ClearActionBindings, GetActionBinding, GetNumActionBindings, RemoveActionBinding
	 */
	ENGINE_API FInputActionBinding& AddActionBinding( FInputActionBinding Binding );

	/**
	 * Removes all action bindings.
	 *
	 * @see AddActionBinding, GetActionBinding, GetNumActionBindings, RemoveActionBinding
	 */
	ENGINE_API virtual void ClearActionBindings();

	/**
	 * Gets the action binding with the specified index.
	 *
	 * @param BindingIndex The index of the binding to get.
	 * @see AddActionBinding, ClearActionBindings, GetNumActionBindings, RemoveActionBinding
	 */
	FInputActionBinding& GetActionBinding(const int32 BindingIndex) const { return *ActionBindings[BindingIndex].Get(); }

	/**
	 * Gets the number of action bindings.
	 *
	 * @return Number of bindings.
	 * @see AddActionBinding, ClearActionBindings, GetActionBinding, RemoveActionBinding
	 */
	int32 GetNumActionBindings() const { return ActionBindings.Num(); }

	/**
	 * Removes the action binding at the specified index.
	 *
	 * @param BindingIndex The index of the binding to remove.
	 * @see AddActionBinding, ClearActionBindings, GetActionBinding, GetNumActionBindings
	 */
	ENGINE_API void RemoveActionBinding( const int32 BindingIndex );
	ENGINE_API void RemoveActionBinding(FName ActionName, EInputEvent KeyEvent);

	/**
	 * Removes the action binding at the specified handle.
	 *
	 * @param Handle The handle of the binding to remove.
	 * @see AddActionBinding, ClearActionBindings, GetActionBinding, GetNumActionBindings
	 */
	ENGINE_API void RemoveActionBindingForHandle(const int32 Handle);

	/**
	 * Removes the action binding (index need for multi-name fixups).
	 *
	 * @param BindingToRemove The binding to remove.
	 * @param BindingIndex The binding's index for actions with the same name to fixup their data.
	 * @see AddActionBinding, ClearActionBindings, GetActionBinding, GetNumActionBindings
	 */
	ENGINE_API void RemoveActionBinding(const FInputActionBinding &BindingToRemove, const int32 BindingIndex);

	/** Clears all cached binding values. */
	ENGINE_API void ClearBindingValues();

	/**
	 * Binds a delegate function to an Action defined in the project settings.
	 * Returned reference is only guaranteed to be valid until another action is bound.
	 */
	template<class UserClass>
	FInputActionBinding& BindAction( const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputActionBinding AB( ActionName, KeyEvent );
		AB.ActionDelegate.BindDelegate(Object, Func);
		return AddActionBinding(MoveTemp(AB));
	}

	/**
	 * Binds a delegate function to an Action defined in the project settings.
	 * Returned reference is only guaranteed to be valid until another action is bound.
	 */
	template<class UserClass>
	FInputActionBinding& BindAction( const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerWithKeySignature::TMethodPtr< UserClass > Func )
	{
		FInputActionBinding AB( ActionName, KeyEvent );
		AB.ActionDelegate.BindDelegate(Object, Func);
		return AddActionBinding(MoveTemp(AB));
	}

	/**
	* Binds a delegate function to an Action defined in the project settings.
	* Returned reference is only guaranteed to be valid until another action is bound.
	*/
	template< class DelegateType, class UserClass, typename... VarTypes >
	FInputActionBinding& BindAction( const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename DelegateType::template TMethodPtr< UserClass > Func, VarTypes... Vars )
	{
		FInputActionBinding AB( ActionName, KeyEvent );
		AB.ActionDelegate.BindDelegate<DelegateType>(Object, Func, Vars...);
		return AddActionBinding(MoveTemp(AB));
	}

	/**
	 * Binds a delegate function an Axis defined in the project settings.
	 * Returned reference is only guaranteed to be valid until another axis is bound.
	 */
	template<class UserClass>
	FInputAxisBinding& BindAxis( const FName AxisName, UserClass* Object, typename FInputAxisHandlerSignature::TMethodPtr< UserClass > Func )
	{
		LogDeprecatedBindingWarning(AxisName);
		
		FInputAxisBinding AB( AxisName );
		AB.AxisDelegate.BindDelegate(Object, Func);
		AxisBindings.Emplace(MoveTemp(AB));
		return AxisBindings.Last();
	}

	/**
	* Removes the axis binding with the specified name.
	*
	* @param AxisName the name of the axis to remove.
	*/
	ENGINE_API void RemoveAxisBinding(const FName AxisName);

	/**
	* Removes all axis bindings.
	*/
	ENGINE_API void ClearAxisBindings();

	/**
	 * Indicates that the InputComponent is interested in knowing the Axis value
	 * (via GetAxisValue) but does not want a delegate function called each frame.
	 * Returned reference is only guaranteed to be valid until another axis is bound.
	 */
	FInputAxisBinding& BindAxis( const FName AxisName )
	{
		LogDeprecatedBindingWarning(AxisName);
		
		FInputAxisBinding AB( AxisName );
		AxisBindings.Emplace(MoveTemp(AB));
		return AxisBindings.Last();
	}

	/**
	 * Binds a delegate function for an axis key (e.g. Mouse X).
	 * Returned reference is only guaranteed to be valid until another axis key is bound.
	 */
	template<class UserClass>
	FInputAxisKeyBinding& BindAxisKey( const FKey AxisKey, UserClass* Object, typename FInputAxisHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputAxisKeyBinding AB(AxisKey);
		AB.AxisDelegate.BindDelegate(Object, Func);
		AxisKeyBindings.Emplace(MoveTemp(AB));
		return AxisKeyBindings.Last();
	}

	/**
	 * Indicates that the InputComponent is interested in knowing/consuming an axis key's
	 * value (via GetAxisKeyValue) but does not want a delegate function called each frame.
	 * Returned reference is only guaranteed to be valid until another axis key is bound.
	 */
	FInputAxisKeyBinding& BindAxisKey( const FKey AxisKey )
	{
		FInputAxisKeyBinding AB(AxisKey);
		AxisKeyBindings.Emplace(MoveTemp(AB));
		return AxisKeyBindings.Last();
	}

	/**
	 * Removes the axis key binding with the specified key.
	 * 
	 * @param AxisKey the key of the axis to remove.
	 */
	ENGINE_API void RemoveAxisKeyBinding( const FKey AxisKey);

	/**
	 * Binds a delegate function to a vector axis key (e.g. Tilt)
	 * Returned reference is only guaranteed to be valid until another vector axis key is bound.
	 */
	template<class UserClass>
	FInputVectorAxisBinding& BindVectorAxis( const FKey AxisKey, UserClass* Object, typename FInputVectorAxisHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputVectorAxisBinding AB(AxisKey);
		AB.AxisDelegate.BindDelegate(Object, Func);
		VectorAxisBindings.Emplace(MoveTemp(AB));
		return VectorAxisBindings.Last();
	}

	/**
	 * Indicates that the InputComponent is interested in knowing/consuming a vector axis key's
	 * value (via GetVectorAxisKeyValue) but does not want a delegate function called each frame.
	 * Returned reference is only guaranteed to be valid until another vector axis key is bound.
	 */
	FInputVectorAxisBinding& BindVectorAxis( const FKey AxisKey )
	{
		FInputVectorAxisBinding AB(AxisKey);
		VectorAxisBindings.Emplace(MoveTemp(AB));
		return VectorAxisBindings.Last();
	}

	/**
	 * Binds a chord event to a delegate function.
	 * Returned reference is only guaranteed to be valid until another input key is bound.
	 */
	template<class UserClass>
	FInputKeyBinding& BindKey( const FInputChord Chord, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputKeyBinding KB(Chord, KeyEvent);
		KB.KeyDelegate.BindDelegate(Object, Func);
		KeyBindings.Emplace(MoveTemp(KB));
		return KeyBindings.Last();
	}

	/**
	 * Binds a key event to a delegate function.
	 * Returned reference is only guaranteed to be valid until another input key is bound.
	 */
	template<class UserClass>
	FInputKeyBinding& BindKey( const FKey Key, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func )
	{
		return BindKey(FInputChord(Key, false, false, false, false), KeyEvent, Object, Func);
	}

	/**
	 * Binds a key event to a delegate function with key signature.
	 * Returned reference is only guaranteed to be valid until another input key is bound.
	 */
	template <class UserClass>
	FInputKeyBinding& BindKey(const FKey Key, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerWithKeySignature::TMethodPtr<UserClass> Func)
	{
		FInputKeyBinding KB(FInputChord(Key, false, false, false, false), KeyEvent);
		KB.KeyDelegate.BindDelegate(Object, Func);
		KeyBindings.Emplace(MoveTemp(KB));
		return KeyBindings.Last();
	}

	/**
	 * Binds this input component to touch events.
	 * Returned reference is only guaranteed to be valid until another touch event is bound.
	 */
	template<class UserClass>
	FInputTouchBinding& BindTouch( const EInputEvent KeyEvent, UserClass* Object, typename FInputTouchHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputTouchBinding TB(KeyEvent);
		TB.TouchDelegate.BindDelegate(Object, Func);
		TouchBindings.Emplace(MoveTemp(TB));
		return TouchBindings.Last();
	}

	/**
	 * Binds a gesture event to a delegate function.
	 * Returned reference is only guaranteed to be valid until another gesture event is bound.
	 */
	template<class UserClass>
	FInputGestureBinding& BindGesture( const FKey GestureKey, UserClass* Object, typename FInputGestureHandlerSignature::TMethodPtr< UserClass > Func )
	{
		FInputGestureBinding GB(GestureKey);
		GB.GestureDelegate.BindDelegate(Object, Func);
		GestureBindings.Emplace(MoveTemp(GB));
		return GestureBindings.Last();
	}

private:
	
	/**
	 * Checks if the given action/axis binding name is deprecated. If it is, then log a warning.
	 * 
	 * @param BindingName The name of the 
	 */
	ENGINE_API static void LogDeprecatedBindingWarning(const FName BindingName);
	
	/** Retrieves the actions bound to the input component which are triggered by a given key. Requires that the internal key map has already been built. */
	ENGINE_API void GetActionsBoundToKey(UPlayerInput* PlayerInput, FKey Key, TArray<TSharedPtr<FInputActionBinding>>& Actions) const;

	friend struct FGetActionsBoundToKey;

	/** Returns true if the given key/button is pressed on the input of the controller (if present) */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.IsInputKeyDown instead."))
	ENGINE_API bool IsControllerKeyDown(FKey Key) const;

	/** Returns true if the given key/button was up last frame and down this frame. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.WasInputKeyJustPressed instead."))
	ENGINE_API bool WasControllerKeyJustPressed(FKey Key) const;

	/** Returns true if the given key/button was down last frame and up this frame. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.WasInputKeyJustReleased instead."))
	ENGINE_API bool WasControllerKeyJustReleased(FKey Key) const;

	/** Returns the analog value for the given key/button.  If analog isn't supported, returns 1 for down and 0 for up. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputAnalogKeyState instead."))
	ENGINE_API float GetControllerAnalogKeyState(FKey Key) const;

	/** Returns the vector value for the given key/button. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputVectorKeyState instead."))
	ENGINE_API FVector GetControllerVectorKeyState(FKey Key) const;

	/** Returns the location of a touch, and if it's held down */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputTouchState instead."))
	ENGINE_API void GetTouchState(int32 FingerIndex, float& LocationX, float& LocationY, bool& bIsCurrentlyPressed) const;

	/** Returns how long the given key/button has been down.  Returns 0 if it's up or it just went down this frame. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputKeyTimeDown instead."))
	ENGINE_API float GetControllerKeyTimeDown(FKey Key) const;

	/** Retrieves how far the mouse moved this frame. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputMouseDelta instead."))
	ENGINE_API void GetControllerMouseDelta(float& DeltaX, float& DeltaY) const;

	/** Retrieves the X and Y displacement of the given analog stick.  For WhickStick, 0 = left, 1 = right. */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage="Use PlayerController.GetInputAnalogStickState instead."))
	ENGINE_API void GetControllerAnalogStickState(EControllerAnalogStick::Type WhichStick, float& StickX, float& StickY) const;

	friend class UEnhancedInputComponent;	// TEMP: Support for ongoing input rework
};

struct FGetActionsBoundToKey
{
private:
	static void Get(UInputComponent* InputComponent, UPlayerInput* PlayerInput, FKey Key, TArray<TSharedPtr<FInputActionBinding>>& Actions)
	{
		InputComponent->GetActionsBoundToKey(PlayerInput, Key, Actions);
	}

	friend UPlayerInput;
};
