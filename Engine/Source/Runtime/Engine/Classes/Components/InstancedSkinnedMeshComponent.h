// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/NaniteInterface.h"
#include "Animation/AnimBank.h"
#include "InstanceDataSceneProxy.h"
#include "InstanceData/InstanceDataManager.h"
#include "InstancedSkinnedMeshSceneProxyDesc.h"
#include "Matrix3x4.h"
#include "SkinningSceneExtensionProxy.h"
#include "InstancedSkinnedMeshComponent.generated.h"

class FPrimitiveSceneProxy;
class USkinnedMeshComponent;
class UMaterialInterface;
class UTexture;
class UAnimBank;
struct FAnimBankRecordHandle;
struct FPrimitiveSceneDesc;
struct FBoneAttachmentSocket;
class FInstancedMeshComponentBodies;

USTRUCT()
struct FSkinnedMeshInstanceData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Instances)
	FTransform3f Transform;

	UPROPERTY(EditAnywhere, Category = Animation)
	uint32 AnimationIndex;

	FSkinnedMeshInstanceData()
	: Transform(FTransform3f::Identity)
	, AnimationIndex(0)
	{
	}

	FSkinnedMeshInstanceData(const FTransform3f& InTransform, uint32 InAnimationIndex)
	: Transform(InTransform)
	, AnimationIndex(InAnimationIndex)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FSkinnedMeshInstanceData& InstanceData)
	{
		Ar << InstanceData.Transform;
		Ar << InstanceData.AnimationIndex;
		return Ar;
	}
};

UCLASS(ClassGroup=Rendering, hidecategories=(Object,Activation,"Components|Activation"), ShowCategories=(Mobility), editinlinenew, meta=(BlueprintSpawnableComponent), Blueprintable, MinimalAPI)
class UInstancedSkinnedMeshComponent : public USkinnedMeshComponent
{
	GENERATED_UCLASS_BODY()
	
protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Animation)
	TObjectPtr<class UTransformProviderData> TransformProvider;

	/** Array of instances, bulk serialized. */
	UPROPERTY(EditAnywhere, SkipSerialization, DisplayName="Instances", Category=Instances, meta=(MakeEditWidget=true, EditFixedOrder))
	TArray<FSkinnedMeshInstanceData> InstanceData;

	/** Defines the number of floats that will be available per instance for custom data */
	UPROPERTY(EditAnywhere, Category=Instances, AdvancedDisplay)
	int32 NumCustomDataFloats = 0;

	/** Array of custom data for instances. This will contains NumCustomDataFloats*InstanceCount entries. The entries are represented sequantially, in instance order. Can be read in a material and manipulated through Blueprints.
	*	Example: If NumCustomDataFloats is 1, then each entry will belong to an instance. Custom data 0 will belong to Instance 0. Custom data 1 will belong to Instance 1 etc.
	*	Example: If NumCustomDataFloats is 2, then each pair of sequential entries belong to an instance. Custom data 0 and 1 will belong to Instance 0. Custom data 2 and 3 will belong to Instance 2 etc.
	*/
	UPROPERTY(EditAnywhere, EditFixedSize, SkipSerialization, DisplayName="Custom Data", Category=Instances, AdvancedDisplay, meta=(EditFixedOrder))
	TArray<float> InstanceCustomData;

	/** Array of previous transforms. Must match the length of InstanceData. */
	UPROPERTY(Transient)
	TArray<FTransform3f> PerInstancePrevTransform;

	/** Screen space footprint (in primary view) cutoff which dictates the far distance the instance will play back animation. 
	 *  Using 0.0 (the default) falls back to a global threshold, a negative value disables the cutoff.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Animation)
	float AnimationMinScreenSize = 0.0f;

	/** Distance from camera at which each instance begins to draw. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	int32 InstanceMinDrawDistance;

	/** Distance from camera at which each instance begins to fade out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	int32 InstanceStartCullDistance;

	/** Distance from camera at which each instance completely fades out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	int32 InstanceEndCullDistance;

	/** If true, this component will avoid serializing its per instance data / those properties will also not be editable */
	UPROPERTY()
	uint8 bInheritPerInstanceData : 1;

	UPROPERTY()
	FBox PrimitiveBoundsOverride;

	UPROPERTY()
	bool bIsInstanceDataGPUOnly = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Instances)
	bool bHasPreviousTransforms = false;

	/** Forces all socket bones to be animated on the GPU so that they are always available for attachment, regardless
     *  of whether the mesh vertices are influenced by them. When false, only bones that are actively influencing vertices
     *  are included in the transform set and can be attached to.
     */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Animation)
	bool bForceAnimateSockets = false;

	/** If true, this component will use GPU LOD selection. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	uint8 bUseGpuLodSelection : 1;


	UPROPERTY()
	int32 NumInstancesGPUOnly = 0;

public:

	ENGINE_API UInstancedSkinnedMeshComponent(FVTableHelper& Helper);
	virtual ~UInstancedSkinnedMeshComponent();

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) override;
#endif
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
	virtual void OnRegister() override;
	virtual void SendRenderInstanceData_Concurrent() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual bool IsHLODRelevant() const override;
#if WITH_EDITOR
	virtual void ComputeHLODHash(class FHLODHashBuilder& HashBuilder) const override;
#endif
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState_Concurrent() override;

	virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent Interface

	//~ Begin USceneComponent Interface
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
	//~ End USceneComponent Interface

	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual UBodySetup* GetBodySetup() override;
	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	ENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	ENGINE_API virtual void RecreateInstanceBody(int32 InstanceBodyIndex) override;
	ENGINE_API virtual bool CanEditSimulatePhysics() override;
	ENGINE_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	ENGINE_API virtual bool SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex) override;
	ENGINE_API virtual bool OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const override;
	ENGINE_API virtual bool ComponentOverlapComponentImpl(UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Quat, const FCollisionQueryParams& Params) override;
	ENGINE_API virtual bool ComponentOverlapMultiImpl(TArray<FOverlapResult>& OutOverlaps, const UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const FComponentQueryParams& Params, const FCollisionObjectQueryParams& ObjectQueryParams) const override;
	ENGINE_API virtual void OnActorEnableCollisionChanged() override;
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	ENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
protected:
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override;
	//~ End UPrimitiveComponent Interface

public:
	//~ Begin USkinnedMeshComponent Interface
	virtual const Nanite::FResources* GetNaniteResources() const;
	virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = nullptr) override;
#if WITH_EDITOR
	/**
	 * Called by the asset compilers (FAnimBankCompilingManager / FSkinnedAssetCompilingManager) when done to allow render state to be created.
	 */
	virtual void PreAssetCompilation();
	virtual void PostAssetCompilation();

#endif
	//~ End USkinnedMeshComponent Interface

public:
	void SetNumGPUInstances(int32 InCount);

	bool UsesGPUOnlyInstances() const { return bIsInstanceDataGPUOnly; }

	FBox GetPrimitiveBoundsOverride() const { return PrimitiveBoundsOverride; }
	void SetPrimitiveBoundsOverride(const FBox& InBounds) { PrimitiveBoundsOverride = InBounds; }

	/** Add an instance to this component. Transform is given in local space of this component unless bWorldSpace is set. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API FPrimitiveInstanceId AddInstance(const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace = false);

	/** Add multiple instances to this component. Transform is given in local space of this component unless bWorldSpace is set. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API TArray<FPrimitiveInstanceId> AddInstances(const TArray<FTransform>& Transforms, const TArray<int32>& AnimationIndices, bool bShouldReturnIds, bool bWorldSpace = false);

	/** Update custom data for specific instance */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool SetCustomDataValue(FPrimitiveInstanceId InstanceId, int32 CustomDataIndex, float CustomDataValue);

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	inline bool SetCustomData(FPrimitiveInstanceId InstanceId, const TArray<float>& CustomDataFloats)
	{
		return SetCustomData(InstanceId, TConstArrayView<float>(CustomDataFloats));
	}

	/** Update all custom data values for specific instance, the size of the array view must match the NumCustomDataFloats. Returns True on success. */
	ENGINE_API bool SetCustomData(FPrimitiveInstanceId InstanceId, TConstArrayView<float> CustomDataFloats);

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	inline bool GetCustomData(FPrimitiveInstanceId InstanceId, TArray<float>& CustomDataFloats) const
	{
		return GetCustomData(InstanceId, TArrayView<float>(CustomDataFloats));
	}

	/** Get all custom data values for specific instance, the size of the array view must match the NumCustomDataFloats. Returns True on success. */
	ENGINE_API bool GetCustomData(FPrimitiveInstanceId InstanceId, TArrayView<float> CustomDataFloats) const;

	/** Update number of custom data entries per instance. This applies to all instances and will reallocate the full custom data buffer and reset all values to 0 */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void SetNumCustomDataFloats(int32 InNumCustomDataFloats);

	/** Get the transform for the instance specified. Instance is returned in local space of this component unless bWorldSpace is set. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool GetInstanceTransform(FPrimitiveInstanceId InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const;
	
	/** Get the previous transform for the instance specified. Instance is returned in local space of this component unless bWorldSpace is set. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool GetInstancePrevTransform(FPrimitiveInstanceId InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const;

	/** Update the transform for the instance specified. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API bool SetInstanceTransform(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, bool bWorldSpace=false);

	/** Update the previous transform for the instance specified. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API bool SetInstancePrevTransform(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, bool bWorldSpace=false);
	
	/** Update both the current and previous transforms for the instance specified. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API bool SetInstanceTransforms(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, const FTransform& InstancePrevTransform, bool bWorldSpace=false);

	/** Enables or disables per-instance previous transforms for motion blur. Use SetInstancePrevTransform or SetInstanceTransforms to provide previous transforms. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void SetHasPerInstancePrevTransforms(bool bInHasPreviousTransforms);

	/** Get the animation index for the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool GetInstanceAnimationIndex(FPrimitiveInstanceId InstanceId, int32& OutAnimationIndex) const;

	/** Set the animation index for the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool SetInstanceAnimationIndex(FPrimitiveInstanceId InstanceId, int32 OutAnimationIndex);

	/** Updates an instance on this component. Transform is given in local space of this component unless bWorldSpace is set. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool UpdateInstance(FPrimitiveInstanceId InstanceId, const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace = false);

	/** Remove the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API bool RemoveInstance(FPrimitiveInstanceId InstanceId);

	/** Remove the instances specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void RemoveInstances(const TArray<FPrimitiveInstanceId>& InstancesToRemove);

	/** Clear all instances being rendered by this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void ClearInstances();

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API class UTransformProviderData* GetTransformProvider() const;

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void SetTransformProvider(class UTransformProviderData* InTransformProvider);

	/**
	 * Optimize the instance data by performing sorting according to spatial hash on the _source_ data.
	 * Note that this reorders the instances and thus any indexing will change. By default resets the ID mapping to identity.
	 *  @param bShouldRetainIdMap	If true, the id mapping is updated instead of reset to identity, this retains the validity of the IDs but adds some memory and storage cost (for the ID mapping).
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void OptimizeInstanceData(bool bShouldRetainIdMap = false);

	ENGINE_API bool IsEnabled() const;

	/** Attach an instance to a parent instance's bone at a socket or bone. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void SetInstanceBoneAttachment(
		FPrimitiveInstanceId InstanceId,
		FPrimitiveInstanceId ParentInstanceId,
		UInstancedSkinnedMeshComponent* Parent,
		FName SocketName);

	/** Detach an instance from bone attachment. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void ClearInstanceBoneAttachment(FPrimitiveInstanceId InstanceId);

	/** Returns true if this component has any bone attachment sockets. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	bool HasBoneAttachmentParents() const { return BoneAttachmentSockets.Num() > 0; }

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API int32 GetInstanceCount() const;

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API FPrimitiveInstanceId GetInstanceId(int32 InstanceIndex) const;

	UFUNCTION(BlueprintCallable, Category = "Components|InstancedSkinnedMesh")
	ENGINE_API void SetAnimationMinScreenSize(float InAnimationMinScreenSize);

	int32 GetNumCustomDataFloats() const { return NumCustomDataFloats; }
	const TArray<float>& GetInstanceCustomData() const { return InstanceCustomData; }
	const TArray<FSkinnedMeshInstanceData>& GetInstanceData() const { return InstanceData; }

	TConstArrayView<FTransform3f> GetInstancePrevTransforms() const
	{
		return PerInstancePrevTransform;
	}

	int32 GetInstanceCountGPUOnly() const { return NumInstancesGPUOnly; }

	int32 GetMinDrawDistance() const { return InstanceMinDrawDistance; }
	void GetCullDistances(int32& OutStartCullDistance, int32& OutEndCullDistance) const { OutStartCullDistance = InstanceStartCullDistance; OutEndCullDistance = InstanceEndCullDistance; }
	ENGINE_API void SetCullDistances(int32 StartCullDistance, int32 EndCullDistance);
protected:

	/** Handle changes that must happen before the proxy is recreated. */
	void PreApplyComponentInstanceData(struct FInstancedSkinnedMeshComponentInstanceData* ComponentInstanceData);

	/** Applies the cached component instance data to a newly blueprint constructed component. */
	void ApplyComponentInstanceData(struct FInstancedSkinnedMeshComponentInstanceData* ComponentInstanceData);

	FInstanceDataManagerSourceDataDesc GetComponentDesc(EShaderPlatform InShaderPlatform);

#if WITH_EDITOR
	/** One bit per instance if the instance is selected. */
	TBitArray<> SelectedInstances;
#endif

	/** Don't create any collision when this bool is set */
	UPROPERTY()
	uint8 bDisableCollision : 1;

	/** Per-instance physics bodies manager. */
	TUniquePtr<FInstancedMeshComponentBodies> InstancePhysicsBodies;

	bool bIsInstanceDataApplyCompleted = true;

	void CreateHitProxyData(TArray<TRefCountPtr<HHitProxy>>& HitProxies);

	/** 
	 */
	FPrimitiveInstanceId AddInstanceInternal(int32 InstanceIndex, const FTransform& InstanceTransform, int32 AnimationIndex, bool bWorldSpace);

	/** 
	 */
	bool RemoveInstanceInternal(int32 InstanceIndex, bool InstanceAlreadyRemoved);

private:
	void ApplyInheritedPerInstanceData(const UInstancedSkinnedMeshComponent* InArchetype);
	bool ShouldInheritPerInstanceData(const UInstancedSkinnedMeshComponent* InArchetype) const;
	bool ShouldInheritPerInstanceData() const;

	void SetInstanceDataGPUOnly(bool bInInstancesGPUOnly);

	/** Sets up new instance data to sensible defaults, creates physics counterparts if possible. */
	void SetupNewInstanceData(FSkinnedMeshInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform3f& InInstanceTransform, int32 AnimationIndex);

	/** Create all physics body instances for all instances. */
	void CreateAllInstanceBodies();
	/** Destroy all physics body instances. */
	void ClearAllInstanceBodies();
	/** Update a single instance's physics body transform. */
	void UpdateInstanceBodyTransform(int32 InstanceIndex, const FTransform& WorldSpaceInstanceTransform, bool bTeleport);

	/** Build a merged UBodySetup that aggregates collision from all bones in the physics asset. */
	void BuildMergedBodySetup();

	/** Cached merged body setup combining collision from all physics asset bones. */
	UPROPERTY(Transient)
	TObjectPtr<UBodySetup> MergedBodySetup;

	static ENGINE_API bool ShouldForceRefPose();
	static ENGINE_API bool ShouldUseAnimationBounds();

	static ENGINE_API FSkeletalMeshObject* CreateMeshObject(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, TObjectPtr<UTransformProviderData> InTransformProvider, FSkeletalMeshRenderData* InRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	static ENGINE_API FPrimitiveSceneProxy* CreateSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin, bool bShouldNaniteSkin, bool bIsEnabled, int32 MinLODIndex);

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> GetOrCreateInstanceDataSceneProxy();

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> GetInstanceDataSceneProxy() const;

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> CreateInstanceDataProxyGPUOnly() const;

	FInstanceDataManager InstanceDataManager;

	struct FBoneAttachmentSocketRef
	{
		TWeakObjectPtr<UInstancedSkinnedMeshComponent> Parent;
		FName SocketName;
		int32 RefCount = 0;
		bool bNeedsResolve = false;

		/** Resolved socket data, populated at acquire time or lazily on first flush. */
		FBoneAttachmentSocket Resolved;
	};

	TSparseArray<FBoneAttachmentSocketRef> BoneAttachmentSockets;
	TArray<FBoneAttachmentBinding> BoneAttachmentBindings;
	TBitArray<> DirtyBoneAttachmentBindings;
	int32 NumDirtyBoneAttachmentBindings = 0;

	void FlushBoneAttachmentSockets();

	int32 AcquireBoneAttachmentSocket(UInstancedSkinnedMeshComponent* Parent, FName SocketName, bool bDeferResolve = false);
	bool ResolveBoneAttachmentSocket(FBoneAttachmentSocketRef& Socket);
	void ReleaseBoneAttachmentSocket(int32 SocketIndex);

	void MarkBoneAttachmentBindingDirty(int32 InstanceIndex);
	void ResetBoneAttachmentDirtyState();

	void SaveBoneAttachments(FArchive& Ar);
	void LoadBoneAttachments(FArchive& Ar);

	TArray<FBoneAttachmentSocket> GetBoneAttachmentSockets();

	friend class FInstancedSkinnedMeshSceneProxy;
	friend class FInstancedSkinnedMeshComponentHelper;
	friend struct FInstancedSkinnedMeshComponentInstanceData;
	friend struct FInstancedSkinnedMeshSceneProxyDesc;
	friend struct FSkinnedMeshComponentDescriptorBase;

private:
	void SetSkinnedAssetCallback();

public:
	UE_DEPRECATED(5.7, "Method removed.")
	void BuildSceneDesc(FPrimitiveSceneProxyDesc* InSceneProxyDesc, FPrimitiveSceneDesc& OutPrimitiveSceneDesc) {}
};

/** Helper class used to preserve state across blueprint re-instancing */
USTRUCT()
struct FInstancedSkinnedMeshComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()

public:
	FInstancedSkinnedMeshComponentInstanceData() = default;
	
	FInstancedSkinnedMeshComponentInstanceData(const UInstancedSkinnedMeshComponent* InComponent)
		: FSceneComponentInstanceData(InComponent)
		, SkinnedAsset(InComponent->GetSkinnedAsset())
		, PrimitiveBoundsOverride(InComponent->GetPrimitiveBoundsOverride())
		, bIsInstanceDataGPUOnly(InComponent->UsesGPUOnlyInstances())
		, NumInstancesGPUOnly(InComponent->GetInstanceCountGPUOnly())
	{
	}

	virtual ~FInstancedSkinnedMeshComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		// The Super::ApplyToComponent will cause the scene proxy to be recreated, so we must do what we can to make sure the state is ok before that.
		CastChecked<UInstancedSkinnedMeshComponent>(Component)->PreApplyComponentInstanceData(this);
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<UInstancedSkinnedMeshComponent>(Component)->ApplyComponentInstanceData(this);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Super::AddReferencedObjects(Collector);
		Collector.AddReferencedObject(SkinnedAsset);
	}

public:
	UPROPERTY()
	TObjectPtr<USkinnedAsset> SkinnedAsset = nullptr;

	UPROPERTY()
	TArray<FSkinnedMeshInstanceData> InstanceData;

	TBitArray<> SelectedInstances;

	UPROPERTY()
	bool bHasPerInstanceHitProxies = false;

	UPROPERTY()
	FBox PrimitiveBoundsOverride;

	UPROPERTY()
	bool bIsInstanceDataGPUOnly = false;

	UPROPERTY()
	int32 NumInstancesGPUOnly = 0;
};
