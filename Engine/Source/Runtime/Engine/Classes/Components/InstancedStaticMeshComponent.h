// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "HitProxies.h"
#include "Misc/Guid.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/StaticMeshComponent.h"
#include "Elements/SMInstance/SMInstanceManager.h"
#include "InstanceDataSceneProxy.h"
#include "InstancedStaticMesh/ISMInstanceDataManager.h"
#include "StaticMeshResources.h"
#include "InstancedStaticMeshComponent.generated.h"

class FLightingBuildOptions;
class FPrimitiveSceneProxy;
class FStaticLightingBuildContext;
class FStaticLightingTextureMapping_InstancedStaticMesh;
class ULightComponent;
struct FNavigableGeometryExport;
struct FNavigationRelevantData;
struct FPerInstanceRenderData;
struct FStaticLightingPrimitiveInfo;

DECLARE_STATS_GROUP(TEXT("Foliage"), STATGROUP_Foliage, STATCAT_Advanced);

class FStaticLightingTextureMapping_InstancedStaticMesh;
class FInstancedLightMap2D;
class FInstancedShadowMap2D;
class FStaticMeshInstanceData;
class FISMInstanceUpdateChangeSet;
struct FInstanceUpdateComponentDesc;
class FInstancedMeshComponentBodies;

USTRUCT()
struct FInstancedStaticMeshInstanceData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Instances)
	FMatrix Transform;

	FInstancedStaticMeshInstanceData()
		: Transform(FMatrix::Identity)
	{
	}

	FInstancedStaticMeshInstanceData(const FMatrix& InTransform)
		: Transform(InTransform)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FInstancedStaticMeshInstanceData& InstanceData)
	{
		// @warning BulkSerialize: FInstancedStaticMeshInstanceData is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << InstanceData.Transform;
		return Ar;
	}
};

USTRUCT()
struct FInstancedStaticMeshMappingInfo
{
	GENERATED_USTRUCT_BODY()

	FStaticLightingTextureMapping_InstancedStaticMesh* Mapping;

	FInstancedStaticMeshMappingInfo()
		: Mapping(nullptr)
	{
	}
};

USTRUCT()
struct FInstancedStaticMeshRandomSeed
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 StartInstanceIndex = 0;

	UPROPERTY()
	int32 RandomSeed = 0;

	friend FArchive& operator<<(FArchive& Ar, FInstancedStaticMeshRandomSeed& InstanceData)
	{
		Ar << InstanceData.StartInstanceIndex;
		Ar << InstanceData.RandomSeed;
		return Ar;
	}
};

USTRUCT()
struct FBoundsCacheElement
{
	GENERATED_USTRUCT_BODY()

	FBoundsCacheElement()
		: bIsValid(false)
		, Hash(0)
		, Value(ForceInit)
	{
	}

	void Invalidate()
	{
		bIsValid = false;
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	void CacheValue(const FBoxSphereBounds& InValue, uint32 InHash = 0)
	{
		bIsValid = true;
		Value = InValue;
		Hash = InHash;
	}

	const FBoxSphereBounds& GetValue(bool bCheckValid = true) const
	{
		ensure(!bCheckValid || bIsValid);
		return Value;
	}

	bool operator == (uint32 InHash) const
	{
		return bIsValid && (Hash == InHash);
	}

	bool operator != (uint32 InHash) const
	{
		return !(*this == InHash);
	}

	static uint32 GetHash(const FTransform& InBoundTransform, const FBox& InInstanceBounds);

private:
	UPROPERTY()
	bool bIsValid;

	UPROPERTY()
	uint32 Hash;

	UPROPERTY()
	FBoxSphereBounds Value;
};

/** A component that efficiently renders multiple instances of the same StaticMesh. */
UCLASS(ClassGroup = Rendering, meta = (BlueprintSpawnableComponent), Blueprintable, MinimalAPI)
class UInstancedStaticMeshComponent : public UStaticMeshComponent, public ISMInstanceManager
{
	GENERATED_UCLASS_BODY()

	/** Needs implementation in InstancedStaticMesh.cpp to compile UniquePtr for forward declared class */
	ENGINE_API UInstancedStaticMeshComponent(FVTableHelper& Helper);
	ENGINE_API virtual ~UInstancedStaticMeshComponent();
	
	/** Array of instances, bulk serialized. */
	UPROPERTY(EditAnywhere, SkipSerialization, DisplayName="Instances", Category=Instances, meta=(MakeEditWidget=true, EditFixedOrder))
	TArray<FInstancedStaticMeshInstanceData> PerInstanceSMData;
	
	// TODO: KevinO cleanup
	/** Array of prev instance transforms. Must match the length of PerInstanceSMData or have 0 elements */
	UPROPERTY(Transient)
	TArray<FMatrix> PerInstancePrevTransform;

	/** Main transform stored to be able to send updates when component's transform changed. */
	UPROPERTY(Transient)
	FTransform PreviousComponentTransform;

	/** Defines the number of floats that will be available per instance for custom data */
	UPROPERTY(EditAnywhere, Category=Instances, AdvancedDisplay)
	int32 NumCustomDataFloats;

	/** Value used to seed the random number stream that generates random numbers for each of this mesh's instances.
	The random number is stored in a buffer accessible to materials through the PerInstanceRandom expression. If
	this is set to zero (default), it will be populated automatically by the editor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InstancedStaticMeshComponent)
	int32 InstancingRandomSeed=0;

	/** Array of custom data for instances. This will contains NumCustomDataFloats*InstanceCount entries. The entries are represented sequantially, in instance order. Can be read in a material and manipulated through Blueprints.
	 *	Example: If NumCustomDataFloats is 1, then each entry will belong to an instance. Custom data 0 will belong to Instance 0. Custom data 1 will belong to Instance 1 etc.
	 *	Example: If NumCustomDataFloats is 2, then each pair of sequential entries belong to an instance. Custom data 0 and 1 will belong to Instance 0. Custom data 2 and 3 will belong to Instance 2 etc.
	 */
	UPROPERTY(EditAnywhere, EditFixedSize, SkipSerialization, DisplayName="Custom data", Category=Instances, AdvancedDisplay, meta=(EditFixedOrder))
	TArray<float> PerInstanceSMCustomData;

	/** Additional random seeds ranges. Each seed entry will be applied from AdditionalRandomSeeds[i].StartInstanceIndex to AdditionalRandomSeeds[i+1].StartInstanceIndex -1 */
	UPROPERTY()
	TArray<FInstancedStaticMeshRandomSeed> AdditionalRandomSeeds;

	/** 
	 * Scale applied to change the computation of LOD distances when using the StaticMesh screen sizes. 
	 * Smaller values make LODs transition earlier.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	float InstanceLODDistanceScale = 1.f;

	/** Distance from camera at which each instance begins to draw. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Culling)
	int32 InstanceMinDrawDistance;

	/** Distance from camera at which each instance begins to fade out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Culling)
	int32 InstanceStartCullDistance;

	/** Distance from camera at which each instance completely fades out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Culling)
	int32 InstanceEndCullDistance;

	/** If true, this component will use GPU LOD selection. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	uint8 bUseGpuLodSelection : 1;

	/** If true, this component will avoid serializing its per instance data / those properties will also not be editable */
	UPROPERTY()
	uint8 bInheritPerInstanceData : 1;

	/** Don't create any collision when this bool is set */
	UPROPERTY()
	uint8 bDisableCollision : 1;

	/** 
	* Flag for using RemoveAtSwap on instance removal. 
	* The implementation is free to ignore this flag, but should honor whatever behavior is being returned by SupportsRemoveSwap().
	*/
	uint8 bSupportRemoveAtSwap : 1;

protected:
	uint8 bIsInstanceDataApplyCompleted : 1;

private:
	uint8 bHasPreviousTransforms : 1;

	/** Flag for whether we are using conservative bounds. */
	uint8 bUseConservativeBounds : 1;

	/** Internally used to know when CalcBounds is called by UpdateBounds. */
	uint8 bIsUpdatingBounds : 1;

#if WITH_EDITORONLY_DATA
	/** Set in OnAsyncCreatePhysicsStateBegin_GameThread to signal that BodyInstance.WalkableSlopeOverride
	 *  was pre-cached from BodySetup on GT, so the worker-side FInstancedMeshComponentBodies::CreateAll
	 *  must skip its own (potentially racy) read of BodySetup->WalkableSlopeOverride during PIE.
	 *  Cleared in OnAsyncCreatePhysicsStateEnd_GameThread. In cooked, BodySetup is immutable
	 *  post-PostLoad and the worker read is uncontested, so this flag is unneeded. */
	uint8 bWalkableSlopeOverrideCachedGT : 1;
#endif

public:
	/** Mapping from PerInstanceSMData order to instance render buffer order. If empty, the PerInstanceSMData order is used. */
	UPROPERTY()
	TArray<int32> InstanceReorderTable;
	
	/** Tracks outstanding proxysize, as this is a bit hard to do with the fire-and-forget grass. */
	SIZE_T ProxySize;

	/** Returns the render instance buffer index. */
	inline int32 GetRenderIndex(int32 InInstanceIndex) const { return InstanceReorderTable.IsValidIndex(InInstanceIndex) ? InstanceReorderTable[InInstanceIndex] : InInstanceIndex; }

	/** Add an instance to this component. Transform is given in local space of this component unless bWorldSpace is set. */
	UFUNCTION(BlueprintCallable, Category="Components|InstancedStaticMesh")
	ENGINE_API virtual int32 AddInstance(const FTransform& InstanceTransform, bool bWorldSpace = false);

	/** Add multiple instances to this component. Transform is given in local space of this component unless bWorldSpace is set. */
	UFUNCTION(BlueprintCallable, Category="Components|InstancedStaticMesh")
	ENGINE_API virtual TArray<int32> AddInstances(const TArray<FTransform>& InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace = false, bool bUpdateNavigation = true);

	/** Add an instance to this component. Transform is given in world space. */
	UE_DEPRECATED(5.0, "Use AddInstance or AddInstances with bWorldSpace set to true.")
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh", meta=(DeprecatedFunction, DeprecationMessage="Use 'Add Instance' or 'Add Instances' with 'World Space' set to true."))
	int32 AddInstanceWorldSpace(const FTransform& WorldTransform)
	{
		return AddInstance(WorldTransform, /*bWorldSpace*/true);
	}

	int32 GetNumInstances() const { return PerInstanceSMData.Num(); }

	/**
	 * Preliminary ID-based interface. May only be used if no other manipulations are performed that cause invalidation of IDs. For example, cannot be used on HISM.
	 * ISMs edited in the editor can also not reliably be used, as some editor changes cause ID tracking to be lost.
	 */

	/**
	 */
	ENGINE_API TArray<FPrimitiveInstanceId> AddInstancesById(const TArrayView<const FTransform>& InstanceTransforms, bool bWorldSpace = false, bool bUpdateNavigation = true);
	ENGINE_API FPrimitiveInstanceId AddInstanceById(const FTransform& InstanceTransforms, bool bWorldSpace = false);
	/**
	 */
	ENGINE_API void SetCustomDataById(const TArrayView<const FPrimitiveInstanceId> &InstanceIds, TArrayView<const float> CustomDataFloats); 
	inline void SetCustomDataById(FPrimitiveInstanceId InstanceId, TArrayView<const float> CustomDataFloats) { SetCustomDataById(MakeArrayView(&InstanceId, 1), CustomDataFloats); }
	ENGINE_API void SetCustomDataValueById(FPrimitiveInstanceId InstanceId, int32 CustomDataIndex, float CustomDataValue);

	/**
	 */
	ENGINE_API virtual void RemoveInstancesById(const TArrayView<const FPrimitiveInstanceId> &InstanceIds, bool bUpdateNavigation = true);
	inline void RemoveInstanceById(FPrimitiveInstanceId InstanceId) { RemoveInstancesById(MakeArrayView(&InstanceId, 1)); }
	/**
	 */
	ENGINE_API void UpdateInstanceTransformById(FPrimitiveInstanceId InstanceId, const FTransform& NewInstanceTransform, bool bWorldSpace=false, bool bTeleport=false);
	/**
	 */
	ENGINE_API void SetPreviousTransformById(FPrimitiveInstanceId InstanceId, const FTransform& NewPrevInstanceTransform, bool bWorldSpace=false);
	/**
	 */
	ENGINE_API bool IsValidId(FPrimitiveInstanceId InstanceId);
	
	/** Fetches current instance index for a given InstanceId */
	inline int32 GetInstanceIndexForId(FPrimitiveInstanceId InstanceId) const { return PrimitiveInstanceDataManager.IdToIndex(InstanceId); }

	ENGINE_API void SetHasPerInstancePrevTransforms(bool bInHasPreviousTransforms);

	/** Update custom data for specific instance */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool SetCustomDataValue(int32 InstanceIndex, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty = false);

	/** Update number of custom data entries per instance. This applies to all instances and will reallocate the full custom data buffer and reset all values to 0 */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual void SetNumCustomDataFloats(int32 InNumCustomDataFloats);

	/** Per Instance Custom Data */
	ENGINE_API virtual bool SetCustomData(int32 InstanceIndex, TArrayView<const float> CustomDataFloats, bool bMarkRenderStateDirty = false);
	ENGINE_API virtual bool SetCustomData(int32 InstanceIndexStart, int32 InstanceIndexEnd, TConstArrayView<float> CustomDataFloats, bool bMarkRenderStateDirty = false);

	/** Preallocated memory to include the new added instances count, to prevent reallloc during the add operation. */
	ENGINE_API virtual void PreAllocateInstancesMemory(int32 AddedInstanceCount);

	/** Get the transform for the instance specified. Instance is returned in local space of this component unless bWorldSpace is set.  Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API bool GetInstanceTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace = false) const;
	
	/** Gets the current LOD scale. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	float GetLODDistanceScale() const { return InstanceLODDistanceScale; }

	/** Sets the LOD scale. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API void SetLODDistanceScale(float InLODDistanceScale);

	// TODO: KevinO cleanup
	/** Get the prev transform for the instance specified. Only works if PerInstancePrevTransform has been setup and updated through BatchUpdateInstancesTransforms */
	ENGINE_API bool GetInstancePrevTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace = false) const;

	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
	ENGINE_API void UpdateComponentTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	/** Get the scale comming form the component, when computing StreamingTexture data. Used to support instanced meshes. */
	ENGINE_API virtual float GetTextureStreamingTransformScale() const override;
	/** Get material, UV density and bounds for a given material index. */
	ENGINE_API virtual bool GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const override;
	/** Build the data to compute accuracte StreaminTexture data. */
	ENGINE_API virtual bool BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData) override;
	/** Get the StreaminTexture data. */
	ENGINE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;

	/**
	* Update the transform for the instance specified.
	*
	* @param InstanceIndex			The index of the instance to update
	* @param NewInstanceTransform	The new transform
	* @param bWorldSpace			If true, the new transform is interpreted as a World Space transform, otherwise it is interpreted as Local Space
	* @param bMarkRenderStateDirty	If true, the change should be visible immediately. If you are updating many instances you should only set this to true for the last instance.
	* @param bTeleport				Whether or not the instance's physics should be moved normally, or teleported (moved instantly, ignoring velocity).
	* @return						True on success.
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool UpdateInstanceTransform(int32 InstanceIndex, const FTransform& NewInstanceTransform, bool bWorldSpace=false, bool bMarkRenderStateDirty=false, bool bTeleport=false);

    /**
	* Update the transform for an array of instances.
	*
	* @param StartInstanceIndex		The starting index of the instances to update
	* @param NewInstancesTransforms	The new transforms
	* @param bWorldSpace			If true, the new transforms are interpreted as a World Space transform, otherwise it is interpreted as Local Space
	* @param bMarkRenderStateDirty	If true, the change should be visible immediately. If you are updating many instances you should only set this to true for the last instance.
	* @param bTeleport				Whether or not the instances physics should be moved normally, or teleported (moved instantly, ignoring velocity).
	* @return						True on success.
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace=false, bool bMarkRenderStateDirty=false, bool bTeleport=false);

	/** this is array view version of the UFUNCTION, blueprints do not support ArrayViews at the time of adding this one  */
	ENGINE_API virtual bool BatchUpdateInstancesTransforms(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	// TODO: KevinO cleanup
	/**
	* Update the transform for an array of instances. Overloaded version which takes an array of NewPreviousFrameTransforms.
	*/
	ENGINE_API virtual bool BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, const TArray<FTransform>& NewInstancesPrevTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	/**
	* Update the transform for a number of instances.
	*
	* @param StartInstanceIndex		The starting index of the instances to update
	* @param NumInstances			The number of instances to update
	* @param NewInstancesTransform	The new transform
	* @param bWorldSpace			If true, the new transform is interpreted as a World Space transform, otherwise it is interpreted as Local Space
	* @param bMarkRenderStateDirty	If true, the change should be visible immediately. If you are updating many instances you should only set this to true for the last instance.
	* @param bTeleport				Whether or not the instances physics should be moved normally, or teleported (moved instantly, ignoring velocity).
	* @return						True on success.
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool BatchUpdateInstancesTransform(int32 StartInstanceIndex, int32 NumInstances, const FTransform& NewInstancesTransform, bool bWorldSpace=false, bool bMarkRenderStateDirty=false, bool bTeleport=false);

	ENGINE_API virtual bool BatchUpdateInstancesData(int32 StartInstanceIndex, int32 NumInstances, FInstancedStaticMeshInstanceData* StartInstanceData, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	/** Remove the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool RemoveInstance(int32 InstanceIndex);

	/** Remove the instances specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual bool RemoveInstances(const TArray<int32>& InstancesToRemove);

	/** Remove the instances specified. Returns True on success.
	* @param InstancesToRemove array of the instance indices to remove ( can be sorted @see bInstanceArrayAlreadySortedInReverseOrder )
	* @param bInstanceArrayAlreadySortedInReverseOrder true is the array is already sorted in reverse order
	*/
	ENGINE_API virtual bool RemoveInstances(const TArray<int32>& InstancesToRemove, bool bInstanceArrayAlreadySortedInReverseOrder);

	/** Clear all instances being rendered by this component. */
	UFUNCTION(BlueprintCallable, Category="Components|InstancedStaticMesh")
	ENGINE_API virtual void ClearInstances();

	/** Get the number of instances in this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API int32 GetInstanceCount() const;

	/** Does the given index map to a valid instance in this component? */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API bool IsValidInstance(int32 InstanceIndex) const;

	/** Sets the fading start and culling end distances for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API void SetCullDistances(int32 StartCullDistance, int32 EndCullDistance);

	/** Gets the fading start and culling end distances for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	void GetCullDistances(int32& OutStartCullDistance, int32& OutEndCullDistance) const { OutStartCullDistance = InstanceStartCullDistance; OutEndCullDistance = InstanceEndCullDistance; }

	/** Returns the instances with instance bounds overlapping the specified sphere. The return value is an array of instance indices. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual TArray<int32> GetInstancesOverlappingSphere(const FVector& Center, float Radius, bool bSphereInWorldSpace=true) const;

	/** Returns the instances with instance bounds overlapping the specified box. The return value is an array of instance indices. */
	UFUNCTION(BlueprintCallable, Category = "Components|InstancedStaticMesh")
	ENGINE_API virtual TArray<int32> GetInstancesOverlappingBox(const FBox& Box, bool bBoxInWorldSpace=true) const;

	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;

	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void OnActorEnableCollisionChanged() override;
	
	/** Sets to use RemoveAtSwap on instance removal. This is an optimization, but will change the resultant instance reordering. */
	void SetRemoveSwap() { bSupportRemoveAtSwap = true; }
	/** Returns true if RemoveAtSwap is enabled. Derived classes may always return true regardless of whether SetRemoveSwapEnabled() was called. */
	ENGINE_API virtual bool SupportsRemoveSwap() const;

	/** Iterates on instance transform (FMatrix) */
	void ForEachInstanceMatrix(TFunctionRef<void(const FMatrix&)> Func) const;

	/** 
	 * Sets whether to use conservative bounds. 
	 * This doesn't fully recalculate bounds on instance addition/removal/transform change.
	 * Instead maintains a conservative bound that only grows.
	 */
	void SetUseConservativeBounds(bool bValue) { bUseConservativeBounds = bValue; CachedConservativeInstanceBounds.Init(); }

#if WITH_EDITOR
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;

	ENGINE_API virtual bool IsInstanceTouchingSelectionBox(int32 InstanceIndex, const FBox& InBox, const bool bMustEncompassEntireInstance) const;
	ENGINE_API virtual bool IsInstanceTouchingSelectionFrustum(int32 InstanceIndex, const FConvexVolume& InFrustum, const bool bMustEncompassEntireInstance) const;

	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	ENGINE_API virtual void ComputeHLODHash(class FHLODHashBuilder& HashBuilder) const override;
#endif

	// Helper function to construct a base-set of instance data flags that in
	ENGINE_API FInstanceDataFlags MakeInstanceDataFlags(bool bAnyMaterialHasPerInstanceRandom, bool bAnyMaterialHasPerInstanceCustomData) const;

private:
	/** Current cached conservativ bounds. */
	FBox CachedConservativeInstanceBounds;

public:

#if WITH_EDITOR
	/** One bit per instance if the instance is selected. */
	TBitArray<> SelectedInstances;

	/** Indicates that the user has purposedly chosen to show the instance list in the details panel, despite the performance warning. */
	bool bForceShowAllInstancesDetails = false;

	/** The reason why a deletion operation is currently happening. */
	enum class EInstanceDeletionReason : uint8
	{
		NotDeleting, /** There is currently no deletion in progress. */
		EntryAlreadyRemoved, /** The instance has been deleted externally. Data synchronization in progress. */
		EntryRemoval, /** The instance is being removed. */
		Clearing /** All instances are being removed. */
	};
	/** This will be set to the appropriate state when one or more instances are in the process of being
	 *  deleted. This is primarily used for functions that round trip to this class, such as callbacks
	 *  for deselecting instances. */
	EInstanceDeletionReason DeletionState = EInstanceDeletionReason::NotDeleting;
#endif

	/** Returns the per-instance physics bodies array. */
	ENGINE_API const TArray<FBodyInstance*>& GetInstanceBodies() const;

	UE_DEPRECATED(5.8, "This member is no longer used. Use GetInstanceBodies() instead.")
	TArray<FBodyInstance*> InstanceBodies;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API virtual void GetComponentChildElements(TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) override;
	ENGINE_API virtual bool IsHLODRelevant() const override;
	ENGINE_API virtual void SendRenderInstanceData_Concurrent() override;
	//~ End UActorComponent Interface

	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual FMatrix GetRenderMatrix() const override;
	ENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	ENGINE_API virtual void RecreateInstanceBody(int32 InstanceBodyIndex) override;
protected:
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
#if WITH_EDITOR
	ENGINE_API virtual void OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
#endif
	ENGINE_API virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	ENGINE_API virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;

public:
	ENGINE_API virtual bool CanEditSimulatePhysics() override;

	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& BoundTransform) const override;
	ENGINE_API virtual void UpdateBounds() override;

	virtual bool SupportsStaticLighting() const override { return true; }
#if WITH_EDITOR
	ENGINE_API virtual void GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options) override;
	ENGINE_API virtual FBox GetStreamingBounds() const override;
#endif
	ENGINE_API virtual void GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const override;

	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	ENGINE_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	ENGINE_API virtual bool SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapRotation, const FCollisionShape& CollisionShape, bool bTraceComplex = false) override;
	ENGINE_API virtual bool OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const override;
protected:
	ENGINE_API virtual bool ComponentOverlapComponentImpl(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Quat, const FCollisionQueryParams& Params) override;
	ENGINE_API virtual bool ComponentOverlapMultiImpl(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const override;
	//~ End UPrimitiveComponent Interface

	//~ Begin IPhysicsComponent Interface.
public:
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	ENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	//~ End IPhysicsComponent Interface.

	//~ Begin UStaticMeshComponentInterface
protected:
	ENGINE_API virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
	//~ End UStaticMeshComponentInterface

	//~ Begin UNavRelevantInterface Interface
public:
	ENGINE_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	ENGINE_API virtual FBox GetNavigationBounds() const override;
	ENGINE_API virtual bool IsNavigationRelevant() const override;
	ENGINE_API virtual bool ShouldSkipDirtyAreaOnAddOrRemove() const override;
	//~ End UNavRelevantInterface Interface

	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	ENGINE_API virtual void PostStaticMeshCompilation() override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditUndo() override;
	/**
	 * See: UObject::BeginCacheForCookedPlatformData
	 */
	ENGINE_API virtual void BeginCacheForCookedPlatformData( const ITargetPlatform* TargetPlatform ) override;
	
	/**
	 * See: UObject::IsCachedCookedPlatformDataLoaded
	 */
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) override;

#endif
	//~ End UObject Interface

	/**
	 * Get the translated space for instance transforms to be passed to the renderer.
	 * In the renderer data structures we only have floating point precision for instance transforms relative to their owning primitive. The primitive transform itself has double precision.
	 * Some ISM that are authored in world space need to adjust the local space to keep instance transforms within precision limits.
	 */
	virtual FVector GetTranslatedInstanceSpaceOrigin() const { return FVector::Zero(); }

	/** Handle changes that must happen before the proxy is recreated. */
	ENGINE_API void PreApplyComponentInstanceData(struct FInstancedStaticMeshComponentInstanceData* ComponentInstanceData);

	/** Applies the cached component instance data to a newly blueprint constructed component. */
	ENGINE_API virtual void ApplyComponentInstanceData(struct FInstancedStaticMeshComponentInstanceData* ComponentInstanceData);

	/** Check to see if an instance is selected. */
	ENGINE_API bool IsInstanceSelected(int32 InInstanceIndex) const;

	/** Select/deselect an instance or group of instances. */
	ENGINE_API void SelectInstance(bool bInSelected, int32 InInstanceIndex, int32 InInstanceCount = 1);

	/** Deselect all instances. */
	ENGINE_API void ClearInstanceSelection();

	/** Precache all PSOs which can be used by the component */
	ENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	
	// Number of instances in the render-side instance buffer
	virtual int32 GetNumRenderInstances() const { return PerInstanceSMData.Num(); }

	ENGINE_API virtual void PropagateLightingScenarioChange() override;

	ENGINE_API void GetInstancesMinMaxScale(FVector& MinScale, FVector& MaxScale) const;


	UE_DEPRECATED(5.4, "This function has been added only for the purposes of moving LWI code outside of the engine. Don't use it, it will be removed soon.")
	void OnPostPopulatePerInstanceData() { OnPostLoadPerInstanceData(); }

	/** Request to navigation system to update only part of navmesh occupied by specified instance. */
	ENGINE_API virtual void PartialNavigationUpdate(int32 InstanceIdx) final;

	/** Request to navigation system to update only part of navmesh occupied specified instances transforms. */
	ENGINE_API virtual void PartialNavigationUpdates(TConstArrayView<FTransform> InstanceTransforms) final;

	ENGINE_API TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe> GetOrCreateInstanceDataSceneProxy();
	ENGINE_API const TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe>& GetInstanceDataSceneProxy() const;

	/**
	 * Mark the "shadowmap" or lightmap uv as modified for the instance since this is stored in external data.
	 */
	ENGINE_API void SetBakedLightingDataChanged(int32 InInstanceIndex);

	/**
	 * Mark the "shadowmap" or lightmap uv as modified for all the instances since this is stored in external data.
	 */
	ENGINE_API void SetBakedLightingDataChangedAll();

	/** 
	 * Clears all the updated instance tracking data AND instance ID association, also forcing a full update of the instance data the next time it is flushed.	 
	 * NOTE: Destroying the instance updated tracking means the renderer has to treat the instances as completely new, preventing e.g., velocity tracking and caching from working reliably.
	 * This function is only intended to be used when some outside entity has modified the instance data, this is not recommended and access to the public data will be removed in a future release.
	 */
	ENGINE_API void InvalidateInstanceDataTracking();

	// Deprecated in 5.7
	inline FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(EShaderPlatform ShaderPlatform) const
	{
		return UPrimitiveComponent::GetUsedMaterialPropertyDesc(ShaderPlatform);
	}

	/** wrapper which is public so we can implement the LIST ISM command */
	UE_DEPRECATED(5.7, "Please use GetUsedMaterialPropertyDesc with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	inline FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(ERHIFeatureLevel::Type FeatureLevel) const
	{
		return GetUsedMaterialPropertyDesc(GetFeatureLevelShaderPlatform(FeatureLevel));
	}

private:
	ENGINE_API void ApplyInheritedPerInstanceData(const UInstancedStaticMeshComponent* InArchetype);
	ENGINE_API bool ShouldInheritPerInstanceData(const UInstancedStaticMeshComponent* InArchetype) const;
	ENGINE_API bool ShouldInheritPerInstanceData() const;

	void PartialNavigateUpdateForCurrentInstances();

	/** Sets up new instance data to sensible defaults, creates physics counterparts if possible. */
	ENGINE_API void SetupNewInstanceData(FInstancedStaticMeshInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform& InInstanceTransform);

	/** Update instance body with a new transform */
	ENGINE_API void UpdateInstanceBodyTransform(int32 InstanceIndex, const FTransform& WorldSpaceInstanceTransform, bool bTeleport);

	/** hidden Implementation of BatchUpdateInstancesTransforms - it is shared by the TArray and TArrayView version of public API */
	ENGINE_API bool BatchUpdateInstancesTransformsInternal(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport);

protected:

	/** Number of pending lightmaps still to be calculated (Apply()'d). */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	int32 NumPendingLightmaps;

	FPrimitiveInstanceDataManager PrimitiveInstanceDataManager;
	
	/** Per-instance physics bodies manager. */
	TUniquePtr<FInstancedMeshComponentBodies> InstancePhysicsBodies;

	void InvalidateCachedBounds();

	UE_DEPRECATED(5.8, "Navigation bounds are now lazily updated and no longer require explicit calls to update them.")
	ENGINE_API void CalcAndCacheNavigationBounds();

	/** Creates body instances for all instances owned by this component. */
	ENGINE_API void CreateAllInstanceBodies();

	/** Terminate all body instances owned by this component. */
	ENGINE_API void ClearAllInstanceBodies();

	/** Request to navigation system to update for the bounds of the ISM. */
	ENGINE_API virtual void FullNavigationUpdate();

	enum class EBoundsCacheType : uint8
	{
		Component,
		Navigation,
		Local,
		Count,
		Other
	};

	UE_DEPRECATED(5.5, "Use version with FBox")
	FBoxSphereBounds CalcBoundsImpl(const FTransform& BoundTransform, bool bForNavigation) const;

	/**
	 * Calculates bounds from all instances.
	 * @param BoundTransform The bounds transform
	 * @param InstanceBounds The instance bounds
	 */
	FBoxSphereBounds CalcBoundsImpl(const FTransform& BoundTransform, const FBox& InstanceBounds) const;
	
	/** Does this component support partial navigation updates */
	virtual bool SupportsPartialNavigationUpdate() const { return true; }

	/** Internal version of AddInstance */
	ENGINE_API int32 AddInstanceInternal(int32 InstanceIndex, FInstancedStaticMeshInstanceData* InNewInstanceData, const FTransform& InstanceTransform, bool bWorldSpace);

	/** Internal implementation of AddInstances */
	ENGINE_API TArray<int32> AddInstancesInternal(TConstArrayView<FTransform> InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace, bool bUpdateNavigation = true);

	/** Internal version of RemoveInstance */	
	ENGINE_API bool RemoveInstanceInternal(int32 InstanceIndex, bool InstanceAlreadyRemoved, bool bForceRemoveAtSwap = false, bool bUpdateNavigation = true);

	/**
	 * Returns the bounds of a single instance in local space. It uses the NavCollision if available,
	 * then the StaticMesh bounds if available or an invalid box otherwise.
	 * @return The bounds of a single instance in local space 
	 */
	FBox GetInstanceNavigationBounds() const;
	
	/** Handles request from navigation system to gather instance transforms in a specific area box. */
	ENGINE_API virtual void GetNavigationPerInstanceTransforms(const FBox& AreaBox, TArray<FTransform>& InstanceData) const;

	/** Initializes the body instance for the specified instance of the static mesh. */
	UE_DEPRECATED(5.8, "Use InstancePhysicsBodies instead.")
	ENGINE_API void InitInstanceBody(int32 InstanceIdx, FBodyInstance* InBodyInstance);
	UE_DEPRECATED(5.8, "Use InstancePhysicsBodies instead.")
	ENGINE_API void InitInstanceBody(int32 InstanceIdx, FBodyInstance* InstanceBodyInstance, FBodyInstance* ReferenceBodyInstance, bool bRuntimeBodyInstance, UBodySetup* BodySetup);

	/** The mappings for all the instances of this component. */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<FInstancedStaticMeshMappingInfo> CachedMappings;

	ENGINE_API void ApplyLightMapping(FStaticLightingTextureMapping_InstancedStaticMesh* InMapping, const FStaticLightingBuildContext* LightingContext);
	
	ENGINE_API void CreateHitProxyData(TArray<TRefCountPtr<HHitProxy>>& HitProxies);

    /** Build instance buffer for rendering from current component data. Only used for cook */
	ENGINE_API void BuildLegacyRenderData(FStaticMeshInstanceData& OutData);
    /** Serialize instance buffer that is used for rendering. Only for cooked content */
	ENGINE_API void SerializeRenderData(FArchive& Ar);
	
	/** Creates rendering buffer from serialized data, if any */
	ENGINE_API virtual void OnPostLoadPerInstanceData();

	// Helper to collect the base delta data from the ISM *notably not transforms*
	ENGINE_API void BuildInstanceDataDeltaChangeSetCommon(FISMInstanceUpdateChangeSet &ChangeSet);

	UE_DEPRECATED(5.7, "Please use BuildComponentInstanceData with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual void BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData);
	ENGINE_API virtual void BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData);

	//~ ISMInstanceManager interface
	ENGINE_API virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const override;
	ENGINE_API virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const override;
	ENGINE_API virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	ENGINE_API virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;
	ENGINE_API virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) override;
	ENGINE_API virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) override;
	ENGINE_API virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) override;

	void InitializeInstancingRandomSeed(const bool bAllowRandomSeedValue = false);

	friend FStaticLightingTextureMapping_InstancedStaticMesh;
	friend FInstancedLightMap2D;
	friend FInstancedShadowMap2D;

#if STATS
	/** Used for dynamic stats */
	TStatId StatId;
#endif

private:

	FBoxSphereBounds GetOrCalcBounds(const FTransform& InBoundTransform, EBoundsCacheType InType) const;

	UPROPERTY()
	mutable FBoundsCacheElement CachedBounds[(int)EBoundsCacheType::Count];
};

/** InstancedStaticMeshInstance hit proxy */
struct HInstancedStaticMeshInstance : public HHitProxy
{
	TObjectPtr<UInstancedStaticMeshComponent> Component;
	int32 InstanceIndex;

	DECLARE_HIT_PROXY(ENGINE_API);
	HInstancedStaticMeshInstance(UInstancedStaticMeshComponent* InComponent, int32 InInstanceIndex) : HHitProxy(HPP_World), Component(InComponent), InstanceIndex(InInstanceIndex) {}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FTypedElementHandle GetElementHandle() const override;
	virtual EMouseCursor::Type GetMouseCursor() override;
};

/** Used to store lightmap data during RerunConstructionScripts */
USTRUCT()
struct FInstancedStaticMeshLightMapInstanceData
{
	GENERATED_BODY()

	/** Transform of component */
	UPROPERTY()
	FTransform Transform;

	/** guid from LODData */
	UPROPERTY()
	TArray<FGuid> MapBuildDataIds;
};

/** Helper class used to preserve lighting/selection state across blueprint reinstancing */
USTRUCT()
struct FInstancedStaticMeshComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FInstancedStaticMeshComponentInstanceData() = default;
	FInstancedStaticMeshComponentInstanceData(const UInstancedStaticMeshComponent* InComponent)
		: FSceneComponentInstanceData(InComponent)
		, StaticMesh(InComponent->GetStaticMesh())
	{}
	virtual ~FInstancedStaticMeshComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		// The Super::ApplyToComponent will cause the scene proxy to be recreated, so we must do what we can to make sure the state is ok before that.
		CastChecked<UInstancedStaticMeshComponent>(Component)->PreApplyComponentInstanceData(this);
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<UInstancedStaticMeshComponent>(Component)->ApplyComponentInstanceData(this);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Super::AddReferencedObjects(Collector);
		Collector.AddReferencedObject(StaticMesh);
	}

public:
	/** Mesh being used by component */
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;

	// Static lighting info
	UPROPERTY()
	FInstancedStaticMeshLightMapInstanceData CachedStaticLighting;
	UPROPERTY()
	TArray<FInstancedStaticMeshInstanceData> PerInstanceSMData;

	/** The cached selected instances */
	TBitArray<> SelectedInstances;

	/* The cached random seed */
	UPROPERTY()
	int32 InstancingRandomSeed = 0;

	/* Additional random seeds */
	UPROPERTY()
	TArray<FInstancedStaticMeshRandomSeed> AdditionalRandomSeeds;

	UPROPERTY()
	bool bHasPerInstanceHitProxies = false;
};
