// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/MovementComponent.h"
#include "UObject/ObjectMacros.h"
#include "InterpToMovementComponent.generated.h"

struct FHitResult;

/**
 * Controls the movement behaviour
 */
UENUM(BlueprintType)
enum class EInterpToBehaviourType : uint8
{
	/** Move to destination and stop */
	OneShot,
	/** Move to destination, return and stop */
	OneShot_Reverse,
	/** Reset back to start when reaching the end */
	Loop_Reset,
	/** Repeatedly lerp from start to end and back */
	PingPong
};

USTRUCT(BlueprintType)
struct FInterpControlPoint
{
	GENERATED_USTRUCT_BODY()
public:
	// The position of the point 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location, meta=(MakeEditWidget=true))
	FVector PositionControlPoint;
	
	// Is the location relative to the root components initial location
 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
 	bool bPositionIsRelative;

	FInterpControlPoint()
	{
		PositionControlPoint = FVector::ZeroVector;
		bPositionIsRelative = true;
	}
	FInterpControlPoint(FVector InPosition, bool bIsRelative)
	{
		PositionControlPoint = InPosition;
		bPositionIsRelative = bIsRelative;
	}

	float StartTime;
	float DistanceToNext;
	float Percentage;
};

/**
 * Move the root component between a series of points over a given time * 
 *
 * @see UMovementComponent
 */
UCLASS(ClassGroup=Movement, meta=(BlueprintSpawnableComponent),HideCategories=Velocity, MinimalAPI)
class UInterpToMovementComponent : public UMovementComponent 
{
	GENERATED_UCLASS_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnInterpToReverseDelegate, const FHitResult&, ImpactResult, float, Time  );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnInterpToStopDelegate, const FHitResult&, ImpactResult, float, Time );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnInterpToWaitBeginDelegate , const FHitResult&, ImpactResult, float, Time );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnInterpToWaitEndDelegate , const FHitResult&, ImpactResult, float, Time );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnInterpToResetDelegate , const FHitResult&, ImpactResult, float, Time );

	/* How long to take to move from the first point to the last (or vice versa) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Control,meta=(UIMin=0.1f, ClampMin=0.1f))
	float Duration;
	
	/** If true, will pause movement on impact. If false it will behave as if the end of the movement range was reached based on the BehaviourType. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Behaviour)
	uint32 bPauseOnImpact:1;

	/** If true, will sweep for blocking collision during movement. If false, it will simply teleport to the next position and ignore collision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Behaviour)
	bool bSweep = true;

	/** Physics teleport type. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Behaviour)
	ETeleportType TeleportType = ETeleportType::None;

	/* Movement behaviour of the component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Behaviour)
	EInterpToBehaviourType BehaviourType;

	/* Do we want this comp to perform CheckStillInWorld checks? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Behaviour)
	bool bCheckIfStillInWorld = true;

	/**
	 * If true, forces sub-stepping to break up movement into discrete smaller steps to improve accuracy of the trajectory.
	 * Objects that move in a straight line typically do *not* need to set this, as movement always uses continuous collision detection (sweeps) so collision is not missed.
	 * Sub-stepping is automatically enabled when under the effects of gravity or when homing towards a target.
	 * @see MaxSimulationTimeStep, MaxSimulationIterations
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Simulation)
	uint32 bForceSubStepping:1;
	
	/** Called when InterpTo impacts something and reverse is enabled. */
	UPROPERTY(BlueprintAssignable)
	FOnInterpToReverseDelegate OnInterpToReverse;

	/** Called when InterpTo has come to a stop. */
	UPROPERTY(BlueprintAssignable)
	FOnInterpToStopDelegate OnInterpToStop;
	
	/** Called when InterpTo has come to a stop but will resume when possible. */
	UPROPERTY(BlueprintAssignable)
	FOnInterpToWaitBeginDelegate OnWaitBeginDelegate;

	/** Called when InterpTo has resumed following a stop. */
	UPROPERTY(BlueprintAssignable)
	FOnInterpToWaitEndDelegate OnWaitEndDelegate;

	/** Called when InterpTo reached the end and reset back to start . */
	UPROPERTY(BlueprintAssignable)
	FOnInterpToResetDelegate OnResetDelegate; 

	//Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//End UActorComponent Interface

	//Begin UMovementComponent Interface
	ENGINE_API virtual void StopMovementImmediately() override;
	//End UMovementComponent Interface

	/**
	 * This will check to see if the InterpTo is still in the world.  It will check things like
	 * the KillZ, outside world bounds, etc. and handle the situation.
	 */
	ENGINE_API virtual bool CheckStillInWorld();

	/** Clears the reference to UpdatedComponent, fires stop event, and stops ticking. */
	UFUNCTION(BlueprintCallable, Category="Game|Components|InterpToMovement")
	ENGINE_API void StopSimulating(const FHitResult& HitResult);

	bool HasStoppedSimulation() { return UpdatedComponent == NULL; }

	/**
	 * Compute remaining time step given remaining time and current iterations.
	 * The last iteration (limited by MaxSimulationIterations) always returns the remaining time, which may violate MaxSimulationTimeStep.
	 *
	 * @param RemainingTime		Remaining time in the tick.
	 * @param Iterations		Current iteration of the tick (starting at 1).
	 * @return The remaining time step to use for the next sub-step of iteration.
	 * @see MaxSimulationTimeStep, MaxSimulationIterations
	 * @see ShouldUseSubStepping()
	 */
	ENGINE_API float GetSimulationTimeStep(float RemainingTime, int32 Iterations) const;

	/**
	 * Determine whether or not to use substepping in the InterpTo motion update.
	 * If true, GetSimulationTimeStep() will be used to time-slice the update. If false, all remaining time will be used during the tick.
	 * @return Whether or not to use substepping in the InterpTo motion update.
	 * @see GetSimulationTimeStep()
	 */
	ENGINE_API virtual bool ShouldUseSubStepping() const;

	/**
	 * Max time delta for each discrete simulation step.
	 * Lowering this value can address issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationIterations, bForceSubStepping
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0.0166", ClampMax="0.50", UIMin="0.0166", UIMax="0.50"), Category=Simulation)
	float MaxSimulationTimeStep;

	/**
	 * Change the speed of movement.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Simulation)
	float SpeedMultiplier;

	/**
	 * Max number of iterations used for each discrete simulation step.
	 * Increasing this value can address issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationTimeStep, bForceSubStepping
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(ClampMin="1", ClampMax="25", UIMin="1", UIMax="25"), Category=Simulation)
	int32 MaxSimulationIterations;
	
	/* List of control points to visit. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Control)
	TArray<FInterpControlPoint> ControlPoints;
		
	/* Add a control point that represents a position. */
	UFUNCTION(BlueprintCallable, Category = "Control")
	ENGINE_API virtual void AddControlPointPosition(FVector Pos, bool bPositionIsRelative = true);
	
	/* Reset to start. Sets time to zero and direction to 1.  */
	UFUNCTION(BlueprintCallable, Category = "Control")
	ENGINE_API void RestartMovement(float InitialDirection = 1.0f);

	/* Initialise the control points array. Call after adding control points if they are add after begin play .  */
	UFUNCTION(BlueprintCallable, Category = "Control")
	ENGINE_API void FinaliseControlPoints();

	/* Clear the control points array and set to stopped. */
	UFUNCTION(BlueprintCallable, Category = "Control")
	ENGINE_API void ResetControlPoints();

#if WITH_EDITOR
	//~ Begin UObject Interface.
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface.
#endif // WITH_EDITOR

protected:

	/** @return true if the simulation should stop. */
	ENGINE_API bool HandleHitWall(const FHitResult& Hit, float TimeTick, const FVector& MoveDelta);

	/** Deal with an impact. Change direction, stop etc depending on the current behaviour setting. */
	ENGINE_API virtual void HandleImpact(const FHitResult& Hit, float TimeSlice=0.f, const FVector& MoveDelta = FVector::ZeroVector) override;

	/* Reverse direction we are moving */
	ENGINE_API void ReverseDirection(const FHitResult& Hit, float Time, bool InBroadcastEvent);

	/** Compute the distance for the given time. */
	ENGINE_API virtual FVector ComputeMoveDelta(float Time) const;

	/* Current position on spline */
	float CurrentTime;

	/* User to */
	float TimeMultiplier;

	/* Current direction along spline timeline */
	float CurrentDirection;

	/* Have we hit something and are waiting for it to move to carry on moving */
	bool bIsWaiting;

	/* Have we stopped (because we hit something, or reached the end of the cycle */
	bool bStopped;

	/* Are there any actor control points in use*/
	bool bContainsActorControlPoints;

	/** Minimum delta time considered when ticking. Delta times below this are not considered. This is a very small non-zero positive value to avoid potential divide-by-zero in simulation code. */
	static ENGINE_API const float MIN_TICK_TIME;
	
	/* Update the control points. Adjusts the positions of there are any actor control points as well as updating the spline type */
	ENGINE_API virtual void UpdateControlPoints(bool InForceUpdate);	

	/* Calculate the new current time */
	ENGINE_API float CalculateNewTime( float TimeNow, float Delta, FHitResult& HitResult, bool InBroadcastEvent, bool& OutStopped, float& OutTimeRemainder );

private:
	float TotalDistance;

	FVector StartLocation;
	bool bPointsFinalized;
};



