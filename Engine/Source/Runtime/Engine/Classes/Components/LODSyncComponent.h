// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Components/ActorComponent.h"
#include "Framework/Commands/InputChord.h"
#include "LODSyncComponent.generated.h"

USTRUCT(BlueprintType)
struct FLODMappingData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FLODMappingData)
	TArray<int32> Mapping;

	UPROPERTY(transient)
	TArray<int32> InverseMapping;
};

UENUM(BlueprintType)
enum class ESyncOption : uint8
{
	/** Drive LOD from this component. It will contribute to the change of LOD */
	Drive,
	/** It follows what's currently driven by other components. It doesn't contribute to the change of LOD*/
	Passive,
	/** It is disabled, it doesn't do anything */
	Disabled, 
};

USTRUCT(BlueprintType)
struct FComponentSync
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FComponentSync)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FComponentSync)
	ESyncOption SyncOption;

	FComponentSync()
	: SyncOption(ESyncOption::Drive)
	{}

	FComponentSync(const FName& InName, ESyncOption InSyncOption)
		: Name (InName)
		, SyncOption(InSyncOption)
	{}
};
/**
 * Implement an Actor component for LOD Sync of different components
 *
 * This is a component that allows multiple different components to sync together of their LOD
 *
 * This allows to find the highest LOD of all the parts, and sync to that LOD
 */
UCLASS(Blueprintable, ClassGroup = Component, BlueprintType, meta = (BlueprintSpawnableComponent), MinimalAPI)
class ULODSyncComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	// if -1, it's default and it will calculate the max number of LODs from all sub components
	// if not, it is a number of LODs (not the max index of LODs)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD, meta=(ScriptName="NumLods;NumLODs"))
	int32 NumLODs = -1;

	// if -1, it's automatically switching
	UPROPERTY(EditAnywhere, BlueprintReadWrite, interp, Category = LOD)
	int32 ForcedLOD = -1;

	// Minimum LOD to use when syncing components
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	int32 MinLOD = 0;

	/** 
	 *	Array of components whose LOD may drive or be driven by this component.
	 *  Components that are flagged as 'Drive' are treated as being in priority order, with the last component having highest priority. The highest priority
	 *  visible component will set the LOD for all other components. If no components are visible, then the highest priority non-visible component will set LOD.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	TArray<FComponentSync> ComponentsToSync;

	// by default, the mapping will be one to one
	// but if you want custom, add here. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	TMap<FName, FLODMappingData> CustomLODMapping;

	/** Returns a string detailing  */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API FString GetLODSyncDebugText() const;

private:
	UPROPERTY(transient)
	int32 CurrentLOD = 0;

	// num of LODs
	UPROPERTY(transient)
	int32 CurrentNumLODs = 0;

	// component that drives the LOD
	UPROPERTY(transient)
	TArray<TObjectPtr<UPrimitiveComponent>> DriveComponents;

	// all the components that ticks
	UPROPERTY(transient)
	TArray<TObjectPtr<UPrimitiveComponent>> SubComponents;

	// BEGIN AActorComponent interface
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// END AActorComponent interface

public: 
	ENGINE_API void RefreshSyncComponents();
	
	/**
	 * Set the LOD of each synced component.
	 *
	 * This is called from TickComponent, so there's no need to call it manually. It's exposed here
	 * for testing purposes.
	 */
	ENGINE_API void UpdateLOD();

private:
	int32 GetCustomMappingLOD(const FName& ComponentName, int32 CurrentWorkingLOD) const;
	int32 GetSyncMappingLOD(const FName& ComponentName, int32 CurrentSourceLOD) const;
	void InitializeSyncComponents();
	void UninitializeSyncComponents();
	const FComponentSync* GetComponentSync(const FName& InName) const;
};

