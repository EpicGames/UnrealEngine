// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "RenderCommandFence.h"
#include "EngineDefines.h"
#include "SceneTypes.h"
#include "RenderResource.h"
#include "Components/LightComponentBase.h"
#include "LightSceneProxyDesc.h"
#include "PSOPrecacheFwd.h" 
#include "PSOPrecacheComponent.h"
#include "LightComponent.generated.h"

class FLightSceneProxy;
class FLightComponentMapBuildData;
class FStaticLightingBuildContext;
class FStaticShadowDepthMapData;
class ULevel;
class UMaterialInterface;
class UPrimitiveComponent;
class UTextureLightProfile;
enum class ELightUnits : uint8;

using FUpdateLightProxyCallback = TFunction<void(FLightSceneProxy*)>;

/** 
 * A texture containing depth values of static objects that was computed during the lighting build.
 * Used by Stationary lights to shadow translucency.
 */
class FStaticShadowDepthMap : public FTexture
{
public:

	FStaticShadowDepthMap() : 
		Data(NULL)
	{}

	const FStaticShadowDepthMapData* Data;

	virtual void InitRHI(FRHICommandListBase& RHICmdList);
};

UCLASS(abstract, HideCategories=(Trigger,Activation,"Components|Activation",Physics), ShowCategories=(Mobility), MinimalAPI)
class ULightComponent : public ULightComponentBase
{
	GENERATED_UCLASS_BODY()

	/**
	* Color temperature in Kelvin of the blackbody illuminant.
	* White (D65) is 6500K.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = Light, meta = (UIMin = "1700.0", UIMax = "12000.0", ShouldShowInViewport = true, DisplayAfter ="bUseTemperature"))
	float Temperature;
	
	/** World distance to where the light stops rendering (lighting & shadows). 0.0 or less means no max draw distance. This can be controlled via a global scalar scalability setting r.LightMaxDrawDistanceScale as well */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Performance)
	float MaxDrawDistance;

	/** Fade the light out as it approaches MaxDrawDistance. The fading starts when the light is MaxDistanceFadeRange distance away from MaxDrawDistance, where it ends up at 0 for a smooth fade-out without popping. Requires MaxDrawDistance to be set. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Performance, meta = (EditCondition = "MaxDrawDistance > 0.0"))
	float MaxDistanceFadeRange;

	/** false: use white (D65) as illuminant. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta=(DisplayName = "Use Temperature", ShouldShowInViewport = true))
	uint32 bUseTemperature : 1;

	/** 
	 * Legacy shadowmap channel from the lighting build, now stored in FLightComponentMapBuildData.
	 */
	UPROPERTY()
	int32 ShadowMapChannel_DEPRECATED;

	/** Transient shadowmap channel used to preview the results of stationary light shadowmap packing. */
	UPROPERTY(NonPIEDuplicateTransient)
	int32 PreviewShadowMapChannel;
	
	/** Min roughness effective for this light. Used for softening specular highlights. */
	UPROPERTY()
	float MinRoughness_DEPRECATED;

	/** 
	 * Multiplier on specular highlights. Use only with great care! Any value besides 1 is not physical!
	 * Can be used to artistically remove highlights mimicking polarizing filters or photo touch up.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = "0", UIMax = "1"))
	float SpecularScale;

	/** 
	* Multiplier on diffuse lighting. Use only with great care! Any value besides 1 is not physical!
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = "0", UIMax = "1"))
	float DiffuseScale;

	/** 
	 * Scales the resolution of shadowmaps used to shadow this light.  By default shadowmap resolution is chosen based on screen size of the caster. 
	 * Setting the scale to zero disables shadow maps, but does not disable, e.g., contact shadows.
	 * Note: shadowmap resolution is still clamped by 'r.Shadow.MaxResolution'
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = ".125", UIMax = "8"))
	float ShadowResolutionScale;

	/** 
	 * Controls how accurate self shadowing of whole scene shadows from this light are.  
	 * At 0, shadows will start at the their caster surface, but there will be many self shadowing artifacts.
	 * larger values, shadows will start further from their caster, and there won't be self shadowing artifacts but object might appear to fly.
	 * around 0.5 seems to be a good tradeoff. This also affects the soft transition of shadows
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = "0", UIMax = "1"))
	float ShadowBias;

	/**
	 * Controls how accurate self shadowing of whole scene shadows from this light are. This works in addition to shadow bias, by increasing the 
	 * amount of bias depending on the slope of a surface.
	 * At 0, shadows will start at the their caster surface, but there will be many self shadowing artifacts.
	 * larger values, shadows will start further from their caster, and there won't be self shadowing artifacts but object might appear to fly.
	 * around 0.5 seems to be a good tradeoff. This also affects the soft transition of shadows
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float ShadowSlopeBias;

	/** Amount to sharpen shadow filtering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "1.0", DisplayName = "Shadow Filter Sharpen"))
	float ShadowSharpen;
	
	/** Length of screen space ray trace for sharp contact shadows. Zero is disabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Light, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "0.1"))
	float ContactShadowLength;

	/** Where Length of screen space ray trace for sharp contact shadows is in world space units or in screen space units. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Light, AdvancedDisplay, meta = (DisplayName = "Contact Shadow Length In World Space Units"))
	uint32 ContactShadowLengthInWS : 1;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" enabled. 0 = no shadow, 1 (default) = fully shadowed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Light, AdvancedDisplay, meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = "0.0", UIMax = "1.0"))
	float ContactShadowCastingIntensity;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" disabled. 0 (default) = no shadow, 1 = fully shadowed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Light, AdvancedDisplay, meta = (ClampMin = 0.0, ClampMax = 1.0))
	float ContactShadowNonCastingIntensity;

	UPROPERTY()
	uint32 InverseSquaredFalloff_DEPRECATED:1;

	/** Whether the light is allowed to cast dynamic shadows from translucency. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 CastTranslucentShadows:1;

	/** 
	 * Whether the light should only cast shadows from components marked as bCastCinematicShadows. 
	 * This is useful for setting up cinematic Movable spotlights aimed at characters and avoiding the shadow depth rendering costs of the background.
	 * Note: this only works with dynamic shadow maps, not with static shadowing or Ray Traced Distance Field shadows.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 bCastShadowsFromCinematicObjectsOnly:1;

	/**
	* Enables cached shadows for movable primitives for this light even if r.shadow.cachedshadowscastfrommovableprimitives is 0
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay)
	uint32 bForceCachedShadowsForMovablePrimitives : 1;

	/**
	 * Whether to allow this light to use MegaLights, if it is enabled in the project settings or Post Process Volume.
	 * When disabled, the renderer will no longer use stochastic sampling to solve this light's lighting, and will fall back to other shadowing methods, adding significant GPU cost.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (DisplayName = "Allow MegaLights"), AdvancedDisplay)
	uint32 bAllowMegaLights : 1;

	/**
	* Selects which shadowing method should MegaLights use for this light.
	* RayTracing - Preferred method, which guarantees fixed MegaLights cost and correct area shadows, but is dependent on the BVH representation quality.
	* VirtualShadowMap - Has a significant per light cost, but can cast shadows directly from the Nanite geometry using rasterization.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (DisplayName = "MegaLights Shadow Method"), AdvancedDisplay)
	TEnumAsByte<EMegaLightsShadowMethod::Type> MegaLightsShadowMethod;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta=(InlineEditConditionToggle), AdvancedDisplay)
	uint32 bOverrideRayEndBias : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (EditCondition = "bOverrideRayEndBias", ClampMin = "0", UIMax = "100"), AdvancedDisplay)
	float RayEndBias;

	/** 
	 * Channels that this light should affect.  
	 * These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	 * Lighting channels are only supported on translucent materials using forward shading (i.e. when not using the translucency lighting volume).
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Light)
	FLightingChannels LightingChannels;

	/** View / light masking support.  Controls which views this light should affect. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Light)
	FViewLightingChannels ViewLightingChannels;

	/** 
	 * The light function material to be applied to this light.
	 * Note that only non-lightmapped lights (UseDirectLightMap=False) can have a light function. 
	 * Light functions are supported within VolumetricFog, but only for Directional, Point and Spot lights. Rect lights are not supported.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightFunction)
	TObjectPtr<class UMaterialInterface> LightFunctionMaterial;

#if WITH_EDITORONLY_DATA
	/** When clearing the light func, e.g. because the light is made static, this field remembers the last value */
	UPROPERTY(Transient)
	TObjectPtr<class UMaterialInterface> StashedLightFunctionMaterial;
#endif

	/** Scales the light function projection.  X and Y scale in the directions perpendicular to the light's direction, Z scales along the light direction. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightFunction, meta=(AllowPreserveRatio = "true"))
	FVector LightFunctionScale;

	/** IES texture (light profiles from real world measured data) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightProfiles, meta=(DisplayName = "IES Texture"))
	TObjectPtr<class UTextureLightProfile> IESTexture;

	/** true: take light brightness from IES profile, false: use the light brightness - the maximum light in one direction is used to define no masking. Use with InverseSquareFalloff. Will be disabled if a valid IES profile texture is not supplied. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightProfiles, meta=(DisplayName = "Use IES Intensity", EditCondition="IESTexture!=nullptr"))
	uint32 bUseIESBrightness : 1;

	/** Global scale for IES brightness contribution. Only available when "Use IES Brightness" is selected, and a valid IES profile texture is set */
	UPROPERTY(BlueprintReadOnly, interp, Category=LightProfiles, meta=(UIMin = "0.0", UIMax = "10.0", DisplayName = "IES Intensity Scale"))
	float IESBrightnessScale;

	/** 
	 * Distance at which the light function should be completely faded to DisabledBrightness.  
	 * This is useful for hiding aliasing from light functions applied in the distance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightFunction, meta=(DisplayName = "Fade Distance"))
	float LightFunctionFadeDistance;

	/** 
	 * Brightness factor applied to the light when the light function is specified but disabled, for example in scene captures that use SceneCapView_LitNoShadows. 
	 * This should be set to the average brightness of the light function material's emissive input, which should be between 0 and 1.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightFunction, meta=(UIMin = "0.0", UIMax = "1.0"))
	float DisabledBrightness;

	/** 
	 * Whether to render light shaft bloom from this light. 
	 * For directional lights, the color around the light direction will be blurred radially and added back to the scene.
	 * for point lights, the color on pixels closer than the light's SourceRadius will be blurred radially and added back to the scene.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightShafts, meta=(DisplayName = "Light Shaft Bloom"))
	uint32 bEnableLightShaftBloom:1;

	/** Scales the additive color. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightShafts, meta=(UIMin = "0", UIMax = "10"))
	float BloomScale;

	/** Scene color must be larger than this to create bloom in the light shafts. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightShafts, meta=(UIMin = "0", UIMax = "4"))
	float BloomThreshold;

	/** After exposure is applied, scene color brightness larger than BloomMaxBrightness will be rescaled down to BloomMaxBrightness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightShafts, meta=(UIMin = "0", UIMax = "100", SliderExponent = 20.0))
	float BloomMaxBrightness;

	/** Multiplies against scene color to create the bloom color. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LightShafts)
	FColor BloomTint;

	/** 
	 * Whether to use ray traced distance field area shadows.  The project setting bGenerateMeshDistanceFields must be enabled for this to have effect.
	 * Distance field shadows support area lights so they create soft shadows with sharp contacts.  
	 * They have less aliasing artifacts than standard shadowmaps, but inherit all the limitations of distance field representations (only uniform scale, no deformation).
	 * These shadows have a low per-object cost (and don't depend on triangle count) so they are effective for distant shadows from a dynamic sun.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldShadows, meta=(DisplayName = "Distance Field Shadows"))
	bool bUseRayTracedDistanceFieldShadows;

	/** 
	 * Controls how large of an offset ray traced shadows have from the receiving surface as the camera gets further away.  
	 * This can be useful to hide self-shadowing artifacts from low resolution distance fields on huge static meshes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldShadows, meta=(UIMin = "0", UIMax = ".1"), AdvancedDisplay)
	float RayStartOffsetDepthScale;

public:
	/** Set intensity of the light */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetIntensity(float NewIntensity);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetIndirectLightingIntensity(float NewIntensity);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetVolumetricScatteringIntensity(float NewIntensity);

	/** Set color of the light */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetLightColor(FLinearColor NewLightColor, bool bSRGB = true);

	/** Set color of the light */
	UFUNCTION(Category="Rendering|Components|Light")
	ENGINE_API void SetLightFColor(FColor NewLightColor);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetTemperature(float NewTemperature);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetUseTemperature(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetLightFunctionMaterial(UMaterialInterface* NewLightFunctionMaterial);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetLightFunctionScale(FVector NewLightFunctionScale);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetLightFunctionFadeDistance(float NewLightFunctionFadeDistance);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetLightFunctionDisabledBrightness(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetAffectTranslucentLighting(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetTransmission(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetEnableLightShaftBloom(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetBloomScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetBloomThreshold(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetBloomMaxBrightness(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetBloomTint(FColor NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light", meta=(DisplayName = "Set IES Texture"))
	ENGINE_API void SetIESTexture(UTextureLightProfile* NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light", meta=(DisplayName = "Set Use IES Intensity"))
	ENGINE_API void SetUseIESBrightness(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light", meta=(DisplayName = "Set IES Intensity Scale"))
	ENGINE_API void SetIESBrightnessScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetShadowBias(float NewValue);
	
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetShadowSlopeBias(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetSpecularScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetDiffuseScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetForceCachedShadowsForMovablePrimitives(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetLightingChannels(bool bChannel0, bool bChannel1, bool bChannel2);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetUseRayTracedDistanceFieldShadows(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetMaxDrawDistance(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetMaxDistanceFadeRange(float NewValue);

public:
	/** The light's scene info. */
	FLightSceneProxy* SceneProxy;

	/**
	 * Pushes new selection state to the render thread light proxy
	 */
	ENGINE_API void PushSelectionToProxy();

	FStaticShadowDepthMap StaticShadowDepthMap;

	/** Fence used to track progress of render resource destruction. */
	FRenderCommandFence DestroyFence;

	/** true when this light component has been added to the scene as a normal visible light. Used to keep track of whether we need to dirty the render state in UpdateColorAndBrightness */
	uint32 bAddedToSceneVisible:1;

	/**
	 * Test whether this light affects the given primitive.  This checks both the primitive and light settings for light relevance
	 * and also calls AffectsBounds.
	 * @param PrimitiveSceneInfo - The primitive to test.
	 * @return True if the light affects the primitive.
	 */
	ENGINE_API bool AffectsPrimitive(const UPrimitiveComponent* Primitive) const;

	/**
	 * Test whether the light affects the given bounding volume.
	 * @param Bounds - The bounding volume to test.
	 * @return True if the light affects the bounding volume
	 */
	ENGINE_API virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const;

	/**
	 * Return the world-space bounding box of the light's influence.
	 */
	virtual FBox GetBoundingBox() const { return FBox(FVector(-HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX),FVector(HALF_WORLD_MAX,HALF_WORLD_MAX,HALF_WORLD_MAX)); }

	virtual FSphere GetBoundingSphere() const
	{
		// Directional lights will have a radius of WORLD_MAX
		return FSphere(FVector::ZeroVector, WORLD_MAX);
	}

#if WITH_EDITOR
	virtual FBox GetStreamingBounds() const override
	{
		return GetBoundingBox();
	}
#endif // WITH_EDITOR

	/**
	 * Return the homogenous position of the light.
	 */
	virtual FVector4 GetLightPosition() const PURE_VIRTUAL(ULightComponent::GetPosition,return FVector4(););

	/**
	* @return ELightComponentType for the light component class
	*/
	virtual ELightComponentType GetLightType() const PURE_VIRTUAL(ULightComponent::GetLightType,return LightType_MAX;);

	virtual FLightmassLightSettings GetLightmassSettings() const PURE_VIRTUAL(ULightComponent::GetLightmassSettings,return FLightmassLightSettings(););

	virtual float GetUniformPenumbraSize() const PURE_VIRTUAL(ULightComponent::GetUniformPenumbraSize,return 0;);

	ENGINE_API virtual ELightUnits GetLightUnits() const;

	/**
	 * Check whether a given primitive will cast shadows from this light.
	 * @param Primitive - The potential shadow caster.
	 * @return Returns True if a primitive blocks this light.
	 */
	ENGINE_API bool IsShadowCast(UPrimitiveComponent* Primitive) const;

	/* Whether to consider light as a sunlight for atmospheric scattering. */  
	virtual bool IsUsedAsAtmosphereSunLight() const
	{
		return false;
	}
	virtual uint8 GetAtmosphereSunLightIndex() const
	{
		return 0;
	}
	virtual FLinearColor GetAtmosphereSunDiskColorScale() const
	{
		return FLinearColor::White;
	}

	/** Compute current light brightness based on whether there is a valid IES profile texture attached, and whether IES brightness is enabled */
	ENGINE_API virtual float ComputeLightBrightness() const;
#if WITH_EDITOR
	/** Set the Intensity using the brightness. The unit of brightness depends on the light type. */
	ENGINE_API virtual void SetLightBrightness(float InBrightness);
#endif

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void UpdateLightSpriteTexture() override;
	ENGINE_API virtual void CheckForErrors() override;
#endif // WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	//~ End UObject Interface.

	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API void ApplyComponentInstanceData(struct FPrecomputedLightInstanceData* ComponentInstanceData);
	ENGINE_API virtual void PropagateLightingScenarioChange() override;
	ENGINE_API virtual bool IsPrecomputedLightingValid() const override;

	/** @return number of material elements in this primitive */
	ENGINE_API virtual int32 GetNumMaterials() const;

	/** @return MaterialInterface assigned to the given material index (if any) */
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const;

	/** Set the MaterialInterface to use for the given element index (if valid) */
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial);

	/** Set the light func to null, but remember the current value so it can be restored later */
	ENGINE_API void ClearLightFunctionMaterial();

	virtual class FLightSceneProxy* CreateSceneProxy() const
	{
		return NULL;
	}
	
	/** Get the PSO precaching data */
	FPSOPrecacheComponentData& GetPSOPrecacheComponentData() { return PSOPrecacheComponentData; }

	ENGINE_API bool CastsStaticShadow() const;
	ENGINE_API bool CastsDynamicShadow() const;
	ENGINE_API bool TransmitsThroughSurfaces() const;
	ENGINE_API bool GetContactShadowLengthInWS() const;
	ENGINE_API float GetRayEndBias() const;
	ENGINE_API const FStaticShadowDepthMap* GetStaticShadowDepthMap() const;

	inline static FLightSceneProxyDesc BuildSceneProxyDesc(const ULightComponent& InLightComponent)
	{
		FLightSceneProxyDesc ProxyDesc;
		BuildSceneProxyDesc(InLightComponent, ProxyDesc);
		return ProxyDesc;
	}

protected:

	ENGINE_API static void BuildSceneProxyDesc(const ULightComponent& InLightComponent, FLightSceneProxyDesc& OutProxyDesc);

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void PrecachePSOs() override;
	//~ Begin UActorComponent Interface

	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	//~ End USceneComponent Interface

	void UpdateProxy(FUpdateLightProxyCallback&& Func);
		
	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;

	bool IsPSOPrecaching() const
	{
		return PSOPrecacheComponentData.IsPSOPrecaching();
	}

private:

	bool CanTraceDistanceFieldShadows() const;

public:
	ENGINE_API virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;

	/** Script interface to retrieve light direction. */
	ENGINE_API FVector GetDirection() const;

	/** Script interface to update the color and brightness on the render thread. */
	ENGINE_API void UpdateColorAndBrightness();

	ENGINE_API const FLightComponentMapBuildData* GetLightComponentMapBuildData() const;

	ENGINE_API void InitializeStaticShadowDepthMap();

	ENGINE_API FLinearColor GetColoredLightBrightness() const;

	/** Get the color temperature in the working color space. */
	ENGINE_API FLinearColor GetColorTemperature() const;

#if WITH_EDITOR	
	/** 
	 * Iterates over ALL stationary light components in the target world and assigns their preview shadowmap channel, and updates light icons accordingly.
	 * Also handles assignment after a lighting build, so that the same algorithm is used for previewing and static lighting.
	 */
	static ENGINE_API void ReassignStationaryLightChannels(UWorld* TargetWorld, bool bAssignForLightingBuild, FStaticLightingBuildContext* LightingContext);
#endif

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateColorAndBrightness, ULightComponent&);

	/** Called When light color or brightness needs update */
	static ENGINE_API FOnUpdateColorAndBrightness UpdateColorAndBrightnessEvent;
};


/** Used to store lightmap data during RerunConstructionScripts */
USTRUCT()
struct FPrecomputedLightInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()

	FPrecomputedLightInstanceData() = default;
	FPrecomputedLightInstanceData(const ULightComponent* SourceComponent)
		: FSceneComponentInstanceData(SourceComponent)
		, Transform(SourceComponent->GetComponentTransform())
		, OriginalLightGuid(SourceComponent->OriginalLightGuid)		
		, LightGuid(SourceComponent->LightGuid)
		, PreviewShadowMapChannel(SourceComponent->PreviewShadowMapChannel)
	{}
	virtual ~FPrecomputedLightInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<ULightComponent>(Component)->ApplyComponentInstanceData(this);
	}

	UPROPERTY()
	FTransform Transform;

	UPROPERTY()
	FGuid OriginalLightGuid;

	UPROPERTY()
	FGuid LightGuid;

	UPROPERTY()
	int32 PreviewShadowMapChannel = 0;
};


