// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "LightComponentId.h"
#include "Components/SceneComponent.h"
#include "LightComponentBase.generated.h"

class UTexture2D;

UCLASS(abstract, HideCategories=(Trigger,Activation,"Components|Activation",Physics), ShowCategories=(Mobility), MinimalAPI)
class ULightComponentBase : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/**
	 * GUID used to associate a light component with precomputed shadowing information across levels.
	 * The GUID changes whenever the light position changes.
	 */
	UPROPERTY()
	FGuid OriginalLightGuid;

	UPROPERTY()
	FGuid LightGuid;
	
	/**  */
	UPROPERTY()
	float Brightness_DEPRECATED;

	/** 
	 * Total energy that the light emits.  
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=Light, meta=(DisplayName = "Intensity", UIMin = "0.0", UIMax = "20.0", ShouldShowInViewport = true))
	float Intensity;

	/** 
	 * Filter color of the light.
	 * Note that this can change the light's effective intensity.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=Light, meta=(HideAlphaChannel, ShouldShowInViewport = true))
	FColor LightColor;

	/** 
	 * Whether the light can affect the world, or whether it is disabled.
	 * A disabled light will not contribute to the scene in any way.  This setting cannot be changed at runtime and unbuilds lighting when changed.
	 * Setting this to false has the same effect as deleting the light, so it is useful for non-destructive experiments.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta = (ShouldShowInViewport = true))
	uint32 bAffectsWorld:1;

	/**
	 * Whether the light should cast any shadows.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light)
	uint32 CastShadows:1;

	/**
	 * Whether the light should cast shadows from static objects.  Also requires Cast Shadows to be set to True.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 CastStaticShadows:1;

	/**
	 * Whether the light should cast shadows from dynamic objects.  Also requires Cast Shadows to be set to True.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 CastDynamicShadows:1;

	/** Whether the light affects translucency or not.  Disabling this can save GPU time when there are many small lights. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 bAffectTranslucentLighting:1;

	/** Whether light from this light transmits through surfaces with subsurface scattering profiles. Requires light to be movable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (EditCondition="Mobility==EComponentMobility::Movable"))
	uint32 bTransmission : 1;

	/** Whether the light shadows volumetric fog.  Disabling this can save GPU time. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay)
	uint32 bCastVolumetricShadow : 1;

	/**
	 * Whether the light should cast high quality hair-strands self-shadowing. When this option is enabled, an extra GPU cost for this light. 
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay)
	uint32 bCastDeepShadow : 1;

	/** Whether the light shadows are computed with shadow-mapping or ray-tracing (when available). */
	UPROPERTY()
	uint32 bCastRaytracedShadow_DEPRECATED : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (DisplayName = "Cast Ray Traced Shadows"), AdvancedDisplay)
	TEnumAsByte<ECastRayTracedShadow::Type> CastRaytracedShadow;

	/** Whether the light affects objects in reflections, when ray-traced reflection is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (DisplayName = "Affect Ray Tracing Reflections"))
	uint32 bAffectReflection : 1;

	/** Whether the light affects global illumination, when ray-traced global illumination is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (DisplayName = "Affect Ray Tracing Global Illumination"))
	uint32 bAffectGlobalIllumination : 1;

	/**
	 *Change the deep shadow layers distribution 0:linear distribution (uniform layer distribution), 1:exponential (more details on near small details).
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DeepShadowLayerDistribution;

	/** 
	 * Scales the indirect lighting contribution from this light. 
	 * A value of 0 disables any GI from this light. Default is 1.
	 */
	UPROPERTY(BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.0", UIMax = "6.0"))
	float IndirectLightingIntensity;

	/** Intensity of the volumetric scattering from this light.  This scales Intensity and LightColor. */
	UPROPERTY(BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.25", UIMax = "4.0"))
	float VolumetricScatteringIntensity;

	/** Samples per pixel for ray tracing */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RayTracing)
	int SamplesPerPixel;

#if WITH_EDITORONLY_DATA
	/** Sprite for static light in the editor. */
	UPROPERTY(transient)
	TObjectPtr<UTexture2D> StaticEditorTexture;

	/** Sprite scaling for static light in the editor. */
	UPROPERTY(transient)
	float StaticEditorTextureScale;

	/** Sprite for dynamic light in the editor. */
	UPROPERTY(transient)
	TObjectPtr<UTexture2D> DynamicEditorTexture;

	/** Sprite scaling for dynamic light in the editor. */
	UPROPERTY(transient)
	float DynamicEditorTextureScale;
#endif

	/** Sets whether this light casts shadows */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetCastShadows(bool bNewValue);

	/** Gets the light color as a linear color */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API FLinearColor GetLightColor() const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetCastVolumetricShadow(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetCastDeepShadow(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetAffectReflection(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetAffectGlobalIllumination(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light", meta = (DeprecatedFunction, DeprecationMessage = "ULightComponentBase::SetCastRaytracedShadow is deprecated. Use ULightComponentBase::SetCastRaytracedShadows instead."))
	ENGINE_API void SetCastRaytracedShadow(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetCastRaytracedShadows(ECastRayTracedShadow::Type bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Light")
	ENGINE_API void SetSamplesPerPixel(int NewValue);

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/**
	 * Called after duplication & serialization and before PostLoad. Used to e.g. make sure GUIDs remains globally unique.
	 */
	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

#if WITH_EDITOR
	/** UObject interface */
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/**
	* @return Path to the editor sprite for the light component class
	*/
	UTexture2D* GetEditorSprite() const
	{
		return (Mobility == EComponentMobility::Movable) ? DynamicEditorTexture : StaticEditorTexture;
	}

	/**
	* @return Uniform scaling factor for the sprite for the light component class
	*/
	float GetEditorSpriteScale() const
	{
		return (Mobility == EComponentMobility::Movable) ? DynamicEditorTextureScale : StaticEditorTextureScale;
	}

	/** Update the texture used on the editor sprite */
	ENGINE_API virtual void UpdateLightSpriteTexture();
#endif

	/**
	 * Validate light GUIDs and resets as appropriate.
	 */
	ENGINE_API void ValidateLightGUIDs();

	/**
	 * Update/reset light GUIDs.
	 */
	ENGINE_API virtual void UpdateLightGUIDs();

	/** Returns true if the light's Mobility is set to Movable */
	bool IsMovable() const
	{
		return (Mobility == EComponentMobility::Movable);
	}

	/**
	 * Return True if a light's parameters as well as its position is static during gameplay, and can thus use static lighting.
	 * A light with HasStaticLighting() == true will always have HasStaticShadowing() == true as well.
	 */
	ENGINE_API bool HasStaticLighting() const;

	/** 
	 * Whether the light has static direct shadowing.  
	 * The light may still have dynamic brightness and color. 
	 * The light may or may not also have static lighting.
	 */
	ENGINE_API bool HasStaticShadowing() const;

#if WITH_EDITOR
	/** UObject Interface */
	ENGINE_API virtual void PostLoad() override;

	/** UActorComponent Interface */
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	/** We return a small bounds to allow us to non-interpenetrates when placing lights in the level. */
	ENGINE_API virtual bool ShouldCollideWhenPlacing() const override;

	/** Get the extent used when placing this component in the editor, used for 'pulling back' hit. */
	ENGINE_API virtual FBoxSphereBounds GetPlacementExtent() const override;

	const FLightComponentId& GetLightComponentId() const { return LightComponentId; }
	TEnumAsByte<ECastRayTracedShadow::Type> GetCastRaytracedShadow() const { return CastShadows == 0 ? (TEnumAsByte<ECastRayTracedShadow::Type>) ECastRayTracedShadow::Disabled : CastRaytracedShadow; }

private:
	/** Runtime unique id used to identify a light component. */
	FLightComponentId LightComponentId;
};



