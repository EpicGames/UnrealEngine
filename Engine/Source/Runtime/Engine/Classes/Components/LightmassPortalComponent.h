// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "LightmassPortalComponent.generated.h"

UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent, Activation, "Components|Activation", Mobility), MinimalAPI)
class ULightmassPortalComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UBoxComponent> PreviewBox;

public:
	void UpdatePreviewShape();

	//~ Begin UActorComponent Interface
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void SendRenderTransform_Concurrent() override;
	//~ End UActorComponent Interface
};

