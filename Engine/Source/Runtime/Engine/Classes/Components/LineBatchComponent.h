// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"

#include "LineBatchComponent.generated.h"

class FPrimitiveSceneProxy;

USTRUCT()
struct FBatchedLine
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector Start;

	UPROPERTY()
	FVector End;

	UPROPERTY()
	FLinearColor Color;

	UPROPERTY()
	float Thickness;

	UPROPERTY()
	float RemainingLifeTime;

	UPROPERTY()
	uint8 DepthPriority;

	UPROPERTY()
	uint32 BatchID;

	FBatchedLine()
		: Start(ForceInit)
		, End(ForceInit)
		, Color(ForceInit)
		, Thickness(0)
		, RemainingLifeTime(0)
		, DepthPriority(0)
		, BatchID(0)
	{}
	FBatchedLine(const FVector& InStart, const FVector& InEnd, const FLinearColor& InColor, float InLifeTime, float InThickness, uint8 InDepthPriority, uint32 InBatchID = 0)
		:	Start(InStart)
		,	End(InEnd)
		,	Color(InColor)
		,	Thickness(InThickness)
		,	RemainingLifeTime(InLifeTime)
		,	DepthPriority(InDepthPriority)
		,	BatchID(InBatchID)
	{}
};

USTRUCT()
struct FBatchedPoint
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	FLinearColor Color;

	UPROPERTY()
	float PointSize;

	UPROPERTY()
	float RemainingLifeTime;

	UPROPERTY()
	uint8 DepthPriority;
	
	UPROPERTY()
	uint32 BatchID;

	FBatchedPoint()
		: Position(ForceInit)
		, Color(ForceInit)
		, PointSize(0)
		, RemainingLifeTime(0)
		, DepthPriority(0)
		, BatchID(0)
	{}
	FBatchedPoint(const FVector& InPosition, const FLinearColor& InColor, float InPointSize, float InLifeTime, uint8 InDepthPriority, uint32 InBatchID = 0)
		:	Position(InPosition)
		,	Color(InColor)
		,	PointSize(InPointSize)
		,	RemainingLifeTime(InLifeTime)
		,	DepthPriority(InDepthPriority)
		,	BatchID(InBatchID)
	{}
};

struct FBatchedMesh
{
	FBatchedMesh() = default;

	/**
	 * MeshVerts - linear array of world space vertex positions
	 * MeshIndices - array of indices into MeshVerts.  Each triplet is a tri.  i.e. [0,1,2] is first tri, [3,4,5] is 2nd tri, etc
	 */
	FBatchedMesh(TArray<FVector> const& InMeshVerts, TArray<int32> const& InMeshIndices, FColor const& InColor, uint8 InDepthPriority, float LifeTime, uint32 InBatchID = 0)
		: MeshVerts(InMeshVerts), MeshIndices(InMeshIndices), 
		  Color(InColor), RemainingLifeTime(LifeTime), DepthPriority(InDepthPriority), BatchID(InBatchID)
	{}

	TArray<FVector> MeshVerts;
	TArray<int32> MeshIndices;
	FColor Color;
	float RemainingLifeTime = 0.f;
	uint8 DepthPriority = 0;
	uint32 BatchID = 0;
};

/** 
 * The line batch component buffers and draws lines (and some other line-based shapes) in a scene. 
 *	This can be useful for debug drawing, but is not very performant for runtime use.
 */
UCLASS(MinimalAPI)
class ULineBatchComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Buffer of lines to draw */
	TArray<FBatchedLine> BatchedLines;
	/** Buffer or points to draw */
	TArray<FBatchedPoint> BatchedPoints;
	/** Default time that lines/points will draw for */
	float DefaultLifeTime;
	/** Buffer of simple meshes to draw */
	TArray<FBatchedMesh> BatchedMeshes;
	/** Whether to calculate a tight accurate bounds (encompassing all points), or use a giant bounds that is fast to compute. */
	uint32 bCalculateAccurateBounds:1;

	/** Defines the value for an invalid id */
	static constexpr uint32 INVALID_ID = 0;

	/** Provide many lines to draw - faster than calling DrawLine many times. */
	ENGINE_API void DrawLines(TArrayView<FBatchedLine> InLines);

	/** Draw a box */
	ENGINE_API void DrawBox(const FBox& Box, const FMatrix& TM, FLinearColor Color, uint8 InDepthPriorityGroup, uint32 BatchID = INVALID_ID);

	/** Draw a box */
	ENGINE_API void DrawBox(const FVector& Center, const FVector& Box, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);
	
	/** Draw a box */
	ENGINE_API void DrawBox(const FVector& Center, const FVector& Box, const FQuat& Rotation, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	/** Draw an arrow */
	ENGINE_API void DrawDirectionalArrow(const FMatrix& ArrowToWorld, FLinearColor InColor, float Length, float ArrowSize, uint8 DepthPriority, uint32 BatchID = INVALID_ID);

	/** Draw an arrow */
	ENGINE_API void DrawDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	/** Draw a circle */
	ENGINE_API void DrawCircle(const FVector& Base, const FVector& X, const FVector& Y, FLinearColor Color, float Radius, int32 NumSides, uint8 DepthPriority, uint32 BatchID = INVALID_ID);

	/** Draw a sphere */
	ENGINE_API void DrawSphere(const FVector& Center, float Radius, int32 Segments, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	/** Draw a cylinder */
	ENGINE_API void DrawCylinder(const FVector& Start, const FVector& End, float Radius, int32 Segments, FLinearColor  Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	/** Draw a cone */
	ENGINE_API void DrawCone(const FVector& Origin, const FVector& Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FLinearColor DrawColor, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	/** Draw a cone */
	ENGINE_API void DrawCapsule(const FVector& Center, float HalfHeight, float Radius, const FQuat& Rotation, FLinearColor Color, float LifeTime, uint8 DepthPriority, float Thickness, uint32 BatchID = INVALID_ID);

	ENGINE_API virtual void DrawLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriority,
		float Thickness = 0.0f,
		float LifeTime = 0.0f,
		uint32 BatchID = INVALID_ID
		);
	ENGINE_API virtual void DrawPoint(
		const FVector& Position,
		const FLinearColor& Color,
		float PointSize,
		uint8 DepthPriority,
		float LifeTime = 0.0f,
		uint32 BatchID = INVALID_ID
		);

	/** Draw a box */
	ENGINE_API void DrawSolidBox(FBox const& Box, FTransform const& Xform, const FColor& Color, uint8 DepthPriority, float LifeTime, uint32 BatchID = INVALID_ID);
	/** Draw a mesh */
	ENGINE_API void DrawMesh(TArray<FVector> const& Verts, TArray<int32> const& Indices, FColor const& Color, uint8 DepthPriority, float LifeTime, uint32 BatchID = INVALID_ID);

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End UPrimitiveComponent Interface.
	
	
	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End UActorComponent Interface.

	/** Clear all batched lines, points and meshes */
	ENGINE_API void Flush();
	
	/** Remove batched lines, points and meshes with given ID */
	ENGINE_API void ClearBatch(uint32 InBatchID);
	
protected:
	void AddHalfCircle(const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, const float Radius, int32 NumSides, const float LifeTime, uint8 DepthPriority, const float Thickness, const uint32 BatchID);
	void AddCircle(const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, const float Radius, int32 NumSides, const float LifeTime, uint8 DepthPriority, const float Thickness, const uint32 BatchID);
};
