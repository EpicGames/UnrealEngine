// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "Math/Color.h"
#include "LocalFogVolumeComponent.generated.h"

class FLocalFogVolumeSceneProxy;

UCLASS(ClassGroup = Rendering, collapsecategories, hidecategories = (Object, Mobility, Activation, "Components|Activation"), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class ULocalFogVolumeComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	~ULocalFogVolumeComponent();

	/** The density of the radial fog representing its extinction coefficient at the center of the sphere. The final look of the volume is determined by combining the "Coverage=1-Transmittance" of both radial and height fog in order to achieve both soft edges and height fog.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Radial Fog Distribution", meta = (DisplayName = "Radial Fog Density", UIMin = "0", UIMax = "2.0", SliderExponent = 2.0, ClampMin = 0.0))
	float RadialFogExtinction = 1.0f;

	/** The density of the radial fog representing its extinction coefficient at height 0 in the unit sphere. The final look of the volume is determined by combining the "Coverage=1-Transmittance" of both radial and height fog in order to achieve both soft edges and height fog.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Height Fog Distribution", meta = (DisplayName = "Height Fog Density", UIMin = "0", UIMax = "2.0", SliderExponent = 2.0, ClampMin = 0.0))
	float HeightFogExtinction = 1.0f;

	/** Controls how the density decreases as height increases. Smaller values make the visible transition larger. 1.0 is the lowest value before visual artifact are visible at the horizon. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Height Fog Distribution", meta = (UIMin = "1.0", UIMax = "5000", SliderExponent = 2.0, ClampMin = 1.0))
	float HeightFogFalloff = 1000.0f;

	/** Height offset, relative to the actor Z position. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Height Fog Distribution", meta = (UIMin = "-2.0", UIMax = "2.0"))
	float HeightFogOffset = 0.0f;

	/** Controls the phase `G` parameter, describing the directionality of the scattering within this fog volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Shading", meta = (DisplayName = "Scattering Distribution", UIMin = "0.0", UIMax = "0.999", ClampMin = 0.0, ClampMax = 0.999))
	float FogPhaseG = 0.2f;

	/** Controls the albedo of this fog volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Shading", meta = (ClampMin = 0.0, ClampMax = 1.0))
	FLinearColor FogAlbedo = FLinearColor::White;

	/** Controls the emissive color of this fog volume. (Unit = Luminance) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Shading")
	FLinearColor FogEmissive = FLinearColor::Black;

	/** Controls the distance after which this volume starts to appear. (Unit = Centimeter) This distance is clamped to be greater than r.LocalFogVolume.GlobalStartDistance. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Shading", meta = (DisplayName = "Start Distance", UIMin = "0.0", ClampMin = 0.0))
	float FogStartDistance = 0.0f;

	/** The priority can be used as a way to override the sorting by distance. A lower value means the volume will be considered further away, i.e. it will draw behind the one with a higher priority value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sorting", meta = (UIMin = "-127", UIMax = "127", ClampMin = -127, ClampMax = 127))
	int32 FogSortPriority = 0;
	
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Radial Fog Density"))
	ENGINE_API void SetRadialFogExtinction(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Height Fog Density"))
	ENGINE_API void SetHeightFogExtinction(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHeightFogFalloff(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHeightFogOffset(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Scattering Distribution"))
	ENGINE_API void SetFogPhaseG(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetFogStartDistance(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetFogAlbedo(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetFogEmissive(FLinearColor NewValue);

public:

	static float GetBaseVolumeSize()
	{
		return 500.0f; // This is immutable and cannot be changed without transform data conversion of the component.
	}

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	/** Callback to create the rendering thread mirror. */
	ENGINE_API FLocalFogVolumeSceneProxy* CreateSceneProxy();

protected:
	//~ Begin UActorComponent Interface.
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void SendRenderTransform_Concurrent() override;
	virtual void DestroyRenderState_Concurrent() override;
	//~ End UActorComponent Interface.

	void SendRenderTransformCommand();

private:

	FLocalFogVolumeSceneProxy* LocalFogVolumeSceneProxy;
};

