// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "LocalLightSceneProxyDesc.h"
#include "Components/LightComponent.h"
#include "LocalLightComponent.generated.h"

class FLightSceneProxy;

/**
 * A light component which emits light from a single point equally in all directions.
 */
UCLASS(abstract, ClassGroup=(Lights,Common), hidecategories=(Object, LightShafts), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class ULocalLightComponent : public ULightComponent
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Units used for the intensity. 
	 * The peak luminous intensity is measured in candelas, while the luminous flux is measured in lumens.
	 * When the units are set in Nits, the light's power is also determined by the size of the light source (larger sources will emit more light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta=(DisplayName="Intensity Units", EditCondition="!bUseIESBrightness"))
	ELightUnits IntensityUnits;

	/**
	* Blend Factor used to blend between Intensity and Intensity/Exposure. 
	* This is useful for gameplay lights that should have constant brighness on screen independent of current exposure.
	* This feature can cause issues with exposure particularly when used on the primary light on a scene, as such it's usage should be limited.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "1.0"))
	float InverseExposureBlend;

	UPROPERTY()
	float Radius_DEPRECATED;

	/**
	 * Bounds the light's visible influence.  
	 * This clamping of the light's influence is not physically correct but very important for performance, larger lights cost more.
	 */
	UPROPERTY(interp, BlueprintReadOnly, Category=Light, meta=(UIMin = "8.0", UIMax = "16384.0", SliderExponent = "5.0", ShouldShowInViewport = true))
	float AttenuationRadius;

	/** The Lightmass settings for this object. */
	UPROPERTY(EditAnywhere, Category=Light, meta=(ShowOnlyInnerProperties))
	struct FLightmassPointLightSettings LightmassSettings;

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetAttenuationRadius(float NewRadius);

	/** Set the units used for the intensity of the light */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetIntensityUnits(ELightUnits NewIntensityUnits);

	UFUNCTION(BlueprintPure, Category="Rendering|Lighting")
	static ENGINE_API float GetUnitsConversionFactor(ELightUnits SrcUnits, ELightUnits TargetUnits, float CosHalfConeAngle = -1);

protected:
	//~ Begin UActorComponent Interface
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	//~ End UActorComponent Interface

public:
	//~ Begin ULightComponent Interface.
	ENGINE_API virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override;
	ENGINE_API virtual FVector4 GetLightPosition() const override;
	ENGINE_API virtual FBox GetBoundingBox() const override;
	ENGINE_API virtual FSphere GetBoundingSphere() const override;
	virtual FLightmassLightSettings GetLightmassSettings() const override
	{
		return LightmassSettings;
	}
	virtual ELightUnits GetLightUnits() const { return IntensityUnits; }

	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	inline static FLocalLightSceneProxyDesc BuildSceneProxyDesc(const ULocalLightComponent& InLightComponent)
	{
		FLocalLightSceneProxyDesc ProxyDesc;
		BuildSceneProxyDesc(InLightComponent, ProxyDesc);
		return ProxyDesc;
	}

protected:

	ENGINE_API static void BuildSceneProxyDesc(const ULocalLightComponent& InLightComponent, FLocalLightSceneProxyDesc& OutProxyDesc);

private:

	/** Pushes the value of radius to the rendering thread. */
	ENGINE_API void PushRadiusToRenderThread();
};



