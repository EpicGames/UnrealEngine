// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "MaterialBillboardComponent.generated.h"

class FPrimitiveSceneProxy;
class UCurveFloat;
class UMaterialInterface;

USTRUCT(BlueprintType)
struct FMaterialSpriteElement
{
	GENERATED_USTRUCT_BODY()

	/** The material that the sprite is rendered with. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	TObjectPtr<class UMaterialInterface> Material;
	
	/** A curve that maps distance on the X axis to the sprite opacity on the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	TObjectPtr<UCurveFloat> DistanceToOpacityCurve;
	
	/** Whether the size is defined in screen-space or world-space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	uint32 bSizeIsInScreenSpace:1;

	/** The base width of the sprite, multiplied with the DistanceToSizeCurve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	float BaseSizeX;

	/** The base height of the sprite, multiplied with the DistanceToSizeCurve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	float BaseSizeY;

	/** A curve that maps distance on the X axis to the sprite size on the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialSpriteElement)
	TObjectPtr<UCurveFloat> DistanceToSizeCurve;

	FMaterialSpriteElement()
		: Material(NULL)
		, DistanceToOpacityCurve(NULL)
		, bSizeIsInScreenSpace(false)
		, BaseSizeX(32)
		, BaseSizeY(32)
		, DistanceToSizeCurve(NULL)
	{
	}

	friend FArchive& operator<<(FArchive& Ar,FMaterialSpriteElement& LODElement);
};

/** 
 * A 2d material that will be rendered always facing the camera.
 */
UCLASS(ClassGroup=Rendering, collapsecategories, hidecategories=(Object,Activation,"Components|Activation",Physics,Collision,Lighting,Mesh,PhysicsVolume), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UMaterialBillboardComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Current array of material billboard elements */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Sprite)
	TArray<FMaterialSpriteElement> Elements;

	/** Set all elements of this material billboard component */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|MaterialSprite")
	ENGINE_API void SetElements(const TArray<FMaterialSpriteElement>& NewElements);

	/** Adds an element to the sprite. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|MaterialSprite")
	ENGINE_API void AddElement(
		class UMaterialInterface* Material,
		class UCurveFloat* DistanceToOpacityCurve,
		bool bSizeIsInScreenSpace,
		float BaseSizeX,
		float BaseSizeY,
		class UCurveFloat* DistanceToSizeCurve
		);

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface.
	// 
	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 Index) const override;
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* Material) override;
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	//~ End UPrimitiveComponent Interface

	/** Collect all the PSO precache data used by the material billboard component */
	ENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;

	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	//~ End USceneComponent Interface
};
