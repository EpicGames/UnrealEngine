// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/SortedMap.h"

#include "MeshComponent.generated.h"

class UMaterialInterface;
struct FMaterialRelevance;

/**
 * MeshComponent is an abstract base for any component that is an instance of a renderable collection of triangles.
 *
 * @see UStaticMeshComponent
 * @see USkeletalMeshComponent
 */
UCLASS(abstract, ShowCategories = (VirtualTexture), MinimalAPI)
class UMeshComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:
	/** Per-Component material overrides.  These must NOT be set directly or a race condition can occur between GC and the rendering thread. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Rendering, Meta=(ToolTip="Material overrides."))
	TArray<TObjectPtr<class UMaterialInterface>> OverrideMaterials;

	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API virtual TArray<class UMaterialInterface*> GetMaterials() const;

	/** Determines if we use the nanite overrides from any materials */
	virtual bool UseNaniteOverrideMaterials() const { return false; }

	/** Returns override materials count */
	ENGINE_API virtual int32 GetNumOverrideMaterials() const;

	/** Translucent material to blend on top of this mesh. Mesh will be rendered twice - once with a base material and once with overlay material */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Rendering)
	TObjectPtr<class UMaterialInterface> OverlayMaterial;
	
	/** The max draw distance for overlay material. A distance of 0 indicates that overlay will be culled using primitive max distance. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Rendering)
	float OverlayMaterialMaxDrawDistance;

	/** Get the overlay material used by this instance. If GetMaterialSlot is enabled, the relative slot material is returned. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API class UMaterialInterface* GetOverlayMaterial(bool bGetMaterialSlot = false, int32 SlotIndex = 0) const;

	/** Change the overlay material used by this instance. If SetMaterialSlot is enabled, the relative slot material is changed. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetOverlayMaterial(class UMaterialInterface* NewOverlayMaterial, bool bSetMaterialSlot = false, int32 SlotIndex = 0);

	/** Get the overlay material used by this instance */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API float GetOverlayMaterialMaxDrawDistance() const;

	/** Change the overlay material max draw distance used by this instance */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetOverlayMaterialMaxDrawDistance(float InMaxDrawDistance);

	/**
	 * Translucent material to blend on top of this mesh. Mesh will be rendered twice - once with a base material and once with overlay material.
	 * The difference with the  global OverlayMaterial is those are per material slot, if the entry is null or doesn't exist the global
	 * OverlayMaterial will be use for sections using the material slot.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Rendering)
	TArray<TObjectPtr<class UMaterialInterface>> MaterialSlotsOverlayMaterial;

	/**
	 * Fill the array with every material slot overlay material use by this instance.
	 * 
	 * If this component material slot overlay material will be used if not null.
	 * If there is no valid component material slot overlay material, the mesh material slot overlay material will be used if not null.
	 * If there is no valid asset material slot overlay material, a null entry will be set for the material slot overlay material.
	 */
	ENGINE_API void GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<class UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const;

#if WITH_EDITOR
	/*
	 * Make sure the Override array is using only the space it should use.
	 * 1. The override array cannot be bigger then the number of mesh material.
	 * 2. The override array must not end with a nullptr UMaterialInterface.
	 */
	ENGINE_API void CleanUpOverrideMaterials();

	/*
	 * Make sure the MaterialSlotsOverlayMaterial is using only the space it should use.
	 * - The data should be only on existing mesh material slot.
	 */
	ENGINE_API void CleanUpMaterialSlotsOverlayMaterial();
#endif

	/** 
	 * This empties all override materials and used by editor when replacing preview mesh 
	 */
	ENGINE_API void EmptyOverrideMaterials();

	/**
	 * Returns true if there are any override materials set for this component
	 */
	ENGINE_API bool HasOverrideMaterials();

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End UActorComponent Interface

	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual int32 GetNumMaterials() const override;
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	ENGINE_API virtual UMaterialInterface* GetMaterialByName(FName MaterialSlotName) const override;
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	ENGINE_API virtual void SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* Material) override;
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;	
	//~ End UPrimitiveComponent Interface

	/** Accesses the scene relevance information for the materials applied to the mesh. Valid from game thread only. */
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API virtual FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const;

	/**
	 *	Tell the streaming system whether or not all mip levels of all textures used by this component should be loaded and remain loaded.
	 *	@param bForceMiplevelsToBeResident		Whether textures should be forced to be resident or not.
	 */
	ENGINE_API virtual void SetTextureForceResidentFlag( bool bForceMiplevelsToBeResident );

#if WITH_EDITOR
	/**
	 *	Set the bMarkAsEditorStreamingPool on all textures used by this component.
	 *	@param bInMarkAsEditorStreamingPool		Whether textures should only be counted for the Editor pool stats.
	 */
	ENGINE_API void SetMarkTextureAsEditorStreamingPool(bool bInMarkAsEditorStreamingPool);
#endif

	/**
	 *	Tell the streaming system to start loading all textures with all mip-levels.
	 *	@param Seconds							Number of seconds to force all mip-levels to be resident
	 *	@param bPrioritizeCharacterTextures		Whether character textures should be prioritized for a while by the streaming system
	 *	@param CinematicTextureGroups			Bitfield indicating which texture groups that use extra high-resolution mips
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API virtual void PrestreamTextures( float Seconds, bool bPrioritizeCharacterTextures, int32 CinematicTextureGroups = 0 );

	/**
	 *	Tell the streaming system to start streaming in all LODs for the mesh.
	*	Note: this function may set bIgnoreStreamingMipBias on this component enable the FastForceResident system.
	 *  @return bool							True if streaming was successfully requested
	 *	@param Seconds							Number of seconds to force all LODs to be resident
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta=(ScriptName="PrestreamMeshLods;PrestreamMeshLODs"))
	virtual bool PrestreamMeshLODs( float Seconds ) { return false; }

	/**
	 * Register a one-time callback that will be called when criteria met
	 * @param Callback
	 * @param LODIdx		The LOD index expected. Specify -1 for the MinLOD.
	 * @param TimeoutSecs	Timeout in seconds
	 * @param bOnStreamIn	To get notified when the expected LOD is streamed in or out
	 */
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& Callback, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn);
	/**
	 * Register a one-time callback that will be called when streaming starts or ends.
	 * @param CallbackStreamingStart	The callback to notify when streaming new LODs in begins. The callback will not always be called if the asset is not streamable, or the asset or component is unloaded.
	 * @param CallbackStreamingDone		The callback to notify when streaming is done. The callback will not be called if the start timeout expired.
	 * @param TimeoutStartSecs			Timeout for streaming to start, in seconds
	 * @param TimeoutDoneSecs			Timeout for streaming to end, in seconds
	 */
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& CallbackStreamingStart, FLODStreamingCallback&& CallbackStreamingDone, float TimeoutStartSecs, float TimeoutDoneSecs);

	/** Get the material info for texture streaming. Return whether the data is valid or not. */
	virtual bool GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const { return false; }

	/** Generate streaming data for all materials. */
	ENGINE_API void GetStreamingTextureInfoInner(FStreamingTextureLevelContext& LevelContext, const TArray<FStreamingTextureBuildInfo>* PreBuiltData, float ComponentScaling, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures) const;

	/** Returns the wireframe color to use for this component. */
	ENGINE_API FColor GetWireframeColorForSceneProxy() const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/**
	 * Output to the log which materials and textures are used by this component.
	 * @param Indent	Number of tabs to put before the log.
	 */
	ENGINE_API virtual void LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const;
#endif

	/** Get the mesh paint texture set on this component. This does not take into account any transient override. */
	virtual UTexture* GetMeshPaintTexture() const { return nullptr; }
	/** Set the mesh paint texture on this component. */
	virtual void SetMeshPaintTexture(UTexture* Texture) {}
	/** Set a transient override mesh paint texture on this component. */
	virtual void SetMeshPaintTextureOverride(UTexture* OverrideTexture) {}
	/** Get the default coordinate index for painting to the mesh paint texture on this component. */
	virtual int32 GetMeshPaintTextureCoordinateIndex() const { return 0; }

public:

	/** Set all occurrences of Scalar Material Parameters with ParameterName in the set of materials of the SkeletalMesh to ParameterValue */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetScalarParameterValueOnMaterials(const FName ParameterName, const float ParameterValue);

	/** Set all occurrences of Vector Material Parameters with ParameterName in the set of materials of the SkeletalMesh to ParameterValue */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetVectorParameterValueOnMaterials(const FName ParameterName, const FVector ParameterValue);

	/** Set all occurrences of Vector Material Parameters with ParameterName in the set of materials of the SkeletalMesh to ParameterValue */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetColorParameterValueOnMaterials(const FName ParameterName, const FLinearColor ParameterValue);

	/**  
	 * Returns default value for the parameter input. 
	 *
	 * NOTE: This is not reliable when cooking, as initializing the default value 
	 *       requires a render resource that only exists if the owning world is rendering.
	 */
	float GetScalarParameterDefaultValue(const FName ParameterName)
	{
		FMaterialParameterCache* ParameterCache = MaterialParameterCache.Find(ParameterName);
		return (ParameterCache ? ParameterCache->ScalarParameterDefaultValue : 0.f);
	}

	/** Retrieve the material slots overly materials assigned to this component */
	ENGINE_API const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const;

	/**
	 * Get all default material slots overlay materials from the mesh.
	 */
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const { return; }

protected:
	//~ Begin UObject Interface.
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

	/** Get the default overlay material used by a mesh */
	virtual UMaterialInterface* GetDefaultOverlayMaterial() const { return nullptr; };
	
	/** Get the default overlay material max draw distance */
	virtual float GetDefaultOverlayMaterialMaxDrawDistance() const { return 0.f; };

	/** Retrieves all the (scalar/vector-)parameters from within the used materials on the SkeletalMesh, and stores material index vs parameter names */
	ENGINE_API void CacheMaterialParameterNameIndices();

	/** Mark cache parameters map as dirty, cache will be rebuild once SetScalar/SetVector functions are called */
	ENGINE_API void MarkCachedMaterialParameterNameIndicesDirty();

	/** Struct containing information about a given parameter name */
	struct FMaterialParameterCache
	{
		/** Material indices for the retrieved scalar material parameter names */
		TArray<int32> ScalarParameterMaterialIndices;
		/** Material indices for the retrieved vector material parameter names */
		TArray<int32> VectorParameterMaterialIndices;
		/** Material default parameter for the scalar parameter
		 * We only cache the last one as we can't trace back from [name, index] 
		 * This data is used for animation system to set default back to it*/
		float ScalarParameterDefaultValue = 0.f;
	};

	ENGINE_API void SetParameterValueOnMaterials(const FName ParameterName
		, const TFunction<void(UMaterialInstanceDynamic* DynamicMaterial)>& SetParameterFunction
		, const TFunction<const TArray<int32>& (const FMaterialParameterCache* ParameterCache)>& GetMaterialIndicesFromCacheFunction);

public:

	/** Whether or not to cache material parameter to speed up setting scalar or vector value on materials */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = MaterialParameters)
	uint8 bEnableMaterialParameterCaching : 1;

protected:

	/** Flag whether or not the cached material parameter indices map is dirty (defaults to true, and is set from SetMaterial/Set(Skeletal)Mesh */
	uint8 bCachedMaterialParameterIndicesAreDirty : 1;

	TSortedMap<FName, FMaterialParameterCache, FDefaultAllocator, FNameFastLess> MaterialParameterCache;
	TSortedMap<FName, FMaterialParameterCache, FDefaultAllocator, FNameFastLess> MaterialSlotsOverlayMaterialParameterCache;
};
