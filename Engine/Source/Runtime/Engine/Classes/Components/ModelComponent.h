// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "UObject/ObjectMacros.h"
#include "Model.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/PrimitiveComponent.h"
#include "ModelComponent.generated.h"

class FLightingBuildOptions;
class FPrimitiveSceneProxy;
class ULightComponent;
class UMaterialInterface;
struct FStaticLightingPrimitiveInfo;

//
// Forward declarations
//
class FModelElement;

/**
 * ModelComponents are PrimitiveComponents that represent elements of BSP geometry in a ULevel object.
 * They are used exclusively by ULevel and are not intended as general-purpose components.
 *
 * @see ULevel
 */
UCLASS(MinimalAPI)
class UModelComponent : public UPrimitiveComponent, public IInterface_CollisionDataProvider
{
	GENERATED_UCLASS_BODY()

private:
	/** The BSP tree. */
	TObjectPtr<class UModel> Model;

	/** The index of this component in the ULevel's ModelComponents array. */
	int32 ComponentIndex;

public:

	/** Description of collision */
	UPROPERTY()
	TObjectPtr<class UBodySetup> ModelBodySetup;

private:
	/** The nodes which this component renders. */
	TArray<uint16> Nodes;

	/** The elements used to render the nodes. */
	TIndirectArray<FModelElement> Elements;

public:

#if WITH_EDITOR
	/**
	 * Minimal initialization.
	 */
	void InitializeModelComponent(UModel* InModel, uint16 InComponentIndex, uint32 MaskedSurfaceFlags, const TArray<uint16>& InNodes);
#endif // WITH_EDITOR

	/**
	 * Commit the editor's changes to BSP surfaces.  Reconstructs rendering info based on the new surface data.
	 * The model should not be attached when this is called.
	 */
	void CommitSurfaces();

	/**
	 * Rebuild the model's rendering info.
	 */
	void BuildRenderData();

	/**
	 * Free empty elements.
	 */
	void ShrinkElements();

	/** Calculate the lightmap resolution to be used by the given surface. */
	ENGINE_API void GetSurfaceLightMapResolution( int32 SurfaceIndex, int32 QualityScale, int32& Width, int32& Height, FMatrix& WorldToMap, TArray<int32>* GatheredNodes=NULL ) const;

	/** Copy model elements from the other component. This is used when duplicating components for PIE to guarantee correct rendering. */
	ENGINE_API void CopyElementsFrom(UModelComponent* OtherModelComponent);

	//~ Begin UPrimitiveComponent Interface.
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual bool GetLightMapResolution( int32& Width, int32& Height ) const override;
	virtual int32 GetStaticLightMapResolution() const override;
	virtual void GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool ShouldRecreateProxyOnUpdateTransform() const override;
#if WITH_EDITOR
	virtual void GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options) override;
	virtual void AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const override;
#endif
	virtual ELightMapInteractionType GetStaticLightingType() const override	{ return LMIT_Texture;	}
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	virtual class UBodySetup* GetBodySetup() override { return ModelBodySetup; };
	virtual int32 GetNumMaterials() const override;
	virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	virtual UMaterialInterface* GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const override;
	virtual bool IsPrecomputedLightingValid() const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UActorComponent Interface.
	virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
	virtual void PropagateLightingScenarioChange() override;
	//~ End UActorComponent Interface.

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR
	virtual bool IsNameStableForNetworking() const override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.

	//~ Begin Interface_CollisionDataProvider Interface
	virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool WantsNegXTriMesh() override { return false; }
	//~ End Interface_CollisionDataProvider Interface


	/** Ensure ModelBodySetup is present and correctly configured. */
	void CreateModelBodySetup();

#if WITH_EDITOR
	/** Selects all surfaces that are part of this model component. */
	ENGINE_API void SelectAllSurfaces();

	/** Invalidate current collision data */
	void InvalidateCollisionData();

	/**
	 *	Generate the Elements array.
	 *
	 *	@param	bBuildRenderData	If true, build render data after generating the elements.
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	virtual bool GenerateElements(bool bBuildRenderData);
#endif // WITH_EDITOR

	//
	// Accessors.
	//

	UModel* GetModel() const { return Model; }
	const TIndirectArray<FModelElement>& GetElements() const { return Elements; }
	TIndirectArray<FModelElement>& GetElements() { return Elements; }

	/**
	 * Create a new temp FModelElement element for the component, which will be applied
	 * when lighting is all done.
	 *
	 * @param Component The UModelComponent to make the temp element in
	 */
	static FModelElement* CreateNewTempElement(UModelComponent* Component);

	/**
	 * Apply all the elements that we were putting into the TempBSPElements map to
	 * the Elements arrays in all components
	 *
	 * @param bLightingWasSuccessful If true, the lighting should be applied, otherwise, the temp lighting should be cleaned up
	 */
	ENGINE_API static void ApplyTempElements(bool bLightingWasSuccessful);


private:
	/** The new BSP elements that are made during lighting, and will be applied to the components when all lighting is done */
	static TMap<UModelComponent*, TIndirectArray<FModelElement> > TempBSPElements;

	friend void SetDebugLightmapSample(TArray<UActorComponent*>* Components, UModel* Model, int32 iSurf, FVector ClickLocation);
	friend class UModel;
	friend class FStaticLightingSystem;

protected:
	/** Whether the component type supports static lighting. */
	virtual bool SupportsStaticLighting() const override
	{
		return true;
	}
};


