// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "PawnNoiseEmitterComponent.generated.h"

class AActor;

/**
 * PawnNoiseEmitterComponent tracks noise event data used by SensingComponents to hear a Pawn.
 * This component is intended to exist on either a Pawn or its Controller. It does nothing on network clients.
 */
UCLASS(ClassGroup=AI, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPawnNoiseEmitterComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

protected:
	// If set to true (default value) will notify AIPerceptionSystem about noise events
	// otherwise only PawnSensingComponents will be able to pick up noises generated by this component
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", AdvancedDisplay)
	uint32 bAIPerceptionSystemCompatibilityMode : 1;

public:
	// Most recent noise made by this pawn not at its own location
	UPROPERTY()
	FVector LastRemoteNoisePosition;

	// After this amount of time, new sound events will overwrite previous sounds even if they are not louder (allows old sounds to decay)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Noise Settings")
	float NoiseLifetime;

	/** Cache noises instigated by the owning pawn for AI sensing 
	  *  @param NoiseMaker - is the actual actor which made the noise
	  *  @param Loudness - is the relative loudness of the noise (0.0 to 1.0)
	  *  @param NoiseLocation - is the position of the noise
	  */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Audio|Components|PawnNoiseEmitter")
	ENGINE_API virtual void MakeNoise(AActor* NoiseMaker, float Loudness, const FVector& NoiseLocation);

	ENGINE_API float GetLastNoiseVolume(bool bSourceWithinNoiseEmitter) const;
	ENGINE_API float GetLastNoiseTime(bool bSourceWithinNoiseEmitter) const;

private:
		// Most recent volume of noise made by this pawn not at its own location
	UPROPERTY()
	float LastRemoteNoiseVolume;

	// Time of last remote noise update
	UPROPERTY()
	float LastRemoteNoiseTime;

	// Most recent noise made by this pawn at its own location
	UPROPERTY()
	float LastLocalNoiseVolume;

	// Time of last local noise update
	UPROPERTY()
	float LastLocalNoiseTime;
};

