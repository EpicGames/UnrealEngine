// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RenderCommandFence.h"
#include "Components/SceneCaptureComponent.h"
#include "PlanarReflectionComponent.generated.h"

/**
 *	UPlanarReflectionComponent
 */
UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent), ClassGroup=Rendering, MinimalAPI, editinlinenew, meta=(BlueprintSpawnableComponent))
class UPlanarReflectionComponent : public USceneCaptureComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UBoxComponent> PreviewBox;

public:

	/** Controls the strength of normals when distorting the planar reflection. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "1000.0"))
	float NormalDistortionStrength;

	/** The roughness value to prefilter the planar reflection texture with, useful for hiding low resolution.  Larger values have larger GPU cost. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = ".04"))
	float PrefilterRoughness;

	/** The distance at which the prefilter roughness value will be achieved. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "100000"), AdvancedDisplay)
	float PrefilterRoughnessDistance;

	/** Downsample percent, can be used to reduce GPU time rendering the planar reflection. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "25", UIMax = "100"), AdvancedDisplay)
	int32 ScreenPercentage;

	/** 
	 * Additional FOV used when rendering to the reflection texture.  
	 * This is useful when normal distortion is causing reads outside the reflection texture. 
	 * Larger values increase rendering thread and GPU cost, as more objects and triangles have to be rendered into the planar reflection.
	 */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "10.0"), AdvancedDisplay)
	float ExtraFOV;

	UPROPERTY()
	float DistanceFromPlaneFadeStart_DEPRECATED;

	UPROPERTY()
	float DistanceFromPlaneFadeEnd_DEPRECATED;

	/** Receiving pixels at this distance from the reflection plane will begin to fade out the planar reflection. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "1500.0"))
	float DistanceFromPlaneFadeoutStart;

	/** Receiving pixels at this distance from the reflection plane will have completely faded out the planar reflection. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "1500.0"))
	float DistanceFromPlaneFadeoutEnd;

	/** Receiving pixels whose normal is at this angle from the reflection plane will begin to fade out the planar reflection. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = "0", UIMax = "90.0"))
	float AngleFromPlaneFadeStart;

	/** Receiving pixels whose normal is at this angle from the reflection plane will have completely faded out the planar reflection. */
	UPROPERTY(EditAnywhere, Category = PlanarReflection, meta = (UIMin = "0", UIMax = "90.0"))
	float AngleFromPlaneFadeEnd;

	UPROPERTY(EditAnywhere, Category = PlanarReflection)
	bool bShowPreviewPlane;

	/** 
	 * Whether to render the scene as two-sided, which can be useful to hide artifacts where normal distortion would read 'under' an object that has been clipped by the reflection plane. 
	 * With this setting enabled, the backfaces of a mesh would be displayed in the clipped region instead of the background which is potentially a bright sky.
	 * Be sure to add the water plane to HiddenActors if enabling this, as the water plane will now block the reflection.
	 */
	UPROPERTY(EditAnywhere, Category = PlanarReflection, AdvancedDisplay)
	bool bRenderSceneTwoSided;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void FinishDestroy() override;
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void SendRenderTransform_Concurrent() override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual void OnRegister() override;
	//~ Begin UActorComponent Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void UpdatePreviewShape();

	void GetProjectionWithExtraFOV(FMatrix& OutMatrix, const uint32 StereoIndex) const
	{
		check(StereoIndex < 2);
		OutMatrix = ProjectionWithExtraFOV[StereoIndex];
	}

	int32 GetPlanarReflectionId() const
	{
		return PlanarReflectionId;
	}

	bool ShouldComponentAddToScene() const;

protected:
#if WITH_EDITORONLY_DATA
	/** The path for the material to use on ProxyMeshComponent */
	FSoftObjectPath CaptureMaterialPath;

#endif

private:

	/** Fence used to track progress of releasing resources on the rendering thread. */
	FRenderCommandFence ReleaseResourcesFence;

	class FPlanarReflectionSceneProxy* SceneProxy;

	class FPlanarReflectionRenderTarget* RenderTarget;

	FMatrix ProjectionWithExtraFOV[2];

	int32 PlanarReflectionId;

	friend class FScene;
};
