// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ReflectionCaptureComponent.h"
#include "PlaneReflectionCaptureComponent.generated.h"

	// -> will be exported to EngineDecalClasses.h
UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent, RuntimeCapture), MinimalAPI)
class UPlaneReflectionCaptureComponent : public UReflectionCaptureComponent
{
	GENERATED_UCLASS_BODY()

	/** Radius of the area that can receive reflections from this capture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReflectionCapture, meta=(UIMin = "8.0", UIMax = "16384.0"))
	float InfluenceRadiusScale;

	UPROPERTY()
	TObjectPtr<class UDrawSphereComponent> PreviewInfluenceRadius;

	UPROPERTY()
	TObjectPtr<class UBoxComponent> PreviewCaptureBox;

public:
	virtual void UpdatePreviewShape() override;
	virtual float GetInfluenceBoundingRadius() const override;

	//~ Begin UObject Interface
	//~ End UObject Interface
};

