// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "PlatformEventsComponent.generated.h"

/**
 * Component to handle receiving notifications from the OS about platform events.
 */
UCLASS(ClassGroup=Utility, HideCategories=(Activation, "Components|Activation", Collision), meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPlatformEventsComponent
	: public UActorComponent
{
	GENERATED_UCLASS_BODY()

	/**
	 * Check whether a convertible laptop is laptop mode.
	 *
	 * @return true if in laptop mode, false otherwise or if not a convertible laptop.
	 * @see IsInTabletMode, SupportsConvertibleLaptops
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation")
	ENGINE_API bool IsInLaptopMode();

	/**
	 * Check whether a convertible laptop is laptop mode.
	 *
	 * @return true if in tablet mode, false otherwise or if not a convertible laptop.
	 * @see IsInLaptopMode, SupportsConvertibleLaptops
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation")
	ENGINE_API bool IsInTabletMode();

	/**
	 * Check whether the platform supports convertible laptops.
	 *
	 * Note: This does not necessarily mean that the platform is a convertible laptop.
	 * For example, convertible laptops running Windows 7 or older will return false,
	 * and regular laptops running Windows 8 or newer will return true.
	 *
	 * @return true for convertible laptop platforms, false otherwise.
	 * @see IsInLaptopMode, IsInTabletMode
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Activation")
	ENGINE_API bool SupportsConvertibleLaptops();

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FPlatformEventDelegate);

	/** This is called when a convertible laptop changed into laptop mode. */
	UPROPERTY(BlueprintAssignable)
	FPlatformEventDelegate PlatformChangedToLaptopModeDelegate;  

	/** This is called when a convertible laptop changed into tablet mode. */
	UPROPERTY(BlueprintAssignable)
	FPlatformEventDelegate PlatformChangedToTabletModeDelegate;  

public:

	// UActorComponent overrides

	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;

private:

	/** Handles the FCoreDelegates.PlatformChangedLaptopMode delegate. */
	void HandlePlatformChangedLaptopMode(EConvertibleLaptopMode NewMode);
};
