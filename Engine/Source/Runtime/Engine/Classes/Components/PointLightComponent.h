// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "PointLightSceneProxyDesc.h"
#include "Components/LocalLightComponent.h"
#include "PointLightComponent.generated.h"

class FLightSceneProxy;

/**
 * A light component which emits light from a single point equally in all directions.
 */
UCLASS(Blueprintable, ClassGroup=(Lights,Common), hidecategories=(Object, LightShafts), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPointLightComponent : public ULocalLightComponent
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Whether to use physically based inverse squared distance falloff, where AttenuationRadius is only clamping the light's contribution.  
	 * Disabling inverse squared falloff can be useful when placing fill lights (don't want a super bright spot near the light).
	 * When enabled, the light's Intensity is in units of lumens, where 1700 lumens is a 100W lightbulb.
	 * When disabled, the light's Intensity is a brightness scale.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	uint32 bUseInverseSquaredFalloff:1;

	/**
	 * Controls the radial falloff of the light when UseInverseSquaredFalloff is disabled. 
	 * 2 is almost linear and very unrealistic and around 8 it looks reasonable.
	 * With large exponents, the light has contribution to only a small area of its influence radius but still costs the same as low exponents.
	 */
	UPROPERTY(interp, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(UIMin = "2.0", UIMax = "16.0"))
	float LightFalloffExponent;

	/** 
	 * Radius of light source shape.
	 * Note that light sources shapes which intersect shadow casting geometry can cause shadowing artifacts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.0", UIMax = "1000.0", ClampMax = "10000", ShouldShowInViewport = true))
	float SourceRadius;

	/**
	* Soft radius of light source shape.
	* Note that light sources shapes which intersect shadow casting geometry can cause shadowing artifacts.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = Light, meta=(UIMin = "0.0", UIMax = "1000.0", ClampMax = "10000"))
	float SoftSourceRadius;

	/** 
	 * Length of light source shape.
	 * Note that light sources shapes which intersect shadow casting geometry can cause shadowing artifacts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.0", UIMax = "1000.0", ClampMax = "10000"))
	float SourceLength;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetUseInverseSquaredFalloff(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetLightFalloffExponent(float NewLightFalloffExponent);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetInverseExposureBlend(float NewInverseExposureBlend);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetSourceRadius(float bNewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetSoftSourceRadius(float bNewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetSourceLength(float NewValue);

public:

	ENGINE_API virtual float ComputeLightBrightness() const override;
#if WITH_EDITOR
	ENGINE_API virtual void SetLightBrightness(float InBrightness) override;
#endif

	//~ Begin ULightComponent Interface.
	ENGINE_API virtual ELightComponentType GetLightType() const override;
	ENGINE_API virtual float GetUniformPenumbraSize() const override;
	ENGINE_API virtual FLightSceneProxy* CreateSceneProxy() const override;
	ENGINE_API virtual void PrecachePSOs() override;

	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	inline static FPointLightSceneProxyDesc BuildSceneProxyDesc(const UPointLightComponent& InLightComponent)
	{
		FPointLightSceneProxyDesc ProxyDesc;
		BuildSceneProxyDesc(InLightComponent, ProxyDesc);
		return ProxyDesc;
	}

protected:

	ENGINE_API static void BuildSceneProxyDesc(const UPointLightComponent& InLightComponent, FPointLightSceneProxyDesc& OutProxyDesc);
};