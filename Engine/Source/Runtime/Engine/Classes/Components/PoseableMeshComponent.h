// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Components/SkinnedMeshComponent.h"
#include "PoseableMeshComponent.generated.h"

class USkeletalMeshComponent;

/**
 *	UPoseableMeshComponent that allows bone transforms to be driven by blueprint.
 */
UCLASS(ClassGroup=Rendering, hidecategories=(Object,Physics), config=Engine, editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPoseableMeshComponent : public USkinnedMeshComponent
{
	GENERATED_UCLASS_BODY()

	// we need this for template function in the cpp
	// skeletalmeshcomponent hides this property, so now creating extra getter for poseablemeshcomponent
	// although it's fine for poseablemeshcomponent to have this as external
	// note that their signature is different, but the template would workf ine
	const TArray<FTransform>& GetBoneSpaceTransforms() const { return BoneSpaceTransforms; }

	/** Temporary array of local-space (ie relative to parent bone) rotation/translation/scale for each bone. */
	TArray<FTransform> BoneSpaceTransforms;
public:

	FBoneContainer RequiredBones;

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void SetBoneTransformByName(FName BoneName, const FTransform& InTransform, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void SetBoneLocationByName(FName BoneName, FVector InLocation, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void SetBoneRotationByName(FName BoneName, FRotator InRotation, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void SetBoneScaleByName(FName BoneName, FVector InScale3D, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh") 
	ENGINE_API FTransform GetBoneTransformByName(FName BoneName, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API FVector GetBoneLocationByName(FName BoneName, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API FRotator GetBoneRotationByName(FName BoneName, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API FVector GetBoneScaleByName(FName BoneName, EBoneSpaces::Type BoneSpace);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void ResetBoneTransformByName(FName BoneName);

	UFUNCTION(BlueprintCallable, Category="Components|PoseableMesh")
	ENGINE_API void CopyPoseFromSkeletalComponent(USkeletalMeshComponent* InComponentToCopy);

	//~ Begin USkinnedMeshComponent Interface
	ENGINE_API virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = NULL) override;
	ENGINE_API virtual bool AllocateTransformData() override;
	ENGINE_API virtual bool ShouldUpdateTransform(bool bLODHasChanged) const override;
	//~ End USkinnedMeshComponent Interface

	bool IsRunningParallelEvaluation() const { return false; }
	/**
	 * Take the BoneSpaceTransforms array (translation vector, rotation quaternion and scale vector) and update the array of component-space bone transformation matrices (SpaceBases).
	 * It will work down hierarchy multiplying the component-space transform of the parent by the relative transform of the child.
	 * This code also applies any per-bone rotators etc. as part of the composition process
	 */
	ENGINE_API void FillComponentSpaceTransforms();

	ENGINE_API void MarkRefreshTransformDirty();
private:
	// this is marked if transform has to be updated
	bool bNeedsRefreshTransform;
};



