// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "Components/SceneComponent.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Scene.h"
#include "Components/ShapeComponent.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "PostProcessComponent.generated.h"

struct FVolumeBoundsMesh;

/**
 *  PostProcessComponent. Enables Post process controls for blueprints.
 *	Will use a parent UShapeComponent or VolumeBoundsMesh to provide volume data if available.
 */

UCLASS(ClassGroup = Rendering, collapsecategories, hidecategories = (Object), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UPostProcessComponent : public USceneComponent, public IInterface_PostProcessVolume
{
	GENERATED_UCLASS_BODY()

	/** Post process settings to use for this volume. */
	UPROPERTY(interp, Category = PostProcessVolume, meta = (ShowOnlyInnerProperties))
	struct FPostProcessSettings Settings;

	/**
	 * Priority of this volume. In the case of overlapping volumes the one with the highest priority
	 * overrides the lower priority ones. The order is undefined if two or more overlapping volumes have the same priority.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessVolume)
	float Priority;

	/** World space radius around the volume that is used for blending (only if not unbound).			*/
	UPROPERTY(interp, Category = PostProcessVolume, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "6000.0"))
	float BlendRadius;

	/** 0:no effect, 1:full effect */
	UPROPERTY(interp, Category = PostProcessVolume, BlueprintReadWrite, meta = (UIMin = "0.0", UIMax = "1.0"))
	float BlendWeight;

	/** Whether this volume is enabled or not.															*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessVolume)
	uint32 bEnabled: 1;

	/** set this to false to use the parent shape component or VolumeBoundsMesh as volume bounds. True affects the whole world regardless.		*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessVolume)
	uint32 bUnbound: 1;
 
	/** Automatically generated GUID for this post process volume, used for deterministic sorting of equal priority volumes. */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient, NonTransactional)
	FGuid VolumeGuid;

	// If set, this mesh will be used as bounds for the volume. 
	TPimplPtr<FVolumeBoundsMesh> VolumeBoundsMesh;
	
	TOptional<FBoxSphereBounds> GetPostProcessBounds() const;

	//~ Begin IInterface_PostProcessVolume Interface
	ENGINE_API virtual bool EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint) override;
	virtual FPostProcessVolumeProperties GetProperties() const override
	{
		TOptional<FBoxSphereBounds> PPBounds = GetPostProcessBounds();

		FPostProcessVolumeProperties Ret;
		Ret.bIsEnabled = bEnabled != 0 && ShouldRender();
		Ret.bIsUnbound = bUnbound != 0 || !PPBounds.IsSet();
		Ret.BlendRadius = BlendRadius;
		Ret.BlendWeight = BlendWeight;
		Ret.Priority = Priority;
		Ret.Settings = &Settings;
		Ret.Size = PPBounds.IsSet() ? PPBounds->GetBox().GetVolume() : DBL_MAX;
		Ret.VolumeGuid = VolumeGuid;
		return Ret;
	}
#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	virtual FString GetDebugName() const override
	{
		return GetName();
	}
#endif
	//~ End IInterface_PostProcessVolume Interface

	/** Adds an Blendable (implements IBlendableInterface) to the array of Blendables (if it doesn't exist) and update the weight */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void AddOrUpdateBlendable(TScriptInterface<IBlendableInterface> InBlendableObject, float InWeight = 1.0f) { Settings.AddBlendable(InBlendableObject, InWeight); }

protected:

	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostInitProperties() override;
	//~ End UObject Interface
};



