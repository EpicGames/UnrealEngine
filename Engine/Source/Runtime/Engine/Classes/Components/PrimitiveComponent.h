// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/Copy.h"
#include "EngineStats.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/Guid.h"
#include "Interfaces/IPhysicsComponent.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/ScopedMovementUpdate.h"
#include "Engine/OverlapInfo.h"
#include "Components/SceneComponent.h"
#include "Components/ActorPrimitiveComponentInterface.h"
#include "Streaming/SimpleStreamableAssetManagerHandle.h"
#include "RenderCommandFence.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "SceneTypes.h"
#include "Engine/EngineTypes.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Engine/TextureStreamingTypes.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "HitProxies.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "HLOD/HLODBatchingPolicy.h"
#include "HLOD/HLODLevelExclusion.h"
#include "Stats/Stats.h"
#include "PSOPrecacheFwd.h"
#include "PSOPrecacheComponent.h"
#include "PipelineStateCache.h"
#include "MeshDrawCommandStatsDefines.h"
#include "PrimitiveSceneInfoData.h"
#include "Streaming/RenderAssetOwnerStreamingState.h"
#include "Materials/MaterialParameters.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "PrimitiveComponent.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("BeginComponentOverlap"), STAT_BeginComponentOverlap, STATGROUP_Game, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("MoveComponent FastOverlap"), STAT_MoveComponent_FastOverlap, STATGROUP_Game, ENGINE_API);

class AController;
class APlayerController;
class FPrimitiveSceneProxy;
class UMaterialInterface;
class UPhysicalMaterial;
class UPrimitiveComponent;
class UTexture;
class URuntimeVirtualTexture;
struct FCollisionShape;
struct FConvexVolume;
struct FEngineShowFlags;
struct FNavigableGeometryExport;
struct FPSOPrecacheParams;
struct FOverlapResult;

enum class EObjectMigrationSendType : uint8;

/**
 * Selects which families of touch component events UPrimitiveComponent::DispatchTouchHitResult should fire.
 * BeginEnd mirrors APlayerController::bEnableTouchEvents; Over mirrors bEnableTouchOverEvents.
 */
enum class ETouchEventFamilies : uint8
{
	None = 0,
	BeginEnd = 1 << 0,
	Over = 1 << 1,
	All = BeginEnd | Over,
};
ENUM_CLASS_FLAGS(ETouchEventFamilies);

namespace PrimitiveComponentCVars
{
	extern float HitDistanceToleranceCVar;
	extern float InitialOverlapToleranceCVar;
	ENGINE_API extern int32 bAllowCachedOverlapsCVar;
	ENGINE_API extern int32 bEnableFastOverlapCheck;
}

/** Determines whether a Character can attempt to step up onto a component when they walk in to it. */
UENUM()
enum ECanBeCharacterBase : int
{
	/** Character cannot step up onto this Component. */
	ECB_No UMETA(DisplayName="No"),
	/** Character can step up onto this Component. */
	ECB_Yes UMETA(DisplayName="Yes"),
	/**
	 * Owning actor determines whether character can step up onto this Component (default true unless overridden in code).
	 * @see AActor::CanBeBaseForCharacter()
	 */
	ECB_Owner UMETA(DisplayName="(Owner)"),
	ECB_MAX,
};

/** Information about the sprite category, used for visualization in the editor */
USTRUCT()
struct FSpriteCategoryInfo
{
	GENERATED_BODY()

	/** Sprite category that the component belongs to */
	UPROPERTY()
	FName Category;

	/** Localized name of the sprite category */
	UPROPERTY()
	FText DisplayName;

	/** Localized description of the sprite category */
	UPROPERTY()
	FText Description;
};

/** Exposed enum to parallel RHI's EStencilMask and show up in the editor. Has a paired struct to convert between the two. */
UENUM()
enum class ERendererStencilMask : uint8
{
	ERSM_Default UMETA(DisplayName = "Default"),
	ERSM_255 UMETA(DisplayName = "All bits (255), ignore depth"),
	ERSM_1 UMETA(DisplayName = "First bit (1), ignore depth"),
	ERSM_2 UMETA(DisplayName = "Second bit (2), ignore depth"),
	ERSM_4 UMETA(DisplayName = "Third bit (4), ignore depth"),
	ERSM_8 UMETA(DisplayName = "Fourth bit (8), ignore depth"),
	ERSM_16 UMETA(DisplayName = "Fifth bit (16), ignore depth"),
	ERSM_32 UMETA(DisplayName = "Sixth bit (32), ignore depth"),
	ERSM_64 UMETA(DisplayName = "Seventh bit (64), ignore depth"),
	ERSM_128 UMETA(DisplayName = "Eighth bit (128), ignore depth")
};

/** How quickly component should be culled. */
UENUM()
enum class ERayTracingGroupCullingPriority : uint8
{
	CP_0_NEVER_CULL UMETA(DisplayName = "0 - Never cull"),
	CP_1 UMETA(DisplayName = "1"),
	CP_2 UMETA(DisplayName = "2"),
	CP_3 UMETA(DisplayName = "3"),
	CP_4_DEFAULT UMETA(DisplayName = "4 - Default"),
	CP_5 UMETA(DisplayName = "5"),
	CP_6 UMETA(DisplayName = "6"),
	CP_7 UMETA(DisplayName = "7"),
	CP_8_QUICKLY_CULL UMETA(DisplayName = "8 - Quickly cull")
};

/** Determines how a primitive interacts with first person rendering. */
UENUM()
enum class EFirstPersonPrimitiveType : uint8
{
	/** Primitive does not interact with first person rendering. */
	None UMETA(DisplayName = "None"),
	/** Primitive is rendered as first person and affected by first person properties on the camera. */
	FirstPerson UMETA(DisplayName = "First Person"),
	/** Primitive represents a first person primitive in world space. This is the primitive that other players would see and is used to cast shadow onto the ground among other things.
		Implicitly sets bCastHiddenShadow=false and bOwnerNoSee=true internally, which is required for first person shadow to work correctly with VSM. */
	WorldSpaceRepresentation UMETA(DisplayName = "World Space Representation"),
};

/** Converts a stencil mask from the editor's USTRUCT version to the version the renderer uses. */
struct FRendererStencilMaskEvaluation
{
	static inline EStencilMask ToStencilMask(const ERendererStencilMask InEnum)
	{
		switch (InEnum)
		{
		case ERendererStencilMask::ERSM_Default:
			return EStencilMask::SM_Default;
		case ERendererStencilMask::ERSM_255:
			return EStencilMask::SM_255;
		case ERendererStencilMask::ERSM_1:
			return EStencilMask::SM_1;
		case ERendererStencilMask::ERSM_2:
			return EStencilMask::SM_2;
		case ERendererStencilMask::ERSM_4:
			return EStencilMask::SM_4;
		case ERendererStencilMask::ERSM_8:
			return EStencilMask::SM_8;
		case ERendererStencilMask::ERSM_16:
			return EStencilMask::SM_16;
		case ERendererStencilMask::ERSM_32:
			return EStencilMask::SM_32;
		case ERendererStencilMask::ERSM_64:
			return EStencilMask::SM_64;
		case ERendererStencilMask::ERSM_128:
			return EStencilMask::SM_128;
		default:
			// Unsupported EStencilMask - return a safe default.
			check(false);
			return EStencilMask::SM_Default;
		}
	}
};

// Predicate to determine if an overlap is with a certain AActor.
struct FPredicateOverlapHasSameActor
{
	FPredicateOverlapHasSameActor(const AActor& Owner)
		: MyOwnerPtr(&Owner)
	{
	}

	bool operator() (const FOverlapInfo& Info)
	{
		// since we only care whether the overlaping actor is the same as MyOwnerPtr
		// we only call GetCachedActor (which resolves the stored handle and returns the actor)
		// rather than calling FindActor which could cause actor spawning if the entity hit
		// is an instanced actor. We obviously don't need to spawn an actor to determine 
		// that it won't be the same as MyOwnerPtr (which has already been spawned).
		const AActor* HitActor = Info.OverlapInfo.HitObjectHandle.GetCachedActor();
		// MyOwnerPtr is always valid, so we don't need the IsValid() checks in the WeakObjectPtr comparison operator.
		return HitActor && MyOwnerPtr.HasSameIndexAndSerialNumber(HitActor);
	}

private:
	const TWeakObjectPtr<const AActor> MyOwnerPtr;
};

// Predicate to determine if an overlap is *NOT* with a certain AActor.
struct FPredicateOverlapHasDifferentActor
{
	FPredicateOverlapHasDifferentActor(const AActor& Owner)
		: MyOwnerPtr(&Owner)
	{
	}

	bool operator() (const FOverlapInfo& Info)
	{
		// since we only care whether the overlaping actor is not the same as MyOwnerPtr
		// we only call GetCachedActor (which resolves the stored handle and returns the actor)
		// rather than calling FindActor which could cause actor spawning if the entity hit
		// is an instanced actor. We obviously don't need to spawn an actor to determine 
		// that it won't be the same as MyOwnerPtr (which has already been spawned).
		const AActor* HitActor = Info.OverlapInfo.HitObjectHandle.GetCachedActor();
		// MyOwnerPtr is always valid, so we don't need the IsValid() checks in the WeakObjectPtr comparison operator.
		return !HitActor || !MyOwnerPtr.HasSameIndexAndSerialNumber(HitActor);
	}

private:
	const TWeakObjectPtr<const AActor> MyOwnerPtr;
};

// TODO: Add sleep and wake state change types to this enum, so that the
// OnComponentWake and OnComponentSleep delegates may be deprecated.
// Doing so would save a couple bytes per primitive component.
UENUM(BlueprintType)
enum class EComponentPhysicsStateChange : uint8
{
	Created,
	Destroyed
};

/** Used to gather information on custom primitive data from the materials. */
struct FCustomPrimitiveDataParameter
{
	TWeakObjectPtr<UPrimitiveComponent> Component;
	TWeakObjectPtr<UMaterialInterface> Material;
	FMaterialParameterInfo Info;
	uint8 IndexOffset;
	EMaterialParameterType Type;
#if WITH_EDITOR
	FGuid ExpressionID;
#endif // WITH_EDITOR
};

/**
 * Delegate for notification of blocking collision against a specific component.  
 * NormalImpulse will be filled in for physics-simulating bodies, but will be zero for swept-component blocking collisions. 
 */
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FiveParams( FComponentHitSignature, UPrimitiveComponent, OnComponentHit, UPrimitiveComponent*, HitComponent, AActor*, OtherActor, UPrimitiveComponent*, OtherComp, FVector, NormalImpulse, const FHitResult&, Hit );
/** Delegate for notification of start of overlap with a specific component */
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_SixParams( FComponentBeginOverlapSignature, UPrimitiveComponent, OnComponentBeginOverlap, UPrimitiveComponent*, OverlappedComponent, AActor*, OtherActor, UPrimitiveComponent*, OtherComp, int32, OtherBodyIndex, bool, bFromSweep, const FHitResult &, SweepResult);
/** Delegate for notification of end of overlap with a specific component */
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FComponentEndOverlapSignature, UPrimitiveComponent, OnComponentEndOverlap, UPrimitiveComponent*, OverlappedComponent, AActor*, OtherActor, UPrimitiveComponent*, OtherComp, int32, OtherBodyIndex);
/** Delegate for notification when a wake event is fired by physics*/
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FComponentWakeSignature, UPrimitiveComponent, OnComponentWake, UPrimitiveComponent*, WakingComponent, FName, BoneName);
/** Delegate for notification when a sleep event is fired by physics*/
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FComponentSleepSignature, UPrimitiveComponent, OnComponentSleep, UPrimitiveComponent*, SleepingComponent, FName, BoneName);
/** Delegate for notification when collision settings change. */
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FComponentCollisionSettingsChangedSignature, UPrimitiveComponent, OnComponentCollisionSettingsChangedEvent, UPrimitiveComponent*, ChangedComponent);
/** Delegate for physics state created */
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FComponentPhysicsStateChanged, UPrimitiveComponent, OnComponentPhysicsStateChanged, UPrimitiveComponent*, ChangedComponent, EComponentPhysicsStateChange, StateChange);

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam( FComponentBeginCursorOverSignature, UPrimitiveComponent, OnBeginCursorOver, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam( FComponentEndCursorOverSignature, UPrimitiveComponent, OnEndCursorOver, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentOnClickedSignature, UPrimitiveComponent, OnClicked, UPrimitiveComponent*, TouchedComponent , FKey, ButtonPressed);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentOnReleasedSignature, UPrimitiveComponent, OnReleased, UPrimitiveComponent*, TouchedComponent, FKey, ButtonReleased);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FComponentOnTouchBeganSignature, UPrimitiveComponent, OnTouchBegan, const FTouchId, TouchId, APlayerController*, Controller, const FHitResult&, HitResult, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FComponentOnTouchEndedSignature, UPrimitiveComponent, OnTouchEnded, const FTouchId, TouchId, APlayerController*, Controller, const FHitResult&, HitResult, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FComponentOnTouchEnteredSignature, UPrimitiveComponent, OnTouchEntered, const FTouchId, TouchId, APlayerController*, Controller, const FHitResult&, HitResult, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FComponentOnTouchExitedSignature, UPrimitiveComponent, OnTouchExited, const FTouchId, TouchId, APlayerController*, Controller, const FHitResult&, HitResult, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentOnInputTouchBeginSignature, UPrimitiveComponent, OnInputTouchBegin, ETouchIndex::Type, FingerIndex, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentOnInputTouchEndSignature, UPrimitiveComponent, OnInputTouchEnd, ETouchIndex::Type, FingerIndex, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentBeginTouchOverSignature, UPrimitiveComponent, OnInputTouchEnter, ETouchIndex::Type, FingerIndex, UPrimitiveComponent*, TouchedComponent );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FComponentEndTouchOverSignature, UPrimitiveComponent, OnInputTouchLeave, ETouchIndex::Type, FingerIndex, UPrimitiveComponent*, TouchedComponent );

/**
 * PrimitiveComponents are SceneComponents that contain or generate some sort of geometry, generally to be rendered or used as collision data.
 * There are several subclasses for the various types of geometry, but the most common by far are the ShapeComponents (Capsule, Sphere, Box), StaticMeshComponent, and SkeletalMeshComponent.
 * ShapeComponents generate geometry that is used for collision detection but are not rendered, while StaticMeshComponents and SkeletalMeshComponents contain pre-built geometry that is rendered, but can also be used for collision detection.
 */
UCLASS(abstract, HideCategories=(Mobility, VirtualTexture), ShowCategories=(PhysicsVolume), MinimalAPI)
class UPrimitiveComponent : public USceneComponent, public INavRelevantInterface, public IInterface_AsyncCompilation, public IPhysicsComponent, public FRenderAssetOwnerStreamingState, public IPhysicsBodyInstanceOwner, public IPhysicsBodyInstanceOwnerResolver
{
	GENERATED_BODY()
	UE_DECLARE_COMPONENT_ACTOR_INTERFACE(PrimitiveComponent)

public:
	/**
	 * Default UObject constructor.
	 */
	ENGINE_API UPrimitiveComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	ENGINE_API UPrimitiveComponent(FVTableHelper& Helper);
	ENGINE_API ~UPrimitiveComponent();

	// Rendering
	static ENGINE_API FName RVTActorDescProperty;

	/**
	 * The minimum distance at which the primitive should be rendered, 
	 * measured in world space units from the center of the primitive's bounding sphere to the camera position.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	float MinDrawDistance;

	float GetMinDrawDistance() const
	{
		return MinDrawDistance;
	}

	/**  Max draw distance exposed to LDs. The real max draw distance is the min (disregarding 0) of this and volumes affecting this object. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD, meta=(DisplayName="Desired Max Draw Distance") )
	float LDMaxDrawDistance;

	/**
	 * The distance to cull this primitive at.  
	 * A CachedMaxDrawDistance of 0 indicates that the primitive should not be culled by distance.
	 */
	UPROPERTY(Category=LOD, AdvancedDisplay, VisibleAnywhere, BlueprintReadOnly, meta=(DisplayName="Current Max Draw Distance") )
	float CachedMaxDrawDistance;

	/** The scene depth priority group to draw the primitive in. */
	UPROPERTY()
	TEnumAsByte<enum ESceneDepthPriorityGroup> DepthPriorityGroup;

	/** The scene depth priority group to draw the primitive in, if it's being viewed by its owner. */
	UPROPERTY()
	TEnumAsByte<enum ESceneDepthPriorityGroup> ViewOwnerDepthPriorityGroup;

	/** Quality of indirect lighting for Movable primitives.  This has a large effect on Indirect Lighting Cache update time. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	TEnumAsByte<EIndirectLightingCacheQuality> IndirectLightingCacheQuality;

	/** Controls the type of lightmap used for this component. */
	UE_DEPRECATED(5.5, "Use GetLightmapType()/SetLightmapType() instead")
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, Getter, Setter,  meta = (AllowPrivateAccess)) 
	ELightmapType LightmapType;
	
	ENGINE_API ELightmapType GetLightmapType() const;
	ENGINE_API void SetLightmapType(ELightmapType InLightmapType);

	/** Determines how the geometry of a component will be incorporated in proxy (simplified) HLODs. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=HLOD, meta=(DisplayName="HLOD Batching Policy", DisplayAfter="bEnableAutoLODGeneration", EditConditionHides, EditCondition="bEnableAutoLODGeneration"))
	EHLODBatchingPolicy HLODBatchingPolicy;

	/** Control shadow invalidation behavior, in particular with respect to Virtual Shadow Maps and material effects like World Position Offset. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(EditCondition="CastShadow"))
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	/** Whether to include this component in HLODs or not. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = HLOD, meta=(DisplayName="Include Component in HLOD"))
	uint8 bEnableAutoLODGeneration : 1;

	/** Indicates that the texture streaming built data is local to the Actor (see UActorTextureStreamingBuildDataComponent). */
	UPROPERTY()
	uint8 bIsActorTextureStreamingBuiltData : 1;

	/** Indicates to the texture streaming wether it can use the pre-built texture streaming data (even if empty). */
	UPROPERTY()
	uint8 bIsValidTextureStreamingBuiltData : 1;

	/** When enabled this object will not be culled by distance. This is ignored if a child of a HLOD. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	uint8 bNeverDistanceCull : 1;

	/** Primitive is part of a batch being bulk reregistered. Applies to UStaticMeshComponent, see FStaticMeshComponentBulkReregisterContext for details. */
	mutable uint8 bBulkReregister : 1;

	FRenderAssetOwnerStreamingState& GetStreamingState() const
	{
		return static_cast<FRenderAssetOwnerStreamingState&>(const_cast<UPrimitiveComponent&>(*this));
	}

	/** 
	 * Indicates if we'd like to create physics state all the time (for collision and simulation). 
	 * If you set this to false, it still will create physics state if collision or simulation activated. 
	 * This can help performance if you'd like to avoid overhead of creating physics state when triggers 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Collision)
	uint8 bAlwaysCreatePhysicsState:1;

	/**
	 * If true, this component will generate overlap events when it is overlapping other components (eg Begin Overlap).
	 * Both components (this and the other) must have this enabled for overlap events to occur.
	 *
	 * @see [Overlap Events](https://docs.unrealengine.com/InteractiveExperiences/Physics/Collision/Overview#overlapandgenerateoverlapevents)
	 * @see UpdateOverlaps(), BeginComponentOverlap(), EndComponentOverlap()
	 */
	UFUNCTION(BlueprintGetter)
	bool GetGenerateOverlapEvents() const;

	/** Modifies value returned by GetGenerateOverlapEvents() */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetGenerateOverlapEvents(bool bInGenerateOverlapEvents);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Components")
	ENGINE_API void SetLightingChannels(bool bChannel0, bool bChannel1, bool bChannel2);

	/** Invalidates Lumen surface cache and forces it to be refreshed. Useful to make material updates more responsive. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void InvalidateLumenSurfaceCache();

private:
	UPROPERTY(EditAnywhere, BlueprintGetter = GetGenerateOverlapEvents, BlueprintSetter = SetGenerateOverlapEvents, Category = Collision)
	uint8 bGenerateOverlapEvents : 1;

public:
	/**
	 * If true, this component will generate individual overlaps for each overlapping physics body if it is a multi-body component. When false, this component will
	 * generate only one overlap, regardless of how many physics bodies it has and how many of them are overlapping another component/body. This flag has no
	 * influence on single body components.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Collision)
	uint8 bMultiBodyOverlap:1;

	/**
	 * If true, component sweeps with this component should trace against complex collision during movement (for example, each triangle of a mesh).
	 * If false, collision will be resolved against simple collision bounds instead.
	 * @see MoveComponent()
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Collision)
	uint8 bTraceComplexOnMove:1;

	/**
	 * If true, component sweeps will return the material in their hit result.
	 * @see MoveComponent(), FHitResult
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Collision)
	uint8 bReturnMaterialOnMove:1;

	/** True if the primitive should be rendered using ViewOwnerDepthPriorityGroup if viewed by its owner. */
	UPROPERTY()
	uint8 bUseViewOwnerDepthPriorityGroup:1;

	/** Whether to accept cull distance volumes to modify cached cull distance. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	uint8 bAllowCullDistanceVolume:1;

	/** If true, this component will be visible in reflection captures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInReflectionCaptures:1;
	
	/** If true, this component will be visible in real-time sky light reflection captures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInRealTimeSkyCaptures :1;

	/** If true, this component will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInRayTracing : 1;

	/** If true, this component will be visible in ray traced reflections. Screen space traces will ignore this flag, as they directly sample SceneColor */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInReflections : 1;

	/** If true, this component will be rendered in the main pass (z prepass, basepass, transparency) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bRenderInMainPass:1;

	/** If true, this component will be rendered in the depth pass even if it's not rendered in the main pass */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta = (EditCondition = "!bRenderInMainPass"))
	uint8 bRenderInDepthPass:1;

	/** Whether the primitive receives decals. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering)
	uint8 bReceivesDecals:1;

	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, Interp)
	uint8 bHoldout : 1;

	/** If this is True, this component won't be visible when the view actor is the component's owner, directly or indirectly. Will be internally set to true when FirstPersonPrimitiveType is set to WorldSpaceRepresentation. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta=(EditCondition="FirstPersonPrimitiveType != EFirstPersonPrimitiveType::WorldSpaceRepresentation"))
	uint8 bOwnerNoSee:1;

	/** If this is True, this component will only be visible when the view actor is the component's owner, directly or indirectly. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bOnlyOwnerSee:1;

	/** Treat this primitive as part of the background for occlusion purposes. This can be used as an optimization to reduce the cost of rendering skyboxes, large ground planes that are part of the vista, etc. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering)
	uint8 bTreatAsBackgroundForOcclusion:1;

	/** 
	 * Whether to render the primitive in the depth only pass.  
	 * This should generally be true for all objects, and let the renderer make decisions about whether to render objects in the depth only pass.
	 * @todo - if any rendering features rely on a complete depth only pass, this variable needs to go away.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering)
	uint8 bUseAsOccluder:1;

	/** If this is True, this component can be selected in the editor. */
	UPROPERTY()
	uint8 bSelectable:1;

	/** When true, this component requests editor effects like outlines and overlays. */
	UPROPERTY()
	uint8 bWantsEditorEffects:1;

#if WITH_EDITORONLY_DATA
	/** If true, this component will be considered for placement when dragging and placing items in the editor even if it is not visible, such as in the case of hidden collision meshes */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Collision)
	uint8 bConsiderForActorPlacementWhenHidden:1;
#endif //WITH_EDITORONLY_DATA

	/** If true, forces mips for textures used by this component to be resident when this component's level is loaded. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=TextureStreaming)
	uint8 bForceMipStreaming:1;

	bool IsMipStreamingForced() const
	{
		return bForceMipStreaming;
	}

	/** If true a hit-proxy will be generated for each instance of instanced static meshes.
	 *  This allows for other systems, like selection in viewports, to function at the individual instance.
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, AdvancedDisplay, Category=Rendering)
	uint8 bHasPerInstanceHitProxies:1;

	// Lighting flags
	
	/** Controls whether the primitive component should cast a shadow or not. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, Interp)
	uint8 CastShadow:1;

	/** Whether the primitive will be used as an emissive light source. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay)
	uint8 bEmissiveLightSource:1;

	/** Controls whether the primitive should influence indirect lighting. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, Interp)
	uint8 bAffectDynamicIndirectLighting:1;

	/** Controls whether the primitive should affect indirect lighting when hidden. This flag is only used if bAffectDynamicIndirectLighting is true. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="bAffectDynamicIndirectLighting", DisplayName = "Affect Indirect Lighting While Hidden"), Interp)
	uint8 bAffectIndirectLightingWhileHidden:1;

	/** Controls whether the primitive should affect dynamic distance field lighting methods.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay)
	uint8 bAffectDistanceFieldLighting:1;

	/** Controls whether the primitive should cast shadows in the case of non precomputed shadowing. This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Dynamic Shadow"))
	uint8 bCastDynamicShadow:1;

	/** Whether the object should cast a static shadow from shadow casting lights. This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Static Shadow"))
	uint8 bCastStaticShadow:1;

	/** 
	 * Whether the object should cast a volumetric translucent shadow.
	 * Volumetric translucent shadows are useful for primitives with smoothly changing opacity like particles representing a volume, 
	 * but have artifacts when used on highly opaque surfaces. This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Volumetric Translucent Shadow"))
	uint8 bCastVolumetricTranslucentShadow:1;

	/**
	 * Whether the object should cast contact shadows.
	 * This flag is only used if CastShadow is true.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(EditCondition="CastShadow", DisplayName = "Contact Shadow"))
	uint8 bCastContactShadow:1;

	/** 
	 * When enabled, the component will only cast a shadow on itself and not other components in the world.  
	 * This is especially useful for first person weapons, and forces bCastInsetShadow to be enabled.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow"))
	uint8 bSelfShadowOnly:1;

	/** 
	 * When enabled, the component will be rendering into the far shadow cascades (only for directional lights). This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Far Shadow"))
	uint8 bCastFarShadow:1;

	/** 
	 * Whether this component should create a per-object shadow that gives higher effective shadow resolution. 
	 * Useful for cinematic character shadowing. Assumed to be enabled if bSelfShadowOnly is enabled.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow", DisplayName = "Dynamic Inset Shadow"))
	uint8 bCastInsetShadow:1;

	/** 
	 * Whether this component should cast shadows from lights that have bCastShadowsFromCinematicObjectsOnly enabled.
	 * This is useful for characters in a cinematic with special cinematic lights, where the cost of shadowmap rendering of the environment is undesired.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow"))
	uint8 bCastCinematicShadow:1;

	/** 
	 *	If true, the primitive will cast shadows even if bHidden is true.
	 *	Controls whether the primitive should cast shadows when hidden.
	 *	This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to WorldSpaceRepresentation.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::WorldSpaceRepresentation)", DisplayName = "Hidden Shadow"), Interp)
	uint8 bCastHiddenShadow:1;

	/** Whether this primitive should cast dynamic shadows as if it were a two sided material. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow", DisplayName = "Shadow Two Sided"))
	uint8 bCastShadowAsTwoSided:1;

	/** @deprecated Replaced by LightmapType */
	UPROPERTY()
	uint8 bLightAsIfStatic_DEPRECATED:1;

	/** 
	 * Whether to light this component and any attachments as a group.  This only has effect on the root component of an attachment tree.
	 * When enabled, attached component shadowing settings like bCastInsetShadow, bCastVolumetricTranslucentShadow, etc, will be ignored.
	 * This is useful for improving performance when multiple movable components are attached together.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	uint8 bLightAttachmentsAsGroup:1;

	/** 
	 * If set, then it overrides any bLightAttachmentsAsGroup set in a parent.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	uint8 bExcludeFromLightAttachmentGroup :1;

	/**
	* Mobile only:
	* If disabled this component will not receive CSM shadows. (Components that do not receive CSM may have reduced shading cost)
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Mobile, meta = (DisplayName = "Receive CSM Shadows"))
	uint8 bReceiveMobileCSMShadows : 1;

	/** 
	 * Whether the whole component should be shadowed as one from stationary lights, which makes shadow receiving much cheaper.
	 * When enabled shadowing data comes from the volume lighting samples precomputed by Lightmass, which are very sparse.
	 * This is currently only used on stationary directional lights.  
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	uint8 bSingleSampleShadowFromStationaryLights:1;

	// Physics
	
	/** Will ignore radial impulses applied to this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	uint8 bIgnoreRadialImpulse:1;

	/** Will ignore radial forces applied to this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	uint8 bIgnoreRadialForce:1;

	/** True for damage to this component to apply physics impulse, false to opt out of these impulses. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	uint8 bApplyImpulseOnDamage : 1;

	/** True if physics should be replicated to autonomous proxies. This should be true for
		server-authoritative simulations, and false for client authoritative simulations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	uint8 bReplicatePhysicsToAutonomousProxy : 1;

	// Navigation

	/** If set, navmesh will not be generated under the surface of the geometry */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint8 bFillCollisionUnderneathForNavmesh:1;

	/** If set, the geometry gathered for navigation data generation will be converted into a filled vertical convex volume.
	 * It means that all collision geometries of the asset are merged vertically resulting in a grid of vertical columns that encompass the asset.
	 * This can be useful to represent the interior of the surface and prevent navmesh inside. */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint8 bRasterizeAsFilledConvexVolume:1;

	// General flags.
	
	/** If this is True, this component must always be loaded on clients, even if Hidden and CollisionEnabled is NoCollision. */
	UPROPERTY()
	uint8 AlwaysLoadOnClient:1;

	/** If this is True, this component must always be loaded on servers, even if Hidden and CollisionEnabled is NoCollision */
	UPROPERTY()
	uint8 AlwaysLoadOnServer:1;

	/** Composite the drawing of this component onto the scene after post processing (only applies to editor drawing) */
	UPROPERTY()
	uint8 bUseEditorCompositing:1;

	/** Set to true while the editor is moving the component, which notifies the Renderer to track velocities even if the component is Static. */
	UPROPERTY(Transient, DuplicateTransient)
	uint8 bIsBeingMovedByEditor:1;

	/** If true, this component will be rendered in the CustomDepth pass (usually used for outlines) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering, meta=(DisplayName = "Render CustomDepth Pass"))
	uint8 bRenderCustomDepth:1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta = (DisplayName = "Visible In Scene Capture Only", ToolTip = "When true, will only be visible in Scene Capture"))
	uint8 bVisibleInSceneCaptureOnly : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta = (DisplayName = "Hidden In Scene Capture", ToolTip = "When true, will not be captured by Scene Capture"))
	uint8 bHiddenInSceneCapture : 1;

	/** If true, this component will be included in the ray tracing far field layer instead of the near field. The ray tracing far field layer ignores whether the component is hidden. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = RayTracing)
	uint8 bRayTracingFarField : 1;

	/** If true, a single top-down Lumen card is allocated */
	UPROPERTY(Experimental)
	uint8 bLumenHeightfield : 1;

	/** If this is set to FirstPerson, the camera FirstPersonFieldOfView and FirstPersonScale parameters will be used on this component. These parameters can be used to render the component with a different field of view and a smaller depth range such that clipping with the scene can be avoided. This is useful for rendering first person view geometry. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	EFirstPersonPrimitiveType FirstPersonPrimitiveType;

protected:
	/** Result of last call to AreAllCollideableDescendantsRelative(). */
	uint8 bCachedAllCollideableDescendantsRelative : 1;

	UPROPERTY()
	uint8 bHasNoStreamableTextures : 1;

	/** When mobility is stationary, use a static underlying physics body. Static bodies do not have
		physical data like mass. If false, even stationary bodies will be generated with all data
		necessary for simulating.
		
		If you need this body's physical parameters on the physics thread (eg, in a sim callback)
		then set this to false. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(
		DisplayName = "Static When Not Moveable",
		ToolTip = "When false, the underlying physics body will contain all sim data (mass, inertia tensor, etc) even if mobility is not set to Moveable"))
	uint8 bStaticWhenNotMoveable:1;

	uint8 bIgnoreBoundsForEditorFocus : 1;
	
public:

#if WITH_EDITOR
	/** If true, then this component was added to the navigation octree when it was registered. */
	uint8 bRegisteredForNavigation : 1;

	uint8 bAlwaysAllowTranslucentSelect : 1;

	uint8 SelectionOutlineColorIndex;

	FColor OverlayColor;
#endif

	/** If true then DoCustomNavigableGeometryExport will be called to collect navigable geometry of this component. */
	UPROPERTY()
	TEnumAsByte<EHasCustomNavigableGeometry::Type> bHasCustomNavigableGeometry;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TEnumAsByte<enum EHitProxyPriority> HitProxyPriority;

	UE_DEPRECATED(5.2, "Use SetExcludedFromHLODLevel/IsExcludedFromHLODLevel")
	UPROPERTY(BlueprintReadWrite, Category = HLOD, BlueprintGetter=GetExcludeForSpecificHLODLevels, BlueprintSetter=SetExcludeForSpecificHLODLevels, meta = (DeprecatedProperty, DeprecationMessage = "WARNING: This property has been deprecated, use the SetExcludedFromHLODLevel/IsExcludedFromHLODLevel functions instead"))
	TArray<int32> ExcludeForSpecificHLODLevels_DEPRECATED;
#endif

	/** Whether this primitive is excluded from the specified HLOD level */
	UFUNCTION(BlueprintCallable, Category = "HLOD", meta = (DisplayName="Is Excluded From HLOD Level"))
	ENGINE_API bool IsExcludedFromHLODLevel(EHLODLevelExclusion HLODLevel) const;

	/** Exclude this primitive from the specified HLOD level */
	UFUNCTION(BlueprintCallable, Category = "HLOD", meta = (DisplayName = "Set Excluded From HLOD Level"))
	ENGINE_API void SetExcludedFromHLODLevel(EHLODLevelExclusion HLODLevel, bool bExcluded);

#if WITH_EDITOR
	ENGINE_API virtual void ComputeHLODHash(class FHLODHashBuilder& HashBuilder) const override;
#endif

private:
	UE_DEPRECATED("5.2", "Use SetExcludedFromHLODLevel instead")
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "HLOD", meta = (BlueprintInternalUseOnly="true"))
	ENGINE_API void SetExcludeForSpecificHLODLevels(const TArray<int32>& InExcludeForSpecificHLODLevels);

	UE_DEPRECATED("5.2", "Use IsExcludedFromHLODLevel instead")
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category = "HLOD", meta = (BlueprintInternalUseOnly="true"))
	ENGINE_API TArray<int32> GetExcludeForSpecificHLODLevels() const;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TEnumAsByte<enum ECanBeCharacterBase> CanBeCharacterBase_DEPRECATED;

	/** Deprecated - represented by HLODBatchingPolicy == EHLODBatchingPolicy::MeshSection */
	UPROPERTY()
	uint8 bUseMaxLODAsImposter_DEPRECATED : 1;

	/** Deprecated - represented by HLODBatchingPolicy == EHLODBatchingPolicy::Instancing */
	UPROPERTY()
	uint8 bBatchImpostersAsInstances_DEPRECATED : 1;
#endif

	FMaskFilter MoveIgnoreMask;

public:
	/**
	 * Determine whether a Character can step up onto this component.
	 * This controls whether they can try to step up on it when they bump in to it, not whether they can walk on it after landing on it.
	 * @see FWalkableSlopeOverride
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Collision)
	TEnumAsByte<enum ECanBeCharacterBase> CanCharacterStepUpOn;

	/** 
	 * Channels that this component should be in.  Lights with matching channels will affect the component.  
	 * These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	 * Lighting channels are only supported on translucent materials using forward shading (i.e. when not using the translucency lighting volume).
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	FLightingChannels LightingChannels;

	/**
	* Defines how quickly it should be culled. For example buildings should have a low priority, but small dressing should have a high priority.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = RayTracing, meta = (DisplayAfter="RayTracingGroupId"))
	ERayTracingGroupCullingPriority RayTracingGroupCullingPriority;

	/** Mask used for stencil buffer writes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = "Rendering", meta = (editcondition = "bRenderCustomDepth", DisplayAfter="CustomDepthStencilValue"))
	ERendererStencilMask CustomDepthStencilWriteMask;

private:
	/** Which specific HLOD levels this component should be excluded from */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = HLOD, meta = (Bitmask, BitmaskEnum = "/Script/Engine.EHLODLevelExclusion", DisplayName = "Exclude from HLOD Levels", DisplayAfter = "bEnableAutoLODGeneration", EditConditionHides, EditCondition = "AllowHLODLevelsExclusion()"))
	uint8 ExcludeFromHLODLevels;

public:

	/**
	 * Defines run-time groups of components. For example allows to assemble multiple parts of a building at runtime.
	 * -1 means that component doesn't belong to any group.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = RayTracing)
	int32 RayTracingGroupId;

	/** Used for precomputed visibility */
	UPROPERTY()
	int32 VisibilityId=0;

	/** Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering,  meta=(UIMin = "0", UIMax = "255", editcondition = "bRenderCustomDepth", DisplayName = "CustomDepth Stencil Value"))
	int32 CustomDepthStencilValue;

private:

	/** Optional user defined default values for the custom primitive data of this primitive */
	UPROPERTY(EditAnywhere, Category=Rendering, meta = (DisplayName = "Custom Primitive Data Defaults"))
	FCustomPrimitiveData CustomPrimitiveData;

	/** Custom data that can be read by a material through a material parameter expression. Set data using SetCustomPrimitiveData* functions */
	UPROPERTY(Transient)
	FCustomPrimitiveData CustomPrimitiveDataInternal;

	/** Payload used to release BodyInstance resources in asynchronous mode (see OnAsyncDestroyPhysicsState). */
	TOptional<FBodyInstance::FAsyncTermBodyPayload> AsyncTermBodyPayload;
	
public:

	/** If non-null, physics state creation has been deferred to ULevel::IncrementalUpdateComponents or this scene's StartFrame.*/
	FPhysScene* DeferredCreatePhysicsStateScene;

	/**
	 * Translucent objects with a lower sort priority draw behind objects with a higher priority.
	 * Translucent objects with the same priority are rendered from back-to-front based on their bounds origin.
	 * This setting is also used to sort objects being drawn into a runtime virtual texture.
	 *
	 * Ignored if the object is not translucent.  The default priority is zero.
	 * Warning: This should never be set to a non-default value unless you know what you are doing, as it will prevent the renderer from sorting correctly.  
	 * It is especially problematic on dynamic gameplay effects.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Rendering)
	int32 TranslucencySortPriority;

	/**
	 * Modified sort distance offset for translucent objects in world units.
	 * A positive number will move the sort distance further and a negative number will move the distance closer.
	 *
	 * Ignored if the object is not translucent.
	 * Warning: Adjusting this value will prevent the renderer from correctly sorting based on distance.  Only modify this value if you are certain it will not cause visual artifacts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Rendering)
	float TranslucencySortDistanceOffset = 0.0f;

	/** 
	 * Array of runtime virtual textures into which we draw the mesh for this actor. 
	 * The material also needs to be set up to output to a virtual texture. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VirtualTexture, meta = (DisplayName = "Draw in Virtual Textures"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	/** Bias to the LOD selected for rendering to runtime virtual textures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture LOD Bias", UIMin = "-7", UIMax = "8"))
	int8 VirtualTextureLodBias = 0;

	/**
	 * Number of lower mips in the runtime virtual texture to skip for rendering this primitive.
	 * Larger values reduce the effective draw distance in the runtime virtual texture.
	 * This culling method doesn't take into account primitive size or virtual texture size.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture Skip Mips", UIMin = "0", UIMax = "7"))
	int8 VirtualTextureCullMips = 0;

	/**
	 * Set the minimum pixel coverage before culling from the runtime virtual texture.
	 * Larger values reduce the effective draw distance in the runtime virtual texture.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (UIMin = "0", UIMax = "7"))
	int8 VirtualTextureMinCoverage = 0;

	/** Controls if this component draws in the main pass as well as in the virtual texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VirtualTexture, meta = (DisplayName = "Draw in Main Pass"))
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Exclusive;

	/** Get the array of runtime virtual textures into which we render the mesh for this actor. */
	virtual TArray<URuntimeVirtualTexture*> const& GetRuntimeVirtualTextures() const { return RuntimeVirtualTextures; }
	/** Get the runtime virtual texture pass settings. */
	virtual ERuntimeVirtualTextureMainPassType GetVirtualTextureRenderPassType() const { return VirtualTextureRenderPassType; }
	/** Get the max draw distance to use in the main pass when also rendering to a runtime virtual texture. This is combined with the other max draw distance settings. */
	virtual float GetVirtualTextureMainPassMaxDrawDistance() const { return 0.f; }
	
	/** Used by the renderer, to identify a component across re-registers. */	
	FPrimitiveComponentId GetPrimitiveSceneId() const { return SceneData.PrimitiveSceneId; }
	
	/** Used to detach physics objects before simulation begins. This is needed because at runtime we can't have simulated objects inside the attachment hierarchy */
	ENGINE_API virtual void BeginPlay() override;

protected:
	/** Returns true if all descendant components that we can possibly overlap with use relative location and rotation. */
	ENGINE_API virtual bool AreAllCollideableDescendantsRelative(bool bAllowCachedValue = true) const;

	/** Last time we checked AreAllCollideableDescendantsRelative(), so we can throttle those tests since it rarely changes once false. */
	float LastCheckedAllCollideableDescendantsTime;

	/** All PSO related data for the component */
	FPSOPrecacheComponentData PSOPrecacheComponentData;

private:
	
	float OcclusionBoundsSlack;

public:

	/** 
	 * Scales the bounds of the object.
	 * This is useful when using World Position Offset to animate the vertices of the object outside of its bounds. 
	 * Warning: Increasing the bounds of an object will reduce performance and shadow quality!
	 * Currently only used by StaticMeshComponent and SkeletalMeshComponent.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Rendering, meta=(UIMin = "1", UIMax = "10.0"))
	float BoundsScale;

private:
	
	FPrimitiveSceneInfoData SceneData;

#if MESH_DRAW_COMMAND_STATS
	/** Optional category name for this component in the mesh draw stat collection. */
	FName MeshDrawCommandStatsCategory;
#endif

	friend class FPrimitiveSceneInfo;
	friend struct FPrimitiveSceneInfoAdapter;

public:	

	FPrimitiveSceneInfoData& GetSceneData() { return SceneData; }
	FAttachmentCounter* GetAttachmentCounter() { return &GetSceneData().AttachmentCounter; }

	ENGINE_API int32 GetRayTracingGroupId() const;

	/**
	 * Returns true if this component has been rendered "recently", with a tolerance in seconds to define what "recent" means.
	 * e.g.: If a tolerance of 0.1 is used, this function will return true only if the actor was rendered in the last 0.1 seconds of game time.
	 *
	 * @param Tolerance  How many seconds ago the actor last render time can be and still count as having been "recently" rendered.
	 * @return Whether this actor was recently rendered.
	 */
	UFUNCTION(Category = "Rendering", BlueprintCallable, meta=(DisplayName="Was Component Recently Rendered", Keywords="scene visible"))
	ENGINE_API bool WasRecentlyRendered(float Tolerance = 0.2f) const;

	ENGINE_API void SetLastRenderTime(float InLastRenderTime);

	ENGINE_API float GetLastRenderTime() const;
	ENGINE_API float GetLastRenderTimeOnScreen() const;

#if MESH_DRAW_COMMAND_STATS
	ENGINE_API void SetMeshDrawCommandStatsCategory(FName StatsCategory);
	FName GetMeshDrawCommandStatsCategory() const;
#else
	void SetMeshDrawCommandStatsCategory(FName StatsCategory) {}
#endif

	/**
	 * Setup the parameter struct used to precache the PSOs used by this component. 
	 * Precaching uses certain component attributes to derive the shader or state used to render the component such as static lighting, cast shadows, ...
	 */
	ENGINE_API virtual void SetupPrecachePSOParams(FPSOPrecacheParams& Params);

	/**
	 * Collect all the data required for PSO precaching 
	 */
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) {}

	/** Precache all PSOs which can be used by the primitive component */
	ENGINE_API virtual void PrecachePSOs() override;

	/** Store all the PSO precache data and schedule task to mark when precaching is done so component can be notified about it */
	ENGINE_API void SetMaterialPSOPrecacheData(TArray<FMaterialPSOPrecacheRequestID>& MaterialPSOPrecacheRequestIDs, const FGraphEventArray& PSOPrecacheCompileEvents);

	/** Check if PSOs are still precaching */	
	bool IsPSOPrecaching() const { return PSOPrecacheComponentData.IsPSOPrecaching(); }

	/** Get the PSO precaching data */
	FPSOPrecacheComponentData& GetPSOPrecacheComponentData() { return PSOPrecacheComponentData; }
	const FPSOPrecacheComponentData& GetPSOPrecacheComponentData() const { return PSOPrecacheComponentData; }

	/**
	 * Check if PSOs are still precaching and boost priority if not done yet.
	 * PSOPrecachePriority specifies the new priority to boost to. calls that would reduce the current priority are ignored, highest pri is used for components that are actively rendering.
	 * Returns true if the PSOs are still precaching.
	 */
	bool CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority PSOPrecachePriority = EPSOPrecachePriority::High)
	{
		return PSOPrecacheComponentData.CheckPSOPrecachingAndBoostPriority(PSOPrecachePriority);
	}

protected:

	/**
	 * Examines the used materials (GetUsedMaterials) and returns a descriptor. This may be called when there is no proxy created
	 * which can be useful. But will use the information in the proxy if it is present.
	 */
	UE_DEPRECATED(5.7, "Please use GetUsedMaterialPropertyDesc with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(ERHIFeatureLevel::Type FeatureLevel) const;
	ENGINE_API FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(EShaderPlatform InShaderPlatform) const;

	/**
	 * Returns true if this component opts in to participate in the render proxy delay mechanism that kicks in
	 * if PSO precaching hasn't finished. Otherwise, PSO precaching will still be active but the render proxy
	 * will be created as normal.
	 */
	ENGINE_API virtual bool UsePSOPrecacheRenderProxyDelay() const;

public:

	/**
	 * Set of actors to ignore during component sweeps in MoveComponent().
	 * All components owned by these actors will be ignored when this component moves or updates overlaps.
	 * Components on the other Actor may also need to be told to do the same when they move.
	 * Does not affect movement of this component when simulating physics.
	 * @see IgnoreActorWhenMoving()
	 */
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<AActor>> MoveIgnoreActors;

	/**
	 * Tells this component whether to ignore collision with all components of a specific Actor when this component is moved.
	 * Components on the other Actor may also need to be told to do the same when they move.
	 * Does not affect movement of this component when simulating physics.
	 */
	UFUNCTION(BlueprintCallable, Category = "Collision", meta=(Keywords="Move MoveIgnore", UnsafeDuringActorConstruction="true"))
	ENGINE_API void IgnoreActorWhenMoving(AActor* Actor, bool bShouldIgnore);

	/**
	 * Returns the list of actors we currently ignore when moving.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName="Get Move Ignore Actors", UnsafeDuringActorConstruction="true"), Category = "Collision")
	ENGINE_API TArray<AActor*> CopyArrayOfMoveIgnoreActors();

	/**
	 * Returns the list of actors (as WeakObjectPtr) we currently ignore when moving.
	 */
	const TArray<AActor*>& GetMoveIgnoreActors() const { return MoveIgnoreActors; }

	/**
	 * Clear the list of actors we ignore when moving.
	 */
	UFUNCTION(BlueprintCallable, Category = "Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void ClearMoveIgnoreActors();

	/**
	* Set of components to ignore during component sweeps in MoveComponent().
	* These components will be ignored when this component moves or updates overlaps.
	* The other components may also need to be told to do the same when they move.
	* Does not affect movement of this component when simulating physics.
	* @see IgnoreComponentWhenMoving()
	*/
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<UPrimitiveComponent>> MoveIgnoreComponents;

	/**
	* Tells this component whether to ignore collision with another component when this component is moved.
	* The other components may also need to be told to do the same when they move.
	* Does not affect movement of this component when simulating physics.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta=(Keywords="Move MoveIgnore", UnsafeDuringActorConstruction="true"))
	ENGINE_API void IgnoreComponentWhenMoving(UPrimitiveComponent* Component, bool bShouldIgnore);

	/**
	* Returns the list of actors we currently ignore when moving.
	*/
	UFUNCTION(BlueprintCallable, meta=(DisplayName="Get Move Ignore Components", UnsafeDuringActorConstruction="true"), Category = "Collision")
	ENGINE_API TArray<UPrimitiveComponent*> CopyArrayOfMoveIgnoreComponents();

	/**
	* Returns the list of components we currently ignore when moving.
	*/
	const TArray<UPrimitiveComponent*>& GetMoveIgnoreComponents() const { return MoveIgnoreComponents; }

	/**
	* Clear the list of components we ignore when moving.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta=(UnsafeDuringActorConstruction="true"))
	void ClearMoveIgnoreComponents() { MoveIgnoreComponents.Empty(); }

	/** Set the mask filter we use when moving. */
	ENGINE_API void SetMoveIgnoreMask(FMaskFilter InMoveIgnoreMask);

	/** Get the mask filter we use when moving. */
	FMaskFilter GetMoveIgnoreMask() const { return MoveIgnoreMask; }

	/** Should the hit result be ignored based on this component */
	ENGINE_API virtual bool ShouldComponentIgnoreHitResult(FHitResult const& TestHit, EMoveComponentFlags MoveFlags);

	/** Set the mask filter checked when others move into us. */
	void SetMaskFilterOnBodyInstance(FMaskFilter InMaskFilter) { BodyInstance.SetMaskFilter(InMaskFilter); }

	/** Get the mask filter checked when others move into us. */
	FMaskFilter GetMaskFilterOnBodyInstance(FMaskFilter InMaskFilter) const { return BodyInstance.GetMaskFilter(); }

	/**
	 * Gets the index of the scalar parameter for the custom primitive data array
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @return	The index of the custom primitive, INDEX_NONE (-1) if not found
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API int32 GetCustomPrimitiveDataIndexForScalarParameter(FName ParameterName) const;

	/**
	 * Gets the index of the vector parameter for the custom primitive data array
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @return	The index of the custom primitive, INDEX_NONE (-1) if not found
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API int32 GetCustomPrimitiveDataIndexForVectorParameter(FName ParameterName) const;

	/**
	 * Set a scalar parameter for custom primitive data. This sets the run-time data only, so it doesn't serialize.
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @param	Value			The new value of the custom primitive
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetScalarParameterForCustomPrimitiveData(FName ParameterName, float Value);

	/**
	 * Set a vector parameter for custom primitive data. This sets the run-time data only, so it doesn't serialize.
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @param	Value			The new value of the custom primitive
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetVectorParameterForCustomPrimitiveData(FName ParameterName, FVector4 Value);

	/** Set custom primitive data at index DataIndex. This sets the run-time data only, so it doesn't serialize. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetCustomPrimitiveDataFloat(int32 DataIndex, float Value);

	/** Set custom primitive data, two floats at once, from index DataIndex to index DataIndex + 1. This sets the run-time data only, so it doesn't serialize. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetCustomPrimitiveDataVector2(int32 DataIndex, FVector2D Value);

	/** Set custom primitive data, two floats at once, from index DataIndex to index DataIndex + 1. This sets the run-time data only, so it doesn't serialize. */
	ENGINE_API void SetCustomPrimitiveDataVector2f(int32 DataIndex, FVector2f Value);

	/** Set custom primitive data, three floats at once, from index DataIndex to index DataIndex + 2. This sets the run-time data only, so it doesn't serialize. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetCustomPrimitiveDataVector3(int32 DataIndex, FVector Value);

	/** Set custom primitive data, three floats at once, from index DataIndex to index DataIndex + 2. This sets the run-time data only, so it doesn't serialize. */
	ENGINE_API void SetCustomPrimitiveDataVector3f(int32 DataIndex, FVector3f Value);

	/** Set custom primitive data, four floats at once, from index DataIndex to index DataIndex + 3. This sets the run-time data only, so it doesn't serialize. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetCustomPrimitiveDataVector4(int32 DataIndex, FVector4 Value);

	/** Set custom primitive data, four floats at once, from index DataIndex to index DataIndex + 3. This sets the run-time data only, so it doesn't serialize. */
	ENGINE_API void SetCustomPrimitiveDataVector4f(int32 DataIndex, FVector4f Value);

	/** Set custom primitive data, an array of floats at once, from index DataIndex to index DataIndex + Values.Num(). This sets the run-time data only, so it doesn't serialize. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetCustomPrimitiveDataFloatArray(int32 DataIndex, const TArray<float>& Values);

	/** Set custom primitive data, an array of floats at once, from index DataIndex to index DataIndex + Values.Num(). This sets the run-time data only, so it doesn't serialize. */
	ENGINE_API void SetCustomPrimitiveDataFloatArray(int32 DataIndex, TConstArrayView<float> Values);

	/** 
	 * Get the custom primitive data for this primitive component.
	 * @return The payload of custom data that will be set on the primitive and accessible in the material through a material expression.
	 */
	const FCustomPrimitiveData& GetCustomPrimitiveData() const { return CustomPrimitiveDataInternal; }

	/** Reset the custom primitive data of this primitive to the optional user defined default */
	ENGINE_API void ResetCustomPrimitiveData();

	/**
	 * Set a scalar parameter for default custom primitive data. This will be serialized and is useful in construction scripts.
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @param	Value			The new value of the custom primitive
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetScalarParameterForDefaultCustomPrimitiveData(FName ParameterName, float Value);

	/**
	 * Set a vector parameter for default custom primitive data. This will be serialized and is useful in construction scripts.
	 * @param	ParameterName	The parameter name of the custom primitive
	 * @param	Value			The new value of the custom primitive
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetVectorParameterForDefaultCustomPrimitiveData(FName ParameterName, FVector4 Value);

	/** Set default custom primitive data at index DataIndex, and marks the render state dirty */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetDefaultCustomPrimitiveDataFloat(int32 DataIndex, float Value);

	/** Set default custom primitive data, two floats at once, from index DataIndex to index DataIndex + 1, and marks the render state dirty */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetDefaultCustomPrimitiveDataVector2(int32 DataIndex, FVector2D Value);

	/** Set default custom primitive data, two floats at once, from index DataIndex to index DataIndex + 1, and marks the render state dirty */
	ENGINE_API void SetDefaultCustomPrimitiveDataVector2f(int32 DataIndex, FVector2f Value);

	/** Set default custom primitive data, three floats at once, from index DataIndex to index DataIndex + 2, and marks the render state dirty */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetDefaultCustomPrimitiveDataVector3(int32 DataIndex, FVector Value);

	/** Set default custom primitive data, three floats at once, from index DataIndex to index DataIndex + 2, and marks the render state dirty */
	ENGINE_API void SetDefaultCustomPrimitiveDataVector3f(int32 DataIndex, FVector3f Value);

	/** Set default custom primitive data, four floats at once, from index DataIndex to index DataIndex + 3, and marks the render state dirty */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetDefaultCustomPrimitiveDataVector4(int32 DataIndex, FVector4 Value);

	/** Set default custom primitive data, four floats at once, from index DataIndex to index DataIndex + 3, and marks the render state dirty */
	ENGINE_API void SetDefaultCustomPrimitiveDataVector4f(int32 DataIndex, FVector4f Value);

	/** Set default custom primitive data, an array floats at once, from index DataIndex to index DataIndex + Values.Num(), and marks the render state dirty */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetDefaultCustomPrimitiveDataFloatArray(int32 DataIndex, const TArray<float>& Values);

	/** Set default custom primitive data, an array floats at once, from index DataIndex to index DataIndex + Values.Num(), and marks the render state dirty */
	ENGINE_API void SetDefaultCustomPrimitiveDataFloatArray(int32 DataIndex, TConstArrayView<float> Values);

	/**
	 * Get the default custom primitive data for this primitive component.
	 * @return The payload of custom data that will be set on the primitive and accessible in the material through a material expression.
	 */
	const FCustomPrimitiveData& GetDefaultCustomPrimitiveData() const { return CustomPrimitiveData; }

	/**
	 * Gather the default custom primitive data parameters. Non-const as it keeps a non-const weak pointer on 'this'.
	 * Outputs an array of pairs of primitive indices with the corresponding parameter and the max primitive data index.
	 */
	ENGINE_API void GatherDefaultCustomPrimitiveDataParametersFromMaterials(TArray<TPair<uint8 /*PrimitiveIndex*/, FCustomPrimitiveDataParameter>>& OutParameters, int32& OutMaxPrimitiveDataIndex);

	/**
	 * Setup the Default Custom Primitive Data based on the assigned materials. Wipes any existing data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetupDefaultCustomPrimitiveDataFromMaterials();

#if WITH_EDITOR
	/** Override delegate used for checking the selection state of a component */
	DECLARE_DELEGATE_RetVal_OneParam( bool, FSelectionOverride, const UPrimitiveComponent* );
	FSelectionOverride SelectionOverrideDelegate;
#endif

protected:

	/** Insert an array of floats into the CustomPrimitiveData, starting at the given index */
	ENGINE_API void SetCustomPrimitiveDataInternal(int32 DataIndex, TConstArrayView<float> Values);

	/** Insert an array of floats into the CustomPrimitiveData defaults, starting at the given index */
	ENGINE_API void SetDefaultCustomPrimitiveData(int32 DataIndex, TConstArrayView<float> Values);

	/** Set of components that this component is currently overlapping. */
	TArray<FOverlapInfo> OverlappingComponents;

public:
	/** Convert a set of overlaps from a sweep to a subset that includes only those at the end location (filling in OverlapsAtEndLocation). */
	template<typename AllocatorType>
	bool ConvertSweptOverlapsToCurrentOverlaps(TArray<FOverlapInfo, AllocatorType>& OutOverlapsAtEndLocation, const TOverlapArrayView& SweptOverlaps, int32 SweptOverlapsIndex, const FVector& EndLocation, const FQuat& EndRotationQuat);

	/** Convert a set of overlaps from a symmetric change in rotation to a subset that includes only those at the end location (filling in OverlapsAtEndLocation). */
	template<typename AllocatorType>
	bool ConvertRotationOverlapsToCurrentOverlaps(TArray<FOverlapInfo, AllocatorType>& OutOverlapsAtEndLocation, const TOverlapArrayView& CurrentOverlaps);

private:
	template<typename AllocatorType>
	bool GetOverlapsWithActor_Template(const AActor* Actor, TArray<FOverlapInfo, AllocatorType>& OutOverlaps) const;

	// FScopedMovementUpdate needs access to the above two functions.
	friend FScopedMovementUpdate;

public:
	/** 
	 * Begin tracking an overlap interaction with the component specified.
	 * @param OtherComp - The component of the other actor that this component is now overlapping
	 * @param bDoNotifies - True to dispatch appropriate begin/end overlap notifications when these events occur.
	 * @see [Overlap Events](https://docs.unrealengine.com/InteractiveExperiences/Physics/Collision/Overview#overlapandgenerateoverlapevents)
	 */
	ENGINE_API void BeginComponentOverlap(const FOverlapInfo& OtherOverlap, bool bDoNotifies);
	
	/** 
	 * Finish tracking an overlap interaction that is no longer occurring between this component and the component specified. 
	 * @param OtherComp The component of the other actor to stop overlapping
	 * @param bDoNotifies True to dispatch appropriate begin/end overlap notifications when these events occur.
	 * @param bSkipNotifySelf True to skip end overlap notifications to this component's.  Does not affect notifications to OtherComp's actor.
	 * @see [Overlap Events](https://docs.unrealengine.com/InteractiveExperiences/Physics/Collision/Overview#overlapandgenerateoverlapevents)
	 */
	ENGINE_API void EndComponentOverlap(const FOverlapInfo& OtherOverlap, bool bDoNotifies=true, bool bSkipNotifySelf=false);

	/**
	 * Check whether this component is overlapping another component.
	 * @param OtherComp Component to test this component against.
	 * @return Whether this component is overlapping another component.
	 */
	UFUNCTION(BlueprintPure, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API bool IsOverlappingComponent(const UPrimitiveComponent* OtherComp) const;
	
	/** Check whether this component has the specified overlap. */
	ENGINE_API bool IsOverlappingComponent(const FOverlapInfo& Overlap) const;

	/**
	 * Check whether this component is overlapping any component of the given Actor.
	 * @param Other Actor to test this component against.
	 * @return Whether this component is overlapping any component of the given Actor.
	 */
	UFUNCTION(BlueprintPure, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API bool IsOverlappingActor(const AActor* Other) const;

	/** Appends list of overlaps with components owned by the given actor to the 'OutOverlaps' array. Returns true if any overlaps were added. */
	ENGINE_API bool GetOverlapsWithActor(const AActor* Actor, TArray<FOverlapInfo>& OutOverlaps) const;

	/** 
	 * Returns a list of actors that this component is overlapping.
	 * @param OverlappingActors		[out] Returned list of overlapping actors
	 * @param ClassFilter			[optional] If set, only returns actors of this class or subclasses
	 */
	UFUNCTION(BlueprintPure, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void GetOverlappingActors(TArray<AActor*>& OverlappingActors, TSubclassOf<AActor> ClassFilter=nullptr) const;

	/** 
	* Returns the set of actors that this component is overlapping.
	* @param OverlappingActors		[out] Returned list of overlapping actors
	* @param ClassFilter			[optional] If set, only returns actors of this class or subclasses
	*/
	ENGINE_API void GetOverlappingActors(TSet<AActor*>& OverlappingActors, TSubclassOf<AActor> ClassFilter=nullptr) const;

	/** Returns unique list of components this component is overlapping. */
	UFUNCTION(BlueprintPure, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void GetOverlappingComponents(TArray<UPrimitiveComponent*>& OutOverlappingComponents) const;

	/** Returns unique set of components this component is overlapping. */
	ENGINE_API void GetOverlappingComponents(TSet<UPrimitiveComponent*>& OutOverlappingComponents) const;

	/** Returns list of components this component is overlapping. */
	const TArray<FOverlapInfo>& GetOverlapInfos() const;

	/** 
	 * Queries world and updates overlap tracking state for this component.
	 * @param NewPendingOverlaps		An ordered list of components that the MovedComponent overlapped during its movement (eg. generated during a sweep). Only used to add potentially new overlaps.
	 *									Might not be overlapping them now.
	 * @param bDoNotifies				True to dispatch being/end overlap notifications when these events occur.
	 * @param OverlapsAtEndLocation		If non-null, the given list of overlaps will be used as the overlaps for this component at the current location, rather than checking for them with a scene query.
	 *									Generally this should only be used if this component is the RootComponent of the owning actor and overlaps with other descendant components have been verified.
	 * @return							True if we can skip calling this in the future (i.e. no useful work is being done.)
	 */
	ENGINE_API virtual bool UpdateOverlapsImpl(const TOverlapArrayView* NewPendingOverlaps=nullptr, bool bDoNotifies=true, const TOverlapArrayView* OverlapsAtEndLocation=nullptr) override;

	/**
	 * Whether or not the bounds of this component should be considered when focusing the editor camera to an actor with this component in it.
	 * Useful for debug components which need a bounds for rendering but don't contribute to the visible part of the mesh in a meaningful way
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	virtual bool GetIgnoreBoundsForEditorFocus() const { return bIgnoreBoundsForEditorFocus; }
	
	/**
	 * Set if we should ignore bounds when focusing the editor camera.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	void SetIgnoreBoundsForEditorFocus(bool bIgnore) { bIgnoreBoundsForEditorFocus = bIgnore; }

	/** Update current physics volume for this component, if bShouldUpdatePhysicsVolume is true. Overridden to use the overlaps to find the physics volume. */
	ENGINE_API virtual void UpdatePhysicsVolume( bool bTriggerNotifiers ) override;

	/**
	 *  Test the collision of the supplied component at the supplied location/rotation, and determine the set of components that it overlaps.
	 *  @note This overload taking rotation as a FQuat is slightly faster than the version using FRotator.
	 *  @note This simply calls the virtual ComponentOverlapMultiImpl() which can be overridden to implement custom behavior.
	 *  @param  OutOverlaps     Array of overlaps found between this component in specified pose and the world
	 *  @param  World			World to use for overlap test
	 *  @param  Pos             Location of component's geometry for the test against the world
	 *  @param  Rot             Rotation of component's geometry for the test against the world
	 *  @param  TestChannel		The 'channel' that this ray is in, used to determine which components to hit
	 *  @param	ObjectQueryParams	List of object types it's looking for. When this enters, we do object query with component shape
	 *  @return true if OutOverlaps contains any blocking results
	 */
	bool ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const struct FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const;
	bool ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FRotator& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const struct FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const;

	/**
	 *	Walks up the attachment tree until a primitive component with LightAttachmentsAsGroup enabled is found. This component will effectively act as the root of the attachment group.
	 *	Return nullptr if none is found. 
	 */
	ENGINE_API const UPrimitiveComponent* GetLightingAttachmentRoot() const;

protected:
	/** Override this method for custom behavior for ComponentOverlapMulti() */
	ENGINE_API virtual bool ComponentOverlapMultiImpl(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const;

public:
	// Internal physics engine data.
	
	/** Physics scene information for this component, holds a single rigid body with multiple shapes. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Collision, meta=(ShowOnlyInnerProperties, SkipUCSModifiedProperties))
	FBodyInstance BodyInstance;

	/** 
	 *	Event called when a component hits (or is hit by) something solid. This could happen due to things like Character movement, using Set Location with 'sweep' enabled, or physics simulation.
	 *	For events when objects overlap (e.g. walking into a trigger) see the 'Overlap' event.
	 *
	 *	@note For collisions during physics simulation to generate hit events, 'Simulation Generates Hit Events' must be enabled for this component.
	 *	@note When receiving a hit from another object's movement, the directions of 'Hit.Normal' and 'Hit.ImpactNormal'
	 *	will be adjusted to indicate force from the other object against this object.
	 *	@note NormalImpulse will be filled in for physics-simulating bodies, but will be zero for swept-component blocking collisions.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FComponentHitSignature OnComponentHit;

	/** 
	 *	Event called when something starts to overlaps this component, for example a player walking into a trigger.
	 *	For events when objects have a blocking collision, for example a player hitting a wall, see 'Hit' events.
	 *
	 *	@note Both this component and the other one must have GetGenerateOverlapEvents() set to true to generate overlap events.
	 *	@note When receiving an overlap from another object's movement, the directions of 'Hit.Normal' and 'Hit.ImpactNormal'
	 *	will be adjusted to indicate force from the other object against this object.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FComponentBeginOverlapSignature OnComponentBeginOverlap;

	/** 
	 *	Event called when something stops overlapping this component 
	 *	@note Both this component and the other one must have GetGenerateOverlapEvents() set to true to generate overlap events.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FComponentEndOverlapSignature OnComponentEndOverlap;

	/** 
	 *	Event called when the underlying physics objects is woken up
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FComponentWakeSignature OnComponentWake;

	/** 
	 *	Event called when the underlying physics objects is put to sleep
	 */
	UPROPERTY(BlueprintAssignable, Category = "Collision")
	FComponentSleepSignature OnComponentSleep;

	/**
	 *	Event called when collision settings change for this component.
	 */
	FComponentCollisionSettingsChangedSignature OnComponentCollisionSettingsChangedEvent;

	/**
	 *	Event called when physics state is created or destroyed for this component
	 */
	UPROPERTY(BlueprintAssignable, Category = "Physics", TextExportTransient)
	FComponentPhysicsStateChanged OnComponentPhysicsStateChanged;

	/** Event called when the mouse cursor is moved over this component and mouse over events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FComponentBeginCursorOverSignature OnBeginCursorOver;
		 
	/** Event called when the mouse cursor is moved off this component and mouse over events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FComponentEndCursorOverSignature OnEndCursorOver;

	/** Event called when the left mouse button is clicked while the mouse is over this component and click events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FComponentOnClickedSignature OnClicked;

	/** Event called when the left mouse button is released while the mouse is over this component click events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FComponentOnReleasedSignature OnReleased;

	/** Event called when a touch input is received over this component when touch events are enabled in the player controller
	 * This will be deprecated in the next engine release.  Use OnTouchBegan with TouchId, Player Controller, and Hit Result arguments instead.
	 */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input", DisplayName="On Touch Began (finger only)")
	FComponentOnInputTouchBeginSignature OnInputTouchBegin;

	/** Event called when a touch input is released over this component when touch events are enabled in the player controller
	 * This will be deprecated in the next engine release.  Use OnTouchEnded with TouchId, Player Controller, and Hit Result arguments instead.
	 */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input", DisplayName="On Touch Ended (finger only)")
	FComponentOnInputTouchEndSignature OnInputTouchEnd;

	/** Event called when a finger is moved over this component when touch over events are enabled in the player controller
	 * This will be deprecated in the next engine release.  Use OnTouchEntered with TouchId, Player Controller, and Hit Result arguments instead.
	 */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input", DisplayName="On Touch Entered (finger only)")
	FComponentBeginTouchOverSignature OnInputTouchEnter;

	/** Event called when a finger is moved off this component when touch over events are enabled in the player controller
	 * This will be deprecated in the next engine release.  Use OnTouchExited with TouchId, Player Controller, and Hit Result arguments instead.
	 */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input", DisplayName="On Touch Exited (finger only)")
	FComponentEndTouchOverSignature OnInputTouchLeave;

	/** Event called when a touch input is received over this component when touch events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FComponentOnTouchBeganSignature OnTouchBegan;

	/** Event called when a touch input is released over this component when touch events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FComponentOnTouchEndedSignature OnTouchEnded;

	/** Event called when a finger is moved over this component when touch over events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FComponentOnTouchEnteredSignature OnTouchEntered;

	/** Event called when a finger is moved off this component when touch over events are enabled in the player controller */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FComponentOnTouchExitedSignature OnTouchExited;

	/** Scale the bounds of this object, used for frustum culling. Useful for features like WorldPositionOffset. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetBoundsScale(float NewBoundsScale=1.f);

	/**
	 * Returns the material used by the element at the specified index
	 * @param ElementIndex - The element to access the material of.
	 * @return the material used by the indexed element of this mesh.
	 */
	UFUNCTION(BlueprintPure, Category="Rendering|Material")
	ENGINE_API virtual class UMaterialInterface* GetMaterial(int32 ElementIndex) const override;

#if WITH_EDITOR
	/** Returns the material to show in the editor details panel as being used. Skips Nanite Override materials. */
	UFUNCTION(BlueprintPure, Category = "Rendering|Material")
	virtual class UMaterialInterface* GetEditorMaterial(int32 ElementIndex) const
	{
		return GetMaterial(ElementIndex);
	}
#endif

	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual int32 GetMaterialIndex(FName MaterialSlotName) const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual TArray<FName> GetMaterialSlotNames() const;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const;

	/**
	* Returns the material used by the element in the slot with the specified name.
	* @param MaterialSlotName - The slot name to access the material of.
	* @return the material used in the slot specified, or null if none exists or the slot name is not found.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual class UMaterialInterface* GetMaterialByName(FName MaterialSlotName) const;

	/**
	 * Changes the material applied to an element of the mesh.
	 * @param ElementIndex - The element to access the material of.
	 * @return the material used by the indexed element of this mesh.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* Material);

	/**
	* Changes the material applied to an element of the mesh.
	* @param MaterialSlotName - The slot name to access the material of.
	* @return the material used by the indexed element of this mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual void SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* Material);

	/**
	 * Creates a Dynamic Material Instance for the specified element index.  The parent of the instance is set to the material being replaced.
	 * @param ElementIndex - The index of the skin to replace the material for.  If invalid, the material is unchanged and NULL is returned.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "CreateMIDForElement", DeprecatedFunction, DeprecationMessage="Use CreateDynamicMaterialInstance instead."), Category="Rendering|Material")
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateAndSetMaterialInstanceDynamic(int32 ElementIndex);

	/**
	 * Creates a Dynamic Material Instance for the specified element index.  The parent of the instance is set to the material being replaced.
	 * @param ElementIndex - The index of the skin to replace the material for.  If invalid, the material is unchanged and NULL is returned.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "CreateMIDForElementFromMaterial", DeprecatedFunction, DeprecationMessage="Use CreateDynamicMaterialInstance instead."), Category="Rendering|Material")
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateAndSetMaterialInstanceDynamicFromMaterial(int32 ElementIndex, class UMaterialInterface* Parent);

	/**
	 * Creates a Dynamic Material Instance for the specified element index, optionally from the supplied material.
	 * @param ElementIndex - The index of the skin to replace the material for.  If invalid, the material is unchanged and NULL is returned.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateDynamicMaterialInstance(int32 ElementIndex, class UMaterialInterface* SourceMaterial = NULL, FName OptionalName = NAME_None);

	/** 
	 * Try and retrieve the material applied to a particular collision face of mesh. Used with face index returned from collision trace. 
	 *	@param	FaceIndex		Face index from hit result that was hit by a trace
	 *	@param	SectionIndex	Section of the mesh that the face belongs to
	 *	@return					Material applied to section that the hit face belongs to
	 */
	UFUNCTION(BlueprintPure, Category = "Rendering|Material")
	ENGINE_API virtual UMaterialInterface* GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const;

	/** Returns the slope override struct for this component. */
	UFUNCTION(BlueprintPure, Category="Physics")
	ENGINE_API virtual const struct FWalkableSlopeOverride& GetWalkableSlopeOverride() const override;

	/** Sets a new slope override for this component instance. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride);

	/** 
	 * When this component is a simple/single body, this will enable or disable simulation on that body. In addition, 
	 * if this component is currently attached to something, beginning simulation will detach it. Note that stopping
	 * simulation will not reattach the component - that would need to be done explicitly. 
	 * 
	 * For more complex components (e.g. skeletal meshes), simulation will apply to the bodies contained by the
	 * component (e.g. using a physics asset). Since these bodies will be free to move independently of the component, 
	 * the component will not be automatically detached. If detachment is required, then that can be done by 
	 * calling DetachFromComponent.
	 *
	 * @param bSimulate New simulation state for the single body or multiple bodies
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetSimulatePhysics(bool bSimulate);

	/*
	 *	
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetStaticWhenNotMoveable(bool bInStaticWhenNotMoveable);

	UFUNCTION(BlueprintCallable, Category="Physics")
	bool GetStaticWhenNotMoveable() const { return bStaticWhenNotMoveable; }

	/**
	 * Determines whether or not the simulate physics setting can be edited interactively on this component
	 */
	ENGINE_API virtual bool CanEditSimulatePhysics();

	/**
	* Sets the constraint mode of the component.
	* @param ConstraintMode	The type of constraint to use.
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Constraint Mode", Keywords = "set locked axis constraint physics"), Category = Physics)
	ENGINE_API virtual void SetConstraintMode(EDOFMode::Type ConstraintMode);

	/**
	 *	Add an impulse to a single rigid body. Good for one time instant burst.
	 *
	 *	@param	Impulse		Magnitude and direction of impulse to apply.
	 *	@param	BoneName	If a SkeletalMeshComponent, name of body to apply impulse to. 'None' indicates root body.
	 *	@param	bVelChange	If true, the Strength is taken as a change in velocity instead of an impulse (ie. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddImpulse(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false);

	/**
	*	Add an angular impulse to a single rigid body. Good for one time instant burst.
	*
	*	@param	AngularImpulse	Magnitude and direction of impulse to apply. Direction is axis of rotation.
	*	@param	BoneName	If a SkeletalMeshComponent, name of body to apply angular impulse to. 'None' indicates root body.
	*	@param	bVelChange	If true, the Strength is taken as a change in angular velocity instead of an impulse (ie. mass will have no effect).
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddAngularImpulseInRadians(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false);

	/**
	*	Add an angular impulse to a single rigid body. Good for one time instant burst.
	*
	*	@param	AngularImpulse	Magnitude and direction of impulse to apply. Direction is axis of rotation.
	*	@param	BoneName	If a SkeletalMeshComponent, name of body to apply angular impulse to. 'None' indicates root body.
	*	@param	bVelChange	If true, the Strength is taken as a change in angular velocity instead of an impulse (ie. mass will have no effect).
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	void AddAngularImpulseInDegrees(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false)
	{
		AddAngularImpulseInRadians(FMath::DegreesToRadians(Impulse), BoneName, bVelChange);
	}

	/**
	 *	Add an impulse to a single rigid body at a specific location. 
	 *
	 *	@param	Impulse		Magnitude and direction of impulse to apply.
	 *	@param	Location	Point in world space to apply impulse at.
	 *	@param	BoneName	If a SkeletalMeshComponent, name of bone to apply impulse to. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None);

	/**
	 *	Add an impulse to a single rigid body at a specific location. The Strength is taken as a change in angular velocity instead of an impulse (ie. mass will have no effect).
	 *
	 *	@param	Impulse		Magnitude and direction of impulse to apply.
	 *	@param	Location	Point in world space to apply impulse at.
	 *	@param	BoneName	If a SkeletalMeshComponent, name of bone to apply impulse to. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (UnsafeDuringActorConstruction = "true"))
	ENGINE_API virtual void AddVelocityChangeImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None);


	/**
	 * Add an impulse to all rigid bodies in this component, radiating out from the specified position.
	 *
	 * @param Origin		Point of origin for the radial impulse blast, in world space
	 * @param Radius		Size of radial impulse. Beyond this distance from Origin, there will be no affect.
	 * @param Strength		Maximum strength of impulse applied to body.
	 * @param Falloff		Allows you to control the strength of the impulse as a function of distance from Origin.
	 * @param bVelChange	If true, the Strength is taken as a change in velocity instead of an impulse (ie. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddRadialImpulse(FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange = false);

	/**
	 *	Add a force to a single rigid body.
	 *  This is like a 'thruster'. Good for adding a burst over some (non zero) time. Should be called every frame for the duration of the force.
	 *
	 *	@param	Force		 Force vector to apply. Magnitude indicates strength of force.
	 *	@param	BoneName	 If a SkeletalMeshComponent, name of body to apply force to. 'None' indicates root body.
	 *  @param  bAccelChange If true, Force is taken as a change in acceleration instead of a physical force (i.e. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddForce(FVector Force, FName BoneName = NAME_None, bool bAccelChange = false);

	/**
	 *	Add a force to a single rigid body at a particular location in world space.
	 *  This is like a 'thruster'. Good for adding a burst over some (non zero) time. Should be called every frame for the duration of the force.
	 *
	 *	@param Force		Force vector to apply. Magnitude indicates strength of force.
	 *	@param Location		Location to apply force, in world space.
	 *	@param BoneName		If a SkeletalMeshComponent, name of body to apply force to. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddForceAtLocation(FVector Force, FVector Location, FName BoneName = NAME_None);

	/**
	 *	Add a force to a single rigid body at a particular location. Both Force and Location should be in body space.
	 *  This is like a 'thruster'. Good for adding a burst over some (non zero) time. Should be called every frame for the duration of the force.
	 *
	 *	@param Force		Force vector to apply. Magnitude indicates strength of force.
	 *	@param Location		Location to apply force, in component space.
	 *	@param BoneName		If a SkeletalMeshComponent, name of body to apply force to. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddForceAtLocationLocal(FVector Force, FVector Location, FName BoneName = NAME_None);

	/**
	 *	Add a force to all bodies in this component, originating from the supplied world-space location.
	 *
	 *	@param Origin		Origin of force in world space.
	 *	@param Radius		Radius within which to apply the force.
	 *	@param Strength		Strength of force to apply.
	 *  @param Falloff		Allows you to control the strength of the force as a function of distance from Origin.
	 *  @param bAccelChange If true, Strength is taken as a change in acceleration instead of a physical force (i.e. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddRadialForce(FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bAccelChange = false);

	/**
	 *	Add a torque to a single rigid body.
	 *	@param Torque		Torque to apply. Direction is axis of rotation and magnitude is strength of torque.
	 *	@param BoneName		If a SkeletalMeshComponent, name of body to apply torque to. 'None' indicates root body.
	 *  @param bAccelChange If true, Torque is taken as a change in angular acceleration instead of a physical torque (i.e. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void AddTorqueInRadians(FVector Torque, FName BoneName = NAME_None, bool bAccelChange = false);

	/**
	 *	Add a torque to a single rigid body.
	 *	@param Torque		Torque to apply. Direction is axis of rotation and magnitude is strength of torque.
	 *	@param BoneName		If a SkeletalMeshComponent, name of body to apply torque to. 'None' indicates root body.
	 *	@param bAccelChange If true, Torque is taken as a change in angular acceleration instead of a physical torque (i.e. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	void AddTorqueInDegrees(FVector Torque, FName BoneName = NAME_None, bool bAccelChange = false)
	{
		AddTorqueInRadians(FMath::DegreesToRadians(Torque), BoneName, bAccelChange);
	}

	/**
	 *	Set the linear velocity of a single body.
	 *	This should be used cautiously - it may be better to use AddForce or AddImpulse.
	 *
	 *	@param NewVel			New linear velocity to apply to physics.
	 *	@param bAddToCurrent	If true, NewVel is added to the existing velocity of the body.
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to modify velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void SetPhysicsLinearVelocity(FVector NewVel, bool bAddToCurrent = false, FName BoneName = NAME_None);

	/** 
	 *	Get the linear velocity of a single body. 
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to get velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))	
	ENGINE_API FVector GetPhysicsLinearVelocity(FName BoneName = NAME_None) const;

	/**
	*	Get the linear velocity of a point on a single body.
	*	@param Point			Point is specified in world space.
	*	@param BoneName			If a SkeletalMeshComponent, name of body to get velocity of. 'None' indicates root body.
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API FVector GetPhysicsLinearVelocityAtPoint(FVector Point, FName BoneName = NAME_None) const;

	/**
	 *	Set the linear velocity of all bodies in this component.
	 *
	 *	@param NewVel			New linear velocity to apply to physics.
	 *	@param bAddToCurrent	If true, NewVel is added to the existing velocity of the body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void SetAllPhysicsLinearVelocity(FVector NewVel, bool bAddToCurrent = false);

	/**
	 *	Set the angular velocity of a single body.
	 *	This should be used cautiously - it may be better to use AddTorque or AddImpulse.
	 *
	 *	@param NewAngVel		New angular velocity to apply to body, in radians per second.
	 *	@param bAddToCurrent	If true, NewAngVel is added to the existing angular velocity of the body.
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to modify angular velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void SetPhysicsAngularVelocityInRadians(FVector NewAngVel, bool bAddToCurrent = false, FName BoneName = NAME_None);

	/**
	 *	Set the angular velocity of a single body.
	 *	This should be used cautiously - it may be better to use AddTorque or AddImpulse.
	 *
	 *	@param NewAngVel		New angular velocity to apply to body, in degrees per second.
	 *	@param bAddToCurrent	If true, NewAngVel is added to the existing angular velocity of the body.
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to modify angular velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	void SetPhysicsAngularVelocityInDegrees(FVector NewAngVel, bool bAddToCurrent = false, FName BoneName = NAME_None)
	{
		SetPhysicsAngularVelocityInRadians(FMath::DegreesToRadians(NewAngVel), bAddToCurrent, BoneName);
	}

	/**
	*	Set the maximum angular velocity of a single body.
	*
	*	@param NewMaxAngVel		New maximum angular velocity to apply to body, in degrees per second.
	*	@param bAddToCurrent	If true, NewMaxAngVel is added to the existing maximum angular velocity of the body.
	*	@param BoneName			If a SkeletalMeshComponent, name of body to modify maximum angular velocity of. 'None' indicates root body.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	void SetPhysicsMaxAngularVelocityInDegrees(float NewMaxAngVel, bool bAddToCurrent = false, FName BoneName = NAME_None)
	{
		SetPhysicsMaxAngularVelocityInRadians(FMath::DegreesToRadians(NewMaxAngVel), bAddToCurrent, BoneName);
	}

	/**
	*	Set the maximum angular velocity of a single body.
	*
	*	@param NewMaxAngVel		New maximum angular velocity to apply to body, in radians per second.
	*	@param bAddToCurrent	If true, NewMaxAngVel is added to the existing maximum angular velocity of the body.
	*	@param BoneName			If a SkeletalMeshComponent, name of body to modify maximum angular velocity of. 'None' indicates root body.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void SetPhysicsMaxAngularVelocityInRadians(float NewMaxAngVel, bool bAddToCurrent = false, FName BoneName = NAME_None);

	/** 
	 *	Get the angular velocity of a single body, in degrees per second. 
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to get velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))	
	FVector GetPhysicsAngularVelocityInDegrees(FName BoneName = NAME_None) const
	{
		return FMath::RadiansToDegrees(GetPhysicsAngularVelocityInRadians(BoneName));
	}

	/** 
	 *	Get the angular velocity of a single body, in radians per second. 
	 *	@param BoneName			If a SkeletalMeshComponent, name of body to get velocity of. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))	
	ENGINE_API FVector GetPhysicsAngularVelocityInRadians(FName BoneName = NAME_None) const;

	/**
	*	Get the center of mass of a single body. In the case of a welded body this will return the center of mass of the entire welded body (including its parent and children)
	*   Objects that are not simulated return (0,0,0) as they do not have COM
	*	@param BoneName			If a SkeletalMeshComponent, name of body to get center of mass of. 'None' indicates root body.
	*/
	UFUNCTION(BlueprintPure, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API FVector GetCenterOfMass(FName BoneName = NAME_None) const;

	/**
	*	Set the center of mass of a single body. This will offset the physx-calculated center of mass.
	*	Note that in the case where multiple bodies are attached together, the center of mass will be set for the entire group.
	*	@param CenterOfMassOffset		User specified offset for the center of mass of this object, from the calculated location.
	*	@param BoneName			If a SkeletalMeshComponent, name of body to set center of mass of. 'None' indicates root body.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void SetCenterOfMass(FVector CenterOfMassOffset, FName BoneName = NAME_None);

	/**
	 *	'Wake' physics simulation for a single body.
	 *	@param	BoneName	If a SkeletalMeshComponent, name of body to wake. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void WakeRigidBody(FName BoneName = NAME_None);

	/** 
	 *	Force a single body back to sleep. 
	 *	@param	BoneName	If a SkeletalMeshComponent, name of body to put to sleep. 'None' indicates root body.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void PutRigidBodyToSleep(FName BoneName = NAME_None);

	/** Changes the value of bNotifyRigidBodyCollision */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision);

	/** Changes the value of bOwnerNoSee. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetOwnerNoSee(bool bNewOwnerNoSee);
	
	/** Changes the value of bOnlyOwnerSee. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetOnlyOwnerSee(bool bNewOnlyOwnerSee);

	/** Changes the value of bIsVisibleInRayTracing. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetVisibleInRayTracing(bool bNewVisibleInRayTracing);

	/** Changes the value of CastShadow. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetCastShadow(bool NewCastShadow);

	/** Changes the value of EmissiveLightSource. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetEmissiveLightSource(bool NewEmissiveLightSource);

	/** Changes the value of CastHiddenShadow. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetCastHiddenShadow(bool NewCastHiddenShadow);

	/** Changes the value of CastInsetShadow. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetCastInsetShadow(UPARAM(DisplayName="CastInsetShadow") bool bInCastInsetShadow);

	/** Changes the value of bCastContactShadow. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetCastContactShadow(UPARAM(DisplayName = "CastContactShadow") bool bInCastContactShadow);

	/** Changes the value of LightAttachmentsAsGroup. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetLightAttachmentsAsGroup(UPARAM(DisplayName="LightAttachmentsAsGroup") bool bInLightAttachmentsAsGroup);

	/** Changes the value of ExcludeFromLightAttachmentGroup. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetExcludeFromLightAttachmentGroup(UPARAM(DisplayName = "ExcludeFromLightAttachmentGroup") bool bInExcludeFromLightAttachmentGroup);

	/** Changes the value of bSingleSampleShadowFromStationaryLights. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetSingleSampleShadowFromStationaryLights(bool bNewSingleSampleShadowFromStationaryLights);

	/** Changes the value of TranslucentSortPriority. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetTranslucentSortPriority(int32 NewTranslucentSortPriority);

	/** Changes the value of TranslucencySortDistanceOffset. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetTranslucencySortDistanceOffset(float NewTranslucencySortDistanceOffset);

	/** Changes the value of Affect Distance Field Lighting */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAffectDistanceFieldLighting(bool NewAffectDistanceFieldLighting);

	/** Changes the value of bReceivesDecals. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetReceivesDecals(bool bNewReceivesDecals);

    /** Changes the value of bHoldout */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHoldout(bool bNewHoldout);

    /** Changes the value of bAffectDynamicIndirectLighting */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAffectDynamicIndirectLighting(bool bNewAffectDynamicIndirectLighting);

    /** Changes the value of bAffectIndirectLightingWhileHidden */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAffectIndirectLightingWhileHidden(bool bNewAffectIndirectLightingWhileHidden);


	/** Controls what kind of collision is enabled for this body */
	UFUNCTION(BlueprintCallable, Category="Collision")
	ENGINE_API virtual void SetCollisionEnabled(ECollisionEnabled::Type NewType);

	/**  
	 * Set Collision Profile Name
	 * This function is called by constructors when they set ProfileName
	 * This will change current CollisionProfileName to be this, and overwrite Collision Setting
	 * 
	 * @param InCollisionProfileName : New Profile Name
	 */
	UFUNCTION(BlueprintCallable, Category="Collision")	
	ENGINE_API virtual void SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps=true);

	/** Get the collision profile name */
	UFUNCTION(BlueprintPure, Category="Collision")
	ENGINE_API FName GetCollisionProfileName() const;

	/**
	 *	Changes the collision channel that this object uses when it moves
	 *	@param      Channel     The new channel for this component to use
	 */
	UFUNCTION(BlueprintCallable, Category="Collision")	
	ENGINE_API virtual void SetCollisionObjectType(ECollisionChannel Channel);

	/** Perform a line trace against a single component
	 * @param TraceStart The start of the trace in world-space
	 * @param TraceEnd The end of the trace in world-space
	 * @param bTraceComplex Whether or not to trace the complex physics representation or just the simple representation
	 * @param bShowTrace Whether or not to draw the trace in the world (for debugging)
	 * @param bPersistentShowTrace Whether or not to make the debugging draw stay in the world permanently
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(DisplayName = "Line Trace Component", ScriptName = "LineTraceComponent", bTraceComplex="true", bPersistentShowTrace="false", UnsafeDuringActorConstruction="true"))	
	ENGINE_API bool K2_LineTraceComponent(FVector TraceStart, FVector TraceEnd, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit);

	/** Perform a sphere trace against a single component
	* @param TraceStart The start of the trace in world-space
	* @param TraceEnd The end of the trace in world-space
	* @param SphereRadius Radius of the sphere to trace against the component
	* @param bTraceComplex Whether or not to trace the complex physics representation or just the simple representation
	* @param bShowTrace Whether or not to draw the trace in the world (for debugging)
	* @param bPersistentShowTrace Whether or not to make the debugging draw stay in the world permanently
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (DisplayName = "Sphere Trace Component", ScriptName = "SphereTraceComponent", bTraceComplex = "true", bPersistentShowTrace="false", UnsafeDuringActorConstruction = "true"))
	ENGINE_API bool K2_SphereTraceComponent(FVector TraceStart, FVector TraceEnd, float SphereRadius, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit);

	/** Perform a box overlap against a single component as an AABB (No rotation)
	* @param InBoxCentre The centre of the box to overlap with the component
	* @param InBox Description of the box to use in the overlap
	* @param bTraceComplex Whether or not to trace the complex physics representation or just the simple representation
	* @param bShowTrace Whether or not to draw the trace in the world (for debugging)
	* @param bPersistentShowTrace Whether or not to make the debugging draw stay in the world permanently
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (DisplayName = "Box Overlap Component", ScriptName = "BoxOverlapComponent", bTraceComplex = "true", bPersistentShowTrace="false", UnsafeDuringActorConstruction = "true"))
	ENGINE_API bool K2_BoxOverlapComponent(FVector InBoxCentre, const FBox InBox, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit);

	/** Perform a sphere overlap against a single component
	* @param InSphereCentre The centre of the sphere to overlap with the component
	* @param InSphereRadius The Radius of the sphere to overlap with the component
	* @param bTraceComplex Whether or not to trace the complex physics representation or just the simple representation
	* @param bShowTrace Whether or not to draw the trace in the world (for debugging)
	* @param bPersistentShowTrace Whether or not to make the debugging draw stay in the world permanently
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (DisplayName = "Sphere Overlap Component", ScriptName = "SphereOverlapComponent", bTraceComplex = "true", bPersistentShowTrace="false", UnsafeDuringActorConstruction = "true"))
	ENGINE_API bool K2_SphereOverlapComponent(FVector InSphereCentre, float InSphereRadius, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit);

	/** Sets the bRenderCustomDepth property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetRenderCustomDepth(bool bValue);

	/** Sets the CustomDepth stencil value (0 - 255) and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta=(UIMin = "0", UIMax = "255"))
	ENGINE_API void SetCustomDepthStencilValue(int32 Value);

	/** Sets the CustomDepth stencil write mask and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetCustomDepthStencilWriteMask(ERendererStencilMask WriteMaskBit);

	/** Sets bRenderInMainPass property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRenderInMainPass(bool bValue);
	
	/** Sets bRenderInDepthPass property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRenderInDepthPass(bool bValue);

	/** Sets bVisibleInSceneCaptureOnly property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetVisibleInSceneCaptureOnly(bool bValue);

	/** Sets bHideInSceneCapture property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHiddenInSceneCapture(bool bValue);

	/** Sets FirstPersonPrimitiveType property and marks the render state dirty. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetFirstPersonPrimitiveType(EFirstPersonPrimitiveType Value);

	/**
	 * Count of all component overlap events (begin or end) ever generated for any components.
	 * Changes to this number within a scope can also be a simple way to know if any events were triggered.
	 * It can also be useful for identifying performance issues due to high numbers of events.
	 */
	static ENGINE_API uint32 GlobalOverlapEventsCounter;

	/** The old primitive's scene info ptr, now superceded by the ptr in SceneData, but it's still here due to pervasive usage. */
	FPrimitiveSceneProxy* SceneProxy;

	FPrimitiveSceneProxy* GetSceneProxy() const { check(SceneProxy == SceneData.SceneProxy); return SceneData.SceneProxy; }

	ENGINE_API void AssignSceneProxy(FPrimitiveSceneProxy* SceneProxy);
	ENGINE_API void ReleaseSceneProxy();

	/** Returns whether the primitive component's scene proxy is always visible. */
	bool IsAlwaysVisible() const { return SceneData.bAlwaysVisible; }

	/** A fence to track when the primitive is detached from the scene in the rendering thread. */
	FRenderCommandFence DetachFence;

private:
	/** LOD parent primitive to draw instead of this one (multiple UPrim's will point to the same LODParent ) */
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<class UPrimitiveComponent> LODParentPrimitive;

	/* Returns true if component can be excluded from specific HLOD levels */
	UFUNCTION()
	bool AllowHLODLevelsExclusion() const;

public:

	/** Set the LOD parent component */
	ENGINE_API void SetLODParentPrimitive(UPrimitiveComponent* InLODParentPrimitive);

	/** Gets the LOD Parent, which is used to compute visibility when hierarchical LOD is enabled */
	ENGINE_API UPrimitiveComponent* GetLODParentPrimitive() const;

#if WITH_EDITOR
	/** This function is used to create hierarchical LOD for the level. You can decide to opt out if you don't want. */
	ENGINE_API virtual const bool ShouldGenerateAutoLOD(const int32 HierarchicalLevelIndex) const;
#endif
	/** Return true if the owner is selected and this component is selectable */
	ENGINE_API virtual bool ShouldRenderSelected() const;

	/** Returns true if the owning actor is part of a level instance which is being edited. */
	ENGINE_API bool GetLevelInstanceEditingState() const;

	/** Component is directly selected in the editor separate from its parent actor */
	ENGINE_API bool IsComponentIndividuallySelected() const;

	/** Return True if a primitive's parameters as well as its position is static during gameplay, and can thus use static lighting. */
	ENGINE_API bool HasStaticLighting() const;

	/** Return true if primitive can skip getting texture streaming render asset info. */
	ENGINE_API bool CanSkipGetTextureStreamingRenderAssetInfo() const;

	virtual float GetStreamingScale() const { return 1.f; }

	/** Returns true if the component is static and has the right static mesh setup to support lightmaps. */
	virtual bool HasValidSettingsForStaticLighting(bool bOverlookInvalidComponents) const 
	{
		return HasStaticLighting();
	}

	/** Returns true if only unlit materials are used for rendering, false otherwise. */
	ENGINE_API virtual bool UsesOnlyUnlitMaterials() const;

	/**
	 * Returns the lightmap resolution used for this primitive instance in the case of it supporting texture light/ shadow maps.
	 * 0 if not supported or no static shadowing.
	 *
	 * @param	Width	[out]	Width of light/shadow map
	 * @param	Height	[out]	Height of light/shadow map
	 * @return	bool			true if LightMap values are padded, false if not
	 */
	ENGINE_API virtual bool GetLightMapResolution( int32& Width, int32& Height ) const;

	/**
	 *	Returns the static lightmap resolution used for this primitive.
	 *	0 if not supported or no static shadowing.
	 *
	 * @return	int32		The StaticLightmapResolution for the component
	 */
	virtual int32 GetStaticLightMapResolution() const { return 0; }

	/**
	 * Returns the light and shadow map memory for this primitive in its out variables.
	 *
	 * Shadow map memory usage is per light whereof lightmap data is independent of number of lights, assuming at least one.
	 *
	 * @param [out] LightMapMemoryUsage		Memory usage in bytes for light map (either texel or vertex) data
	 * @param [out]	ShadowMapMemoryUsage	Memory usage in bytes for shadow map (either texel or vertex) data
	 */
	ENGINE_API virtual void GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const;

#if WITH_EDITOR
	/**
	 * Requests the information about the component that the static lighting system needs.
	 * @param OutPrimitiveInfo - Upon return, contains the component's static lighting information.
	 * @param InRelevantLights - The lights relevant to the primitive.
	 * @param InOptions - The options for the static lighting build.
	 */
	virtual void GetStaticLightingInfo(struct FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<class ULightComponent*>& InRelevantLights,const class FLightingBuildOptions& Options) {}

	/** Add the used GUIDs from UMapBuildDataRegistry::MeshBuildData. Used to preserve hidden level data in lighting scenario. */
	virtual void AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const {}

	/**
	 *	Remaps the texture streaming built data that was built for the actor back to the level.
	 *	
	 */
	virtual bool RemapActorTextureStreamingBuiltDataToLevel(const class UActorTextureStreamingBuildDataComponent* InActorTextureBuildData) { return false; }

	/** Computes a hash of component's texture streaming built data. */
	virtual uint32 ComputeHashTextureStreamingBuiltData() const { return 0; }
#endif // WITH_EDITOR

	/**
	 *	Requests whether the component will use texture, vertex or no lightmaps.
	 *
	 *	@return	ELightMapInteractionType		The type of lightmap interaction the component will use.
	 */
	virtual ELightMapInteractionType GetStaticLightingType() const	{ return LMIT_None;	}

	/**
	 * Returns the streamable Nanite asset owned by this primitive.
	 */
	ENGINE_API virtual UStreamableRenderAsset* GetStreamableNaniteAsset() const;

	/**
	 * Enumerates the streaming textures/meshes used by the primitive.
	 * @param LevelContext - Level scope context used to process texture streaming build data.
	 * @param OutStreamingRenderAssets - Upon return, contains a list of the streaming textures/meshes used by the primitive.
	 */
	ENGINE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const;

	/**
	 * Call GetStreamingRenderAssetInfo and remove the elements with a NULL texture
	 * @param OutStreamingRenderAssets - Upon return, contains a list of the non-null streaming textures or meshes used by the primitive.
	 */
	UE_DEPRECATED(5.8, "This has been replaced with LevelContext.GetStreamingRenderAssetInfoWithNULLRemoval")
	ENGINE_API void GetStreamingRenderAssetInfoWithNULLRemoval(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const;

	/**
	 *	Update the streaming data of this component.
	 *
	 *	@param	BuildType		[in]		The type of build. Affects what the build is allowed to do.
	 *	@param	QualityLevel	[in]		The quality level being used in the texture streaming build.
	 *	@param	FeatureLevel	[in]		The feature level being used in the texture streaming build.
	 *	@param	DependentResources [out]	The resource the build depends on.
	 *	@return								Returns false if some data needs rebuild but couldn't be rebuilt (because of the build type).
	 */
	ENGINE_API bool BuildTextureStreamingData(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources);

	/**
	 *	Component type implementation of updating the streaming data of this component.
	 *
	 *	@param	BuildType		[in]		The type of build. Affects what the build is allowed to do.
	 *	@param	QualityLevel	[in]		The quality level being used in the texture streaming build.
	 *	@param	FeatureLevel	[in]		The feature level being used in the texture streaming build.
	 *	@param	DependentResources [out]	The resource the build depends on.
	 *	@return								Returns false if some data needs rebuild but couldn't be rebuilt (because of the build type).
	 */
	ENGINE_API virtual bool BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData);

	/**
	 * Determines the DPG the primitive's primary elements are drawn in.
	 * Even if the primitive's elements are drawn in multiple DPGs, a primary DPG is needed for occlusion culling and shadow projection.
	 * @return The DPG the primitive's primary elements will be drawn in.
	 */
	virtual ESceneDepthPriorityGroup GetStaticDepthPriorityGroup() const { return DepthPriorityGroup; }

	/** 
	 * Retrieves the materials used in this component 
	 * 
	 * @param OutMaterials	The list of used materials.
	 */
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const {}

	/**
	 * Returns the material textures used to render this primitive for the given platform.
	 * Internally calls GetUsedMaterials() and GetUsedTextures() for each material.
	 *
	 * @param OutTextures	[out] The list of used textures.
	 */
	ENGINE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel);

	/** Return the BodySetup to use for this PrimitiveComponent (single body case) */
	virtual class UBodySetup* GetBodySetup() { return NULL; }

	/** Move this component to match the physics rigid body pose. Note, a warning will be generated if you call this function on a component that is attached to something */
	ENGINE_API void SyncComponentToRBPhysics();
	
	/** 
	 * Returns the matrix that should be used to render this component. 
	 * Allows component class to perform graphical distortion to the component not supported by an FTransform 
	 * NOTE: When overriding this method to alter the transform used for rendering it is typically neccessary to also implement USceneComponent::UpdateBounds.
	 *       Otherwise the Local bounds will not match the world space bounds, causing incorrect culling.
	 */
	ENGINE_API virtual FMatrix GetRenderMatrix() const;

	/** Return number of material elements in this primitive */
	UFUNCTION(BlueprintPure, Category="Rendering|Material")
	ENGINE_API virtual int32 GetNumMaterials() const override;
	
	/**
	 * Returns BodyInstance of the component.
	*
	* @param BoneName				Used to get body associated with specific bone. NAME_None automatically gets the root most body
	* @param bGetWelded				If the component has been welded to another component and bGetWelded is true we return the single welded BodyInstance that is used in the simulation
	* @param Index					Index used in Components with multiple body instances
	*
	* @return		Returns the BodyInstance based on various states (does component have multiple bodies? Is the body welded to another body?)
	*/
	ENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;

	/**
	 * Returns BodyInstanceAsyncPhysicsTickHandle of the component. For use in the Async Physics Tick event
	*
	* @param BoneName				Used to get body associated with specific bone. NAME_None automatically gets the root most body
	* @param bGetWelded				If the component has been welded to another component and bGetWelded is true we return the single welded BodyInstance that is used in the simulation
	* @param Index					Index used in Components with multiple body instances
	*
	* @return		Returns the BodyInstanceAsyncPhysicsTickHandle based on various states (does component have multiple bodies? Is the body welded to another body?)
	*/
	UFUNCTION(BlueprintPure, Category = "Physics")
		ENGINE_API FBodyInstanceAsyncPhysicsTickHandle GetBodyInstanceAsyncPhysicsTickHandle(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = -1) const;

	/** 
	 * Returns The square of the distance to closest Body Instance surface. 
	 *
	 * @param Point				World 3D vector
	 * @param OutSquaredDistance The squared distance to closest Body Instance surface. 0 if inside of the body
	 * @param OutPointOnBody	Point on the surface of collision closest to Point
	 * 
	 * @return		true if a distance to the body was found and OutDistanceSquared has been populated
	 */
	ENGINE_API virtual bool GetSquaredDistanceToCollision(const FVector& Point, float& OutSquaredDistance, FVector& OutClosestPointOnCollision) const;

	/** 
	 * Returns Distance to closest Body Instance surface. 
	*
	* @param Point				World 3D vector
	* @param OutPointOnBody	Point on the surface of collision closest to Point
	* 
	* @return		Success if returns > 0.f, if returns 0.f, point is inside the geometry
	*				If returns < 0.f, this primitive does not have collsion or if geometry is not supported
	*/	
	float GetDistanceToCollision(const FVector& Point, FVector& ClosestPointOnCollision) const 
	{
		float DistanceSqr = -1.f;
		return (GetSquaredDistanceToCollision(Point, DistanceSqr, ClosestPointOnCollision) ? FMath::Sqrt(DistanceSqr) : -1.f);
	}

	/**
	* Returns the distance and closest point to the collision surface.
	* Component must have simple collision to be queried for closest point.
	*
	* @param Point				World 3D vector
	* @param OutPointOnBody		Point on the surface of collision closest to Point
	* @param BoneName			If a SkeletalMeshComponent, name of body to set center of mass of. 'None' indicates root body.
	*
	* @return		Success if returns > 0.f, if returns 0.f, it is either not convex or inside of the point
	*				If returns < 0.f, this primitive does not have collsion
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API float GetClosestPointOnCollision(const FVector& Point, FVector& OutPointOnBody, FName BoneName = NAME_None) const;

	/**
	 * Creates a proxy to represent the primitive to the scene manager in the rendering thread.
	 * @return The proxy object.
	 */
	virtual FPrimitiveSceneProxy* CreateSceneProxy()
	{
		return NULL;
	}

#if WITH_EDITOR
	/**
	 * Creates a HHitProxy to represent the component at SectionIndex / MaterialIndex
	 * @return The proxy object.
	 */
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const
	{
		return nullptr;
	}
#endif

	/**
	 * Determines whether the proxy for this primitive type needs to be recreated whenever the primitive moves.
	 * @return true to recreate the proxy when UpdateTransform is called.
	 */
	virtual bool ShouldRecreateProxyOnUpdateTransform() const
	{
		return false;
	}

	/** 
	 * This isn't bound extent, but for shape component to utilize extent is 0. 
	 * For normal primitive, this is 0, for ShapeComponent, this will have valid information
	 */
	virtual bool IsZeroExtent() const
	{
		return false;
	}

	/** Called when the CollisionEnabled of the BodyInstance is changed. */
	ENGINE_API virtual void RecreateInstanceBody(int32 InstanceBodyIndex);

	/** Event called when a component is 'damaged', allowing for component class specific behaviour */
	ENGINE_API virtual void ReceiveComponentDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser);

	/**
	*   Welds this component to another scene component, optionally at a named socket. Component is automatically attached if not already
	*	Welding allows the child physics object to become physically connected to its parent. This is useful for creating compound rigid bodies with correct mass distribution.
	*   @param InParent the component to be physically attached to
	*   @param InSocketName optional socket to attach component to
	*	@param bWeldToKinematicParent if true, children will be welded onto the parent even if the parent is kinematic (default false)
	*
	* By default if the root is kinematic then the welded bodies are set to kinematic rather than actually welded. This is beneficial if
	* the actor is not moved very often or only contains a few welded shapes. However it can be expensive to move a kinematic actor
	* that contains a large number of (unwelded) kinematic children, in which case you can setting bWeldToKinematicParent to true to generate
	* a welded kinematic actor. There is no real benefit to welding if the actor does not move and, since the initial weld cost is fairly high,
	* you generally would not enable welding on all kinematics in the world if you have a lot of them.
	*/
	ENGINE_API virtual void WeldTo(class USceneComponent* InParent, FName InSocketName = NAME_None, bool bWeldToKinematicParent = false);

	/**
	*	Does the actual work for welding.
	*	@param bWeldSimulatedChild if true, simulated children will be welded onto the parent (default true)
	*	@param bWeldToKinematicParent if true, children will be welded onto the parent even if the parent is kinematic (default false)
	*	@return true if did a true weld of shapes, meaning body initialization is not needed
	*/
	ENGINE_API virtual bool WeldToImplementation(USceneComponent * InParent, FName ParentSocketName = NAME_None, bool bWeldSimulatedChild = true, bool bWeldToKinematicParent = false);

	/**
	*   UnWelds this component from its parent component. Attachment is maintained (DetachFromParent automatically unwelds)
	*/
	ENGINE_API virtual void UnWeldFromParent();

	/**
	*   Unwelds the children of this component. Attachment is maintained
	*/
	ENGINE_API virtual void UnWeldChildren();

	/**
	*	Adds the bodies that are currently welded to the OutWeldedBodies array 
	*/
	ENGINE_API virtual void GetWeldedBodies(TArray<FBodyInstance*> & OutWeldedBodies, TArray<FName> & OutLabels, bool bIncludingAutoWeld = false);

	/** Whether the component has been welded to another simulating component */
	ENGINE_API bool IsWelded() const;
	
	/**
	 * Called to get the Component To World Transform from the Root BodyInstance
	 * This needs to be virtual since SkeletalMeshComponent Root has to undo its own transform
	 * Without this, the root LocalToAtom is overridden by physics simulation, causing kinematic velocity to 
	 * accelerate simulation
	 *
	 * @param : UseBI - root body instance
	 * @return : New GetComponentTransform() to use
	 */
	ENGINE_API virtual FTransform GetComponentTransformFromBodyInstance(FBodyInstance* UseBI);	

	/**
	 * Would this primitive be shown with these rendering flags.
	 * 
	 * @Note: Currently this only implemented properly for the editor selectable primitives.
	 */
	ENGINE_API virtual bool IsShown(const FEngineShowFlags& ShowFlags) const;

	/** Returns false if this primitive should never output velocity based on its WPO state. */
	virtual bool SupportsWorldPositionOffsetVelocity() const { return true; }

#if WITH_EDITOR
	/**
	 * Determines whether the supplied bounding box intersects with the component.
	 * Used by the editor in orthographic viewports.
	 *
	 * @param	InSelBBox						Bounding box to test against
	 * @param	bConsiderOnlyBSP				If only BSP geometry should be tested
	 * @param	bMustEncompassEntireComponent	Whether the component bounding box must lay wholly within the supplied bounding box
	 *
	 * @return	true if the supplied bounding box is determined to intersect the component (partially or wholly)
	 */
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const;

	/**
	 * Determines whether the supplied frustum intersects with the component.
	 * Used by the editor in perspective viewports.
	 *
	 * @param	InFrustum						Frustum to test against
	 * @param	bConsiderOnlyBSP				If only BSP geometry should be tested
	 * @param	bMustEncompassEntireComponent	Whether the component bounding box must lay wholly within the supplied bounding box
	 *
	 * @return	true if the supplied bounding box is determined to intersect the component (partially or wholly)
	 */
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const;
#endif

protected:

	/** Give the static mesh component recreate render state context access to Create/DestroyRenderState_Concurrent(). */
	friend class FStaticMeshComponentRecreateRenderStateContext;

	/** Whether the component type supports static lighting. */
	virtual bool SupportsStaticLighting() const 
	{
		return false;
	}

public:
#if UE_ENABLE_DEBUG_DRAWING
	/** Updates the renderer with the center of mass data */
	ENGINE_API virtual void SendRenderDebugPhysics(FPrimitiveSceneProxy* OverrideSceneProxy = nullptr);
private:
	/** Only currently used for static mesh primitives, but could eventually be applied to other types */
	static ENGINE_API void BatchSendRenderDebugPhysics(TArrayView<UPrimitiveComponent*> InPrimitives);
	friend class FStaticMeshComponentBulkReregisterContext;
public:
#endif

	//~ Begin UActorComponent Interface
	using Super::SendRenderDynamicData_Concurrent;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void OnRegister()  override;
	ENGINE_API virtual void OnUnregister()  override;
	ENGINE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
	ENGINE_API virtual void OnActorEnableCollisionChanged() override;
	ENGINE_API virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
	ENGINE_API virtual bool IsEditorOnly() const override;
	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	ENGINE_API virtual bool HasValidPhysicsState() const override;
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const;
#endif // WITH_EDITOR	
	//~ End UActorComponent Interface

	//~ Begin IAsyncPhysicsStateProcessor
	ENGINE_API virtual void OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	ENGINE_API virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
	ENGINE_API virtual void CollectBodySetupsWithPhysicsMeshesToCreate(TSet<UBodySetup*>& OutBodySetups) const override;
	//~ End IAsyncPhysicsStateProcessor

protected:
	/** Internal function that updates physics objects to match the component collision settings. */
	ENGINE_API virtual void UpdatePhysicsToRBChannels();

	/** Called to send a transform update for this component to the physics engine */
	ENGINE_API void SendPhysicsTransform(ETeleportType Teleport);

	/** Ensure physics state created **/
	ENGINE_API void EnsurePhysicsStateCreated();

	/**  Go through attached primitive components and call MarkRenderStateDirty */
	ENGINE_API void MarkChildPrimitiveComponentRenderStateDirty();

	/** Conditionally notify streamers that this primitive has updated its render state */
	ENGINE_API void ConditionalNotifyStreamingPrimitiveUpdated_Concurrent() const;
public:

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual bool NeedsLoadForClient() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void UpdateCollisionProfile();
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin USceneComponent Interface

	/** Returns the form of collision for this component */
	UFUNCTION(BlueprintPure, Category="Collision")
	ENGINE_API virtual ECollisionEnabled::Type GetCollisionEnabled() const override;

	/** Utility to see if there is any form of collision (query or physics) enabled on this component. */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Collision Enabled", ScriptName="IsCollisionEnabled"), Category="Collision")
	bool K2_IsCollisionEnabled() const;

	/** Utility to see if there is any query collision enabled on this component. */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Query Collision Enabled", ScriptName="IsQueryCollisionEnabled"), Category="Collision")
	bool K2_IsQueryCollisionEnabled() const;

	/** Utility to see if there is any physics collision enabled on this component. */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Physics Collision Enabled", ScriptName="IsPhysicsCollisionEnabled"), Category="Collision")
	bool K2_IsPhysicsCollisionEnabled() const;

	/** Gets the response type given a specific channel */
	UFUNCTION(BlueprintPure, Category="Collision")
	ENGINE_API virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const override;

	/** Gets the collision object type */
	UFUNCTION(BlueprintPure, Category="Collision")
	ENGINE_API virtual ECollisionChannel GetCollisionObjectType() const override;
	
	ENGINE_API virtual void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex);
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None) override;
	ENGINE_API virtual void OnAttachmentChanged() override;
	ENGINE_API virtual bool IsSimulatingPhysics(FName BoneName = NAME_None) const override;
	ENGINE_API virtual bool MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit = NULL, EMoveComponentFlags MoveFlags = MOVECOMP_NoFlags, ETeleportType Teleport = ETeleportType::None) override;
	ENGINE_API virtual bool IsWorldGeometry() const override;
	ENGINE_API virtual const FCollisionResponseContainer& GetCollisionResponseToChannels() const override;
	ENGINE_API virtual FVector GetComponentVelocity() const override;
#if WITH_EDITOR
	ENGINE_API virtual void UpdateBounds() override;
	ENGINE_API virtual const int32 GetNumUncachedStaticLightingInteractions() const override;
#endif
	//~ End USceneComponentInterface

	ENGINE_API void UpdateOcclusionBoundsSlack(float NewSlack);

	/**
	 * Dispatch notifications for the given HitResult.
	 *
	 * @param Owner: AActor that owns this component
	 * @param BlockingHit: FHitResult that generated the blocking hit.
	 */
	ENGINE_API void DispatchBlockingHit(AActor& OutOwner, FHitResult const& BlockingHit);

	/**
	 * Dispatch notification for wake events and propagate to any welded bodies
	 */

	ENGINE_API void DispatchWakeEvents(ESleepEvent WakeEvent, FName BoneName);

	/**
	 * Whether or not the primitive component should dispatch sleep/wake events.
	 */
	ENGINE_API virtual bool ShouldDispatchWakeEvents(FName BoneName) const;

	/**
	 * Set collision params on OutParams (such as CollisionResponse) to match the settings on this PrimitiveComponent.
	 */
	ENGINE_API virtual void InitSweepCollisionParams(FCollisionQueryParams &OutParams, FCollisionResponseParams& OutResponseParam) const;

	/**
	 * Return a CollisionShape that most closely matches this primitive.
	 */
	ENGINE_API virtual struct FCollisionShape GetCollisionShape(float Inflation = 0.0f) const;

	/**
	 * Returns true if the given transforms result in the same bounds, due to rotational symmetry.
	 * For example, this is true for a sphere with uniform scale undergoing any rotation.
	 * This is NOT intended to detect every case where this is true, only the common cases to aid optimizations.
	 */
	virtual bool AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const { return A.Equals(B); }

	/**
	 * Pushes new selection state to the render thread primitive proxy
	 */
	ENGINE_API virtual void PushSelectionToProxy();

	/**
	 * Pushes new LevelInstance editing state to the render thread primitive proxy.
	 */
	ENGINE_API void PushLevelInstanceEditingStateToProxy(bool bInEditingState);

	/**
	 * Pushes new hover state to the render thread primitive proxy
	 * @param bInHovered - true if the proxy should display as if hovered
	 */
	ENGINE_API void PushHoveredToProxy(const bool bInHovered);

	/** Sends editor visibility updates to the render thread */
	ENGINE_API void PushEditorVisibilityToProxy( uint64 InVisibility );

	/** Gets the emissive boost for the primitive component. */
	virtual float GetEmissiveBoost(int32 ElementIndex) const		{ return 1.0f; };

	/** Gets the diffuse boost for the primitive component. */
	virtual float GetDiffuseBoost(int32 ElementIndex) const		{ return 1.0f; };
	
	/** Disable dynamic shadow casting if the primitive only casts indirect shadows, since dynamic shadows are always shadowing direct lighting */
	virtual bool GetShadowIndirectOnly() const { return false; }
	
	/** Returns whether this component is still being compiled or dependent on other objects being compiled. */
	virtual bool IsCompiling() const { return false; }

	ENGINE_API virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Sends primitive color updates to the render thread */
	ENGINE_API void PushPrimitiveColorToProxy(const FLinearColor& InPrimitiveColor);
#endif

#if WITH_EDITOR
	/** Returns mask that represents in which views this primitive is hidden */
	ENGINE_API virtual uint64 GetHiddenEditorViews() const;

	/** Sets whether this component is being moved by the editor so the renderer can render velocities for it, even when Static. */
	ENGINE_API void SetIsBeingMovedByEditor(bool bNewIsBeingMoved);


	ENGINE_API void SetSelectionOutlineColorIndex(uint8 SelectionOutlineColorIndex);
	
	ENGINE_API void SetOverlayColor(FColor OverlayColor);
	ENGINE_API void RemoveOverlayColor();
#endif// WITH_EDITOR

	/** Resets the cached scene velocity. Useful to prevent motion blur when teleporting components. See also SetIsBeingMovedByEditor(). */
	ENGINE_API void ResetSceneVelocity();

	/**
	 *	Set the angular velocity of all bodies in this component.
	 *
	 *	@param NewAngVel		New angular velocity to apply to physics, in degrees per second.
	 *	@param bAddToCurrent	If true, NewAngVel is added to the existing angular velocity of all bodies.
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (UnsafeDuringActorConstruction = "true"))
	void SetAllPhysicsAngularVelocityInDegrees(const FVector& NewAngVel, bool bAddToCurrent = false)
	{
		SetAllPhysicsAngularVelocityInRadians(FMath::DegreesToRadians(NewAngVel), bAddToCurrent);
	}

	/**
	 *	Set the angular velocity of all bodies in this component.
	 *
	 *	@param NewAngVel		New angular velocity to apply to physics, in radians per second.
	 *	@param bAddToCurrent	If true, NewAngVel is added to the existing angular velocity of all bodies.
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (UnsafeDuringActorConstruction = "true"))
	ENGINE_API virtual void SetAllPhysicsAngularVelocityInRadians(const FVector& NewAngVel, bool bAddToCurrent = false);

	/**
	 *	Set the position of all bodies in this component.
	 *	If a SkeletalMeshComponent, the root body will be placed at the desired position, and the same delta is applied to all other bodies.
	 *
	 *	@param	NewPos		New position for the body
	 */
	ENGINE_API virtual void SetAllPhysicsPosition(FVector NewPos);
	
	/**
	 *	Set the rotation of all bodies in this component.
	 *	If a SkeletalMeshComponent, the root body will be changed to the desired orientation, and the same delta is applied to all other bodies.
	 *
	 *	@param NewRot	New orienatation for the body
	 */
	ENGINE_API virtual void SetAllPhysicsRotation(FRotator NewRot);

	/**
	 *	Set the rotation of all bodies in this component.
	 *	If a SkeletalMeshComponent, the root body will be changed to the desired orientation, and the same delta is applied to all other bodies.
	 *
	 *	@param NewRot	New orienatation for the body
	 */
	ENGINE_API virtual void SetAllPhysicsRotation(const FQuat& NewRot);
	
	/**
	 *	Ensure simulation is running for all bodies in this component.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual void WakeAllRigidBodies();
	
	/** Enables/disables whether this component is affected by gravity. This applies only to components with bSimulatePhysics set to true. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetEnableGravity(bool bGravityEnabled);

	/** Returns whether this component is affected by gravity. Returns always false if the component is not simulated. */
	UFUNCTION(BlueprintPure, Category="Physics")
	ENGINE_API virtual bool IsGravityEnabled() const;

	/** Enables/disables whether this component should be updated by simulation when it is kinematic. This is needed if (for example) its velocity needs to be accessed. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation);

	/** Returns whether this component should be updated by simulation when it is kinematic. */
	UFUNCTION(BlueprintPure, Category="Physics")
	ENGINE_API virtual bool GetUpdateKinematicFromSimulation() const;

	/** Enabled/disables whether this body is affected by gyroscopic torque, mainly useful for long/thin objects that spin */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetGyroscopicTorqueEnabled(bool bInGyroscopicTorqueEnabled);

	/** Returns whether this component is affected by gyroscopic torque. */
	UFUNCTION(BlueprintPure, Category = "Physics")
	ENGINE_API virtual bool GetGyroscopicTorqueEnabled() const;

	/** Sets the linear damping of this component. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetLinearDamping(float InDamping);

	/** Returns the linear damping of this component. */
	UFUNCTION(BlueprintPure, Category="Physics")
	ENGINE_API virtual float GetLinearDamping() const;

	/** Sets the angular damping of this component. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetAngularDamping(float InDamping);
	
	/** Returns the angular damping of this component. */
	UFUNCTION(BlueprintPure, Category="Physics")
	ENGINE_API virtual float GetAngularDamping() const;

	/** Change the mass scale used to calculate the mass of a single physics body */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetMassScale(FName BoneName = NAME_None, float InMassScale = 1.f);

	/** Returns the mass scale used to calculate the mass of a single physics body */
	UFUNCTION(BlueprintPure, Category = "Physics")
	ENGINE_API virtual float GetMassScale(FName BoneName = NAME_None) const;

	/** Change the mass scale used fo all bodies in this component */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual void SetAllMassScale(float InMassScale = 1.f);

	/**
	*	Override the mass (in Kg) of a single physics body.
	*	Note that in the case where multiple bodies are attached together, the override mass will be set for the entire group.
	*	Set the Override Mass to false if you want to reset the body's mass to the auto-calculated physx mass.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetMassOverrideInKg(FName BoneName = NAME_None, float MassInKg = 1.f, bool bOverrideMass = true);

	/** Returns the mass of this component in kg. */
	UFUNCTION(BlueprintPure, Category="Physics", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual float GetMass() const;

	/** Returns the inertia tensor of this component in kg cm^2. The inertia tensor is in local component space.*/
	UFUNCTION(BlueprintPure, Category = "Physics", meta =(Keywords = "physics moment of inertia tensor MOI", UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual FVector GetInertiaTensor(FName BoneName = NAME_None) const;

	/** Scales the given vector by the world space moment of inertia. Useful for computing the torque needed to rotate an object.*/
	UFUNCTION(BlueprintPure, Category = "Physics", meta = (Keywords = "physics moment of inertia tensor MOI", UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual FVector ScaleByMomentOfInertia(FVector InputVector, FName BoneName = NAME_None) const;

	/** Returns the calculated mass in kg. This is not 100% exactly the mass physx will calculate, but it is very close ( difference < 0.1kg ). */
	ENGINE_API virtual float CalculateMass(FName BoneName = NAME_None);

	/**
	 * The maximum velocity used to depenetrate this object from others when spawned or teleported with initial overlaps (does not affect overlaps as a result of normal movement).
	 * A value of zero will allow objects that are spawned overlapping to go to sleep without moving rather than pop out of each other. E.g., use zero if you spawn dynamic rocks
	 * partially embedded in the ground and want them to be interactive but not pop out of the ground when touched.
	 * A negative value means that the config setting CollisionInitialOverlapDepenetrationVelocity will be used.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual float GetMaxDepenetrationVelocity(FName BoneName = NAME_None);

	/**
	 * The maximum velocity used to depenetrate this object from others when spawned or teleported with initial overlaps (does not affect overlaps as a result of normal movement).
	 * A value of zero will allow objects that are spawned overlapping to go to sleep without moving rather than pop out of each other. E.g., use zero if you spawn dynamic rocks
	 * partially embedded in the ground and want them to be interactive but not pop out of the ground when touched.
	 * A negative value means that the config setting CollisionInitialOverlapDepenetrationVelocity will be used.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetMaxDepenetrationVelocity(FName BoneName = NAME_None, float InMaxDepenetrationVelocity = -1.0f);

	/** Set whether this component should use Continuous Collision Detection */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetUseCCD(bool InUseCCD, FName BoneName = NAME_None);

	/** Set whether all bodies in this component should use Continuous Collision Detection */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetAllUseCCD(bool InUseCCD);

	/** [EXPERIMENTAL] Set whether this component should use Motion-Aware Collision Detection */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetUseMACD(bool InUseMACD, FName BoneName = NAME_None);

	/** [EXPERIMENTAL] Set whether all bodies in this component should use Motion-Aware Collision Detection */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetAllUseMACD(bool InUseMACD);	

	/** [EXPERIMENTAL] Set whether this component allows Partial Island Sleeping for the island it is in.
	Note that Partial Island Sleeping is a CPU optimization disabling non-moving bodies in the solver. 
	This feature is useful for scenes with many rigid bodies but can affect the physics behavior.*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetAllowPartialIslandSleep(bool InAllowPartialIslandSleep = true, FName BoneName = NAME_None);

	/** [EXPERIMENTAL] Set whether all bodies in this component allow Partial Island Sleeping for the island they are in.
	Note that Partial Island Sleeping is a CPU optimization disabling non-moving bodies in the solver.
	This feature is useful for scenes with many rigid bodies but can affect the physics behavior. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetAllAllowPartialIslandSleep(bool InAllowPartialIslandSleep = true);

	/**
	 *	Force all bodies in this component to sleep.
	 */
	ENGINE_API virtual void PutAllRigidBodiesToSleep();
	
	/**
	 *	Returns if a single body is currently awake and simulating.
	 *	@param	BoneName	If a SkeletalMeshComponent, name of body to return wakeful state from. 'None' indicates root body.
	 */
	ENGINE_API bool RigidBodyIsAwake(FName BoneName = NAME_None) const;

	/**
	 *	Returns if any body in this component is currently awake and simulating.
	 */
	UFUNCTION(BlueprintPure, Category = "Physics", meta = (Keywords = "physics asleep sleeping awake simulating", UnsafeDuringActorConstruction="true"))
	ENGINE_API virtual bool IsAnyRigidBodyAwake();
	
	/**
	 *	Changes a member of the ResponseToChannels container for this PrimitiveComponent.
	 *
	 * @param       Channel      The channel to change the response of
	 * @param       NewResponse  What the new response should be to the supplied Channel
	 */
	UFUNCTION(BlueprintCallable, Category="Collision")
	ENGINE_API virtual void SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse);
	
	/**
	 *	Changes all ResponseToChannels container for this PrimitiveComponent. to be NewResponse
	 *
	 * @param       NewResponse  What the new response should be to the supplied Channel
	 */
	UFUNCTION(BlueprintCallable, Category="Collision")
	ENGINE_API virtual void SetCollisionResponseToAllChannels(ECollisionResponse NewResponse);
	
	/**
	 *	Changes the whole ResponseToChannels container for this PrimitiveComponent.
	 *
	 * @param       NewResponses  New set of responses for this component
	 */
	ENGINE_API virtual void SetCollisionResponseToChannels(const FCollisionResponseContainer& NewReponses);
	
protected:
	/** Called when the BodyInstance ResponseToChannels, CollisionEnabled or bNotifyRigidBodyCollision changes, in case subclasses want to use that information. */
	ENGINE_API virtual void OnComponentCollisionSettingsChanged(bool bUpdateOverlaps=true);

	/** Called when bGenerateOverlapEvents changes, in case subclasses want to use that information. */
	ENGINE_API virtual void OnGenerateOverlapEventsChanged();

	/** Ends all current component overlaps. Generally used when destroying this component or when it can no longer generate overlaps. */
	ENGINE_API void ClearComponentOverlaps(bool bDoNotifies, bool bSkipNotifySelf);

private:
	/** Check if mobility is set to non-static. If BodyInstanceRequiresSimulation is non-null we check that it is simulated. Triggers a PIE warning if conditions fails */
	ENGINE_API void WarnInvalidPhysicsOperations_Internal(const FText& ActionText, const FBodyInstance* BodyInstanceRequiresSimulation, FName BoneName) const;

public:
	/**
	 * Applies RigidBodyState only if it needs to be updated
	 * NeedsUpdate flag will be removed from UpdatedState after all velocity corrections are finished
	 */
	ENGINE_API void SetRigidBodyReplicatedTarget(FRigidBodyState& UpdatedState, const FName BoneName = NAME_None, int32 ServerFrame = 0, int32 ServerHandle = 0);

	/** Register this components physics object as a focal particle in Physics Replication LOD */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API void RegisterAsFocalPointInPhysicsReplicationLOD() const;

	/** Unregister this components physics object from being a focal particle in Physics Replication LOD */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API void UnregisterAsFocalPointInPhysicsReplicationLOD() const;

protected:
	virtual bool CanBeUsedInPhysicsReplication(const FName BoneName = NAME_None) const { return true; }

public:

	/** 
	 *	Get the state of the rigid body responsible for this Actor's physics, and fill in the supplied FRigidBodyState struct based on it.
	 *
	 *	@return	true if we successfully found a physics-engine body and update the state structure from it.
	 */
	ENGINE_API bool GetRigidBodyState(FRigidBodyState& OutState, FName BoneName = NAME_None);

	/** 
	 *	Changes the current PhysMaterialOverride for this component. 
	 *	Note that if physics is already running on this component, this will _not_ alter its mass/inertia etc,  
	 *	it will only change its surface properties like friction.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(DisplayName="Set PhysicalMaterial Override"))
	ENGINE_API virtual void SetPhysMaterialOverride(class UPhysicalMaterial* NewPhysMaterial);

	/** 
	 *  Looking at various values of the component, determines if this
	 *  component should be added to the scene
	 * @return true if the component is visible and should be added to the scene, false otherwise
	 */
	ENGINE_API bool ShouldComponentAddToScene() const;
	
	/**
	 * Changes the value of CullDistance.
	 * @param NewCullDistance - The value to assign to CullDistance.
	 */
	UFUNCTION(BlueprintCallable, Category="LOD", meta=(DisplayName="Set Max Draw Distance"))
	ENGINE_API void SetCullDistance(float NewCullDistance);
	
	/**
	 * Utility to cache the max draw distance based on cull distance volumes or the desired max draw distance
	 */
	ENGINE_API void SetCachedMaxDrawDistance(const float NewCachedMaxDrawDistance);

	/**
	 * Changes the value of DepthPriorityGroup.
	 * @param NewDepthPriorityGroup - The value to assign to DepthPriorityGroup.
	 */
	ENGINE_API void SetDepthPriorityGroup(ESceneDepthPriorityGroup NewDepthPriorityGroup);
	
	/**
	 * Changes the value of bUseViewOwnerDepthPriorityGroup and ViewOwnerDepthPriorityGroup.
	 * @param bNewUseViewOwnerDepthPriorityGroup - The value to assign to bUseViewOwnerDepthPriorityGroup.
	 * @param NewViewOwnerDepthPriorityGroup - The value to assign to ViewOwnerDepthPriorityGroup.
	 */
	ENGINE_API void SetViewOwnerDepthPriorityGroup(
		bool bNewUseViewOwnerDepthPriorityGroup,
		ESceneDepthPriorityGroup NewViewOwnerDepthPriorityGroup
		);
	
	/** 
	 *  Trace a ray against just this component.
	 *  @param  OutHit          Information about hit against this component, if true is returned
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  Params          Additional parameters used for the trace
	 *  @return true if a hit is found
	 */
	ENGINE_API virtual bool LineTraceComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params );

	/**
	 *  Trace a ray against just this component.
	 *  @param  OutHit          Information about hit against this component, if true is returned
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @return true if a hit is found
	 */
	ENGINE_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams);
	
	/** 
	 *  Trace a shape against just this component.
	 *  @param  OutHit          	Information about hit against this component, if true is returned
	 *  @param  Start           	Start location of the box
	 *  @param  End             	End location of the box
	 *  @param  ShapeWorldRotation  The rotation applied to the collision shape in world space.
	 *  @param  CollisionShape  	Collision Shape
	 *	@param	bTraceComplex	Whether or not to trace complex
	 *  @return true if a hit is found
	 */
	ENGINE_API virtual bool SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape &CollisionShape, bool bTraceComplex=false);

	/**
	 *  Trace a shape against just this component.
	 *  @param  OutHit          	Information about hit against this component, if true is returned
	 *  @param  Start           	Start location of the box
	 *  @param  End             	End location of the box
	 *  @param  ShapeWorldRotation  The rotation applied to the collision shape in world space.
	 *  @param  Geometry			Geometry to sweep with.
	 *  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @return true if a hit is found
	 */
	ENGINE_API virtual bool SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams);

	/** 
	 *  Test the collision of the supplied component at the supplied location/rotation, and determine if it overlaps this component.
	 *  @note This overload taking rotation as a FQuat is slightly faster than the version using FRotator.
	 *  @note This simply calls the virtual ComponentOverlapComponentImpl() which can be overridden to implement custom behavior.
	 *  @param  PrimComp        Component to use geometry from to test against this component. Transform of this component is ignored.
	 *  @param  Pos             Location to place PrimComp geometry at 
	 *  @param  Rot             Rotation to place PrimComp geometry at 
	 *  @param  Params          Parameter for trace. TraceTag is only used.
	 *  @return true if PrimComp overlaps this component at the specified location/rotation
	 */
	bool ComponentOverlapComponent(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Rot, const FCollisionQueryParams& Params);
	bool ComponentOverlapComponent(class UPrimitiveComponent* PrimComp, const FVector Pos, const FRotator Rot, const FCollisionQueryParams& Params);

	/**
	 *  Test the collision of the supplied component at the supplied location/rotation, and determine if it overlaps this component.
	 *  @note This overload taking rotation as a FQuat is slightly faster than the version using FRotator.
	 *  @note This simply calls the virtual ComponentOverlapComponentImpl() which can be overridden to implement custom behavior.
	 *  @param  PrimComp        Component to use geometry from to test against this component. Transform of this component is ignored.
	 *  @param  Pos             Location to place PrimComp geometry at
	 *  @param  Rot             Rotation to place PrimComp geometry at
	 *  @param  Params          Parameter for trace. TraceTag is only used.
	 *  @param  OutOverlap      Also returns all the sub-overlaps within the component.
	 *  @return true if PrimComp overlaps this component at the specified location/rotation
	 */
	bool ComponentOverlapComponentWithResult(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FQuat& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const;
	bool ComponentOverlapComponentWithResult(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FRotator& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const;
protected:
	/** Override this method for custom behavior for ComponentOverlapComponent() */
	ENGINE_API virtual bool ComponentOverlapComponentImpl(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Rot, const FCollisionQueryParams& Params);
	ENGINE_API virtual bool ComponentOverlapComponentWithResultImpl(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FQuat& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const;

public:	
	/** 
	 *  Test the collision of the supplied shape at the supplied location, and determine if it overlaps this component.
	 *
	 *  @param  Pos             Location to place PrimComp geometry at 
	 *	@param  Rot             Rotation of PrimComp geometry
	 *  @param  CollisionShape  Shape of collision of PrimComp geometry
	 *  @return true if PrimComp overlaps this component at the specified location/rotation
	 */

    //UE_DEPRECATED(5.0, "Use the const version of OverlapComponent. This deprecation cannot be uncommented as it would produce false positives." )
	virtual bool OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) final 
	{
		return const_cast<const UPrimitiveComponent*>(this)->OverlapComponent(Pos, Rot, CollisionShape);
	}
	ENGINE_API virtual bool OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const;

	/**
	 * Test the collision of the supplied shape at the supplied location, and determine if it overlaps this component.
	 * Also will return information about the overlap.
	 *  @param  Pos             Location to place PrimComp geometry at
	 *	@param  Rot             Rotation of PrimComp geometry
	 *  @param  CollisionShape  Shape of collision of PrimComp geometry
	 *  @param  OutOverlap      Additional information about what exactly was overlapped.
	 *  @return true if PrimComp overlaps this component at the specified location/rotation
	 */
	ENGINE_API virtual bool OverlapComponentWithResult(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape, TArray<FOverlapResult>& OutOverlap) const;

	/**
	 * Test the collision of the supplied shape at the supplied location, and determine if it overlaps this component.
	 * Also will return information about the overlap.
	 *  @param  Pos             Location to place PrimComp geometry at
	 *	@param  Rot             Rotation of PrimComp geometry
	 *  @param  Geometry		Geometry to use for the overlap check.
	 *  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param  OutOverlap      Additional information about what exactly was overlapped.
	 *  @return true if PrimComp overlaps this component at the specified location/rotation
	 */
	ENGINE_API virtual bool OverlapComponentWithResult(const FVector& Pos, const FQuat& Rot, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& OutOverlap) const;

	/**
	 * Computes the minimum translation direction (MTD) when an overlap exists between the component and the given shape.
	 * @param OutMTD			Outputs the MTD to move CollisionShape out of penetration
	 * @param CollisionShape	Shape information for the geometry testing against
	 * @param Pos				Location of collision shape
	 * @param Rot				Rotation of collision shape
	 * @return true if the computation succeeded - assumes that there is an overlap at the specified position/rotation
	 */
	ENGINE_API virtual bool ComputePenetration(FMTDResult & OutMTD, const FCollisionShape& CollisionShape, const FVector& Pos, const FQuat& Rot);

	/**
	 * Return true if the given Pawn can step up onto this component.
	 * This controls whether they can try to step up on it when they bump in to it, not whether they can walk on it after landing on it.
	 * @param Pawn the Pawn that wants to step onto this component.
	 * @see CanCharacterStepUpOn
	 */
	UFUNCTION(BlueprintCallable, Category=Collision)
	ENGINE_API virtual bool CanCharacterStepUp(class APawn* Pawn) const;

	//~ Begin INavRelevantInterface Interface
	ENGINE_API virtual void GetNavigationData(FNavigationRelevantData& OutData) const override;
	ENGINE_API virtual FBox GetNavigationBounds() const override;
	ENGINE_API virtual bool IsNavigationRelevant() const override;
	ENGINE_API virtual UBodySetup* GetNavigableGeometryBodySetup() override final; // marked as final since PrimitiveComponent derived classes relies on GetBodySetup()
	ENGINE_API virtual FTransform GetNavigableGeometryTransform() const override final; // marked as final since PrimitiveComponent derived classes relies on GetComponentTransform()

	/** If true then DoCustomNavigableGeometryExport will be called to collect navigable geometry of this component. */
	ENGINE_API virtual EHasCustomNavigableGeometry::Type HasCustomNavigableGeometry() const override;

	/** Collects custom navigable geometry of component.
	 *	@return true if regular navigable geometry exporting should be run as well
	 */
	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	//~ End INavRelevantInterface Interface

	// Returns true if we should check the GetGenerateOverlapEvents() flag when gathering overlaps, otherwise we'll always just do it.
	inline bool ShouldCheckOverlapFlagToQueueOverlaps(const UPrimitiveComponent& ThisComponent) const;

	/** Set value of HasCustomNavigableGeometry */
	ENGINE_API void SetCustomNavigableGeometry(const EHasCustomNavigableGeometry::Type InType);

	/**
	 * Dispatches Begin/End/Over component events for a single touch.
	 *
	 * @param Controller the responsible Player Controller, or null.
	 * @param EnabledFamilies selects which event families to fire (BeginEnd, Over, or both).
	 * @param TouchedPrimitives per-finger-index array of the caller's currently touched components.  Assumes caller is coupled to a single device.
	 */
	static ENGINE_API void DispatchTouchHitResult(
		const FTouchId TouchId,
		const ETouchType::Type Type,
		const FHitResult& HitResult,
		const bool bHit,
		APlayerController* const Controller,
		const ETouchEventFamilies EnabledFamilies,
		TArrayView<TWeakObjectPtr<UPrimitiveComponent>> TouchedPrimitives);

	static ENGINE_API void DispatchMouseOverEvents(UPrimitiveComponent* CurrentComponent, UPrimitiveComponent* NewComponent);

	UE_DEPRECATED(5.8, "Use overload with Touch ID, Player Controller, and Hit Result." )
	static ENGINE_API void DispatchTouchOverEvents(ETouchIndex::Type FingerIndex, UPrimitiveComponent* CurrentComponent, UPrimitiveComponent* NewComponent);

	static ENGINE_API void DispatchTouchOverEvents(const FTouchId TouchId, APlayerController* const Controller, const FHitResult& HitResult, UPrimitiveComponent* const CurrentComponent, UPrimitiveComponent* const NewComponent);

	ENGINE_API void DispatchOnClicked(FKey ButtonClicked = EKeys::LeftMouseButton);
	ENGINE_API void DispatchOnReleased(FKey ButtonReleased = EKeys::LeftMouseButton);

	UE_DEPRECATED(5.8, "Use overload with Touch ID, Player Controller, and Hit Result." )
	ENGINE_API void DispatchOnInputTouchBegin(const ETouchIndex::Type Key);

	UE_DEPRECATED(5.8, "Use overload with Touch ID, Player Controller, and Hit Result." )
	ENGINE_API void DispatchOnInputTouchEnd(const ETouchIndex::Type Key);

	ENGINE_API void DispatchOnInputTouchBegin(const FTouchId TouchId, APlayerController* const Controller, const FHitResult& HitResult);

	ENGINE_API void DispatchOnInputTouchEnd(const FTouchId TouchId, APlayerController* const Controller, const FHitResult& HitResult);

	//~ Begin IPhysicsComponent Interface.
public:
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectByName(const FName& Name) const override;
	ENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	ENGINE_API virtual Chaos::FPhysicsObjectId GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const override;
	//~ End IPhysicsComponent Interface.

	//~Begin IPhysicsBodyInstanceOwnerResolver
	ENGINE_API virtual IPhysicsBodyInstanceOwner* ResolvePhysicsBodyInstanceOwner(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	//~End IPhysicsBodyInstanceOwnerResolver
	
	//~ Begin IPhysicsBodyInstanceOwner Interface.

	/** Whether the physics is static for this component. */
	virtual bool IsStaticPhysics() const override { return Mobility != EComponentMobility::Movable && bStaticWhenNotMoveable; }

	/**
	 * If true, this body instance owner will generate individual overlaps for each overlapping physics body if it is a multi-body object. When false, it will
	 * generate only one overlap, regardless of how many physics bodies it has and how many of them are overlapping another body. This flag has no
	 * influence on single body objects.
	 */
	virtual bool IsMultiBodyOverlap() const override { return bMultiBodyOverlap; }

	/** Returns the source UObject of this body instance owner. */
	virtual UObject* GetSourceObject() const override { return const_cast<UPrimitiveComponent*>(this); }

	/** Returns the owner of the source UObject of this body instance owner. */
	ENGINE_API virtual UObject* GetSourceObjectOwner() const override;

	/** Returns the transform of this body instance owner. */
	virtual FTransform GetPhysicsOwnerTransform() const override { return GetComponentTransform(); }

	/** Return the BodySetup to use for this body instance owner (single body case) */
	virtual UBodySetup* GetPhysicsBodySetup() const override { return const_cast<UPrimitiveComponent*>(this)->GetBodySetup(); }

	/** Returns the body instance's physical material override. */
	ENGINE_API virtual UPhysicalMaterial* GetPhysicsMaterialOverride() const override;

	/** Returns the body instance owner mesh material. */
	ENGINE_API virtual UMaterialInterface* GetPhysicsMaterialBase() const override;

	/** Whether this body instance owner is movable or not. On UPrimitiveComponents this checks the Mobility type property. */
	ENGINE_API virtual bool IsPhysicsOwnerMovable() const override;
	
	/** Whether this body instance owner is simulating physics or not */
	ENGINE_API virtual bool IsPhysicsOwnerSimulatingPhysics() const override;
	
	/** Gets the body instance owner velocity. Since this is a UPrimitiveComponent we will also check component velocity. */
	ENGINE_API virtual FVector GetPhysicsOwnerVelocity() const override;
	
	/** Returns the attachment root of the body instance owner if it exists */
	ENGINE_API virtual UObject* GetPhysicsOwnerAttachmentRoot() const override;
	
	/** Whether this body instance is world geometry or not */
	ENGINE_API virtual bool IsPhysicsObjectWorldGeometry() const override;
	
	ENGINE_API virtual FTransform GetPhysicsOwnerSocketTransform(FName InSocketName) const override;
	
	/** Checks for a socket on the body instance owner */
	ENGINE_API virtual bool DoesSocketExistOnPhysicsOwner(FName InSocketName) const override;
	
	//~ End IPhysicsBodyInstanceOwner Interface.

	/** Wheter the component is relevant for first person */
	inline bool IsFirstPersonRelevant() const { return FirstPersonPrimitiveType != EFirstPersonPrimitiveType::None; }

	/** Thread-safe accessor for this component's SimpleStreamableAssetManager (SSAM) handle.
	 *  Returns the default-invalid handle if the component was never registered (SSAM disabled,
	 *  or ClearPrimitiveHandle already ran). */
	FSSAMPrimitiveHandle GetSSAMPrimitiveHandle() const { return SSAMPrimitiveHandle; }
private:
	friend class FSimpleStreamableAssetManager;
	/** SSAM handle for this component. Aliases the proxy's SSAMPrimitiveHandle when both are
	 *  live. Same atomicity reasoning as the proxy-side field -- see PrimitiveSceneProxy.h. */
	FSSAMAtomicPrimitiveHandle SSAMPrimitiveHandle;
protected:

	UFUNCTION()
	ENGINE_API virtual void HandlePostMigrationPhysicsStateChanged(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);

#if UE_WITH_REMOTE_OBJECT_HANDLE

	ENGINE_API bool CanSendMigratedPhysicsState(const FUObjectMigrationContext* MigrationContext, EObjectMigrationSendType ObjMigrationSendType) const;

	ENGINE_API virtual void MigratePhysicsState(FArchive& Ar) override;
	
	ENGINE_API virtual void ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData = nullptr) override;

	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif

protected:
	void AddNavigationModifier(FNavigationRelevantData& OutData) const;
};

/** 
 *  Component instance cached data base class for primitive components. 
 *  Stores a list of instance components attached to the 
 */
USTRUCT()
struct FPrimitiveComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FPrimitiveComponentInstanceData() = default;
	ENGINE_API FPrimitiveComponentInstanceData(const UPrimitiveComponent* SourceComponent);
	virtual ~FPrimitiveComponentInstanceData() = default;

	ENGINE_API virtual bool ContainsData() const override;

	ENGINE_API virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;
	ENGINE_API virtual void FindAndReplaceInstances(const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	const FTransform& GetComponentTransform() const { return ComponentTransform; }

private:
	UPROPERTY()
	FTransform ComponentTransform;

	UPROPERTY()
	int32 VisibilityId = INDEX_NONE;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> LODParent = nullptr;

#if WITH_EDITOR
	FColor OverlayColor;
#endif
	
};


//////////////////////////////////////////////////////////////////////////
// PrimitiveComponent inlines

inline bool UPrimitiveComponent::ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams) const
{
	return ComponentOverlapMultiImpl(OutOverlaps, InWorld, Pos, Rot, TestChannel, Params, ObjectQueryParams);
}

inline bool UPrimitiveComponent::ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FRotator& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams) const
{
	return ComponentOverlapMultiImpl(OutOverlaps, InWorld, Pos, Rot.Quaternion(), TestChannel, Params, ObjectQueryParams);
}

inline bool UPrimitiveComponent::ComponentOverlapComponent(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Rot, const FCollisionQueryParams& Params)
{
	return ComponentOverlapComponentImpl(PrimComp, Pos, Rot, Params);
}

inline bool UPrimitiveComponent::ComponentOverlapComponent(class UPrimitiveComponent* PrimComp, const FVector Pos, const FRotator Rot, const FCollisionQueryParams& Params)
{
	return ComponentOverlapComponentImpl(PrimComp, Pos, Rot.Quaternion(), Params);
}

inline bool UPrimitiveComponent::ComponentOverlapComponentWithResult(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FQuat& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const
{
	return ComponentOverlapComponentWithResultImpl(PrimComp, Pos, Rot, Params, OutOverlap);
}

inline bool UPrimitiveComponent::ComponentOverlapComponentWithResult(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FRotator& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const
{
	return ComponentOverlapComponentWithResultImpl(PrimComp, Pos, Rot.Quaternion(), Params, OutOverlap);
}

inline const TArray<FOverlapInfo>& UPrimitiveComponent::GetOverlapInfos() const
{
	return OverlappingComponents;
}

inline bool UPrimitiveComponent::K2_IsCollisionEnabled() const
{
	return IsCollisionEnabled();
}

inline bool UPrimitiveComponent::K2_IsQueryCollisionEnabled() const
{
	return IsQueryCollisionEnabled();
}

inline bool UPrimitiveComponent::K2_IsPhysicsCollisionEnabled() const
{
	return IsPhysicsCollisionEnabled();
}

inline bool UPrimitiveComponent::GetGenerateOverlapEvents() const
{
	return bGenerateOverlapEvents;
}

inline bool UPrimitiveComponent::ShouldCheckOverlapFlagToQueueOverlaps(const UPrimitiveComponent& ThisComponent) const
{
	const FScopedMovementUpdate* CurrentUpdate = ThisComponent.GetCurrentScopedMovement();
	if (CurrentUpdate)
	{
		return CurrentUpdate->RequiresOverlapsEventFlag();
	}
	// By default we require the GetGenerateOverlapEvents() to queue up overlaps, since we require it to trigger events.
	return true;
}

//////////////////////////////////////////////////////////////////////////
// PrimitiveComponent templates

template<typename AllocatorType>
bool UPrimitiveComponent::ConvertSweptOverlapsToCurrentOverlaps(
	TArray<FOverlapInfo, AllocatorType>& OverlapsAtEndLocation, const TOverlapArrayView& SweptOverlaps, int32 SweptOverlapsIndex,
	const FVector& EndLocation, const FQuat& EndRotationQuat)
{
	checkSlow(SweptOverlapsIndex >= 0);

	bool bResult = false;
	const bool bForceGatherOverlaps = !ShouldCheckOverlapFlagToQueueOverlaps(*this);
	if ((GetGenerateOverlapEvents() || bForceGatherOverlaps) && PrimitiveComponentCVars::bAllowCachedOverlapsCVar)
	{
		const AActor* Actor = GetOwner();
		if (Actor && Actor->GetRootComponent() == this)
		{
			// We know we are not overlapping any new components at the end location. Children are ignored here (see note below).
			if (PrimitiveComponentCVars::bEnableFastOverlapCheck)
			{
				SCOPE_CYCLE_COUNTER(STAT_MoveComponent_FastOverlap);

				// Check components we hit during the sweep, keep only those still overlapping
				const FCollisionQueryParams UnusedQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId());
				const int32 NumSweptOverlaps = SweptOverlaps.Num();
				OverlapsAtEndLocation.Reserve(OverlapsAtEndLocation.Num() + NumSweptOverlaps);
				for (int32 Index = SweptOverlapsIndex; Index < NumSweptOverlaps; ++Index)
				{
					const FOverlapInfo& OtherOverlap = SweptOverlaps[Index];
					UPrimitiveComponent* OtherPrimitive = OtherOverlap.OverlapInfo.GetComponent();
					if (OtherPrimitive && (OtherPrimitive->GetGenerateOverlapEvents() || bForceGatherOverlaps))
					{
						if (OtherPrimitive->bMultiBodyOverlap)
						{
							// Not handled yet. We could do it by checking every body explicitly and track each body index in the overlap test, but this seems like a rare need.
							return false;
						}
						else if (Cast<USkeletalMeshComponent>(OtherPrimitive) || Cast<USkeletalMeshComponent>(this))
						{
							// SkeletalMeshComponent does not support this operation, and would return false in the test when an actual query could return true.
							return false;
						}
						else if (OtherPrimitive->ComponentOverlapComponent(this, EndLocation, EndRotationQuat, UnusedQueryParams))
						{
							OverlapsAtEndLocation.Add(OtherOverlap);
						}
					}
				}

				// Note: we don't worry about adding any child components here, because they are not included in the sweep results.
				// Children test for their own overlaps after we update our own, and we ignore children in our own update.
				checkfSlow(OverlapsAtEndLocation.FindByPredicate(FPredicateOverlapHasSameActor(*Actor)) == nullptr,
					TEXT("Child overlaps should not be included in the SweptOverlaps() array in UPrimitiveComponent::ConvertSweptOverlapsToCurrentOverlaps()."));

				bResult = true;
			}
			else
			{
				if (SweptOverlaps.Num() == 0 && AreAllCollideableDescendantsRelative())
				{
					// Add overlaps with components in this actor.
					GetOverlapsWithActor_Template(Actor, OverlapsAtEndLocation);
					bResult = true;
				}
			}
		}
	}

	return bResult;
}

template<typename AllocatorType>
bool UPrimitiveComponent::ConvertRotationOverlapsToCurrentOverlaps(TArray<FOverlapInfo, AllocatorType>& OutOverlapsAtEndLocation, const TOverlapArrayView& CurrentOverlaps)
{
	bool bResult = false;
	const bool bForceGatherOverlaps = !ShouldCheckOverlapFlagToQueueOverlaps(*this);
	if ((GetGenerateOverlapEvents() || bForceGatherOverlaps) && PrimitiveComponentCVars::bAllowCachedOverlapsCVar)
	{
		const AActor* Actor = GetOwner();
		if (Actor && Actor->GetRootComponent() == this)
		{
			if (PrimitiveComponentCVars::bEnableFastOverlapCheck)
			{
				// Add all current overlaps that are not children. Children test for their own overlaps after we update our own, and we ignore children in our own update.
				OutOverlapsAtEndLocation.Reserve(OutOverlapsAtEndLocation.Num() + CurrentOverlaps.Num());
				Algo::CopyIf(CurrentOverlaps, OutOverlapsAtEndLocation, FPredicateOverlapHasDifferentActor(*Actor));
				bResult = true;
			}
		}
	}

	return bResult;
}

template<typename AllocatorType>
bool UPrimitiveComponent::GetOverlapsWithActor_Template(const AActor* Actor, TArray<FOverlapInfo, AllocatorType>& OutOverlaps) const
{
	const int32 InitialCount = OutOverlaps.Num();
	if (Actor)
	{
		for (int32 OverlapIdx = 0; OverlapIdx < OverlappingComponents.Num(); ++OverlapIdx)
		{
			UPrimitiveComponent const* const PrimComp = OverlappingComponents[OverlapIdx].OverlapInfo.Component.Get();
			if (PrimComp && (PrimComp->GetOwner() == Actor))
			{
				OutOverlaps.Add(OverlappingComponents[OverlapIdx]);
			}
		}
	}

	return InitialCount != OutOverlaps.Num();
}
