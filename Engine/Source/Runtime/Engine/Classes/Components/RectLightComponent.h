// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "RectLightSceneProxyDesc.h"
#include "Components/LocalLightComponent.h"
#include "RectLightComponent.generated.h"

float ENGINE_API GetRectLightBarnDoorMaxAngle();

void ENGINE_API CalculateRectLightCullingBarnExtentAndDepth(float Size, float Length, float AngleRad, float Radius, float& OutExtent, float& OutDepth);
void ENGINE_API CalculateRectLightBarnCorners(float SourceWidth, float SourceHeight, float BarnExtent, float BarnDepth, TStaticArray<FVector, 8>& OutCorners);

class FLightSceneProxy;

/**
 * A light component which emits light from a rectangle.
 */
UCLASS(Blueprintable, ClassGroup=(Lights), hidecategories=(Object, LightShafts), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class URectLightComponent : public ULocalLightComponent
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Width of light source rect.
	 * Note that light sources shapes which intersect shadow casting geometry can cause shadowing artifacts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.0", UIMax = "1000.0", ClampMax = "100000"))
	float SourceWidth;

	/** 
	 * Height of light source rect.
	 * Note that light sources shapes which intersect shadow casting geometry can cause shadowing artifacts.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.0", UIMax = "1000.0", ClampMax = "100000"))
	float SourceHeight;

	/**
	 * Angle of barn door attached to the light source rect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = Light, meta = (UIMin = "0.0", UIMax = "90.0"))
	float BarnDoorAngle;
	
	/**
	 * Length of barn door attached to the light source rect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = Light, meta = (UIMin = "0.0"))
	float BarnDoorLength;

	/**
	 * Aperture of cone angle for the perspective projection of the light function material.
	 * If 0, an orthographic projection is used instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = LightFunction, meta = (UIMin = "0.0", UIMax = "89.0", ClampMin = "0.0", ClampMax = "89.0"))
	float LightFunctionConeAngle;

	/** Texture mapped to the light source rectangle */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (GetAssetFilter = "ShouldFilterSourceTexture"))
	TObjectPtr<class UTexture> SourceTexture;

	/** Scales the source texture. Value in 0..1. (default=1) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (UIMin = "0.0", UIMax = "1.0"), AdvancedDisplay)
	FVector2f SourceTextureScale;

	/** Offsets the source texture. Value in 0..1. (default=0) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, meta = (UIMin = "0.0", UIMax = "1.0"), AdvancedDisplay)
	FVector2f SourceTextureOffset;

	/** Maintain compatibility with lights created before an inconsistency in the EV lighting unit was fixed */
	UPROPERTY(meta = (Hidden))
	bool bLightRequiresBrokenEVMath = false;

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetSourceTexture(UTexture* NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetSourceWidth(float NewValue);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetSourceHeight(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetBarnDoorAngle(float NewValue);
	
	UFUNCTION(BlueprintCallable, Category = "Rendering|Lighting")
	ENGINE_API void SetBarnDoorLength(float NewValue);

public:

	ENGINE_API virtual float ComputeLightBrightness() const override;
#if WITH_EDITOR
	ENGINE_API virtual void SetLightBrightness(float InBrightness) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void CheckForErrors() override;
#endif

	//~ Begin ULightComponent Interface.
	ENGINE_API virtual ELightComponentType GetLightType() const override;
	ENGINE_API virtual float GetUniformPenumbraSize() const override;
	ENGINE_API virtual FLightSceneProxy* CreateSceneProxy() const override;

	ENGINE_API virtual void BeginDestroy() override;
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

	inline static FRectLightSceneProxyDesc BuildSceneProxyDesc(const URectLightComponent& InLightComponent)
	{
		FRectLightSceneProxyDesc ProxyDesc;
		BuildSceneProxyDesc(InLightComponent, ProxyDesc);
		return ProxyDesc;
	}

protected:

	ENGINE_API static void BuildSceneProxyDesc(const URectLightComponent& InLightComponent, FRectLightSceneProxyDesc& OutProxyDesc);

#if WITH_EDITOR
	/** Validate source texture is of a valid type. */
	void ValidateTexture() const;
	
	UFUNCTION()
	bool ShouldFilterSourceTexture(const FAssetData& AssetData) const;
#endif
};
