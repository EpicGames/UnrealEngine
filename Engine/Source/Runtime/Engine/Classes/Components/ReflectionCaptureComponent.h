// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Misc/CoreStats.h"
#include "Templates/RefCounting.h"
#include "Components/SceneComponent.h"
#include "RenderCommandFence.h"
#include "RHIShaderPlatform.h"
#include "ReflectionCaptureComponent.generated.h"

class FReflectionCaptureProxy;
class UBillboardComponent;
class FTexture;

UENUM()
enum class EReflectionSourceType : uint8
{
	/** Construct the reflection source from the captured scene*/
	CapturedScene,
	/** Construct the reflection source from the specified cubemap. */
	SpecifiedCubemap,
};

// -> will be exported to EngineDecalClasses.h
UCLASS(abstract, hidecategories=(Collision, Object, Physics, SceneComponent, Activation, "Components|Activation", Mobility), MinimalAPI)
class UReflectionCaptureComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UBillboardComponent> CaptureOffsetComponent;

	/** Indicates where to get the reflection source from. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ReflectionCapture)
	EReflectionSourceType ReflectionSourceType;

	/** Cubemap to use for reflection if ReflectionSourceType is set to RS_SpecifiedCubemap. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ReflectionCapture)
	TObjectPtr<class UTextureCube> Cubemap;

	/** Angle to rotate the source cubemap when SourceType is set to SLS_SpecifiedCubemap. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ReflectionCapture, meta = (UIMin = "0", UIMax = "360"))
	float SourceCubemapAngle;

	/** A brightness control to scale the captured scene's reflection intensity. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ReflectionCapture, meta=(UIMin = ".5", UIMax = "4"))
	float Brightness;

	/** World space offset to apply before capturing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReflectionCapture, AdvancedDisplay)
	FVector CaptureOffset;

	/** Skylight scale factor applied to runtime reflection capture.  Represents amount of sky light to leak where the sky isn't otherwise visible (so pitch black areas have some light). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReflectionCapture)
	FLinearColor RuntimeSkylightScale;

	/** Guid for map build data */
	UPROPERTY()
	FGuid MapBuildDataId;

	/** Cubemap texture resource used for rendering with the encoded HDR values. */
	class FReflectionTextureCubeResource* EncodedHDRCubemapTexture;

	/** Reflection capture is generated at runtime.  If CVar r.ReflectionCapture.Runtime==2, all captures are runtime, regardless of this setting. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=RuntimeCapture)
	bool bRuntimeCapture;

	/** Post process material applied to runtime reflection captures.  Must use Blendable Location "Scene Color Before Bloom". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RuntimeCapture, meta=(EditCondition="bRuntimeCapture", EditConditionHides))
	TObjectPtr<class UMaterialInterface> PostProcessMaterial;

	/** if > 0, sets a maximum render distance override.  Can be used to cull distant objects if in an enclosed area like a hallway or room. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RuntimeCapture, meta=(EditCondition="bRuntimeCapture", EditConditionHides, UIMin="0", UIMax="100000"))
	float MaxViewDistance = 0.0f;

	/**
	 * When true, the runtime capture renders with a finite far clipping plane.  Requires MaxViewDistance > 0 to take effect.  Useful to
	 * exclude distant geometry such as translucent Sky from the captured cubemap, to prevent it from overwriting sky cubemap pixels.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RuntimeCapture, meta=(EditCondition="bRuntimeCapture && MaxViewDistance > 0", EditConditionHides))
	bool bFiniteFarPlane = false;

#if WITH_EDITOR
	/** Check to see if MapBuildDataId was loaded - otherwise we need to display a warning on cook */
	bool bMapBuildDataIdLoaded;
#endif

	/** The rendering thread's mirror of this reflection capture. */
	FReflectionCaptureProxy* SceneProxy;

	/** Callback to create the rendering thread mirror. */
	ENGINE_API FReflectionCaptureProxy* CreateSceneProxy();

	/** Called to update the preview shapes when something they are dependent on has changed. */
	virtual void UpdatePreviewShape();

	/** Adds the capture to the capture queue processed by UpdateReflectionCaptureContents. */
	ENGINE_API void MarkDirtyForRecaptureOrUpload();

	/** Generates a new MapBuildDataId and adds the capture to the capture queue processed by UpdateReflectionCaptureContents. */
	ENGINE_API void MarkDirtyForRecapture();

	/** Marks this component has having been recaptured. */
	void SetCaptureCompleted() { bNeedsRecaptureOrUpload = false; }

	/** Gets the radius that bounds the shape's influence, used for culling. */
	virtual float GetInfluenceBoundingRadius() const PURE_VIRTUAL(UReflectionCaptureComponent::GetInfluenceBoundingRadius,return 0;);

	/**
	  * Generally called each tick to recapture any queued reflection captures.  In some cases, it's also called from Editor utility functions or commands.
	  * Set "bInsideTick" to true if called from inside a Tick function, indicating a render frame is already active, and the capture doesn't need to start one.
	  */
	ENGINE_API static void UpdateReflectionCaptureContents(UWorld* WorldToUpdate, const TCHAR* CaptureReason = nullptr, bool bVerifyOnlyCapturing = false, bool bCapturingForMobile = false, bool bInsideTick = false);

	ENGINE_API static bool HasReflectionCapturesToUpdate();

	ENGINE_API class FReflectionCaptureMapBuildData* GetMapBuildData() const;

	ENGINE_API static bool IsEncodedHDRCubemapTextureRequired(EShaderPlatform ShaderPlatform);
	
	ENGINE_API void AllocateEncodedHDRCubemapTexture(FStaticShaderPlatform ShaderPlatform);
	
	virtual void PropagateLightingScenarioChange() override;

	ENGINE_API static int32 GetReflectionCaptureSize();

	//~ Begin UActorComponent Interface
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual void SendRenderTransform_Concurrent() override;
	virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
	//~ End UActorComponent Interface

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;	
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* Property) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel) override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void FinishDestroy() override;
	//~ End UObject Interface

public:
	/** Accessor for bRuntimeCapture setting. */
	bool IsRuntimeCapture() const { return bRuntimeCapture; }

	/**
	 * Request this runtime capture to refresh on the next available frame.  If bFastRender is true, the capture renders all
	 * 6 faces in a single frame rather than timeslicing across frames.  If bSmoothBlend is true and a blend slot is available
	 * (see r.ReflectionCapture.Runtime.SmoothBlendSlots), the capture will cross-fade from its old content to the newly
	 * rendered content over r.ReflectionCapture.Runtime.SmoothBlendTime seconds; if no slot is free the refresh pops.
	 */
	UFUNCTION(BlueprintCallable, Category="RuntimeCapture", meta=(AdvancedDisplay="bFastRender,bSmoothBlend"))
	ENGINE_API void RefreshCapture(bool bFastRender = false, bool bSmoothBlend = false);

private:

	/** Last frame this reflection capture was rendered at runtime. */
	uint32 RuntimeLastRenderedFrame;

	/**
	 * Fade-in/out state for runtime captures, for when captures go in or out of budget.  The current blend value at a given
	 * time is computed by applying a CVar specified fade delta rate, relative to RuntimeFade.StartTime.  The FScene friend class
	 * handles the fade, so all the related data and members can be private.
	 */
	struct FRuntimeFade
	{
		float StartValue = 1.0f;
		float TargetValue = 1.0f;
		double StartTime = 0.0;
	};
	FRuntimeFade RuntimeFade;

	/** Struct that handles propagation of fade state to render thread.  Comp is not dereferenced, just used as a key for state lookup. */
	struct FRuntimeFadeEntry : FRuntimeFade
	{
		const UReflectionCaptureComponent* Comp;
	};

	/** When true, the capture will be prioritized for update on the next frame, then this flag is cleared. */
	bool bRefreshRequested = false;

	/** When true, the capture's next update renders all 6 faces in one frame instead of timeslicing; cleared after processing. */
	bool bFastRenderRequested = false;

	/** When true, the capture's next refresh attempts to cross-fade from old to new content; cleared after processing. */
	bool bSmoothBlendRequested = false;

	/**
	 * Flag set at the start of time slicing of a capture being refreshed; cleared after last timeslice renders.  Used by
	 * the budget loop to differentiate fading in and refreshed captures.  The former need to initialize their fade to
	 * zero for when timeslicing finishes, while the latter should do nothing, as the capture is assumed to already be
	 * in the process of fading in, or fully faded in.
	 */
	bool bRefreshInFlight = false;

	/** Whether the reflection capture needs to re-capture the scene. */
	bool bNeedsRecaptureOrUpload;

	/** Cached Average Brightness from MapBuildData used for rendering with the encoded HDR values. */
	float CachedAverageBrightness;

	/** Fence used to track progress of releasing resources on the rendering thread. */
	FRenderCommandFence ReleaseResourcesFence;

	/** 
	 * List of reflection captures that need to be recaptured.
	 * These have to be queued because we can only render the scene to update captures at certain points, after the level has loaded.
	 * This queue should be in the UWorld or the FSceneInterface, but those are not available yet in PostLoad.
	 */
	static TArray<UReflectionCaptureComponent*> ReflectionCapturesToUpdate;

	/** 
	 * List of reflection captures that need to be recaptured because they were dirty on load.
	 * These have to be queued because we can only render the scene to update captures at certain points, after the level has loaded.
	 * This queue should be in the UWorld or the FSceneInterface, but those are not available yet in PostLoad.
	 */
	static TArray<UReflectionCaptureComponent*> ReflectionCapturesToUpdateForLoad;
	static FCriticalSection ReflectionCapturesToUpdateForLoadLock;

	//void UpdateDerivedData(FReflectionCaptureFullHDR* NewDerivedData);
	void SerializeLegacyData(FArchive& Ar);

	void SafeReleaseEncodedHDRCubemapTexture();

	/** Reset all runtime capture related state. */
	void RuntimeCaptureResetState()
	{
		RuntimeLastRenderedFrame = 0;
		RuntimeFade = {};
		bRefreshRequested = false;
		bFastRenderRequested = false;
		bSmoothBlendRequested = false;
		bRefreshInFlight = false;
	}

	/** True if the current fade state is heading toward 0 (capture is exiting the budget). */
	bool IsRuntimeCaptureFadingOut() const
	{
		return RuntimeFade.TargetValue == 0.0f;
	}

	/** Compute the current blend value for the supplied time and fade duration. */
	ENGINE_API float ComputeCurrentRuntimeCaptureFade(double Now, float Duration) const;

	/** Begin a fade toward 1 (active), from StartValue.  Used for initial activation or reversing a fade-out. */
	ENGINE_API void RuntimeCaptureFadeIn(double Now, float FromValue, TArray<FRuntimeFadeEntry>& BlendQueue);

	/** Begin a fade toward 0 (eviction) starting from the current blend value. */
	ENGINE_API void RuntimeCaptureFadeOut(double Now, float FromValue, TArray<FRuntimeFadeEntry>& BlendQueue);

	/** Set fade to given steady state value. */
	ENGINE_API void RuntimeCaptureFadeSet(float Value, TArray<FRuntimeFadeEntry>& BlendQueue);

	friend class FReflectionCaptureProxy;
	friend class FScene;
};

ENGINE_API extern void GenerateEncodedHDRData(const TArray<uint8>& FullHDRData, int32 CubemapSize, TArray<uint8>& OutEncodedHDRData);