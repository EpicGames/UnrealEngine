// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureDefines.h"
#include "UObject/PerPlatformProperties.h"
#include "RenderCommandFence.h"
#include "SceneComponent.h"
#include "SceneTypes.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "VirtualTextureEnum.h"
#include "RuntimeVirtualTextureComponent.generated.h"

class URuntimeVirtualTexture;
class UTexture2D;
class UVirtualTextureBuilder;
enum class EShadingPath;

/** Enumeration of material quality. Similar to EMaterialQualityLevel, but as a UENUM for appearance in UI. */
UENUM()
enum class ERuntimeVirtualTextureMaterialQuality : uint8
{
	Low = 0,
	Medium = 1,
	High = 2,
	Epic = 3,
};

UENUM()
enum class ERuntimeVirtualTextureUseStreamingMipsInEditorMode : uint8
{
	Never UMETA(ToolTip = "Never show the streaming low mips in the editor"),
	PIEOnly UMETA(ToolTip = "Only show the streaming low mips when Play In Editor is active"),
	Always UMETA(ToolTip = "Always show the streaming low mips in the editor"),
};

/** Component used to place a URuntimeVirtualTexture in the world. */
UCLASS(Blueprintable, ClassGroup = Rendering, HideCategories = (Activation, Collision, Cooking, HLOD, Mobility, LOD, Navigation, Object, Physics), MinimalAPI)
class URuntimeVirtualTextureComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	friend class FRuntimeVirtualTextureComponentDetailsCustomization;

protected:
	/** Actor to align rotation to. If set this actor is always included in the bounds calculation. */
	UPROPERTY(EditAnywhere, Category = VolumeBounds)
	TSoftObjectPtr<AActor> BoundsAlignActor = nullptr;

	/** Placeholder for details customization button. */
	UPROPERTY(VisibleAnywhere, Transient, Category = VolumeBounds)
	bool bSetBoundsButton;

	/** If the Bounds Align Actor is a Landscape then this will snap the bounds so that virtual texture texels align with landscape vertex positions. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VolumeBounds, meta = (DisplayName = "Snap To Landscape"))
	bool bSnapBoundsToLandscape;

	/** Amount to expand the Bounds during calculation. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VolumeBounds, meta = (UIMin = "0", ClampMin = "0"))
	float ExpandBounds = 0;

	/** The virtual texture object to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, TextExportTransient, Category = RuntimeVirtualTexture)
	TObjectPtr<URuntimeVirtualTexture> VirtualTexture = nullptr;

	/** Set to true to enable scalability settings for the virtual texture. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, meta = (InlineEditConditionToggle))
	bool bEnableScalability = false;

	/** Group index of the scalability settings to use for the virtual texture. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, meta = (UIMin = "0", UIMax = "2", EditCondition = bEnableScalability))
	uint32 ScalabilityGroup = 0;

	/** Enable the virtual texture only when Nanite is enabled. Can be used for a Displacement virtual texture with Nanite tessellation. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture)
	bool bEnableForNaniteOnly = false;

	/** Use a minimum material quality to determine if we enable the virtual texture. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, Meta = (InlineEditConditionToggle))
	bool bUseMinMaterialQuality = false;

	/** The minimum material quality for which we enable the virtual texture. Only affects In-Game and PIE. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, Meta = (EditCondition = "bUseMinMaterialQuality"))
	ERuntimeVirtualTextureMaterialQuality MinInGameMaterialQuality = ERuntimeVirtualTextureMaterialQuality::Low;

	/** Use a minimum material quality to determine if we enable the virtual texture on mobile. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, Meta = (InlineEditConditionToggle))
	bool bUseMinMaterialQualityMobile = false;

	/** The minimum material quality for which we enable the virtual texture on mobile. Only affects In-Game and PIE. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture, Meta = (EditCondition = "bUseMinMaterialQualityMobile"))
	ERuntimeVirtualTextureMaterialQuality MinInGameMaterialQualityMobile = ERuntimeVirtualTextureMaterialQuality::Low;

	/** Per platform overrides for enabling the virtual texture. Only affects In-Game and PIE. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture)
	FPerPlatformBool EnableInGamePerPlatform;

	/** Hide primitives in the main pass. Hidden primitives will be those that draw to this virtual texture with 'Draw in Main Pass' set to 'From Virtual Texture'. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = RuntimeVirtualTexture)
	bool bHidePrimitives = false;

	/** Texture object containing streamed low mips. This can reduce rendering update cost. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, TextExportTransient, NonPIEDuplicateTransient, Category = StreamingVirtualTexture)
	TObjectPtr<UVirtualTextureBuilder> StreamingTexture = nullptr;

	/** Number of streaming low mips to build for the virtual texture. */
	UPROPERTY(EditAnywhere, Category = StreamingVirtualTexture, meta = (UIMin = "0", UIMax = "12", DisplayName = "Build Levels"))
	int32 StreamLowMips = 0;

	/** Placeholder for details customization button. */
	UPROPERTY(VisibleAnywhere, Transient, Category = StreamingVirtualTexture)
	bool bBuildStreamingMipsButton;

	/** 
	 * How aggressively should any relevant lossy compression be applied. 
	 * For compressors that support EncodeSpeed (i.e. Oodle), this is only applied if enabled (see Project Settings -> Texture Encoding). 
	 * Note that this is in addition to any unavoidable loss due to the target format. Selecting "No Lossy Compression" will not result in zero distortion for BCn formats.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = StreamingVirtualTexture)
	TEnumAsByte<ETextureLossyCompressionAmount> LossyCompressionAmount = TLCA_Default;

	/** Build the streaming low mips using a fixed color. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = StreamingVirtualTexture, meta = (InlineEditConditionToggle))
	bool bUseStreamingMipsFixedColor = false;

	/** Fixed color to use when building the streaming low mips. This only affects BaseColor and Displacement attributes. The Red channel is used for fixed Displacement. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = StreamingVirtualTexture, meta = (DisplayName = "Fixed Color", HideAlphaChannel, EditCondition = bUseStreamingMipsFixedColor))
	FLinearColor StreamingMipsFixedColor;

	/** Whenever streaming low mips are in use, only show the streaming mips and never show runtime generated pages. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StreamingVirtualTexture, meta = (DisplayName = "Use Only Streaming Texture"))
	bool bUseStreamingMipsOnly = false;

	/** 
	 * Use streaming low mips when rendering this runtime virtual texture in the editor. Allows to visualize the baked streaming low mips. 
	 * r.VT.RVT.StreamingMips.UseInEditor can also be used to allow this across all RVT components (for debugging purposes)
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StreamingVirtualTexture, meta = (DisplayName = "View in Editor"))
	ERuntimeVirtualTextureUseStreamingMipsInEditorMode UseStreamingMipsInEditorMode = ERuntimeVirtualTextureUseStreamingMipsInEditorMode::Never;

	UPROPERTY()
	bool bUseStreamingMipsInEditor_DEPRECATED = false;

#if WITH_EDITOR
	/** Delegate handle for our function called on PIE end. */
	FDelegateHandle PieEndDelegateHandle;
#endif

	/** Delegate that this virtual texture will call to evaluated the full HidePrimitives state. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FGetHidePrimitivesDelegate, bool&, bool&);
	FGetHidePrimitivesDelegate HidePrimitivesDelegate;

	/** A fence to track render thread has finished with StreamingTexture data before destroy. */
	FRenderCommandFence DestroyFence;

public:
	/**
	 * This function marks an area of the runtime virtual texture as dirty.
	 * @param WorldBounds : The world space bounds of the pages to invalidate.
	 * @param InvalidatePriority Allows the pages affected by this area to get processed in priority. 
	 * This allows increased responsiveness when there are more pages being updated than can be handled in a given frame (throttling) 
	 */
	UFUNCTION(BlueprintCallable, Category = VirtualTexture)
	ENGINE_API void Invalidate(FBoxSphereBounds const& WorldBounds, EVTInvalidatePriority InvalidatePriority = EVTInvalidatePriority::High);

	/** 
	 * Request preload of an area of the runtime virtual texture at a given mip level. 
	 * @param WorldBounds : The world space bounds of the pages to preload.
	 * @param Level : The mip map level to preload.
	 */
	UFUNCTION(BlueprintCallable, Category = VirtualTexture)
	ENGINE_API void RequestPreload(FBoxSphereBounds const& WorldBounds, int32 Level);

	/** Set the runtime virtual texture object on this component. */
	ENGINE_API void SetVirtualTexture(URuntimeVirtualTexture* InVirtualTexture);

	/** Get the runtime virtual texture object on this component. */
	URuntimeVirtualTexture* GetVirtualTexture() const { return VirtualTexture; }

	/** Get if the runtime virtual texture should be fully instantiated by it's render proxy. */
	ENGINE_API bool IsEnabledInScene() const;

	/** Public getter for the custom material data on this component. */
	ENGINE_API FVector4f GetCustomMaterialData() const;

	/** Get if scalability settings are enabled. */
	bool IsScalable() const { return bEnableScalability; }

	/** Get group index of the scalability settings. */
	uint32 GetScalabilityGroup() const { return ScalabilityGroup; }

	/** Get the delegate used to extend the calculation of the HidePrimitives state. */
	FGetHidePrimitivesDelegate& GetHidePrimitivesDelegate() { return HidePrimitivesDelegate; }

	/** Get the full hide primitive state including the evaluating the GetHidePrimitivesDelegate delegate. */
	ENGINE_API void GetHidePrimitiveSettings(bool& OutHidePrimitiveEditor, bool& OutHidePrimitiveGame) const;

	/** Get the streaming virtual texture object on this component. */
	UVirtualTextureBuilder* GetStreamingTexture() const { return StreamingTexture; }

	/** Public getter for virtual texture streaming low mips */
	int32 NumStreamingMips() const { return FMath::Clamp(StreamLowMips, 0, 12); }

	/** Get if we want to use any streaming low mips on this component. */
	ENGINE_API bool IsStreamingLowMips(EShadingPath ShadingPath) const;

	/** Public getter for using streaming low mips only. */
	ENGINE_API bool IsStreamingLowMipsOnly();

	/** Public getter for streaming mips fixed color. */
	ENGINE_API FLinearColor GetStreamingMipsFixedColor() const;

	/** Public getter for lossy compression setting. */
	TEnumAsByte<ETextureLossyCompressionAmount> GetLossyCompressionAmount() const { return LossyCompressionAmount; }

	/** Returns whether the StreamingTexure contents is valid for use for the specified shading path */
	ENGINE_API bool IsStreamingTextureInvalid(EShadingPath ShadingPath) const;

	/** Public getter for priority setting. */
	ENGINE_API EVTProducerPriority GetPriority() const;

	enum class EStreamingTextureStatusFlags : uint8
	{
		None = 0,
		HasVirtualTexture = (1 << 0),
		HasStreamingTexture = (1 << 1),
		InvalidStreamingTexture = (1 << 2),
		NonMatchingStreamingTextureSettings = (1 << 3),
	};
	/** Returns the status of the StreamingTexure contents for the specified shading path */
	ENGINE_API EStreamingTextureStatusFlags GetStreamingTextureStatus(EShadingPath ShadingPath) const;

#if WITH_EDITOR
	/** Returns whether the StreamingTexure contents is valid for use, taking into account all rendering modes */
	ENGINE_API bool IsStreamingTextureInvalid() const;
	/**  Returns the status of the StreamingTexure contents, taking into account all rendering modes */
	ENGINE_API EStreamingTextureStatusFlags GetStreamingTextureStatus() const;
	/** Set a new asset to hold the low mip streaming texture. This should only be called directly before setting data to the new asset. */
	void SetStreamingTexture(UVirtualTextureBuilder* InTexture) { StreamingTexture = InTexture; }
	/** Initialize the low mip streaming texture with the passed in size and data. */
	ENGINE_API void InitializeStreamingTexture(EShadingPath ShadingPath, uint32 InSizeX, uint32 InSizeY, uint8* InData);
#endif

#if WITH_EDITOR
	/** Get the BoundsAlignActor on this component. */
	ENGINE_API void SetBoundsAlignActor(AActor* InActor);
	/** Get the BoundsAlignActor on this component. */
	TSoftObjectPtr<AActor>& GetBoundsAlignActor() { return BoundsAlignActor; }
	/** Get if SnapBoundsToLandscape is set on this component. */
	bool GetSnapBoundsToLandscape() const { return bSnapBoundsToLandscape; }
	/** Get amount to expand the calculated bounds on this component. */
	float GetExpandBounds() const { return ExpandBounds; }
#endif
	/** Get a translation to account for any vertex sample offset from the use of bSnapBoundsToLandscape. */
	ENGINE_API FTransform GetTexelSnapTransform() const;

protected:
	//~ Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
	ENGINE_API virtual bool ShouldCreateRenderState() const override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
#endif
	//~ End UActorComponent Interface

	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
#endif
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface

protected:
	/** Calculate a hash used to determine if the StreamingTexture contents are valid for use. The hash doesn't include whether the contents are up to date. */
	ENGINE_API uint64 CalculateStreamingTextureSettingsHash() const;

	/** @return true if the owning World is one where URuntimeVirtualTextureComponent should actually do anything (avoids updating RVT for non-game/PIE/editor world types) */
	ENGINE_API bool IsActiveInWorld() const;

public:
	/** Scene proxy object. Managed by the scene but stored here. */
	class FRuntimeVirtualTextureSceneProxy* SceneProxy;
};

ENUM_CLASS_FLAGS(URuntimeVirtualTextureComponent::EStreamingTextureStatusFlags);