// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Engine/GameViewportClient.h"
#include "SceneCaptureComponent.generated.h"

class AActor;
class FSceneViewStateInterface;
class UMaterialParameterCollection;
class UPrimitiveComponent;

/** 
* View state needed to create a scene capture renderer 
* Inherits from FSceneViewProjectionData to unify resolving of possible projection correction calculations.
*/
struct FSceneCaptureViewInfo : public FSceneViewProjectionData
{
	FVector ViewLocation;
	FRotator ViewRotation;
	EStereoscopicPass StereoPass;
	int32 StereoViewIndex;
	float FOV;
};

#if WITH_EDITORONLY_DATA
/** Editor only structure for gathering memory size */
struct FSceneCaptureMemorySize : public FRefCountedObject
{
	uint64 Size = 0;
};
#endif

USTRUCT(BlueprintType)
struct FEngineShowFlagsSetting
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	FString ShowFlagName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	bool Enabled = false;


	bool operator == (const FEngineShowFlagsSetting& Other) const
	{
		return ShowFlagName == Other.ShowFlagName && Other.Enabled == Enabled;
	}
};

UENUM()
enum class ESceneCapturePrimitiveRenderMode : uint8
{
	/** Legacy */
	PRM_LegacySceneCapture UMETA(DisplayName = "Render Scene Primitives (Legacy)"),
	/** Render primitives in the scene, minus HiddenActors. */
	PRM_RenderScenePrimitives UMETA(DisplayName = "Render Scene Primitives"),
	/** Render only primitives in the ShowOnlyActors list, or components specified with ShowOnlyComponent(). */
	PRM_UseShowOnlyList UMETA(DisplayName = "Use ShowOnly List")
};

	// -> will be exported to EngineDecalClasses.h
UCLASS(abstract, hidecategories=(Collision, Object, Physics, SceneComponent, Mobility), MinimalAPI)
class USceneCaptureComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Controls what primitives get rendered into the scene capture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture)
	ESceneCapturePrimitiveRenderMode PrimitiveRenderMode;

	UPROPERTY(interp, Category = SceneCapture, meta = (DisplayName = "Capture Source"))
	TEnumAsByte<enum ESceneCaptureSource> CaptureSource;

	/** Whether to update the capture's contents every frame.  If disabled, the component will render once on load and then only when moved. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	uint8 bCaptureEveryFrame : 1;

	/** Whether to update the capture's contents on movement.  Disable if you are going to capture manually from blueprint. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	uint8 bCaptureOnMovement : 1;

	/** 
	 * Cancels throttling on the virtual texture system during the scene capture. Use this to make sure that the virtual textures don't look blurry on the scene capture. 
	 * Please note that this does *not* pre-warm the RVT pages (use a prior scene capture in order to do this and issue the appropriate RVT requests) and has a performance cost. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = SceneCapture)
	uint8 bOverrideVirtualTextureThrottle : 1;

	/** Capture a GPU frame for this scene capture, next time it renders (capture program must be connected). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, DuplicateTransient, SkipSerialization, NonTransactional, AdvancedDisplay, Category = SceneCapture)
	uint8 bCaptureGpuNextRender : 1;

	/** Run DumpGPU for this scene capture, next time it renders. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, DuplicateTransient, SkipSerialization, NonTransactional, AdvancedDisplay, Category = SceneCapture)
	uint8 bDumpGpuNextRender : 1;

	/**
	 * Flag used to suppress bCaptureGpuNextRender or bDumpGpuNextRender on reregistration of the component.  Editing any property
	 * of the component, including the capture/dump flags, forces it to be reregistered, which also triggers the capture to render.
	 * The purpose of the flags is to allow a capture or dump to be queued and triggered when the next render occurs organically
	 * (for example, on a blueprint event or movement of the actor), not based on the flag itself being set.  When a property change
	 * event for one of the flags occurs, this is set to true, to skip the capture/dump on that automatic first render.
	 */
	uint8 bSuppressGpuCaptureOrDump : 1;

	/**
	 * Whether this capture should be excluded from tracking scene texture extents.  This should be set when this capture is not expected to be
	 * frequently used, especially if the capture resolution is very large.  Setting this for a single-use capture will avoid influencing other
	 * scene texture extent decisions and avoid a possible ongoing increase in memory usage.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = SceneCapture)
	uint8 bExcludeFromSceneTextureExtents : 1;

	/**
	 * Whether this capture should use a finite far clipping plane in its projection matrix.  Requires Max View Distance Override and
	 * perspective projection.  Useful to exclude very distant rendering, such as translucent Sky.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = SceneCapture, meta = (EditCondition = "MaxViewDistanceOverride > 0"))
	uint8 bFiniteFarPlane : 1;

	/** Whether to persist the rendering state even if bCaptureEveryFrame==false.  This allows velocities for Motion Blur and Temporal AA to be computed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture, meta = (editcondition = "!bCaptureEveryFrame"))
	bool bAlwaysPersistRenderingState;

	/** The components won't rendered by current component.*/
 	UPROPERTY()
 	TArray<TWeakObjectPtr<UPrimitiveComponent> > HiddenComponents;

	/** The actors to hide in the scene capture. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=SceneCapture)
	TArray<TObjectPtr<AActor>> HiddenActors;

	/** The only components to be rendered by this scene capture, if PrimitiveRenderMode is set to UseShowOnlyList. */
 	UPROPERTY()
 	TArray<TWeakObjectPtr<UPrimitiveComponent> > ShowOnlyComponents;

	/** The only actors to be rendered by this scene capture, if PrimitiveRenderMode is set to UseShowOnlyList.*/
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=SceneCapture)
	TArray<TObjectPtr<AActor>> ShowOnlyActors;

	/** Scales the distance used by LOD. Set to values greater than 1 to cause the scene capture to use lower LODs than the main view to speed up the scene capture pass. */
	UPROPERTY(EditAnywhere, Category=PlanarReflection, meta=(UIMin = ".1", UIMax = "10"), AdvancedDisplay)
	float LODDistanceFactor;

	/** if > 0, sets a maximum render distance override.  Can be used to cull distant objects from a reflection if the reflecting plane is in an enclosed area like a hallway or room */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture, meta=(UIMin = "100", UIMax = "10000"))
	float MaxViewDistanceOverride;

	/** Capture priority within the frame to sort scene capture on GPU to resolve interdependencies between multiple capture components. Highest come first. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SceneCapture)
	int32 CaptureSortPriority;

	/** Whether to use ray tracing for this capture. Ray Tracing must be enabled in the project. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	bool bUseRayTracingIfEnabled;

	/** Store WorldToLocal and/or Projection matrices (2D capture only) to a Material Parameter Collection on render. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture, AdvancedDisplay)
	TObjectPtr<UMaterialParameterCollection> CollectionTransform;

	/** Parameter name of the first element of the transform in the CollectionTransform Material Parameter Collection set above.  Requires space for 5 vectors (large world coordinate transform). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture, AdvancedDisplay)
	FName CollectionTransformWorldToLocal;

	/** Parameter name of the first element of the transform in the CollectionTransform Material Parameter Collection set above.  Requires space for 4 vectors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture, AdvancedDisplay)
	FName CollectionTransformProjection;

	/** View / light masking support.  Controls which lights should affect this view. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture)
	FViewLightingChannels ViewLightingChannels;

	/** ShowFlags for the SceneCapture's ViewFamily, to control rendering settings for this view. Hidden but accessible through details customization */
	UE_DEPRECATED_FORGAME(5.5, "Public access to this property is deprecated, and it will become private in a future release. Please use SetShowFlagSettings and GetShowFlagSettings instead.")
	UPROPERTY(EditAnywhere, interp, Blueprintgetter = GetShowFlagSettings, BlueprintSetter = SetShowFlagSettings, Category=SceneCapture)
	TArray<FEngineShowFlagsSetting> ShowFlagSettings;

	// TODO: Make this a UStruct to set directly?
	/** Settings stored here read from the strings and int values in the ShowFlagSettings array */
	FEngineShowFlags ShowFlags;

	/** Get the show flag settings. */
	UFUNCTION(BlueprintGetter)
	ENGINE_API const TArray<FEngineShowFlagsSetting>& GetShowFlagSettings() const;

	/** Set the show flag settings. */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetShowFlagSettings(const TArray<FEngineShowFlagsSetting>& InShowFlagSettings);

public:
	/** Name of the profiling event. */
	UPROPERTY(EditAnywhere, interp, Category = SceneCapture)
	FString ProfilingEventName;

	/** Skylight color multiplier, used by reflection captures (not exposed as a UPROPERTY for now) */
	FLinearColor SkylightScale;

	ENGINE_API virtual void BeginDestroy() override;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void OnRegister() override;
	//~ End UActorComponent Interface

	/** Adds the component to our list of hidden components. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void HideComponent(UPrimitiveComponent* InComponent);

	/**
	 * Adds all primitive components in the actor to our list of hidden components.
	 * @param bIncludeFromChildActors Whether to include the components from child actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void HideActorComponents(AActor* InActor, const bool bIncludeFromChildActors = false);

	/** Adds the component to our list of show-only components. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void ShowOnlyComponent(UPrimitiveComponent* InComponent);

	/**
	 * Adds all primitive components in the actor to our list of show-only components.
	 * @param bIncludeFromChildActors Whether to include the components from child actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void ShowOnlyActorComponents(AActor* InActor, const bool bIncludeFromChildActors = false);

	/** Removes a component from the Show Only list. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void RemoveShowOnlyComponent(UPrimitiveComponent* InComponent);

	/**
	 * Removes an actor's components from the Show Only list.
	 * @param bIncludeFromChildActors Whether to remove the components from child actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void RemoveShowOnlyActorComponents(AActor* InActor, const bool bIncludeFromChildActors = false);

	/** Clears the Show Only list. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void ClearShowOnlyComponents();

	/** Clears the hidden list. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void ClearHiddenComponents();

	/** Changes the value of TranslucentSortPriority. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|SceneCapture")
	ENGINE_API void SetCaptureSortPriority(int32 NewCaptureSortPriority);

	/** Returns the view state, if any, and allocates one if needed. This function can return NULL, e.g. when bCaptureEveryFrame is false. */
	ENGINE_API FSceneViewStateInterface* GetViewState(int32 ViewIndex);

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif	

	ENGINE_API virtual void Serialize(FArchive& Ar);

	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void PostLoad() override;

	/** To leverage a component's bOwnerNoSee/bOnlyOwnerSee properties, the capture view requires an "owner". Override this to set a "ViewActor" for the scene. */
	virtual const AActor* GetViewOwner() const { return nullptr; }

	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	static ENGINE_API void UpdateDeferredCaptures(FSceneInterface* Scene);

	/** Whether this component is a USceneCaptureComponent2D */
	virtual bool Is2D() const { return false; }

	/** Whether this component is a USceneCaptureComponentCube */
	virtual bool IsCube() const { return false; }

	UE_DEPRECATED(5.6, "SetFrameUpdated is no longer used")
	bool SetFrameUpdated() { return false; }

	UE_DEPRECATED(5.6, "UpdateSceneCaptureContents now requires a SceneRenderBuilder")
	virtual void UpdateSceneCaptureContents(FSceneInterface* Scene) {};

	virtual void UpdateSceneCaptureContents(FSceneInterface* Scene, class ISceneRenderBuilder& SceneRenderBuilder) {};

	/**
	 * Returns whether a scene capture doesn't want or need lighting, and can disable several additional rendering features to save performance
	 * (see DisableFeaturesForUnlit in ShowFlags.h).  Depth or base pass outputs aren't affected by lighting, while SceneColor outputs may be generated
	 * with the Lighting flag disabled by the user.  FinalColor requires post processing, and so is assumed to require additional features, and can't
	 * take advantage of disabling features for unlit.
	 */
	bool IsUnlit() const
	{
		return CaptureSource == SCS_SceneDepth || CaptureSource == SCS_DeviceDepth || CaptureSource == SCS_Normal || CaptureSource == SCS_BaseColor ||
			(ShowFlags.Lighting == false && (CaptureSource == SCS_SceneColorHDR || CaptureSource == SCS_SceneColorHDRNoAlpha || CaptureSource == SCS_SceneColorSceneDepth));
	}

protected:
	/** Update the show flags from our show flags settings (ideally, you'd be able to set this more directly, but currently unable to make FEngineShowFlags a UStruct to use it as a FProperty...) */
	ENGINE_API void UpdateShowFlags();

	ENGINE_API void RegisterDelegates();
	ENGINE_API void UnregisterDelegates();
	ENGINE_API void ReleaseGarbageReferences();

	ENGINE_API bool IsCulledByDetailMode() const;

	/**
	 * The view state holds persistent scene rendering state and enables occlusion culling in scene captures.
	 * NOTE: This object is used by the rendering thread. When the game thread attempts to destroy it, FDeferredCleanupInterface will keep the object around until the RT is done accessing it.
	 * NOTE: It is not safe to put a FSceneViewStateReference in a TArray, which moves its contents around without calling element constructors during realloc.
	 */
	TIndirectArray<FSceneViewStateReference> ViewStates;

#if WITH_EDITORONLY_DATA
	/** The mesh to show visually where the camera is placed */
	TObjectPtr<class UStaticMeshComponent> ProxyMeshComponent;
	
	/** The path for the mesh used by ProxyMeshComponent */
	FSoftObjectPath CaptureMeshPath;

public:
	/** Thread safe storage for memory statistics for a scene capture */
	TRefCountPtr<FSceneCaptureMemorySize> CaptureMemorySize;
#endif
};

