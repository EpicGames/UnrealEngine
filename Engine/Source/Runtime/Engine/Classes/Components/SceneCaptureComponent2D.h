// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "Engine/BlendableInterface.h"
#include "Camera/CameraTypes.h"
#include "Components/SceneCaptureComponent.h"
#include "SceneCaptureComponent2D.generated.h"

class ISceneViewExtension;
class FSceneInterface;

UENUM()
enum class ESceneCaptureUnlitViewmode : uint8
{
	/** Disabled */
	Disabled,

	/** Enabled for regular captures */
	Capture,

	/** Enabled for scene captures and custom render passes (Render In Main Renderer) */
	CaptureOrCustomRenderPass
};

/**
 *	Used to capture a 'snapshot' of the scene from a single plane and feed it to a render target.
 */
UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent), ClassGroup=Rendering, editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class USceneCaptureComponent2D : public USceneCaptureComponent
{
	GENERATED_UCLASS_BODY()
		
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta=(DisplayName = "Projection Type"))
	TEnumAsByte<ECameraProjectionMode::Type> ProjectionType;

	/** Camera field of view (in degrees). */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category=Projection, meta=(DisplayName = "Field of View", UIMin = "5.0", UIMax = "170", ClampMin = "0.001", ClampMax = "360.0", editcondition = "ProjectionType==0"))
	float FOVAngle;

	/** The horizontal field of view (in degrees) used for primitives tagged as FirstPerson. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (UIMin = "5.0", UIMax = "170", ClampMin = "0.001", ClampMax = "360.0", Units = deg, EditCondition = "bEnableFirstPersonFieldOfView"))
	float FirstPersonFieldOfView;

	/** The scale to apply to primitives tagged as FirstPerson. This is used to scale down primitives towards the camera such that they are small enough not to intersect with the scene. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category=Projection, meta=(UIMin = "0.001", UIMax = "1.0", ClampMin = "0.001", ClampMax = "1.0", EditCondition = "bEnableFirstPersonScale"))
	float FirstPersonScale;

	/** The desired width (in world units) of the orthographic view (ignored in Perspective mode) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (editcondition = "ProjectionType==1"))
	float OrthoWidth;

	/** Automatically determine a min/max Near/Far clip plane position depending on OrthoWidth value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (editcondition = "ProjectionType==1"))
	bool bAutoCalculateOrthoPlanes;

	/** Manually adjusts the planes of this camera, maintaining the distance between them. Positive moves out to the farplane, negative towards the near plane */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (editcondition = "ProjectionType==1 && bAutoCalculateOrthoPlanes"))
	float AutoPlaneShift;

	/** Adjusts the near/far planes and the view origin of the current camera automatically to avoid clipping and light artefacting*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (editcondition = "ProjectionType==1"))
	bool bUpdateOrthoPlanes;

	/** If UpdateOrthoPlanes is enabled, this setting will use the cameras current height to compensate the distance to the general view (as a pseudo distance to view target when one isn't present) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection, meta = (editcondition = "ProjectionType==1 && bUpdateOrthoPlanes"))
	bool bUseCameraHeightAsViewTarget;

	/** Amount to increase the view frustum by, from 0.0 for no increase to 1.0 for 100% increase */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Projection, meta = (UIMin="0.0", ClampMin="0.0", UIMax="1.0", ClampMax="1.0"))
	float Overscan;
	
	/** Output render target of the scene capture that can be read in materials. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture)
	TObjectPtr<class UTextureRenderTarget2D> TextureTarget;

	/** When enabled, the scene capture will composite into the render target instead of overwriting its contents. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SceneCapture)
	TEnumAsByte<enum ESceneCaptureCompositeMode> CompositeMode;

	UPROPERTY(interp, Category=PostProcessVolume, meta=(ShowOnlyInnerProperties))
	struct FPostProcessSettings PostProcessSettings;

	/** Range (0.0, 1.0) where 0 indicates no effect, 1 indicates full effect. */
	UPROPERTY(interp, Category=PostProcessVolume, BlueprintReadWrite, meta=(UIMin = "0.0", UIMax = "1.0"))
	float PostProcessBlendWeight;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Projection, meta = (InlineEditConditionToggle))
	uint32 bOverride_CustomNearClippingPlane : 1;

	/** 
	 * Set bOverride_CustomNearClippingPlane to true if you want to use a custom clipping plane instead of GNearClippingPlane.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Projection, meta = (editcondition = "bOverride_CustomNearClippingPlane"))
	float CustomNearClippingPlane = 0;

	/** Whether a custom projection matrix will be used during rendering. Use with caution. Does not currently affect culling */
	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, Category = Projection)
	bool bUseCustomProjectionMatrix;

	/** The custom projection matrix to use */
	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, Category = Projection)
	FMatrix CustomProjectionMatrix;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.4 - bUseFauxOrthoViewPos has been deprecated alongside updates to Orthographic camera fixes"))
	bool bUseFauxOrthoViewPos = false;

	/** Render the scene in n frames (i.e TileCount) - Ignored in Perspective mode, works only in Orthographic mode when CaptureSource uses SceneColor (not FinalColor)
	* If CaptureSource uses FinalColor, tiling will be ignored and a Warning message will be logged	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Projection, meta = (editcondition = "ProjectionType==1"))
	bool bEnableOrthographicTiling = false;

	/** Number of X tiles to render. Ignored in Perspective mode, works only in Orthographic mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Projection, meta = (ClampMin = "1", ClampMax = "64", editcondition = "ProjectionType==1 && bEnableOrthographicTiling"))
	int32 NumXTiles = 4;

	/** Number of Y tiles to render. Ignored in Perspective mode, works only in Orthographic mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Projection, meta = (ClampMin = "1", ClampMax = "64", editcondition = "ProjectionType==1 && bEnableOrthographicTiling"))
	int32 NumYTiles = 4;

	/**
	 * Enables a clip plane while rendering the scene capture which is useful for portals.  
	 * The global clip plane must be enabled in the renderer project settings for this to work.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=SceneCapture)
	bool bEnableClipPlane;

	/** Base position for the clip plane, can be any position on the plane. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=SceneCapture)
	FVector ClipPlaneBase;

	/** Normal for the plane. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=SceneCapture)
	FVector ClipPlaneNormal;
	
	/** Render scene capture as additional render passes of the main renderer rather than as an independent renderer. Applies to scene depth, device depth, base color, normal, and scene color modes (disables lighting and shadows). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo, meta = (EditCondition = "CaptureSource == ESceneCaptureSource::SCS_SceneDepth || CaptureSource == ESceneCaptureSource::SCS_DeviceDepth || CaptureSource == ESceneCaptureSource::SCS_BaseColor || CaptureSource == ESceneCaptureSource::SCS_Normal || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDRNoAlpha || CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth"))
	bool bRenderInMainRenderer = false;

	/**
	 * Option to enable a debug feature which outputs base color to the emissive channel when lighting is disabled via ShowFlag
	 * or via Render In Main Renderer, which renders the capture as a Custom Render Pass.  Note that the debug feature requires
	 * development shaders to be compiled, generally only true in non-shipping builds on PC!  To work in other cases, materials
	 * should directly write to the emissive channel (or be unlit materials), rather than counting on the debug feature.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo)
	ESceneCaptureUnlitViewmode UnlitViewmode = ESceneCaptureUnlitViewmode::Capture;

	/** 
	 * True if we did a camera cut this frame. Automatically reset to false at every capture.
	 * This flag affects various things in the renderer (such as whether to use the occlusion queries from last frame, and motion blur).
	 * Similar to UPlayerCameraManager::bGameCameraCutThisFrame.
	 */
	UPROPERTY(Transient, BlueprintReadWrite, Category = SceneCapture)
	uint32 bCameraCutThisFrame : 1;

	/** True if the first person field of view should be used for primitives tagged as FirstPerson. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection)
	uint32 bEnableFirstPersonFieldOfView : 1;

	/** True if the first person scale should be used for primitives tagged as FirstPerson. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projection)
	uint32 bEnableFirstPersonScale : 1;

	/** Whether to only render exponential height fog on opaque pixels which were rendered by the scene capture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture, meta = (DisplayName = "Fog only on rendered pixels"))
	uint32 bConsiderUnrenderedOpaquePixelAsFullyTranslucent : 1;

	/** Render with main view family, for example with the main editor or game viewport which mark their view families as "main". */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo)
	uint32 bMainViewFamily : 1;

	/** Render with main view resolution, ignoring the dimensions in the resource.  Enables Main View Family. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PassInfo)
	uint32 bMainViewResolution : 1;

	/** Render with main view camera.  Enables Main View Family.  Temporal AA jitter is matched with main view if Main View Resolution is enabled and no Resolution Divisor is set. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo)
	uint32 bMainViewCamera : 1;

	/** Inherit the main view camera post-process settings and ignore local default values. Local active overrides will function as usual. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PassInfo, meta = (EditCondition = "bMainViewCamera"))
	uint32 bInheritMainViewCameraPostProcessSettings : 1;

	/** When rendering with main view resolution, ignore screen percentage scale and render at full resolution.  Temporal AA jitter is also disabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PassInfo)
	uint32 bIgnoreScreenPercentage : 1;

	/** Divisor when rendering at Main View Resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo, meta = (EditCondition = "bMainViewResolution || bMainViewCamera"))
	FIntPoint MainViewResolutionDivisor = { 1, 1 };

	/** Expose BaseColor as a UserSceneTexture.  Requires "Render In Main Renderer".  Enables Main View Family and Resolution, disables "Ignore Screen Percentage".  Useful to get multiple outputs from a Custom Render Pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo, meta = (EditCondition = "bRenderInMainRenderer && (CaptureSource == ESceneCaptureSource::SCS_SceneDepth || CaptureSource == ESceneCaptureSource::SCS_DeviceDepth || CaptureSource == ESceneCaptureSource::SCS_BaseColor || CaptureSource == ESceneCaptureSource::SCS_Normal || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDRNoAlpha || CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth)"))
	FName UserSceneTextureBaseColor;

	/** Expose Normal as a UserSceneTexture.  Requires "Render In Main Renderer".  Enables Main View Family and Resolution, disables "Ignore Screen Percentage".  Useful to get multiple outputs from a Custom Render Pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo, meta = (EditCondition = "bRenderInMainRenderer && (CaptureSource == ESceneCaptureSource::SCS_SceneDepth || CaptureSource == ESceneCaptureSource::SCS_DeviceDepth || CaptureSource == ESceneCaptureSource::SCS_BaseColor || CaptureSource == ESceneCaptureSource::SCS_Normal || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDRNoAlpha || CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth)"))
	FName UserSceneTextureNormal;

	/** Expose SceneColor (emissive/unlit) as a UserSceneTexture.  Requires "Render In Main Renderer".  Enables Main View Family and Resolution, disables "Ignore Screen Percentage".  Useful to get multiple outputs from a Custom Render Pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PassInfo, meta = (EditCondition = "bRenderInMainRenderer && (CaptureSource == ESceneCaptureSource::SCS_SceneDepth || CaptureSource == ESceneCaptureSource::SCS_DeviceDepth || CaptureSource == ESceneCaptureSource::SCS_BaseColor || CaptureSource == ESceneCaptureSource::SCS_Normal || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDRNoAlpha || CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth)"))
	FName UserSceneTextureSceneColor;

	inline bool ShouldRenderInMainRenderer() const
	{
		return bRenderInMainRenderer && (CaptureSource == ESceneCaptureSource::SCS_SceneDepth || CaptureSource == ESceneCaptureSource::SCS_DeviceDepth || CaptureSource == ESceneCaptureSource::SCS_BaseColor || CaptureSource == ESceneCaptureSource::SCS_Normal || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR || CaptureSource == ESceneCaptureSource::SCS_SceneColorHDRNoAlpha || CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth);
	}

	inline bool ShouldRenderWithMainViewResolution() const
	{
		return bMainViewResolution || (ShouldRenderInMainRenderer() && (!UserSceneTextureBaseColor.IsNone() || !UserSceneTextureNormal.IsNone() || !UserSceneTextureSceneColor.IsNone()));
	}

	inline bool ShouldRenderWithMainViewFamily() const
	{
		return bMainViewFamily || bMainViewCamera || ShouldRenderWithMainViewResolution();
	}

	inline bool ShouldRenderWithMainViewCamera() const
	{
		return bMainViewCamera;
	}

	inline bool ShouldIgnoreScreenPercentage() const
	{
		// User Scene Texture outputs are always in the scaled view resolution, so ignore the bIgnoreScreenPercentage flag if either is set
		return ShouldRenderInMainRenderer() ? bIgnoreScreenPercentage && UserSceneTextureBaseColor.IsNone() && UserSceneTextureNormal.IsNone() && UserSceneTextureSceneColor.IsNone() : bIgnoreScreenPercentage;
	}

	/** Array of scene view extensions specifically to apply to this scene capture */
	TArray< TWeakPtr<ISceneViewExtension, ESPMode::ThreadSafe> > SceneViewExtensions;

	/** Which tile to render of the orthographic view (ignored in Perspective mode) */
	int32 TileID = 0;

	/** Transient pointer set during calls to UpdateSceneCaptureContents from UpdateDeferredCaptures */
	const FSceneViewFamily* MainViewFamily = nullptr;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	virtual bool RequiresGameThreadEndOfFrameUpdates() const override
	{
		// this method could probably be removed allowing them to run on any thread, but it isn't worth the trouble
		return true;
	}
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	/** Reset Orthographic tiling counter */
	ENGINE_API void ResetOrthographicTilingCounter();

	//~ End UActorComponent Interface

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void Serialize(FArchive& Ar);

	//~ End UObject Interface

	ENGINE_API void SetCameraView(const FMinimalViewInfo& DesiredView);

	ENGINE_API virtual void GetCameraView(float DeltaTime, FMinimalViewInfo& OutDesiredView);

	/** Adds an Blendable (implements IBlendableInterface) to the array of Blendables (if it doesn't exist) and update the weight */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void AddOrUpdateBlendable(TScriptInterface<IBlendableInterface> InBlendableObject, float InWeight = 1.0f) { PostProcessSettings.AddBlendable(InBlendableObject, InWeight); }

	/** Removes a blendable. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	void RemoveBlendable(TScriptInterface<IBlendableInterface> InBlendableObject) { PostProcessSettings.RemoveBlendable(InBlendableObject); }

	/**
	 * Render the scene to the texture the next time the main view is rendered.
	 * If r.SceneCapture.CullByDetailMode is set, nothing will happen if DetailMode is higher than r.DetailMode.
	 */
	ENGINE_API void CaptureSceneDeferred();

	// For backwards compatibility
	void UpdateContent() { CaptureSceneDeferred(); }

	/** 
	 * Render the scene to the texture target immediately.  
	 * This should not be used if bCaptureEveryFrame is enabled, or the scene capture will render redundantly. 
	 * If r.SceneCapture.CullByDetailMode is set, nothing will happen if DetailMode is higher than r.DetailMode.
	 */
	UFUNCTION(BlueprintCallable,Category = "Rendering|SceneCapture")
	ENGINE_API void CaptureScene();

	ENGINE_API void UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder) override;

	/* Return if orthographic tiling rendering is enabled or not */
	ENGINE_API bool GetEnableOrthographicTiling() const;

	/* Return number of X tiles to render (to be used when orthographic tiling rendering is enabled) */
	ENGINE_API int32 GetNumXTiles() const;

	/* Return number of Y tiles to render (to be used when orthographic tiling rendering is enabled) */
	ENGINE_API int32 GetNumYTiles() const;

	/** Whether this component is a USceneCaptureComponent2D */
	virtual bool Is2D() const override { return true; }

#if WITH_EDITORONLY_DATA
	ENGINE_API void UpdateDrawFrustum();

	/** The frustum component used to show visually where the camera field of view is */
	TObjectPtr<class UDrawFrustumComponent> DrawFrustum;
#endif
};
