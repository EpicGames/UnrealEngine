// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneCaptureComponent.h"
#include "SceneCaptureComponentCube.generated.h"

class FSceneInterface;
class UReflectionCaptureComponent;

/**
 *	Used to capture a 'snapshot' of the scene from a 6 planes and feed it to a render target.
 */
UCLASS(hidecategories = (Collision, Object, Physics, SceneComponent), ClassGroup=Rendering, editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USceneCaptureComponentCube : public USceneCaptureComponent
{
	GENERATED_UCLASS_BODY()

	/** Temporary render target that can be used by the editor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	TObjectPtr<class UTextureRenderTargetCube> TextureTarget;

	/** Preserve the rotation of the actor when updating the capture. The default behavior is to capture the cube aligned to the world axis system.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SceneCapture)
	bool bCaptureRotation;

	/** Transient, used for runtime reflection captures, to support time slicing. */
	uint8 ReflectionTimeSliceFirst;			// First slice to render
	uint8 ReflectionTimeSliceCount;			// Number of slices to render

	/** Transient, used for runtime reflection captures.  Not dereferenced, just used to look up the reflection cube map array index. */
	UReflectionCaptureComponent* ReflectionComponent;

public:
	UPROPERTY(interp, Category = PostProcessVolume, meta = (ShowOnlyInnerProperties))
	struct FPostProcessSettings PostProcessSettings;

	/** Range (0.0, 1.0) where 0 indicates no effect, 1 indicates full effect. */
	UPROPERTY(interp, Category = PostProcessVolume, BlueprintReadWrite, meta = (UIMin = "0.0", UIMax = "1.0"))
	float PostProcessBlendWeight;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface

	//~ Begin UObject Interface
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void Serialize(FArchive& Ar);
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	/**
	 * Render the scene to the texture the next time the main view is rendered.
	 * If r.SceneCapture.CullByDetailMode is set, nothing will happen if DetailMode is higher than r.DetailMode.
	 */
	ENGINE_API void CaptureSceneDeferred();

	/** 
	 * Render the scene to the texture target immediately.  
	 * This should not be used if bCaptureEveryFrame is enabled, or the scene capture will render redundantly. 
	 * If r.SceneCapture.CullByDetailMode is set, nothing will happen if DetailMode is higher than r.DetailMode.
	 */
	UFUNCTION(BlueprintCallable,Category = "Rendering|SceneCapture")
	ENGINE_API void CaptureScene();

	// For backwards compatibility
	void UpdateContent() { CaptureSceneDeferred(); }

	ENGINE_API void UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder) override;

	/** Whether this component is a USceneCaptureComponentCube */
	virtual bool IsCube() const override { return true; }

#if WITH_EDITORONLY_DATA
	ENGINE_API void UpdateDrawFrustum();

	/** The frustum component used to show visually where the camera field of view is */
	TObjectPtr<class UDrawFrustumComponent> DrawFrustum;
#endif

	friend class FScene;
};
