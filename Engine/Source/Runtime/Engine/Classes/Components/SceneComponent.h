// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "UObject/ObjectMacros.h"
#include "Math/BoxSphereBounds.h"
#include "UObject/UObjectGlobals.h"
#include "ComponentInstanceDataCache.h"
#include "Components/ActorComponent.h"
#include "TransformStateStreamHandle.h"

#include "SceneComponent.generated.h"

class AActor;
class APhysicsVolume;
class USceneComponent;
class FScopedMovementUpdate;
struct FHitResult;
struct FLevelCollection;

struct FOverlapInfo;
typedef TArrayView<const FOverlapInfo> TOverlapArrayView;
namespace ERHIFeatureLevel { enum Type : int; }

/** 
 * Detail mode for scene component rendering, corresponds with the world GetDetailMode which is set based on platform and scalability settings.
 * If settings like r.CookOutUnusedDetailModeComponents are used, this can result in components being removed entirely when the target detail mode is higher.
 */
UENUM()
enum EDetailMode : int
{
	DM_Low UMETA(DisplayName="Low"),
	DM_Medium UMETA(DisplayName="Medium"),
	DM_High UMETA(DisplayName="High"),
	DM_Epic UMETA(DisplayName="Epic"),
	DM_MAX,
};

/** The space for the transform */
UENUM()
enum ERelativeTransformSpace : int
{
	/** World space transform. */
	RTS_World,
	/** Actor space transform. */
	RTS_Actor,
	/** Component space transform. */
	RTS_Component,
	/** Parent bone space transform */
	RTS_ParentBoneSpace,
};

/** MoveComponent options, stored as bitflags */
enum EMoveComponentFlags
{
	/** Default options */
	MOVECOMP_NoFlags							= 0x0000,	
	/** Ignore collisions with things the Actor is based on */
	MOVECOMP_IgnoreBases						= 0x0001,	
	/** When moving this component, do not move the physics representation. Used internally to avoid looping updates when syncing with physics. */
	MOVECOMP_SkipPhysicsMove					= 0x0002,	
	/** Never ignore initial blocking overlaps during movement, which are usually ignored when moving out of an object. MOVECOMP_IgnoreBases is still respected. */
	MOVECOMP_NeverIgnoreBlockingOverlaps		= 0x0004,	
	/** avoid dispatching blocking hit events when the hit started in penetration (and is not ignored, see MOVECOMP_NeverIgnoreBlockingOverlaps). */
	MOVECOMP_DisableBlockingOverlapDispatch		= 0x0008,	
	/** Compare the root actor of a blocking hit with the ignore UPrimitiveComponent::MoveIgnoreActors array */
	MOVECOMP_CheckBlockingRootActorInIgnoreList	= 0x0010,	
};
// Declare bitwise operators to allow EMoveComponentFlags to be combined but still retain type safety
ENUM_CLASS_FLAGS(EMoveComponentFlags);

/** Comparison tolerance for checking if two FQuats are the same when moving SceneComponents. */
#define SCENECOMPONENT_QUAT_TOLERANCE		(1.e-8f) 
/** Comparison tolerance for checking if two FRotators are the same when moving SceneComponents. */
#define SCENECOMPONENT_ROTATOR_TOLERANCE	(1.e-4f) 

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPhysicsVolumeChanged, USceneComponent, PhysicsVolumeChangedDelegate, class APhysicsVolume*, NewVolume);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FIsRootComponentChanged, USceneComponent, IsRootComponentChanged, USceneComponent*, UpdatedComponent, bool, bIsRootComponent);
DECLARE_EVENT_ThreeParams(USceneComponent, FTransformUpdated, USceneComponent* /*UpdatedComponent*/, EUpdateTransformFlags /*UpdateTransformFlags*/, ETeleportType /*Teleport*/);

/**
 * A SceneComponent has a transform and supports attachment, but has no rendering or collision capabilities.
 * Useful as a 'dummy' component in the hierarchy to offset others.
 * @see [Scene Components](https://docs.unrealengine.com/latest/INT/Programming/UnrealArchitecture/Actors/Components/index.html#scenecomponents)
 */
UCLASS(ClassGroup=(Utility, Common), BlueprintType, hideCategories=(Trigger, PhysicsVolume), meta=(BlueprintSpawnableComponent, IgnoreCategoryKeywordsInSubclasses, ShortTooltip="A Scene Component is a component that has a scene transform and can be attached to other scene components."), MinimalAPI)
class USceneComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	/** The name to use for the default scene root variable */
	static ENGINE_API FName GetDefaultSceneRootVariableName();

	/** UObject constructor that takes an optional ObjectInitializer */
	ENGINE_API USceneComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Cached level collection that contains the level this component is registered in, for fast access in IsVisible(). */
	const FLevelCollection* CachedLevelCollection;

private:
	/** Physics Volume in which this SceneComponent is located **/
	UPROPERTY(transient)
	TWeakObjectPtr<class APhysicsVolume> PhysicsVolume;

	/** What we are currently attached to. If valid, RelativeLocation etc. are used relative to this object */
	UPROPERTY(ReplicatedUsing = OnRep_AttachParent, meta=(DisableLevelInstancePropertyOverride))
	TObjectPtr<USceneComponent> AttachParent;

	/** Optional socket name on AttachParent that we are attached to. */
	UPROPERTY(ReplicatedUsing = OnRep_AttachSocketName, meta = (DisableLevelInstancePropertyOverride))
	FName AttachSocketName;

	FName NetOldAttachSocketName;

	/** List of child SceneComponents that are attached to us. */
	UPROPERTY(ReplicatedUsing = OnRep_AttachChildren, Transient)
	TArray<TObjectPtr<USceneComponent>> AttachChildren;

	/** Set of attached SceneComponents that were attached by the client so we can fix up AttachChildren when it is replicated to us. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<USceneComponent>> ClientAttachedChildren;

	USceneComponent* NetOldAttachParent;

public:
	/** Current bounds of the component */
	FBoxSphereBounds Bounds;

	const FBoxSphereBounds& GetBounds() const
	{
		return Bounds;
	}

private:
	/** Location of the component relative to its parent */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, ReplicatedUsing=OnRep_Transform, Category = Transform, meta=(AllowPrivateAccess="true", LinearDeltaSensitivity = "1", Delta = "1.0"))
	FVector RelativeLocation;

	/** Rotation of the component relative to its parent */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, ReplicatedUsing=OnRep_Transform, Category=Transform, meta=(AllowPrivateAccess="true", LinearDeltaSensitivity = "1", Delta = "1.0"))
	FRotator RelativeRotation;

	/**
	*	Non-uniform scaling of the component relative to its parent.
	*	Note that scaling is always applied in local space (no shearing etc)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, ReplicatedUsing=OnRep_Transform, Category=Transform, meta=(AllowPrivateAccess="true", LinearDeltaSensitivity = "1", Delta = "0.0025"))
	FVector RelativeScale3D;

public:
	/**
	* Velocity of the component.
	* @see GetComponentVelocity()
	*/
	UPROPERTY()
	FVector ComponentVelocity;

private:
	/** True if we have ever updated ComponentToWorld based on RelativeLocation/Rotation/Scale. Used at startup to make sure it is initialized. */
	UPROPERTY(Transient)
	uint8 bComponentToWorldUpdated : 1;

	/** If true it indicates we don't need to call UpdateOverlaps. This is an optimization to avoid tree traversal when no attached components require UpdateOverlaps to be called.
	* This should only be set to true as a result of UpdateOverlaps. To dirty this flag see ClearSkipUpdateOverlaps() which is expected when state affecting UpdateOverlaps changes (attachment, Collision settings, etc...) */
	uint8 bSkipUpdateOverlaps : 1;

	/** If RelativeLocation should be considered relative to the world, rather than the parent */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, ReplicatedUsing=OnRep_Transform, Category=Transform, meta=(AllowPrivateAccess="true"))
	uint8 bAbsoluteLocation:1;

	/** If RelativeRotation should be considered relative to the world, rather than the parent */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, ReplicatedUsing=OnRep_Transform, Category=Transform, meta=(AllowPrivateAccess="true"))
	uint8 bAbsoluteRotation:1;

	/** If RelativeScale3D should be considered relative to the world, rather than the parent */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, ReplicatedUsing=OnRep_Transform, Category=Transform, meta=(AllowPrivateAccess="true"))
	uint8 bAbsoluteScale:1;

	/** Whether to completely draw the primitive; if false, the primitive is not drawn, does not cast a shadow. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, ReplicatedUsing=OnRep_Visibility,  Category = Rendering, meta=(AllowPrivateAccess="true"))
	uint8 bVisible:1;

	/** Whether or not we should be attached. */
	UPROPERTY(Transient, Replicated)
	uint8 bShouldBeAttached : 1;

	UPROPERTY(Transient, Replicated)
	uint8 bShouldSnapLocationWhenAttached : 1;

	UPROPERTY(Transient, Replicated)
	uint8 bShouldSnapRotationWhenAttached : 1;

	UPROPERTY(Transient, Replicated)
	uint8 bShouldSnapScaleWhenAttached : 1;

#if WITH_EDITORONLY_DATA
	/** Whether this component is temporarily hidden within the editor. */
	UPROPERTY(Transient)
	uint8 bHiddenEdTemporary : 1;
#endif // WITH_EDITORONLY_DATA

	/** Sets bShouldBeAttached, push model aware. */
	ENGINE_API void SetShouldBeAttached(bool bNewShouldBeAttached);

	/** Sets bShouldSnapLocationWhenAttached, push model aware. */
	ENGINE_API void SetShouldSnapLocationWhenAttached(bool bShouldSnapLocation);

	/** Sets bShouldSnapRotationWhenAttached, push model aware. */
	ENGINE_API void SetShouldSnapRotationWhenAttached(bool bShouldSnapRotation);

	/** Sets bShouldSnapScaleWhenAttached, push model aware. */
	ENGINE_API void SetShouldSnapScaleWhenAttached(bool bShouldSnapScale);

	/**
	 * Whether or not the cached PhysicsVolume this component overlaps should be updated when the component is moved.
	 * @see GetPhysicsVolume()
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintGetter=GetShouldUpdatePhysicsVolume, BlueprintSetter=SetShouldUpdatePhysicsVolume, Category=Physics)
	uint8 bShouldUpdatePhysicsVolume:1;

public:
	/** Whether to hide the primitive in game, if the primitive is Visible. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadOnly, Category=Rendering, meta=(SequencerTrackClass = "/Script/MovieSceneTracks.MovieSceneVisibilityTrack"))
	uint8 bHiddenInGame:1;

	/** If true, a change in the bounds of the component will call trigger a streaming data rebuild */
	UPROPERTY()
	uint8 bBoundsChangeTriggersStreamingDataRebuild:1;

	/** If true, this component uses its parents bounds when attached.
	 *  This can be a significant optimization with many components attached together.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Rendering)
	uint8 bUseAttachParentBound:1;

	/** If true, this component will use its current bounds transformed back into local space instead of calling CalcBounds with an identity transform. */
	UPROPERTY()
	uint8 bComputeFastLocalBounds : 1;

	/** If true, this component will cache its bounds during cooking or in PIE and never recompute it again. This is for components that are known to be static. */
	UPROPERTY()
	uint8 bComputeBoundsOnceForGame : 1;

	/** If true, this component has already cached its bounds during cooking or in PIE and will never recompute it again.  */
	UPROPERTY()
	uint8 bComputedBoundsOnceForGame : 1;

	/** Get the current local bounds of the component */
	ENGINE_API FBoxSphereBounds GetLocalBounds() const;

	/** Clears the skip update overlaps flag. This should be called any time a change to state would prevent the result of UpdateOverlaps. For example attachment, changing collision settings, etc... */
	ENGINE_API void ClearSkipUpdateOverlaps();

	/** If true, we can use the old computed overlaps */
	bool ShouldSkipUpdateOverlaps() const
	{
		return SkipUpdateOverlapsOptimEnabled == 1 && bSkipUpdateOverlaps;
	}

	/** Gets whether or not the cached PhysicsVolume this component overlaps should be updated when the component is moved.	*/
	UFUNCTION(BlueprintGetter)
	bool GetShouldUpdatePhysicsVolume() const;

	/** Sets whether or not the cached PhysicsVolume this component overlaps should be updated when the component is moved. */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetShouldUpdatePhysicsVolume(bool bInShouldUpdatePhysicsVolume);

	/** If true, this component stops the walk up the attachment chain in GetActorPositionForRenderer(). Instead this component's child will be used as the attachment root. */
	UPROPERTY()
	uint8 bIsNotRenderAttachmentRoot : 1;

protected:
	/** Transient flag that temporarily disables UpdateOverlaps within DetachFromParent(). */
	uint8 bDisableDetachmentUpdateOverlaps:1;

	/** If true, OnUpdateTransform virtual will be called each time this component is moved. */
	uint8 bWantsOnUpdateTransform:1;
	
	/**
    * If true, then this scene component has requested to have some movement
    * applied as part of a deferred group. 
    */
    uint8 bHasGroupedMoveThisFrame : 1 = false;

private:
	uint8 bNetUpdateTransform : 1;
	uint8 bNetUpdateAttachment : 1;

public:
	/** Global flag to enable/disable overlap optimizations, settable with p.SkipUpdateOverlapsOptimEnabled cvar */ 
	static ENGINE_API int32 SkipUpdateOverlapsOptimEnabled;

#if WITH_EDITORONLY_DATA
	/** This component should create a sprite component for visualization in the editor */
	UPROPERTY()
	uint8 bVisualizeComponent : 1;
#endif

	/** How often this component is allowed to move, used to make various optimizations. Only safe to set in constructor. */
	UPROPERTY(Category = Mobility, EditAnywhere, BlueprintReadOnly, Replicated)
	TEnumAsByte<EComponentMobility::Type> Mobility;

	/** 
	 * Set the required detail level of this component.
	 * It will be deleted or made invisible if the world's detail level is higher, based on platform and scalability settings.
	 * For example, a detail mode of High may prevent this component from loading on mobile platforms.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = LOD)
	TEnumAsByte<enum EDetailMode> DetailMode;

	/** Delegate that will be called when PhysicsVolume has been changed **/
	UPROPERTY(BlueprintAssignable, Category=PhysicsVolume, meta=(DisplayName="Physics Volume Changed"))
	FPhysicsVolumeChanged PhysicsVolumeChangedDelegate;

	/** Delegate invoked when this scene component becomes the actor's root component or when it no longer is. */
	FIsRootComponentChanged IsRootComponentChanged;

#if WITH_EDITORONLY_DATA
protected:
	/** Editor only component used to display the sprite so as to be able to see the location of the Component  */
	TObjectPtr<class UBillboardComponent> SpriteComponent;
	/** Creates the editor only component used to display the sprite */
	ENGINE_API void CreateSpriteComponent(class UTexture2D* SpriteTexture = nullptr);
	ENGINE_API void CreateSpriteComponent(class UTexture2D* SpriteTexture, bool bRegister);
#endif

public:
	/** Delegate called when this component is moved */
	FTransformUpdated TransformUpdated;

	/** Returns the current scoped movement update, or NULL if there is none. @see FScopedMovementUpdate */
	FScopedMovementUpdate* GetCurrentScopedMovement() const;

	/**
	* Process a single instance of a scoped movement. This is called from the world when we are flushing the 
	* deferred group movement queue.
	* @see UWorld::ProcessPendingGroupMoves
	*/
	void ProcessDeferredMovementGroup(const struct FDeferredComponentMoveData& MoveData);

	/**
	 * @return true if grouped component movement can be used, set by s.GroupedComponentMovement.Enable
	 */
	UFUNCTION()
	static ENGINE_API bool IsGroupedComponentMovementEnabled();

private:
	/** Stack of current movement scopes. */
	TArray<FScopedMovementUpdate*> ScopedMovementStack;

	ENGINE_API void BeginScopedMovementUpdate(FScopedMovementUpdate& ScopedUpdate);
	ENGINE_API void EndScopedMovementUpdate(FScopedMovementUpdate& ScopedUpdate);

	/** Internally called by OnRegister() or OnPreRegister() to handle parent attachement. */
	void AttachToParent();

	/** Cache that avoids Quat<->Rotator conversions if possible. Only to be used with GetComponentTransform().GetRotation(). */
	FRotationConversionCache WorldRotationCache;

	/** Cache that avoids Quat<->Rotator conversions if possible. Only to be used with RelativeRotation. */
	FRotationConversionCache RelativeRotationCache;

	/** Current transform of the component, relative to the world */
	FTransform ComponentToWorld;

public:
	/** Sets the RelativeRotationCache. Used to ensure component ends up with the same RelativeRotation after calling SetWorldTransform(). */
	ENGINE_API void SetRelativeRotationCache(const FRotationConversionCache& InCache);
	
	/** Get the RelativeRotationCache.  */
	inline const FRotationConversionCache& GetRelativeRotationCache() const { return RelativeRotationCache; }


#if WITH_STATE_STREAM_ACTOR
protected:
	FTransformHandle TransformHandle;

public:
	ENGINE_API FTransformHandle& GetOrCreateTransformHandle();
#endif

private:
	UFUNCTION()
	ENGINE_API void OnRep_Transform();

	UFUNCTION()
	ENGINE_API void OnRep_AttachParent();

	UFUNCTION()
	ENGINE_API void OnRep_AttachChildren();

	UFUNCTION()
	ENGINE_API void OnRep_AttachSocketName();

	UFUNCTION()
	ENGINE_API void OnRep_Visibility(bool OldValue);

public:
	/**  
	 * Convenience function to get the relative rotation from the passed in world rotation
	 * @param WorldRotation  World rotation that we want to convert to relative to the components parent
	 * @return Returns the relative rotation
	 */
	ENGINE_API FQuat GetRelativeRotationFromWorld(const FQuat & WorldRotation);

	/**
	* Set the rotation of the component relative to its parent and force RelativeRotation to be equal to new rotation.
	* This allows us to set and save Rotators with angles out side the normalized range, Note that doing so may break the 
	* RotatorCache so use with care.
	* @param NewRotation		New rotation of the component relative to its parent. We will force RelativeRotation to this value.
	* @param SweepHitResult	Hit result from any impact if sweep is true.
	* @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	* @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	*							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	*							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	*/
	ENGINE_API void SetRelativeRotationExact(FRotator NewRotation, bool bSweep = false, FHitResult* OutSweepHitResult = nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the location of the component relative to its parent
	 * @param NewLocation		New location of the component relative to its parent.		
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Relative Location", ScriptName="SetRelativeLocation"))
	ENGINE_API void K2_SetRelativeLocation(FVector NewLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	void SetRelativeLocation(FVector NewLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the rotation of the component relative to its parent
	 * @param NewRotation		New rotation of the component relative to its parent
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Relative Rotation", ScriptName="SetRelativeRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_SetRelativeRotation(FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetRelativeRotation(FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	void SetRelativeRotation(const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the transform of the component relative to its parent
	 * @param NewTransform		New transform of the component relative to its parent.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Relative Transform", ScriptName="SetRelativeTransform"))
	ENGINE_API void K2_SetRelativeTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetRelativeTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Returns the transform of the component relative to its parent */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FTransform GetRelativeTransform() const;

	/** Reset the transform of the component relative to its parent. Sets relative location to zero, relative rotation to no rotation, and Scale to 1. */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void ResetRelativeTransform();

	/** Set the non-uniform scale of the component relative to its parent */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void SetRelativeScale3D(FVector NewScale3D);

	/**
	 * Adds a delta to the translation of the component relative to its parent
	 * @param DeltaLocation		Change in location of the component relative to its parent
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Relative Location", ScriptName="AddRelativeLocation"))
	ENGINE_API void K2_AddRelativeLocation(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	void AddRelativeLocation(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta the rotation of the component relative to its parent
	 * @param DeltaRotation		Change in rotation of the component relative to is parent.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Relative Rotation", ScriptName="AddRelativeRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_AddRelativeRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	void AddRelativeRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void AddRelativeRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the location of the component in its local reference frame
	 * @param DeltaLocation		Change in location of the component in its local reference frame.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Local Offset", ScriptName="AddLocalOffset", Keywords="location position"))
	ENGINE_API void K2_AddLocalOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddLocalOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the rotation of the component in its local reference frame
	 * @param DeltaRotation		Change in rotation of the component in its local reference frame.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Local Rotation", ScriptName="AddLocalRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_AddLocalRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddLocalRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void AddLocalRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the transform of the component in its local reference frame. Scale is unchanged.
	 * @param DeltaTransform	Change in transform of the component in its local reference frame. Scale is unchanged.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Local Transform", ScriptName="AddLocalTransform"))
	ENGINE_API void K2_AddLocalTransform(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddLocalTransform(const FTransform& DeltaTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Put this component at the specified location in world space. Updates relative location to achieve the final world location.
	 * @param NewLocation		New location in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set World Location", ScriptName="SetWorldLocation"))
	ENGINE_API void K2_SetWorldLocation(FVector NewLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetWorldLocation(FVector NewLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/*
	 * Put this component at the specified rotation in world space. Updates relative rotation to achieve the final world rotation.
	 * @param NewRotation		New rotation in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set World Rotation", ScriptName="SetWorldRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_SetWorldRotation(FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetWorldRotation(FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void SetWorldRotation(const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the relative scale of the component to put it at the supplied scale in world space.
	 * @param NewScale		New scale in world space for this component.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void SetWorldScale3D(FVector NewScale);

	/**
	 * Set the transform of the component in world space.
	 * @param NewTransform		New transform in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set World Transform", ScriptName="SetWorldTransform"))
	ENGINE_API void K2_SetWorldTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetWorldTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the location of the component in world space.
	 * @param DeltaLocation		Change in location in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add World Offset", ScriptName="AddWorldOffset", Keywords="location position"))
	ENGINE_API void K2_AddWorldOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddWorldOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the rotation of the component in world space.
	 * @param DeltaRotation		Change in rotation in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination (currently not supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add World Rotation", ScriptName="AddWorldRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_AddWorldRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddWorldRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void AddWorldRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the transform of the component in world space. Ignores scale and sets it to (1,1,1).
	 * @param DeltaTransform	Change in transform in world space for the component. Scale is ignored.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add World Transform", ScriptName="AddWorldTransform"))
	ENGINE_API void K2_AddWorldTransform(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddWorldTransform(const FTransform& DeltaTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the transform of the component in world space. Scale is unchanged.
	 * @param DeltaTransform	Change in transform in world space for the component. Scale is ignored since we preserve the original scale.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add World Transform Keep Scale", ScriptName="AddWorldTransformKeepScale"))
	ENGINE_API void K2_AddWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Return location of the component, in world space */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get World Location", ScriptName = "GetWorldLocation", Keywords = "position"), Category="Transformation")
	ENGINE_API FVector K2_GetComponentLocation() const;

	/** Returns rotation of the component, in world space. */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get World Rotation", ScriptName = "GetWorldRotation"), Category="Transformation")
	ENGINE_API FRotator K2_GetComponentRotation() const;
	
	/** Returns scale of the component, in world space. */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get World Scale", ScriptName = "GetWorldScale"), Category="Transformation")
	ENGINE_API FVector K2_GetComponentScale() const;

	/** Get the current component-to-world transform for this component */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get World Transform", ScriptName = "GetWorldTransform"), Category="Transformation")
	ENGINE_API FTransform K2_GetComponentToWorld() const;

	/** Get the forward (X) unit direction vector from this component, in world space.  */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FVector GetForwardVector() const;

	/** Get the up (Z) unit direction vector from this component, in world space.  */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FVector GetUpVector() const;

	/** Get the right (Y) unit direction vector from this component, in world space.  */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FVector GetRightVector() const;

	/** Returns whether the specified body is currently using physics simulation */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual bool IsSimulatingPhysics(FName BoneName = NAME_None) const;

	/** Returns whether the specified body is currently using physics simulation */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API virtual bool IsAnySimulatingPhysics() const;

	/** Get the SceneComponents that are attached to this component. */
	const TArray<TObjectPtr<USceneComponent>>& GetAttachChildren() const;

	/** Get the SceneComponent we are attached to. */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	USceneComponent* GetAttachParent() const;

	/** Get the socket we are attached to. */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	FName GetAttachSocketName() const;

	/** Gets all attachment parent components up to and including the root component */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API void GetParentComponents(TArray<USceneComponent*>& Parents) const;

	/** Gets the number of attached children components */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API int32 GetNumChildrenComponents() const;

	/** Gets the attached child component at the specified location */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API USceneComponent* GetChildComponent(int32 ChildIndex) const;

	/** 
	 * Gets all components that are attached to this component, possibly recursively
	 * @param bIncludeAllDescendants Whether to include all descendants in the list of children (i.e. grandchildren, great grandchildren, etc.)
	 * @param Children The list of attached child components
	 */
	UFUNCTION(BlueprintCallable, Category="Components")
	ENGINE_API void GetChildrenComponents(bool bIncludeAllDescendants, TArray<USceneComponent*>& Children) const;

	/** 
	* Initializes desired Attach Parent and SocketName to be attached to when the component is registered.
	* Generally intended to be called from its Owning Actor's constructor and should be preferred over AttachToComponent when
	* a component is not registered.
	* @param  InParent				Parent to attach to.
	* @param  InSocketName			Optional socket to attach to on the parent.
	*/
	ENGINE_API void SetupAttachment(USceneComponent* InParent, FName InSocketName = NAME_None);

	/** Backwards compatibility: Used to convert old-style EAttachLocation to new-style EAttachmentRules */
	static ENGINE_API void ConvertAttachLocation(EAttachLocation::Type InAttachLocation, EAttachmentRule& InOutLocationRule, EAttachmentRule& InOutRotationRule, EAttachmentRule& InOutScaleRule);

	/** DEPRECATED - Use AttachToComponent() instead */
	UE_DEPRECATED(4.17, "This function is deprecated, please use AttachToComponent() instead.")
	UFUNCTION(BlueprintCallable, Category = "Transformation", meta = (DeprecationMessage = "OVERRIDE BAD MESSAGE", DisplayName = "AttachTo (Deprecated)", AttachType = "KeepRelativeOffset"))
	ENGINE_API bool K2_AttachTo(USceneComponent* InParent, FName InSocketName = NAME_None, EAttachLocation::Type AttachType = EAttachLocation::KeepRelativeOffset, bool bWeldSimulatedBodies = true);

	/**
	* Attach this component to another scene component, optionally at a named socket. It is valid to call this on components whether or not they have been Registered, however from
	* constructor or when not registered it is preferable to use SetupAttachment.
	* @param  Parent				Parent to attach to.
	* @param  AttachmentRules		How to handle transforms & welding when attaching.
	* @param  SocketName			Optional socket to attach to on the parent.
	* @return True if attachment is successful (or already attached to requested parent/socket), false if attachment is rejected and there is no change in AttachParent.
	*/
	ENGINE_API virtual bool AttachToComponent(USceneComponent* InParent, const FAttachmentTransformRules& AttachmentRules, FName InSocketName = NAME_None );

	/**
	* Attach this component to another scene component, optionally at a named socket. It is valid to call this on components whether or not they have been Registered.
	* @param  Parent					Parent to attach to.
	* @param  SocketName				Optional socket to attach to on the parent.
	* @param  LocationRule				How to handle translation when attaching.
	* @param  RotationRule				How to handle rotation when attaching.
	* @param  ScaleRule					How to handle scale when attaching.
	* @param  bWeldSimulatedBodies		Whether to weld together simulated physics bodies. This transfers the shapes in the welded object into the parent (if simulated), which can result in permanent changes that persist even after subsequently detaching.
	* @return True if attachment is successful (or already attached to requested parent/socket), false if attachment is rejected and there is no change in AttachParent.
	*/
	UFUNCTION(BlueprintCallable, Category = "Transformation", meta = (DisplayName = "Attach Component To Component", ScriptName = "AttachToComponent", bWeldSimulatedBodies=true))
	ENGINE_API bool K2_AttachToComponent(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule, bool bWeldSimulatedBodies);

	/** DEPRECATED - Use DetachFromComponent() instead */
	UE_DEPRECATED(4.12, "This function is deprecated, please use DetachFromComponent() instead.")
	UFUNCTION(BlueprintCallable, Category = "Transformation", meta = (DisplayName = "DetachFromParent (Deprecated)"))
	ENGINE_API virtual void DetachFromParent(bool bMaintainWorldPosition = false, bool bCallModify = true);

	/** 
	 * Detach this component from whatever it is attached to. Automatically unwelds components that are welded together (see AttachToComponent), though note that some effects of welding may not be undone.
	 * @param LocationRule				How to handle translations when detaching.
	 * @param RotationRule				How to handle rotation when detaching.
	 * @param ScaleRule					How to handle scales when detaching.
	 * @param bCallModify				If true, call Modify() on the component and the current attach parent component
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Detach From Component", ScriptName = "DetachFromComponent"), Category = "Transformation")
	ENGINE_API void K2_DetachFromComponent(EDetachmentRule LocationRule = EDetachmentRule::KeepRelative, EDetachmentRule RotationRule = EDetachmentRule::KeepRelative, EDetachmentRule ScaleRule = EDetachmentRule::KeepRelative, bool bCallModify = true);

	/** 
	 * Detach this component from whatever it is attached to. Automatically unwelds components that are welded together (See AttachToComponent), though note that some effects of welding may not be undone.
	 * @param DetachmentRules			How to handle transforms & modification when detaching.
	 */
	ENGINE_API virtual void DetachFromComponent(const FDetachmentTransformRules& DetachmentRules);

	/** 
	 * Gets the names of all the sockets on the component.
	 * @return Get the names of all the sockets on the component.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	ENGINE_API TArray<FName> GetAllSocketNames() const;

	/** 
	 * Get world-space socket transform.
	 * @param InSocketName Name of the socket or the bone to get the transform 
	 * @return Socket transform in world space if socket if found. Otherwise it will return component's transform in world space.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	ENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const;

	/** 
	 * Get world-space socket or bone location.
	 * @param InSocketName Name of the socket or the bone to get the transform 
	 * @return Socket transform in world space if socket is found. Otherwise it will return component's transform in world space.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	ENGINE_API virtual FVector GetSocketLocation(FName InSocketName) const;

	/** 
	 * Get world-space socket or bone  FRotator rotation.
	 * @param InSocketName Name of the socket or the bone to get the transform 
	 * @return Socket transform in world space if socket if found. Otherwise it will return component's transform in world space.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	ENGINE_API virtual FRotator GetSocketRotation(FName InSocketName) const;

	/** 
	 * Get world-space socket or bone FQuat rotation.
	 * @param InSocketName Name of the socket or the bone to get the transform 
	 * @return Socket transform in world space if socket if found. Otherwise it will return component's transform in world space.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone", DeprecatedFunction, DeprecationMessage="Use GetSocketRotation instead, Quat is not fully supported in blueprints."))
	ENGINE_API virtual FQuat GetSocketQuaternion(FName InSocketName) const;

	/** 
	 * Return true if socket with the given name exists
	 * @param InSocketName Name of the socket or the bone to get the transform 
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	ENGINE_API virtual bool DoesSocketExist(FName InSocketName) const;

	/**
	 * Returns true if this component has any sockets
	 */
	ENGINE_API virtual bool HasAnySockets() const;

	/**
	 * Get a list of sockets this component contains
	 */
	ENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const;

	/** 
	 * Get velocity of the component: either ComponentVelocity, or the velocity of the physics body if simulating physics.
	 * @return Velocity of the component
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API virtual FVector GetComponentVelocity() const;

	/** Returns true if this component is visible in the current context */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API virtual bool IsVisible() const;

#if WITH_EDITOR
	/**
	 * Returns full material property path and UObject owner property object
	 * Path examples:
	 * Material property path with array element and inner struct Materials[0].InnerStruct.Material
	 * Material property path with array element Materials[0]
	 * Simple material property path Materials
	 * 
	 * @param ElementIndex		- The element to access the material of.
	 * @param OutOwner			- Property UObject owner.
	 * @param OutPropertyPath	- Full material property path.
	 * @param OutProperty		- Material Property.
	 * @return true if that was successfully resolved and component has material
	 */
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty);
#endif // WITH_EDITOR

protected:
	/**
	 * Overridable internal function to respond to changes in the visibility of the component.
	 */
	ENGINE_API virtual void OnVisibilityChanged();

	/**
	* Overridable internal function to respond to changes in the hidden in game value of the component.
	*/
	ENGINE_API virtual void OnHiddenInGameChanged();

private:
	/** 
	 * Enum that dictates what propagation policy to follow when calling SetVisibility or SetHiddenInGame recursively 
	 */
	enum class EVisibilityPropagation : uint8
	{
		/** Only change the visibility if needed */
		NoPropagation, 

		/** If the visibility changed, mark all attached component's render states as dirty */
		DirtyOnly,

		/** Call function recursively on attached components and also mark their render state as dirty */
		Propagate
	};

	/**
	 * Internal function to set visibility of the component. Enum controls propagation rules.
	 */
	ENGINE_API void SetVisibility(bool bNewVisibility, EVisibilityPropagation PropagateToChildren);

	/**
	* Internal function to set hidden in game for the component. Enum controls propagation rules.
	*/
	ENGINE_API void SetHiddenInGame(bool bNewHiddenInGame, EVisibilityPropagation PropagateToChildren);

	/** Appends all descendants (recursively) of this scene component to the list of Children.  NOTE: It does NOT clear the list first. */
	ENGINE_API void AppendDescendants(TArray<USceneComponent*>& Children) const;

public:
	/** 
	 * Set visibility of the component, if during game use this to turn on/off
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void SetVisibility(bool bNewVisibility, bool bPropagateToChildren=false)
	{
		SetVisibility(bNewVisibility, bPropagateToChildren ? EVisibilityPropagation::Propagate : EVisibilityPropagation::DirtyOnly);
	}

	/** 
	 * Toggle visibility of the component
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void ToggleVisibility(bool bPropagateToChildren = false)
	{
		SetVisibility(!GetVisibleFlag(), bPropagateToChildren);
	}

	/** Changes the value of bHiddenInGame, if false this will disable Visibility during gameplay */
	UFUNCTION(BlueprintCallable, Category="Development")
	void SetHiddenInGame(bool NewHidden, bool bPropagateToChildren=false)
	{
		SetHiddenInGame(NewHidden, bPropagateToChildren ? EVisibilityPropagation::Propagate : EVisibilityPropagation::DirtyOnly);
	}

#if WITH_EDITOR
	UE_DEPRECATED(5.7, "IsTemporarilyHiddenInEditor is now deprecated.")
	bool IsTemporarilyHiddenInEditor(const bool bIncludeParent) const
	{
		return bHiddenEdTemporary;
	}
	/**
 	 * Explicitly sets whether or not this component is hidden in the editor for the duration of the current editor session
 	 * @param bIsHidden	True if the component is hidden
 	 */
	ENGINE_API void SetIsTemporarilyHiddenInEditor(const bool bInIsHidden);
#endif // WITH_EDITOR

	//~ Begin ActorComponent Interface
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual void OnPreRegister() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void EndPlay(EEndPlayReason::Type Reason) override;
	virtual bool ShouldCreateRenderState() const override { return true; }
	virtual void UpdateComponentToWorld(EUpdateTransformFlags UpdateTransformFlags = EUpdateTransformFlags::None, ETeleportType Teleport = ETeleportType::None) override final
	{
		UpdateComponentToWorldWithParent(GetAttachParent(), GetAttachSocketName(), UpdateTransformFlags, RelativeRotationCache.RotatorToQuat(GetRelativeRotation()), Teleport);
	}
	ENGINE_API virtual void DestroyComponent(bool bPromoteChildren = false) override;
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
#if WITH_EDITOR
	ENGINE_API virtual FBox GetStreamingBounds() const override;
#endif // WITH_EDITOR

	//~ End ActorComponent Interface

	//~ Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void PreNetReceive() override;
	ENGINE_API virtual void PostNetReceive() override;
	ENGINE_API virtual void PostRepNotifies() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif

#if WITH_EDITOR
	ENGINE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
	ENGINE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* Property) const override;
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif
	//~ End UObject Interface

protected:
	/**
	 * Internal helper, for use from MoveComponent().  Special codepath since the normal setters call MoveComponent.
	 * @return: true if location or rotation was changed.
	 */
	ENGINE_API bool InternalSetWorldLocationAndRotation(FVector NewLocation, const FQuat& NewQuat, bool bNoPhysics = false, ETeleportType Teleport = ETeleportType::None);

	/** Native callback when this component is moved */
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None);

	/** Check if mobility is set to non-static. If it's static we trigger a PIE warning and return true*/
	ENGINE_API bool CheckStaticMobilityAndWarn(const FText& ActionText) const;

	/** Internal helper for UpdateOverlaps */
	ENGINE_API virtual bool UpdateOverlapsImpl(const TOverlapArrayView* PendingOverlaps = nullptr, bool bDoNotifies = true, const TOverlapArrayView* OverlapsAtEndLocation = nullptr);

private:
	ENGINE_API void PropagateTransformUpdate(bool bTransformChanged, EUpdateTransformFlags UpdateTransformFlags = EUpdateTransformFlags::None, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void UpdateComponentToWorldWithParent(USceneComponent* Parent, FName SocketName, EUpdateTransformFlags UpdateTransformFlags, const FQuat& RelativeRotationQuat, ETeleportType Teleport = ETeleportType::None);

public:

	/** Queries world and updates overlap tracking state for this component */
	ENGINE_API bool UpdateOverlaps(const TOverlapArrayView* PendingOverlaps = nullptr, bool bDoNotifies = true, const TOverlapArrayView* OverlapsAtEndLocation = nullptr);

	/**
	 * Tries to move the component by a movement vector (Delta) and sets rotation to NewRotation.
	 * Assumes that the component's current location is valid and that the component does fit in its current Location.
	 * Dispatches blocking hit notifications (if bSweep is true), and calls UpdateOverlaps() after movement to update overlap state.
	 *
	 * @note This simply calls the virtual MoveComponentImpl() which can be overridden to implement custom behavior.
	 * @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 * @param Delta			The desired location change in world space.
	 * @param NewRotation	The new desired rotation in world space.
	 * @param bSweep		Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *						Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param Teleport		Whether we teleport the physics state (if physics collision is enabled for this object).
	 *						If TeleportPhysics, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *						If None, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *						If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 * @param Hit			Optional output describing the blocking hit that stopped the move, if any.
	 * @param MoveFlags		Flags controlling behavior of the move. @see EMoveComponentFlags
	 * @param Teleport      Determines whether to teleport the physics body or not. Teleporting will maintain constant velocity and avoid collisions along the path
	 * @return				True if some movement occurred, false if no movement occurred.
	 */
	bool MoveComponent( const FVector& Delta, const FQuat& NewRotation,    bool bSweep, FHitResult* Hit=NULL, EMoveComponentFlags MoveFlags = MOVECOMP_NoFlags, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API bool MoveComponent( const FVector& Delta, const FRotator& NewRotation, bool bSweep, FHitResult* Hit=NULL, EMoveComponentFlags MoveFlags = MOVECOMP_NoFlags, ETeleportType Teleport = ETeleportType::None);

protected:
	/** Override this method for custom behavior for MoveComponent */
	ENGINE_API virtual bool MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* Hit = NULL, EMoveComponentFlags MoveFlags = MOVECOMP_NoFlags, ETeleportType Teleport = ETeleportType::None);

	ENGINE_API bool IsDeferringMovementUpdates(const FScopedMovementUpdate& ScopedUpdate) const;

	ENGINE_API bool HasGroupedMovementUpdate(const FScopedMovementUpdate& ScopedUpdate) const;

public:
	/** Call UpdateComponentToWorld if bComponentToWorldUpdated is false. */
	void ConditionalUpdateComponentToWorld();

	/** Returns true if movement is currently within the scope of an FScopedMovementUpdate. */
	bool IsDeferringMovementUpdates() const;

	/** Returns true if this component is part of a grouped movement update that will apply later */
	bool HasGroupedMovementUpdates() const;

	/** Called when AttachParent changes, to allow the scene to update its attachment state. */
	virtual void OnAttachmentChanged() {}

	/** Return location of the component, in world space */
	inline FVector GetComponentLocation() const
	{
		return GetComponentTransform().GetLocation();
	}

	/** Return rotation of the component, in world space */
	inline FRotator GetComponentRotation() const
	{
		return WorldRotationCache.NormalizedQuatToRotator(GetComponentTransform().GetRotation());
	}

	/** Return rotation quaternion of the component, in world space */
	inline FQuat GetComponentQuat() const
	{
		return GetComponentTransform().GetRotation();
	}

	/** Return scale of the component, in world space */
	inline FVector GetComponentScale() const
	{
		return GetComponentTransform().GetScale3D();
	}

	/** Sets the cached component to world directly. This should be used very rarely. */
	inline void SetComponentToWorld(const FTransform& NewComponentToWorld)
	{
		bComponentToWorldUpdated = true;
		ComponentToWorld = NewComponentToWorld;
	}

	/** 
	 * Get the current component-to-world transform for this component 
	 * TODO: probably deprecate this in favor of GetComponentTransform
	 */
	inline const FTransform& GetComponentToWorld() const 
	{ 
		return ComponentToWorld;
	}

	/** Get the current component-to-world transform for this component */
	inline const FTransform& GetComponentTransform() const
	{
		return ComponentToWorld;
	}

	/** Update transforms of any components attached to this one. */
	ENGINE_API void UpdateChildTransforms(EUpdateTransformFlags UpdateTransformFlags = EUpdateTransformFlags::None, ETeleportType Teleport = ETeleportType::None);

	/** Calculate the bounds of the component. Default behavior is a bounding box/sphere of zero size. */
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const;

	/** Calculate the local bounds of the component. Default behavior is calling CalcBounds with an identity transform. */
	virtual FBoxSphereBounds CalcLocalBounds() const 
	{ 
		return GetLocalBounds();
	}

	/**
	 * Calculate the axis-aligned bounding cylinder of the component (radius in X-Y, half-height along Z axis).
	 * Default behavior is just a cylinder around the box of the cached BoxSphereBounds.
	 */
	ENGINE_API virtual void CalcBoundingCylinder(float& CylinderRadius, float& CylinderHalfHeight) const;

	/** Update the Bounds of the component.*/
	ENGINE_API virtual void UpdateBounds();

	/** If true, bounds should be used when placing component/actor in level. Does not affect spawning. */
	virtual bool ShouldCollideWhenPlacing() const
	{
		return false;
	}

	/** 
	 * Updates the PhysicsVolume of this SceneComponent, if bShouldUpdatePhysicsVolume is true.
	 * 
	 * @param bTriggerNotifiers		if true, send zone/volume change events
	 */
	ENGINE_API virtual void UpdatePhysicsVolume( bool bTriggerNotifiers );

	/**
	 * Replace current PhysicsVolume to input NewVolume
	 *
	 * @param NewVolume				NewVolume to replace
	 * @param bTriggerNotifiers		if true, send zone/volume change events
	 */
	ENGINE_API void SetPhysicsVolume( APhysicsVolume * NewVolume,  bool bTriggerNotifiers );

	/** 
	 * Get the PhysicsVolume overlapping this component.
	 */
	UFUNCTION(BlueprintCallable, Category=PhysicsVolume, meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API APhysicsVolume* GetPhysicsVolume() const;

	/** Return const reference to CollsionResponseContainer */
	ENGINE_API virtual const FCollisionResponseContainer& GetCollisionResponseToChannels() const;

	/** Return true if visible in editor **/
	ENGINE_API virtual bool IsVisibleInEditor() const;

	/** return true if it should render **/
	ENGINE_API bool ShouldRender() const;

	/** return true if it can ever render **/
	ENGINE_API bool CanEverRender() const;

	/** 
	 *  Looking at various values of the component, determines if this
	 *  component should be added to the scene
	 * @return true if the component is visible and should be added to the scene, false otherwise
	 */
	ENGINE_API bool ShouldComponentAddToScene() const;

#if WITH_EDITOR
	/** Called when this component is moved in the editor */
	ENGINE_API virtual void PostEditComponentMove(bool bFinished);

	/** Returns number of lighting interactions that need to be recalculated */
	ENGINE_API virtual const int32 GetNumUncachedStaticLightingInteractions() const;

	/** Called to update any visuals needed for a feature level change */
	virtual void PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel) {}
#endif // WITH_EDITOR

protected:

	/** Calculate the new ComponentToWorld transform for this component.
		Parent is optional and can be used for computing ComponentToWorld based on arbitrary USceneComponent.
		If Parent is not passed in we use the component's AttachParent*/
	inline FTransform CalcNewComponentToWorld(const FTransform& NewRelativeTransform, const USceneComponent* Parent = nullptr, FName SocketName = NAME_None) const
	{
		SocketName = Parent ? SocketName : GetAttachSocketName();
		Parent = Parent ? Parent : GetAttachParent();
		if (Parent)
		{
			const bool bGeneral = IsUsingAbsoluteLocation() || IsUsingAbsoluteRotation() || IsUsingAbsoluteScale();
			if (!bGeneral)
			{
				return NewRelativeTransform * Parent->GetSocketTransform(SocketName);
			}
			
			return CalcNewComponentToWorld_GeneralCase(NewRelativeTransform, Parent, SocketName);
		}
		else
		{
			return NewRelativeTransform;
		}
	}

	/** Utility function to handle calculating transform with a parent */
	ENGINE_API FTransform CalcNewComponentToWorld_GeneralCase(const FTransform& NewRelativeTransform, const USceneComponent* Parent, FName SocketName) const;

public:
	/**
	 * Set the location and rotation of the component relative to its parent
	 * @param NewLocation		New location of the component relative to its parent.
	 * @param NewRotation		New rotation of the component relative to its parent.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Relative Location And Rotation", ScriptName="SetRelativeLocationAndRotation"))
	ENGINE_API void K2_SetRelativeLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetRelativeLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void SetRelativeLocationAndRotation(FVector NewLocation, const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Set which parts of the relative transform should be relative to parent, and which should be relative to world */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void SetAbsolute(bool bNewAbsoluteLocation = false, bool bNewAbsoluteRotation = false, bool bNewAbsoluteScale = false);

	/**
	 * Set the relative location and rotation of the component to put it at the supplied pose in world space.
	 * @param NewLocation		New location in world space for the component.
	 * @param NewRotation		New rotation in world space for the component.
	 * @param SweepHitResult	Hit result from any impact if sweep is true.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set World Location And Rotation", ScriptName="SetWorldLocationAndRotation"))
	ENGINE_API void K2_SetWorldLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetWorldLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Set the relative location and FQuat rotation of the component to put it at the supplied pose in world space. */
	ENGINE_API void SetWorldLocationAndRotation(FVector NewLocation, const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Special version of SetWorldLocationAndRotation that does not affect physics. */
	ENGINE_API void SetWorldLocationAndRotationNoPhysics(const FVector& NewLocation, const FRotator& NewRotation);

	/** Is this component considered 'world' geometry, by default checks if this uses the WorldStatic collision channel */
	ENGINE_API virtual bool IsWorldGeometry() const;

	/** Returns the form of collision for this component */
	ENGINE_API virtual ECollisionEnabled::Type GetCollisionEnabled() const;

	/** Utility to see if there is any form of collision (query or physics) enabled on this component. */
	inline bool IsCollisionEnabled() const
	{
		return GetCollisionEnabled() != ECollisionEnabled::NoCollision;
	}

	/** Utility to see if there is any query collision enabled on this component. */
	inline bool IsQueryCollisionEnabled() const
	{
		return CollisionEnabledHasQuery(GetCollisionEnabled());
	}

	/** Utility to see if there is any physics collision enabled on this component. */
	inline bool IsPhysicsCollisionEnabled() const
	{
		return CollisionEnabledHasPhysics(GetCollisionEnabled());
	}

	/** Returns the response that this component has to a specific collision channel. */
	ENGINE_API virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const;

	/** Returns the channel that this component belongs to when it moves. */
	ENGINE_API virtual ECollisionChannel GetCollisionObjectType() const;

	/** Compares the CollisionObjectType of each component against the Response of the other, to see what kind of response we should generate */
	ENGINE_API ECollisionResponse GetCollisionResponseToComponent(USceneComponent* OtherComponent) const;

	/** Set how often this component is allowed to move during runtime. Causes a component re-register if the component is already registered */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API virtual void SetMobility(EComponentMobility::Type NewMobility);

	/** Walks up the attachment chain from this SceneComponent and returns the SceneComponent at the top. If AttachParent is NULL, returns this. */
	ENGINE_API USceneComponent* GetAttachmentRoot() const;
	
	/** Walks up the attachment chain from this SceneComponent and returns the top-level actor it's attached to.  Returns Owner if unattached. */
	ENGINE_API AActor* GetAttachmentRootActor() const;

	/** Returns the ActorPosition for use by rendering. This comes from walking the attachment chain to the top-level actor. */
	ENGINE_API FVector GetActorPositionForRenderer() const;

	/** Gets the owner of the attach parent */
	ENGINE_API AActor* GetAttachParentActor() const;

	/** Walks up the attachment chain to see if this component is attached to the supplied component. If TestComp == this, returns false.*/
	ENGINE_API bool IsAttachedTo(const USceneComponent* TestComp) const;

	/**
	 * Find the world-space location and rotation of the given named socket.
	 * If the socket is not found, then it returns the component's location and rotation in world space.
	 * @param InSocketName the name of the socket to find
	 * @param OutLocation (out) set to the world space location of the socket
	 * @param OutRotation (out) set to the world space rotation of the socket
	 * @return whether or not the socket was found
	 */
	ENGINE_API void GetSocketWorldLocationAndRotation(FName InSocketName, FVector& OutLocation, FRotator& OutRotation) const;
	ENGINE_API void GetSocketWorldLocationAndRotation(FName InSocketName, FVector& OutLocation, FQuat& OutRotation) const;

	/**
	 * Called to see if it's possible to attach another scene component as a child.
	 * Note: This can be called on template component as well!
	 */
	virtual bool CanAttachAsChild(const USceneComponent* ChildComponent, FName SocketName) const { return true; }

	/** Get the extent used when placing this component in the editor, used for 'pulling back' hit. */
	ENGINE_API virtual FBoxSphereBounds GetPlacementExtent() const;

private:
	friend class AActor;

	void NotifyIsRootComponentChanged(bool bIsRootComponent) { IsRootComponentChanged.Broadcast(this, bIsRootComponent); }

protected:
	/**
	 * Called after a child scene component is attached to this component.
	 * Note: Do not change the attachment state of the child during this call.
	 */
	virtual void OnChildAttached(USceneComponent* ChildComponent) {}

	/**
	 * Called after a child scene component is detached from this component.
	 * Note: Do not change the attachment state of the child during this call.
	 */
	virtual void OnChildDetached(USceneComponent* ChildComponent) {}

	/** Called after changing transform, tries to update navigation octree for this component */
	ENGINE_API void UpdateNavigationData();

	/** Called after changing transform, tries to update navigation octree for owner */
	ENGINE_API void PostUpdateNavigationData();

	/**
	 * Determine if dynamic data is allowed to be changed.
	 * 
	 * @param bIgnoreStationary Whether or not to ignore stationary mobility when checking. Default is true (i.e. - check for static mobility only).
	 * @return Whether or not dynamic data is allowed to be changed.
	 */
	inline bool AreDynamicDataChangesAllowed(bool bIgnoreStationary = true) const
	{
		return (IsOwnerRunningUserConstructionScript()) || !(IsRegistered() && (Mobility == EComponentMobility::Static || (!bIgnoreStationary && Mobility == EComponentMobility::Stationary)));
	}

public:
	/** Determines whether or not the component can have its mobility set to static */
	virtual const bool CanHaveStaticMobility() const { return true; }

	/** Updates any visuals after the lighting has changed */
	virtual void PropagateLightingScenarioChange() {}

	/** True if our precomputed lighting is up to date */
	virtual bool IsPrecomputedLightingValid() const
	{
		return false;
	}

private:
	friend class FScopedMovementUpdate;
	friend class FScopedPreventAttachedComponentMove;
	friend struct FDirectAttachChildrenAccessor;

	//~ Begin Methods for Replicated Members.
private:

	/**
	 * Sets the value of AttachParent without causing other side effects to this instance.
	 * Other systems may leverage this to get notifications for when the value is changed.
	 */
	ENGINE_API void SetAttachParent(USceneComponent* NewAttachParent);
	
	/**
	 * Sets the value of AttachSocketName without causing other side effects to this instance.
	 * Other systems may leverage this to get notifications for when the value is changed.
	 */
	ENGINE_API void SetAttachSocketName(FName NewSocketName);
	
	/**
	 * Called when AttachChildren is modified.
	 * Other systems may leverage this to get notifications for when the value is changed.
	 */
	ENGINE_API void ModifiedAttachChildren();

public:

	/**
	* Called when client receive replication data, before replication is performed.
	* Can be overridden in derived components to make use of replication data locally.
	* Note that replication still applies when overriding this, it's not intended to replace replication.
	*/
	virtual void OnReceiveReplicatedState(const FVector X, const FQuat R, const FVector V, const FVector W) {};

	/**
	 * Gets the property name for RelativeLocation.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeLocation property so it
	 * can be made private later.
	 */
	static const FName GetRelativeLocationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, RelativeLocation);
	}
	
	/**
	 * Gets the literal value of RelativeLocation.
	 * Note, this may be an absolute location if this is a root component (not attached to anything) or
	 * when IsUsingAbsoluteLocation returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeLocation property so it
	 * can be made private later.
	 */
	FVector GetRelativeLocation() const
	{
		return RelativeLocation;
	}
	
	/**
	 * Gets a refence to RelativeLocation with the expectation that it will be modified.
	 * Note, this may be an absolute location if this is a root component (not attached to anything) or
	 * when IsUsingAbsoluteLocation returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeLocation property so it
	 * can be made private later.
	 *
	 * You should not use this method. The standard SetRelativeLocation variants should be used.
	 */
	ENGINE_API FVector& GetRelativeLocation_DirectMutable();

	/**
	 * Sets the value of RelativeLocation without causing other side effects to this instance.
	 *
	 * You should not use this method. The standard SetRelativeLocation variants should be used.
	 */
	ENGINE_API void SetRelativeLocation_Direct(const FVector NewRelativeLocation);

	/**
	 * Gets the property name for RelativeRotation.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeRotation property so it
	 * can be made private later.
	 */
	static const FName GetRelativeRotationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, RelativeRotation);
	}
	
	/**
	 * Gets the literal value of RelativeRotation.
	 * Note, this may be an absolute rotation if this is a root component (not attached to anything) or
	 * when GetAbsoluteRotation returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeRotation property so it
	 * can be made private later.
	 */
	FRotator GetRelativeRotation() const
	{
		return RelativeRotation;
	}
	
	/**
	 * Gets a refence to RelativeRotation with the expectation that it will be modified.
	 * Note, this may be an absolute rotation if this is a root component (not attached to anything) or
	 * when GetAbsoluteRotation returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeRotation property so it
	 * can be made private later.
	 *
	 * You should not use this method. The standard SetRelativeRotation variants should be used.
	 */
	ENGINE_API FRotator& GetRelativeRotation_DirectMutable();

	/**
	 * Sets the value of RelativeRotation without causing other side effects to this instance.
	 *
	 * You should not use this method. The standard SetRelativeRotation variants should be used.
	 */
	ENGINE_API void SetRelativeRotation_Direct(const FRotator NewRelativeRotation);

	/**
	 * Gets the property name for RelativeScale3D.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeScale3D property so it
	 * can be made private later.
	 */
	static const FName GetRelativeScale3DPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, RelativeScale3D);
	}
	
	/**
	 * Gets the literal value of RelativeScale3D.
	 * Note, this may be an absolute scale if this is a root component (not attached to anything) or
	 * when GetAbsoluteScale3D returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeScale3D property so it
	 * can be made private later.
	 */
	FVector GetRelativeScale3D() const
	{
		return RelativeScale3D;
	}
	
	/**
	 * Gets a refence to RelativeRotation with the expectation that it will be modified.
	 * Note, this may be an absolute scale if this is a root component (not attached to anything) or
	 * when GetAbsoluteScale3D returns true.
	 *
	 * This exists so subclasses don't need to have direct access to the RelativeScale3D property so it
	 * can be made private later.
	 *
	 * You should not use this method. The standard SetRelativeScale3D variants should be used.
	 */
	ENGINE_API FVector& GetRelativeScale3D_DirectMutable();

	/**
	 * Sets the value of RelativeScale3D without causing other side effects to this instance.
	 *
	 * You should not use this method. The standard SetRelativeScale3D variants should be used.
	 */
	ENGINE_API void SetRelativeScale3D_Direct(const FVector NewRelativeScale3D);

	/**
	 * Helper function to set the location, rotation, and scale without causing other side effects to this instance.
	 *
	 * You should not use this method. The standard SetRelativeTransform variants should be used.
	 */
	void SetRelativeTransform_Direct(const FTransform& NewRelativeTransform)
	{
		SetRelativeLocation_Direct(NewRelativeTransform.GetLocation());
		SetRelativeRotation_Direct(NewRelativeTransform.Rotator());
		SetRelativeScale3D_Direct(NewRelativeTransform.GetScale3D());
	}

	/**
	 * Gets the property name for bAbsoluteLocation.
	 *
	 * This exists so subclasses don't need to have direct access to the bAbsoluteLocation property so it
	 * can be made private later.
	 */
	static const FName GetAbsoluteLocationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, bAbsoluteLocation);
	}

	/**
	 * Gets the literal value of bAbsoluteLocation.
	 *
	 * This exists so subclasses don't need to have direct access to the bAbsoluteLocation property so it
	 * can be made private later.
	 */
	bool IsUsingAbsoluteLocation() const
	{
		return bAbsoluteLocation;
	}
	
	/** Sets the value of bAbsoluteLocation without causing other side effects to this instance. */
	ENGINE_API void SetUsingAbsoluteLocation(const bool bInAbsoluteLocation);

	/**
	 * Gets the property name for bAbsoluteRotation.
	 *
	 * This exists so subclasses don't need to have direct access to the bAbsoluteRotation property so it
	 * can be made private later.
	 */
	static const FName GetAbsoluteRotationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, bAbsoluteRotation);
	}

	/**
	 * Gets the literal value of bAbsoluteRotation.
	 *
	 * This exists so subclasses don't need to have direct access to the bAbsoluteRotation property so it
	 * can be made private later.
	 */
	bool IsUsingAbsoluteRotation() const
	{
		return bAbsoluteRotation;
	}
	
	/** Sets the value of bAbsoluteRotation without causing other side effects to this instance. */
	ENGINE_API void SetUsingAbsoluteRotation(const bool bInAbsoluteRotation);

	/**
	 * Gets the property name for bAbsoluteScale.
	 * This exists so subclasses don't need to have direct access to the bAbsoluteScale property so it
	 * can be made private later.
	 */
	static const FName GetAbsoluteScalePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, bAbsoluteScale);
	}

	/**
	 * Gets the literal value of bAbsoluteScale.
	 *
	 * This exists so subclasses don't need to have direct access to the bReplicates property so it
	 * can be made private later.
	 '*/
	bool IsUsingAbsoluteScale() const
	{
		return bAbsoluteScale;
	}
	
	/** Sets the value of bAbsoluteScale without causing other side effects to this instance. */
	ENGINE_API void SetUsingAbsoluteScale(const bool bInAbsoluteRotation);

	/**
	 * Gets the property name for bVisible.
	 * This exists so subclasses don't need to have direct access to the bVisible property so it
	 * can be made private later.
	 */
	static const FName GetVisiblePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneComponent, bVisible);
	}

	/**
	 * Gets the literal value of bVisible.
	 *
	 * This exists so subclasses don't need to have direct access to the bVisible property so it
	 * can be made private later.
	 *
	 * IsVisible and IsVisibleInEditor are preferred in most cases because they respect virtual behavior.
	 */
	bool GetVisibleFlag() const
	{
		return bVisible;
	}
	
	/**
	 * Sets the value of bVisible without causing other side effects to this instance.
	 *
	 * ToggleVisible and SetVisibility are preferred in most cases because they respect virtual behavior and side effects.
	 */
	ENGINE_API void SetVisibleFlag(const bool bInVisible);
	
	//~ End Methods for Replicated Members.

	/** 
	 * Return the Mobility of this component.
	 */
	EComponentMobility::Type GetMobility() const
	{
		return Mobility;
	}
};

/** 
  * Struct to allow direct access to the AttachChildren array for a handful of cases that will require more work than can be done  
  * immediately to fix up in light of the privatization steps
  */
struct FDirectAttachChildrenAccessor
{
private:
	static TArray<TObjectPtr<USceneComponent>>& Get(USceneComponent* Component)
	{ 
		return Component->AttachChildren;
	}

	friend class UChildActorComponent;
	friend class FBlueprintThumbnailScene;
	friend class FClassThumbnailScene;
	friend class FComponentEditorUtils;
	friend class FBlueprintCompileReinstancer;
	friend struct FResetSceneComponentAfterCopy;
};


//////////////////////////////////////////////////////////////////////////
// USceneComponent inlines

inline const TArray<TObjectPtr<USceneComponent>>& USceneComponent::GetAttachChildren() const
{
	return AttachChildren;
}

inline USceneComponent* USceneComponent::GetAttachParent() const
{
	return AttachParent;
}

inline FName USceneComponent::GetAttachSocketName() const
{
	return AttachSocketName;
}

inline void USceneComponent::ConditionalUpdateComponentToWorld()
{
	if (!bComponentToWorldUpdated)
	{
		UpdateComponentToWorld();
	}
}

inline bool USceneComponent::MoveComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* Hit, EMoveComponentFlags MoveFlags, ETeleportType Teleport)
{
	return MoveComponentImpl(Delta, NewRotation, bSweep, Hit, MoveFlags, Teleport);
}

inline void USceneComponent::SetRelativeLocation(FVector NewLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	SetRelativeLocationAndRotation(NewLocation, RelativeRotationCache.RotatorToQuat(GetRelativeRotation()), bSweep, OutSweepHitResult, Teleport);
}

inline void USceneComponent::SetRelativeRotation(const FQuat& NewRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	SetRelativeLocationAndRotation(GetRelativeLocation(), NewRotation, bSweep, OutSweepHitResult, Teleport);
}

inline void USceneComponent::AddRelativeLocation(FVector DeltaLocation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	SetRelativeLocationAndRotation(GetRelativeLocation() + DeltaLocation, RelativeRotationCache.RotatorToQuat(GetRelativeRotation()), bSweep, OutSweepHitResult, Teleport);
}

inline void USceneComponent::AddRelativeRotation(FRotator DeltaRotation, bool bSweep, FHitResult* OutSweepHitResult, ETeleportType Teleport)
{
	SetRelativeRotation(GetRelativeRotation() + DeltaRotation, bSweep, OutSweepHitResult, Teleport);
}

//////////////////////////////////////////////////////////////////////////

/** 
 *  Component instance cached data base class for scene components. 
 *  Stores a list of instance components attached to the 
 */
USTRUCT()
struct FSceneComponentInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()

	ENGINE_API FSceneComponentInstanceData();
	ENGINE_API FSceneComponentInstanceData(const USceneComponent* SourceComponent);
			
	ENGINE_API virtual ~FSceneComponentInstanceData();

	ENGINE_API virtual bool ContainsData() const override;

	ENGINE_API virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;
	ENGINE_API virtual void FindAndReplaceInstances(const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	UPROPERTY() 
	TMap<TObjectPtr<USceneComponent>, FTransform> AttachedInstanceComponents;
};


//////////////////////////////////////////////////////////////////////////

/**
 * Utility for temporarily changing the behavior of a SceneComponent to use absolute transforms, and then restore it to the behavior at the start of the scope.
 */
class FScopedPreventAttachedComponentMove : private FNoncopyable
{
public:

	/**
	 * Init scoped behavior for a given Component.
	 * Note that null is perfectly acceptable here (does nothing) as a simple way to toggle behavior at runtime without weird conditional compilation.
	 */
	ENGINE_API FScopedPreventAttachedComponentMove(USceneComponent* Component);
	ENGINE_API ~FScopedPreventAttachedComponentMove();

private:

	USceneComponent* Owner;
	uint32 bSavedAbsoluteLocation:1;
	uint32 bSavedAbsoluteRotation:1;
	uint32 bSavedAbsoluteScale:1;
	uint32 bSavedNonAbsoluteComponent:1; // Whether any of the saved location/rotation/scale flags were false (or equivalently: not all were true).

	// This class can only be created on the stack, otherwise the ordering constraints
	// of the constructor and destructor between encapsulated scopes could be violated.
	void* operator new (size_t) = delete;
	void* operator new[] (size_t) = delete;
	void operator delete (void *) = delete;
	void operator delete[] (void*) = delete;
};

inline FScopedMovementUpdate* USceneComponent::GetCurrentScopedMovement() const
{
	if (ScopedMovementStack.Num() > 0)
	{
		return ScopedMovementStack.Last();
	}
	return nullptr;
}

inline bool USceneComponent::IsDeferringMovementUpdates() const
{
	if (ScopedMovementStack.Num() > 0)
	{
		checkSlow(IsDeferringMovementUpdates(*ScopedMovementStack.Last()));
		return true;
	}
	return false;
}

inline bool USceneComponent::HasGroupedMovementUpdates() const
{
	if (ScopedMovementStack.Num() > 0)
	{
		checkSlow(HasGroupedMovementUpdate(*ScopedMovementStack.Last()));
		return true;
	}
	return false;
}

inline bool USceneComponent::GetShouldUpdatePhysicsVolume() const
{
	return bShouldUpdatePhysicsVolume;
}
