// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Components/PrimitiveComponent.h"
#include "ShapeComponent.generated.h"

class FPrimitiveSceneProxy;
struct FNavigableGeometryExport;
struct FNavigationRelevantData;

namespace physx
{
	class PxShape;
}

/**
 * ShapeComponent is a PrimitiveComponent that is represented by a simple geometrical shape (sphere, capsule, box, etc).
 */
UCLASS(abstract, hidecategories=(Object,LOD,Lighting,TextureStreaming,Activation,"Components|Activation"), editinlinenew, meta=(BlueprintSpawnableComponent), showcategories=(Mobility), MinimalAPI)
class UShapeComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Description of collision */
	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<class UBodySetup> ShapeBodySetup;

#if WITH_EDITORONLY_DATA
	/** Navigation area type (empty / none has no effect on underlying area type).
	 *  Deprecated! Use AreaClassOverride / bUseSystemDefaultObstacleAreaClass instead. 
	 *  NOTE Adding _DEPRECATED to this variable can causes known bugs when patching it up.
	 */
	UE_DEPRECATED(5.0, "AreaClass is deprecated, use AreaClassOverride / bUseSystemDefaultObstacleAreaClass instead!.")
	UPROPERTY()
	TSubclassOf<class UNavAreaBase> AreaClass;
#endif // WITH_EDITORONLY_DATA

	/** Color used to draw the shape. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Shape)
	FColor ShapeColor;

	/** Only show this component if the actor is selected */
	UPROPERTY()
	uint8 bDrawOnlyIfSelected:1;

	/** If true it allows Collision when placing even if collision is not enabled*/
	UPROPERTY()
	uint8 bShouldCollideWhenPlacing:1;

	/** If set, shape will be exported for navigation as dynamic modifier instead of using regular collision data */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint8 bDynamicObstacle : 1;

protected:
	/** Uses FNavigationSystem::GetDefaultObstacleArea() by default instead of AreaClassOverride, bDynamicObstacle must be true to use this.  */
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (EditCondition = "bDynamicObstacle"))
	uint8 bUseSystemDefaultObstacleAreaClass : 1;

	/** If the body setup can be shared (i.e. there have been no alterations compared to the CDO)*/
	uint8 bUseArchetypeBodySetup : 1;

	/** Navigation area type override, null / none = no change to nav mesh.
	 *  bDynamicObstacle must be true and bUseSystemDefaultAreaClass false to use this.
	 */
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (EditCondition = "bDynamicObstacle && !bUseSystemDefaultObstacleAreaClass"))
	TSubclassOf<class UNavAreaBase> AreaClassOverride;

	/** Used to control the line thickness when rendering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Shape)
	float LineThickness;
	
	/** Checks if a shared body setup is available (and if we're eligible for it). If successful you must still check for staleness */
	template<typename ComponentType>
	bool PrepareSharedBodySetup()
	{
		bool bSuccess = bUseArchetypeBodySetup;
		if (bUseArchetypeBodySetup && ShapeBodySetup == nullptr)
		{
			ShapeBodySetup = CastChecked<ComponentType>(GetArchetype())->GetBodySetup();
			bSuccess = ShapeBodySetup != nullptr;
		}

		return bSuccess;
	}

	template <typename ShapeElemType> void AddShapeToGeomArray();
	template <typename ShapeElemType> void SetShapeToNewGeom(const FPhysicsShapeHandle& Shape);
	template <typename ShapeElemType> void CreateShapeBodySetupIfNeeded();

public:
	// Set the LineThickness
	UFUNCTION(BlueprintCallable, Category=Shape)
	ENGINE_API void SetLineThickness(float Thickness);

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual class UBodySetup* GetBodySetup() override;
	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	ENGINE_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin INavRelevantInterface Interface
	ENGINE_API virtual bool IsNavigationRelevant() const override;
	//~ End INavRelevantInterface Interface

	//~ Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual bool ShouldCollideWhenPlacing() const override
	{
		return bShouldCollideWhenPlacing || IsCollisionEnabled();
	}
	//~ End USceneComponent Interface

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	virtual bool GetIgnoreBoundsForEditorFocus() const override { return true; }
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	/** Update the body setup parameters based on shape information*/
	ENGINE_API virtual void UpdateBodySetup();

	ENGINE_API TSubclassOf<class UNavAreaBase> GetDesiredAreaClass() const;
	ENGINE_API void SetAreaClassOverride(TSubclassOf<class UNavAreaBase> InAreaClassOverride);
	ENGINE_API void SetUseSystemDefaultObstacleAreaClass();
};

enum class EShapeBodySetupHelper
{
	InvalidateSharingIfStale,
	UpdateBodySetup
};
