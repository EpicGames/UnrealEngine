// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Components/SceneComponent.h"
#include "EngineDefines.h"
#include "CollisionQueryParams.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimCurveTypes.h"
#include "Components/SkinnedMeshComponent.h"
#include "ClothSimData.h"
#include "SingleAnimationPlayData.h"
#include "Animation/PoseSnapshot.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Animation/AttributesRuntime.h"
#include "Chaos/Serialization/SerializedDataBuffer.h"
#if WITH_ENGINE
#include "Engine/PoseWatchRenderData.h"
#endif
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "ClothingSimulationInterface.h"
#include "ClothingSimulationFactory.h"
#endif

#include "SkeletalMeshComponent.generated.h"


class Error;
class FPrimitiveDrawInterface;
class FCanvas;
class FSceneView;
class UAnimInstance;
class UPhysicalMaterial;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FClothCollisionData;
struct FClothCollisionSource;
struct FClothingSimulationInstance;
struct FConstraintInstance;
struct FConstraintProfileProperties;
struct FNavigableGeometryExport;
struct FCompactPose;

namespace UE::UAF
{
	class FDataRegistry;	// Temporary
}

enum class EClothingTeleportMode : uint8;

#ifndef WITH_CLOTH_COLLISION_DETECTION
#define WITH_CLOTH_COLLISION_DETECTION 1
#endif

DECLARE_MULTICAST_DELEGATE(FOnSkelMeshPhysicsCreatedMultiCast);
typedef FOnSkelMeshPhysicsCreatedMultiCast::FDelegate FOnSkelMeshPhysicsCreated;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAnimInitialized);

DECLARE_MULTICAST_DELEGATE(FOnSkelMeshTeleportedMultiCast);
typedef FOnSkelMeshTeleportedMultiCast::FDelegate FOnSkelMeshTeleported;

class UE_DEPRECATED(5.5, "use FOnBoneTransformsFinalizedMultiCast instead (see SkinnedMeshComponent.h).") FOnBoneTransformsFinalized;
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBoneTransformsFinalized);  // Deprecated, use FOnBoneTransformsFinalizedMultiCast instead

DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnLODRequiredBonesUpdateMulticast, USkeletalMeshComponent*, int32, const TArray<FBoneIndexType>&);
typedef FOnLODRequiredBonesUpdateMulticast::FDelegate FOnLODRequiredBonesUpdate;

/** Method used when retrieving a attribute value*/
UENUM()
enum class ECustomBoneAttributeLookup : uint8
{
	/** Only look for the attribute using the provided bone (name) */
	BoneOnly,
	/** Look for the attribute using the provided bone (name) and its direct parent bone */
	ImmediateParent,
	/** Look for the attribute using the provided bone (name) and its direct bone parent hierarchy up and until the root bone */
	ParentHierarchy
};

struct FAnimationEvaluationContext
{
	// The anim instance we are evaluating
	UAnimInstance* AnimInstance;

	// The post process instance we are evaluating
	UAnimInstance* PostProcessAnimInstance;

	// The SkeletalMesh we are evaluating for
	USkeletalMesh* SkeletalMesh;

	// Evaluation data, swapped in from the component when we are running parallel eval
	TArray<FTransform> ComponentSpaceTransforms;
	TArray<FTransform> BoneSpaceTransforms;
	TArray<FTransform> CachedComponentSpaceTransforms;
	TArray<FTransform> CachedBoneSpaceTransforms;
	FVector RootBoneTranslation;

	// Are we performing interpolation this tick
	bool bDoInterpolation;

	// Are we evaluating this tick
	bool bDoEvaluation;

	// Are we storing data in cache bones this tick
	bool bDuplicateToCacheBones;

	// duplicate the cache curves
	bool bDuplicateToCacheCurve;

	// duplicate the cached attributes
	bool bDuplicateToCachedAttributes;

	// Force reference pose
	bool bForceRefPose;

	// Curve data, swapped in from the component when we are running parallel eval
	FBlendedHeapCurve	Curve;
	FBlendedHeapCurve	CachedCurve;

	// attribute data, swapped in from the component when we are running parallel eval
	UE::Anim::FMeshAttributeContainer CustomAttributes;
	UE::Anim::FMeshAttributeContainer CachedCustomAttributes;

	FAnimationEvaluationContext()
	{
		Clear();
	}

	void Copy(const FAnimationEvaluationContext& Other)
	{
		AnimInstance = Other.AnimInstance;
		PostProcessAnimInstance = Other.PostProcessAnimInstance;
		SkeletalMesh = Other.SkeletalMesh;
		ComponentSpaceTransforms.Reset();
		ComponentSpaceTransforms.Append(Other.ComponentSpaceTransforms);
		BoneSpaceTransforms.Reset();
		BoneSpaceTransforms.Append(Other.BoneSpaceTransforms);
		CachedComponentSpaceTransforms.Reset();
		CachedComponentSpaceTransforms.Append(Other.CachedComponentSpaceTransforms);
		CachedBoneSpaceTransforms.Reset();
		CachedBoneSpaceTransforms.Append(Other.CachedBoneSpaceTransforms);
		RootBoneTranslation = Other.RootBoneTranslation;
		Curve.CopyFrom(Other.Curve);
		CachedCurve.CopyFrom(Other.CachedCurve);
		bDoInterpolation = Other.bDoInterpolation;
		bDoEvaluation = Other.bDoEvaluation;
		bDuplicateToCacheBones = Other.bDuplicateToCacheBones;
		bDuplicateToCacheCurve = Other.bDuplicateToCacheCurve;
		bDuplicateToCachedAttributes = Other.bDuplicateToCachedAttributes;
		bForceRefPose = Other.bForceRefPose;

		CustomAttributes.CopyFrom(Other.CustomAttributes);
		CachedCustomAttributes.CopyFrom(Other.CachedCustomAttributes);
	}

	void Clear()
	{
		AnimInstance = nullptr;
		PostProcessAnimInstance = nullptr;
		SkeletalMesh = nullptr;
	}

};

/** This enum defines how you'd like to update bones to physics world.
	If bone is simulating, you don't have to waste time on updating bone transform from kinematic.
	But also sometimes you don't like fixed bones to be updated by animation data. */
UENUM()
namespace EKinematicBonesUpdateToPhysics
{
	enum Type : int
	{
		/** Update any bones that are not simulating. */
		SkipSimulatingBones,
		/** Skip physics update from kinematic changes. */
		SkipAllBones
	};
}

UENUM()
namespace EAnimationMode
{
	enum Type : int
	{
		AnimationBlueprint UMETA(DisplayName="Use Animation Blueprint"), 
		AnimationSingleNode UMETA(DisplayName="Use Animation Asset"), 
		// This is custom type, engine leaves AnimInstance as it is
		AnimationCustomMode UMETA(DisplayName = "Use Custom Mode"),
	};
}

UENUM()
namespace EPhysicsTransformUpdateMode
{
	enum Type : int
	{
		SimulationUpatesComponentTransform UMETA(DisplayName="Simulation Updates Component Transform"), 
		ComponentTransformIsKinematic
	};
}

/** Enum for indicating whether kinematic updates can be deferred */
enum class EAllowKinematicDeferral
{
	AllowDeferral,
	DisallowDeferral
};

/**
* Tick function that does post physics work on skeletal mesh component. This executes in EndPhysics (after physics is done)
**/
USTRUCT()
struct FSkeletalMeshComponentEndPhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	 USkeletalMeshComponent*	Target;

	/**
	* Abstract function to execute the tick.
	* @param DeltaTime - frame time to advance, in seconds.
	* @param TickType - kind of tick for this frame.
	* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created.
	* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	*/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph. */
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FSkeletalMeshComponentEndPhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FSkeletalMeshComponentEndPhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};


/**
* Tick function that prepares and simulates cloth
**/
USTRUCT()
struct FSkeletalMeshComponentClothTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	USkeletalMeshComponent*	Target;

	/**
	* Abstract function to execute the tick.
	* @param DeltaTime - frame time to advance, in seconds.
	* @param TickType - kind of tick for this frame.
	* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created.
	* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	*/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph. */
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FSkeletalMeshComponentClothTickFunction> : public TStructOpsTypeTraitsBase2<FSkeletalMeshComponentClothTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/**
* Tick function that waits on the parallel cloth simulation task in TG_LastDemotable
**/
USTRUCT()
struct FSkeletalMeshComponentClothEndTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	USkeletalMeshComponent* Target = nullptr;

	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FSkeletalMeshComponentClothEndTickFunction> : public TStructOpsTypeTraitsBase2<FSkeletalMeshComponentClothEndTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

struct FClosestPointOnPhysicsAsset
{
	/** The closest point in world space */
	FVector ClosestWorldPosition;

	/** The normal associated with the surface of the closest body */
	FVector Normal;

	/** The name of the bone associated with the closest body */
	FName BoneName;

	/** The distance of the closest point and the original world position. 0 Indicates world position is inside the closest body. */
	float Distance;

	FClosestPointOnPhysicsAsset()
		: ClosestWorldPosition(FVector::ZeroVector)
		, Normal(FVector::ZeroVector)
		, BoneName(NAME_None)
		, Distance(-1.f)
	{
	}
};

namespace Chaos
{
	struct FSerializedDataBuffer;
}

/**
 * SkeletalMeshComponent is used to create an instance of an animated SkeletalMesh asset.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Content/Types/SkeletalMeshes/
 * @see USkeletalMesh
 */
UCLASS(Blueprintable, ClassGroup=(Rendering, Common), hidecategories=(Object, "Mesh|SkeletalAsset"), config=Engine, editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class USkeletalMeshComponent : public USkinnedMeshComponent, public IInterface_CollisionDataProvider
{
	GENERATED_UCLASS_BODY()

	friend class FSkinnedMeshComponentRecreateRenderStateContext;
	friend class FParallelAnimationCompletionTask;
	friend class USkeletalMesh; 
	friend class UAnimInstance;
	friend struct FAnimNode_LinkedAnimGraph;
	friend struct FAnimNode_LinkedAnimLayer;
	friend struct FLinkedInstancesAdapter;
	friend struct FLinkedAnimLayerClassData;
	friend struct FRigUnit_AnimNextWriteSkeletalMeshComponentPose;
	friend struct FRigUnit_AnimNextRemapPose;
	
#if WITH_EDITORONLY_DATA 
private:
	/** The skeletal mesh used by this component. */
	UE_DEPRECATED(5.1, "This property isn't deprecated, but getter and setter must be used at all times to preserve correct operations.")
	UPROPERTY(EditAnywhere, Transient, Setter = SetSkeletalMeshAsset, BlueprintSetter = SetSkeletalMeshAsset, Getter = GetSkeletalMeshAsset, BlueprintGetter = GetSkeletalMeshAsset, Category = Mesh)
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset;
#endif

public:
	/**
	 * Set the SkeletalMesh rendered for this mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	void SetSkeletalMeshAsset(USkeletalMesh* NewMesh) { SetSkeletalMesh(NewMesh, false); }

	/**
	 * Get the SkeletalMesh rendered for this mesh.
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh")
	ENGINE_API USkeletalMesh* GetSkeletalMeshAsset() const;

	/** Sets the SkeletalMeshAsset to the current SkeletalMesh. */
	void RefreshSkeletalMeshAsset();

	/** Gets the shared bone container used between all owned anim instances. Creates it on the first call. */
	ENGINE_API TSharedPtr<struct FBoneContainer> GetSharedRequiredBones();

#if WITH_EDITORONLY_DATA
	/** The blueprint for creating an AnimationScript. */
	UPROPERTY()
	TObjectPtr<class UAnimBlueprint> AnimationBlueprint_DEPRECATED;

	UE_DEPRECATED(4.11, "This property is deprecated. Please use AnimClass instead")
	UPROPERTY(BlueprintReadOnly, Category = Animation, meta = (DeprecationMessage = "This property is deprecated. Please use AnimClass instead"))
	TObjectPtr<class UAnimBlueprintGeneratedClass> AnimBlueprintGeneratedClass;
#endif

	/* The AnimBlueprint class to use. Use 'SetAnimInstanceClass' to change at runtime. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter = SetAnimInstanceClass_Internal, Category = Animation, meta=(EditCondition = bEnableAnimation))
	class TSubclassOf<UAnimInstance> AnimClass;

	/** The active animation graph program instance. */
	UPROPERTY(transient, NonTransactional)
	TObjectPtr<UAnimInstance> AnimScriptInstance;

#if WITH_EDITORONLY_DATA
	/** Any running linked anim instances */
	UE_DEPRECATED(4.24, "Direct access to this property is deprecated and the array is no longer used. Storage is now in LinkedInstances. Please use GetLinkedAnimInstances() instead.")
	UPROPERTY(transient)
	TArray<TObjectPtr<UAnimInstance>> SubInstances;
#endif

	/** Post-processing AnimBP to use for the given skeletal mesh component, overriding the one set in the skeletal mesh asset. */
	UPROPERTY(transient)
	TSubclassOf<UAnimInstance> OverridePostProcessAnimBP;

	/**
	 * Get the post-processing AnimBP to be used for this skeletal mesh component.
	 * In case an override post-processing AnimBP is set, it will always return the AnimBP class of that.
	 * Otherwise the one set in skeletal mesh asset will be returned.
	 */
	ENGINE_API TSubclassOf<UAnimInstance> GetPostProcessAnimBPClassToBeUsed() const;

	/** An instance created from the PostPhysicsBlueprint property of the skeletal mesh we're using,
	 *  Runs after (and receives pose from) the main anim instance.
	 */
	UPROPERTY(transient)
	TObjectPtr<UAnimInstance> PostProcessAnimInstance;

public:
	/**
	 * Set the post-processing AnimBP to be used for this skeletal mesh component.
	 * In case an override post-processing AnimBP is set, the one set in skeletal mesh asset will be ignored and not used.
	 * @param ReinitAnimInstances Can be false when called e.g. from the construction script in a Blueprint. True when this is called while
	 *							  the game is running and the anim instances need to be re-initialized.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetOverridePostProcessAnimBP(TSubclassOf<UAnimInstance> InPostProcessAnimBlueprint, bool ReinitAnimInstances = true);

	/** Toggles whether the post process blueprint will run for this component */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API void ToggleDisablePostProcessBlueprint();

	/** Gets whether the post process blueprint is currently disabled for this component */
	UFUNCTION(BlueprintGetter)
	ENGINE_API bool GetDisablePostProcessBlueprint() const;

	/** Sets whether the post process blueprint is currently running for this component.
	 *  If it is not currently running, and is set to run, the instance will be reinitialized
	 */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetDisablePostProcessBlueprint(bool bInDisablePostProcess);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Animation, meta=(ShowOnlyInnerProperties, EditCondition = bEnableAnimation))
	struct FSingleAnimationPlayData AnimationData;

	// Blocks on the evaluation task (if necessary) and then returns a copy of the bone space
	// transforms. A copy is returned because the original array can/will be modified during the
	// evaluation, which can be running in a parallel task.
	ENGINE_API TArray<FTransform> GetBoneSpaceTransforms();

	// Blocks on the evaluation task (if necessary) and then returns bone space transforms as an
	// array view.
	ENGINE_API TArrayView<const FTransform> GetBoneSpaceTransformsView();

public:
	/** Offset of the root bone from the reference pose. Used to offset bounding box. */
	UPROPERTY(transient)
	FVector RootBoneTranslation;

	/** If bEnableLineCheckWithBounds is true, scale the bounds by this value before doing line check. */
	UPROPERTY()
	FVector LineCheckBoundsScale;

	UE_DEPRECATED(5.6, "Direct access to this property is deprecated, please use GetAnimCurves or GetCurveValue instead. We will move to private in the future.")
	FBlendedHeapCurve AnimCurves;

	/** Access cached component space transforms */
	ENGINE_API const TArray<FTransform>& GetCachedComponentSpaceTransforms() const;

private:
	/**
	 * Temporary array of local-space (relative to parent bone) rotation/translation for each bone.
	 * This property is not safe to access directly during evaluation - see GetBoneSpaceTransforms() etc
	 */
	TArray<FTransform> BoneSpaceTransforms;

	/** Any running linked anim instances */
	UPROPERTY(transient)
	TArray<TObjectPtr<UAnimInstance>> LinkedInstances;

	/** Shared bone container between all anim instances owned by this skeletal mesh component */
	TSharedPtr<struct FBoneContainer> SharedRequiredBones;

	// Update Rate

	/** Cached BoneSpaceTransforms for Update Rate optimization. */
	UPROPERTY(Transient)
	TArray<FTransform> CachedBoneSpaceTransforms;

	/** Cached SpaceBases for Update Rate optimization. */
	UPROPERTY(Transient)
	TArray<FTransform> CachedComponentSpaceTransforms;

	/** Cached Curve result for Update Rate optimization */
	FBlendedHeapCurve CachedCurve;

	/** Current and cached attribute evaluation data, used for Update Rate optimization */
	UE::Anim::FMeshAttributeContainer CachedAttributes;
	UE::Anim::FMeshAttributeContainer CustomAttributes;
public:
	/** 
	 * Get float type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
     * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category=CustomAttributes)
	ENGINE_API bool GetFloatAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) float& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/**
	 * Get vector type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetVectorAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) FVector& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/**
	 * Get quaternion type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetQuaternionAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) FQuat& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get FTransform type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
     * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetTransformAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) FTransform& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get integer type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
     * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetIntegerAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) int32& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get string type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
     * @param OutValue (reference) Retrieved attribute value if found
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetStringAttribute_Ref(const FName& BoneName, const FName& AttributeName, UPARAM(ref) FString& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get float type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param DefaultValue In case the attribute could not be found
     * @param OutValue Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetFloatAttribute(const FName& BoneName, const FName& AttributeName, float DefaultValue, float& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);
	
	/**
	 * Get vector type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param DefaultValue In case the attribute could not be found
	 * @param OutValue Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetVectorAttribute(const FName& BoneName, const FName& AttributeName, FVector DefaultValue, FVector& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/**
	 * Get quaternion type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param DefaultValue In case the attribute could not be found
	 * @param OutValue Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetQuaternionAttribute(const FName& BoneName, const FName& AttributeName, FQuat DefaultValue, FQuat& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/**
	 * Get FTransform type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param OutValue (reference) Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetTransformAttribute(const FName& BoneName, const FName& AttributeName, FTransform DefaultValue, FTransform& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get integer type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param DefaultValue In case the attribute could not be found
     * @param OutValue Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetIntegerAttribute(const FName& BoneName, const FName& AttributeName, int32 DefaultValue, int32& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get string type attribute value.

	 * @param BoneName Name of the bone to retrieve try and retrieve the attribute from
	 * @param AttributeName Name of the attribute to retrieve
	 * @param DefaultValue In case the attribute could not be found
     * @param OutValue Retrieved attribute value if found, otherwise is set to DefaultValue
	 * @param LookupType Determines how the attribute is retrieved from the specified BoneName (see ECustomBoneAttributeLookup)
	 * @return Whether or not the atttribute was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = CustomAttributes)
	ENGINE_API bool GetStringAttribute(const FName& BoneName, const FName& AttributeName, FString DefaultValue, FString& OutValue, ECustomBoneAttributeLookup LookupType = ECustomBoneAttributeLookup::BoneOnly);

	/** 
	 * Get curve value
	 * @param CurveName Name of the curve to retrieve
	 * @param DefaultValue In case the curve could not be found
	 * @param Value Retrieved curve value if found, otherwise is set to DefaultValue
	 * @return Whether the curve was successfully retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = Curve) 
	ENGINE_API bool GetCurveValue(FName CurveName, float DefaultValue, float& Value) const;

	/** Get the current set of curve values */
	ENGINE_API const FBlendedHeapCurve& GetAnimCurves() const;

protected:
	/** Templated version to try and retrieve a typed bone attribute's value */
	template<typename DataType, typename CustomAttributeType>
	bool FindAttributeChecked(const FName& BoneName, const FName& AttributeName, DataType DefaultValue, DataType& OutValue, ECustomBoneAttributeLookup LookupType);	

public:
	/** Used to scale speed of all animations on this skeletal mesh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Animation, meta=(EditCondition = bEnableAnimation, ClampMin = 0.f))
	float GlobalAnimRateScale;
	
	/** If we are running physics, should we update non-simulated bones based on the animation bone positions. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=SkeletalMesh)
	TEnumAsByte<EKinematicBonesUpdateToPhysics::Type> KinematicBonesUpdateType;

	/** Whether physics simulation updates component transform. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	TEnumAsByte<EPhysicsTransformUpdateMode::Type> PhysicsTransformUpdateMode;

	/** whether we need to teleport cloth. */
	UPROPERTY(Interp, Transient, BlueprintReadOnly, VisibleAnywhere,  Category=Clothing) // This property is explicitly hidden from the details panel inside FSkeletalMeshComponentDetails::UpdatePhysicsCategory
	EClothingTeleportMode ClothTeleportMode;

protected:
	/** Whether to use Animation Blueprint or play Single Animation Asset. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Animation, meta=(EditCondition = bEnableAnimation))
	TEnumAsByte<EAnimationMode::Type>	AnimationMode;
public:
	// helper function to get the member name and verify it exists, without making it public
	static ENGINE_API const FName& GetAnimationModePropertyNameChecked();

	// helper function to get the member name and verify it exists, without making it public
#if WITH_EDITORONLY_DATA
	static ENGINE_API FName GetSkeletalMeshAssetPropertyNameChecked();
#endif // WITH_EDITORONLY_DATA

private:
	/** Teleport type to use on the next update */
	ETeleportType PendingTeleportType;

	/** Controls whether or not this component will evaluate its post process instance. The post-process
	 *  Instance is dictated by the skeletal mesh so this is used for per-instance control.
	 */
	UPROPERTY(EditAnywhere, BlueprintGetter=GetDisablePostProcessBlueprint, BlueprintSetter=SetDisablePostProcessBlueprint, Category = Animation, meta=(EditCondition = bEnableAnimation))
	uint8 bDisablePostProcessBlueprint:1;

public:
	/** Indicates that simulation (if it's enabled) is entirely responsible for children transforms. This is only ok if you are not animating attachment points relative to the simulation */
	uint8 bSimulationUpdatesChildTransforms:1;

	/** Controls whether blending in physics bones will refresh overlaps on this component, defaults to true but can be disabled in cases where we know anim->physics blending doesn't meaningfully change overlaps */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Physics)
	uint8 bUpdateOverlapsOnAnimationFinalize:1;

	/** Temporary fix for local space kinematics. This only works for bodies that have no constraints and is needed by vehicles. Proper support will remove this flag */
	uint8 bLocalSpaceKinematics:1;

	/** If true, there is at least one body in the current PhysicsAsset with a valid bone in the current SkeletalMesh */
	UPROPERTY(transient)
	uint8 bHasValidBodies:1;


	/** Forces use of the physics bodies irrespective of whether they are simulated or using the physics blend weight */
	UPROPERTY(transient)
	uint8 bBlendPhysics:1;

	/**
	 *  If true, simulate physics for this component on a dedicated server.
	 *  This should be set if simulating physics and replicating with a dedicated server.
	 *	Note: This property cannot be changed at runtime.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bEnablePhysicsOnDedicatedServer:1;

	/** 
	 * If true, then the physics bodies will be used to drive the skeletal mesh even when they are
	 * kinematic (not simulating), otherwise the skeletal mesh will be driven by the animation input 
	 * when the bodies are kinematic
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bUpdateMeshWhenKinematic:1;

	/**
	 *	If we should pass joint position to joints each frame, so that they can be used by motorized joints to drive the
	 *	ragdoll based on the animation.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=SkeletalMesh)
	uint8 bUpdateJointsFromAnimation:1;

	/**
	 * Toggles creation of cloth simulation. Distinct from the simulation toggle below in that, if off, avoids allocating
	 * the actors entirely instead of just skipping the simulation step.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Clothing)
	uint8 bAllowClothActors:1;

	/** Disable cloth simulation and play original animation without simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Clothing)
	uint8 bDisableClothSimulation:1;

	/** Indicates that this SkeletalMeshComponent has deferred kinematic bone updates until next physics sim if not INDEX_NONE. */
	int32 DeferredKinematicUpdateIndex;

private:

	/** Disable rigid body animation nodes and play original animation without simulation */
	UPROPERTY(EditAnywhere, Category = Physics)
	uint8 bDisableRigidBodyAnimNode:1;

	/** Disable Anim Dynamics animation nodes and play original animation without simulation */
	UPROPERTY(EditAnywhere, Category = Physics)
	uint8 bDisableAnimDynamicsAnimNode:1;

	/** Disable animation curves for this component. If this is set true, no curves will be processed */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = SkeletalMesh)
	uint8 bAllowAnimCurveEvaluation : 1;

#if WITH_EDITORONLY_DATA
	/** DEPRECATED. Use bAllowAnimCurveEvaluation instead */
	UE_DEPRECATED(4.18, "This property is deprecated. Please use bAllowAnimCurveEvaluatiuon instead. Note that the meaning is reversed.")	
	UPROPERTY()
	uint8 bDisableAnimCurves_DEPRECATED : 1;
#endif

	/** Whether or not we're taking cloth sim information from our leader component */
	uint8 bBindClothToLeaderComponent:1;

	/** Flag denoting whether or not the clothing transform needs to update */
	uint8 bPendingClothTransformUpdate:1;

	/** Flag denoting whether or not the clothing collision needs to update from its physics asset */
	uint8 bPendingClothCollisionUpdate:1;

public:
	/** can't collide with part of environment if total collision volumes exceed 16 capsules or 32 planes per convex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Clothing)
	uint8 bCollideWithEnvironment:1;

	/** can't collide with part of attached children if total collision volumes exceed 16 capsules or 32 planes per convex */
	UE_DEPRECATED(5.8, "bCollideWithAttachedChildren is deprecated. Use CollisionSources to add external collisions instead.")
	UPROPERTY(BlueprintReadWrite, Category = Clothing, meta=(DeprecatedProperty, DeprecationMessage="CollideWithAttachedChildren is deprecated. Use CollisionSources to add external collisions instead."))
	uint8 bCollideWithAttachedChildren:1;

	/**
	 * Forces the cloth simulation to constantly update its external collisions, at the expense of performance.
	 * This will help to prevent missed collisions if the cloth's skeletal mesh component isn't moving,
	 * and when instead, wind or other moving objects are causing new collision shapes to come into the cloth's vicinity.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Clothing)
	uint8 bForceCollisionUpdate : 1;

	/**
	 * Scale applied to the component induced velocity of all cloth particles prior to stepping the cloth solver.
	 * Use 1.0 for fully induced velocity (default), or use 0.0 for no induced velocity, and any other values in between for a reduced induced velocity.
	 * When set to 0.0, it also provides a way to force the clothing to simulate in local space.
	 * This value multiplies to individual cloth's velocity scale settings, still allowing for differences in behavior between the various pieces of clothing within the same component.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Clothing, Meta = (UIMin = 0.f, UIMax = 1.f, ClampMin = 0.f, ClampMax = 1.f))
	float ClothVelocityScale = 1.f;

	/** reset the clothing after moving the clothing position (called teleport) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Clothing)
	uint8 bResetAfterTeleport:1;

	/** To save previous state */
	uint8 bPrevDisableClothSimulation : 1;

	/** 
	 * Optimization
	 */
	
	 /** Whether animation and world transform updates are deferred. If this is on, the kinematic bodies (scene query data) will not update until the next time the physics simulation is run */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = SkeletalMesh)
	uint8 bDeferKinematicBoneUpdate : 1;

	/** Skips Ticking and Bone Refresh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=SkeletalMesh)
	uint8 bNoSkeletonUpdate:1;

	/** pauses this component's animations (doesn't tick them, but still refreshes bones) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Animation, meta=(EditCondition = bEnableAnimation))
	uint8 bPauseAnims:1;

	/**
	 * Whether the built-in animation of this component should run when the component ticks.
	 * It is assumed that if this is false then some external system will be animating this mesh.
	 * Note that disabling animation will also cause cloth simulation not to run and the component's tick to run on any thread. 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Animation)
	uint8 bEnableAnimation : 1;

	/** On InitAnim should we set to ref pose (if false use first tick of animation data). If enabled, takes precedence over UAnimationSettings::bTickAnimationOnSkeletalMeshInit*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Animation, meta=(EditCondition = bEnableAnimation))
	uint8 bUseRefPoseOnInitAnim:1;

	/**
	* Uses skinned data for collision data.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=SkeletalMesh)
	uint8 bEnablePerPolyCollision:1;

	/**
	 * Misc 
	 */
	
	/** If true, force the mesh into the reference pose - is an optimization. */
	UPROPERTY()
	uint8 bForceRefpose:1;
	
	/** If true TickPose() will not be called from the Component's TickComponent function.
	* It will instead be called from Autonomous networking updates. See ACharacter. */
	UPROPERTY(Transient)
	uint8 bOnlyAllowAutonomousTickPose : 1;

	/** True if calling TickPose() from Autonomous networking updates. See ACharacter. */
	UPROPERTY(Transient)
	uint8 bIsAutonomousTickPose : 1;

	/** If bForceRefPose was set last tick. */
	UPROPERTY()
	uint8 bOldForceRefPose:1;

	/** Bool that enables debug drawing of the skeleton before it is passed to the physics. Useful for debugging animation-driven physics. */
	UPROPERTY()
	uint8 bShowPrePhysBones:1;

	/** If false, indicates that on the next call to UpdateSkelPose the RequiredBones array should be recalculated. */
	UPROPERTY(transient)
	uint8 bRequiredBonesUpToDate:1;

	/** If true, AnimTree has been initialised. */
	UPROPERTY(transient)
	uint8 bAnimTreeInitialised:1;

	/** If true, line checks will test against the bounding box of this skeletal mesh component and return a hit if there is a collision. */
	UPROPERTY()
	uint8 bEnableLineCheckWithBounds:1;

	/** If true, propagates calls to ApplyAnimationCurvesToComponent for follower components, only needed if follower components do not tick themselves */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LeaderPoseComponent)
	uint8 bPropagateCurvesToFollowers : 1;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property is deprecated. Please use bPropagateCurvesToFollowers instead")
	uint8 bPropagateCurvesToSlaves : 1;
#endif // WITH_EDITORONLY_DATA

	/** Whether to skip UpdateKinematicBonesToAnim() when interpolating. Kinematic bones are updated to the target interpolation pose only on ticks when they are evaluated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Optimization)
	uint8 bSkipKinematicUpdateWhenInterpolating:1;

	/** Whether to skip bounds update when interpolating. Bounds are updated to the target interpolation pose only on ticks when they are evaluated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Optimization)
	uint8 bSkipBoundsUpdateWhenInterpolating:1;

	/** Prevent notify events from being dispatched for this mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bSuppressNotifyEventDispatch : 1;

protected:

	/** Whether the clothing simulation is suspended (not the same as disabled, we no longer run the sim but keep the last valid sim data around) */
	uint8 bClothingSimulationSuspended:1;

#if WITH_EDITORONLY_DATA
	/** If true, this will Tick until disabled */
	UPROPERTY(AdvancedDisplay, EditInstanceOnly, transient, Category = SkeletalMesh)
	uint8 bUpdateAnimationInEditor : 1;
	/** If true, will play cloth in editor */
	UPROPERTY(AdvancedDisplay, EditInstanceOnly, transient, Category = SkeletalMesh)
	uint8 bUpdateClothInEditor : 1;

	/** If true, DefaultAnimatingRigOverride will be used. If false, use the DefaultAnimatingRig in the SkeletalMesh */
	UPROPERTY(AdvancedDisplay,EditAnywhere, Category = SkeletalMesh, meta = (InlineEditConditionToggle))
	uint8 bOverrideDefaultAnimatingRig : 1;

#endif

	/** If true, OnSyncComponentToRBPhysics() notify will be called */
	uint8 bNotifySyncComponentToRBPhysics : 1;

private:

	UPROPERTY(Transient)
	uint8 bNeedsQueuedAnimEventsDispatched:1;

	uint8 bPostEvaluatingAnimation:1;

public:
	/** Physics-engine representation of aggregate which contains a physics asset instance with more than numbers of bodies. */
	FPhysicsAggregateHandle Aggregate;

public:

	/** Cache AnimCurveUidVersion from Skeleton and this will be used to identify if it needs to be updated */
	UPROPERTY(transient)
	uint16 CachedAnimCurveUidVersion;

	/**
	 * weight to blend between simulated results and key-framed positions
	 * if weight is 1.0, shows only cloth simulation results and 0.0 will show only skinned results
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = Clothing)
	float ClothBlendWeight;

	/** Whether we should stall the Cloth tick task until the cloth simulation is complete. This is required if we want up-to-date
	 * cloth data on the game thread, for example if we want to generate particles at cloth vertices.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Clothing)
	bool bWaitForParallelClothTask;

private:

	/** Whether the FilteredAnimCurves list is an allow or deny list */
	UPROPERTY(transient)
	bool bFilteredAnimCurvesIsAllowList = false;

	/** Cache curve metadata from mesh and this will be used to identify if it needs to be updated */
	UPROPERTY(transient)
	uint16 CachedMeshCurveMetaDataVersion;
	
	/** You can choose to disable certain curves if you prefer. 
	 * This is transient curves that will be ignored by animation system if you choose this */
	UPROPERTY(transient)
	TArray<FName> FilteredAnimCurves;

public:

	/**
	* Used for per poly collision. In 99% of cases you will be better off using a Physics Asset.
	* This BodySetup is per instance because all modification of vertices is done in place */
	UPROPERTY(transient)
	TObjectPtr<class UBodySetup>  BodySetup;

	ENGINE_API void CreateBodySetup();

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API virtual void SendRenderDebugPhysics(FPrimitiveSceneProxy* OverrideSceneProxy = nullptr) override;
#endif

public:

	/** Threshold for physics asset bodies above which we use an aggregate for broadphase collisions */
	int32 RagdollAggregateThreshold;

	UPROPERTY(Interp, BlueprintReadWrite, Category=Clothing, meta=(UIMin = 0.0, UIMax = 10.0, ClampMin = 0.0, ClampMax = 10000.0))
	float ClothMaxDistanceScale;
	
	/** This scale is applied to all cloth geometry (e.g., cloth meshes and collisions) in order to simulate in a different scale space than world.This scale is not applied to distance-based simulation parameters such as MaxDistance. 
	* This property is currently only read by the cloth solver when creating cloth actors, but may become animatable in the future.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = Clothing, meta = (UIMin = 0.0, UIMax = 10.0, ClampMin = 0.0, ClampMax = 10000.0))
	float ClothGeometryScale = 1.f;

private:
	/*
	 * Max LOD level that post-process AnimBPs are evaluated. Overrides the setting of the same name on the skeletal mesh.
	 * For example if you have the threshold set to 2, it will evaluate until including LOD 2 (based on 0 index). In case the LOD level gets set to 3, it will stop evaluating the post-process AnimBP.
	 * Setting it to -1 will always evaluate it and disable LODing overrides for this component.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Animation, meta= (DisplayName = "Post-Process AnimBP LOD Threshold", AllowPrivateAccess = "true"))
	int32 PostProcessAnimBPLODThreshold = INDEX_NONE;

public:
	/** Notification when constraint is broken. */
	UPROPERTY(BlueprintAssignable)
	FConstraintBrokenSignature OnConstraintBroken;

	/** Notification when constraint plasticity drive target changes. */
	UPROPERTY(BlueprintAssignable)
	FPlasticDeformationEventSignature OnPlasticDeformation;

	/** Class of the object responsible for  */
	UE_DEPRECATED(5.7, "This property will be removed since the ClothingSimulationFactory is no longer set per component.")
	UPROPERTY()
	TSubclassOf<class UClothingSimulationFactory> ClothingSimulationFactory;

	struct FPendingRadialForces
	{
		enum EType
		{
			AddImpulse,
			AddForce
		};

		FVector Origin;
		float Radius;
		float Strength;
		ERadialImpulseFalloff Falloff;
		bool bIgnoreMass;
		EType Type;
		int32 FrameNum;

		FPendingRadialForces(FVector InOrigin, float InRadius, float InStrength, ERadialImpulseFalloff InFalloff, bool InIgnoreMass, EType InType)
			: Origin(InOrigin)
			, Radius(InRadius)
			, Strength(InStrength)
			, Falloff(InFalloff)
			, bIgnoreMass(InIgnoreMass)
			, Type(InType)
			, FrameNum(GFrameNumber)
		{
		}
	};

	const TArray<FPendingRadialForces>& GetPendingRadialForces() const
	{
		return PendingRadialForces;
	}
	/** Array of physical interactions for the frame. This is a temporary solution for a more permanent force system and should not be used directly*/
	TArray<FPendingRadialForces> PendingRadialForces;

	UE_DEPRECATED(4.23, "This function is deprecated. Please use SetAnimInstanceClass instead. ")
	ENGINE_API virtual void K2_SetAnimInstanceClass(class UClass* NewClass);

	/** Set the anim instance class. Clears and re-initializes the anim instance with the new class and sets animation mode to 'AnimationBlueprint' */
	UE_DEPRECATED(5.5, "This function is deprecated. Please use 'SetAnimInstanceClass' instead. ")
	ENGINE_API virtual void SetAnimClass(class UClass* NewClass);

	/** Get the anim instance class via getter callable by sequencer.  */
	UFUNCTION(BlueprintInternalUseOnly)
	ENGINE_API class UClass*  GetAnimClass();

	/**
	 * Set the anim instance class. Clears and re-initializes the anim instance with the new class and sets animation mode to 'AnimationBlueprint'
	 * This function will fail with a warning if it is called while the current anim instance is evaluating (e.g. from an anim notify).
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (Keywords = "AnimBlueprint", DisplayName = "Set Anim Instance Class"))
	ENGINE_API virtual void SetAnimInstanceClass(class UClass* NewClass);

	/**
	* Whether the built-in animation of this component should run when the component ticks. See USkeletalMeshComponent::bEnableAnimation for more details.
	* Note that this call defers setting the flag to the end of the frame as the flag cannot change during world ticking (as other tasks may be in flight)
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (Keywords = "Animation"))
	ENGINE_API void SetEnableAnimation(bool bNewEnableAnimation);

	/**
	 * Returns the animation instance that is driving the class (if available). This is typically an instance of
	 * the class set as AnimBlueprintGeneratedClass (generated by an animation blueprint)
	 * Since this instance is transient, it is not safe to be used during construction script
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh", meta=(Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API class UAnimInstance * GetAnimInstance() const;
	
private:
	/* Setter function for anim instance class */
	void SetAnimInstanceClass_Internal(const TSubclassOf<UAnimInstance>& InAnimClass);

public:

	/**
	 * Returns the active post process instance is one is available. This is set on the mesh that this
	 * component is using, and is evaluated immediately after the main instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API UAnimInstance* GetPostProcessInstance() const;

	/** Get the anim instances linked to the main AnimScriptInstance */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<UAnimInstance*>& GetLinkedAnimInstances() const { return LinkedInstances; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** 
	 * If true, next time that ConditionallyDispatchQueuedAnimEvents() is called it will trigger any 
	 * queued anim notifies available. Only to be used when not performing a regular anim instance update.
	 */
	ENGINE_API void AllowQueuedAnimEventsNextDispatch();
	
private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<TObjectPtr<UAnimInstance>>& GetLinkedAnimInstances() { return LinkedInstances; }
ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Clear the linked anim instances and mark them pending kill */
	void ResetLinkedAnimInstances();

	/** Internal helper - copies the mesh's reference pose to the local space transforms and regenerates component space transforms accordingly */ 
	void ResetToRefPose();

public:
	UE_DEPRECATED(4.23, "This function is deprecated. Please use GetLinkedAnimGraphInstanceByTag")
	UAnimInstance* GetSubInstanceByName(FName InTag) const { return GetLinkedAnimGraphInstanceByTag(InTag); }

	UE_DEPRECATED(4.24, "This function is deprecated. Please use GetLinkedAnimGraphInstanceByTag")
	UAnimInstance* GetSubInstanceByTag(FName InTag) const { return GetLinkedAnimGraphInstanceByTag(InTag); }

	/**
	 * Returns the a tagged linked instance node. If no linked instances are found or none are tagged with the
	 * supplied name, this will return NULL.
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API UAnimInstance* GetLinkedAnimGraphInstanceByTag(FName InTag) const;

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimGraphInstancesByTag")
	void GetSubInstancesByTag(FName InTag, TArray<UAnimInstance*>& OutSubInstances) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetLinkedAnimGraphInstancesByTag(InTag, OutSubInstances);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Returns all tagged linked instance nodes that match the tag.
	 */
	UE_DEPRECATED(5.0, "Tags are unique so this function is no longer supported. Please use GetLinkedAnimGraphInstanceByTag instead")
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", DeprecatedFunction, DeprecationMessage="Tags are unique so this function is no longer supported. Please use GetLinkedAnimGraphInstanceByTag instead"))
	ENGINE_API void GetLinkedAnimGraphInstancesByTag(FName InTag, TArray<UAnimInstance*>& OutLinkedInstances) const;

	UE_DEPRECATED(4.24, "Function renamed, please use LinkAnimGraphByTag")
	void SetSubInstanceClassByTag(FName InTag, TSubclassOf<UAnimInstance> InClass) { LinkAnimGraphByTag(InTag, InClass); }

	/** Runs through all nodes, attempting to find linked instance by name/tag, then sets the class of each node if the tag matches */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void LinkAnimGraphByTag(FName InTag, TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use LinkAnimClassLayers")
	void SetLayerOverlay(TSubclassOf<UAnimInstance> InClass) { LinkAnimClassLayers(InClass); }

	/** 
	 * Runs through all layer nodes, attempting to find layer nodes that are implemented by the specified class, then sets up a linked instance of the class for each.
	 * Allocates one linked instance to run each of the groups specified in the class, so state is shared. If a layer is not grouped (ie. NAME_None), then state is not shared
	 * and a separate linked instance is allocated for each layer node.
	 * If InClass is null, then all layers are reset to their defaults.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void LinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use UnlinkAnimClassLayers")
	void ClearLayerOverlay(TSubclassOf<UAnimInstance> InClass) { UnlinkAnimClassLayers(InClass); }

	/** 
	 * Runs through all layer nodes, attempting to find layer nodes that are currently running the specified class, then resets each to its default value.
	 * State sharing rules are as with SetLayerOverlay.
	 * If InClass is null, does nothing.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void UnlinkAnimClassLayers(TSubclassOf<UAnimInstance> InClass);

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedLayerInstanceByGroup")
	UAnimInstance* GetLayerSubInstanceByGroup(FName InGroup) const { return GetLinkedAnimLayerInstanceByGroup(InGroup); }

	/** Gets the layer linked instance corresponding to the specified group */
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API UAnimInstance* GetLinkedAnimLayerInstanceByGroup(FName InGroup) const;

	UE_DEPRECATED(4.24, "Function renamed, please use GetLinkedAnimLayerInstanceByClass")
	UAnimInstance* GetLayerSubInstanceByClass(TSubclassOf<UAnimInstance> InClass) const { return GetLinkedAnimLayerInstanceByClass(InClass);  }

	/** Gets the first layer linked instance corresponding to the specified class */
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh|Animation Blueprint Linking", meta = (Keywords = "AnimBlueprint", UnsafeDuringActorConstruction = "true"))
	ENGINE_API UAnimInstance* GetLinkedAnimLayerInstanceByClass(TSubclassOf<UAnimInstance> InClass) const;

	/** Calls a function on each of the anim instances that this mesh component hosts, including linked and post-process instances */
	ENGINE_API void ForEachAnimInstance(TFunctionRef<void(UAnimInstance*)> InFunction);

	/** 
	 * Returns whether there are any valid instances to run, currently this means whether we have
	 * have an animation instance or a post process instance available to process.
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkeletalMesh", meta = (Keywords = "AnimBlueprint"))
	ENGINE_API bool HasValidAnimationInstance() const;

	/**
	 * Informs any active anim instances (main instance, linked instances, post instance) that a dynamics reset is required
	 * for example if a teleport occurs.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (Keywords = "Dynamics,Physics", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void ResetAnimInstanceDynamics(ETeleportType InTeleportType = ETeleportType::ResetPhysics);

	/** Below are the interface to control animation when animation mode, not blueprint mode */

	/**
	* Set the Animation Mode
	* @param InAnimationMode : Requested AnimationMode
	* @param bForceInitAnimScriptInstance : Init AnimScriptInstance if the AnimationMode is AnimationBlueprint even if the new animation mode is the same as current (this allows to use BP construction script to do this)
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation"))
	ENGINE_API void SetAnimationMode(EAnimationMode::Type InAnimationMode, bool bForceInitAnimScriptInstance = true);
	
	UFUNCTION(BlueprintPure, Category = "Components|Animation", meta = (Keywords = "Animation"))
	ENGINE_API EAnimationMode::Type GetAnimationMode() const;

	/* Animation play functions
	 *
	 * These changes status of animation instance, which is transient data, which means it won't serialize with this component
	 * Because of that reason, it is not safe to be used during construction script
	 * Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized 
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void PlayAnimation(class UAnimationAsset* NewAnimToPlay, bool bLooping);

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetAnimation(class UAnimationAsset* NewAnimToPlay);

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void Play(bool bLooping);

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void Stop();

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API bool IsPlaying() const;

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetPosition(float InPos, bool bFireNotifies = true);

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API float GetPosition() const;

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetPlayRate(float Rate);

	/* Animation play functions
	*
	* These changes status of animation instance, which is transient data, which means it won't serialize with this component
	* Because of that reason, it is not safe to be used during construction script
	* Please use OverrideAnimationData for construction script. That will override AnimationData to be serialized
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation", UnsafeDuringActorConstruction = "true"))
	ENGINE_API float GetPlayRate() const;

	/**
	 * This overrides current AnimationData parameter in the SkeletalMeshComponent. This will serialize when the component serialize
	 * so it can be used during construction script. However note that this will override current existing data
	 * This can be useful if you'd like to make a blueprint with custom default animation per component
	 * This sets single player mode, which means you can't use AnimBlueprint with it
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|Animation", meta = (Keywords = "Animation"))
	ENGINE_API void OverrideAnimationData(UAnimationAsset* InAnimToPlay, bool bIsLooping = true, bool bIsPlaying = true, float Position = 0.f, float PlayRate = 1.f);

	/**
	 * Set Morph Target with Name and Value(0-1)
	 *
	 * @param bRemoveZeroWeight : Used by editor code when it should stay in the active list with zero weight
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void SetMorphTarget(FName MorphTargetName, float Value, bool bRemoveZeroWeight=true);

	/**
	 * Clear all Morph Target that are set to this mesh
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API void ClearMorphTargets();

	/**
	 * Get Morph target with given name
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API float GetMorphTarget(FName MorphTargetName) const;

	/**
	 * Takes a snapshot of this skeletal mesh component's pose and saves it to the specified snapshot.
	 * The snapshot is taken at the current LOD, so if for example you took the snapshot at LOD1 
	 * and then used it at LOD0 any bones not in LOD1 will use the reference pose 
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SnapshotPose(UPARAM(ref) FPoseSnapshot& Snapshot);

	/**
	 * Sets whether cloth assets should be created/simulated in this component.
	 * This will update the conditional flag and you will want to call RecreateClothingActors for it to take effect.
	 * @param bInAllow Whether to allow the creation of cloth assets and simulation.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API void SetAllowClothActors(bool bInAllow);

	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	bool GetAllowClothActors() const { return bAllowClothActors; }

	/**
	 * Get/Set the max distance scale of clothing mesh vertices
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing")
	ENGINE_API float GetClothMaxDistanceScale() const;
	UFUNCTION(BlueprintCallable, Category="Clothing")
	ENGINE_API void SetClothMaxDistanceScale(float Scale);

	/** 
	 * Used to indicate we should force 'teleport' during the next call to UpdateClothState, 
	 * This will transform positions and velocities and thus keep the simulation state, just translate it to a new pose.
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing")
	ENGINE_API void ForceClothNextUpdateTeleport();
	/** 
	 * Used to indicate we should force 'teleport and reset' during the next call to UpdateClothState.
	 * This can be used to reset it from a bad state or by a teleport where the old state is not important anymore.
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing")
	ENGINE_API void ForceClothNextUpdateTeleportAndReset();

	/** Stops simulating clothing, but does not show clothing ref pose. Keeps the last known simulation state */
	UFUNCTION(BlueprintCallable, Category="Clothing", meta=(UnsafeDuringActorConstruction))
	ENGINE_API void SuspendClothingSimulation();

	/** Resumes a previously suspended clothing simulation, teleporting the clothing on the next tick */
	UFUNCTION(BlueprintCallable, Category = "Clothing", meta=(UnsafeDuringActorConstruction))
	ENGINE_API void ResumeClothingSimulation();

	/** Gets whether or not the clothing simulation is currently suspended */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API bool IsClothingSimulationSuspended() const;

	/**
	 * Reset the teleport mode of a next update to 'Continuous'
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing")
	ENGINE_API void ResetClothTeleportMode();

	/** 
	 * If this component has a valid LeaderPoseComponent then this function makes cloth items on the follower component
	 * take the transforms of the cloth items on the leader component instead of simulating separately.
	 * @Note This will FORCE any cloth actor on the leader component to simulate in local space. Also
	 * The meshes used in the components must be identical for the cloth to bind correctly
	 */
	UFUNCTION(BlueprintCallable, Category = "Clothing", meta = (UnsafeDuringActorConstruction = "true"))
	ENGINE_API void BindClothToLeaderPoseComponent();

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use BindClothToLeaderPoseComponent instead.")
	void BindClothToMasterPoseComponent() { BindClothToLeaderPoseComponent(); }

	/** 
	 * If this component has a valid LeaderPoseComponent and has previously had its cloth bound to the
	 * MCP, this function will unbind the cloth and resume simulation.
	 * @param bRestoreSimulationSpace if true and the leader pose cloth was originally simulating in world
	 * space, we will restore this setting. This will cause the leader component to reset which may be
	 * undesirable.
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void UnbindClothFromLeaderPoseComponent(bool bRestoreSimulationSpace = true);

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use UnbindClothFromLeaderPoseComponent instead.")
	void UnbindClothFromMasterPoseComponent(bool bRestoreSimulationSpace = true) { UnbindClothFromLeaderPoseComponent(bRestoreSimulationSpace); }

	/**
	 * Sets whether or not to allow rigid body animation nodes for this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetAllowRigidBodyAnimNode(bool bInAllow, bool bReinitAnim = true);

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	bool GetAllowRigidBodyAnimNode() const { return !bDisableRigidBodyAnimNode; }

	/**
	 * Sets whether or not to allow Anim Dynamics animation nodes for this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetAllowAnimDynamicsAnimNode(bool bInAllow, bool bReinitAnim = true);

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	bool GetAllowAnimDynamicsAnimNode() const { return !bDisableAnimDynamicsAnimNode; }

	/**
	 * Initialise animation nodes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void ReinitializeAnimNodes();

	/**
	* Sets whether or not to force tick component in order to update animation and refresh transform for this component
	* This is supported only in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (DevelopmentOnly, UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetUpdateAnimationInEditor(const bool NewUpdateState);

	/**
	* Sets whether or not to animate cloth in the editor. Requires Update Animation In Editor to also be true.
	* This is supported only in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh", meta = (DevelopmentOnly, UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetUpdateClothInEditor(const bool NewUpdateState);

#if WITH_EDITOR
	/**
	 * return true if currently updating in editor is true
	 * this is non BP because this is only used for follower component to detect leader component ticking state
	 */
	bool GetUpdateAnimationInEditor() const 
	{		
		return bUpdateAnimationInEditor;	
	}

	bool GetUpdateClothInEditor() const
	{
		return bUpdateClothInEditor;
	}
#endif 

	UE_DEPRECATED(4.18, "This function is deprecated. Please use SetAllowAnimCurveEvaluation instead. Note that the meaning is reversed.")
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetDisableAnimCurves(bool bInDisableAnimCurves);

	UE_DEPRECATED(4.18, "This function is deprecated. Please use GetAllowedAnimCurveEvaluate instead. Note that the meaning is reversed.")
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	bool GetDisableAnimCurves() const { return !bAllowAnimCurveEvaluation; }

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetAllowAnimCurveEvaluation(bool bInAllow);

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	bool GetAllowedAnimCurveEvaluate() const { return bAllowAnimCurveEvaluation; }

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void AllowAnimCurveEvaluation(FName NameOfCurve, bool bAllow);

	/** By reset, it will allow all the curves to be evaluated */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void ResetAllowedAnimCurveEvaluation();

	/** resets, and then only allow the following list to be allowed/disallowed */
	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API void SetAllowedAnimCurvesEvaluation(const TArray<FName>& List, bool bAllow);

	UE_DEPRECATED(5.3, "Please use GetCurveFilterSettings")
	const TArray<FName>& GetDisallowedAnimCurvesEvaluation() const { return FilteredAnimCurves; }

	/**
	 * Get the curve settings that will be used for anim evaluation.
	 * @param InLODOverride		Override the LOD that curves will be calculated for if not INDEX_NONE
	 * @return the curve settings used for evaluation
	 */
	ENGINE_API UE::Anim::FCurveFilterSettings GetCurveFilterSettings(int32 InLODOverride = INDEX_NONE) const;
	
	/** We detach the Component once we are done playing it.
	 *
	 * @param	ParticleSystemComponent that finished
	 */
	ENGINE_API virtual void SkelMeshCompOnParticleSystemFinished( class UParticleSystemComponent* PSC );

	ENGINE_API class UAnimSingleNodeInstance * GetSingleNodeInstance() const;
	ENGINE_API bool InitializeAnimScriptInstance(bool bForceReinit = true, bool bInDeferRootNodeInitialization = false);

	/** @return true if wind is enabled */
	ENGINE_API virtual bool IsWindEnabled() const;

#if WITH_EDITOR
	/**
	* Subclasses such as DebugSkelMeshComponent keep track of errors in the anim notifies so they can be displayed to the user. This function adds an error.
	* Errors are added uniquely and only removed when they're cleared by ClearAnimNotifyError.
	*/
	virtual void ReportAnimNotifyError(const FText& Error, UObject* InSourceNotify){}
	
	/**
	* Clears currently stored errors. Call before triggering anim notifies for a particular mesh.
	*/
	virtual void ClearAnimNotifyErrors(UObject* InSourceNotify){}
#endif

public:
	/** 
	 *	Index of the 'Root Body', or top body in the asset hierarchy. 
	 *	Filled in by InitInstance, so we don't need to save it.
	 */
	/** To save root body index/bone index consistently **/
	struct 
	{
		int32 BodyIndex;
		FTransform TransformToRoot;
	} RootBodyData;

	/** Set Root Body Index */
	ENGINE_API void SetRootBodyIndex(int32 InBodyIndex);
	/** Reset Root Body Index */
	ENGINE_API void ResetRootBodyIndex();

	/** Temporary array of bone indices required this frame. Filled in by UpdateSkelPose. */
	TArray<FBoneIndexType> RequiredBones;

	/** Temporary array of bone indices required to populate component space transforms */
	TArray<FBoneIndexType> FillComponentSpaceTransformsRequiredBones;

	/** Array of FBodyInstance objects, storing per-instance state about about each body. */
	TArray<struct FBodyInstance*> Bodies;

	/** Array of FConstraintInstance structs, storing per-instance state about each constraint. */
	TArray<struct FConstraintInstance*> Constraints;

	FSkeletalMeshComponentClothTickFunction ClothTickFunction;
	FSkeletalMeshComponentClothEndTickFunction ClothEndTickFunction;

	/**
	* Gets the teleportation rotation threshold.
	* 
	* @return Threshold in degrees.
	*/
	UFUNCTION(BlueprintGetter, Category=Clothing)
	ENGINE_API float GetTeleportRotationThreshold() const;

	/**
	* Sets the teleportation rotation threshold.
	* 
	* @param threshold Threshold in degrees.
	*/
	UFUNCTION(BlueprintSetter, Category=Clothing)
	ENGINE_API void SetTeleportRotationThreshold(float Threshold);

	/**
	* Gets the teleportation distance threshold.
	* 
	* @return Threshold value.
	*/
	UFUNCTION(BlueprintGetter, Category=Clothing)
	ENGINE_API float GetTeleportDistanceThreshold() const;

	/**
	* Sets the teleportation distance threshold.
	* 
	* @param threshold Threshold value.
	*/
	UFUNCTION(BlueprintSetter, Category=Clothing)
	ENGINE_API void SetTeleportDistanceThreshold(float Threshold);

private:
	/**
	* Conduct teleportation if the character's movement is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	* You can also do force teleport manually using ForceNextUpdateTeleport() / ForceNextUpdateTeleportAndReset().
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter=GetTeleportDistanceThreshold, BlueprintSetter=SetTeleportDistanceThreshold, Category=Clothing)
	float TeleportDistanceThreshold;

	/**
	* Rotation threshold in degrees, ranging from 0 to 180.
	* Conduct teleportation if the character's rotation is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter=GetTeleportRotationThreshold, BlueprintSetter=SetTeleportRotationThreshold, Category=Clothing)
	float TeleportRotationThreshold;

	/** Used for pre-computation using TeleportRotationThreshold property */
	float ClothTeleportCosineThresholdInRad;
	/** Used for pre-computation using tTeleportDistanceThreshold property */
	float ClothTeleportDistThresholdSquared;

public:

	/** Checked whether we have already ticked the pose this frame */
	ENGINE_API bool PoseTickedThisFrame() const;

	bool IsClothBoundToLeaderComponent() const { return bBindClothToLeaderComponent; }
	
	UE_DEPRECATED(5.1, "This method has been deprecated. Please, use IsClothBoundToLeaderComponent instead.")
	bool IsClothBoundToMasterComponent() const { return IsClothBoundToLeaderComponent(); }

	/** Get the current clothing simulation (read only) */
	UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
	ENGINE_API const class IClothingSimulation* GetClothingSimulation() const;

	/** Get the current clothing simulation (read/write) */
	UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
	ENGINE_API class IClothingSimulation* GetClothingSimulation();

	/** Get the current clothing simulation context (read only) */
	UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
	ENGINE_API const class IClothingSimulationContext* GetClothingSimulationContext() const;
	
	/** Get the current clothing simulation context (read/write) */
	ENGINE_API class IClothingSimulationContext* GetClothingSimulationContext();
	
	/** Get the current clothing simulation instances (read only) */
	const TArray<FClothingSimulationInstance>& GetClothingSimulationInstances() const
	{
		return ClothingSimulationInstances;
	}

	/** Get the current clothing simulation instances (read/write) */
	TArray<FClothingSimulationInstance>& GetClothingSimulationInstances()
	{
		return ClothingSimulationInstances;
	}

	/**
	 * Get the interactor for a clothing simulation, if that simulation supports runtime interaction.
	 * @param ClothingDataName the name of a clothing data that determines the simulation, or NAME_None to get the interactor of the first registered simulation if any.
	 * @note Using NAME_None when there are clothing data of different types on the Skeletal Mesh asset will cause a warning.
	 */
	UFUNCTION(BlueprintCallable, Category = Clothing)
	ENGINE_API UClothingSimulationInteractor* GetClothingSimulationInteractor(const FName ClothingAssetName = NAME_None) const;

	/** Callback when the parallel clothing task finishes, copies needed data back to component for gamethread */
	ENGINE_API void CompleteParallelClothSimulation();

	/** Get the current simulation data map for the clothing on this component. For use on the game thread and only valid if bWaitForParallelClothTask is true. */
	ENGINE_API const TMap<int32, FClothSimulData>& GetCurrentClothingData_GameThread() const;

	/** Get the current simulation data map for the clothing on this component. This will stall until the cloth simulation is complete. */
	ENGINE_API const TMap<int32, FClothSimulData>& GetCurrentClothingData_AnyThread() const;

	/** Stalls on any currently running clothing simulations.*/
	ENGINE_API void WaitForExistingParallelClothSimulation_GameThread();

private:

	/** Let the cloth tick and completion tasks have access to private clothing data */
	friend FSkeletalMeshComponentClothTickFunction;
	friend FSkeletalMeshComponentClothEndTickFunction;
	friend class FParallelClothCompletionTask;

	/** Debug mesh component should be able to access for visualisation */
	friend class UDebugSkelMeshComponent;

	/** Copies the data from the external cloth simulation context. We copy instead of flipping because the API has to return the full struct to make backwards compat easy*/
	ENGINE_API void UpdateClothSimulationContext(float InDeltaTime);

	/** previous root bone matrix to compare the difference and decide to do clothing teleport  */
	FMatrix	PrevRootBoneMatrix;

	/** previous root body scale to compare the difference when the actor scale is updated. */
	FVector3f PrevRootRigidBodyScale;

	/** Clothing simulation instance. */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, SkipSerialization, NonTransactional)
	TArray<FClothingSimulationInstance> ClothingSimulationInstances;

	/** Array of sources for cloth collision */
	TArray<FClothCollisionSource> ClothCollisionSources;

	/** Ref for the clothing parallel task, so we can detect whether or not a sim is running */
	FGraphEventRef ParallelClothTask;

	/** Whether we should stall the Cloth tick task until the cloth simulation is complete. This is required if we want up-to-date
	 * cloth data on the game thread, for example if we want to generate particles at cloth vertices. When the data is not required
	 * except for rendering, we can set this to false to eliminate a potential game thread stall while we wait for the cloth sim 
	 */
	ENGINE_API bool ShouldWaitForClothInTickFunction() const;

	/** Whether the parallel cloth task wait is owned by the TG_LastDemotable end tick. Returns false to fall back to the legacy OnPreEndOfFrameSync wait. */
	bool ShouldUseClothEndTickWait() const;

	/** Stalls on any currently running clothing simulations, needed when changing core sim state, or to access the clothing data */
	ENGINE_API void HandleExistingParallelClothSimulation();

	/** Called by the clothing completion event to perform a writeback of the simulation data 
	 * to the game thread, the task is friended to gain access to this and not allow any
	 * external callers to trigger writebacks
	 */
	ENGINE_API void WritebackClothingSimulationData();

protected:

	/** Simulation data written back to the component after the simulation has taken place. If this data is required
	 * by any system other than rendering, bWaitForParallelClothTask must be true. If bWaitForParallelClothTask is false,
	 * this data cannot be read on the game thread, and there must be a call to HandleExistingParallelClothSimulation() prior
	 * to accessing it. 
	*/
	TMap<int32, FClothSimulData> CurrentSimulationData;

private:

	/** Called by OnRegister/OnPreRegister before calling Super. */
	void InitializeBeforeRegister();

	/** Called by OnRegister/OnPreRegister after calling Super. */
	void InitializeOnRegister();

	/** Shared code between synchronous and asynchronous phyiscs destruction. */
	void DestroyPhysicsStateInternal();

	/** Wrapper that calls our constraint broken delegate */
	ENGINE_API void OnConstraintBrokenWrapper(int32 ConstraintIndex);

	/** Wrapper that calls our constraint plasticity delegate */
	ENGINE_API void OnPlasticDeformationWrapper(int32 ConstraintIndex);

	/** 
	 * Morph Target Curves. This will override AnimInstance MorphTargetCurves
	 * if same curve is found
	 **/
	UPROPERTY(Transient)
	TMap<FName, float>	MorphTargetCurves;

public:
	const TMap<FName, float>& GetMorphTargetCurves() const { return MorphTargetCurves;  }
	// 
	// Animation
	//
	ENGINE_API virtual void InitAnim(bool bForceReinit);
 
	// Broadcast when the components anim instance is initialized
	UPROPERTY(BlueprintAssignable, Category = Animation)
	FOnAnimInitialized OnAnimInitialized;

	/**
		If VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::OnlyTickMontagesWhenNotRendered
		Should we tick Montages only?
	*/
	ENGINE_API bool ShouldOnlyTickMontages(const float DeltaTime) const;

	ENGINE_API bool ShouldOnlyTickMontagesAndRefreshBones(const float DeltaTime) const;

	/** @return whether we should tick animation (we may want to skip it due to URO) */
	ENGINE_API bool ShouldTickAnimation() const;

	/** Tick Animation system */
	ENGINE_API void TickAnimation(float DeltaTime, bool bNeedsValidRootMotion);

	/** Tick all of our anim instances (linked instances, main instance and post process) */
	ENGINE_API void TickAnimInstances(float DeltaTime, bool bNeedsValidRootMotion);

	/** Tick Clothing Animation , basically this is called inside TickComponent */
	ENGINE_API void TickClothing(float DeltaTime, FTickFunction& ThisTickFunction);

	/** Tick only montages on the main anim instance to support legacy montages in UAF */
	ENGINE_API void TickMontagesOnly(float DeltaTime, bool bNeedsValidRootMotion);

	/** Store cloth simulation data into OutClothSimulData. Override USkinnedMeshComponent. */
	ENGINE_API virtual void GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutClothBlendWeight) const override;

	UE_DEPRECATED(5.7, "Use ReleaseAllClothingResources instead.")
	ENGINE_API void RemoveAllClothingActors();

	/** Remove all clothing actors from their simulation and clear any other necessary clothing data to leave the simulations in a clean state */
	ENGINE_API void ReleaseAllClothingResources();

	/**
	 * Draw the currently clothing state, using the editor extender interface
	 * @param PDI The draw interface to use
	 */
	ENGINE_API void DebugDrawClothing(FPrimitiveDrawInterface* PDI);

	/**
	 * Draw the currently clothing state, using the editor extender interface
	 * @param Canvas The canvas to draw the text on
	 * @param SceneView The view to project the text with
	 */
	ENGINE_API void DebugDrawClothingTexts(FCanvas* Canvas, const FSceneView* SceneView);

#if WITH_EDITOR
	ENGINE_API void UpdatePoseWatches();
	TArray<FAnimNodePoseWatch> PoseWatches;
#endif

	/** Changes the value of bNotifyRigidBodyCollision
	* @param bNewNotifyRigidBodyCollision - The value to assign to bNotifyRigidBodyCollision
	*/
	ENGINE_API virtual void SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision) override;

	/** Changes the value of bNotifyRigidBodyCollision for a given body
	* @param bNewNotifyRigidBodyCollision	The value to assign to bNotifyRigidBodyCollision
	* @param BoneName						Name of the body to turn hit notifies on/off. None implies root body
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetBodyNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision, FName BoneName = NAME_None);

	/** Changes the value of bNotifyRigidBodyCollision on all bodies below a given bone
	* @param bNewNotifyRigidBodyCollision	The value to assign to bNotifyRigidBodyCollision
	* @param BoneName						Name of the body to turn hit notifies on (and below)
	* @param bIncludeSelf					Whether to modify the given body (useful for roots with multiple children)
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void SetNotifyRigidBodyCollisionBelow(bool bNewNotifyRigidBodyCollision, FName BoneName = NAME_None, bool bIncludeSelf = true);

	/** 
	 * Recalculates the RequiredBones array in this SkeletalMeshComponent based on current SkeletalMesh, LOD and PhysicsAsset.
	 * Is called when bRequiredBonesUpToDate = false
	 *
	 * @param	LODIndex	Index of LOD [0-(MaxLOD-1)]
	 */
	ENGINE_API void RecalcRequiredBones(int32 LODIndex);

	/** Computes the required bones in this SkeletalMeshComponent based on current SkeletalMesh, LOD and PhysicsAsset
	  * @param	LODIndex	Index of LOD [0-(MaxLOD-1)]
	*/
	ENGINE_API void ComputeRequiredBones(TArray<FBoneIndexType>& OutRequiredBones, TArray<FBoneIndexType>& OutFillComponentSpaceTransformsRequiredBones, int32 LODIndex, bool bIgnorePhysicsAsset) const;
	
	/** Computes a dense bit array of valid bone transforms, based on a list of required bones.
	 * The bit array is sized to the full number of bones of our RefSkeleton.
	 * This can be used to check our component space bone transforms for validity, without iterating over the full required bones list.
	 */
	ENGINE_API void ComputeValidMeshPoseTransforms(TArray<FBoneIndexType>& InRequiredBones, TBitArray<>& OutValidTransforms) const;

	// Get the required virtual bones from the SkeletalMesh Reference Skeleton
	static ENGINE_API void GetRequiredVirtualBones(const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones);

	// Removes the bones expicitly hidden or hiden by parent
	static ENGINE_API void ExcludeHiddenBones(const USkeletalMeshComponent* SkeletalMeshComponent, const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones);

	// Get the bones used for mirroring in the skeletal mesh asset (if any)
	UE_DEPRECATED(5.3, "This method has been deprecated. Please use UMirrorDataTable for mirroring support.")
	static ENGINE_API void GetMirroringRequiredBones(const USkeletalMesh* SkeletalMesh, TArray<FBoneIndexType>& OutRequiredBones);

	// Get the bones required for shadow shapes
	static ENGINE_API void GetShadowShapeRequiredBones(const USkeletalMeshComponent* SkeletalMeshComponent, TArray<FBoneIndexType>& OutRequiredBones);

	/**
	* Recalculates the AnimCurveUids array in RequiredBone of this SkeletalMeshComponent based on current required bone set
	* Is called when Skeleton->IsRequiredCurvesUpToDate() = false
	*/
	ENGINE_API void RecalcRequiredCurves();

public:
	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditImport() override;
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnSkeletalMeshPropertyChangedMulticaster)
	FOnSkeletalMeshPropertyChangedMulticaster OnSkeletalMeshPropertyChanged;
	typedef FOnSkeletalMeshPropertyChangedMulticaster::FDelegate FOnSkeletalMeshPropertyChanged;

	/** Register / Unregister delegates called when the skeletal mesh property is changed */
	ENGINE_API FDelegateHandle RegisterOnSkeletalMeshPropertyChanged(const FOnSkeletalMeshPropertyChanged& Delegate);
	ENGINE_API void UnregisterOnSkeletalMeshPropertyChanged(FDelegateHandle Handle);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSkeletalMeshUnregisterd, const USkeletalMeshComponent*)
	FOnSkeletalMeshUnregisterd OnSkeletalMeshUnregistered;
	typedef FOnSkeletalMeshUnregisterd::FDelegate FOnSkeletalMeshUnregisteredDelegate;

	/** Register / Unregister delegates called when the mesh component is unregistered. */
	ENGINE_API FDelegateHandle RegisterOnSkeletalMeshUnregistered(const FOnSkeletalMeshUnregisteredDelegate& Delegate);
	ENGINE_API void UnregisterOnSkeletalMeshUnregistered(FDelegateHandle Handle);

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Validates the animation asset or blueprint, making sure it is compatible with the current skeleton */
	ENGINE_API void ValidateAnimation();

	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
	ENGINE_API virtual void UpdateCollisionProfile() override;
#endif // WITH_EDITOR
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface.

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual bool HasValidPhysicsState() const override;

protected:
	ENGINE_API virtual void OnPreRegister() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
	ENGINE_API virtual void SendRenderDynamicData_Concurrent() override;
	ENGINE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
public:
	ENGINE_API virtual bool AllowsAsyncPhysicsStateCreation() const override;
	ENGINE_API virtual bool AllowsAsyncPhysicsStateDestruction() const override;
	ENGINE_API virtual void OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual bool OnAsyncCreatePhysicsState(const UE::FTimeout& TimeOut) override;
	ENGINE_API virtual void OnAsyncCreatePhysicsStateEnd_GameThread() override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) override;
	ENGINE_API virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout) override;

	ENGINE_API virtual void InitializeComponent() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void SetComponentTickEnabled(bool bEnabled) override;
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;

#if WITH_EDITOR
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
#endif // WITH_EDITOR

	//Handle registering our end physics tick function
	ENGINE_API virtual void RegisterEndPhysicsTick(bool bRegister);

	ENGINE_API virtual bool RequiresPreEndOfFrameSync() const override;
	ENGINE_API virtual void OnPreEndOfFrameSync() override;

	//Handle registering our pre cloth tick function
	ENGINE_API void RegisterClothTick(bool bRegister);

	//Handle registering our post cloth wait tick function (TG_LastDemotable)
	void RegisterClothEndTick(bool bRegister);

	//~ End UActorComponent Interface.

	//~ Begin USceneComponent Interface.
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual bool IsAnySimulatingPhysics() const override;
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None) override;
	ENGINE_API virtual bool UpdateOverlapsImpl(const TOverlapArrayView* PendingOverlaps=NULL, bool bDoNotifies=true, const TOverlapArrayView* OverlapsAtEndLocation=NULL) override;
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
protected:
	/**
	 *  Test the collision of the supplied component at the supplied location/rotation, and determine the set of components that it overlaps
	 *  @param  OutOverlaps     Array of overlaps found between this component in specified pose and the world
	 *  @param  World			World to use for overlap test
	 *  @param  Pos             Location of the component's geometry for the test against the world
	 *  @param  Rot             Rotation of the component's geometry for the test against the world
	 *  @param  TestChannel		The 'channel' that this ray is in, used to determine which components to hit
	 *	@param	ObjectQueryParams	List of object types it's looking for. When this enters, we do object query with component shape
	 *  @return TRUE if OutOverlaps contains any blocking results
	 */
	ENGINE_API virtual bool ComponentOverlapMultiImpl(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const override;
	
	ENGINE_API virtual bool ComponentOverlapComponentImpl(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Quat, const FCollisionQueryParams& Params) override;

	ENGINE_API virtual bool MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit = NULL, EMoveComponentFlags MoveFlags = MOVECOMP_NoFlags, ETeleportType Teleport = ETeleportType::None) override;

public:

	ENGINE_API virtual class UBodySetup* GetBodySetup() override;
	ENGINE_API virtual bool CanEditSimulatePhysics() override;
	ENGINE_API virtual bool IsSimulatingPhysics(FName BoneName = NAME_None) const override;
	ENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	ENGINE_API virtual void UpdatePhysicsToRBChannels() override;
	ENGINE_API virtual void SetAllPhysicsAngularVelocityInRadians(FVector const& NewVel, bool bAddToCurrent = false) override;
	ENGINE_API virtual void SetAllPhysicsPosition(FVector NewPos) override;
	ENGINE_API virtual void SetAllPhysicsRotation(FRotator NewRot) override;
	ENGINE_API virtual void SetAllPhysicsRotation(const FQuat& NewRot) override;
	ENGINE_API virtual void WakeAllRigidBodies() override;
	ENGINE_API virtual void PutAllRigidBodiesToSleep() override;
	ENGINE_API virtual bool IsAnyRigidBodyAwake() override;
	ENGINE_API virtual void SetEnableGravity(bool bGravityEnabled);
	ENGINE_API virtual bool IsGravityEnabled() const override;
	ENGINE_API virtual void OnComponentCollisionSettingsChanged(bool bUpdateOverlaps=true) override;
	ENGINE_API virtual void SetPhysMaterialOverride(UPhysicalMaterial* NewPhysMaterial) override;
	ENGINE_API virtual bool GetSquaredDistanceToCollision(const FVector& Point, float& OutSquaredDistance, FVector& OutClosestPointOnCollision) const override;

	/**
	 *	Enables or disables per poly collision. This will recreate the physics state.
	 *
	 *	@param bEnablePerPolyCollision	Whether per poly collision should be enabled or disabled.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetEnablePerPolyCollision(bool bInEnablePerPolyCollision);

	/**
	 *	Enables or disables gravity for the given bone.
	 *	NAME_None indicates the root body will be edited.
	 *	If the bone name given is otherwise invalid, nothing happens.
	 *
	 *	@param bEnableGravity	Whether gravity should be enabled or disabled.
	 *	@param BoneName			The name of the bone to modify.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetEnableBodyGravity(bool bEnableGravity, FName BoneName);

	/**
	 *	Checks whether or not gravity is enabled on the given bone.
	 *	NAME_None indicates the root body should be queried.
	 *	If the bone name given is otherwise invalid, false is returned.
	 *
	 *	@param BoneName	The name of the bone to check.
	 *	@return True if gravity is enabled on the bone.
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API bool IsBodyGravityEnabled(FName BoneName);

	/**
	*	Enables or disables gravity to all bodies below the given bone.
	*   NAME_None indicates all bodies will be edited.
	*	In that case, consider using UPrimitiveComponent::EnableGravity.
	*
	*	@param bEnableGravity	Whether gravity should be enabled or disabled.
	*	@param BoneName			The name of the top most bone.
	*	@param bIncludeSelf		Whether the bone specified should be edited.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetEnableGravityOnAllBodiesBelow(bool bEnableGravity, FName BoneName, bool bIncludeSelf=true);

	/** 
	 *	Given a world position, find the closest point on the physics asset. Note that this is independent of collision and welding. This is based purely on animation position
	 *  @param	WorldPosition				The point we want the closest point to (i.e. for all bodies in the physics asset, find the one that has a point closest to WorldPosition)
	 *  @param	ClosestPointOnPhysicsAsset	The data associated with the closest point (position, normal, etc...)
	 *  @param	bApproximate				The closest body is found using bone transform distance instead of body distance. This approximation means the final point is the closest point on a potentially not closest body. This approximation gets worse as the size of Bodies gets bigger.
	 *  @return	true if we found a closest point
	 */
	ENGINE_API bool GetClosestPointOnPhysicsAsset(const FVector& WorldPosition, FClosestPointOnPhysicsAsset& ClosestPointOnPhysicsAsset, bool bApproximate) const;

	/** 
	 *	Given a world position, find the closest point on the physics asset. Note that this is independent of collision and welding. This is based purely on animation position
	 *  @param	WorldPosition				The point we want the closest point to (i.e. for all bodies in the physics asset, find the one that has a point closest to WorldPosition)
	 *  @param	ClosestPointOnPhysicsAsset	The data associated with the closest point (position, normal, etc...)
	 *  @return	true if we found a closest point
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh", meta=(DisplayName="Get Closest Point On Physics Asset", ScriptName="GetClosestPointOnPhysicsAsset", Keywords="closest point"))
	ENGINE_API bool K2_GetClosestPointOnPhysicsAsset(const FVector& WorldPosition, FVector& ClosestWorldPosition, FVector& Normal, FName& BoneName, float& Distance) const;

	ENGINE_API virtual bool LineTraceComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params ) override;
	
	/** 
	 *  Trace a shape against just this component. Will trace against each body, returning as soon as any collision is found. Note that this collision may not be the closest.
	 *  @param  OutHit          	Information about hit against this component, if true is returned
	 *  @param  Start           	Start location of the trace
	 *  @param  End             	End location of the trace
	 *  @param  ShapeWorldRotation  The rotation applied to the collision shape in world space
	 *  @param  CollisionShape  	Collision Shape
	 *	@param	bTraceComplex	Whether or not to trace complex
	 *  @return true if a hit is found
	 */
	 ENGINE_API virtual bool SweepComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapRotation, const FCollisionShape& CollisionShape, bool bTraceComplex=false) override;
	
	ENGINE_API virtual bool OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const override;
	ENGINE_API virtual void SetSimulatePhysics(bool bEnabled) override;
	ENGINE_API virtual void AddRadialImpulse(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange=false) override;
	ENGINE_API virtual void AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange=false) override;
	ENGINE_API virtual void SetAllPhysicsLinearVelocity(FVector NewVel,bool bAddToCurrent = false) override;
	ENGINE_API virtual void SetAllMassScale(float InMassScale = 1.f) override;
	ENGINE_API virtual float GetMass() const override;
	ENGINE_API virtual void SetAllUseCCD(bool InUseCCD) override;

	/**
	*	Returns the mass (in kg) of the given bone
	*
	*	@param BoneName		Name of the body to return. 'None' indicates root body.
	*	@param bScaleMass	If true, the mass is scaled by the bone's MassScale.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API float GetBoneMass(FName BoneName = NAME_None, bool bScaleMass = true) const;

	/**
	*	Returns the center of mass of the skeletal mesh, instead of the root body's location
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API FVector GetSkeletalCenterOfMass() const;


	ENGINE_API virtual float CalculateMass(FName BoneName = NAME_None) override;
	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	/**
	*	Add a force to all rigid bodies below.
	*   This is like a 'thruster'. Good for adding a burst over some (non zero) time. Should be called every frame for the duration of the force.
	*
	*	@param	Force		 Force vector to apply. Magnitude indicates strength of force.
	*	@param	BoneName	 If a SkeletalMeshComponent, name of body to apply force to. 'None' indicates root body.
	*   @param  bAccelChange If true, Force is taken as a change in acceleration instead of a physical force (i.e. mass will have no effect).
	*   @param  bIncludeSelf If false, Force is only applied to bodies below but not given bone name.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void AddForceToAllBodiesBelow(FVector Force, FName BoneName = NAME_None, bool bAccelChange = false, bool bIncludeSelf = true);

	/**
	*	Add impulse to all single rigid bodies below. Good for one time instant burst.
	*
	*	@param	Impulse		Magnitude and direction of impulse to apply.
	*	@param	BoneName	If a SkeletalMeshComponent, name of body to apply impulse to. 'None' indicates root body.
	*	@param	bVelChange	If true, the Strength is taken as a change in velocity instead of an impulse (ie. mass will have no effect).
	*	@param bIncludeSelf If false, Force is only applied to bodies below but not given bone name.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API virtual void AddImpulseToAllBodiesBelow(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false, bool bIncludeSelf = true);

	ENGINE_API virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;
#if WITH_EDITOR
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
protected:
	/**
	 * If PhysicsTransformUpdateMode is set to SimulationUpdatesComponentTransform, this returns the 
	 * world-space transform of the body passed in (assumed to be the root body), after applying the 
	 * TransformToRoot offset. Otherwise, it returns the component transform.
	 */
	ENGINE_API virtual FTransform GetComponentTransformFromBodyInstance(FBodyInstance* UseBI) override;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void MigratePhysicsState(FArchive& Ar) override;
	ENGINE_API virtual void ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData = nullptr) override;
#endif

	//~ End UPrimitiveComponent Interface.

public:
	//~ Begin USkinnedMeshComponent Interface
	ENGINE_API virtual bool UpdateLODStatus() override;
	ENGINE_API virtual void SetPredictedLODLevel(int32 InPredictedLODLevel) override;
	ENGINE_API virtual void UpdateVisualizeLODString(FString& DebugString) override;
	ENGINE_API virtual void RefreshBoneTransforms( FActorComponentTickFunction* TickFunction = NULL ) override;
protected:
	ENGINE_API virtual bool AreBoneTransformsValid() const override;
	ENGINE_API virtual void DispatchParallelTickPose( FActorComponentTickFunction* TickFunction ) override;
public:
	ENGINE_API virtual void TickPose(float DeltaTime, bool bNeedsValidRootMotion) override;
	ENGINE_API virtual void UpdateFollowerComponent() override;
	UE_DEPRECATED(5.1, "This method has been deprecated. Please use UpdateFollowerComponent instead.")
	virtual void UpdateSlaveComponent() override { UpdateFollowerComponent(); };
	ENGINE_API virtual bool ShouldUpdateTransform(bool bLODHasChanged) const override;
	ENGINE_API virtual bool ShouldTickPose() const override;
	ENGINE_API virtual bool AllocateTransformData() override;
	ENGINE_API virtual void DeallocateTransformData() override;
	ENGINE_API virtual void HideBone( int32 BoneIndex, EPhysBodyOp PhysBodyOption ) override;
	ENGINE_API virtual void UnHideBone( int32 BoneIndex ) override;
	ENGINE_API virtual void SetPhysicsAsset(class UPhysicsAsset* NewPhysicsAsset,bool bForceReInit = false) override;
	ENGINE_API virtual void SetSkeletalMesh(class USkeletalMesh* NewMesh, bool bReinitPose = true) override;  // SetSkeletalMesh may remain and become a UFUNCTION but lose its virtual after it is removed from the SkinnedMeshComponent API

	/**
	 *  Set the new asset to render and update this component (this function is identical to SetSkeletalMesh).
	 *  The USkinnedAsset pointer is first cast to a USkeletalMesh, therefore this function will only set assets of type USkeletalMesh.
	 * 
	 *  @param InSkinnedAsset The new asset.
	 *  @param bReinitPose    Whether to re-initialize the animation.
	 */
	ENGINE_API virtual void SetSkinnedAssetAndUpdate(class USkinnedAsset* InSkinnedAsset, bool bReinitPose = true) override;

	static ENGINE_API FVector3f GetSkinnedVertexPosition(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& Model, const FSkinWeightVertexBuffer& SkinWeightBuffer);
	static ENGINE_API FVector3f GetSkinnedVertexPosition(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& Model, const FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals);
	static ENGINE_API void ComputeSkinnedPositions(USkeletalMeshComponent* Component, TArray<FVector3f> & OutPositions, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& Model, const FSkinWeightVertexBuffer& SkinWeightBuffer);

	static ENGINE_API void GetSkinnedTangentBasis(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& Model, const FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ);
	static ENGINE_API void ComputeSkinnedTangentBasis(USkeletalMeshComponent* Component, TArray<FVector3f>& OutTangenXYZ, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& Model, const FSkinWeightVertexBuffer& SkinWeightBuffer);

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use SetSkeletalMesh(NewMesh, false) instead.")
	ENGINE_API void SetSkeletalMeshWithoutResettingAnimation(class USkeletalMesh* NewMesh);

	ENGINE_API virtual bool IsPlayingRootMotion() const override;
	ENGINE_API virtual bool IsPlayingNetworkedRootMotionMontage() const override;
	ENGINE_API virtual bool IsPlayingRootMotionFromEverything() const override;
	ENGINE_API virtual void FinalizeBoneTransform() override;
	ENGINE_API virtual void SetRefPoseOverride(const TArray<FTransform>& NewRefPoseTransforms) override;
	ENGINE_API virtual void ClearRefPoseOverride() override;
	//~ End USkinnedMeshComponent Interface

	// Conditions used to gate when post process events happen
	ENGINE_API bool ShouldEvaluatePostProcessAnimBP() const;
	ENGINE_API bool ShouldUpdatePostProcessInstance() const;
	ENGINE_API bool ShouldPostUpdatePostProcessInstance() const;
	ENGINE_API bool ShouldEvaluatePostProcessInstance() const;

	/** 
	 *	Iterate over each joint in the physics for this mesh, setting its AngularPositionTarget based on the animation information.
	 */
	ENGINE_API void UpdateRBJointMotors();

	/**
	* Runs the animation evaluation for the current pose into the supplied variables
	* PerformAnimationProcessing runs evaluation based on bInDoEvaluation. PerformAnimationEvaluation 
	* always runs evaluation (and exists for backward compatibility)
	*
	* @param	InSkeletalMesh			The skeletal mesh we are animating
	* @param	InAnimInstance			The anim instance we are evaluating
	* @param	bInDoEvaluation			Whether to perform evaluation (we may just want to update)
	* @param	OutSpaceBases			Component space bone transforms
	* @param	OutBoneSpaceTransforms	Local space bone transforms
	* @param	OutRootBoneTranslation	Calculated root bone translation
	* @param	OutCurves				Blended Curve
	*/
#if WITH_EDITOR
	ENGINE_API void PerformAnimationEvaluation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes);

	UE_DEPRECATED(4.26, "Please use PerformAnimationEvaluation with different signature")
	ENGINE_API void PerformAnimationEvaluation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve);
#endif

	ENGINE_API void PerformAnimationProcessing(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInDoEvaluation, bool bInForceRefPose, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes);

	UE_DEPRECATED(5.5, "Please use PerformAnimationEvaluation with different signature")
	ENGINE_API void PerformAnimationProcessing(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInDoEvaluation, TArray<FTransform>& OutSpaceBases, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, UE::Anim::FMeshAttributeContainer& OutAttributes);

	/**
	 * Evaluates the post process instance from the skeletal mesh this component is using.
	 */
	ENGINE_API void EvaluatePostProcessMeshInstance(TArray<FTransform>& OutBoneSpaceTransforms, FCompactPose& InOutPose, FBlendedHeapCurve& OutCurve, const USkeletalMesh* InSkeletalMesh, FVector& OutRootBoneTranslation, UE::Anim::FHeapAttributeContainer& OutAttributes, bool bInForceRefPose) const;

	UE_DEPRECATED(5.5, "Please use EvaluatePostProcessMeshInstance with different signature")
	ENGINE_API void EvaluatePostProcessMeshInstance(TArray<FTransform>& OutBoneSpaceTransforms, FCompactPose& InOutPose, FBlendedHeapCurve& OutCurve, const USkeletalMesh* InSkeletalMesh, FVector& OutRootBoneTranslation, UE::Anim::FHeapAttributeContainer& OutAttributes) const;

	ENGINE_API void PostAnimEvaluation(FAnimationEvaluationContext& EvaluationContext);

	/** */
	ENGINE_API void InitCollisionRelationships();

	/** */
	ENGINE_API void TermCollisionRelationships();

	/**
	 * Blend of Physics Bones with PhysicsWeight and Animated Bones with (1-PhysicsWeight)
	 *
	 * @param	RequiredBones	List of bones to be blend
	 */
	UE_DEPRECATED(4.26, "This function is deprecated and should not be called directly. Please use the mechanism provided in USkeletalMeshComponent::EndPhysicsTickComponent")
	void BlendPhysicsBones( TArray<FBoneIndexType>& Bones )
	{
		PerformBlendPhysicsBones(Bones, AnimEvaluationContext.BoneSpaceTransforms, AnimEvaluationContext.BoneSpaceTransforms);
	}


	/** Take the results of the physics and blend them with the animation state (based on the PhysicsWeight parameter), and update the SpaceBases array. */
	UE_DEPRECATED(4.26, "Public access to this function is deprecated. Please use the mechanism provided in USkeletalMeshComponent::EndPhysicsTickComponent")
	void BlendInPhysics(FTickFunction& ThisTickFunction) { BlendInPhysicsInternal(ThisTickFunction); }

	/** 
	 * Initialize PhysicsAssetInstance for the physicsAsset 
	 * 
	 * @param	PhysScene	Physics Scene
	 */
	ENGINE_API void InitArticulated(FPhysScene* PhysScene);

	/** Instantiates bodies given a physics asset. Typically you should call InitArticulated unless you are planning to do something special with the bodies. The Created bodies and constraints are owned by the calling code and must be freed when necessary.*/
	ENGINE_API void InstantiatePhysicsAsset(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle()) const;

	/** Instantiates only the bodies given a physics asset, not the constraints. The Created bodies are owned by the calling code and must be freed when necessary.*/
	ENGINE_API void InstantiatePhysicsAssetBodies(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle()) const;

	/** Instantiates only the bodies given a physics asset, not the constraints. Instead of reading the current component state, this reads the ref-pose from the reference skeleton of the mesh. Useful if trying to create bodies to be used during any evaluation work. The Created bodies are owned by the calling code and must be freed when necessary.*/
	ENGINE_API void InstantiatePhysicsAssetBodiesRefPose(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle()) const;

	/** Instantiates bodies given a physics asset like InstantiatePhysicsAsset but instead of reading the current component state, this reads the ref-pose from the reference skeleton of the mesh. Useful if trying to create bodies to be used during any evaluation work */
	ENGINE_API void InstantiatePhysicsAssetRefPose(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle(), bool bCreateBodiesInRefPose = false) const;

	/** Turn off all physics and remove the instance. */
	ENGINE_API void TermArticulated();

	/** Find the root body index*/
	ENGINE_API int32 FindRootBodyIndex() const;

	/** Terminate physics on all bodies below the named bone, effectively disabling collision forever. If you terminate, you won't be able to re-init later. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void TermBodiesBelow(FName ParentBoneName);

	/** Find instance of the constraint that matches the name supplied. */
	ENGINE_API FConstraintInstance* FindConstraintInstance(FName ConName);

	/** get instance of the constraint that matches the index . */
	ENGINE_API FConstraintInstance* GetConstraintInstanceByIndex(uint32 Index);

	/** Utility which returns total mass of all bones below the supplied one in the hierarchy (including this one). */
	ENGINE_API float GetTotalMassBelowBone(FName InBoneName);

	/** Set the collision object type on the skeletal mesh */
	ENGINE_API virtual void SetCollisionObjectType(ECollisionChannel Channel) override;

	/** Set the movement channel of all bodies */
	ENGINE_API void SetAllBodiesCollisionObjectType(ECollisionChannel NewChannel);

	/** Set the rigid body notification state for all bodies. */
	ENGINE_API void SetAllBodiesNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision);

	/** Set bSimulatePhysics to true for all bone bodies. Does not change the component bSimulatePhysics flag. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetAllBodiesSimulatePhysics(bool bNewSimulate);

	/** This is global set up for setting physics blend weight
	 * This does multiple things automatically
	 * If PhysicsBlendWeight == 1.f, it will enable Simulation, and if PhysicsBlendWeight == 0.f, it will disable Simulation. 
	 * Also it will respect each body's setup, so if the body is fixed, it won't simulate. Vice versa
	 * So if you'd like all bodies to change manually, do not use this function, but SetAllBodiesPhysicsBlendWeight
	 */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetPhysicsBlendWeight(float PhysicsBlendWeight);

	/** Disable physics blending of bones **/
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetEnablePhysicsBlending(bool bNewBlendPhysics);

	/** [WARNING: Chaos Only] 
	* Set all of the bones below passed in bone to be disabled or not for the associated physics solver 
	* Bodies will not be colliding or be part of the physics simulation. 
	* This is different from SetAllBodiesBelowSimulatePhysics that changes bodies to Kinematic/simulated 	
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetAllBodiesBelowPhysicsDisabled(const FName& InBoneName, bool bDisabled, bool bIncludeSelf = true);

	/** set the linear velocity of the child bodies to match that of the given parent bone */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetAllBodiesBelowLinearVelocity(const FName& InBoneName, const FVector& LinearVelocity, bool bIncludeSelf = true);

	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API FVector GetBoneLinearVelocity(const FName& InBoneName);

	/** Set all of the bones below passed in bone to be simulated */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetAllBodiesBelowSimulatePhysics(const FName& InBoneName, bool bNewSimulate, bool bIncludeSelf = true );

	/** Set a single bone to be simulated (or not) */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetBodySimulatePhysics(const FName& InBoneName, bool bSimulate);

	/** Allows you to reset bodies Simulate state based on where bUsePhysics is set to true in the BodySetup. */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void ResetAllBodiesSimulatePhysics();

	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetAllBodiesPhysicsBlendWeight(float PhysicsBlendWeight, bool bSkipCustomPhysicsType = false );

	/** Set all of the bones below passed in bone to be simulated */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void SetAllBodiesBelowPhysicsBlendWeight(const FName& InBoneName, float PhysicsBlendWeight, bool bSkipCustomPhysicsType = false, bool bIncludeSelf = true );

	/** Accumulate AddPhysicsBlendWeight to physics blendweight for all of the bones below passed in bone to be simulated */
	UFUNCTION(BlueprintCallable, Category="Physics")
	ENGINE_API void AccumulateAllBodiesBelowPhysicsBlendWeight(const FName& InBoneName, float AddPhysicsBlendWeight, bool bSkipCustomPhysicsType = false );

	/** Enable or Disable AngularPositionDrive. If motor is in SLERP mode it will be turned on if either bEnableSwingDrive OR bEnableTwistDrive are enabled. In Twist and Swing mode the twist and the swing can be controlled individually.*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetAllMotorsAngularPositionDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, bool bSkipCustomPhysicsType = false);

	/** Enable or Disable AngularVelocityDrive. If motor is in SLERP mode it will be turned on if either bEnableSwingDrive OR bEnableTwistDrive are enabled. In Twist and Swing mode the twist and the swing can be controlled individually.*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetAllMotorsAngularVelocityDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, bool bSkipCustomPhysicsType = false);

	/** Set Angular Drive motors params for all constraint instances */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetAllMotorsAngularDriveParams(float InSpring, float InDamping, float InForceLimit, bool bSkipCustomPhysicsType = false);

	/** Sets the constraint profile properties (limits, motors, etc...) to match the constraint profile as defined in the physics asset. If profile name is not found the joint is set to use the default constraint profile.*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetConstraintProfile(FName JointName, FName ProfileName, bool bDefaultIfNotFound = false);

	/** Sets the constraint profile properties (limits, motors, etc...) to match the constraint profile as defined in the physics asset for all constraints. If profile name is not found the joint is set to use the default constraint profile.*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void SetConstraintProfileForAll(FName ProfileName, bool bDefaultIfNotFound = false);

	/**
	 * Gets the constraint profile properties that a joint drive would adopt if it were set to 
	 * the given constraint profile. The default will be returned if an empty or invalid profile name 
	 * is passed in. Returns true if the successful, or false if the joint can't be found.
	 */
	ENGINE_API bool GetConstraintProfilePropertiesOrDefault(FConstraintProfileProperties& OutProperties, FName JointName, FName ProfileName);

	/** Enable or Disable AngularPositionDrive based on a list of bone names. If motor is in SLERP mode it will be turned on/off if either bEnableSwingDrive OR bEnableTwistDrive are enabled.   */
	ENGINE_API void SetNamedMotorsAngularPositionDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, const TArray<FName>& BoneNames, bool bSetOtherBodiesToComplement = false);

	/** Enable or Disable AngularVelocityDrive based on a list of bone names. If motor is in SLERP mode it will be turned on/off if either bEnableSwingDrive OR bEnableTwistDrive are enabled.   */
	ENGINE_API void SetNamedMotorsAngularVelocityDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, const TArray<FName>& BoneNames, bool bSetOtherBodiesToComplement = false);

	ENGINE_API void GetWeldedBodies(TArray<FBodyInstance*> & OutWeldedBodies, TArray<FName> & OutChildrenLabels, bool bIncludingAutoWeld = false) override;

	/** Iterates over all bodies below and executes Func. Returns number of bodies found */
	ENGINE_API int32 ForEachBodyBelow(FName BoneName, bool bIncludeSelf, bool bSkipCustomType, TFunctionRef<void(FBodyInstance*)> Func);

	/** 
	 * Change whether to force mesh into ref pose (and use cheaper vertex shader) 
	 *
	 * @param	bNewForceRefPose	true if it would like to force ref pose
	 */
	ENGINE_API void SetForceRefPose(bool bNewForceRefPose);
	
	/** Update bHasValidBodies flag */
	ENGINE_API void UpdateHasValidBodies();

	/** Update the bone mapping on each body instance. This is useful when changing skeletal mesh without recreating bodies */
	ENGINE_API void UpdateBoneBodyMapping();
	
	/** 
	 * Initialize SkelControls
	 */
	ENGINE_API void InitSkelControls();
	
	/**
	 * Find Constraint Index from the name
	 * 
	 * @param	ConstraintName	Joint Name of constraint to look for
	 * @return	Constraint Index
	 */
	ENGINE_API int32	FindConstraintIndex(FName ConstraintName);
	
	/**
	 * Find Constraint Name from index
	 * 
	 * @param	ConstraintIndex	Index of constraint to look for
	 * @return	Constraint Joint Name
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API FName	FindConstraintBoneName(int32 ConstraintIndex);

	/** 
	 *	Iterate over each physics body in the physics for this mesh, and for each 'kinematic' (ie fixed or default if owner isn't simulating) one, update its
	 *	transform based on the animated transform. This update is also done for simulating bodies if a teleport is being requested. 
	 *	@param	InComponentSpaceTransforms	Array of bone transforms in component space
	 *	@param	Teleport					Whether movement is a 'teleport' (ie infers no physics velocity, but moves simulating bodies) or not
	 *	@param	bNeedsSkinning				Whether we may need  to send new triangle data for per-poly skeletal mesh collision
	 *	@param	AllowDeferral				Whether we can defer actual update of bodies (if 'physics only' collision)
	 */
	ENGINE_API void UpdateKinematicBonesToAnim(const TArray<FTransform>& InComponentSpaceTransforms, ETeleportType Teleport, bool bNeedsSkinning, EAllowKinematicDeferral DeferralAllowed = EAllowKinematicDeferral::AllowDeferral);

	/**
	 * Look up all bodies for broken constraints.
	 * Makes sure child bodies of a broken constraints are not fixed and using bone springs, and child joints not motorized.
	 */
	ENGINE_API void UpdateMeshForBrokenConstraints();
	
	/**
	 * Notifier when look at control goes beyond of limit - candidate for delegate
	 */
	ENGINE_API virtual void NotifySkelControlBeyondLimit(class USkelControlLookAt* LookAt);

	/** 
	 * Break a constraint off a Gore mesh. 
	 * 
	 * @param	Impulse	vector of impulse
	 * @param	HitLocation	location of the hit
	 * @param	InBoneName	Name of bone to break constraint for
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (Keywords = "Constraint"))
	ENGINE_API void BreakConstraint(FVector Impulse, FVector HitLocation, FName InBoneName);

	/** Gets a constraint by its name 
	* @param ConstraintName		name of the constraint
	* @param IncludesTerminated whether or not to return a terminated constraint
	* */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (Keywords = "Components|SkeletalMesh"))
	ENGINE_API FConstraintInstanceAccessor GetConstraintByName(FName ConstraintName, bool bIncludesTerminated);

	/** Gets all the constraints
	* @param IncludesTerminated whether or not to return terminated constraints
	* @param OutConstraints returned list of constraints matching the parameters
	* */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (Keywords = "Components|SkeletalMesh"))
	ENGINE_API void GetConstraints(bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints);

	/** Gets all the constraints attached to a body 
	* @param BodyName name of the body to get the attached constraints from 
	* @param bParentConstraints return constraints where BodyName is the child of the constraint
	* @param bChildConstraints return constraints where BodyName is the parent of the constraint
	* @param bParentConstraints return constraints attached to the parent of the body 
	* @param IncludesTerminated whether or not to return terminated constraints
	* @param OutConstraints returned list of constraints matching the parameters
	* */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (Keywords = "Components|SkeletalMesh"))
	ENGINE_API void GetConstraintsFromBody(FName BodyName, bool bParentConstraints, bool bChildConstraints, bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints);

	/** Sets the Angular Motion Ranges for a named constraint
	*  @param InBoneName  Name of bone to adjust constraint ranges for
	*  @param Swing1LimitAngle	 Size of limit in degrees, 0 means locked, 180 means free
	*  @param TwistLimitAngle	 Size of limit in degrees, 0 means locked, 180 means free
	*  @param Swing2LimitAngle	 Size of limit in degrees, 0 means locked, 180 means free
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void  SetAngularLimits(FName InBoneName,float Swing1LimitAngle, float TwistLimitAngle, float Swing2LimitAngle);

	/** Gets the current Angular state for a named bone constraint 
	*  @param InBoneName  Name of bone to get constraint ranges for
	*  @param Swing1Angle current angular state of the constraint
	*  @param TwistAngle  current angular state of the constraint
	*  @param Swing2Angle current angular state of the constraint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	ENGINE_API void GetCurrentJointAngles(FName InBoneName,float& Swing1Angle, float& TwistAngle, float& Swing2Angle) ;


	/** iterates through all bodies in our PhysicsAsset and returns the location of the closest bone associated
	 * with a body that has collision enabled.
	 * @param TestLocation - location to check against
	 * @return location of closest colliding rigidbody, or TestLocation if there were no bodies to test
	 */
	ENGINE_API FVector GetClosestCollidingRigidBodyLocation(const FVector& TestLocation) const;
	
	/** Set physics transforms for all bodies */
	ENGINE_API void ApplyDeltaToAllPhysicsTransforms(const FVector& DeltaLocation, const FQuat& DeltaRotation);

	/** Destroys and recreates the clothing actors in the current simulation */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API void RecreateClothingActors();

	/** Given bounds InOutBounds, expand them to also enclose the clothing simulation mesh */
	ENGINE_API void AddClothingBounds(FBoxSphereBounds& InOutBounds, const FTransform& LocalToWorld) const;

	/** Check linear and angular thresholds for clothing teleport */
	ENGINE_API virtual void CheckClothTeleport();

	/** Update the clothing simulation state and trigger the simulation task */
	ENGINE_API void UpdateClothStateAndSimulate(float DeltaTime, FTickFunction& ThisTickFunction);
	
	/** 
	 * Updates cloth collision outside the cloth asset (environment collision, child collision, etc...)
	 * Should be called when scene changes or world position changes
	 */
	ENGINE_API void UpdateClothTransform(ETeleportType TeleportType);

	/** Update rigid body scaling when the actor is set with a new uniform scale. */
	void UpdateRigidBodyScaling(ETeleportType TeleportType);

	/** Set the cloth transform update to trigger with no teleport option. */
	void UpdateClothTransform() { UpdateClothTransform(ETeleportType::None); }

	/**
	 * Updates cloth collision inside the cloth asset (from a physics asset).
	 * Should be called when the physics asset changes and the effects are needed straight away.
	 */
	ENGINE_API void UpdateClothCollision();

	/** if the vertex index is valid for simulated vertices, returns the position in world space */
	ENGINE_API bool GetClothSimulatedPosition_GameThread(const FGuid& AssetGuid, int32 VertexIndex, FVector& OutSimulPos) const;

	const TArray<FClothCollisionSource>& GetClothCollisionSources() const { return ClothCollisionSources; }

	/**
	 * Add a collision source for the cloth on this component.
	 * Each cloth tick, the collision defined by the physics asset, transformed by the bones in the source
	 * component, will be applied to cloth.
	 * @param	InSourceComponent		The component to extract collision transforms from
	 * @param	InSourcePhysicsAsset	The physics asset that defines the collision primitives (that will be transformed by InSourceComponent's bones)
	 */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API void AddClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset);

	/** Remove a cloth collision source defined by a component */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API void RemoveClothCollisionSources(USkeletalMeshComponent* InSourceComponent);

	/** Remove a cloth collision source defined by a component */
	UE_DEPRECATED(5.5, "This function has been renamed RemoveClothCollisionSources")
	void RemoveClothCollisionSource(USkeletalMeshComponent* InSourceComponent)
	{
		RemoveClothCollisionSources(InSourceComponent);
	}

	/** Remove a cloth collision source defined by both a component and a physics asset */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API void RemoveClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset);

	/** Remove all cloth collision sources */
	UFUNCTION(BlueprintCallable, Category = "Clothing")
	ENGINE_API void ResetClothCollisionSources();

protected:

#if WITH_CLOTH_COLLISION_DETECTION
	/** copy cloth collision sources to this, where parent means components above it in the hierarchy */
	ENGINE_API void CopyClothCollisionSources();

	ENGINE_API void ProcessClothCollisionWithEnvironment();

	/** copy parent's cloth collisions to attached children, where parent means this component */
	UE_DEPRECATED(5.8, "CopyClothCollisionsToChildren has been deprecated. Use CollisionSources to add external collisions instead.")
	void CopyClothCollisionsToChildren() {}

	/** copy children's cloth collisions to parent, where parent means this component */
	UE_DEPRECATED(5.8, "CopyChildrenClothCollisionsToParent has been deprecated. Use CollisionSources to add external collisions instead.")
	void CopyChildrenClothCollisionsToParent(){}

	/** find if this component has collisions for clothing and return the results calculated by bone transforms */
	UE_DEPRECATED(5.8, "FindClothCollisions has been deprecated. Use CollisionSources to add external collisions instead.")
	void FindClothCollisions(FClothCollisionData& OutCollisions) const {}

#endif

public:
	ENGINE_API bool IsAnimBlueprintInstanced() const;
	ENGINE_API void ClearAnimScriptInstance();

	/** Clear cached animation data generated for URO during evaluation */
	ENGINE_API void ClearCachedAnimProperties();

protected:
	ENGINE_API bool NeedToSpawnAnimScriptInstance() const;
	ENGINE_API bool NeedToSpawnPostPhysicsInstance(bool bForceReinit) const;

	ENGINE_API virtual bool ShouldBlendPhysicsBones() const;

	UE_DEPRECATED(5.7, "Use ClothCollisionSource::ExtractCollisions() instead.")
	static ENGINE_API void ExtractCollisionsForCloth(USkeletalMeshComponent* SourceComponent, UPhysicsAsset* PhysicsAsset, USkeletalMeshComponent* DestClothComponent, FClothCollisionData& OutCollisions, FClothCollisionSource& ClothCollisionSource);

	/** Notify called just before syncing physics update, called only if bNotifySyncComponentToRBPhysics flag is set */
	virtual void OnSyncComponentToRBPhysics() { }

	FSkeletalMeshComponentEndPhysicsTickFunction EndPhysicsTickFunction;

private:

	virtual void OnClearAnimScriptInstance() {};

	friend struct FSkeletalMeshComponentEndPhysicsTickFunction;

	/** Update systems after physics sim is done */
	ENGINE_API void EndPhysicsTickComponent(FSkeletalMeshComponentEndPhysicsTickFunction& ThisTickFunction);

	/** Evaluate Anim System **/
	ENGINE_API void EvaluateAnimation(const USkeletalMesh* InSkeletalMesh, UAnimInstance* InAnimInstance, bool bInForceRefPose, FVector& OutRootBoneTranslation, FBlendedHeapCurve& OutCurve, FCompactPose& OutPose, UE::Anim::FHeapAttributeContainer& OutAttributes) const;

	/** Queues up tasks for parallel update/evaluation, as well as the chained game thread completion task */
	ENGINE_API void DispatchParallelEvaluationTasks(FActorComponentTickFunction* TickFunction);

	/** Performs parallel eval/update work, but on the game thread */
	ENGINE_API void DoParallelEvaluationTasks_OnGameThread();

	/** Swaps buffers into the evaluation context before and after task dispatch */
	ENGINE_API void SwapEvaluationContextBuffers();

	/** Duplicates cached transforms/curves and performs interpolation */
	ENGINE_API void ParallelDuplicateAndInterpolate(FAnimationEvaluationContext& InAnimEvaluationContext);

	ENGINE_API bool DoAnyPhysicsBodiesHaveWeight() const;

	ENGINE_API virtual void RefreshMorphTargets() override;

	ENGINE_API void GetWindForCloth_GameThread(FVector& WindVector, float& WindAdaption) const;

	ENGINE_API void InstantiatePhysicsAsset_Internal(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, TFunctionRef<FTransform(int32)> BoneTransformGetter, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle()) const;
	ENGINE_API void InstantiatePhysicsAssetBodies_Internal(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, TFunctionRef<FTransform(int32)> BoneTransformGetter, TMap<FName, FBodyInstance*>* OutNameToBodyMap = nullptr, FPhysScene* PhysScene = nullptr, USkeletalMeshComponent* OwningComponent = nullptr, int32 UseRootBodyIndex = INDEX_NONE, const FPhysicsAggregateHandle& UseAggregate = FPhysicsAggregateHandle()) const;

	// Reference to our current parallel animation evaluation task (if there is one)
	FGraphEventRef				ParallelAnimationEvaluationTask;

	// Reference to our current blend physics task (if there is one)
	FGraphEventRef				ParallelBlendPhysicsCompletionTask;

	//Data for parallel evaluation of animation
	FAnimationEvaluationContext AnimEvaluationContext;

public:
	// Parallel evaluation wrappers
	virtual ENGINE_API void ParallelAnimationEvaluation();
	ENGINE_API virtual void CompleteParallelAnimationEvaluation(bool bDoPostAnimEvaluation);


	// Returns whether we are currently trying to run a parallel animation evaluation task
	bool IsRunningParallelEvaluation() const { return IsValidRef(ParallelAnimationEvaluationTask); }

	// Management function for if we want to do an evaluation but may already be running one
	// bBlockOnTask - if true and we are currently performing parallel eval we wait for it to finish
	// bPerformPostAnimEvaluation - if true and we are currently performing parallel eval we call PostAnimEvaluation too
	// return true if parallel task was running.
	ENGINE_API bool HandleExistingParallelEvaluationTask(bool bBlockOnTask, bool bPerformPostAnimEvaluation);

	friend class FSkeletalMeshComponentDetails;

	/** Apply animation curves to this component */
	ENGINE_API void ApplyAnimationCurvesToComponent(const TMap<FName, float>* InMaterialParameterCurves, const TMap<FName, float>* InAnimationMorphCurves);
	
	// Returns whether we're able to run a simulation (ignoring the suspend flag)
	ENGINE_API bool CanSimulateClothing() const;

protected:

	// Prep anim instances for evaluation
	ENGINE_API void DoInstancePreEvaluation();

	// Handle post evaluation on all anim instances
	ENGINE_API void DoInstancePostEvaluation();

	// Handle post animation on all anim instances, whether or not an evaluation took place 
	ENGINE_API void DoInstanceFinalizeAnimation(bool bWasEvaluated);

	// Returns whether we need to run the Cloth Tick or not
	ENGINE_API virtual bool ShouldRunClothTick() const;

private:
	/** Override USkinnedMeshComponent */
	ENGINE_API virtual void AddFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent) override;
	ENGINE_API virtual void RemoveFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent) override;

	// Returns whether we need to run the Pre Cloth Tick or not
	ENGINE_API bool ShouldRunEndPhysicsTick() const;

	// Handles registering/unregistering the pre cloth tick as it is needed
	ENGINE_API void UpdateEndPhysicsTickRegisteredState();

	// Handles registering/unregistering the cloth tick as it is needed
	ENGINE_API void UpdateClothTickRegisteredState();

	// Handles registering/unregistering the 'during animation' tick as it is needed
	ENGINE_API void UpdateDuringAnimationTickRegisteredState();

	// Finalizes pose to OutBoneSpaceTransforms
	ENGINE_API void FinalizePoseEvaluationResult(const USkeletalMesh* InMesh, TArray<FTransform>& OutBoneSpaceTransforms, FVector& OutRootBoneTranslation, FCompactPose& InFinalPose) const;

	// Finalizes attributes (remapping from compact to mesh bone-indices)
	ENGINE_API void FinalizeAttributeEvaluationResults(const FBoneContainer& BoneContainer, const UE::Anim::FHeapAttributeContainer& FinalContainer, UE::Anim::FMeshAttributeContainer& OutContainer) const;

	friend class FParallelBlendPhysicsTask;
	
	ENGINE_API void BlendInPhysicsInternal(FTickFunction& ThisTickFunction);

	// Blends the simulation output with the component and bone space transforms coming from
	// animation (just calls PerformBlendPhysicsBones)
	void ParallelBlendPhysics() { PerformBlendPhysicsBones(RequiredBones, AnimEvaluationContext.ComponentSpaceTransforms, AnimEvaluationContext.BoneSpaceTransforms); }

	// Blends the simulation output with the component and bone space transforms coming from animation
	ENGINE_API void PerformBlendPhysicsBones(const TArray<FBoneIndexType>& InRequiredBones, TArray<FTransform>& InOutComponentSpaceTransforms, TArray<FTransform>& InOutBoneSpaceTransforms);

	friend class FParallelClothTask;
	// This is the parallel function that updates the cloth data and runs the simulation. This is safe to call from worker threads.
	//static void ParallelEvaluateCloth(float DeltaTime, const FClothingActor& ClothingActor, const FClothSimulationContext& ClothSimulationContext);

	friend class FParallelBlendPhysicsCompletionTask;
	ENGINE_API void CompleteParallelBlendPhysics();
	ENGINE_API void FinalizeAnimationUpdate();

	/** See UpdateClothTransform for documentation. */
	ENGINE_API void UpdateClothTransformImp();

	friend class FClothingSimulationContextCommon;

	friend class FTickClothingTask;

#if WITH_EDITORONLY_DATA
	// these are deprecated variables from removing SingleAnimSkeletalComponent
	// remove if this version goes away : VER_UE4_REMOVE_SINGLENODEINSTANCE
	// deprecated variable to be re-save
	UPROPERTY()
	TObjectPtr<class UAnimSequence> SequenceToPlay_DEPRECATED;

	// The default sequence to play on this skeletal mesh
	UPROPERTY()
	TObjectPtr<class UAnimationAsset> AnimToPlay_DEPRECATED;

	// Default setting for looping for SequenceToPlay. This is not current state of looping.
	UPROPERTY()
	uint32 bDefaultLooping_DEPRECATED:1;

	// Default setting for playing for SequenceToPlay. This is not current state of playing.
	UPROPERTY()
	uint32 bDefaultPlaying_DEPRECATED:1;

	// Default setting for position of SequenceToPlay to play. 
	UPROPERTY()
	float DefaultPosition_DEPRECATED;

	// Default setting for playrate of SequenceToPlay to play. 
	UPROPERTY()
	float DefaultPlayRate_DEPRECATED;
#endif

	/*
	 * Update MorphTargetCurves from mesh - these are not animation curves, but SetMorphTarget and similar functions that can set to this mesh component
	 */
	ENGINE_API void UpdateMorphTargetOverrideCurves();

	/*
	 * Reset MorphTarget Curves - Reset all morphtarget curves
	 */
	ENGINE_API void ResetMorphTargetCurves();

public:
	/** Take extracted RootMotion and convert it from local space to world space. */
	ENGINE_API FTransform ConvertLocalRootMotionToWorld(const FTransform& InTransform);

	/** Consume and return pending root motion from our internal anim instances (main, sub and post) */
	ENGINE_API FRootMotionMovementParams ConsumeRootMotion();

#if WITH_EDITOR
	/** Called after modifying Component Space Transforms externally */
	ENGINE_API void ApplyEditedComponentSpaceTransforms();
#endif

protected:

	/** Consume and return pending root motion from our internal anim instances (main, sub and post) */
	ENGINE_API FRootMotionMovementParams ConsumeRootMotion_Internal(float InAlpha);

private:

#if WITH_EDITOR
	/** This is required for recording animations, so save for editor only */
	/** Temporary array of curve arrays that are active on this component - keeps same buffer index as SpaceBases - Please check SkinnedMeshComponent*/
	FBlendedHeapCurve	CurvesArray[2];
public: 
	/** Access Curve Array for reading */
	const FBlendedHeapCurve& GetAnimationCurves() const { return CurvesArray[CurrentReadComponentTransforms]; }

	/** Get Access to the current editable Curve Array - uses same buffer as space bases*/
	FBlendedHeapCurve& GetEditableAnimationCurves() { return CurvesArray[CurrentEditableComponentTransforms]; }
	const FBlendedHeapCurve& GetEditableAnimationCurves() const { return CurvesArray[CurrentEditableComponentTransforms]; }
#endif 


private:
	/** Temporary array of attributes that are active on this component - keeps same buffer index as SpaceBases - Please check SkinnedMeshComponent*/
	UE::Anim::FMeshAttributeContainer AttributesArray[2];

	UE::Anim::FMeshAttributeContainer& GetEditableCustomAttributes() { return AttributesArray[CurrentEditableComponentTransforms]; }

public:
	const UE::Anim::FMeshAttributeContainer& GetCustomAttributes() const { return AttributesArray[CurrentReadComponentTransforms]; }
public:
	/** Skeletal mesh component should not be able to have its mobility set to static */
	virtual const bool CanHaveStaticMobility() const override { return false; }

public:
	/** Register/Unregister for physics state creation callback */
	ENGINE_API FDelegateHandle RegisterOnPhysicsCreatedDelegate(const FOnSkelMeshPhysicsCreated& Delegate);
	ENGINE_API void UnregisterOnPhysicsCreatedDelegate(const FDelegateHandle& DelegateHandle);

	/** Register/Unregister for teleport callback */
	ENGINE_API FDelegateHandle RegisterOnTeleportDelegate(const FOnSkelMeshTeleported& Delegate);
	ENGINE_API void UnregisterOnTeleportDelegate(const FDelegateHandle& DelegateHandle);

	/** Register/Unregister for OnBoneTransformsFinalized callback */
	ENGINE_API virtual FDelegateHandle RegisterOnBoneTransformsFinalizedDelegate(const FOnBoneTransformsFinalizedMultiCast::FDelegate& Delegate) override;
	ENGINE_API virtual void UnregisterOnBoneTransformsFinalizedDelegate(const FDelegateHandle& DelegateHandle) override;

	/** Register/Unregister for OnLODRequiredBonesUpdate callback */
	UE_DEPRECATED(5.6, "These should not be used as they rely on a global delegate, ANY change in LOD from ANY skeletal mesh component triggers the callback")
	ENGINE_API FDelegateHandle RegisterOnLODRequiredBonesUpdate(const FOnLODRequiredBonesUpdate& Delegate);

	UE_DEPRECATED(5.6, "These should not be used as they rely on a global delegate, ANY change in LOD from ANY skeletal mesh component triggers the callback")
	ENGINE_API void UnregisterOnLODRequiredBonesUpdate(const FDelegateHandle& DelegateHandle);

private:
	// NOTE: We anticipate that this will go away in the future, it is meant to be a temporary solution
	/** Register/Unregister for OnLODRequiredBonesUpdate callback */
	ENGINE_API FDelegateHandle RegisterOnLODRequiredBonesUpdate_Member(const FOnLODRequiredBonesUpdate& Delegate);
	ENGINE_API void UnregisterOnLODRequiredBonesUpdate_Member(const FDelegateHandle& DelegateHandle);
	friend class UE::UAF::FDataRegistry;
	
private:

#if WITH_EDITORONLY_DATA

	/** Default Animating Rig to Use if bOverrideDefaultAnimatingRig is true */
	UPROPERTY(EditAnywhere, Category = AnimationRig, BlueprintGetter = GetDefaultAnimatingRigOverride, BlueprintSetter = SetDefaultAnimatingRigOverride, meta = (AllowedClasses = "/Script/ControlRigDeveloper.ControlRigBlueprint, /Script/ControlRig.ControlRigBlueprintGeneratedClass"), meta = (editcondition = "bOverrideDefaultAnimatingRig"))
	TSoftObjectPtr<UObject> DefaultAnimatingRigOverride;

#endif

#if WITH_EDITOR
public:

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetDefaultAnimatingRigOverride(TSoftObjectPtr<UObject> InAnimatingRig);

	UFUNCTION(BlueprintGetter)
	ENGINE_API TSoftObjectPtr<UObject> GetDefaultAnimatingRigOverride() const;

	UFUNCTION(BlueprintCallable, Category = "Components|SkeletalMesh")
	ENGINE_API TSoftObjectPtr<UObject> GetDefaultAnimatingRig() const;


#endif
private:

	/** Multicaster fired when this component creates physics state (in case external objects rely on physics state)*/
	FOnSkelMeshPhysicsCreatedMultiCast OnSkelMeshPhysicsCreated;

	/** Multicaster fired when this component teleports */
	FOnSkelMeshTeleportedMultiCast OnSkelMeshPhysicsTeleported;

	/** Multicaster fired when this component bone transforms are finalized */
	FOnBoneTransformsFinalizedMultiCast OnBoneTransformsFinalizedMC;

	/** Multicaster fired when this component finalizes the regeneration of the required bones list for the current LOD */
	static ENGINE_API FOnLODRequiredBonesUpdateMulticast OnLODRequiredBonesUpdate_Static;
	FOnLODRequiredBonesUpdateMulticast OnLODRequiredBonesUpdate_Member;

	/** Mark current anim UID version to up-to-date. Called when it's recalculated */
	ENGINE_API void MarkRequiredCurveUpToDate();
	
	/* This will check if the required curves are up-to-date by checking version number with skeleton. 
	 * Skeleton's curve list changes whenever newer is added or deleted. 
	 * This still has to happen in editor as well as in game as 
	 * There is no guarantee of Skeleton having all curves as we've seen over and over again. 
	 * Cooking does not guarantee skeleton containing all names
	 */
	ENGINE_API bool AreRequiredCurvesUpToDate() const;

private:
	// Can't rely on time value, because those may be affected by dilation and whether or not
	// the game is paused.
	// Also can't just rely on a flag as other components (like CharacterMovementComponent) may tick
	// the pose and we can't guarantee tick order.
	UPROPERTY(Transient)
	uint32 LastPoseTickFrame; 

	/** Payload used by asynchronous destruction of physics state (see OnAsyncDestroyPhysicsState). */
	TArray<FBodyInstance*> AsyncDestroyPhysicsStatePayload;
	TArray<FConstraintInstance*> AsyncDestroyConstraintsPayload;
	FPhysicsAggregateHandle AggregatePayload;

public:
	ENGINE_API void ConditionallyDispatchQueuedAnimEvents();

	// Are we currently within PostAnimEvaluation
	bool IsPostEvaluatingAnimation() const { return bPostEvaluatingAnimation; }

	//~ Begin IPhysicsComponent Interface.
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectByName(const FName& Name) const override;
	ENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	//~ End IPhysicsComponent Interface.

private:
	bool bBoneTransformsInitialized = false;
};

#if WITH_EDITOR
struct FPoseWatchDynamicData
{
public:

	FPoseWatchDynamicData(USkeletalMeshComponent* InComponent);

	TArray<FAnimNodePoseWatch> PoseWatches;
};
#endif

struct FLinkedInstancesAdapter
{
	static void AddLinkedInstance(USkeletalMeshComponent* InComponent, UAnimInstance* InAnimInstance)
	{
		if (InComponent && InAnimInstance)
		{
			InComponent->LinkedInstances.AddUnique(InAnimInstance);
		}
	}

	static void RemoveLinkedInstance(USkeletalMeshComponent* InComponent, UAnimInstance* InAnimInstance)
	{
		if (InComponent && InAnimInstance)
		{
			InComponent->LinkedInstances.Remove(InAnimInstance);
		}
	}

	static void ResetLinkedInstance(USkeletalMeshComponent* InComponent)
	{
		if (InComponent)
		{
			InComponent->LinkedInstances.Reset();
		}
	}
};
