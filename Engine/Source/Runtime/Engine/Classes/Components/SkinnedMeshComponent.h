// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "ActorSkinnedMeshComponentInterface.h"
#include "GPUSkinPublicDefs.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/MeshComponent.h"
#include "Containers/SortedMap.h"
#include "LODSyncInterface.h"
#include "BoneContainer.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Rendering/SkinWeightProfile.h"
#include "StateStream/SkinnedMeshStateStreamHandle.h"
#include "Async/RecursiveMutex.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITORONLY_DATA
#include "Animation/MeshDeformerGeometryReadback.h"
#endif // WITH_EDITORONLY_DATA

#include "SkinnedMeshComponent.generated.h"

enum class ESkinCacheUsage : uint8;
enum class EPreviousBoneTransformUpdateMode;
class FPrimitiveSceneProxy;
class FColorVertexBuffer;
class FSkinWeightVertexBuffer;
class FSkeletalMeshObject;
class FSkeletalMeshRenderData;
class FSkeletalMeshLODRenderData;
struct FSkelMeshRenderSection;
class FPositionVertexBuffer;
class UMeshDeformer;
class UMeshDeformerInstance;
class UMeshDeformerInstanceSettings;
class UMorphTarget;
class USkinnedAsset;
struct FExternalMorphSet;
struct FExternalMorphWeightData;
struct FSkinWeightProfileStack;
struct FMeshDescription;

namespace Nanite
{
	struct FResources;
}

namespace UE::Anim
{
	struct FSkinnedMeshComponentExtensions;
	ENGINE_API extern bool bUseSafeMeshPoseIndices;
}

DECLARE_DELEGATE_OneParam(FOnAnimUpdateRateParamsCreated, FAnimUpdateRateParameters*)
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnTickPose, USkinnedMeshComponent* /*SkinnedMeshComponent*/, float /*DeltaTime*/, bool /*bNeedsValidRootMotion*/)
DECLARE_MULTICAST_DELEGATE(FOnBoneTransformsFinalizedMultiCast);

//
// Bone Visibility.
//

/** The valid BoneVisibilityStates values; A bone is only visible if it is *exactly* 1 */
UENUM()
enum EBoneVisibilityStatus : int
{
	/** Bone is hidden because it's parent is hidden. */
	BVS_HiddenByParent,
	/** Bone is visible. */
	BVS_Visible,
	/** Bone is hidden directly. */
	BVS_ExplicitlyHidden,
	BVS_MAX,
};

/** PhysicsBody options when bone is hidden */
UENUM()
enum EPhysBodyOp : int
{
	/** Don't do anything. */
	PBO_None,
	/** Terminate - if you terminate, you won't be able to re-init when unhidden. */
	PBO_Term,
	PBO_MAX,
};

/** Skinned Mesh Animation Tick option based on rendered or not. This dictates "TickPose and RefreshBoneTransforms" */
UENUM(BlueprintType)
enum class EVisibilityBasedAnimTickOption : uint8
{
	/** Always Tick and Refresh BoneTransforms whether rendered or not. */
	AlwaysTickPoseAndRefreshBones,
	/** Always Tick, but Refresh BoneTransforms only when rendered. */
	AlwaysTickPose,
	/**
		When rendered Tick Pose and Refresh Bone Transforms,
		otherwise, just update montages alongside BoneTransforms.
		(AnimBP graph will not be updated).
	*/
	OnlyTickMontagesAndRefreshBonesWhenPlayingMontages,
	/**
		When rendered Tick Pose and Refresh Bone Transforms,
		otherwise, just update montages and skip everything else.
		(AnimBP graph will not be updated).
	*/
	OnlyTickMontagesWhenNotRendered,
	/** Tick only when rendered, and it will only RefreshBoneTransforms when rendered. */
	OnlyTickPoseWhenRendered,
};

/** Previous deprecated animation tick option. */
namespace EMeshComponentUpdateFlag
{
	UE_DEPRECATED(4.21, "EMeshComponentUpdateFlag has been deprecated use EVisibilityBasedAnimTickOption instead")
	typedef int32 Type;
	UE_DEPRECATED(4.21, "EMeshComponentUpdateFlag has been deprecated use EVisibilityBasedAnimTickOption instead")
	static const uint8 AlwaysTickPoseAndRefreshBones = 0;
	UE_DEPRECATED(4.21, "EMeshComponentUpdateFlag has been deprecated use EVisibilityBasedAnimTickOption instead")
	static const uint8 AlwaysTickPose = 1;
	UE_DEPRECATED(4.21, "EMeshComponentUpdateFlag has been deprecated use EVisibilityBasedAnimTickOption instead")
	static const uint8 OnlyTickMontagesWhenNotRendered = 2;
	UE_DEPRECATED(4.21, "EMeshComponentUpdateFlag has been deprecated use EVisibilityBasedAnimTickOption instead")
	static const uint8 OnlyTickPoseWhenRendered = 3;
};

/** Values for specifying bone space. */
UENUM()
namespace EBoneSpaces
{
	enum Type : int
	{
		/** Set absolute position of bone in world space. */
		WorldSpace		UMETA(DisplayName = "World Space"),
		/** Set position of bone in components reference frame. */
		ComponentSpace	UMETA(DisplayName = "Component Space"),
		/** Set position of bone relative to parent bone. */
		//LocalSpace		UMETA( DisplayName = "Parent Bone Space" ),
	};
}


/** Values for specifying which layer a skin weight profile is applied at.
 */
UENUM(BlueprintType)
enum class ESkinWeightProfileLayer : uint8
{
	Primary,		/** Primary skin weight profile layer */
	Secondary,		/** Secondary skin weight profile layer */
};

/** WeightIndex is an into the MorphTargetWeights array */
using FMorphTargetWeightMap = TMap<const UMorphTarget* /* MorphTarget */, int32 /* WeightIndex */>;

/** Vertex skin weight info supplied for a component override. */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/Engine.KismetRenderingLibrary.MakeSkinWeightInfo", HasNativeBreak = "/Script/Engine.KismetRenderingLibrary.BreakSkinWeightInfo"))
struct FSkelMeshSkinWeightInfo
{
	GENERATED_USTRUCT_BODY()

	/** Index of bones that influence this vertex */
	UPROPERTY()
	int32	Bones[MAX_TOTAL_INFLUENCES] = {};
	/** Influence of each bone on this vertex */
	UPROPERTY()
	uint8	Weights[MAX_TOTAL_INFLUENCES] = {};
};

/** LOD specific setup for the skeletal mesh component. */
USTRUCT()
struct FSkelMeshComponentLODInfo
{
	GENERATED_USTRUCT_BODY()

	/** Material corresponds to section. To show/hide each section, use this. */
	UPROPERTY()
	TArray<bool> HiddenMaterials;

	/** Vertex buffer used to override vertex colors */
	FColorVertexBuffer* OverrideVertexColors = nullptr;

	/** Vertex buffer used to override skin weights */
	FSkinWeightVertexBuffer* OverrideSkinWeights = nullptr;

	/** Vertex buffer used to override skin weights from one of the profiles */
	FSkinWeightVertexBuffer* OverrideProfileSkinWeights = nullptr;
	
	ENGINE_API FSkelMeshComponentLODInfo();
	ENGINE_API ~FSkelMeshComponentLODInfo();

	ENGINE_API void SetActiveSkinWeightProfileStack(const FSkinWeightProfileStack& InProfileStack);
	ENGINE_API bool GetActiveSkinWeightProfileStack(FSkinWeightProfileStack& OutProfileStack) const;

	ENGINE_API void ReleaseOverrideVertexColorsAndBlock();
	ENGINE_API bool BeginReleaseOverrideVertexColors();
	ENGINE_API void EndReleaseOverrideVertexColors();

private:
	void CleanUpOverrideVertexColors();

public:
	ENGINE_API void ReleaseOverrideSkinWeightsAndBlock();
	ENGINE_API bool BeginReleaseOverrideSkinWeights();
	ENGINE_API void EndReleaseOverrideSkinWeights();

private:
	void CleanUpOverrideSkinWeights();
	
	FSkinWeightProfileStack ProfileStackRequest;
};


USTRUCT(BlueprintType)
struct FVertexOffsetUsage
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (Bitmask, BitmaskEnum = "/Script/Engine.EVertexOffsetUsageType"))
	int32 Usage = 0;
};

/** Specifies which mesh deformer should be used on each LOD of a mesh */
struct FMeshDeformerSet
{
	/** Each deformer in this array should be used on at least one LOD */
	TArray<TObjectPtr<UMeshDeformer>, TInlineAllocator<2>> Deformers;

	/**
	 * Indexed by mesh LOD. Each element is either a valid index into the Deformers array, or
	 * INDEX_NONE to signify that no deformer should be used for this LOD.
	 * MAX_MESH_LOD_COUNT here cannot be a hard limit as editor allows more LODs to be created
	 * however there may be other systems assuming MAX_MESH_LOD_COUNT is respected so generally it is not a good idea
	 * to exceed it
	 */
	TArray<int8, TInlineAllocator<MAX_MESH_LOD_COUNT>> DeformerIndexForLOD;
};

/** Same as FMeshDeformerSet, except for mesh deformer instances */
USTRUCT()
struct FMeshDeformerInstanceSet
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UMeshDeformerInstance>> DeformerInstances;

	TArray<int8> InstanceIndexForLOD;
};

/** The map of external morph sets registered on the skinned mesh component. */
using FExternalMorphSets = TMap<int32, TSharedPtr<FExternalMorphSet>>;

/**
 *
 * Skinned mesh component that supports bone skinned mesh rendering.
 * This class does not support animation.
 *
 * @see USkeletalMeshComponent
*/
UCLASS(hidecategories=Object, config=Engine, editinlinenew, abstract, MinimalAPI)
class USkinnedMeshComponent : public UMeshComponent, public ILODSyncInterface, public IClothSimulationDataProvider
{
	GENERATED_UCLASS_BODY()
	UE_DECLARE_COMPONENT_ACTOR_INTERFACE(SkinnedMeshComponent)

public:
	/** Access granted to the render state recreator in order to trigger state rebuild */
	friend class FSkinnedMeshComponentRecreateRenderStateContext;

	/** The skeletal mesh used by this component. */
	UE_DEPRECATED(5.1, "Replaced by SkinnedAsset. Use GetSkinnedAsset()/SetSkinnedAsset() instead, or GetSkeletalMeshAsset/SetSkeletalMeshAsset() when called from a USkeletalMeshComponent.")
	UPROPERTY(EditAnywhere, Setter = SetSkeletalMesh_DEPRECATED, BlueprintGetter = GetSkeletalMesh_DEPRECATED, Category = "Mesh|SkeletalAsset", meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh", DeprecatedProperty, DeprecationMessage = "Use USkeletalMeshComponent::GetSkeletalMeshAsset() or GetSkinnedAsset() instead."))
	TObjectPtr<class USkeletalMesh> SkeletalMesh;

private:
	/** The skinned asset used by this component. */
	UE_DEPRECATED(5.1, "This property isn't deprecated, but getter and setter must be used instead in order to preserve backward compatibility with the SkeletalMesh pointer.")
	UPROPERTY(BlueprintGetter = GetSkinnedAsset, Category = "Mesh")
	TObjectPtr<class USkinnedAsset> SkinnedAsset;

#if WITH_EDITOR
	/** Used to track down when SkinnedAsset has been modified for notification purpose */
	USkinnedAsset* KnownSkinnedAsset = nullptr;
#endif

#if WITH_EDITOR
	/** Called when the SkinnedAsset property gets overwritten without us knowing about it */
	ENGINE_API void OutdatedKnownSkinnedAssetDetected() const;
#endif

protected:
	ENGINE_API void NotifyIfSkinnedAssetChanged();

public:
	ENGINE_API USkinnedMeshComponent(FVTableHelper& Helper);
	ENGINE_API ~USkinnedMeshComponent();

	//
	// LeaderPoseComponent.
	//
	
	/**
	 *	If set, this SkeletalMeshComponent will not use its SpaceBase for bone transform, but will
	 *	use the component space transforms from the LeaderPoseComponent. This is used when constructing a character using multiple skeletal meshes sharing the same
	 *	skeleton within the same Actor.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Mesh")
	TWeakObjectPtr<USkinnedMeshComponent> LeaderPoseComponent;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property is deprecated. Please use LeaderPoseComponent instead")
	TWeakObjectPtr<USkinnedMeshComponent> MasterPoseComponent;
#endif // WITH_EDITORONLY_DATA

	/**
	 * How this Component's LOD uses the skin cache feature. Auto will defer to the asset's (SkeletalMesh) option. If Ray Tracing is enabled, will imply Enabled
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Mesh")
	TArray<ESkinCacheUsage> SkinCacheUsage;

protected:
	/** If true, MeshDeformer will be used. If false, use the default mesh deformer on the SkeletalMesh. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer", meta = (InlineEditConditionToggle))
	bool bSetMeshDeformer = false;

	/** The mesh deformer to use. Set to None to disable the deformer on the SkeletalMesh. If no deformer is set from here or the SkeletalMesh, we fall back to the fixed function deformation, unless AlwaysUseMeshDeformer is on. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer", meta = (editcondition = "bSetMeshDeformer"))
	TObjectPtr<UMeshDeformer> MeshDeformer;

	/** If true, and if no mesh deformer is set from here or the SkeletalMesh, fall back to the default deformer specified in the project settings, unless DefaultMode is set to "Never" in project settings*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer")
	bool bAlwaysUseMeshDeformer = false;

	/** Set the MeshDeformer and update the internal MeshDeformerInstance. */
	ENGINE_API void SetMeshDeformer(bool bInSetMeshDeformer, UMeshDeformer* InMeshDeformer);

	/** Get the currently active MeshDeformer. This may come from the SkeletalMesh default or the Component override. */
	ENGINE_API FMeshDeformerSet GetActiveMeshDeformers() const;

	/** Object containing instance settings for the bound MeshDeformer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Instanced, Category = "Deformer", meta = (DisplayName = "Deformer Settings", EditCondition = "MeshDeformerInstanceSettings!=nullptr", ShowOnlyInnerProperties))
	TObjectPtr<UMeshDeformerInstanceSettings> MeshDeformerInstanceSettings;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "Replaced by MeshDeformerInstances. Call GetMeshDeformerInstance() to get the same behavior as reading MeshDeformerInstance.")
	UPROPERTY(Transient, BlueprintReadOnly, BlueprintGetter = GetMeshDeformerInstance, Category = "Deformer", meta = (DeprecatedProperty, DeprecationMessage = "Use the GetMeshDeformerInstance function instead"))
	TObjectPtr<UMeshDeformerInstance> MeshDeformerInstance_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Object containing state for the bound MeshDeformer. */
	UPROPERTY(Transient)
	FMeshDeformerInstanceSet MeshDeformerInstances;
public:
	const FMeshDeformerInstanceSet& GetMeshDeformerInstances() const { return MeshDeformerInstances; }

	TObjectPtr<UMeshDeformer> GetComponentMeshDeformer() const { return MeshDeformer; }
private:
	/** Initializes the MeshDeformerInstances property */
	void CreateMeshDeformerInstances(const FMeshDeformerSet& DeformerSet);
	/** Release resource contents of MeshDeformerInstances */
	void ReleaseMeshDeformerInstances();

public:
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh")
	ENGINE_API UMeshDeformerInstance* GetMeshDeformerInstance() const;

	/** Returns the MeshDeformerInstanceSettings pointer. (can be nullptr) */
	ENGINE_API UMeshDeformerInstanceSettings* GetMeshDeformerInstanceSettings() const;
	
	/** 
	 * Gets the MeshDeformer for the given LOD.
	 * 
	 * Returns null if the LOD doesn't use a deformer or if the index is out of range.
	 * 
	 * This function takes GetMeshDeformerMaxLOD() into account, so there's no need to call both.
	 */
	ENGINE_API UMeshDeformerInstance* GetMeshDeformerInstanceForLOD(int32 LODIndex) const;

	/** Max LOD at which to update or apply the MeshDeformer. */
	ENGINE_API virtual int32 GetMeshDeformerMaxLOD() const;

#if WITH_EDITORONLY_DATA
protected:
	TArray<TUniquePtr<FMeshDeformerGeometryReadbackRequest>> GeometryReadbackRequests;
public:
	/**
	 * Readback the render geometry async, a FMeshDescription will be generated on a background thread for additional operation
	 * Note: currently it only supports mesh deformed by a MeshDeformer
	 */
	ENGINE_API bool RequestReadbackRenderGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest);
#endif // WITH_EDITORONLY_DATA
	
public:
	
	/** const getters for previous transform idea */
	const TArray<uint8>& GetPreviousBoneVisibilityStates() const
	{
		return bDoubleBufferedComponentSpaceTransforms ? GetEditableBoneVisibilityStates() : PreviousBoneVisibilityStates;
	}

	const TArray<FTransform>& GetPreviousComponentTransformsArray() const
	{
		return bDoubleBufferedComponentSpaceTransforms ? GetEditableComponentSpaceTransforms() : PreviousComponentSpaceTransformsArray;
	}

	uint32 GetBoneTransformRevisionNumber() const 
	{
		const USkinnedMeshComponent* LeaderPoseComponentPtr = LeaderPoseComponent.Get();
		return (LeaderPoseComponentPtr ? LeaderPoseComponentPtr->CurrentBoneTransformRevisionNumber : CurrentBoneTransformRevisionNumber);
	}

	uint32 GetPreviousBoneTransformRevisionNumber() const
	{
		const USkinnedMeshComponent* LeaderPoseComponentPtr = LeaderPoseComponent.Get();
		return (LeaderPoseComponentPtr ? LeaderPoseComponentPtr->PreviousBoneTransformRevisionNumber : PreviousBoneTransformRevisionNumber);
	}

	uint32 GetCurrentBoneTransformFrame() const
	{
		const USkinnedMeshComponent* LeaderPoseComponentPtr = LeaderPoseComponent.Get();
		return (LeaderPoseComponentPtr ? LeaderPoseComponentPtr->CurrentBoneTransformFrame : CurrentBoneTransformFrame);
	}

	/* this update renderer with new revision number twice so to clear bone velocity for motion blur or temporal AA */
	ENGINE_API void ClearMotionVector();
	
	/* Forcibly update the renderer with a new revision number to assign the current bone velocity for motion blur or temporal AA */
	ENGINE_API void ForceMotionVector();

private:
	ENGINE_API void UpdateSkinWeightProfileReadbackIfNeeded(int32 LODIndex);
	ENGINE_API void UpdateBoneTransformRevisionNumber();
	ENGINE_API EPreviousBoneTransformUpdateMode GetPreviousBoneTransformUpdateMode();

	enum class EBoneTransformUpdateMethod
	{
		AnimationUpdate,
		ClearMotionVector,
		ForceMotionVector
	};
	TArray<EBoneTransformUpdateMethod> BoneTransformUpdateMethodQueue;

	/** Temporary array of of component-space bone matrices, update each frame and used for rendering the mesh. */
	TArray<FTransform> ComponentSpaceTransformsArray[2];

	/**
	 * Mesh Pose (sized to our skinned asset's ref skeleton) bit array of valid bone indices. 
	 * This should be populated by derived classes to avoid children copying invalid transforms through i.e. leader pose/copy pose.
	 */
	TBitArray<> ValidMeshPoseTransforms;

#if WITH_STATE_STREAM_ACTOR
	FSkinnedMeshHandle MeshHandle;
#endif

protected:
	/** Array of bone visibilities (containing one of the values in EBoneVisibilityStatus for each bone).  A bone is only visible if it is *exactly* 1 (BVS_Visible) */
	/** Note these are only used if we are NOT double-buffering bone transforms */
	TArray<uint8> PreviousBoneVisibilityStates;

	/** Array of previous bone transforms */
	/** Note these are only used if we are NOT double-buffering bone transforms */
	TArray<FTransform> PreviousComponentSpaceTransformsArray;

protected:
	/** The bounds radius at the point we last notified the streamer of a bounds radius change */
	float LastStreamerUpdateBoundsRadius;

	/** The index for the ComponentSpaceTransforms buffer we can currently write to */
	int32 CurrentEditableComponentTransforms;

	/** The index for the ComponentSpaceTransforms buffer we can currently read from */
	int32 CurrentReadComponentTransforms;

	/** Cache previous bone transform revision number to help compute CurrentBoneTransformRevisionNumber */
	uint32 PreviousBoneTransformRevisionNumber;

	/** current bone transform revision number */
	uint32 CurrentBoneTransformRevisionNumber;

	/** Stores GFrameCounter when CurrentBoneTransformRevisionNumber was updated, to make sure it's updated once per frame */
	uint32 CurrentBoneTransformFrame;

	/** Incremented every time the leader bone map changes. Used to keep in sync with any duplicate data needed by other threads */
	int32 LeaderBoneMapCacheCount;

	/** 
	 * If set, this component has follower pose components that are associated with this 
	 * Note this is weak object ptr, so it will go away unless you have other strong reference
	 */
	TArray< TWeakObjectPtr<USkinnedMeshComponent> > FollowerPoseComponents;

	/**
	 *	Mapping between bone indices in this component and the parent one. Each element is the index of the bone in the LeaderPoseComponent.
	 *	Size should be the same as USkeletalMesh.RefSkeleton size (ie number of bones in this skeleton).
	 */
	TArray<int32> LeaderBoneMap;

	/** Cached relative transform for follower bones that are missing in the leader */
	struct FMissingLeaderBoneCacheEntry
	{
		FMissingLeaderBoneCacheEntry()
			: RelativeTransform(FTransform::Identity)
			, CommonAncestorBoneIndex(INDEX_NONE)
		{}

		FMissingLeaderBoneCacheEntry(const FTransform& InRelativeTransform, int32 InCommonAncestorBoneIndex)
			: RelativeTransform(InRelativeTransform)
			, CommonAncestorBoneIndex(InCommonAncestorBoneIndex)
		{}

		/** 
		 * Relative transform of the missing bone's ref pose, based on the earliest common ancestor 
		 * this will be equivalent to the component space transform of the bone had it existed in the leader. 
		 */
		FTransform RelativeTransform;

		/** The index of the earliest common ancestor of the leader mesh. Index is the bone index in *this* mesh. */
		int32 CommonAncestorBoneIndex;
	};

	/**  
	 * Map of missing bone indices->transforms so that calls to GetBoneTransform() succeed when bones are not
	 * present in a leader mesh when using leader-pose. Index key is the bone index of *this* mesh.
	 */
	TMap<int32, FMissingLeaderBoneCacheEntry> MissingLeaderBoneMap;

	/**
	*	Mapping for socket overrides, key is the Source socket name and the value is the override socket name
	*/
	TSortedMap<FName, FName, FDefaultAllocator, FNameFastLess> SocketOverrideLookup;

public:
#if WITH_EDITORONLY_DATA
	/**
	 * Wireframe color
	 */
	UPROPERTY()
	FColor WireframeColor_DEPRECATED;
#endif

#if UE_ENABLE_DEBUG_DRAWING
	/** Debug draw color */
	TOptional<FLinearColor> DebugDrawColor;
#endif

protected:
	/** Information for current ref pose override, if present */
	TSharedPtr<FSkelMeshRefPoseOverride> RefPoseOverride;

public:
	const TArray<int32>& GetLeaderBoneMap() const { return LeaderBoneMap; }
	TArray<int32>& GetMutableLeaderBoneMap() { return LeaderBoneMap; }

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use the GetLeaderBoneMap.")
	const TArray<int32>& GetMasterBoneMap() const { return GetLeaderBoneMap(); }

	/** Get the weights in read-only mode for a given external morph target set at a specific LOD. */
	ENGINE_API const FExternalMorphWeightData& GetExternalMorphWeights(int32 LOD) const;

	/** Get the weights for a given external morph target set at a specific LOD. */
	ENGINE_API FExternalMorphWeightData& GetExternalMorphWeights(int32 LOD);

	/**
	 * Register an external set of GPU compressed morph targets.
	 * These compressed morph targets are GPU only morph targets that will not appear inside the UI and are owned by external systems.
	 * Every set of these morph targets has some unique ID.
	 * @param LOD The LOD index.
	 * @param ID The unique ID for this set of morph targets.
	 * @param MorphSet A shared pointer to a morph set, which basically contains a GPU friendly morph buffer. This buffer should be owned by an external system that calls this method.
	 */
	ENGINE_API void AddExternalMorphSet(int32 LOD, int32 ID, TSharedPtr<FExternalMorphSet> MorphSet);

	/** Remove a given set of external GPU based morph targets. */
	ENGINE_API void RemoveExternalMorphSet(int32 LOD, int32 ID);

	/** Clear all externally registered morph target buffers. */
	ENGINE_API void ClearExternalMorphSets(int32 LOD);

	/** Do we have a given set of external morph targets? */
	ENGINE_API bool HasExternalMorphSet(int32 LOD, int32 ID) const;

	/** Check whether a given LOD index is valid for the external morph sets. */
	ENGINE_API bool IsValidExternalMorphSetLODIndex(int32 LOD) const;

	/** Get the external morph sets for a given LOD. */
	const FExternalMorphSets& GetExternalMorphSets(int32 LOD) const { return ExternalMorphSets[LOD]; }
	FExternalMorphSets& GetExternalMorphSets(int32 LOD) { return ExternalMorphSets[LOD]; }

	/** Get the array of external morph target sets. It is an array, one entry for each LOD. */
	const TArray<FExternalMorphSets>& GetExternalMorphSetsArray() const { return ExternalMorphSets; }

	/** 
	 * Get CPU skinned vertices for the specified LOD level. Includes morph targets if they are enabled.
	 * Note: This function is very SLOW as it needs to flush the render thread.
	 * @param	OutVertices		The skinned vertices
	 * @param	InLODIndex		The LOD we want to export
	 */
	ENGINE_API void GetCPUSkinnedVertices(TArray<struct FFinalSkinVertex>& OutVertices, int32 InLODIndex) const;

	ENGINE_API void GetCPUSkinnedCachedFinalVertices(TArray<FFinalSkinVertex>& OutVertices) const;

	ENGINE_API virtual const Nanite::FResources* GetNaniteResources() const;

	/**
	 * Returns true if the component has valid Nanite render data.
	 */
	ENGINE_API virtual bool HasValidNaniteData() const;


#if WITH_EDITOR
	/**
	 * Called by the asset compilers (FAnimBankCompilingManager / FSkinnedAssetCompilingManager) when done to allow render state to be created.
	 */
	virtual void PostAssetCompilation() {};
#endif

#if UE_ENABLE_DEBUG_DRAWING
	/** Get whether to draw this mesh's debug skeleton */
	bool ShouldDrawDebugSkeleton() const
	{
		return bDrawDebugSkeleton
#if WITH_EDITORONLY_DATA
		|| bDisplayBones
#endif
		; 
	}

	/** Set whether to draw this mesh's debug skeleton */
	void SetDrawDebugSkeleton(bool bInDraw) { bDrawDebugSkeleton = bInDraw; }

	/** Get debug draw color */
	const TOptional<FLinearColor>& GetDebugDrawColor() const { return DebugDrawColor; }

	/** Set debug draw color */
	void SetDebugDrawColor(const FLinearColor& InColor) { DebugDrawColor = InColor; }
#endif

	/** Array indicating all active morph targets. This map is updated inside RefreshBoneTransforms based on the Anim Blueprint. */
	FMorphTargetWeightMap ActiveMorphTargets;

	/** Array of weights for all morph targets. This array is updated inside RefreshBoneTransforms based on the Anim Blueprint. */
	TArray<float> MorphTargetWeights;

	/** The external morph target set weight data, for each LOD. This data is (re)initialized by RefreshExternalMorphTargetWeights(). */
	TArray<FExternalMorphWeightData> ExternalMorphWeightData;

	/**
	 * External GPU based morph target buffers, for each LOD.
	 * This contains an additional set of GPU only morphs that come from external other systems that generate morph targets.
	 * These morph targets can only be updated on the GPU, will not be serialized as part of the Skeletal Mesh, and will not show up in the editors morph target list.
	 * Every set of external morph targets has some given ID. Each LOD level has a map indexed by LOD number.
	 */
	TArray<FExternalMorphSets> ExternalMorphSets;

#if WITH_EDITORONLY_DATA
private:
	/** Index of the section to preview... If set to -1, all section will be rendered */
	int32 SectionIndexPreview;

	/** Index of the material to preview... If set to -1, all section will be rendered */
	int32 MaterialIndexPreview;

	/** The section currently selected in the Editor. Used for highlighting */
	int32 SelectedEditorSection;

	/** The Material currently selected. need to remember this index for reimporting cloth */
	int32 SelectedEditorMaterial;
#endif // WITH_EDITORONLY_DATA

public:
	//
	// Physics.
	//
	
	/**
	 *	PhysicsAsset is set in SkeletalMesh by default, but you can override with this value
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Physics)
	TObjectPtr<class UPhysicsAsset> PhysicsAssetOverride;

	//
	// Level of detail.
	//
	
	/** If 0, auto-select LOD level. if >0, force to (ForcedLodModel-1). */
	UE_DEPRECATED(4.24, "Direct access to ForcedLodModel is deprecated. Please use its getter and setter instead.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	int32 ForcedLodModel;

private:
	/**
	 * Forces the specified LOD to stream in when LOD streaming is enabled.
	 * 
	 * If 0, streaming behavior is based on screen size, regardless of ForcedLodModel.
	 * 
	 * If > 0, (ForceStreamedLodModel - 1) will be streamed in.
	 * 
	 * This value will be copied from ForcedLodModel on initialization and will be set along with
	 * ForcedLodModel by the SetForcedLOD function. It will only differ from ForcedLodModel if the
	 * ILODSyncInterface functions are called, as these allow ForcedLodModel and
	 * ForceStreamedLodModel to be set individually.
	 * 
	 * When ForcedLodModel and ForceStreamedLodModel are different, ForcedLodModel determines which
	 * LOD to use for rendering and ForceStreamedLodModel determines which LOD to request for
	 * streaming.
	 */
	int32 ForceStreamedLodModel;
public:

	/**
	 * This is the min LOD that this component will use.  (e.g. if set to 2 then only 2+ LOD Models will be used.) This is useful to set on
	 * meshes which are known to be a certain distance away and still want to have better LODs when zoomed in on them.
	 **/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	int32 MinLodModel;

	/** 
	 *	Best LOD that was 'predicted' by UpdateSkelPose. 
	 *	This is what bones were updated based on, so we do not allow rendering at a better LOD than this. 
	 */
	UE_DEPRECATED(4.27, "Direct access to PredictedLODLevel has been deprecated and will be removed in a future version. Please use Get/SetPredictedLODLevel() accessors.")
	int32 PredictedLODLevel;

	/**	High (best) DistanceFactor that was desired for rendering this USkeletalMesh last frame. Represents how big this mesh was in screen space   */
	float MaxDistanceFactor;

	/**
	 * Allows adjusting the desired streaming distance of streaming textures that uses UV 0.
	 * 1.0 is the default, whereas a higher value makes the textures stream in sooner from far away.
	 * A lower value (0.0-1.0) makes the textures stream in later (you have to be closer).
	 * Value can be < 0 (from legcay content, or code changes)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=SkeletalMesh)
	float StreamingDistanceMultiplier;

	/** Used to forcefully disable pixel programmable rasterization of Nanite when the mesh is further than a given distance from the camera. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering)
	float NanitePixelProgrammableDistance;

	/** Distance at which to disable World Position Offset for an entire instance (0 = Never disable WPO). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	int32 WorldPositionOffsetDisableDistance = 0;

	/** Whether to evaluate World Position Offset. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bEvaluateWorldPositionOffset : 1;

	/** Forces this component to use fallback (non-Nanite) rendering even if the asset has Nanite resources. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bDisallowNanite : 1;

	/** Forces this component to use fallback mesh for rendering if Nanite is enabled on the mesh (run-time override). */
	UPROPERTY()
	uint8 bForceDisableNanite : 1;

	bool IsDisallowNanite() const { return bDisallowNanite; }
	bool IsForceDisableNanite() const { return bForceDisableNanite; }
	int32 GetWorldPositionOffsetDisableDistance() const { return WorldPositionOffsetDisableDistance; }
	bool EvaluateWorldPositionOffset() const { return bEvaluateWorldPositionOffset; }

protected:
	/** Non-URO-based interpolation alpha */
	float ExternalInterpolationAlpha;

	/* Non-URO-based delta time used to tick the pose */
	float ExternalDeltaTime;

public:
	/** LOD array info. Each index will correspond to the LOD index **/
	UPROPERTY(transient)
	TArray<struct FSkelMeshComponentLODInfo> LODInfo;

protected:
	/** Array of bone visibilities (containing one of the values in EBoneVisibilityStatus for each bone).  A bone is only visible if it is *exactly* 1 (BVS_Visible) */
	TArray<uint8> BoneVisibilityStates[2];

	/** Cache the scene feature level */
	ERHIFeatureLevel::Type CachedSceneFeatureLevel;

public:
	/*
	 * This is tick animation frequency option based on this component rendered or not or using montage
	 *  You can change this default value in the INI file 
	 * Mostly related with performance
	 */
	UPROPERTY(Interp, EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Config, Category=Optimization)
	EVisibilityBasedAnimTickOption VisibilityBasedAnimTickOption;

#if WITH_EDITOR
	UE_DEPRECATED(4.21, "MeshComponentUpdateFlag has been renamed VisibilityBasedAnimTickOption")
	uint8& MeshComponentUpdateFlag = *(uint8*)&VisibilityBasedAnimTickOption;
#endif

protected:
	/** Record of the tick rate we are using when externally controlled */
	uint8 ExternalTickRate;

	/** The highest streaming LOD index clamp among components with SyncAttachParentLOD enabled in the attachment tree */
	uint8 MaxFirstLODIdxInAttachmentTree;

protected:
	/** used to cache previous bone transform or not */
	uint8 bHasValidBoneTransform:1;

	/** used to cache our valid mesh pose transform list for the current animation update. */
	uint8 bValidMeshPoseTransformsUpToDate : 1;

	/** Whether or not a Skin Weight profile is currently set for this component */
	uint8 bSkinWeightProfileSet:1;

	/** Whether or not a Skin Weight profile is currently pending load and creation for this component */
	uint8 bSkinWeightProfilePending:1;

	/** Whether actual Skin Weight Profile buffer is currently pending override */
	uint8 bSkinWeightProfileBufferOverridePending : 1;
public:
	/** Do skin weight profile management on render thread exclusively */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bSkinWeightProfileOnRenderThread: 1;

	/** Whether we should use the min lod specified in MinLodModel for this component instead of the min lod in the mesh */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = LOD)
	uint8 bOverrideMinLod:1;

	/** 
	 * When true, we will just using the bounds from our LeaderPoseComponent.  This is useful for when we have a Mesh Parented
	 * to the main SkelMesh (e.g. outline mesh or a full body overdraw effect that is toggled) that is always going to be the same
	 * bounds as parent.  We want to do no calculations in that case.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bUseBoundsFromLeaderPoseComponent : 1;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property is deprecated. Please use bUseBoundsFromLeaderPoseComponent instead")
	uint8 bUseBoundsFromMasterPoseComponent : 1;
#endif // WITH_EDITORONLY_DATA

	/** If true, the Location of this Component will be included into its bounds calculation
	* (this can be useful when using SMU_OnlyTickPoseWhenRendered on a character that moves away from the root and no bones are left near the origin of the component) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = SkeletalMesh)
	uint8 bIncludeComponentLocationIntoBounds : 1;

	/** Forces the mesh to draw in wireframe mode. */
	UPROPERTY()
	uint8 bForceWireframe:1;

#if WITH_EDITORONLY_DATA
	/** Draw the skeleton hierarchy for this skel mesh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Rendering)
	uint8 bDisplayBones:1;
#endif

	/** Disable Morphtarget for this component. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = SkeletalMesh)
	uint8 bDisableMorphTarget:1;

	/** Don't bother rendering the skin. */
	UPROPERTY()
	uint8 bHideSkin:1;
	/**
	 *	If true, use per-bone motion blur on this skeletal mesh (requires additional rendering, can be disabled to save performance).
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=SkeletalMesh)
	uint8 bPerBoneMotionBlur:1;

	//
	// Misc.
	//
	
	/** When true, skip using the physics asset etc. and always use the fixed bounds defined in the SkeletalMesh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Optimization)
	uint8 bComponentUseFixedSkelBounds:1;

	/** If true, when updating bounds from a PhysicsAsset, consider _all_ BodySetups, not just those flagged with bConsiderForBounds. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Optimization)
	uint8 bConsiderAllBodiesForBounds:1;

	/** If true, this component uses its parents LOD when attached if available
	* ForcedLOD can override this change. By default, it will use parent LOD.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Rendering)
	uint8 bSyncAttachParentLOD : 1;


	/** Whether or not we can highlight selected sections - this should really only be done in the editor */
	UPROPERTY(transient)
	uint8 bCanHighlightSelectedSections:1;

	/** true if mesh has been recently rendered, false otherwise */
	UPROPERTY(transient)
	uint8 bRecentlyRendered:1;

	/** 
	 * Whether to use the capsule representation (when present) from a skeletal mesh's ShadowPhysicsAsset for direct shadowing from lights.
	 * This type of shadowing is approximate but handles extremely wide area shadowing well.  The softness of the shadow depends on the light's LightSourceAngle / SourceRadius.
	 * This flag will force bCastInsetShadow to be enabled. This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Capsule Direct Shadow"))
	uint8 bCastCapsuleDirectShadow:1;

	/** 
	 * Whether to use the capsule representation (when present) from a skeletal mesh's ShadowPhysicsAsset for shadowing indirect lighting (from lightmaps or skylight).
	 * This flag is only used if CastShadow is true and if FirstPersonPrimitiveType is not set to FirstPerson.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="CastShadow && (FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson)", DisplayName = "Capsule Indirect Shadow"))
	uint8 bCastCapsuleIndirectShadow:1;

	/** Whether or not to CPU skin this component, requires render data refresh after changing */
	UE_DEPRECATED(4.24, "Direct access to bCPUSkinning is deprecated. Please use its getter and setter instead.")
	UPROPERTY(transient)
	uint8 bCPUSkinning : 1;

	/** 
	 * If set, use the screen render flag instead of the default render flag when processing offscreen-rendering optimizations 
	 * (such as VisibilityBasedAnimTickOption) that look to reduce animation work when the mesh is not rendered. 
	 * Using this option can result in meshes that are occlusion culled ceasing to perform animation work.
	 * Note that this can however result in shadows not being animated when meshes are not directly visible.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Optimization)
	uint8 bUseScreenRenderStateForUpdate:1;

	// Update Rate
	/** if TRUE, Owner will determine how often animation will be updated and evaluated. See AnimUpdateRateTick() 
	 * This allows to skip frames for performance. (For example based on visibility and size on screen). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Optimization)
	uint8 bEnableUpdateRateOptimizations:1;

	/** Enable on screen debugging of update rate optimization. 
	 * Red = Skipping 0 frames, Green = skipping 1 frame, Blue = skipping 2 frames, black = skipping more than 2 frames. 
	 * @todo: turn this into a console command. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Optimization)
	uint8 bDisplayDebugUpdateRateOptimizations:1;

	/**
	 *	If true, render as static in reference pose.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Optimization)
	uint8 bRenderStatic:1;

	/** Flag that when set will ensure UpdateLODStatus will not take the LeaderPoseComponent's current LOD in consideration when determining the correct LOD level (this requires LeaderPoseComponent's LOD to always be >= determined LOD otherwise bone transforms could be missing */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = LOD)
	uint8 bIgnoreLeaderPoseComponentLOD : 1;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property is deprecated. Please use bIgnoreLeaderPoseComponentLOD instead")
	uint8 bIgnoreMasterPoseComponentLOD : 1;
#endif // WITH_EDITORONLY_DATA

	/** Enable dynamic sort mesh's triangles to remove ordering issue when rendered with a translucent material */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (UIMin = "0", UIMax = "1", DisplayName = "Sort Triangles"))
	uint8 bSortTriangles : 1;

protected:
	/** Are we using double buffered ComponentSpaceTransforms */
	uint8 bDoubleBufferedComponentSpaceTransforms : 1;

	/** Track whether we still need to flip to recently modified buffer */
	uint8 bNeedToFlipSpaceBaseBuffers : 1;

	/** true when CachedLocalBounds is up to date. */
	UPROPERTY(Transient)
	mutable uint8 bCachedLocalBoundsUpToDate:1;
	UPROPERTY(Transient)
	mutable uint8 bCachedWorldSpaceBoundsUpToDate:1;

	/** Whether we have updated bone visibility this tick */
	uint8 bBoneVisibilityDirty:1;

private:
	/** If true, UpdateTransform will always result in a call to MeshObject->Update. */
	UPROPERTY(transient)
	uint8 bForceMeshObjectUpdate:1;

protected:
	/** Whether we are externally controlling tick rate */
	uint8 bExternalTickRateControlled:1;

	/** Non URO-based interpolation flag */
	uint8 bExternalInterpolate:1;

	/** Non URO-based update flag */
	uint8 bExternalUpdate:1;

	/** External flag indicating that we may not be evaluated every frame */
	uint8 bExternalEvaluationRateLimited:1;

	/** Whether mip callbacks have been registered and need to be removed on destroy */
	uint8 bMipLevelCallbackRegistered:1;

	/** If false, Follower components ShouldTickPose function will return false (default) */
	UPROPERTY(Transient)
	uint8 bFollowerShouldTickPose : 1;

	/** Whether this component currently has any attachments that are driven by sockets */
	uint8 bHasSocketAttachments : 1;

#if UE_ENABLE_DEBUG_DRAWING
private:
	/** Whether to draw this mesh's debug skeleton (regardless of showflags) */
	uint8 bDrawDebugSkeleton:1;
#endif

public:
	/** Set whether we have our tick rate externally controlled non-URO-based interpolation */
	void EnableExternalTickRateControl(bool bInEnable) { bExternalTickRateControlled = bInEnable; }

	/** Check whether we we have our tick rate externally controlled */
	bool IsUsingExternalTickRateControl() const { return bExternalTickRateControlled; }

	/** Set the external tick rate */
	void SetExternalTickRate(uint8 InTickRate) { ExternalTickRate = InTickRate; }

	/** Get the external tick rate */
	uint8 GetExternalTickRate() const { return ExternalTickRate; }

	/** Enable non-URO-based interpolation */
	void EnableExternalInterpolation(bool bInEnable) { bExternalInterpolate = bInEnable; }

	/** Check whether we should be interpolating due to external settings */
	bool IsUsingExternalInterpolation() const { return bExternalInterpolate && bExternalEvaluationRateLimited; }

	/** Set us to tick this frame for non-URO-based interpolation */
	void EnableExternalUpdate(bool bInEnable) { bExternalUpdate = bInEnable; }

	/** Set non-URO-based interpolation alpha */
	void SetExternalInterpolationAlpha(float InAlpha) { ExternalInterpolationAlpha = InAlpha; }

	/** Set non-URO-based delta time */
	void SetExternalDeltaTime(float InDeltaTime) { ExternalDeltaTime = InDeltaTime; }

	/** Manually enable/disable animation evaluation rate limiting */
	void EnableExternalEvaluationRateLimiting(bool bInEnable) { bExternalEvaluationRateLimited = bInEnable; }

	/** Retrieve the mesh object currently assigned to this component */
	const FSkeletalMeshObject* GetMeshObject() const { return MeshObject; }

	/** 
	 * Controls how dark the capsule indirect shadow can be.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(UIMin = "0", UIMax = "1", EditCondition="bCastCapsuleIndirectShadow", DisplayName = "Capsule Indirect Shadow Min Visibility"))
	float CapsuleIndirectShadowMinVisibility;

	/** Object responsible for sending bone transforms, morph target state etc. to render thread. */
	FSkeletalMeshObject*	MeshObject;

	/** Supports user-defined FSkeletalMeshObjects */
	FSkeletalMeshObject* (*MeshObjectFactory)(void* UserData, USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);

	/** Passed into MeshObjectFactory */
	void* MeshObjectFactoryUserData;

	/** Previous copy of MeshObject set during recreate of the render state */
	FSkeletalMeshObject* PreviousMeshObject;

	/** Gets the skeletal mesh resource used for rendering the component. */
	ENGINE_API FSkeletalMeshRenderData* GetSkeletalMeshRenderData() const;

	/** Override this function to pass on cloth simulation data (or any deformable using the cloth pipeline) to the skeletal renderer. */
	ENGINE_API virtual void GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutClothBlendWeight) const override;

	/** 
	 * Override the Physics Asset of the mesh. It uses SkeletalMesh.PhysicsAsset, but if you'd like to override use this function
	 * 
	 * @param	NewPhysicsAsset	New PhysicsAsset
	 * @param	bForceReInit	Force reinitialize
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API virtual void SetPhysicsAsset(class UPhysicsAsset* NewPhysicsAsset, bool bForceReInit = false);

	/** Get the number of LODs on this component */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh", meta=(ScriptName="GetNumLods;GetNumLODs"))
	ENGINE_API int32 GetNumLODs() const;

	/**
	 * Set MinLodModel of the mesh component
	 *
	 * @param	InNewMinLOD	Set new MinLodModel that make sure the LOD does not go below of this value. Range from [0, Max Number of LOD - 1]. This will affect in the next tick update. 
	 */
	UE_DEPRECATED(5.5, "Use USkinnedMeshComponent::OverrideMinLOD() instead.")
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh", meta = (DeprecatedFunction, DeprecationMessage = "Use USkinnedMeshComponent::OverrideMinLOD() instead."))
	ENGINE_API void SetMinLOD(int32 InNewMinLOD);

	/**
	 * Override the Min LOD of the mesh component
	 *
	 * @param	InNewMinLOD	Override new MinLodModel that make sure the LOD does not go below of this value. Range from [0, Max Number of LOD - 1]. This will affect in the next tick update.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void OverrideMinLOD(int32 InNewMinLOD);

	/**
	 * Set ForcedLodModel of the mesh component
	 *
	 * @param	InNewForcedLOD	Set new ForcedLODModel that forces to set the incoming LOD. Range from [1, Max Number of LOD]. This will affect in the next tick update. 
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API void SetForcedLOD(int32 InNewForcedLOD);

	/** Get ForcedLodModel of the mesh component. Note that the actual forced LOD level is the return value minus one and zero means no forced LOD */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API int32 GetForcedLOD() const;

#if WITH_EDITOR
	/**
	 * Get the LOD Bias of this component
	 *
	 * @return	The LOD bias of this component. Derived classes can override this to ignore or override LOD bias settings.
	 */
	ENGINE_API virtual int32 GetLODBias() const;
#endif

	UFUNCTION(BlueprintCallable, Category="Lighting")
	ENGINE_API void SetCastCapsuleDirectShadow(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Lighting")
	ENGINE_API void SetCastCapsuleIndirectShadow(bool bNewValue);

	UFUNCTION(BlueprintCallable, Category="Lighting")
	ENGINE_API void SetCapsuleIndirectShadowMinVisibility(float NewValue);

	/** Set whether World Position Offset is evaluated for this component. Toggles WPO on/off without rebuilding the proxy. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	ENGINE_API void SetEvaluateWorldPositionOffset(bool bNewValue);

	/** Set the distance beyond which World Position Offset is suppressed for this component (0 disables). */
	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	ENGINE_API void SetWorldPositionOffsetDisableDistance(int32 NewValue);

	/** Force disable Nanite for this component at runtime. When true, the component renders via the classic skinned path. */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetForceDisableNanite(bool bInForceDisableNanite);

	/**
	*  Returns the number of bones in the skeleton.
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API int32 GetNumBones() const;

	/**
	 * Find the index of bone by name. Looks in the current SkeletalMesh being used by this SkeletalMeshComponent.
	 * 
	 * @param BoneName Name of bone to look up
	 * 
	 * @return Index of the named bone in the current SkeletalMesh. Will return INDEX_NONE if bone not found.
	 *
	 * @see USkeletalMesh::GetBoneIndex.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API int32 GetBoneIndex( FName BoneName ) const;

	/** 
	 * Get Bone Name from index
	 * @param BoneIndex Index of the bone
	 *
	 * @return the name of the bone at the specified index 
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API FName GetBoneName(int32 BoneIndex) const;

	/**
	 * Returns bone name linked to a given named socket on the skeletal mesh component.
	 * If you're unsure to deal with sockets or bones names, you can use this function to filter through, and always return the bone name.
	 *
	 * @param	bone name or socket name
	 *
	 * @return	bone name
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API FName GetSocketBoneName(FName InSocketName) const;

	/** 
	 * Change the SkeletalMesh that is rendered for this Component. Will re-initialize the animation tree etc. 
	 *
	 * @param NewMesh New mesh to set for this component
	 * @param bReinitPose Whether we should keep current pose or reinitialize.
	 */
	UE_DEPRECATED(5.1, "Use USkeletalMeshComponent::SetSkeletalMesh() or SetSkinnedAssetAndUpdate() instead.")
	ENGINE_API virtual void SetSkeletalMesh(class USkeletalMesh* NewMesh, bool bReinitPose = true);

	/**
	 * Get the SkeletalMesh rendered for this mesh.
	 * This function is not technically deprecated but shouldn't be used other than for Blueprint backward compatibility purposes.
	 * It is used to access the correct SkinnedAsset pointer value through the deprecated SkeletalMesh property in blueprints.
	 * 
	 * @return the SkeletalMesh set to this mesh.
	 */
	UE_DEPRECATED(5.1, "Use USkeletalMeshComponent::GetSkeletalMeshAsset() or GetSkinnedAsset() instead.")
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh", DisplayName = "Get Skeletal Mesh", meta = (DeprecatedFunction, DeprecationMessage = "Use USkeletalMeshComponent::GetSkeletalMeshAsset() or GetSkinnedAsset() instead."))
	ENGINE_API class USkeletalMesh* GetSkeletalMesh_DEPRECATED() const;

	UE_DEPRECATED(5.1, "Use USkeletalMeshComponent::SetSkinnedAssetAndUpdate() instead.")
	ENGINE_API void SetSkeletalMesh_DEPRECATED(USkeletalMesh* NewMesh);

	/**
	 * Change the SkinnedAsset that is rendered for this Component. Will re-initialize the animation tree etc.
	 *
	 * @param NewMesh New mesh to set for this component
	 * @param bReinitPose Whether we should keep current pose or reinitialize.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API virtual void SetSkinnedAssetAndUpdate(class USkinnedAsset* NewMesh, bool bReinitPose = true);

	/**
	 * Change the SkinnedAsset that is rendered without reinitializing this Component.
	 *
	 * @param InSkinnedAsset New mesh to set for this component
	 */
	ENGINE_API void SetSkinnedAsset(class USkinnedAsset* InSkinnedAsset);

	/**
	 * Get the SkinnedAsset rendered for this mesh.
	 *
	 * @return the SkinnedAsset set to this mesh.
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh")
	ENGINE_API USkinnedAsset* GetSkinnedAsset() const;

	/**
	 * Change the MeshDeformer that is used for this Component.
	 *
	 * @param InMeshDeformer New mesh deformer to set for this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void SetMeshDeformer(UMeshDeformer* InMeshDeformer);

	/**
	 * Unset any MeshDeformer applied to this Component.
	 *
	 * @param InMeshDeformer New mesh deformer to set for this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void UnsetMeshDeformer();

	/**
	 * Always use a MeshDeformer as long as one can be found in the project settings
	 *
	 * @param bShouldAlwaysUseMeshDeformer Always use mesh deformer for this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void SetAlwaysUseMeshDeformer(bool bShouldAlwaysUseMeshDeformer);

	/**
	 * Returns whether the component is set to always use a mesh deformer if one can be found in the project settings
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API bool GetAlwaysUseMeshDeformer() const;
	
	/** 
	 * Get Parent Bone of the input bone
	 * 
	 * @param BoneName Name of the bone
	 *
	 * @return the name of the parent bone for the specified bone. Returns 'None' if the bone does not exist or it is the root bone 
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API FName GetParentBone(FName BoneName) const;

	/**
	* Get delta transform from reference pose based on BaseNode.
	* This uses last frame up-to-date transform, so it will have a frame delay if you use this info in the AnimGraph
	*
	* @param BoneName Name of the bone
	* @param BaseName Name of the base bone - if none, it will use parent as a base
	* 
	* @return the delta transform from refpose in that given space (BaseName)
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API FTransform GetDeltaTransformFromRefPose(FName BoneName, FName BaseName = NAME_None) const;

	/** 
	 * Get Twist and Swing Angle in Degree of Delta Rotation from Reference Pose in Local space 
	 *
	 * First this function gets rotation of current, and rotation of ref pose in local space, and 
	 * And gets twist/swing angle value from refpose aligned. 
	 * 
	 * @param BoneName Name of the bone
	 * @param OutTwistAngle TwistAngle in degree
	 * @param OutSwingAngle SwingAngle in degree
	 *
	 * @return true if succeed. False otherwise. Often due to incorrect bone name. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API bool GetTwistAndSwingAngleOfDeltaRotationFromRefPose(FName BoneName, float& OutTwistAngle, float& OutSwingAngle) const;

	ENGINE_API bool IsSkinCacheAllowed(int32 LodIdx) const;
	ENGINE_API bool IsSkinCacheAllowed(int32 LodIdx, ERHIFeatureLevel::Type FeatureLevel) const;

	ENGINE_API bool HasMeshDeformer() const;
	ENGINE_API bool HasMeshDeformer(int32 LodIdx) const;

	UE_DEPRECATED(5.5, "This is no longer relevant. Dynamic data is always updated immediately.")
	bool GetForceUpdateDynamicDataImmediately() const { return true; }

	UE_DEPRECATED(5.5, "This is no longer relevant. Dynamic data is always updated immediately.")
	void SetForceUpdateDynamicDataImmediately(bool bForceUpdateImmediately) {}

	/**
	 *	Compute SkeletalMesh MinLOD that will be used by this component
	 */
	ENGINE_API int32 ComputeMinLOD() const;

	/**
	 * Validate the min LOD value of the mesh component by iterating over render data to make sure we get something usable.
	 *
	 * @param	InMinLOD	The starting LOD index to iterate from to make sure the render data is valid. Range from [0, Max Number of LOD - 1].
	 */
	int32 GetValidMinLOD(const int32 InMinLODIndex) const;

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostApplyToComponent() override;
	ENGINE_API virtual void PostReinitProperties() override;
	ENGINE_API virtual void PostInitProperties() override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

protected:
	//~ Begin UActorComponent Interface
	ENGINE_API virtual void InitializeComponent() override;
	ENGINE_API virtual void OnPreRegister() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderDynamicData_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual bool RequiresGameThreadEndOfFrameRecreate() const override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual UObject const* AdditionalStatObject() const override;
	//~ End UActorComponent Interface

public:
	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual void UpdateBounds() override;
	ENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	ENGINE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	ENGINE_API virtual bool HasAnySockets() const override;
	ENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	ENGINE_API virtual bool UpdateOverlapsImpl(const TOverlapArrayView* PendingOverlaps=nullptr, bool bDoNotifies=true, const TOverlapArrayView* OverlapsAtEndLocation=nullptr) override;
	ENGINE_API virtual void OnChildAttached(USceneComponent* ChildComponent) override;
	ENGINE_API virtual void OnChildDetached(USceneComponent* ChildComponent) override;
	//~ End USceneComponent Interface

	//~ Begin UPrimitiveComponent Interface
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	ENGINE_API virtual int32 GetMaterialIndex(FName MaterialSlotName) const override;
	ENGINE_API virtual TArray<FName> GetMaterialSlotNames() const override;
	ENGINE_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	ENGINE_API virtual bool GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const override;
	ENGINE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	ENGINE_API virtual void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex) override;
	ENGINE_API virtual int32 GetNumMaterials() const override;
	virtual float GetStreamingScale() const override { return GetComponentTransform().GetMaximumAxisScale(); }
	ENGINE_API virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const override;
	//~ End UPrimitiveComponent Interface

	//~ Begin UMeshComponent Interface
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& Callback, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn) override;
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& CallbackStreamingStart, FLODStreamingCallback&& CallbackStreamingDone, float TimeoutStartSecs, float TimeoutDoneSecs) override;
	ENGINE_API virtual bool PrestreamMeshLODs(float Seconds) override;
	//~ End UMeshComponent Interface

	/** Get the pre-skinning local space bounds for this component. */
	ENGINE_API void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const;

	/** Resize the morph target sets array to the number of LODs. */
	ENGINE_API void ResizeExternalMorphTargetSets();

	/** Refresh the external morph target weight buffers. This makes sure the amount of morph sets and number of weights are valid. */
	ENGINE_API void RefreshExternalMorphTargetWeights(bool bZeroOldWeights = true);

	/**
	 *	Sets the value of the bForceWireframe flag and reattaches the component as necessary.
	 *
	 *	@param	InForceWireframe		New value of bForceWireframe.
	 */
	ENGINE_API void SetForceWireframe(bool InForceWireframe);

	/** Precache all PSOs which can be used by the component */
	ENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	
#if WITH_EDITOR
	/** Return value of SectionIndexPreview  */
	int32 GetSectionPreview() const { return SectionIndexPreview;  }
	/** Sets the value of the SectionIndexPreview option. */
	ENGINE_API void SetSectionPreview(int32 InSectionIndexPreview);

	/** Return value of MaterialIndexPreview  */
	int32 GetMaterialPreview() const { return MaterialIndexPreview; }
	/** Sets the value of the MaterialIndexPreview option. */
	ENGINE_API void SetMaterialPreview(int32 InMaterialIndexPreview);

	/** Return value of SelectedEditorSection  */
	int32 GetSelectedEditorSection() const { return SelectedEditorSection; }
	/** Sets the value of the SelectedEditorSection option. */
	ENGINE_API void SetSelectedEditorSection(int32 NewSelectedEditorSection);

	/** Return value of SelectedEditorMaterial  */
	int32 GetSelectedEditorMaterial() const { return SelectedEditorMaterial; }
	/** Sets the value of the SelectedEditorMaterial option. */
	ENGINE_API void SetSelectedEditorMaterial(int32 NewSelectedEditorMaterial);

#endif // WITH_EDITOR

	/** Returns true if skinned meshes are allowed to be rendered as Nanite when enabled on the mesh. */
	static ENGINE_API bool ShouldRenderNaniteSkinnedMeshes();

	/** Function returns whether Nanite should be used to render and skin this mesh. */
	ENGINE_API virtual bool ShouldNaniteSkin();

	/**
	 * Function returns whether or not CPU skinning should be applied
	 * Allows the editor to override the skinning state for editor tools
	 *
	 * @return true if should CPU skin. false otherwise
	 */
	ENGINE_API virtual bool ShouldCPUSkin() const;

	/**
	 * Getter for bCPUSkinning member variable
	 * May return a different value from ShouldCPUSkin()
	 */
	ENGINE_API bool GetCPUSkinningEnabled() const;

	/**
	 * Set whether this component uses CPU skinning
	 * Notes:
	 * - If enabled, skeletal mesh referenced by this component will be removed from streaming manager
	 * - Streaming cannot be (re)enabled as long as any component uses CPU skinning
	 * - This function is expensive
	 */
	ENGINE_API void SetCPUSkinningEnabled(bool bEnable, bool bRecreateRenderStateImmediately = false);

	/** 
	 * Function to operate on mesh object after its created, 
	 * but before it's attached.
	 * 
	 * @param MeshObject - Mesh Object owned by this component
	 */
	virtual void PostInitMeshObject(FSkeletalMeshObject*) {}

	/**
	* Simple, CPU evaluation of a vertex's skinned position (returned in component space)
	*
	* @param VertexIndex Vertex Index. If compressed, this will be slow.
	* @param Model The Model to use.
	* @param SkinWeightBuffer The SkinWeightBuffer to use.
	* @param CachedRefToLocals Cached RefToLocal matrices.
	*/
	static ENGINE_API FVector3f GetSkinnedVertexPosition(USkinnedMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODDatal, FSkinWeightVertexBuffer& SkinWeightBuffer);

	/**
	 * Simple, CPU evaluation of a vertex's skinned position (returned in component space)
	 *
	 * @param VertexIndex Vertex Index. If compressed, this will be slow.
	 * @param Model The Model to use.
	 * @param SkinWeightBuffer The SkinWeightBuffer to use.
	 * @param CachedRefToLocals Cached RefToLocal matrices.
	*/
	static ENGINE_API FVector3f GetSkinnedVertexPosition(USkinnedMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals);

	/**
	* CPU evaluation of the positions of all vertices (returned in component space)
	*
	* @param OutPositions buffer to place positions into
	* @param CachedRefToLocals Cached RefToLocal matrices.
	* @param Model The Model to use.
	* @param SkinWeightBuffer The SkinWeightBuffer to use.
	*/
	static ENGINE_API void ComputeSkinnedPositions(USkinnedMeshComponent* Component, TArray<FVector3f> & OutPositions, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer);

	/** Caches the RefToLocal matrices. */
	ENGINE_API void CacheRefToLocalMatrices(TArray<FMatrix44f>& OutRefToLocal) const;

	/** Return the skinning matrices used for rendering. */
	ENGINE_API void GetCurrentRefToLocalMatrices(TArray<FMatrix44f>& OutRefToLocals, int32 InLodIdx, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices = nullptr) const;

	inline	const USkinnedMeshComponent* GetBaseComponent()const
	{
		return LeaderPoseComponent.IsValid() ? LeaderPoseComponent.Get() : this;
	}

	/** 
	 * Get world-space bone transform.
	 * @param InBoneName Name of the the bone to get the transform 
	 * @return Bone transform in world space if bone is found. Otherwise it will return component's transform in world space.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(Keywords="Bone"))
	FTransform GetBoneTransform(FName InBoneName, ERelativeTransformSpace TransformSpace = RTS_World) const { return GetSocketTransform(InBoneName, TransformSpace); }

	/**
	* Returns color of the vertex.
	*
	* @param VertexIndex Vertex Index. If compressed, this will be slow.
	*/
	ENGINE_API FColor GetVertexColor(int32 VertexIndex) const;

	/** Allow override of vertex colors on a per-component basis. */
	ENGINE_API void SetVertexColorOverride(int32 LODIndex, const TArray<FColor>& VertexColors);

	/** Allow override of vertex colors on a per-component basis, taking array of Blueprint-friendly LinearColors. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh", meta = (DisplayName = "Set Vertex Color Override"))
	ENGINE_API void SetVertexColorOverride_LinearColor(int32 LODIndex, const TArray<FLinearColor>& VertexColors);

	/** Clear any applied vertex color override */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ClearVertexColorOverride(int32 LODIndex);
	/**
	* Returns texture coordinates of the vertex.
	*
	* @param VertexIndex		Vertex Index. If compressed, this will be slow.
	* @param TexCoordChannel	Texture coordinate channel Index.
	*/
	ENGINE_API FVector2D GetVertexUV(int32 VertexIndex, uint32 UVChannel) const;

	/** Allow override of skin weights on a per-component basis. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void SetSkinWeightOverride(int32 LODIndex, const TArray<FSkelMeshSkinWeightInfo>& SkinWeights);

	/** Clear any applied skin weight override */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ClearSkinWeightOverride(int32 LODIndex);

	/** Set up an override skin weight profile for this component on the given layer.
	 *  The values from the secondary layer (if set to have a profile) are applied first, followed by the values from the primary layer.
	 *  Since skin weight profiles are stored as sparse data, where only weight values different from the base are kept in storage, it's
	 *  possible to set up layers such that they don't interfere with one another.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API bool SetSkinWeightProfile(FName InProfileName, ESkinWeightProfileLayer InLayer = ESkinWeightProfileLayer::Primary);

	/** Clear the skin weight profile from the given layer on this component, in case it is set. If no profile is set for the layer,
	 *  then this call does nothing. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ClearSkinWeightProfile(ESkinWeightProfileLayer InLayer = ESkinWeightProfileLayer::Primary);

	/** Clear the skin Weight Profile from all layers on this component. If no profiles are set for any layer, then this call does nothing. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ClearAllSkinWeightProfiles();
	
	/** Unload a Skin Weight Profile's skin weight buffer (if created) */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void UnloadSkinWeightProfile(FName InProfileName);

	/** Return the name of the skin weight profile that is currently set on the given layer, otherwise returns 'None' */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API FName GetCurrentSkinWeightProfileName(ESkinWeightProfileLayer InLayer = ESkinWeightProfileLayer::Primary) const;

	/** Return the names of the skin weight profiles for all the layers */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API TArray<FName> GetCurrentSkinWeightProfileLayerNames() const;
	
	/** Check whether a skin weight profile is currently set on any layer. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API bool IsUsingSkinWeightProfile() const;

	/** Check whether a skin weight profile is currently pending load / create */
	bool IsSkinWeightProfilePending() const { return bSkinWeightProfilePending; }

	bool IsSkinWeightProfileManagementOnRenderThread() const;

	/** Queues an update of the Skin Weight Buffer used by the current MeshObject */
	ENGINE_API void UpdateSkinWeightOverrideBuffer();
protected:
	/** Set the currently active skin weight profile stack to the given stack. If the stack is empty, then the skin weight profile buffer
	  * will be set to the base buffer for this mesh.
	  */
	bool SetSkinWeightProfileStack(const FSkinWeightProfileStack& InProfileStack);

	/** Name of currently set up Skin Weight layers, all set to NAME_None for no override */
	FName CurrentSkinWeightProfileLayers[2] = {NAME_None};
public:
	/** Returns skin weight vertex buffer to use for specific LOD (will look at override) */
	ENGINE_API FSkinWeightVertexBuffer* GetSkinWeightBuffer(int32 LODIndex) const;

	/** Apply an override for the current mesh ref pose */
	ENGINE_API virtual void SetRefPoseOverride(const TArray<FTransform>& NewRefPoseTransforms);

	/** Accessor for RefPoseOverride */
	virtual const TSharedPtr<FSkelMeshRefPoseOverride>& GetRefPoseOverride() const { return RefPoseOverride; }

	/** Clear any applied ref pose override */
	ENGINE_API virtual void ClearRefPoseOverride();

	/**
	 * Update functions
	 */

	/** 
	 * Refresh Bone Transforms
	 * Each class will need to implement this function
	 * Ideally this function should be atomic (not relying on Tick or any other update.) 
	 * 
	 * @param TickFunction Supplied as non null if we are running in a tick, allows us to create graph tasks for parallelism
	 * 
	 */
	virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = nullptr) PURE_VIRTUAL(USkinnedMeshComponent::RefreshBoneTransforms, );

protected:
	/** 
	 * Parallel Tick Pose
	 * In the case where we do not want to refresh bone transforms (and would therefore not normally kick off a parallel eval task)
	 * we perform this 'mini tick' that kicks off the task.
	 * 
	 * @param TickFunction Allows us to create graph tasks for parallelism
	 * 
	 */
	virtual void DispatchParallelTickPose(FActorComponentTickFunction* TickFunction) {}

	/** Helper function for UpdateLODStatus, called with a valid index for InLeaderPoseComponentPredictedLODLevel when updating LOD status for follower components */
	ENGINE_API bool UpdateLODStatus_Internal(int32 InLeaderPoseComponentPredictedLODLevel, bool bRequestedByLeaderPoseComponent = false);

	/** Traverse the attachment tree and compute the highest streaming LOD index clamp among components with SyncAttachParentLOD enabled */
	uint8 ComputeMaxFirstLODIdxInAttachmentTree() const;

private:

	/** Called by OnRegister/OnPreRegister before calling Super. */
	void InitializeBeforeRegister();

	/** Called by OnRegister/OnPreRegister after calling Super. */
	void InitializeOnRegister();

public:
	/**
	 * Tick Pose, this function ticks and do whatever it needs to do in this frame, should be called before RefreshBoneTransforms
	 *
	 * @param DeltaTime DeltaTime
	 *
	 * @return	Return true if anything modified. Return false otherwise
	 * @param bNeedsValidRootMotion - Networked games care more about this, but if false we can do less calculations
	 */
	ENGINE_API virtual void TickPose(float DeltaTime, bool bNeedsValidRootMotion);

	/**
	 * Invoked at the beginning of TickPose before doing the bulk of the tick work
	 */
	FOnTickPose OnTickPose;

	/** 
	 * Update Follower Component. This gets called when LeaderPoseComponent!=nullptr
	 * 
	 */
	ENGINE_API virtual void UpdateFollowerComponent();

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use UpdateFollowerComponent instead.")
	virtual void UpdateSlaveComponent() { UpdateFollowerComponent(); }

	/** 
	 * Update the PredictedLODLevel and MaxDistanceFactor in the component from its MeshObject. 
	 * 
	 * @return true if LOD has been changed. false otherwise.
	 */
	ENGINE_API virtual bool UpdateLODStatus();

	/** Get predicted LOD level. This value is usually calculated in UpdateLODStatus, but can be modified by skeletal mesh streaming. */
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh")
	int32 GetPredictedLODLevel() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PredictedLODLevel;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Set predicted LOD level. */
	virtual void SetPredictedLODLevel(int32 InPredictedLODLevel)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PredictedLODLevel = InPredictedLODLevel;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

protected:
	friend class FSkeletalMeshStreamOut;

	inline TBitArray<>& GetMutableValidMeshPoseTransforms() { return ValidMeshPoseTransforms; }

public:
	virtual void UpdateVisualizeLODString(FString& DebugString) {}

	/**
	 * Finalize bone transform of this current tick
	 * After this function, any query to bone transform should be latest of the data
	 */
	ENGINE_API virtual void FinalizeBoneTransform();

	/** Initialize the LOD entries for the component */
	ENGINE_API void InitLODInfos();

	/**
	 * Rebuild BoneVisibilityStates array. Mostly refresh information of bones for BVS_HiddenByParent 
	 */
	ENGINE_API void RebuildVisibilityArray();

	/**
	 * Checks/updates material usage on proxy based on current morph target usage
	 */
	ENGINE_API void UpdateMorphMaterialUsageOnProxy();

	/**
	 * Toggles whether this skinned mesh component can issue duplicate transform updates
	 */
	UE_DEPRECATED(5.8, "Duplicate transform control will be removed.")
	ENGINE_API void AllowDuplicateTransformUpdates(bool bAllow);

	/** Access ComponentSpaceTransforms for reading */
	const TArray<FTransform>& GetComponentSpaceTransforms() const 
	{ 
		return ComponentSpaceTransformsArray[CurrentReadComponentTransforms]; 
	}

	/** Get Access to the current editable space bases */
	TArray<FTransform>& GetEditableComponentSpaceTransforms() 
	{
		return ComponentSpaceTransformsArray[CurrentEditableComponentTransforms];
	}
	const TArray<FTransform>& GetEditableComponentSpaceTransforms() const
	{ 
		return ComponentSpaceTransformsArray[CurrentEditableComponentTransforms];
	}

public:
	/** Get current number of component space transorms */
	int32 GetNumComponentSpaceTransforms() const 
	{ 
		return GetComponentSpaceTransforms().Num(); 
	}

	inline const TBitArray<>& GetValidMeshPoseTransforms() const { return ValidMeshPoseTransforms; }

	/** Access BoneVisibilityStates for reading */
	const TArray<uint8>& GetBoneVisibilityStates() const 
	{
		return BoneVisibilityStates[CurrentReadComponentTransforms];
	}

	/** Get Access to the current editable bone visibility states */
	TArray<uint8>& GetEditableBoneVisibilityStates() 
	{
		return BoneVisibilityStates[CurrentEditableComponentTransforms];
	}
	const TArray<uint8>& GetEditableBoneVisibilityStates() const
	{
		return BoneVisibilityStates[CurrentEditableComponentTransforms];
	}

	ENGINE_API void SetComponentSpaceTransformsDoubleBuffering(bool bInDoubleBufferedComponentSpaceTransforms);

	FBoxSphereBounds GetCachedLocalBounds()  const
	{ 
		ensure(bCachedLocalBoundsUpToDate || bCachedWorldSpaceBoundsUpToDate);
		if (bCachedWorldSpaceBoundsUpToDate)
		{
			return CachedWorldOrLocalSpaceBounds.TransformBy(CachedWorldToLocalTransform);
		}
		else
		{
			return CachedWorldOrLocalSpaceBounds;
		}
	} 

	/**
	* Should update transform in Tick
	*
	* @param bLODHasChanged	: Has LOD been changed since last time?
	*
	* @return : return true if need transform update. false otherwise.
	*/
	ENGINE_API virtual bool ShouldUpdateTransform(bool bLODHasChanged) const;

	class FRenderStateLockScope
	{
	public:
		FRenderStateLockScope(USkinnedMeshComponent* InComponent)
			: Lock(InComponent->RenderStateLock)
		{}

	private:
		UE::TScopeLock<UE::FRecursiveMutex> Lock;
	};

protected:
	friend class FRenderStateLockScope;
	UE::FRecursiveMutex RenderStateLock;

	/** Flip the editable space base buffer */
	ENGINE_API void FlipEditableSpaceBases();

public:
	/** 
	 * Should tick  pose (by calling TickPose) in Tick
	 * 
	 * @return : return true if should Tick. false otherwise.
	 */
	ENGINE_API virtual bool ShouldTickPose() const;
protected:

	/**
	 * Allocate Transform Data array including SpaceBases, BoneVisibilityStates 
	 *
	 */	
	ENGINE_API virtual bool AllocateTransformData();
	ENGINE_API virtual void DeallocateTransformData();

	/** Bounds cached, so they're computed just once, either in local or worldspace depending on cvar 'a.CacheLocalSpaceBounds'. */
	UPROPERTY(Transient)
	mutable FBoxSphereBounds CachedWorldOrLocalSpaceBounds;
	UPROPERTY(Transient)
	mutable FMatrix CachedWorldToLocalTransform;

public:
#if WITH_EDITOR
	//~ Begin IInterface_AsyncCompilation Interface.
	ENGINE_API virtual bool IsCompiling() const override;
	//~ End IInterface_AsyncCompilation Interface.
#endif

	/** Invalidate Cached Bounds, when Mesh Component has been updated. */
	ENGINE_API void InvalidateCachedBounds();

protected:

	/** Update Mesh Bound information based on input
	 * 
	 * @param RootOffset	: Root Bone offset from mesh location
	 *						  If LeaderPoseComponent exists, it will applied to LeaderPoseComponent's bound
	 * @param UsePhysicsAsset	: Whether or not to use PhysicsAsset for calculating bound of mesh
	 */
	ENGINE_API FBoxSphereBounds CalcMeshBound(const FVector3f& RootOffset, bool UsePhysicsAsset, const FTransform& Transform) const;

	/**
	 * return true if it needs update. Return false if not
	 */
	ENGINE_API bool ShouldUpdateBoneVisibility() const;

protected:

	/** Removes update rate params and internal tracker data */
	ENGINE_API void ReleaseUpdateRateParams();

	/** Recreates update rate params and internal tracker data */
	ENGINE_API void RefreshUpdateRateParams();

private:
	/** Update Rate Optimization ticking. */
	ENGINE_API void TickUpdateRate(float DeltaTime, bool bNeedsValidRootMotion);
	
public:
	/**
	 * Set LeaderPoseComponent for this component
	 *
	 * @param NewLeaderBoneComponent New LeaderPoseComponent
	 * @param bForceUpdate If false, the function will be skipped if NewLeaderBoneComponent is the same as currently setup (default)
	 * @param bInFollowerShouldTickPose If false, Follower components will not execute TickPose (default)
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void SetLeaderPoseComponent(USkinnedMeshComponent* NewLeaderBoneComponent, bool bForceUpdate = false, bool bInFollowerShouldTickPose = false);

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use SetLeaderPoseComponent instead.")
	void SetMasterPoseComponent(USkinnedMeshComponent* NewMasterBoneComponent, bool bForceUpdate = false) { SetLeaderPoseComponent(NewMasterBoneComponent, bForceUpdate); }

	/** Return current active list of follower components */
	ENGINE_API const TArray< TWeakObjectPtr<USkinnedMeshComponent> >& GetFollowerPoseComponents() const;

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use GetFollowerPoseComponents instead.")
	const TArray< TWeakObjectPtr<USkinnedMeshComponent> >& GetSlavePoseComponents() const { return GetFollowerPoseComponents(); }

protected:
	/** Add a follower component to the FollowerPoseComponents array */
	ENGINE_API virtual void AddFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent);

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use AddFollowerPoseComponent instead.")
	virtual void AddSlavePoseComponent(USkinnedMeshComponent* SkinnedMeshComponent) { AddFollowerPoseComponent(SkinnedMeshComponent); }

	/** Remove a follower component from the FollowerPoseComponents array */
	ENGINE_API virtual void RemoveFollowerPoseComponent(USkinnedMeshComponent* SkinnedMeshComponent);

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use RemoveFollowerPoseComponent instead.")
	virtual void RemoveSlavePoseComponent(USkinnedMeshComponent* SkinnedMeshComponent) { RemoveFollowerPoseComponent(SkinnedMeshComponent); }

protected:
	/** Get the default overlay material used by a mesh */
	ENGINE_API virtual UMaterialInterface* GetDefaultOverlayMaterial() const override;

	/** Get the default overlay material max draw distance */
	ENGINE_API virtual float GetDefaultOverlayMaterialMaxDrawDistance() const override;
	
	/*
	 * Get all default LOD sections overlay materials used by a mesh.
	 * The implementer must add all LOD sections and set them to null if there is no value.
	 * We need all sections in case they are all active we wont add the global mesh MaterialOverlay in GetUseMaterials or GetMaterialRelevance.
	 */
	ENGINE_API virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const override;

	void UpdateCachedSocketAttachmentFlag();
	
public:
	/** 
	 * Refresh Follower Components if exists
	 * 
	 * This isn't necessary in any other case except in editor where you need to mark them as dirty for rendering
	 */
	ENGINE_API void RefreshFollowerComponents();

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use RefreshFollowerComponents instead.")
	void RefreshSlaveComponents() { RefreshFollowerComponents(); }

	/**
	 * Update LeaderBoneMap for LeaderPoseComponent and this component
	 */
	ENGINE_API void UpdateLeaderBoneMap();

	UE_DEPRECATED(5.1, "This method has been deprecated. Please use UpdateLeaderBoneMap instead.")
	void UpdateMasterBoneMap() { UpdateLeaderBoneMap(); }

	/**
	 * @param InSocketName	The name of the socket to find
	 * @param OutBoneIndex	The socket bone index in this skeletal mesh, or INDEX_NONE if the socket is not found or not a bone-relative socket
	 * @param OutTransform	The socket local transform, or identity if the socket is not found.
	 * @return SkeletalMeshSocket of named socket on the skeletal mesh component, or nullptr if not found.
	 */
	ENGINE_API class USkeletalMeshSocket const* GetSocketInfoByName(FName InSocketName, FTransform& OutTransform, int32& OutBoneIndex) const;

	/**
	 * @param InSocketName	The name of the socket to find
	 * @return SkeletalMeshSocket of named socket on the skeletal mesh component, or nullptr if not found.
	 */
	ENGINE_API class USkeletalMeshSocket const* GetSocketByName( FName InSocketName ) const;

	ENGINE_API void AddSocketOverride(FName SourceSocketName, FName OverrideSocketName, bool bWarnHasOverrided = true);
	ENGINE_API void RemoveSocketOverrides(FName SourceSocketName);
	ENGINE_API void RemoveAllSocketOverrides();

	/** @return true if this component has any attach children that are attached via sockets */
	ENGINE_API bool HasAnyAttachedSockets() const;

protected:
	/**
	 * Returns whether the bone transforms are valid.
	 */
	ENGINE_API virtual bool AreBoneTransformsValid() const;

public:
	/** 
	 * Get Bone Matrix from index
	 *
	 * @param BoneIndex Index of the bone
	 * 
	 * @return the matrix of the bone at the specified index 
	 */
	ENGINE_API FMatrix GetBoneMatrix( int32 BoneIndex ) const;

	/** 
	 * Get world space bone transform from bone index, also specifying the component transform to use
	 * 
	 * @param BoneIndex Index of the bone
	 *
	 * @return the transform of the bone at the specified index 
	 */
	ENGINE_API FTransform GetBoneTransform( int32 BoneIndex, const FTransform& LocalToWorld ) const;

	/** 
	 * Get Bone Transform from index
	 * 
	 * @param BoneIndex Index of the bone
	 *
	 * @return the transform of the bone at the specified index 
	 */
	ENGINE_API FTransform GetBoneTransform( int32 BoneIndex ) const;

	/** Get Bone Rotation in Quaternion
	 *
	 * @param BoneName Name of the bone
	 * @param Space	0 == World, 1 == Local (Component)
	 * 
	 * @return Quaternion of the bone
	 */
	ENGINE_API FQuat GetBoneQuaternion(FName BoneName, EBoneSpaces::Type Space = EBoneSpaces::WorldSpace) const;

	/** Get Bone Location
	 *
	 * @param BoneName Name of the bone
	 * @param Space	0 == World, 1 == Local (Component)
	 * 
	 * @return Vector of the bone
	 */
	ENGINE_API FVector GetBoneLocation( FName BoneName, EBoneSpaces::Type Space = EBoneSpaces::WorldSpace) const; 

	/** 
	 * Fills the given array with the names of all the bones in this component's current SkeletalMesh 
	 * 
	 * @param (out) Array to fill the names of the bones
	 */
	ENGINE_API void GetBoneNames(TArray<FName>& BoneNames);

	/**
	 * Tests if BoneName is child of ParentBoneName.
	 *
	 * @param BoneName Name of the bone
	 * @param ParentBone Name to check
	 *
	 * @return true if child (strictly, not same). false otherwise
	 * Note - will return false if ChildBoneIndex is the same as ParentBoneIndex i.e. must be strictly a child.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API bool BoneIsChildOf(FName BoneName, FName ParentBoneName) const;

	/** 
	 * Gets the local-space position of a bone in the reference pose. 
	 *
	 * @param BoneIndex Index of the bone
	 *
	 * @return Local space reference position 
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh")
	ENGINE_API FVector GetRefPosePosition(int32 BoneIndex) const;

	/** 
	 * Gets the local-space transform of a bone in the reference pose. 
	 *
	 * @param BoneIndex Index of the bone
	 *
	 * @return Local space reference transform 
	 */
	UFUNCTION(BlueprintPure, Category = "Components|SkinnedMesh")
	ENGINE_API FTransform GetRefPoseTransform(int32 BoneIndex) const;

	/** finds a vector pointing along the given axis of the given bone
	 *
	 * @param BoneName the name of the bone to find
	 * @param Axis the axis of that bone to return
	 *
	 * @return the direction of the specified axis, or (0,0,0) if the specified bone was not found
	 */
	ENGINE_API FVector GetBoneAxis(FName BoneName, EAxis::Type Axis) const;

	/**
	 *	Transform a location/rotation from world space to bone relative space.
	 *	This is handy if you know the location in world space for a bone attachment, as AttachComponent takes location/rotation in bone-relative space.
	 *
	 * @param BoneName Name of bone
	 * @param InPosition Input position
	 * @param InRotation Input rotation
	 * @param OutPosition (out) Transformed position
	 * @param OutRotation (out) Transformed rotation
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API void TransformToBoneSpace( FName BoneName, FVector InPosition, FRotator InRotation, FVector& OutPosition, FRotator& OutRotation ) const;

	/**
	 *	Transform a location/rotation in bone relative space to world space.
	 *
	 * @param BoneName Name of bone
	 * @param InPosition Input position
	 * @param InRotation Input rotation
	 * @param OutPosition (out) Transformed position
	 * @param OutRotation (out) Transformed rotation
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API void TransformFromBoneSpace( FName BoneName, FVector InPosition, FRotator InRotation, FVector& OutPosition, FRotator& OutRotation );

	/** finds the closest bone to the given location
	 *
	 * @param TestLocation the location to test against
	 * @param BoneLocation (optional, out) if specified, set to the world space location of the bone that was found, or (0,0,0) if no bone was found
	 * @param IgnoreScale (optional) if specified, only bones with scaling larger than the specified factor are considered
	 * @param bRequirePhysicsAsset (optional) if true, only bones with physics will be considered
	 *
	 * @return the name of the bone that was found, or 'None' if no bone was found
	 */
	ENGINE_API FName FindClosestBone(FVector TestLocation, FVector* BoneLocation = nullptr, float IgnoreScale = 0.f, bool bRequirePhysicsAsset = false) const;

	/** finds the closest bone to the given location
	*
	* @param TestLocation the location to test against
	* @param BoneLocation (optional, out) if specified, set to the world space location of the bone that was found, or (0,0,0) if no bone was found
	* @param IgnoreScale (optional) if specified, only bones with scaling larger than the specified factor are considered
	* @param bRequirePhysicsAsset (optional) if true, only bones with physics will be considered
	*
	* @return the name of the bone that was found, or 'None' if no bone was found
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh", meta=(DisplayName="Find Closest Bone", AdvancedDisplay="bRequirePhysicsAsset"))
	ENGINE_API FName FindClosestBone_K2(FVector TestLocation, FVector& BoneLocation, float IgnoreScale = 0.f, bool bRequirePhysicsAsset = false) const;

	/**
	 * Find a named MorphTarget from the current SkeletalMesh
	 *
	 * @param MorphTargetName Name of MorphTarget to look for.
	 *
	 * @return Pointer to found MorphTarget. Returns nullptr if could not find target with that name.
	 */
	ENGINE_API virtual class UMorphTarget* FindMorphTarget( FName MorphTargetName ) const;

	/**
	 *	Hides the specified bone. You can also set option for physics body.
	 *
	 *	@param	BoneIndex			Index of the bone
	 *	@param	PhysBodyOption		Option for physics bodies that attach to the bones to be hidden
	 */
	ENGINE_API virtual void HideBone( int32 BoneIndex, EPhysBodyOp PhysBodyOption );

	/**
	 *	Unhides the specified bone.  
	 *
	 *	@param	BoneIndex			Index of the bone
	 */
	ENGINE_API virtual void UnHideBone( int32 BoneIndex );

	/** 
	 *	Determines if the specified bone is hidden. 
	 *
	 *	@param	BoneIndex			Index of the bone
	 *
	 *	@return true if hidden
	 */
	ENGINE_API bool IsBoneHidden( int32 BoneIndex ) const;

	/**
	 *	Hides the specified bone with name.  Currently this just enforces a scale of 0 for the hidden bones.
	 *	Compared to HideBone By Index - This keeps track of list of bones and update when LOD changes
	 *
	 *	@param  BoneName            Name of bone to hide
	 *	@param	PhysBodyOption		Option for physics bodies that attach to the bones to be hidden
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API void HideBoneByName( FName BoneName, EPhysBodyOp PhysBodyOption );

	/**
	 *	UnHide the specified bone with name.  Currently this just enforces a scale of 0 for the hidden bones.
	 *	Compared to HideBone By Index - This keeps track of list of bones and update when LOD changes
	 *	@param  BoneName            Name of bone to unhide
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API void UnHideBoneByName( FName BoneName );

	/** 
	 *	Determines if the specified bone is hidden. 
	 *
	 *	@param  BoneName            Name of bone to check
	 *
	 *	@return true if hidden
	 */
	UFUNCTION(BlueprintCallable, Category="Components|SkinnedMesh")
	ENGINE_API bool IsBoneHiddenByName( FName BoneName );

	/**
	 *	Allows hiding of a particular material (by ID) on this instance of a SkeletalMesh.
	 *
	 * @param MaterialID - Index of the material show/hide
	 * @param bShow - True to show the material, false to hide it
	 * @param LODIndex - Index of the LOD to modify material visibility within
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ShowMaterialSection(int32 MaterialID, int32 SectionIndex, bool bShow, int32 LODIndex);

	/** Clear any material visibility modifications made by ShowMaterialSection */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void ShowAllMaterialSections(int32 LODIndex);

	/** Returns whether a specific material section is currently hidden on this component (by using ShowMaterialSection) */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API bool IsMaterialSectionShown(int32 MaterialID, int32 LODIndex);

	/**
	 * Set whether this skinned mesh should be rendered as static mesh in a reference pose
	 *
	 * @param	whether this skinned mesh should be rendered as static
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	ENGINE_API void SetRenderStatic(bool bNewValue);

	/** 
	 * Return PhysicsAsset for this SkeletalMeshComponent
	 * It will return SkeletalMesh's PhysicsAsset unless PhysicsAssetOverride is set for this component
	 *
	 * @return : PhysicsAsset that's used by this component
	 */
	ENGINE_API class UPhysicsAsset* GetPhysicsAsset() const;

private:
	/**
	* This refresh all morphtarget curves including SetMorphTarget as well as animation curves
	*/
	virtual void RefreshMorphTargets() {};

	/**  
	 * When bones are not resent in a leader mesh when using leader-pose, we call this to evaluate 
	 * relative transforms.
	 */
	ENGINE_API bool GetMissingLeaderBoneRelativeTransform(int32 InBoneIndex, FMissingLeaderBoneCacheEntry& OutInfo) const;

	// BEGIN ILODSyncComponent
	ENGINE_API virtual int32 GetDesiredSyncLOD() const override;
	ENGINE_API virtual int32 GetBestAvailableLOD() const override;
	ENGINE_API virtual void SetForceStreamedLOD(int32 LODIndex) override;
	ENGINE_API virtual void SetForceRenderedLOD(int32 LODIndex) override;
	ENGINE_API virtual int32 GetNumSyncLODs() const override;
	ENGINE_API virtual int32 GetForceStreamedLOD() const override;
	ENGINE_API virtual int32 GetForceRenderedLOD() const override;
	// END ILODSyncComponent

	// Animation update rate control.
public:
	/** Delegate when AnimUpdateRateParams is created, to override its default settings. */
	FOnAnimUpdateRateParamsCreated OnAnimUpdateRateParamsCreated;

	/** Animation Update Rate optimization parameters. */
	struct FAnimUpdateRateParameters* AnimUpdateRateParams;

	virtual bool IsPlayingRootMotion() const { return false; }
	virtual bool IsPlayingNetworkedRootMotionMontage() const { return false; }
	virtual bool IsPlayingRootMotionFromEverything() const { return false; }

	ENGINE_API bool ShouldUseUpdateRateOptimizations() const;

	/** Release any rendering resources owned by this component */
	ENGINE_API void ReleaseResources();

#if WITH_EDITOR
	/** Helpers allowing us to cache feature level */
	static ENGINE_API void BindWorldDelegates();
	static ENGINE_API void HandlePostWorldCreation(UWorld* InWorld);
	static ENGINE_API void HandleFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel, TWeakObjectPtr<UWorld> InWorld);
#endif

	friend class FSkinnedMeshComponentHelper;
	friend class FRenderStateRecreator;
	friend class FSkeletalMeshStreamOut;
	friend struct UE::Anim::FSkinnedMeshComponentExtensions;

	//
	// Animation required bones
	//

	/** Get the bones used by the physics asset (if any), as these have to be updated even if LODed out. */
	static ENGINE_API void GetPhysicsRequiredBones(const USkinnedAsset* SkinnedAsset, const UPhysicsAsset* PhysicsAsset, TArray<FBoneIndexType>& OutRequiredBones);

	/**
	 * Get the bones that are used by sockets.
	 * @param OutRequiredBones - the bones used by sockets that are set as always animate
	 * @param NeededBonesForFillComponentSpaceTransforms - the rest of bones used by sockets (that are not set to always animate)
	 */ 
	static ENGINE_API void GetSocketRequiredBones(const USkinnedAsset* SkinnedAsset, TArray<FBoneIndexType>& OutRequiredBones, TArray<FBoneIndexType>& NeededBonesForFillComponentSpaceTransforms);

	/** 
	 * Takes sorted array Base and then adds any elements from sorted array Insert which is missing from it, preserving order.
	 * This assumes both arrays are sorted and contain unique bone indices.
	 */
	static ENGINE_API void MergeInBoneIndexArrays(TArray<FBoneIndexType>& BaseArray, const TArray<FBoneIndexType>& InsertArray);

	/**
	 * Override this function to add any additional required bones from followers to functions such as USkeletalMeshComponent::ComputeRequiredBones().
	 * The additional bones will need to be passed using the Leader's bone index mapping, sorted in hierarchical order, and merged into the InOutRequiredBones array.
	 * @param LODIndex - the LOD to get the required bones for.
	 * @param InOutRequiredBones - the array in which to merge the additional required bones.
	 */
	virtual void GetAdditionalRequiredBonesForLeader(int32 LODIndex, TArray<FBoneIndexType>& InOutRequiredBones) const {}

	/**
	 * Register an OnBoneTransformsFinalized callback which can be called in FinalizeBoneTransform().
	 * Note the inherited class has to implement the Broadcast call in FinalizeBoneTransform() if it wants to mimic the same behavior as in USkeletalMeshComponent.
	 * @param Delegate - the delegate to be broadcasted in FinalizeBoneTransform by the inherited class.
	 */
	virtual FDelegateHandle RegisterOnBoneTransformsFinalizedDelegate(const FOnBoneTransformsFinalizedMultiCast::FDelegate& /*Delegate*/) { return FDelegateHandle(); }

	/**
	 * Unregister an OnBoneTransformsFinalized callback.
	 * @param DelegateHandle - the handle of the delegate to remove from the list.
	 */
	virtual void UnregisterOnBoneTransformsFinalizedDelegate(const FDelegateHandle& /*DelegateHandle*/) {}
};

class FRenderStateRecreator
{
	USkinnedMeshComponent* Component;
	const bool bWasInitiallyRegistered;
	const bool bWasRenderStateCreated;

public:

	FRenderStateRecreator(USkinnedMeshComponent* InActorComponent) :
		Component(InActorComponent),
		bWasInitiallyRegistered(Component->IsRegistered()),
		bWasRenderStateCreated(Component->IsRenderStateCreated())
	{
		if (bWasRenderStateCreated)
		{
			if (!bWasInitiallyRegistered)
			{
				UE_LOGF(LogSkeletalMesh, Warning, "Created a FRenderStateRecreator with an unregistered component: %ls", *Component->GetPathName());
			}

			Component->DestroyRenderState_Concurrent();
		}
	}

	~FRenderStateRecreator()
	{
		const bool bIsRegistered = Component->IsRegistered();

		const FCoreTexts& CoreTexts = FCoreTexts::Get();

		ensureMsgf(bWasInitiallyRegistered == bIsRegistered,
			TEXT("Component Registered state changed from %s to %s within FRenderStateRecreator scope."),
			*((bWasInitiallyRegistered ? CoreTexts.True : CoreTexts.False).ToString()),
			*((bIsRegistered ? CoreTexts.True : CoreTexts.False).ToString()));

		if (bWasRenderStateCreated && bIsRegistered)
		{
			Component->CreateRenderState_Concurrent(nullptr);
		}
	}
};

/** Simple, CPU evaluation of a vertex's skinned tangent basis */
void GetTypedSkinnedTangentBasis(
	const USkinnedMeshComponent* SkinnedComp,
	const FSkelMeshRenderSection& Section,
	const FStaticMeshVertexBuffers& StaticVertexBuffers,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer,
	const int32 VertIndex,
	const TArray<FMatrix44f> & RefToLocals,
	FVector3f& OutTangentX,
	FVector3f& OutTangentY,
	FVector3f& OutTangentZ
);

/** Simple, CPU evaluation of a vertex's skinned position helper function */
template <bool bCachedMatrices>
FVector3f GetTypedSkinnedVertexPosition(
	const USkinnedMeshComponent* SkinnedComp,
	const FSkelMeshRenderSection& Section,
	const FPositionVertexBuffer& PositionVertexBuffer,
	const FSkinWeightVertexBuffer& SkinWeightVertexBuffer,
	const int32 VertIndex,
	const TArray<FMatrix44f> & RefToLocals = TArray<FMatrix44f>()
);

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Rendering/SkinWeightProfile.h"
#endif
