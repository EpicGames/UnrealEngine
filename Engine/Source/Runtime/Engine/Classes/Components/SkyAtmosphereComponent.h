// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "EngineDefines.h"
#include "GameFramework/Info.h"
#include "Misc/Guid.h"
#include "RenderResource.h"
#include "StateStream/SkyAtmosphereStateStreamHandle.h"

#include "SkyAtmosphereComponent.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSkyAtmosphereComponent, Log, All)


class FSkyAtmosphereSceneProxy;


USTRUCT(BlueprintType)
struct FTentDistribution
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Tent", meta = (UIMin = 0.0, UIMax = 60.0))
	float TipAltitude = 0.0f;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Tent", meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0, SliderExponent = 4.0))
	float TipValue = 0.0f;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Tent", meta = (UIMin = 0.01, UIMax = 20.0, ClampMin = 0.0))
	float Width = 1.0f;
};

UENUM()
enum class ESkyAtmosphereTransformMode : uint8
{
	PlanetTopAtAbsoluteWorldOrigin,
	PlanetTopAtComponentTransform,
	PlanetCenterAtComponentTransform,
};

/**
 * A component that represents a planet atmosphere material and simulates sky and light scattering within it.
 * @see https://docs.unrealengine.com/en-US/Engine/Actors/FogEffects/SkyAtmosphere/index.html
 */
UCLASS(ClassGroup = Rendering, collapsecategories, hidecategories = (Object, Mobility, Activation, "Components|Activation"), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USkyAtmosphereComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	~USkyAtmosphereComponent();


	/** The ground albedo that will tint the atmosphere when the sun light will bounce on it. Only taken into account when MultiScattering>0.0. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Planet", meta = (HideAlphaChannel))
	ESkyAtmosphereTransformMode TransformMode;

	/** The radius in kilometers from the center of the planet to the ground level. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Planet", meta = (DisplayName = "Ground Radius", UIMin = 1.0, UIMax = 7000.0, ClampMin = 0.1, ClampMax = 10000.0, SliderExponent = 5.0))
	float BottomRadius;

	/** The ground albedo that will tint the atmosphere when the sun light will bounce on it. Only taken into account when MultiScattering>0.0. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Planet", meta = (HideAlphaChannel))
	FColor GroundAlbedo;



	/** The height of the atmosphere layer above the ground in kilometers. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere", meta = (UIMin = 1.0, UIMax = 200.0, ClampMin = 0.1, SliderExponent = 2.0))
	float AtmosphereHeight;

	/** Factor applied to multiple scattering only (after the sun light has bounced around in the atmosphere at least once). 
	 * Multiple scattering is evaluated using a dual scattering approach. 
	 * A value of 2 is recommended to better represent default atmosphere when r.SkyAtmosphere.MultiScatteringLUT.HighQuality=0. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere", meta = (DisplayName = "MultiScattering", UIMin = 0.0, UIMax = 2.0, ClampMin = 0.0, ClampMax = 100.0))
	float MultiScatteringFactor;

	/**
	 * Scale the atmosphere tracing sample count. Quality level scalability
	 * The sample count is still clamped according to scalability setting to 'r.SkyAtmosphere.SampleCountMax' when 'r.SkyAtmosphere.FastSkyLUT' is 0.
	 * The sample count is still clamped according to scalability setting to 'r.SkyAtmosphere.FastSkyLUT.SampleCountMax' when 'r.SkyAtmosphere.FastSkyLUT' is 1.
	 * The sample count is still clamped for aerial perspective according to  'r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Atmosphere", AdvancedDisplay, meta = (UIMin = "0.25", UIMax = "8", ClampMin = "0.25", SliderExponent = 3.0))
	float TraceSampleCountScale;
	


	/** Rayleigh scattering coefficient scale.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Rayleigh", meta = (UIMin = 0.0, UIMax = 2.0, ClampMin = 0.0, SliderExponent = 4.0))
	float RayleighScatteringScale;

	/** The Rayleigh scattering coefficients resulting from molecules in the air at an altitude of 0 kilometer. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Rayleigh", meta=(HideAlphaChannel))
	FLinearColor RayleighScattering;

	/** The altitude in kilometer at which Rayleigh scattering effect is reduced to 40%.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Rayleigh", meta = (UIMin = 0.01, UIMax = 20.0, ClampMin = 0.001, SliderExponent = 5.0))
	float RayleighExponentialDistribution;



	/** Mie scattering coefficient scale.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (UIMin = 0.0, UIMax = 5.0, ClampMin = 0.0, SliderExponent = 4.0))
	float MieScatteringScale;

	/** The Mie scattering coefficients resulting from particles in the air at an altitude of 0 kilometer. As it becomes higher, light will be scattered more. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (HideAlphaChannel))
	FLinearColor MieScattering;

	/** Mie absorption coefficient scale.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (UIMin = 0.0, UIMax = 5.0, ClampMin = 0.0, SliderExponent = 4.0))
	float MieAbsorptionScale;

	/** The Mie absorption coefficients resulting from particles in the air at an altitude of 0 kilometer. As it becomes higher, light will be absorbed more. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (HideAlphaChannel))
	FLinearColor MieAbsorption;
	
	/** A value of 0 mean light is uniformly scattered. A value closer to 1 means lights will scatter more forward, resulting in halos around light sources. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (UIMin = 0.0, UIMax = 0.999, ClampMin = 0.0, ClampMax = 0.999))
	float MieAnisotropy;

	/** The altitude in kilometer at which Mie effects are reduced to 40%.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Mie", meta = (UIMin = 0.01, UIMax = 10.0, ClampMin = 0.001, SliderExponent = 5.0))
	float MieExponentialDistribution;



	/** Absorption coefficients for another atmosphere layer. Density increase from 0 to 1 between 10 to 25km and decreases from 1 to 0 between 25 to 40km. This approximates ozone molecules distribution in the Earth atmosphere. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Absorption", meta = (DisplayName = "Absorption Scale", UIMin = 0.0, UIMax = 0.2, ClampMin = 0.0, SliderExponent = 3.0))
	float OtherAbsorptionScale;

	/** Absorption coefficients for another atmosphere layer. Density increase from 0 to 1 between 10 to 25km and decreases from 1 to 0 between 25 to 40km. The default values represents ozone molecules absorption in the Earth atmosphere. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Atmosphere - Absorption", meta = (DisplayName = "Absorption", HideAlphaChannel))
	FLinearColor OtherAbsorption;

	/** Represents the altitude based tent distribution of absorption particles in the atmosphere. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Atmosphere - Absorption", meta = (DisplayName = "Tent Distribution"))
	FTentDistribution OtherTentDistribution;
	


	/** Scales the luminance of pixels representing the sky. This will impact the captured sky light. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (HideAlphaChannel))
	FLinearColor SkyLuminanceFactor;

	/** Scales the luminance of pixels representing the sky and the aerial perspective. This will impact the captured sky light. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (HideAlphaChannel))
	FLinearColor SkyAndAerialPerspectiveLuminanceFactor;

	/** Makes the aerial perspective look thicker by scaling distances from view to surfaces (opaque and translucent). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (DisplayName = "Aerial Perspective View Distance Scale", UIMin = 0.0, UIMax = 3.0, ClampMin = 0.0, SliderExponent = 2.0))
	float AerialPespectiveViewDistanceScale;

	/** Scale the sky and atmosphere lights contribution to the height fog when SupportSkyAtmosphereAffectsHeightFog project setting is true.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0, SliderExponent = 2.0))
	float HeightFogContribution;

	/** The minimum elevation angle in degree that should be used to evaluate the sun transmittance to the ground. Useful to maintain a visible sun light and shadow on meshes even when the sun has started going below the horizon. This does not affect the aerial perspective.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = -90.0, UIMax = 90.0, ClampMin = -90.0f, ClampMax = 90.0f))
	float TransmittanceMinLightElevationAngle;

	/** The distance (kilometers) at which we start evaluating the aerial perspective. Having the aerial perspective starts away from the camera can help with performance: pixels not affected by the aerial perspective will have their computation skipped using early depth test.*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.001f, UIMax = 10.0f, ClampMin = 0.001f))
	float AerialPerspectiveStartDepth;

	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, Interp)
	uint8 bHoldout : 1;

	/** If true, this component will be rendered in the main pass (basepass, transparency) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bRenderInMainPass : 1;


	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void OverrideAtmosphereLightDirection(int32 AtmosphereLightIndex, const FVector& LightDirection);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API bool IsAtmosphereLightDirectionOverriden(int32 AtmosphereLightIndex);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API FVector GetOverridenAtmosphereLightDirection(int32 AtmosphereLightIndex);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void ResetAtmosphereLightDirectionOverride(int32 AtmosphereLightIndex);

	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Ground Radius"))
	ENGINE_API void SetBottomRadius(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetGroundAlbedo(const FColor& NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAtmosphereHeight(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMultiScatteringFactor(float NewValue);
	
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRayleighScatteringScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRayleighScattering(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRayleighExponentialDistribution(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieScatteringScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieScattering(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieAbsorptionScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieAbsorption(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieAnisotropy(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMieExponentialDistribution(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Absorption Scale"))
	ENGINE_API void SetOtherAbsorptionScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Absorption"))
	ENGINE_API void SetOtherAbsorption(FLinearColor NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetSkyLuminanceFactor(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetSkyAndAerialPerspectiveLuminanceFactor(FLinearColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAerialPespectiveViewDistanceScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetAerialPerspectiveStartDepth(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHeightFogContribution(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetTransmittanceMinLightElevationAngle(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHoldout(bool bNewHoldout);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRenderInMainPass(bool bValue);

	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (DisplayName = "Get Atmosphere Transmitance On Ground At Planet Top"))
	ENGINE_API FLinearColor GetAtmosphereTransmitanceOnGroundAtPlanetTop(UDirectionalLightComponent* DirectionalLight);

	// This function can be used for instance in order to evaluate a directional atmospheric light outer space illuminance for a desired illuminance on ground given a direction. 
	// This is given for the position at the top of the virtual planet. Plus the output outer space illuminance into the light intensity.
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (DisplayName = "Get Atmospheric Light To Match Illuminance On Ground"))
	ENGINE_API float GetAtmosphericLightToMatchIlluminanceOnGround(FVector LightDirection = FVector(0.0f, 0.0f, 1.0f), float IlluminanceOnGround = 1.0);

	// This is used to position the SkyAtmosphere similarly to the deprecated AtmosphericFog component
	void SetPositionToMatchDeprecatedAtmosphericFog();

protected:
	//~ Begin UActorComponent Interface.
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void SendRenderTransform_Concurrent() override;
	virtual void DestroyRenderState_Concurrent() override;
	//~ End UActorComponent Interface.

public:

	//~ Begin UObject Interface. 
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UActorComponent Interface.
#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif // WITH_EDITOR
	//~ End UActorComponent Interface.

	FGuid GetStaticLightingBuiltGuid() const { return bStaticLightingBuiltGUID; }

	void GetOverrideLightStatus(bool* OverrideAtmosphericLight, FVector* OverrideAtmosphericLightDirection) const;

private:

	FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy;

	bool OverrideAtmosphericLight[NUM_ATMOSPHERE_LIGHTS];
	FVector OverrideAtmosphericLightDirection[NUM_ATMOSPHERE_LIGHTS];

	/**
	 * GUID used to associate a atmospheric component with precomputed lighting/shadowing information across levels.
	 * The GUID changes whenever the atmospheric properties change, e.g. LUTs.
	 */
	UPROPERTY()
	FGuid bStaticLightingBuiltGUID;
	/**
	 * Validate static lighting GUIDs and update as appropriate.
	 */
	void ValidateStaticLightingGUIDs();
	/**
	 * Update static lighting GUIDs.
	 */
	void UpdateStaticLightingGUIDs();

	void SendRenderTransformCommand();

protected:
	// When true, this means that this SkyAtmosphere is use as replacement for the deprecated AtmosphericFogComponent as a parent class. 
	// This is used to adapt the serialisation.
	bool bIsAtmosphericFog = false;
	
	/**
	 * Sets a dummy, known static lighting GUID to avoid cook non-determinism
	 * when a map contains the deprecated AtmosphericFogComponent.
	 */
	void SetDummyStaticLightingGUIDs();

	#if WITH_STATE_STREAM_ACTOR
	FSkyAtmosphereHandle Handle;
	#endif
};


/**
 * A placeable actor that represents a planet atmosphere material and simulates sky and light scattering within it.
 * @see https://docs.unrealengine.com/en-US/Engine/Actors/FogEffects/SkyAtmosphere/index.html
 */
UCLASS(showcategories = (Movement, Rendering, Transformation, DataLayers, "Input|MouseInput", "Input|TouchInput"), ClassGroup = Fog, hidecategories = (Info, Object, Input), MinimalAPI)
class ASkyAtmosphere : public AInfo
{
	GENERATED_UCLASS_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Atmosphere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USkyAtmosphereComponent> SkyAtmosphereComponent;

#if WITH_EDITORONLY_DATA
	/** Arrow component to indicate default sun rotation */
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
#endif

public:

	/** Returns SkyAtmosphereComponent subobject */
	USkyAtmosphereComponent* GetComponent() const { return SkyAtmosphereComponent; }

};
