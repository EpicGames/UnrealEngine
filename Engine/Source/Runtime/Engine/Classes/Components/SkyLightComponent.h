// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "Components/LightComponentBase.h"
#include "Math/SHMath.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RenderDeferredCleanup.h"
#include "SkyLightComponent.generated.h"

class FSkyLightSceneProxy;
class UTextureCube;

/** 
 * A cubemap texture resource that knows how to upload the capture data from a sky capture. 
 */
class FSkyTextureCubeResource : public FTexture, private FDeferredCleanupInterface
{
	// @todo - support compression

public:

	FSkyTextureCubeResource() :
		Size(0),
		NumMips(0),
		Format(PF_Unknown),
		TextureCubeRHI(NULL),
		NumRefs(0)
	{}

	virtual ~FSkyTextureCubeResource() { check(NumRefs == 0); }

	void SetupParameters(int32 InSize, int32 InNumMips, EPixelFormat InFormat)
	{
		Size = InSize;
		NumMips = InNumMips;
		Format = InFormat;
	}

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual void ReleaseRHI() override
	{
		TextureCubeRHI.SafeRelease();
		FTexture::ReleaseRHI();
	}

	virtual uint32 GetSizeX() const override
	{
		return Size;
	}

	// PVS-Studio notices that the implementation of GetSizeX is identical to this one
	// and warns us. In this case, it is intentional, so we disable the warning:
	virtual uint32 GetSizeY() const override
	{
		return Size;
	}

	// Reference counting.
	void AddRef()
	{
		check( IsInGameThread() );
		NumRefs++;
	}

	ENGINE_API void Release();
private:

	int32 Size;
	int32 NumMips;
	EPixelFormat Format;
	FTextureRHIRef TextureCubeRHI;
	int32 NumRefs;
};

UENUM()
enum ESkyLightSourceType : int
{
	/** Construct the sky light from the captured scene, anything further than SkyDistanceThreshold from the sky light position will be included. */
	SLS_CapturedScene,
	/** Construct the sky light from the specified cubemap. */
	SLS_SpecifiedCubemap,
	SLS_MAX,
};

enum class ESkyLightCaptureStatus
{
	SLCS_Uninitialized,
	SLCS_CapturedButIncomplete,
	SLCS_CapturedAndComplete,
};

UCLASS(Blueprintable, ClassGroup=Lights, HideCategories=(Trigger,Activation,"Components|Activation",Physics), meta=(BlueprintSpawnableComponent), MinimalAPI)
class USkyLightComponent : public ULightComponentBase
{
	GENERATED_UCLASS_BODY()

	/** When enabled, the sky will be captured and convolved to achieve dynamic diffuse and specular environment lighting. 
	 * SkyAtmosphere, VolumetricCloud Components as well as sky domes with Sky materials are taken into account. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light)
	bool bRealTimeCapture;

	/** Indicates where to get the light contribution from. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light)
	TEnumAsByte<enum ESkyLightSourceType> SourceType;

	/** Cubemap to use for sky lighting if SourceType is set to SLS_SpecifiedCubemap. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light)
	TObjectPtr<class UTextureCube> Cubemap;

	/** Angle to rotate the source cubemap when SourceType is set to SLS_SpecifiedCubemap. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta=(UIMin = "0", UIMax = "360"))
	float SourceCubemapAngle;

	/** Maximum resolution for the very top processed cubemap mip. Must be a power of 2. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light)
	int32 CubemapResolution;

	/** 
	 * Distance from the sky light at which any geometry should be treated as part of the sky. 
	 * This is also used by reflection captures, so update reflection captures to see the impact.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light)
	float SkyDistanceThreshold;

	/** Only capture emissive materials. Skips all lighting making the capture cheaper. Recomended when using CaptureEveryFrame */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay)
	bool bCaptureEmissiveOnly;

	/** 
	 * Whether all distant lighting from the lower hemisphere should be set to LowerHemisphereColor.  
	 * Enabling this is accurate when lighting a scene on a planet where the ground blocks the sky, 
	 * However disabling it can be useful to approximate skylight bounce lighting (eg Movable light).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay, meta=(DisplayName = "Lower Hemisphere Is Solid Color"))
	bool bLowerHemisphereIsBlack;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, AdvancedDisplay)
	FLinearColor LowerHemisphereColor;

	/** 
	 * Max distance that the occlusion of one point will affect another.
	 * Higher values increase the cost of Distance Field AO exponentially.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion, meta=(UIMin = "200", UIMax = "1500"))
	float OcclusionMaxDistance;

	/** 
	 * Contrast S-curve applied to the computed AO.  A value of 0 means no contrast increase, 1 is a significant contrast increase.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion, meta=(UIMin = "0", UIMax = "1", DisplayName = "Occlusion Contrast"))
	float Contrast;

	/** 
	 * Exponent applied to the computed AO.  Values lower than 1 brighten occlusion overall without losing contact shadows.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion, meta=(UIMin = ".6", UIMax = "1.6"))
	float OcclusionExponent;

	/** 
	 * Controls the darkest that a fully occluded area can get.  This tends to destroy contact shadows, use Contrast or OcclusionExponent instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion, meta=(UIMin = "0", UIMax = "1"))
	float MinOcclusion;

	/** Tint color on occluded areas, artistic control. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion)
	FColor OcclusionTint;

	/**
	 * Whether the cloud should occlude sky contribution within the atmosphere (progressively fading multiple scattering out) or not.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud)
	uint32 bCloudAmbientOcclusion : 1;
	/**
	 * The strength of the ambient occlusion, higher value will block more light.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = AtmosphereAndCloud, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", SliderExponent = 1.0))
	float CloudAmbientOcclusionStrength;
	/**
	 * The world space radius of the cloud ambient occlusion map around the camera in kilometers.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, meta = (UIMin = "1", ClampMin = "1"))
	float CloudAmbientOcclusionExtent;
	/**
	 * Scale the cloud ambient occlusion map resolution, base resolution is 512. The resolution is still clamped to 'r.VolumetricCloud.SkyAO.MaxResolution'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, meta = (UIMin = "0.25", UIMax = "8", ClampMin = "0.25", SliderExponent = 1.0))
	float CloudAmbientOcclusionMapResolutionScale;
	/**
	 * Controls the cone aperture angle over which the sky occlusion due to volumetric clouds is evaluated. A value of 1 means `take into account the entire hemisphere` resulting in blurry occlusion, while a value of 0 means `take into account a single up occlusion direction up` resulting in sharp occlusion.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AtmosphereAndCloud, meta = (UIMin = "0.0", UIMax = "0.1", ClampMin = "0.0", ClampMax = "1.0", SliderExponent = 2.0))
	float CloudAmbientOcclusionApertureScale;


	/** Controls how occlusion from Distance Field Ambient Occlusion is combined with Screen Space Ambient Occlusion. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=DistanceFieldAmbientOcclusion)
	TEnumAsByte<enum EOcclusionCombineMode> OcclusionCombineMode;
		
	ENGINE_API class FSkyLightSceneProxy* CreateSceneProxy() const;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void CheckForErrors() override;
#endif // WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	//~ End UObject Interface

	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API void ApplyComponentInstanceData(struct FPrecomputedSkyLightInstanceData* ComponentInstanceData);

	/** Called each tick to recapture and queued sky captures. */
	static ENGINE_API bool HasSkyCapturesToUpdate();
	static ENGINE_API void UpdateSkyCaptureContents(UWorld* WorldToUpdate);
	static ENGINE_API void UpdateSkyCaptureContentsArray(UWorld* WorldToUpdate, TArray<USkyLightComponent*>& ComponentArray, bool bBlendSources);

	/** Computes a radiance map using only emissive contribution from the sky light. */
	ENGINE_API void CaptureEmissiveRadianceEnvironmentCubeMap(FSHVectorRGB3& OutIrradianceMap, TArray<FFloat16Color>& OutRadianceMap) const;

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetIntensity(float NewIntensity);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetIndirectLightingIntensity(float NewIntensity);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Light")
	ENGINE_API void SetVolumetricScatteringIntensity(float NewIntensity);

	/** Set color of the light */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetLightColor(FLinearColor NewLightColor);

	/** Sets the cubemap used when SourceType is set to SpecifiedCubemap, and causes a skylight update on the next tick. */
	UFUNCTION(BlueprintCallable, Category="SkyLight")
	ENGINE_API void SetCubemap(UTextureCube* NewCubemap);

	/** Sets the angle of the cubemap used when SourceType is set to SpecifiedCubemap and it is non static. It will cause the skylight to update on the next tick. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|SkyLight")
	ENGINE_API void SetSourceCubemapAngle(float NewValue);

	/** 
	 * Creates sky lighting from a blend between two cubemaps, which is only valid when SourceType is set to SpecifiedCubemap. 
	 * This can be used to seamlessly transition sky lighting between different times of day.
	 * The caller should continue to update the blend until BlendFraction is 0 or 1 to reduce rendering cost.
	 * The caller is responsible for avoiding pops due to changing the source or destination.
	 */
	UFUNCTION(BlueprintCallable, Category="SkyLight")
	ENGINE_API void SetCubemapBlend(UTextureCube* SourceCubemap, UTextureCube* DestinationCubemap, float InBlendFraction);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetLowerHemisphereColor(const FLinearColor& InLowerHemisphereColor);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetOcclusionTint(const FColor& InTint);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetOcclusionContrast(float InOcclusionContrast);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetOcclusionExponent(float InOcclusionExponent);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetMinOcclusion(float InMinOcclusion);

	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void SetRealTimeCapture(bool bInRealTimeCapture);

protected:
	ENGINE_API virtual void OnVisibilityChanged() override;

public:

	/** Indicates that the capture needs to recapture the scene, adds it to the recapture queue. */
	ENGINE_API void SetCaptureIsDirty();
	ENGINE_API void SetBlendDestinationCaptureIsDirty();
	ENGINE_API void SanitizeCubemapSize();

	/** Whether sky occlusion is supported by current feature level */
	ENGINE_API bool IsOcclusionSupported() const;

	ENGINE_API void SetRealTimeCaptureEnabled(bool bNewRealTimeCaptureEnabled);
	ENGINE_API bool IsRealTimeCaptureEnabled() const;

	/** 
	 * Recaptures the scene for the skylight. 
	 * This is useful for making sure the sky light is up to date after changing something in the world that it would capture.
	 * Warning: this is very costly and will definitely cause a hitch.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|SkyLight")
	ENGINE_API void RecaptureSky();

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	const FTexture* GetProcessedSkyTexture() const { return ProcessedSkyTexture; }
	FSHVectorRGB3 GetIrradianceEnvironmentMap() { return IrradianceEnvironmentMap; }
protected:

#if WITH_EDITOR
	/** shadow copy saved before effects of PostEditChange() to provide option to roll back edit. */
	int32 PreEditCubemapResolution = 128;
#endif

	/** Indicates whether the cached data stored in GetComponentInstanceData is valid to be applied in ApplyComponentInstanceData. */
	bool bSavedConstructionScriptValuesValid;

	bool bHasEverCaptured;

#if WITH_EDITOR
	// In an attempt to get valid lighting data as soon as possible in a level, we will always trigger a skylight capture at creation.
	// And then, a soon as the world is finally loaded, the final capture will be taken.
	ESkyLightCaptureStatus CaptureStatus;
	float SecondsSinceLastCapture;
#endif

	TRefCountPtr<FSkyTextureCubeResource> ProcessedSkyTexture;
	FSHVectorRGB3 IrradianceEnvironmentMap;
	float AverageBrightness=0.0f;

	/** If 0, no blend is present.  If > 0, BlendDestinationProcessedSkyTexture and BlendDestinationIrradianceEnvironmentMap must be generated and used for rendering. */
	float BlendFraction;

	UPROPERTY(transient)
	TObjectPtr<class UTextureCube> BlendDestinationCubemap;
	TRefCountPtr<FSkyTextureCubeResource> BlendDestinationProcessedSkyTexture;
	FSHVectorRGB3 BlendDestinationIrradianceEnvironmentMap;
	float BlendDestinationAverageBrightness;

	FLinearColor SpecifiedCubemapColorScale;

	/** Tracks when the rendering thread has completed its writes to IrradianceEnvironmentMap. */
	FRenderCommandFence IrradianceMapFence;

	/** Fence used to track progress of releasing resources on the rendering thread. */
	FRenderCommandFence ReleaseResourcesFence;

	FSkyLightSceneProxy* SceneProxy;

	/** 
	 * List of sky captures that need to be recaptured.
	 * These have to be queued because we can only render the scene to update captures at certain points, after the level has loaded.
	 * This queue should be in the UWorld or the FSceneInterface, but those are not available yet in PostLoad.
	 */
	static ENGINE_API TArray<USkyLightComponent*> SkyCapturesToUpdate;
	static ENGINE_API TArray<USkyLightComponent*> SkyCapturesToUpdateBlendDestinations;
	static ENGINE_API FTransactionallySafeCriticalSection SkyCapturesToUpdateLock;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	//~ Begin UActorComponent Interface

	ENGINE_API void UpdateLimitedRenderingStateFast();
	ENGINE_API void UpdateOcclusionRenderingStateFast();

	friend class FSkyLightSceneProxy;
};


/** Used to store lightmap data during RerunConstructionScripts */
USTRUCT()
struct FPrecomputedSkyLightInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FPrecomputedSkyLightInstanceData() = default;
	FPrecomputedSkyLightInstanceData(const USkyLightComponent* SourceComponent)
		: FSceneComponentInstanceData(SourceComponent)
	{}
	virtual ~FPrecomputedSkyLightInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<USkyLightComponent>(Component)->ApplyComponentInstanceData(this);
	}

	UPROPERTY()
	FGuid OriginalLightGuid;

	UPROPERTY()
	FGuid LightGuid;

	UPROPERTY()
	float AverageBrightness = 1.0f;

	// This has to be refcounted to keep it alive during the handoff without doing a deep copy
	TRefCountPtr<FSkyTextureCubeResource> ProcessedSkyTexture;

	FSHVectorRGB3 IrradianceEnvironmentMap;
};

