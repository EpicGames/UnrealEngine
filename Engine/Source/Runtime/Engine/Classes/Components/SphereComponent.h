// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ShapeComponent.h"
#include "SphereComponent.generated.h"

class FPrimitiveSceneProxy;

/** 
 * A sphere generally used for simple collision. Bounds are rendered as lines in the editor.
 */
UCLASS(ClassGroup="Collision", editinlinenew, hidecategories=(Object,LOD,Lighting,TextureStreaming), meta=(DisplayName="Sphere Collision", BlueprintSpawnableComponent), MinimalAPI)
class USphereComponent : public UShapeComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** The radius of the sphere **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category=Shape)
	float SphereRadius;

public:
	/**
	 * Change the sphere radius. This is the unscaled radius, before component scale is applied.
	 * @param	InSphereRadius: the new sphere radius
	 * @param	bUpdateOverlaps: if true and this shape is registered and collides, updates touching array for owner actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Sphere")
	ENGINE_API void SetSphereRadius(float InSphereRadius, bool bUpdateOverlaps=true);

	// @return the radius of the sphere, with component scale applied.
	UFUNCTION(BlueprintCallable, Category="Components|Sphere")
	float GetScaledSphereRadius() const;

	// @return the radius of the sphere, ignoring component scale.
	UFUNCTION(BlueprintCallable, Category="Components|Sphere")
	float GetUnscaledSphereRadius() const;

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual bool IsZeroExtent() const override;
	ENGINE_API virtual struct FCollisionShape GetCollisionShape(float Inflation = 0.0f) const override;
	ENGINE_API virtual bool AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual void CalcBoundingCylinder(float& CylinderRadius, float& CylinderHalfHeight) const override;
	//~ End USceneComponent Interface

	//~ Begin UShapeComponent Interface
	ENGINE_API virtual void UpdateBodySetup() override;
	//~ End UShapeComponent Interface

	// Get the scale used by this shape. This is a uniform scale that is the minimum of any non-uniform scaling.
	// @return the scale used by this shape.
	UFUNCTION(BlueprintCallable, Category="Components|Sphere")
	float GetShapeScale() const;

	// Sets the sphere radius without triggering a render or physics update.
	inline void InitSphereRadius(float InSphereRadius) { SphereRadius = InSphereRadius; }
};


// ----------------- INLINES ---------------

inline float USphereComponent::GetScaledSphereRadius() const
{
	return SphereRadius * GetShapeScale();
}

inline float USphereComponent::GetUnscaledSphereRadius() const
{
	return SphereRadius;
}

inline float USphereComponent::GetShapeScale() const
{
	return GetComponentTransform().GetMinimumAxisScale();
}
