// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ReflectionCaptureComponent.h"
#include "SphereReflectionCaptureComponent.generated.h"

	// -> will be exported to EngineDecalClasses.h
UCLASS(hidecategories=(Collision, Object, Physics, SceneComponent), MinimalAPI)
class USphereReflectionCaptureComponent : public UReflectionCaptureComponent
{
	GENERATED_UCLASS_BODY()

	/** Radius of the area that can receive reflections from this capture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReflectionCapture, meta=(UIMin = "8.0", UIMax = "16384.0"))
	float InfluenceRadius;

	/** Not needed anymore, not yet removed in case the artist setup values are needed in the future */
	UPROPERTY()
	float CaptureDistanceScale;

	UPROPERTY()
	TObjectPtr<class UDrawSphereComponent> PreviewInfluenceRadius;

public:
	virtual void UpdatePreviewShape() override;

	virtual float GetInfluenceBoundingRadius() const override
	{
		return InfluenceRadius;
	}

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface
};

