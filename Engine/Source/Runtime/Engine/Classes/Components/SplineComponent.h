// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "Math/InterpCurve.h"
#include "BoxTypes.h"

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
#include "Curves/Spline.h"
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

#include "SplineComponent.generated.h"

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS

class FPrimitiveSceneProxy;
class FPrimitiveDrawInterface;
class FSceneView;

/** Permitted spline point types for SplineComponent. */
UENUM(BlueprintType)
namespace ESplinePointType
{
	enum Type : int
	{
		Linear,
		Curve,
		Constant,
		CurveClamped,
		CurveCustomTangent
	};
}

/** Types of coordinate space accepted by the functions. */
UENUM()
namespace ESplineCoordinateSpace
{
	enum Type : int
	{
		Local,
		World
	};
}

UENUM()
enum class ELastAuthority : uint8
{
	Unset,
	Spline,
	SplineCurves
};

UCLASS(Abstract, MinimalAPI)
class USplineMetadata : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Insert point before index, lerping metadata between previous and next key values */
	virtual void InsertPoint(int32 Index, float t, bool bClosedLoop) PURE_VIRTUAL(USplineMetadata::InsertPoint, );
	/** Update point at index by lerping metadata between previous and next key values */
	virtual void UpdatePoint(int32 Index, float t, bool bClosedLoop) PURE_VIRTUAL(USplineMetadata::UpdatePoint, );
	virtual void AddPoint(float InputKey) PURE_VIRTUAL(USplineMetadata::AddPoint, );
	virtual void RemovePoint(int32 Index) PURE_VIRTUAL(USplineMetadata::RemovePoint, );
	virtual void DuplicatePoint(int32 Index) PURE_VIRTUAL(USplineMetadata::DuplicatePoint, );
	virtual void CopyPoint(const USplineMetadata* FromSplineMetadata, int32 FromIndex, int32 ToIndex) PURE_VIRTUAL(USplineMetadata::CopyPoint, );
	virtual void Reset(int32 NumPoints) PURE_VIRTUAL(USplineMetadata::Reset, );
	virtual void Fixup(int32 NumPoints, USplineComponent* SplineComp) PURE_VIRTUAL(USplineMetadata::Fixup, );
};

USTRUCT()
struct FSplineCurves
{
	GENERATED_BODY()

	static FSplineCurves FromSplineInterface(const FSpline& InProxy);
	
	/** Spline built from position data. */
	UPROPERTY()
	FInterpCurveVector Position;

	/** Spline built from rotation data. */
	UPROPERTY()
	FInterpCurveQuat Rotation;

	/** Spline built from scale data. */
	UPROPERTY()
	FInterpCurveVector Scale;

	/** Input: distance along curve, output: parameter that puts you there. */
	UPROPERTY()
	FInterpCurveFloat ReparamTable;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<USplineMetadata> Metadata_DEPRECATED = nullptr;
#endif

	UPROPERTY(transient)
	uint32 Version = 0xffffffff;

	bool operator==(const FSplineCurves& Other) const
	{
		return Position == Other.Position && Rotation == Other.Rotation && Scale == Other.Scale;
	}

	bool operator!=(const FSplineCurves& Other) const
	{
		return !(*this == Other);
	}

	/** 
	 * Update the spline's internal data according to the passed-in params 
	 * @param	bClosedLoop				Whether the spline is to be considered as a closed loop.
	 * @param	bStationaryEndpoints	Whether the endpoints of the spline are considered stationary when traversing the spline at non-constant velocity.  Essentially this sets the endpoints' tangents to zero vectors.
	 * @param	ReparamStepsPerSegment	Number of steps per spline segment to place in the reparameterization table
	 * @param	bLoopPositionOverride	Whether to override the loop position with LoopPosition
	 * @param	LoopPosition			The loop position to use instead of the last key
	 * @param	Scale3D					The world scale to override
	 */
	ENGINE_API void UpdateSpline(bool bClosedLoop = false, bool bStationaryEndpoints = false, int32 ReparamStepsPerSegment = 10, bool bLoopPositionOverride = false, float LoopPosition = 0.0f, const FVector& Scale3D = FVector(1.0f));

	/** Returns the length of the specified spline segment up to the parametric value given */
	ENGINE_API float GetSegmentLength(const int32 Index, const float Param, bool bClosedLoop = false, const FVector& Scale3D = FVector(1.0f)) const;

	/** Returns total length along this spline */
	ENGINE_API float GetSplineLength() const;
};

/** A single point in linear approximation of a spline */
struct FSplinePositionLinearApproximation
{
	FSplinePositionLinearApproximation(const FVector& InPosition, float InSplineParam)
		: Position(InPosition)
		, SplineParam(InSplineParam)
	{}

	/**
	 * Build a linear approximation to the passed-in spline curves.
	 * @param	InCurves	The curves to approximate
	 * @param	OutPoints	The array of points to fill as a linear approximation
	 * @param	InDensity	Scalar applied to determine how many points are generated in the approximation. 1.0 = one point per distance unit.
	 */
	static ENGINE_API void Build(const FSplineCurves& InCurves, TArray<FSplinePositionLinearApproximation>& OutPoints, float InDensity = 0.5f);

	/** Position on the spline */
	FVector Position;

	/** Param of the spline at this position */
	float SplineParam;
};

USTRUCT(BlueprintType)
struct FSplinePoint
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	float InputKey;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	FVector Position;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	FVector ArriveTangent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	FVector LeaveTangent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	FRotator Rotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	FVector Scale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SplinePoint)
	TEnumAsByte<ESplinePointType::Type> Type;

	/** Default constructor */
	FSplinePoint()
		: InputKey(0.0f), Position(0.0f), ArriveTangent(0.0f), LeaveTangent(0.0f), Rotation(0.0f), Scale(1.0f), Type(ESplinePointType::Curve)
	{}

	/** Constructor taking a point position */
	FSplinePoint(float InInputKey, const FVector& InPosition)
		: InputKey(InInputKey), Position(InPosition), ArriveTangent(0.0f), LeaveTangent(0.0f), Rotation(0.0f), Scale(1.0f), Type(ESplinePointType::Curve)
	{}

	/** Constructor taking a point position and type, and optionally rotation and scale */
	FSplinePoint(float InInputKey, const FVector& InPosition, ESplinePointType::Type InType, const FRotator& InRotation = FRotator(0.0f), const FVector& InScale = FVector(1.0f))
		: InputKey(InInputKey), Position(InPosition), ArriveTangent(0.0f), LeaveTangent(0.0f), Rotation(InRotation), Scale(InScale), Type(InType)
	{}

	/** Constructor taking a point position and tangent, and optionally rotation and scale */
	FSplinePoint(float InInputKey,
				 const FVector& InPosition,
				 const FVector& InArriveTangent,
				 const FVector& InLeaveTangent,
				 const FRotator& InRotation = FRotator(0.0f),
				 const FVector& InScale = FVector(1.0f),
				 ESplinePointType::Type InType = ESplinePointType::CurveCustomTangent)
		: InputKey(InInputKey), Position(InPosition), ArriveTangent(InArriveTangent), LeaveTangent(InLeaveTangent), Rotation(InRotation), Scale(InScale), Type(InType)
	{}
};

/** 
 *	A spline component is a spline shape which can be used for other purposes (e.g. animating objects). It contains debug rendering capabilities.
 *	@see https://docs.unrealengine.com/latest/INT/Resources/ContentExamples/Blueprint_Splines
 */
UCLASS(ClassGroup=Utility, ShowCategories = (Mobility), HideCategories = (Activation, Physics, Collision, HLOD, LOD, TextureStreaming, Navigation, RayTracing, Lighting, Rendering, Mobile), meta=(BlueprintSpawnableComponent), MinimalAPI)
class USplineComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

protected:

	UPROPERTY(EditAnywhere, Category=Points)
	FSpline Spline;
	
public:

	/** Return the name of the authoritative spline property. */
	ENGINE_API static FName GetSplinePropertyName();

	/** Return the set of names of the backing spline properties. */
	ENGINE_API static TSet<FName> GetSplinePropertyNames();
	
	UPROPERTY(EditAnywhere, Replicated, Category=Points)
	FSplineCurves SplineCurves;

#if WITH_EDITORONLY_DATA
	/** Deprecated - please use GetSplinePointsPosition() to fetch this FInterpCurve */
	UPROPERTY()
	FInterpCurveVector SplineInfo_DEPRECATED;

	/** Deprecated - please use GetSplinePointsRotation() to fetch this FInterpCurve */
	UPROPERTY()
	FInterpCurveQuat SplineRotInfo_DEPRECATED;

	/** Deprecated - please use GetSplinePointsScale() to fetch this FInterpCurve */
	UPROPERTY()
	FInterpCurveVector SplineScaleInfo_DEPRECATED;

	UPROPERTY()
	FInterpCurveFloat SplineReparamTable_DEPRECATED;

	UPROPERTY()
	bool bAllowSplineEditingPerInstance_DEPRECATED;
#endif

	/** Number of steps per spline segment to place in the reparameterization table */
	UPROPERTY(EditAnywhere, Replicated, AdvancedDisplay, Category = Spline, meta=(ClampMin=4, UIMin=4, ClampMax=100, UIMax=100))
	int32 ReparamStepsPerSegment;

	/** Specifies the duration of the spline in seconds */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Spline)
	float Duration;

	/** Whether the endpoints of the spline are considered stationary when traversing the spline at non-constant velocity.  Essentially this sets the endpoints' tangents to zero vectors. */
	UPROPERTY(EditAnywhere, Replicated, BlueprintReadWrite, AdvancedDisplay, Category = Spline, meta=(EditCondition="!bClosedLoop"))
	bool bStationaryEndpoints;

	/** Whether the spline has been edited from its default by the spline component visualizer */
	UPROPERTY(EditAnywhere, Replicated, Category = Spline, meta=(DisplayName="Override Construction Script"))
	bool bSplineHasBeenEdited;

	UPROPERTY()
	/** Whether the UCS has made changes to the spline points */
	bool bModifiedByConstructionScript;

	/**
	 * Whether the spline points should be passed to the User Construction Script so they can be further manipulated by it.
	 * If false, they will not be visible to it, and it will not be able to influence the per-instance positions set in the editor.
	 */
	UPROPERTY(EditAnywhere, Replicated, BlueprintReadWrite, Category = Spline)
	bool bInputSplinePointsToConstructionScript;

	/** If true, the spline will be rendered if the Splines showflag is set. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Spline)
	bool bDrawDebug;

private:
	/**
	 * Whether the spline is to be considered as a closed loop.
	 * Use SetClosedLoop() to set this property, and IsClosedLoop() to read it.
	 */
	UPROPERTY(EditAnywhere, Replicated, Category = Spline)
	bool bClosedLoop;

	UPROPERTY(EditAnywhere, Replicated, Category = Spline, meta=(InlineEditConditionToggle=true))
	bool bLoopPositionOverride;

	UPROPERTY(EditAnywhere, Replicated, Category = Spline, meta=(EditCondition="bLoopPositionOverride"))
	float LoopPosition;

private:
	
	/** Material used for rendering spline segments. */
	UPROPERTY(Transient)
	TSoftObjectPtr<UMaterialInterface> LineMaterial = nullptr;

	/** Material used for rendering spline points. */
	UPROPERTY(Transient)
	TSoftObjectPtr<UMaterialInterface> PointMaterial = nullptr;

	/** Holds a strong reference to LineMaterial once loaded. */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> LineMaterialLifetimePtr = nullptr;

	/** Holds a strong reference to PointMaterial once loaded. */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> PointMaterialLifetimePtr = nullptr;

	int32 LineMaterialLoadID = INDEX_NONE;
	int32 PointMaterialLoadID = INDEX_NONE;
	
public:
	/** Default up vector in local space to be used when calculating transforms along the spline */
	UPROPERTY(EditAnywhere, Replicated, BlueprintReadWrite, Category = Spline, meta=(Spatial))
	FVector DefaultUpVector;

#if WITH_EDITORONLY_DATA
	/** Color of unselected spline component parts in the editor */
	UPROPERTY(EditAnywhere, Category = Editor, meta = (DisplayName="Editor Spline Unselected Color"))
	FLinearColor EditorUnselectedSplineSegmentColor;

	/** Color of selected spline component parts in the editor */
	UPROPERTY(EditAnywhere, Category = Editor, meta = (DisplayName="Editor Spline Selected Color"))
	FLinearColor EditorSelectedSplineSegmentColor;

	/** Color of spline point tangents in the editor */
	UPROPERTY(EditAnywhere, Category = Editor, meta = (DisplayName="Editor Spline Tangent Color"))
	FLinearColor EditorTangentColor;

	/** Whether the spline's leave and arrive tangents can be different */
	UPROPERTY(EditAnywhere, Category = Editor)
	bool bAllowDiscontinuousSpline;

	/** Adjust tangents after snapping. */
	UPROPERTY(EditAnywhere, Category = Editor)
	bool bAdjustTangentsOnSnap;

	/** Whether scale visualization should be displayed */
	UPROPERTY(EditAnywhere, Category = Editor, meta=(InlineEditConditionToggle=true))
	bool bShouldVisualizeScale;

	/** Width of spline in editor for use with scale visualization */
	UPROPERTY(EditAnywhere, Category = Editor, meta=(EditCondition="bShouldVisualizeScale"))
	float ScaleVisualizationWidth;

	/** Delegate that's called when this component is deselected in the editor */
	DECLARE_MULTICAST_DELEGATE_OneParam(DeselectedInEditorDelegate, TObjectPtr<USplineComponent>)
	DeselectedInEditorDelegate OnDeselectedInEditor;
#endif

	//~ Begin UObject Interface
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual bool GetIgnoreBoundsForEditorFocus() const override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent Interface.

#if UE_ENABLE_DEBUG_DRAWING
	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif
	
#if WITH_EDITOR
	ENGINE_API virtual void PushSelectionToProxy() override;
#endif
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface
#if UE_ENABLE_DEBUG_DRAWING
	/** Helper function to draw a vector curve */
	static ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FInterpCurveVector& SplineInfo, const FMatrix& LocalToWorld, const FLinearColor& LineColor, uint8 DepthPriorityGroup);
#endif

	// Safe for caching spline data in the legacy format. Will auto convert data if necessary.
	ENGINE_API FSplineCurves GetSplineCurves() const;
	ENGINE_API FSpline GetSpline() const;
	ENGINE_API void SetSpline(const FSpline& InSpline);
	ENGINE_API void SetSpline(const FSplineCurves& InSplineCurves);

	UE_EXPERIMENTAL(5.7, "Spline change notifications are experimental while the API is under active development.")
	FSimpleMulticastDelegate& GetOnSplineChanged() { return OnSplineChanged; }

	UE_EXPERIMENTAL(5.7, "Spline update notifications are experimental while the API is under active development.")
	FSimpleMulticastDelegate& GetOnSplineUpdated() { return OnSplineUpdated; }

	UE_EXPERIMENTAL(5.7, "Spline display setting change notifications are experimental while the API is under active development.")
	FSimpleMulticastDelegate& GetOnSplineDisplayChanged() { return OnSplineDisplayChanged; }

	ENGINE_API int32 GetVersion() const;
	ENGINE_API const FInterpCurveVector& GetSplinePointsPosition() const;
	ENGINE_API const FInterpCurveQuat& GetSplinePointsRotation() const;
	ENGINE_API const FInterpCurveVector& GetSplinePointsScale() const;

	virtual USplineMetadata* GetSplinePointsMetadata() { return nullptr; }
	virtual const USplineMetadata* GetSplinePointsMetadata() const { return nullptr; }

	/** Get the enabled Spline Point types for this spline component. */
	ENGINE_API virtual TArray<ESplinePointType::Type> GetEnabledSplinePointTypes() const;

	/** Controls the visibility of the Spline point location editor in the details panel. */
	virtual bool AllowsSpinePointLocationEditing() const { return true; }
	/** Controls the visibility of the Spline point rotation editor in the details panel. */
	virtual bool AllowsSplinePointRotationEditing() const { return true; }
	/** Controls the visibility of the Spline point scale editor in the details panel. */
	virtual bool AllowsSplinePointScaleEditing() const { return true; }
	/** Controls the visibility of the Spline point arrive tangent editor in the details panel. */
	virtual bool AllowsSplinePointArriveTangentEditing() const { return true; }
	/** Controls the visibility of the Spline point leave tangent editor in the details panel. */
	virtual bool AllowsSplinePointLeaveTangentEditing() const { return true; }

	ENGINE_API void ApplyComponentInstanceData(struct FSplineInstanceData* ComponentInstanceData, const bool bPostUCS);
	ENGINE_API void ApplyComponentInstanceData(struct FSplineComponentInstanceData* ComponentInstanceData, const bool bPostUCS);
	
	/** Reset the spline to its default shape (a spline of two points) */
	ENGINE_API void ResetToDefault();
	ENGINE_API bool CanResetToDefault() const;
	ENGINE_API static FSpline GetDefaultSpline();

	/** Update the spline tangents and SplineReparamTable */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API virtual void UpdateSpline();

	/** Set the spline to be edited outside of the construction script */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API virtual void SetOverrideConstructionScript(bool InOverride);

	/** Get location along spline at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get tangent along spline at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetTangentAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get unit direction along spline at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetDirectionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get rotator corresponding to rotation along spline at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FRotator GetRotationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get quaternion corresponding to rotation along spline at the provided input key value */
	ENGINE_API FQuat GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get up vector at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetUpVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get right vector at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetRightVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get transform at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FTransform GetTransformAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;

	/** Get roll in degrees at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetRollAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get scale at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetScaleAtSplineInputKey(float InKey) const;

	/** Get distance along the spline at the provided input key value */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetDistanceAlongSplineAtSplineInputKey(float InKey) const;

	/** Get distance along the spline at closest point of the provided input location */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetDistanceAlongSplineAtLocation(const FVector& InLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Specify unselected spline component segment color in the editor */
	UFUNCTION(BlueprintCallable, Category = Editor)
	ENGINE_API void SetUnselectedSplineSegmentColor(const FLinearColor& SegmentColor);

	/** Specify selected spline component segment color in the editor */
	UFUNCTION(BlueprintCallable, Category = Editor)
	ENGINE_API void SetSelectedSplineSegmentColor(const FLinearColor& SegmentColor);

	/** Specify selected spline component segment color in the editor */
	UFUNCTION(BlueprintCallable, Category = Editor)
	ENGINE_API void SetTangentColor(const FLinearColor& TangentColor);

	/** Specify whether this spline should be rendered when the Editor/Game spline show flag is set */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetDrawDebug(bool bShow);

	/** Specify whether the spline is a closed loop or not. The loop position will be at 1.0 after the last point's input key */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetClosedLoop(bool bInClosedLoop, bool bUpdateSpline = true);

	/** Specify whether the spline is a closed loop or not, and if so, the input key corresponding to the loop point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetClosedLoopAtPosition(bool bInClosedLoop, float Key, bool bUpdateSpline = true);

	/** Check whether the spline is a closed loop or not */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API bool IsClosedLoop() const;

	/** Clears all the points in the spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void ClearSplinePoints(bool bUpdateSpline = true);

	/** Adds an FSplinePoint to the spline. This contains its input key, position, tangent, rotation and scale. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void AddPoint(const FSplinePoint& Point, bool bUpdateSpline = true);

	/** Adds an array of FSplinePoints to the spline. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void AddPoints(const TArray<FSplinePoint>& Points, bool bUpdateSpline = true);

	/** Adds a point to the spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void AddSplinePoint(const FVector& Position, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Adds a point to the spline at the specified index */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void AddSplinePointAtIndex(const FVector& Position, int32 Index, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Removes point at specified index from the spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void RemoveSplinePoint(int32 Index, bool bUpdateSpline = true);

	/** Adds a world space point to the spline */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use AddSplinePoint, specifying SplineCoordinateSpace::World"))
	void AddSplineWorldPoint(const FVector& Position);

	/** Adds a local space point to the spline */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use AddSplinePoint, specifying SplineCoordinateSpace::Local"))
	void AddSplineLocalPoint(const FVector& Position);

	/** Sets the spline to an array of points */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetSplinePoints(const TArray<FVector>& Points, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Sets the spline to an array of world space points */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use SetSplinePoints, specifying SplineCoordinateSpace::World"))
	void SetSplineWorldPoints(const TArray<FVector>& Points);

	/** Sets the spline to an array of local space points */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use SetSplinePoints, specifying SplineCoordinateSpace::Local"))
	void SetSplineLocalPoints(const TArray<FVector>& Points);

	/** Move an existing point to a new location */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetLocationAtSplinePoint(int32 PointIndex, const FVector& InLocation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Move an existing point to a new world location */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use SetLocationAtSplinePoint, specifying SplineCoordinateSpace::World"))
	void SetWorldLocationAtSplinePoint(int32 PointIndex, const FVector& InLocation);

	/** Specify the tangent at a given spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetTangentAtSplinePoint(int32 PointIndex, const FVector& InTangent, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Specify the tangents at a given spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetTangentsAtSplinePoint(int32 PointIndex, const FVector& InArriveTangent, const FVector& InLeaveTangent, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Specify the up vector at a given spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetUpVectorAtSplinePoint(int32 PointIndex, const FVector& InUpVector, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Set the rotation of an existing spline point. This also aligns tangents with the forward vector of the rotation. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetRotationAtSplinePoint(int32 PointIndex, const FRotator& InRotation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);
	ENGINE_API void SetQuaternionAtSplinePoint(int32 PointIndex, const FQuat& InQuaternion, ESplineCoordinateSpace::Type CoordinateSpace, bool bUpdateSpline = true);

	/** Set the scale at a given spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetScaleAtSplinePoint(int32 PointIndex, const FVector& InScaleVector, bool bUpdateSpline = true); 

	/** Get the type of a spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API ESplinePointType::Type GetSplinePointType(int32 PointIndex) const;

	/** Specify the type of a spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetSplinePointType(int32 PointIndex, ESplinePointType::Type Type, bool bUpdateSpline = true);

	/** Get the number of points that make up this spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API int32 GetNumberOfSplinePoints() const;

	/** Get the number of segments that make up this spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API int32 GetNumberOfSplineSegments() const;

	/** Get the input key of the control point of the spline at the specified index. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetInputKeyValueAtSplinePoint(int32 PointIndex) const;

	/** Set the input key of the control point of the spline at the specified index. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API void SetInputKeyValueAtSplinePoint(int32 PointIndex, float InputKey, bool bUpdateSpline = true);

	/** Gets the spline point of the spline at the specified index */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FSplinePoint GetSplinePointAt(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;
		
	/** Get the location at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetLocationAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the world location at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetLocationAtSplinePoint, specifying SplineCoordinateSpace::World"))
	FVector GetWorldLocationAtSplinePoint(int32 PointIndex) const;

	/** Get the direction at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetDirectionAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the tangent at spline point. This fetches the Leave tangent of the point. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the arrive tangent at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetArriveTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the leave tangent at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetLeaveTangentAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the rotation at spline point as a quaternion */
	ENGINE_API FQuat GetQuaternionAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the rotation at spline point as a rotator */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FRotator GetRotationAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the up vector at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetUpVectorAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the right vector at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetRightVectorAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the amount of roll at spline point, in degrees */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetRollAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get the scale at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetScaleAtSplinePoint(int32 PointIndex) const;

	/** Get the transform at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FTransform GetTransformAtSplinePoint(int32 PointIndex, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;

	/** Get location and tangent at a spline point */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API void GetLocationAndTangentAtSplinePoint(int32 PointIndex, FVector& Location, FVector& Tangent, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Get local location and tangent at a spline point */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetLocationAndTangentAtSplinePoint, specifying SplineCoordinateSpace::Local"))
	void GetLocalLocationAndTangentAtSplinePoint(int32 PointIndex, FVector& LocalLocation, FVector& LocalTangent) const;

	/** Get the distance along the spline at the spline point */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API float GetDistanceAlongSplineAtSplinePoint(int32 PointIndex) const;

    /** Get a metadata property float value along the spline at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetFloatPropertyAtSplinePoint(int32 Index, FName PropertyName) const;
	
 	/** Get a metadata property vector value along the spline at spline point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetVectorPropertyAtSplinePoint(int32 Index, FName PropertyName) const;

	/** Returns total length along this spline */
	UFUNCTION(BlueprintCallable, Category=Spline) 
	ENGINE_API float GetSplineLength() const;

	/** Sets the default up vector used by this spline */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API void SetDefaultUpVector(const FVector& UpVector, ESplineCoordinateSpace::Type CoordinateSpace);

	/** Gets the default up vector used by this spline */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetDefaultUpVector(ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Convert a time value (in the range [0, Duration]) to an input key. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetInputKeyValueAtTime(float Time) const;

	/** Given a distance along the length of this spline, return the corresponding input key at that point */
	/** This method has been deprecated because it was incorrectly returning the input key at time. To maintain the same behavior,
	 *  replace it with GetTimeAtDistanceAlongSpline. To actually get the input key, instead call GetInputKeyValueAtDistanceAlongSpline. */
	UFUNCTION(BlueprintCallable, Category=Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetInputKeyValueAtDistanceAlongSpline to get input key at distance or GetTimeAtDistanceAlongSpline to get time value (normalized to duration) at distance (same logic as deprecated function)."))
	ENGINE_API float GetInputKeyAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return the corresponding input key at that point 
      * with a fractional component between the current input key and the next as a percentage. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetInputKeyValueAtDistanceAlongSpline(float Distance) const;
	
	/** Given a distance along the length of this spline, return the corresponding time at that point */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetTimeAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return the point in space where this puts you */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetLocationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return the point in world space where this puts you */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetLocationAtDistanceAlongSpline, specifying SplineCoordinateSpace::World"))
	FVector GetWorldLocationAtDistanceAlongSpline(float Distance) const;
	
	/** Given a distance along the length of this spline, return a unit direction vector of the spline tangent there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetDirectionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return a unit direction vector of the spline tangent there, in world space. */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetDirectionAtDistanceAlongSpline, specifying SplineCoordinateSpace::World"))
	FVector GetWorldDirectionAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return the tangent vector of the spline there. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetTangentAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return the tangent vector of the spline there, in world space. */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetTangentAtDistanceAlongSpline, specifying SplineCoordinateSpace::World"))
	FVector GetWorldTangentAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return a quaternion corresponding to the spline's rotation there. */
	ENGINE_API FQuat GetQuaternionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return a rotation corresponding to the spline's rotation there. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FRotator GetRotationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return a rotation corresponding to the spline's rotation there, in world space. */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetRotationAtDistanceAlongSpline, specifying SplineCoordinateSpace::World"))
	FRotator GetWorldRotationAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return a unit direction vector corresponding to the spline's up vector there. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetUpVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return a unit direction vector corresponding to the spline's right vector there. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetRightVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return the spline's roll there, in degrees. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API float GetRollAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a distance along the length of this spline, return the spline's scale there. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector GetScaleAtDistanceAlongSpline(float Distance) const;

	/** Given a distance along the length of this spline, return an FTransform corresponding to that point on the spline. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FTransform GetTransformAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;

	/** Given a time from 0 to the spline duration, return the point in space where this puts you */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetLocationAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the point in space where this puts you */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetLocationAtTime, specifying SplineCoordinateSpace::World"))
	FVector GetWorldLocationAtTime(float Time, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return a unit direction vector of the spline tangent there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetDirectionAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return a unit direction vector of the spline tangent there. */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetDirectionAtTime, specifying SplineCoordinateSpace::World"))
	FVector GetWorldDirectionAtTime(float Time, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the spline's tangent there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetTangentAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return a quaternion corresponding to the spline's rotation there. */
	ENGINE_API FQuat GetQuaternionAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return a rotation corresponding to the spline's position and direction there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FRotator GetRotationAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return a rotation corresponding to the spline's position and direction there, in world space. */
	UFUNCTION(BlueprintCallable, Category = Spline, meta = (DeprecatedFunction, DeprecationMessage = "Please use GetRotationAtTime, specifying SplineCoordinateSpace::World"))
	FRotator GetWorldRotationAtTime(float Time, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the spline's up vector there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetUpVectorAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the spline's right vector there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetRightVectorAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the spline's transform at the corresponding position. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FTransform GetTransformAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false, bool bUseScale = false) const;

	/** Given a time from 0 to the spline duration, return the spline's roll there, in degrees. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API float GetRollAtTime(float Time, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseConstantVelocity = false) const;

	/** Given a time from 0 to the spline duration, return the spline's scale there. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector GetScaleAtTime(float Time, bool bUseConstantVelocity = false) const;

    /** Given a location, in world space, return the input key closest to that location. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const;

	/** Given a location, in world space, return the input key closest to that location on the specified segment. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API float FindInputKeyOnSegmentClosestToWorldLocation(const FVector& WorldLocation, int32 Index) const;
	
	/** Given a location, in world space, return the point on the curve that is closest to the location. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector FindLocationClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return a unit direction vector of the spline tangent closest to the location. */
	UFUNCTION(BlueprintCallable, Category=Spline)
	ENGINE_API FVector FindDirectionClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return the tangent vector of the spline closest to the location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector FindTangentClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return a quaternion corresponding to the spline's rotation closest to the location. */
	ENGINE_API FQuat FindQuaternionClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return rotation corresponding to the spline's rotation closest to the location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FRotator FindRotationClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return a unit direction vector corresponding to the spline's up vector closest to the location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector FindUpVectorClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

    /** Given a location, in world space, return a unit direction vector corresponding to the spline's right vector closest to the location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector FindRightVectorClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

    /** Given a location, in world space, return the spline's roll closest to the location, in degrees. */
	UFUNCTION(BlueprintCallable, Category = Spline)
    ENGINE_API float FindRollClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace) const;

	/** Given a location, in world space, return the spline's scale closest to the location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FVector FindScaleClosestToWorldLocation(const FVector& WorldLocation) const;

	/** Given a location, in world space, return an FTransform closest to that location. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API FTransform FindTransformClosestToWorldLocation(const FVector& WorldLocation, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;

	/** Given a threshold, recursively sub-divides the spline section until the list of segments (polyline) matches the spline shape. Note: Prefer ConvertSplineToPolyline_InDistanceRange */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API bool DivideSplineIntoPolylineRecursive(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;

	/** Given a threshold, recursively sub-divides the spline section until the list of segments (polyline) matches the spline shape. Note: Prefer ConvertSplineToPolyline_InDistanceRange */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API bool DivideSplineIntoPolylineRecursiveWithDistances(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const;

	/** Given a threshold, returns a list of vertices along the spline segment that, treated as a list of segments (polyline), matches the spline shape. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API bool ConvertSplineSegmentToPolyLine(int32 SplinePointStartIndex, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;

	/** Given a threshold, returns a list of vertices along the spline that, treated as a list of segments (polyline), matches the spline shape. */
	UFUNCTION(BlueprintCallable, Category = Spline)
	ENGINE_API bool ConvertSplineToPolyLine(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;

	/** Given a threshold, returns a list of vertices along the spline that, treated as a list of segments (polyline), matches the spline shape. Also fills a list of corresponding distances along the spline for each point. */
	UFUNCTION(BlueprintPure = false, Category = Spline)
	ENGINE_API bool ConvertSplineToPolyLineWithDistances(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const;

	/** Given a threshold and a start and end distance range, returns a list of vertices along the spline that, treated as a list of segments (polyline), matches the spline shape in that range. Also fills a list of corresponding distances along the spline for each point. */
	UFUNCTION(BlueprintPure = false, Category = Spline)
	ENGINE_API bool ConvertSplineToPolyline_InDistanceRange(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, float StartDistAlongSpline, float EndDistAlongSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline, bool bAllowWrappingIfClosed = true) const;

	/** Given a threshold and start and end time range, returns a list of vertices along the spline that, treated as a list of segments (polyline), matches the spline shape in that range. Also fills a list of corresponding distances along the spline for each point. */
	UFUNCTION(BlueprintPure = false, Category = Spline)
	ENGINE_API bool ConvertSplineToPolyline_InTimeRange(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, float StartTimeAlongSpline, float EndTimeAlongSpline, bool bUseConstantVelocity, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline, bool bAllowWrappingIfClosed = true) const;

	/* ---- Metadata API ---- */
	
	/**
	 * @return True if this component currently supports arbitrary attribute channels, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API bool SupportsAttributes() const;

	/**
	 * Remove an attribute channel.
	 * @param PropertyName The name of the channel to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API bool RemovePropertyChannel(FName PropertyName);
	
	/**
	 * Get the number of attribute values in a specified attribute channel.
	 * @param PropertyName The name of the attribute channel.
	 * @return The number of attribute values in the specified channel.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API int32 GetNumberOfPropertyValues(FName PropertyName) const;
		
	/**
	 * @return An array of all float attribute channel names.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API TArray<FName> GetFloatPropertyChannels() const;

	/**
	 * Create a float attribute channel.
	 * @param PropertyName The desired name of the new attribute channel.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API void CreateFloatPropertyChannel(FName PropertyName);
	
	/**
	 * Get a float attribute value by parameter.
	 * @param InKey The parameter to use when evaluating the specified attribute channel.
	 * @param PropertyName The name of the attribute channel to evaluate.
	 * @return The attribute value.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API float GetFloatPropertyAtSplineInputKey(float InKey, FName PropertyName) const;

	/**
	 * Add a float attribute value by parameter.
	 * @param InKey The desired attribute parameter.
	 * @param Value The desired attribute value.
	 * @param PropertyName The name of the attribute channel that will contain the new attribute value.
	 * @return The index of the new attribute.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API int32 AddFloatPropertyAtSplineInputKey(float InKey, float Value, FName PropertyName);

	/**
	 * Get a float attribute parameter by index.
	 * @param Index The index of the specified attribute.
	 * @param PropertyName The name of the attribute channel containing the specified attribute.
	 * @return The attribute parameter.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API float GetFloatPropertyInputKeyAtIndex(int32 Index, FName PropertyName) const;

	/**
	 * Set a float attribute parameter by index.
	 * @param Index The index of the specified attribute.
	 * @param InKey The desired attribute parameter.
	 * @param PropertyName The name of the attribute channel containing the specified attribute.
	 * @return The index of the specified attribute after changing the attribute parameter (it can change).
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API int32 SetFloatPropertyInputKeyAtIndex(int32 Index, float InKey, FName PropertyName);

	/**
	 * Get a float attribute value by index.
	 * @param Index The index of the specified attribute.
	 * @param PropertyName The name of the attribute channel containing the specified attribute.
	 * @return The attribute value.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API float GetFloatPropertyAtIndex(int32 Index, FName PropertyName) const;

	/**
	 * Set a float attribute value by index.
	 * @param Index The index of the specified attribute.
	 * @param Value The desired attribute value.
	 * @param PropertyName The name of the attribute channel containing the specified attribute.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API void SetFloatPropertyAtIndex(int32 Index, float Value, FName PropertyName);

	/**
	 * Remove an attribute value by index.
	 * @param Index The index of the specified attribute.
	 * @param PropertyName The name of the attribute channel containing the specified attribute.
	 */
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata")
	ENGINE_API void RemovePropertyAtIndex(int32 Index, FName PropertyName);

	// todo: deprecate
	FInterpCurveVector& GetSplinePointsPosition() { return WarninglessSplineCurves().Position; }
	FInterpCurveQuat& GetSplinePointsRotation() { return WarninglessSplineCurves().Rotation; }
	FInterpCurveVector& GetSplinePointsScale() { return WarninglessSplineCurves().Scale; }

public: // Deprecated Interface

	UE_DEPRECATED(5.8, "SynchronizeSplines is deprecated.")
	void SynchronizeSplines() {}

	UE_DEPRECATED(5.8, "PopulateFromLegacy is deprecated.")
	void PopulateFromLegacy() {}

	UE_DEPRECATED(5.8, "SetFloatPropertyAtSplineInputKey is deprecated, please use AddFloatPropertyAtSplineInputKey")
	UFUNCTION(BlueprintCallable, Category = "Spline|Metadata", meta = (DeprecatedFunction, DeprecationMessage = "Please use AddFloatPropertyAtSplineInputKey"))
	int32 SetFloatPropertyAtSplineInputKey(float InKey, float Value, FName PropertyName) const { return INDEX_NONE; }

protected:

	/** Returns the length of the specified spline segment up to the parametric value given */
	ENGINE_API float GetSegmentLength(const int32 Index, const float Param = 1.0f) const;

	/** Delegate broadcast any time the spline has potentially been mutated. */
	FSimpleMulticastDelegate OnSplineChanged;

	/** Delegate broadcast any time UpdateSpline is called (regardless of whether or not the spline has changed). */
	FSimpleMulticastDelegate OnSplineUpdated;

	/** Delegate broadcast any time the spline's visualization properties have changed. */
	FSimpleMulticastDelegate OnSplineDisplayChanged;
	
private:
	
	/** Whether or not FSplineCurves is the authoritative data. If false, we assume Spline is implemented as FTangentSpline. */
	ENGINE_API bool ShouldUseSplineCurves() const;

	/** Returns an interval representing the full range of valid input keys, or an empty interval if the spline is empty. */
	UE::Geometry::FInterval1f GetInputKeyRange() const;
	
	/**
	 * This function takes on the roles of the previously existing functions GetPositionPointSafe, GetRotationPointSafe, and GetScalePointSafe.
	 * Specifically, it makes sure that a provided index is valid by
	 * 1) Specially handling Index == NumPoints if bClosedLoop is true in order to allow users to imagine the connecting point as an extra virtual point
	 * 2) Preventing out of bound accesses by clamping the index. This clamping happens after 1).
	 *
	 * This function is only used where the 3 aforementioned functions were called. It should probably be used more widely and consistently in the future.
	 */
	int32 GetClampedIndex(int32 Index) const
	{
		// Treat Index == GetNumberOfSplinePoints() as Index == 0, then clamp to valid index range.
		return FMath::Clamp(bClosedLoop && Index == GetNumberOfSplinePoints() ? 0 : Index, 0, GetNumberOfSplinePoints() - 1);
	}
	
	/** The dummy value used for queries when there are no point in a spline */
	static ENGINE_API const FInterpCurveVector DummyPointPosition;
	static ENGINE_API const FInterpCurveQuat DummyPointRotation;
	static ENGINE_API const FInterpCurveVector DummyPointScale;

	// Remove when appending _DEPRECATED to SplineCurves.
	FSplineCurves& WarninglessSplineCurves()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SplineCurves;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Remove when appending _DEPRECATED to SplineCurves.
	const FSplineCurves& WarninglessSplineCurves() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SplineCurves;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UPROPERTY()
	ELastAuthority LastAuthority = ELastAuthority::Unset;
	
	/** Checks for consistency between Spline and SplineCurves. */
	ENGINE_API bool Validate() const;
	
	/** Set the SplineCurves with the default shape i.e. a spline of two points (Used by default constructor) */
	ENGINE_API void SetDefaultSpline();

	/** Returns the parametric value t which would result in a spline segment of the given length between S(0)...S(t) */
	ENGINE_API float GetSegmentParamFromLength(const int32 Index, const float Length, const float SegmentLength) const;

	// Internal helper function called by ConvertSplineSegmentToPolyLine -- assumes the input is within a half-segment, so testing the distance to midpoint will be an accurate guide to subdivision
	bool DivideSplineIntoPolylineRecursiveWithDistancesHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const;
	// Internal helper function called by ConvertSplineSegmentToPolyLine -- assumes the input is within a half-segment, so testing the distance to midpoint will be an accurate guide to subdivision
	bool DivideSplineIntoPolylineRecursiveHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;

	// FSplineComponentVisualizer will access some private members when attempting to call NotifyPropertiesModified
	friend class FSplineComponentVisualizer;

	// FSplinePointDetails will access ShouldUseSplineCurves for enabling/disabling some editing functionality
	friend class FSplinePointDetails;

	// AFunctionalSplineTestBase accesses private methods for functional testing.
	friend class AFunctionalSplineTestBase;
};

/** Used to store spline data during RerunConstructionScripts */
USTRUCT()
struct FSplineInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FSplineInstanceData()
		: bSplineHasBeenEdited(false)
	{}
	explicit FSplineInstanceData(const USplineComponent* SourceComponent)
		: FSceneComponentInstanceData(SourceComponent)
		, bSplineHasBeenEdited(false)
	{}
	virtual ~FSplineInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return Super::ContainsData() || bSplineHasBeenEdited;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<USplineComponent>(Component)->ApplyComponentInstanceData(this, (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript));
	}

	UPROPERTY()
	bool bSplineHasBeenEdited;

	UPROPERTY()
	FSplineCurves SplineCurves;

	UPROPERTY()
	bool bClosedLoop = false;

	UPROPERTY()
	FSplineCurves SplineCurvesPreUCS;
};

/** Used to store spline data during RerunConstructionScripts */
USTRUCT()
struct FSplineComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FSplineComponentInstanceData()
		: bSplineHasBeenEdited(false)
	{}
	explicit FSplineComponentInstanceData(const USplineComponent* SourceComponent)
		: FSceneComponentInstanceData(SourceComponent)
		, bSplineHasBeenEdited(false)
	{}
	explicit FSplineComponentInstanceData(const USplineComponent* SourceComponent, const FSpline& SourceSpline)
		: FSceneComponentInstanceData(SourceComponent)
		, bSplineHasBeenEdited(false)
		, Spline(SourceSpline)
	{}
	virtual ~FSplineComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return Super::ContainsData() || bSplineHasBeenEdited;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<USplineComponent>(Component)->ApplyComponentInstanceData(this, (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript));
	}

	UPROPERTY()
	bool bSplineHasBeenEdited;

	UPROPERTY()
	FSpline Spline;

	UPROPERTY()
	bool bClosedLoop = false;

	UPROPERTY()
	FSpline SplinePreUCS;
};

ENGINE_API EInterpCurveMode ConvertSplinePointTypeToInterpCurveMode(ESplinePointType::Type SplinePointType);
ENGINE_API ESplinePointType::Type ConvertInterpCurveModeToSplinePointType(EInterpCurveMode InterpCurveMode);

// Deprecated method definitions
inline void USplineComponent::AddSplineWorldPoint(const FVector& Position) { AddSplinePoint(Position, ESplineCoordinateSpace::World); }
inline void USplineComponent::AddSplineLocalPoint(const FVector& Position) { AddSplinePoint(Position, ESplineCoordinateSpace::Local); }
inline void USplineComponent::SetSplineWorldPoints(const TArray<FVector>& Points) { SetSplinePoints(Points, ESplineCoordinateSpace::World); }
inline void USplineComponent::SetSplineLocalPoints(const TArray<FVector>& Points) { SetSplinePoints(Points, ESplineCoordinateSpace::Local); }
inline void USplineComponent::SetWorldLocationAtSplinePoint(int32 PointIndex, const FVector& InLocation) { SetLocationAtSplinePoint(PointIndex, InLocation, ESplineCoordinateSpace::World); }
inline FVector USplineComponent::GetWorldLocationAtSplinePoint(int32 PointIndex) const { return GetLocationAtSplinePoint(PointIndex, ESplineCoordinateSpace::World); }
inline void USplineComponent::GetLocalLocationAndTangentAtSplinePoint(int32 PointIndex, FVector& LocalLocation, FVector& LocalTangent) const { GetLocationAndTangentAtSplinePoint(PointIndex, LocalLocation, LocalTangent, ESplineCoordinateSpace::Local); }
inline FVector USplineComponent::GetWorldLocationAtDistanceAlongSpline(float Distance) const { return GetLocationAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World); }
inline FVector USplineComponent::GetWorldDirectionAtDistanceAlongSpline(float Distance) const { return GetDirectionAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World); }
inline FVector USplineComponent::GetWorldTangentAtDistanceAlongSpline(float Distance) const { return GetTangentAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World); }
inline FRotator USplineComponent::GetWorldRotationAtDistanceAlongSpline(float Distance) const { return GetRotationAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World); }
inline FVector USplineComponent::GetWorldLocationAtTime(float Time, bool bUseConstantVelocity) const { return GetLocationAtTime(Time, ESplineCoordinateSpace::World, bUseConstantVelocity); }
inline FVector USplineComponent::GetWorldDirectionAtTime(float Time, bool bUseConstantVelocity) const { return GetDirectionAtTime(Time, ESplineCoordinateSpace::World, bUseConstantVelocity); }
inline FRotator USplineComponent::GetWorldRotationAtTime(float Time, bool bUseConstantVelocity) const { return GetRotationAtTime(Time, ESplineCoordinateSpace::World, bUseConstantVelocity); }

PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS