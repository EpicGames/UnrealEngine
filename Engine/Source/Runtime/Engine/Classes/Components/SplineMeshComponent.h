// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Components/StaticMeshComponent.h"
#include "SplineMeshShaderParams.h"

#include "SplineMeshComponent.generated.h"

class FPrimitiveSceneProxy;
class ULightComponent;
struct FNavigableGeometryExport;
class UBodySetup;
struct FMeshDescription;

// Helper for packing spline mesh shader parameters into a float4 buffer
ENGINE_API void PackSplineMeshParams(const FSplineMeshShaderParams& Params, const TArrayView<FVector4f>& Output);

UENUM(BlueprintType)
namespace ESplineMeshAxis
{
	enum Type : int
	{
		X,
		Y,
		Z,
	};
}

namespace UE::SplineMesh
{
	ENGINE_API float RealToFloatChecked(const double Value);
}

/** 
 * Structure that holds info about spline, passed to renderer to deform UStaticMesh.
 * Also used by Lightmass, so be sure to update Lightmass::FSplineMeshParams and the static lighting code if this changes!
 */
USTRUCT()
struct FSplineMeshParams
{
	GENERATED_USTRUCT_BODY()

	/** Start location of spline, in component space. */
	UPROPERTY(EditAnywhere, Category=SplineMesh)
	FVector StartPos;

	/** Start tangent of spline, in component space. */
	UPROPERTY(EditAnywhere, Category=SplineMesh)
	FVector StartTangent;

	/** X and Y scale applied to mesh at start of spline. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	FVector2D StartScale;

	/** Roll around spline applied at start, in radians. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	float StartRoll;

	/** Roll around spline applied at end, in radians. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay, meta=(DisplayAfter="EndTangent"))
	float EndRoll;

	/** Starting offset of the mesh from the spline, in component space. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	FVector2D StartOffset;

	/** End location of spline, in component space. */
	UPROPERTY(EditAnywhere, Category=SplineMesh)
	FVector EndPos;

	/** X and Y scale applied to mesh at end of spline. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	FVector2D EndScale;

	/** End tangent of spline, in component space. */
	UPROPERTY(EditAnywhere, Category = SplineMesh)
	FVector EndTangent;

	/** Ending offset of the mesh from the spline, in component space. */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	FVector2D EndOffset;

	/**
	 * How much to scale the calculated culling bounds of Nanite clusters after deformation.
	 * NOTE: This should only be set greater than 1.0 if it fixes visible issues with clusters being
	 * incorrectly culled.
	 */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay, meta=(ClampMin=1.0))
	float NaniteClusterBoundsScale;


	FSplineMeshParams()
		: StartPos(ForceInit)
		, StartTangent(ForceInit)
		, StartScale(ForceInit)
		, StartRoll(0)
		, EndRoll(0)
		, StartOffset(ForceInit)
		, EndPos(ForceInit)
		, EndScale(ForceInit)
		, EndTangent(ForceInit)
		, EndOffset(ForceInit)
		, NaniteClusterBoundsScale(1.0f)
	{
	}

};

/** 
 *	A Spline Mesh Component is a derivation of a Static Mesh Component which can be deformed using a spline. Only a start and end position (and tangent) can be specified.  
 *	@see https://docs.unrealengine.com/latest/INT/Resources/ContentExamples/Blueprint_Splines
 */
UCLASS(ClassGroup=Rendering, hidecategories=(Physics), meta=(BlueprintSpawnableComponent), MinimalAPI)
class USplineMeshComponent : public UStaticMeshComponent, public IInterface_CollisionDataProvider
{
	GENERATED_UCLASS_BODY()

	/** Spline that is used to deform mesh */
	UPROPERTY(EditAnywhere, Category=SplineMesh, meta=(ShowOnlyInnerProperties))
	FSplineMeshParams SplineParams;

	/** Axis (in component space) that is used to determine X axis for co-ordinates along spline */
	UPROPERTY(EditAnywhere, Category=SplineMesh)
	FVector SplineUpDir;

	/** Minimum coordinate along the spline forward axis which corresponds to start of spline. If set to 0.0, will use bounding box to determine bounds */
	UPROPERTY(EditAnywhere, Category = SplineMesh, AdvancedDisplay)
	float SplineBoundaryMin;

	// Used to automatically trigger rebuild of collision data
	UPROPERTY()
	FGuid CachedMeshBodySetupGuid;

	/** 
	* The max draw distance to use in the main pass when also rendering to a runtime virtual texture. 
	* This is only exposed to the user through the same setting on ULandscapeSplineSegment. 
	*/
	UPROPERTY()
	float VirtualTextureMainPassMaxDrawDistance = 0.f;

	// Navigation bounds can differ from primitive bounds since NavCollision can hold more geometry
	FBox CachedNavigationBounds;

	// Physics data.
	UPROPERTY()
	TObjectPtr<UBodySetup> BodySetup;

	/** Maximum coordinate along the spline forward axis which corresponds to end of spline. If set to 0.0, will use bounding box to determine bounds */
	UPROPERTY(EditAnywhere, Category = SplineMesh, AdvancedDisplay)
	float SplineBoundaryMax;

	/** Chooses the forward axis for the spline mesh orientation */
	UPROPERTY(EditAnywhere, Category=SplineMesh)
	TEnumAsByte<ESplineMeshAxis::Type> ForwardAxis;

	/** If true, spline mesh properties - StartPos, EndPos, StartTangent and EndTangent- may be edited per instance in the level viewport. Otherwise, the spline mesh should be initialized in the construction script. */
	UPROPERTY(EditDefaultsOnly, Category = Spline)
	uint8 bAllowSplineEditingPerInstance:1;

	/** If true, will use smooth interpolation (ease in/out) for Scale, Roll, and Offset along this section of spline. If false, uses linear */
	UPROPERTY(EditAnywhere, Category=SplineMesh, AdvancedDisplay)
	uint8 bSmoothInterpRollScale:1;

	// Indicates that the mesh needs updating
	UPROPERTY(transient)
	uint8 bMeshDirty : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	uint8 bSelected:1;

	UPROPERTY(EditAnywhere, Category = HLOD, AdvancedDisplay)
	TObjectPtr<UHLODLayer> HLODLayerOverrideForSplinePartitioning;
#endif

private:
	/** Indicates that we will never use convex or trimesh shapes. This is an optimization to skip checking for binary data. */
	/**
	* TODO Chaos this is to opt out of CreatePhysicsMeshes for certain meshes
	* Better long term mesh is to not call CreatePhysicsMeshes until it is known there is a mesh instance that needs it.
	*/
	UPROPERTY(EditAnywhere, Getter, Setter, Category = Collision)
	uint8 bNeverNeedsCookedCollisionData:1;

public:
	ENGINE_API void InitVertexFactory(int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer);

	//Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API bool IsRelevantForSplinePartitioning() const;
	ENGINE_API UHLODLayer* GetHLODLayerForSplinePartitioning() const;
	ENGINE_API virtual bool IsEditorOnly() const override;
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//End UObject Interface

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent Interface.

	ENGINE_API void ApplyComponentInstanceData(struct FSplineMeshInstanceData* ComponentInstanceData);

	//Begin USceneComponent Interface
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	ENGINE_API virtual void UpdateBounds() override;
	//End USceneComponent Interface

	//Begin UPrimitiveComponent Interface
protected:
	ENGINE_API virtual void OnCreatePhysicsState() override;

public:
	// OnCreatePhysicsState calls RecreateCollision() which uses DuplicateObject
	// and other GT-only operations, making it unsafe for async execution.
	virtual bool AllowsAsyncPhysicsStateCreation() const override { return false; }
	virtual bool AllowsAsyncPhysicsStateDestruction() const override { return false; }

	float ComputeRatioAlongSpline(float DistanceAlong) const;

	/** Returns the normalized range on the spline where the visual mesh is located taking custom range into account. */
	void ComputeVisualMeshSplineTRange(float& MinT, float& MaxT) const;

	/**
	 * Computes the bounding box, in world space, for a given bounding box distorted by the spline in local space.
	 * By default this method uses the provided mesh bounds that were used to define the spline range [0,1] so all points are expected
	 * to stay in that range. In case the bounds to deform are overriden by the optional parameter then linear extrapolation
	 * will be applied at the beginning and at the end of the spline for the exceeding part.
	 * @param InLocalToWorld Transformation to apply to the computed bounds to convert them from local space to world space.
	 * @param InMeshBounds Bounds of the static mesh that get distorted by the spline.
	 * @param InBoundsToDistort Optional bounds to distort instead of using the mesh bounds.
	 * @return Bounds, in world space, of the provided bounds distorted by the spline.
	 */
	FBox ComputeDistortedBounds(const FTransform& InLocalToWorld, const FBoxSphereBounds& InMeshBounds, const FBoxSphereBounds* InBoundsToDistort = nullptr) const;

public:
	ENGINE_API virtual bool SupportsDefaultCollision() override;
	ENGINE_API virtual class UBodySetup* GetBodySetup() override;
	ENGINE_API class UBodySetup* GetBodySetup() const;
#if WITH_EDITOR
	virtual bool ShouldRenderSelected() const override
	{
		return Super::ShouldRenderSelected() || bSelected;
	}
#endif
	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	virtual float GetVirtualTextureMainPassMaxDrawDistance() const override { return VirtualTextureMainPassMaxDrawDistance; }
	//End UPrimitiveComponent Interface

	//Begin UStaticMeshComponent Interface
public:
	ENGINE_API virtual class FStaticMeshStaticLightingMesh* AllocateStaticLightingMesh(int32 LODIndex, const TArray<ULightComponent*>& InRelevantLights) override;
protected:
	ENGINE_API virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(::Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
	//End UStaticMeshComponent Interface

	//~ Begin Interface_CollisionDataProvider Interface
public:
	ENGINE_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	ENGINE_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool WantsNegXTriMesh() override { return false; }
	ENGINE_API virtual void GetMeshId(FString& OutMeshId) override;
	ENGINE_API void GetMeshId(FString& OutMeshId) const;
	ENGINE_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	//~ End Interface_CollisionDataProvider Interface

	//~ Begin INavRelevantInterface
	ENGINE_API virtual FBox GetNavigationBounds() const override;
	//~ End  INavRelevantInterface

	/** Generates FSplineMeshShaderParams for the current state of the component */
	ENGINE_API FSplineMeshShaderParams CalculateShaderParams() const;

	/** Called to notify render thread and possibly collision of a change in spline params or mesh */
	ENGINE_API void UpdateRenderStateAndCollision();

	/** Update the collision and render state on the spline mesh following changes to its geometry */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void UpdateMesh();

	/** Same as UpdateMesh, but does not wait until the end of frame and can be used in non-game threads */
	ENGINE_API void UpdateMesh_Concurrent();

	/** Get the start position of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector GetStartPosition() const;

	/** Set the start position of spline in local space */
	UFUNCTION(BlueprintCallable, Category=SplineMesh)
	ENGINE_API void SetStartPosition(FVector StartPos, bool bUpdateMesh = true);

	/** Get the start tangent vector of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector GetStartTangent() const;

	/** Set the start tangent vector of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetStartTangent(FVector StartTangent, bool bUpdateMesh = true);

	/** Get the end position of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector GetEndPosition() const;

	/** Set the end position of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetEndPosition(FVector EndPos, bool bUpdateMesh = true);

	/** Get the end tangent vector of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector GetEndTangent() const;

	/** Set the end tangent vector of spline in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetEndTangent(FVector EndTangent, bool bUpdateMesh = true);

	/** Set the start and end, position and tangent, all in local space */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetStartAndEnd(FVector StartPos, FVector StartTangent, FVector EndPos, FVector EndTangent, bool bUpdateMesh = true);

	/** Get the start scaling */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector2D GetStartScale() const;

	/** Set the start scaling */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetStartScale(FVector2D StartScale = FVector2D(1,1), bool bUpdateMesh = true);

	/** Get the start roll, in radians */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API float GetStartRoll() const;

	/** Set the start roll, in radians */
	UFUNCTION(BlueprintCallable, Category = SplineMesh, meta = (DisplayName = "Set Start Roll Radians"))
	ENGINE_API void SetStartRoll(float StartRoll, bool bUpdateMesh = true);

	/** Set the start roll in degrees */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetStartRollDegrees(float StartRollDegrees, bool bUpdateMesh = true);

	/** Get the start offset */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector2D GetStartOffset() const;

	/** Set the start offset */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetStartOffset(FVector2D StartOffset, bool bUpdateMesh = true);

	/** Get the end scaling */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector2D GetEndScale() const;

	/** Set the end scaling */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetEndScale(FVector2D EndScale = FVector2D(1,1), bool bUpdateMesh = true);

	/** Get the end roll, in radians */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API float GetEndRoll() const;

	/** Set the end roll, in radians */
	UFUNCTION(BlueprintCallable, Category = SplineMesh, meta = (DisplayName = "Set End Roll Radians"))
	ENGINE_API void SetEndRoll(float EndRoll, bool bUpdateMesh = true);

	/** Set the end roll in degrees */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetEndRollDegrees(float EndRollDegrees, bool bUpdateMesh = true);

	/** Get the end offset */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector2D GetEndOffset() const;

	/** Set the end offset */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetEndOffset(FVector2D EndOffset, bool bUpdateMesh = true);

	/** Get the forward axis */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API ESplineMeshAxis::Type GetForwardAxis() const;

	/** Set the forward axis */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetForwardAxis(ESplineMeshAxis::Type InForwardAxis, bool bUpdateMesh = true);

	/** Get the spline up direction */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API FVector GetSplineUpDir() const;

	/** Set the spline up direction */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetSplineUpDir(const FVector& InSplineUpDir, bool bUpdateMesh = true);

	/** Get the boundary min */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API float GetBoundaryMin() const;

	/** Set the boundary min */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetBoundaryMin(float InBoundaryMin, bool bUpdateMesh = true);

	/** Get the boundary max */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API float GetBoundaryMax() const;

	/** Set the boundary max */
	UFUNCTION(BlueprintCallable, Category = SplineMesh)
	ENGINE_API void SetBoundaryMax(float InBoundaryMax, bool bUpdateMesh = true);

	/** Setter for bNeverNeedsCookedCollisionData */
	ENGINE_API void SetbNeverNeedsCookedCollisionData(bool bInValue); 

	/** Getter for bNeverNeedsCookedCollisionData */
	bool GetbNeverNeedsCookedCollisionData() const { return bNeverNeedsCookedCollisionData; }

	// Destroys the body setup, used to clear collision if the mesh goes missing
	ENGINE_API void DestroyBodySetup();
	// Builds collision for the spline mesh (if collision is enabled)
	ENGINE_API void RecreateCollision();

	/**
	 * Calculates the spline transform, including roll, scale, and offset along the spline at a specified distance
	 */
	ENGINE_API FTransform CalcSliceTransform(const float DistanceAlong) const;

	/**
	 * Calculates the spline transform, including roll, scale, and offset along the spline at a specified alpha interpolation parameter along the spline
	 * @Note:  This is mirrored to Lightmass::CalcSliceTransform() and LocalVertexShader.usf.  If you update one of these, please update them all!
	 */
	ENGINE_API FTransform CalcSliceTransformAtSplineOffset(const float Alpha, const float MinT=0.f, const float MaxT=1.0f) const;

	/** Propagates deformation along spline to mesh data */
	ENGINE_API void PropagateSplineDeformationToMesh(FMeshDescription& InOutMeshDescription) const;

	inline static const double& GetAxisValueRef(const FVector3d& InVector, ESplineMeshAxis::Type InAxis);
	inline static double& GetAxisValueRef(FVector3d& InVector, ESplineMeshAxis::Type InAxis);

	inline static const float& GetAxisValueRef(const FVector3f& InVector, ESplineMeshAxis::Type InAxis);
	inline static float& GetAxisValueRef(FVector3f& InVector, ESplineMeshAxis::Type InAxis);

	/** Returns a vector which, when componentwise-multiplied by another vector, will zero all the components not corresponding to the supplied ESplineMeshAxis */
	inline static FVector GetAxisMask(ESplineMeshAxis::Type InAxis);
	ENGINE_API static bool ShouldRenderNaniteSplineMeshes();

	ENGINE_API virtual float GetTextureStreamingTransformScale() const override;

	ENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;

private:
	ENGINE_API void UpdateRenderStateAndCollision_Internal(bool bConcurrent);
};

/** Used to store spline mesh data during RerunConstructionScripts */
USTRUCT()
struct FSplineMeshInstanceData : public FStaticMeshComponentInstanceData
{
	GENERATED_BODY()
public:
	FSplineMeshInstanceData() = default;
	explicit FSplineMeshInstanceData(const USplineMeshComponent* SourceComponent);

	virtual ~FSplineMeshInstanceData() override = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<USplineMeshComponent>(Component)->ApplyComponentInstanceData(this);
	}

	UPROPERTY()
	FVector StartPos = FVector::ZeroVector;

	UPROPERTY()
	FVector EndPos = FVector::ZeroVector;

	UPROPERTY()
	FVector StartTangent = FVector::ZeroVector;

	UPROPERTY()
	FVector EndTangent = FVector::ZeroVector;
};

const double& USplineMeshComponent::GetAxisValueRef(const FVector3d& InVector, ESplineMeshAxis::Type InAxis)
{
	switch (InAxis)
	{
	case ESplineMeshAxis::X:
		return InVector.X;
	case ESplineMeshAxis::Y:
		return InVector.Y;
	case ESplineMeshAxis::Z:
		return InVector.Z;
	default:
		check(0);
		return InVector.Z;
	}
}

double& USplineMeshComponent::GetAxisValueRef(FVector3d& InVector, ESplineMeshAxis::Type InAxis)
{
	switch (InAxis)
	{
	case ESplineMeshAxis::X:
		return InVector.X;
	case ESplineMeshAxis::Y:
		return InVector.Y;
	case ESplineMeshAxis::Z:
		return InVector.Z;
	default:
		check(0);
		return InVector.Z;
	}
}


const float& USplineMeshComponent::GetAxisValueRef(const FVector3f& InVector, ESplineMeshAxis::Type InAxis)
{
	switch (InAxis)
	{
	case ESplineMeshAxis::X:
		return InVector.X;
	case ESplineMeshAxis::Y:
		return InVector.Y;
	case ESplineMeshAxis::Z:
		return InVector.Z;
	default:
		check(0);
		return InVector.Z;
	}
}

float& USplineMeshComponent::GetAxisValueRef(FVector3f& InVector, ESplineMeshAxis::Type InAxis)
{
	switch (InAxis)
	{
	case ESplineMeshAxis::X:
		return InVector.X;
	case ESplineMeshAxis::Y:
		return InVector.Y;
	case ESplineMeshAxis::Z:
		return InVector.Z;
	default:
		check(0);
		return InVector.Z;
	}
}

FVector USplineMeshComponent::GetAxisMask(ESplineMeshAxis::Type InAxis)
{
	switch (InAxis)
	{
	case ESplineMeshAxis::X:
		return FVector(0, 1, 1);
	case ESplineMeshAxis::Y:
		return FVector(1, 0, 1);
	case ESplineMeshAxis::Z:
		return FVector(1, 1, 0);
	default:
		check(0);
		return FVector::ZeroVector;
	}
}
