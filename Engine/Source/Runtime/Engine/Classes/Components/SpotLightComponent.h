// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SpotLightSceneProxyDesc.h"
#include "Components/PointLightComponent.h"
#include "SpotLightComponent.generated.h"

class FLightSceneProxy;

/**
 * A spot light component emits a directional cone shaped light (Eg a Torch).
 */
UCLASS(Blueprintable, ClassGroup=Lights, hidecategories=Object, editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class USpotLightComponent : public UPointLightComponent
{
	GENERATED_UCLASS_BODY()

	/** Degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta=(UIMin = "1.0", UIMax = "80.0", ShouldShowInViewport = true))
	float InnerConeAngle;

	/** Degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Light, meta=(UIMin = "1.0", UIMax = "80.0", ShouldShowInViewport = true))
	float OuterConeAngle;

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetInnerConeAngle(float NewInnerConeAngle);

	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetOuterConeAngle(float NewOuterConeAngle);

	// ULightComponent interface.
	ENGINE_API virtual FSphere GetBoundingSphere() const override;
	ENGINE_API virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override;
	ENGINE_API virtual ELightComponentType GetLightType() const override;
	ENGINE_API virtual FLightSceneProxy* CreateSceneProxy() const override;
	ENGINE_API virtual void PrecachePSOs() override;

	ENGINE_API virtual float ComputeLightBrightness() const override;
#if WITH_EDITOR
	ENGINE_API virtual void SetLightBrightness(float InBrightness) override;
	ENGINE_API virtual FBox GetStreamingBounds() const override;
#endif

	static FVector2f GetClampedConeAngles(float InInnerConeAngle, float InOuterConeAngle);
	ENGINE_API FVector2f GetClampedConeAngles() const;
	ENGINE_API float GetHalfConeAngle() const;
	ENGINE_API float GetCosHalfConeAngle() const;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	inline static FSpotLightSceneProxyDesc BuildSceneProxyDesc(const USpotLightComponent& InLightComponent)
	{
		FSpotLightSceneProxyDesc ProxyDesc;
		BuildSceneProxyDesc(InLightComponent, ProxyDesc);
		return ProxyDesc;
	}

protected:

	ENGINE_API static void BuildSceneProxyDesc(const USpotLightComponent& InLightComponent, FSpotLightSceneProxyDesc& OutProxyDesc);
};