// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIResources.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Engine/EngineTypes.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/MeshComponent.h"
#include "Components/ActorStaticMeshComponentInterface.h"
#include "PackedNormal.h"
#include "Templates/UniquePtr.h"
#include "Runtime/Launch/Resources/Version.h"
#include "UObject/RenderingObjectVersion.h"
#include "Runtime/Engine/Internal/MaterialCache/MaterialCacheVirtualTexture.h"
#include "SceneTypes.h"
#include "DrawDebugHelpers.h"
#include "StateStream/StaticMeshStateStreamHandle.h"
#include "StaticMeshComponent.generated.h"

class FColorVertexBuffer;
class FLightingBuildOptions;
class FMeshMapBuildData;
class FPrimitiveSceneProxy;
class FStaticMeshStaticLightingMesh;
class FVertexFactoryType;
class ULightComponent;
class UStaticMesh;
class UStaticMeshSocket;
class UStaticMeshComponent;
class UMaterialCacheVirtualTexture;
class UNavCollisionBase;
class UWorld;
struct FConvexVolume;
struct FEngineShowFlags;
struct FNavigableGeometryExport;
struct FNavigationRelevantData;
struct FStaticMeshComponentLODInfo;
struct FStaticLightingPrimitiveInfo;
struct FStaticMeshLODResources;

/** Whether FStaticMeshSceneProxy should to store data and enable codepaths needed for debug rendering */
#define STATICMESH_ENABLE_DEBUG_RENDERING			ENABLE_DRAW_DEBUG

namespace Nanite
{
	struct FResources;
	struct FMaterialAudit;
}

/** Cached vertex information at the time the mesh was painted. */
USTRUCT()
struct FPaintedVertex
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	FColor Color;

	UPROPERTY()
	FVector4 Normal;

	FPaintedVertex()
		: Position(ForceInit)
		, Color(ForceInit)
	{
	}

	inline friend FArchive& operator<<(FArchive& Ar, FPaintedVertex& PaintedVertex)
	{
		Ar << PaintedVertex.Position;

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IncreaseNormalPrecision)
		{
			FDeprecatedSerializedPackedNormal Temp;
			Ar << Temp;
			PaintedVertex.Normal = Temp;
		}
		else
		{
			Ar << PaintedVertex.Normal;
		}
		
		Ar << PaintedVertex.Color;
		return Ar;
	}
	
};

/**
 * StaticMeshComponent is used to create an instance of a UStaticMesh.
 * A static mesh is a piece of geometry that consists of a static set of polygons.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Content/Types/StaticMeshes/
 * @see UStaticMesh
 */
UCLASS(Blueprintable, ClassGroup=(Rendering, Common), hidecategories=(Object,Activation,"Components|Activation"), ShowCategories=(Mobility), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UStaticMeshComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()
	UE_DECLARE_COMPONENT_ACTOR_INTERFACE(StaticMeshComponent)

public:
	/** If 0, auto-select LOD level. if >0, force to (ForcedLodModel-1). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	int32 ForcedLodModel;

	/** 
	 * Specifies the smallest LOD that will be used for this component.  
	 * This is ignored if ForcedLodModel is enabled.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD, meta=(editcondition = "bOverrideMinLOD"))
	int32 MinLOD;

	/** Subdivision step size for static vertex lighting.				*/
	UPROPERTY()
	int32 SubDivisionStepSize;

	/** Wireframe color to use if bOverrideWireframeColor is true */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering, meta=(editcondition = "bOverrideWireframeColor"))
	FColor WireframeColorOverride;

private:
	/** The static mesh that this component uses to render */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=StaticMesh, ReplicatedUsing=OnRep_StaticMesh, meta=(AllowPrivateAccess="true"))
	TObjectPtr<UStaticMesh> StaticMesh;

	ENGINE_API void SetStaticMeshInternal(UStaticMesh* StaticMesh);

#if WITH_EDITOR
	/** Used to track down when StaticMesh has been modified for notification purpose */
	UStaticMesh* KnownStaticMesh = nullptr;
#endif
	ENGINE_API void NotifyIfStaticMeshChanged();

#if WITH_STATE_STREAM_ACTOR
	FStaticMeshHandle MeshHandle;
#endif

public:
	/** Helper function to get the FName of the private static mesh member */
	static const FName GetMemberNameChecked_StaticMesh() { return GET_MEMBER_NAME_CHECKED(UStaticMeshComponent, StaticMesh); }

	UFUNCTION()
	ENGINE_API void OnRep_StaticMesh(UStaticMesh *OldStaticMesh);

	/**
	* Distance at which to disable World Position Offset for an entire instance (0 = Never disable WPO).
	**/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	int32 WorldPositionOffsetDisableDistance = 0;

	/** Forces this component to always use Nanite for masked materials, even if FNaniteSettings::bAllowMaskedMaterials=false */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bForceNaniteForMasked : 1;

	/** Forces this component to use fallback mesh for rendering if Nanite is enabled on the mesh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bDisallowNanite : 1;

	/** Forces this component to use fallback mesh for rendering if Nanite is enabled on the mesh (run-time override) */
	UPROPERTY()
	uint8 bForceDisableNanite : 1;

	/** 
	 * Whether to evaluate World Position Offset. 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bEvaluateWorldPositionOffset : 1;

	/** 
	 * Whether world position offset turns on velocity writes.
	 * If the WPO isn't static then setting false may give incorrect motion vectors.
	 * But if we know that the WPO is static then setting false may save performance.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	uint8 bWorldPositionOffsetWritesVelocity : 1;

	/** 
	 * Whether to evaluate World Position Offset for ray tracing. 
	 * This is only used when running with r.RayTracing.Geometry.StaticMeshes.WPO=1 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = RayTracing)
	uint8 bEvaluateWorldPositionOffsetInRayTracing : 1;

protected:
	/** Initial value of bEvaluateWorldPositionOffset when BeginPlay() was called. Can be useful if we want to reset to initial state. */
	uint8 bInitialEvaluateWorldPositionOffset : 1;

	/** Whether mip callbacks have been registered and need to be removed on destroy */
	uint8 bMipLevelCallbackRegistered : 1;

	/** Whether mesh painting is supported. This is can be set by derived component types that don't support mesh painting. */
	uint8 bSupportMeshPainting : 1;

public:

#if WITH_EDITORONLY_DATA
	/** The section currently selected in the Editor. Used for highlighting */
	UPROPERTY(transient)
	int32 SelectedEditorSection;
	/** The material currently selected in the Editor. Used for highlighting */
	UPROPERTY(transient)
	int32 SelectedEditorMaterial;
	/** Index of the section to preview. If set to INDEX_NONE, all section will be rendered. Used for isolating in Static Mesh Tool **/
	UPROPERTY(transient)
	int32 SectionIndexPreview;
	/** Index of the material to preview. If set to INDEX_NONE, all section will be rendered. Used for isolating in Static Mesh Tool **/
	UPROPERTY(transient)
	int32 MaterialIndexPreview;

	/*
	 * The import version of the static mesh when it was assign this is update when:
	 * - The user assign a new staticmesh to the component
	 * - The component is serialize (IsSaving)
	 * - Default value is BeforeImportStaticMeshVersionWasAdded
	 *
	 * If when the component get load (PostLoad) the version of the attach staticmesh is newer
	 * then this value, we will remap the material override because the order of the materials list
	 * in the staticmesh can be changed. Hopefully there is a remap table save in the staticmesh.
	 */
	UPROPERTY()
	int32 StaticMeshImportVersion;
#endif

	/** If true, WireframeColorOverride will be used. If false, color is determined based on mobility and physics simulation settings */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Rendering, meta=(InlineEditConditionToggle))
	uint8 bOverrideWireframeColor:1;

	/** Whether to override the MinLOD setting of the static mesh asset with the MinLOD of this component. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=LOD)
	uint8 bOverrideMinLOD:1;

	/** If true, bForceNavigationObstacle flag will take priority over navigation data stored in StaticMesh */
	UPROPERTY(transient)
	uint8 bOverrideNavigationExport : 1;

	/** Allows overriding navigation export behavior per component: full collisions or dynamic obstacle */
	UPROPERTY(transient)
	uint8 bForceNavigationObstacle : 1;

	/** Deprecated. Use bEnableVertexColorMeshPainting instead. */
	UPROPERTY()
	uint8 bDisallowMeshPaintPerInstance_DEPRECATED : 1;

#if STATICMESH_ENABLE_DEBUG_RENDERING
	/** Draw mesh collision if used for complex collision */
	uint8 bDrawMeshCollisionIfComplex : 1;
	/** Draw mesh collision if used for simple collision */
	uint8 bDrawMeshCollisionIfSimple : 1;
#endif

	/**
	 *	Ignore this instance of this static mesh when calculating streaming information.
	 *	This can be useful when doing things like applying character textures to static geometry,
	 *	to avoid them using distance-based streaming.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=TextureStreaming)
	uint8 bIgnoreInstanceForTextureStreaming:1;

	/** Whether to override the lightmap resolution defined in the static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, meta=(InlineEditConditionToggle))
	uint8 bOverrideLightMapRes:1;

	/** 
	 * Whether to use the mesh distance field representation (when present) for shadowing indirect lighting (from lightmaps or skylight) on Movable components.
	 * This works like capsule shadows on skeletal meshes, except using the mesh distance field so no physics asset is required.
	 * The StaticMesh must have 'Generate Mesh Distance Field' enabled, or the project must have 'Generate Mesh Distance Fields' enabled for this feature to work.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(DisplayName = "Distance Field Indirect Shadow"))
	uint8 bCastDistanceFieldIndirectShadow:1;

	/** Whether to override the DistanceFieldSelfShadowBias setting of the static mesh asset with the DistanceFieldSelfShadowBias of this component. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	uint8 bOverrideDistanceFieldSelfShadowBias:1;

	/** Whether to use subdivisions or just the triangle's vertices.	*/
	UPROPERTY()
	uint8 bUseSubDivisions:1;

	/** Use the collision profile specified in the StaticMesh asset.*/
	UPROPERTY(EditAnywhere, Category = Collision)
	uint8 bUseDefaultCollision:1;

#if WITH_EDITORONLY_DATA
	/** The component has some custom painting on LODs or not. */
	UPROPERTY()
	uint8 bCustomOverrideVertexColorPerLOD:1;

	UPROPERTY(transient)
	uint8 bDisplayVertexColors:1;

	UPROPERTY(transient)
	uint8 bDisplayPhysicalMaterialMasks : 1;

	/** For Nanite enabled meshes, we'll only show the proxy mesh if this is true */
	UPROPERTY()
	uint8 bDisplayNaniteFallbackMesh:1;

	/**
	 * Transient flag used during registration to handle edge case with mesh compilation completion callback.
	 * We perform actions to register the mesh properly when it gets called async, but we end up doing those
	 * twice when it gets called while the registration is not completed.
	 */
	UPROPERTY(transient)
	uint8 bRegistering : 1;
#endif // WITH_EDITORONLY_DATA

	/** Enable dynamic sort mesh's triangles to remove ordering issue when rendered with a translucent material */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (UIMin = "0", UIMax = "1", DisplayName = "Sort Triangles"))
	uint8 bSortTriangles : 1;

	/**
	 * Controls whether the static mesh component's backface culling should be reversed
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	uint8 bReverseCulling : 1;

	/** 
	 * If false, vertex color mesh painting is disabled on this component. 
	 * This may be set to false by blueprint functions that override vertex colors in construction script.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Painting")
	uint8 bEnableVertexColorMeshPainting : 1;

	/** If false, texture color mesh painting is disabled on this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Painting")
	uint8 bEnableTextureColorMeshPainting : 1;

	/** If true, allows writing to the material cache. */
	UPROPERTY(Experimental, EditAnywhere, BlueprintReadWrite, Category = "Material Cache")
	uint8 bWritesToMaterialCache : 1;

	/** Whether to override the MeshPaintTextureCoordinateIndex set on the static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Painting", meta=(InlineEditConditionToggle))
	uint8 bOverrideMeshPaintTextureCoordinateIndex : 1;

	/** Whether to override the MeshPaintTextureCoordinateIndex set on the static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Painting", meta=(InlineEditConditionToggle))
	uint8 bOverrideMeshPaintTextureResolution : 1;

	/** The overriden coordinate index to use when texture color painting on this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Mesh Painting", meta=(UIMin = "0", UIMax = "3", editcondition = "bOverrideMeshPaintTextureCoordinateIndex"))
	int32 OverriddenMeshPaintTextureCoordinateIndex;

	/** The overriden resolution of texture color mesh paint textures on this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Mesh Painting", meta = (UIMin = "0", UIMax = "4096", ClampMax = "4096", editcondition = "bOverrideMeshPaintTextureResolution"))
	int32 OverriddenMeshPaintTextureResolution;

	/** Light map resolution to use on this component, used if bOverrideLightMapRes is true and there is a valid StaticMesh. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, meta=(ClampMax = 4096, editcondition="bOverrideLightMapRes") )
	int32 OverriddenLightMapRes;

#if WITH_EDITORONLY_DATA
	/** Texture containing texture color mesh painting for this mesh component. */
	UPROPERTY(VisibleAnywhere, Category = "Mesh Painting")
	TObjectPtr<UTexture> MeshPaintTexture;
#endif

	/** Cooked pointer to texture containing mesh painting for this mesh component. This will be taken from MeshPaintTexture but can be empty on some platforms if we choose to strip the data. */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UTexture> MeshPaintTextureCooked;

	/** Set this to override the locally stored mesh paint texture. */
	UPROPERTY(Transient)
	TObjectPtr<UTexture> MeshPaintTextureOverride;

	/** Number of tiles allocated to the material cache textures, may be modified by tag properties */
	UPROPERTY(EditAnywhere, Experimental, BlueprintReadWrite, Category = "Material Cache")
	FIntPoint MaterialCacheTileCount = FIntPoint(8, 8);
	
	/** Effective UV range in the material cache textures */
	UPROPERTY(EditAnywhere, Experimental, BlueprintReadWrite, Category = "Material Cache")
	FBox2f MaterialCacheUVRegion = FBox2f(FVector2f::Zero(), FVector2f::One());
	
	/** UV coordinate to unwrap with for the material cache textures */
	UPROPERTY(EditAnywhere, Experimental, BlueprintReadWrite, Category = "Material Cache")
	int32 MaterialCacheUVCoordinateIndex = 0;

	/** All allocated material cache textures */
	UPROPERTY(Experimental)
	TArray<TObjectPtr<UMaterialCacheVirtualTexture>> MaterialCacheTextures;

	/** 
	 * Controls how dark the dynamic indirect shadow can be.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(UIMin = "0", UIMax = "1", DisplayName = "Distance Field Indirect Shadow Min Visibility"))
	float DistanceFieldIndirectShadowMinVisibility;

	/** Useful for reducing self shadowing from distance field methods when using world position offset to animate the mesh's vertices. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting)
	float DistanceFieldSelfShadowBias;

	/**
	 * Allows adjusting the desired streaming distance of streaming textures that uses UV 0.
	 * 1.0 is the default, whereas a higher value makes the textures stream in sooner from far away.
	 * A lower value (0.0-1.0) makes the textures stream in later (you have to be closer).
	 * Value can be < 0 (from legcay content, or code changes)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=TextureStreaming, meta=(ClampMin = 0, ToolTip="Allows adjusting the desired resolution of streaming textures that uses UV 0.  1.0 is the default, whereas a higher value increases the streamed-in resolution."))
	float StreamingDistanceMultiplier;

	/** Used to forcefully disable pixel programmable rasterization of Nanite when the mesh is further than a given distance from the camera. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering)
	float NanitePixelProgrammableDistance;

	float GetNanitePixelProgrammableDistance() const
	{
		return NanitePixelProgrammableDistance;
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FGuid> IrrelevantLights_DEPRECATED;
#endif

	/** Static mesh LOD data.  Contains static lighting data along with instanced mesh vertex colors. */
	UPROPERTY(transient)
	TArray<struct FStaticMeshComponentLODInfo> LODData;

	/** The list of texture, bounds and scales. As computed in the texture streaming build process. */
	UPROPERTY(NonTransactional)
	TArray<FStreamingTextureBuildInfo> StreamingTextureData;

#if WITH_EDITORONLY_DATA
	/** Derived data key of the static mesh, used to determine if an update from the source static mesh is required. */
	UPROPERTY()
	FString StaticMeshDerivedDataKey;

	/** Material Bounds used for texture streaming. */
	UPROPERTY(NonTransactional)
	TArray<uint32> MaterialStreamingRelativeBoxes;
#endif

	/** The Lightmass settings for this object. */
	UPROPERTY(EditAnywhere, Category=Lighting)
	struct FLightmassPrimitiveSettings LightmassSettings;

	ENGINE_API virtual ~UStaticMeshComponent();

	/** Change the StaticMesh used by this instance. */
	UFUNCTION(BlueprintCallable, Category="Components|StaticMesh")
	ENGINE_API virtual bool SetStaticMesh(UStaticMesh* NewMesh);

	/** Get the StaticMesh used by this instance. */
	TObjectPtr<UStaticMesh> GetStaticMesh() const 
	{ 
#if WITH_EDITOR
		// This should never happen and is a last resort, we should have caught the property overwrite well before we reach this code
		if (KnownStaticMesh != StaticMesh)
		{
			OutdatedKnownStaticMeshDetected();
		}
#endif
		return StaticMesh; 
	}

	ENGINE_API virtual const Nanite::FResources* GetNaniteResources() const;

	/**
	 * Returns true if the component has valid Nanite render data.
	 */
	ENGINE_API virtual bool HasValidNaniteData() const;

	/** Determines if we use the nanite overrides from any materials */
	ENGINE_API virtual bool UseNaniteOverrideMaterials() const override;

	UFUNCTION(BlueprintCallable, Category="Rendering|LOD")
	ENGINE_API void SetForcedLodModel(int32 NewForcedLodModel);

	/** Sets the component's DistanceFieldSelfShadowBias.  bOverrideDistanceFieldSelfShadowBias must be enabled for this to have an effect. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetDistanceFieldSelfShadowBias(float NewValue);

	UFUNCTION(BlueprintCallable, Category=RayTracing)
	ENGINE_API void SetEvaluateWorldPositionOffsetInRayTracing(bool NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	ENGINE_API void SetEvaluateWorldPositionOffset(bool NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	ENGINE_API void SetWorldPositionOffsetDisableDistance(int32 NewValue);

	int32 GetWorldPositionOffsetDisableDistance() const
	{
		return WorldPositionOffsetDisableDistance;
	}

	bool EvaluateWorldPositionOffsetInRayTracing() const
	{
		return bEvaluateWorldPositionOffsetInRayTracing;
	}

	/** Get the initial value of bEvaluateWorldPositionOffset. This is the value when BeginPlay() was last called, or if UpdateInitialEvaluateWorldPositionOffset is called. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	bool GetInitialEvaluateWorldPositionOffset() { return bInitialEvaluateWorldPositionOffset; }

	/** This manually updates the initial value of bEvaluateWorldPositionOffset to be the current value.
	 *	This is useful if the default value of bEvaluateWorldPositionOffset is changed after constructing
	 *	the component. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|LOD")
	void UpdateInitialEvaluateWorldPositionOffset() { bInitialEvaluateWorldPositionOffset = bEvaluateWorldPositionOffset; }

	/** 
	 * Get Local bounds
	 */
	UFUNCTION(BlueprintCallable, Category="Components|StaticMesh")
	ENGINE_API void GetLocalBounds(FVector& Min, FVector& Max) const;

	/** 
	 * Set forced reverse culling
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	ENGINE_API void SetReverseCulling(bool ReverseCulling);

	/** Force disabling of Nanite rendering. When true, Will swap to the the fallback mesh instead. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	ENGINE_API void SetForceDisableNanite(bool bInForceDisableNanite);

	ENGINE_API virtual void SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps=true) override;

public:

	//~ Begin UObject Interface.
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	ENGINE_API virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;	
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostReinitProperties() override;
	ENGINE_API virtual void PostApplyToComponent() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void InitializeComponent() override;
	ENGINE_API virtual void UpdateBounds() override;
#endif // WITH_EDITOR
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITORONLY_DATA
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual bool AreNativePropertiesIdenticalTo( UObject* Other ) const override;
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.

	//~ Begin USceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual bool HasAnySockets() const override;
	ENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	ENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	ENGINE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	virtual bool ShouldCollideWhenPlacing() const override
	{
		// Current Method of collision does not work with non-capsule shapes, enable when it works with static meshes
		// return IsCollisionEnabled() && (StaticMesh != NULL);
		return false;
	}
#if WITH_EDITOR
	ENGINE_API virtual bool ShouldRenderSelected() const override;
#endif
	//~ End USceneComponent Interface

	//~ Begin UActorComponent Interface.
protected: 
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual bool RequiresGameThreadEndOfFrameRecreate() const override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
	ENGINE_API virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() override;
	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	ENGINE_API virtual bool ShouldCreateRenderState() const override;
	ENGINE_API virtual bool ShouldIncrementalPreRegister(UWorld* WorldContext) const override;

	
	/**
	 * Get all default LOD sections overlay materials used by a mesh.
	 *
	 * The implementer must add all LOD sections and set them to null if there is no value.
	 * We need all sections in case they are all active we wont add the global mesh MaterialOverlay in GetUseMaterials or GetMaterialRelevance.
	 */
	ENGINE_API virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const override;

public:

	ENGINE_API virtual bool AllowsAsyncPhysicsStateCreation() const override;
	ENGINE_API virtual bool AllowsAsyncPhysicsStateDestruction() const override;
	ENGINE_API virtual void PrecachePSOs() override;
	ENGINE_API virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
	ENGINE_API virtual UObject const* AdditionalStatObject() const override;
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool IsCompiling() const override;
#endif
	ENGINE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	ENGINE_API virtual bool IsHLODRelevant() const override;
#if WITH_EDITOR
	ENGINE_API virtual void ComputeHLODHash(class FHLODHashBuilder& HashBuilder) const override;
#endif
	//~ End UActorComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual int32 GetNumMaterials() const override;
#if WITH_EDITOR
	ENGINE_API virtual void GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options) override;
	ENGINE_API virtual void AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const override;
#endif
	ENGINE_API virtual float GetEmissiveBoost(int32 ElementIndex) const override;
	ENGINE_API virtual float GetDiffuseBoost(int32 ElementIndex) const override;
	virtual bool GetShadowIndirectOnly() const override
	{
		return LightmassSettings.bShadowIndirectOnly;
	}
	ENGINE_API virtual ELightMapInteractionType GetStaticLightingType() const override;
	ENGINE_API virtual bool IsPrecomputedLightingValid() const override;

	/** Get the scale comming form the component, when computing StreamingTexture data. Used to support instanced meshes. */
	ENGINE_API virtual float GetTextureStreamingTransformScale() const;
	/** Get material, UV density and bounds for a given material index. */
	ENGINE_API virtual bool GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const override;
	/** Build the data to compute accuracte StreaminTexture data. */
	ENGINE_API virtual bool BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData) override;
	ENGINE_API virtual UStreamableRenderAsset* GetStreamableNaniteAsset() const override;
	/** Get the StreaminTexture data. */
	ENGINE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	ENGINE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel) override;
#if WITH_EDITOR
	ENGINE_API virtual bool RemapActorTextureStreamingBuiltDataToLevel(const class UActorTextureStreamingBuildDataComponent* InActorTextureBuildData) override;
	ENGINE_API virtual uint32 ComputeHashTextureStreamingBuiltData() const override;
#endif 

	ENGINE_API virtual class UBodySetup* GetBodySetup() override;
	ENGINE_API virtual bool CanEditSimulatePhysics() override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual bool ShouldRecreateProxyOnUpdateTransform() const override;
	ENGINE_API virtual bool UsesOnlyUnlitMaterials() const override;
	ENGINE_API virtual bool GetLightMapResolution( int32& Width, int32& Height ) const override;
	ENGINE_API virtual int32 GetStaticLightMapResolution() const override;
	/** Returns true if the component is static AND has the right static mesh setup to support lightmaps. */
	ENGINE_API virtual bool HasValidSettingsForStaticLighting(bool bOverlookInvalidComponents) const override;

	ENGINE_API virtual void GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const override;
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const final;
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const final;
#if WITH_EDITOR
	ENGINE_API virtual UMaterialInterface* GetEditorMaterial(int32 MaterialIndex) const override;
#endif
	ENGINE_API virtual int32 GetMaterialIndex(FName MaterialSlotName) const override;
	ENGINE_API virtual UMaterialInterface* GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const override;
	ENGINE_API virtual TArray<FName> GetMaterialSlotNames() const override;
	ENGINE_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;

	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	ENGINE_API virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;
#if WITH_EDITOR
	ENGINE_API void OnMeshRebuild(bool bRenderDataChanged);
	ENGINE_API virtual void PreStaticMeshCompilation();
	ENGINE_API virtual void PostStaticMeshCompilation();
	ENGINE_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	ENGINE_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
	virtual float GetStreamingScale() const override { return GetComponentTransform().GetMaximumAxisScale(); }
	virtual bool SupportsWorldPositionOffsetVelocity() const override { return bWorldPositionOffsetWritesVelocity; }
	ENGINE_API virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const override;	
#if WITH_EDITOR
	ENGINE_API virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const override;	
#endif
//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& Callback, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn) override;
	ENGINE_API virtual void RegisterLODStreamingCallback(FLODStreamingCallback&& CallbackStreamingStart, FLODStreamingCallback&& CallbackStreamingDone, float TimeoutStartSecs, float TimeoutDoneSecs) override;
	ENGINE_API virtual bool PrestreamMeshLODs(float Seconds) override;
	ENGINE_API virtual UTexture* GetMeshPaintTexture() const override;
	ENGINE_API virtual void SetMeshPaintTexture(UTexture* Texture) override;
	ENGINE_API virtual void SetMeshPaintTextureOverride(UTexture* OverrideTexture) override;
	ENGINE_API virtual int32 GetMeshPaintTextureCoordinateIndex() const override;
	//~ End UMeshComponent Interface

	//~ Begin INavRelevantInterface Interface.
	ENGINE_API virtual bool IsNavigationRelevant() const override;
	ENGINE_API virtual FBox GetNavigationBounds() const override;
	ENGINE_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	//~ End INavRelevantInterface Interface.
	/**
	 *	Returns true if the component uses texture lightmaps
	 *
	 *	@param	InWidth		[in]	The width of the light/shadow map
	 *	@param	InHeight	[in]	The width of the light/shadow map
	 *
	 *	@return	bool				true if texture lightmaps are used, false if not
	 */
	ENGINE_API virtual bool UsesTextureLightmaps(int32 InWidth, int32 InHeight) const;

	/**
	 *	Returns true if the static mesh the component uses has valid lightmap texture coordinates
	 */
	ENGINE_API virtual bool HasLightmapTextureCoordinates() const;

	/**
	 *	Get the memory used for texture-based light and shadow maps of the given width and height
	 *
	 *	@param	InWidth						The desired width of the light/shadow map
	 *	@param	InHeight					The desired height of the light/shadow map
	 *	@param	OutLightMapMemoryUsage		The resulting lightmap memory used
	 *	@param	OutShadowMapMemoryUsage		The resulting shadowmap memory used
	 */
	ENGINE_API virtual void GetTextureLightAndShadowMapMemoryUsage(int32 InWidth, int32 InHeight, int32& OutLightMapMemoryUsage, int32& OutShadowMapMemoryUsage) const;

	/**
	 *	Returns the lightmap resolution used for this primitive instance in the case of it supporting texture light/ shadow maps.
	 *	This will return the value assuming the primitive will be automatically switched to use texture mapping.
	 *
	 *	@param Width	[out]	Width of light/shadow map
	 *	@param Height	[out]	Height of light/shadow map
	 */
	ENGINE_API virtual void GetEstimatedLightMapResolution(int32& Width, int32& Height) const;


	/**
	 * Returns the light and shadow map memory for this primite in its out variables.
	 *
	 * Shadow map memory usage is per light whereof lightmap data is independent of number of lights, assuming at least one.
	 *
	 *	@param [out]	TextureLightMapMemoryUsage		Estimated memory usage in bytes for light map texel data
	 *	@param [out]	TextureShadowMapMemoryUsage		Estimated memory usage in bytes for shadow map texel data
	 *	@param [out]	VertexLightMapMemoryUsage		Estimated memory usage in bytes for light map vertex data
	 *	@param [out]	VertexShadowMapMemoryUsage		Estimated memory usage in bytes for shadow map vertex data
	 *	@param [out]	StaticLightingResolution		The StaticLightingResolution used for Texture estimates
	 *	@param [out]	bIsUsingTextureMapping			Set to true if the mesh is using texture mapping currently; false if vertex
	 *	@param [out]	bHasLightmapTexCoords			Set to true if the mesh has the proper UV channels
	 *
	 *	@return			bool							true if the mesh has static lighting; false if not
	 */
	ENGINE_API virtual bool GetEstimatedLightAndShadowMapMemoryUsage(
		int32& TextureLightMapMemoryUsage, int32& TextureShadowMapMemoryUsage,
		int32& VertexLightMapMemoryUsage, int32& VertexShadowMapMemoryUsage,
		int32& StaticLightingResolution, bool& bIsUsingTextureMapping, bool& bHasLightmapTexCoords) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Determines whether any of the component's LODs require override vertex color fixups
	 *
	 * @return	true if any LODs require override vertex color fixups
	 */
	ENGINE_API bool RequiresOverrideVertexColorsFixup();

	/**
	 * Update the vertex override colors if necessary (i.e. vertices from source mesh have changed from override colors)
	 * @param bRebuildingStaticMesh	true if we are rebuilding the static mesh used by this component
	 * @returns true if any fixup was performed.
	 */
	ENGINE_API bool FixupOverrideColorsIfNecessary( bool bRebuildingStaticMesh = false );

	/** Save off the data painted on to this mesh per LOD if necessary */
	ENGINE_API void CachePaintedDataIfNecessary();

	/**
	 * Copies instance vertex colors from the SourceComponent into this component
	 * @param SourceComponent The component to copy vertex colors from
	 */
	ENGINE_API void CopyInstanceVertexColorsIfCompatible( const UStaticMeshComponent* SourceComponent );
#endif

	UE_DEPRECATED(5.6, "Use UPrimitiveComponent::IsFirstPersonRelevant() instead.")
	bool IsStaticMeshFirstPersonRelevant() const { return FirstPersonPrimitiveType != EFirstPersonPrimitiveType::None; }

	/**
	 * Removes instance vertex colors from the specified LOD
	 * @param LODToRemoveColorsFrom Index of the LOD to remove instance colors from
	 */
	ENGINE_API void RemoveInstanceVertexColorsFromLOD( int32 LODToRemoveColorsFrom );

#if WITH_EDITORONLY_DATA
	/**
	 * Removes instance vertex colors from all LODs
	 */
	ENGINE_API void RemoveInstanceVertexColors();
#endif

#if WITH_EDITORONLY_DATA
	/**
	*	Sets the value of the SectionIndexPreview flag and reattaches the component as necessary.
	*	@param	InSectionIndexPreview		New value of SectionIndexPreview.
	*/
	ENGINE_API void SetSectionPreview(int32 InSectionIndexPreview);

	/**
	*	Sets the value of the MaterialIndexPreview flag and reattaches the component as necessary.
	*	@param	InMaterialIndexPreview		New value of MaterialIndexPreview.
	*/
	ENGINE_API void SetMaterialPreview(int32 InMaterialIndexPreview);
#endif

	/** Sets the BodyInstance to use the mesh's body setup for external collision information*/
	ENGINE_API void UpdateCollisionFromStaticMesh();

	/** Whether or not the component supports default collision from its static mesh asset */
	ENGINE_API virtual bool SupportsDefaultCollision();

	UE_DEPRECATED(5.7, "Please use SupportsDitheredLODTransitions with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual bool SupportsDitheredLODTransitions(ERHIFeatureLevel::Type FeatureLevel);

	/** Whether we can support dithered LOD transitions (default behavior checks all materials). Used for HISMC LOD. */
	ENGINE_API virtual bool SupportsDitheredLODTransitions(EShaderPlatform ShaderPlatform);

	ENGINE_API UMaterialInterface* GetNaniteAuditMaterial(int32 MaterialIndex) const;

	/* Returns true if mesh vertex color painting is supported on this component. */
	ENGINE_API bool CanMeshPaintVertexColors() const;
	/* Returns true if mesh texture color painting is supported on this component. */
	ENGINE_API bool CanMeshPaintTextureColors() const;
	/** Returns the size to use when creating a mesh paint texture on this component. */
	ENGINE_API int32 GetMeshPaintTextureResolution() const;

	inline bool IsReverseCulling() const { return bReverseCulling; }
	inline bool IsDisallowNanite() const { return bDisallowNanite; }
	inline bool IsForceDisableNanite() const { return bForceDisableNanite; }
	inline bool IsForceNaniteForMasked() const { return bForceNaniteForMasked; }
	inline int32 GetForcedLodModel() const { return ForcedLodModel; }
	inline bool GetOverrideMinLOD() const { return bOverrideMinLOD; }
	inline int32 GetMinLOD() const { return MinLOD; }
	inline bool GetForceNaniteForMasked() const { return bForceNaniteForMasked; }

#if WITH_EDITORONLY_DATA
	bool IsDisplayNaniteFallbackMesh() const { return bDisplayNaniteFallbackMesh; }
#endif

	ENGINE_API void UpdateMaterialCacheTextures();
	ENGINE_API void ReleaseMaterialCacheTextures();
	ENGINE_API UMaterialCacheVirtualTexture* GetMaterialCacheVirtualTexture(FGuid TagGuid) const;

private:
	/** Initializes the resources used by the static mesh component. */
	ENGINE_API void InitResources();
		
#if WITH_EDITOR
	/** Update the vertex override colors */
	ENGINE_API void PrivateFixupOverrideColors();

	/** Called when the StaticMesh property gets overwritten without us knowing about it */
	ENGINE_API void OutdatedKnownStaticMeshDetected() const;

	/** Clears texture streaming data. */
	ENGINE_API void ClearStreamingTextureData();
#endif

	ENGINE_API bool UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const;

	ENGINE_API UMaterialInterface* GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit) const;

protected:
	/** Collect all the PSO precache data used by the static mesh component */
	ENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	/** Shared implementation for all StaticMesh derived components */
	using GetPSOVertexElementsFn = TFunctionRef<void(const FStaticMeshLODResources& LODRenderData, int32 LODIndex, bool bSupportsManualVertexFetch, FVertexDeclarationElementList& Elements)>;
	ENGINE_API void CollectPSOPrecacheDataImpl(const FVertexFactoryType* VFType, const FPSOPrecacheParams& BasePrecachePSOParams, GetPSOVertexElementsFn GetVertexElements, FMaterialInterfacePSOPrecacheParamsList& OutParams) const;
		
	/** Whether the component type supports static lighting. */
	virtual bool SupportsStaticLighting() const override
	{
		return true;
	}

	ENGINE_API bool ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials = nullptr) const;

	// Overload this in child implementations that wish to extend Static Mesh or Nanite scene proxy implementations
	ENGINE_API virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite);

	ENGINE_API bool ShouldExportAsObstacle(const UNavCollisionBase& InNavCollision) const;

public:

	ENGINE_API void ReleaseResources();

	/** Allocates an implementation of FStaticLightingMesh that will handle static lighting for this component */
	ENGINE_API virtual FStaticMeshStaticLightingMesh* AllocateStaticLightingMesh(int32 LODIndex, const TArray<ULightComponent*>& InRelevantLights);

	/** Add or remove elements to have the size in the specified range. Reconstructs elements if MaxSize<MinSize. Returns true if an element was added or removed. */
	ENGINE_API bool SetLODDataCount( const uint32 MinSize, const uint32 MaxSize );

	/**
	 *	Switches the static mesh component to use either Texture or Vertex static lighting.
	 *
	 *	@param	bTextureMapping		If true, set the component to use texture light mapping.
	 *								If false, set it to use vertex light mapping.
	 *	@param	ResolutionToUse		If != 0, set the resolution to the given value.
	 *
	 *	@return	bool				true if successfully set; false if not
	 *								If false, set it to use vertex light mapping.
	 */
	ENGINE_API virtual bool SetStaticLightingMapping(bool bTextureMapping, int32 ResolutionToUse);

	/**
	 * Returns the named socket on the static mesh component.
	 *
	 * @return UStaticMeshSocket of named socket on the static mesh component. None if not found.
	 */
	ENGINE_API UStaticMeshSocket const* GetSocketByName( FName InSocketName ) const;

	/** Returns the wireframe color to use for this component. */
	ENGINE_API FColor GetWireframeColor() const;

	/** Get this components index in its parents blueprint created components array (used for matching instance data) */
	ENGINE_API int32 GetBlueprintCreatedComponentIndex() const;

	ENGINE_API void ApplyComponentInstanceData(struct FStaticMeshComponentInstanceData* ComponentInstanceData);

	ENGINE_API virtual void PropagateLightingScenarioChange() override;

	ENGINE_API const FMeshMapBuildData* GetMeshMapBuildData(const FStaticMeshComponentLODInfo& LODInfo, bool bCheckForResourceCluster = true) const;

	void UpdateMapBuildDataId();
	void UpdateStaticLightingData();

	/** Called during scene proxy creation to get the Nanite resource data */
	DECLARE_DELEGATE_RetVal(const Nanite::FResources*, FOnGetNaniteResources);
	virtual FOnGetNaniteResources& OnGetNaniteResources() { return OnGetNaniteResourcesEvent; }
	virtual const FOnGetNaniteResources& OnGetNaniteResources() const { return OnGetNaniteResourcesEvent; }

#if WITH_EDITOR
	/** Called when the static mesh changes  */
	DECLARE_EVENT_OneParam(UStaticMeshComponent, FOnStaticMeshChanged, UStaticMeshComponent*);
	virtual FOnStaticMeshChanged& OnStaticMeshChanged() { return OnStaticMeshChangedEvent; }
#endif

private:
#if WITH_EDITOR
	FOnStaticMeshChanged OnStaticMeshChangedEvent;
#endif
	FOnGetNaniteResources OnGetNaniteResourcesEvent;

	friend class FStaticMeshComponentDetails;
	friend class FStaticMeshComponentRecreateRenderStateContext;
	friend class FStaticMeshComponentHelper;
	friend class FInstancedStaticMeshComponentHelper;
};

/** Vertex data stored per-LOD */
USTRUCT()
struct FStaticMeshVertexColorLODData
{
	GENERATED_BODY()

	/** copy of painted vertex data */
	UPROPERTY()
	TArray<FPaintedVertex> PaintedVertices;

	/** Copy of vertex buffer colors */
	UPROPERTY()
	TArray<FColor> VertexBufferColors;

	/** Index of the LOD that this data came from */
	UPROPERTY()
	uint32 LODIndex = 0;

	/** Check whether this contains valid data */
	bool IsValid() const
	{
		return PaintedVertices.Num() > 0 && VertexBufferColors.Num() > 0;
	}
};

USTRUCT()
struct FStaticMeshComponentInstanceData : public FPrimitiveComponentInstanceData
{
	GENERATED_BODY()
public:
	FStaticMeshComponentInstanceData() = default;
	FStaticMeshComponentInstanceData(const UStaticMeshComponent* SourceComponent);
	virtual ~FStaticMeshComponentInstanceData() = default;

	virtual bool ContainsData() const override;

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Add vertex color data for a specified LOD before RerunConstructionScripts is called */
	void AddVertexColorData(const struct FStaticMeshComponentLODInfo& LODInfo, uint32 LODIndex);

	/** Re-apply vertex color data after RerunConstructionScripts is called */
	bool ApplyVertexColorData(UStaticMeshComponent* StaticMeshComponent) const;

	/** Mesh being used by component */
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;

	/** Array of cached vertex colors for each LOD */
	UPROPERTY()
	TArray<FStaticMeshVertexColorLODData> VertexColorLODs;

	/** Used to store lightmap data during RerunConstructionScripts */
	UPROPERTY()
	TArray<FGuid> CachedStaticLighting;

	/** Texture streaming build data */
	UPROPERTY()
	TArray<FStreamingTextureBuildInfo> StreamingTextureData;

#if WITH_EDITORONLY_DATA
	/** Texture streaming editor data (for viewmodes) */
	UPROPERTY()
	TArray<uint32> MaterialStreamingRelativeBoxes;
#endif
};


