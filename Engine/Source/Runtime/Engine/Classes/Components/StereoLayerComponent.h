// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "IStereoLayers.h"

#if WITH_EDITORONLY_DATA
#include "Materials/Material.h"
#endif

#include "StereoLayerComponent.generated.h"

class UTexture;

/** Used by IStereoLayer */
UENUM(BlueprintType)
enum EStereoLayerType : int
{
	/** Location within the world */
	SLT_WorldLocked		UMETA(DisplayName = "World Locked"),

	/** Location within the HMD tracking space */
	SLT_TrackerLocked	UMETA(DisplayName = "Tracker Locked"),

	/** Location within the view space */
	SLT_FaceLocked		UMETA(DisplayName = "Face Locked"),

	SLT_MAX,
};

/** The shape to use for the stereo layer.  Note that some shapes might not be supported on all platforms! */
UENUM()
enum EStereoLayerShape : int
{
	/** Quad layer */
	SLSH_QuadLayer		UMETA(DisplayName = "Quad Layer"),

	/** Cylinder layer */
	SLSH_CylinderLayer	UMETA(DisplayName = "Cylinder Layer"),

	/** Cubemap layer */
	SLSH_CubemapLayer	UMETA(DisplayName = "Cubemap Layer"),

	/** Equirect layer */
	SLSH_EquirectLayer	UMETA(DisplayName = "Equirect Layer"),

	SLSH_MAX,
};

UCLASS()
class UEditorFlagCollector : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION()
	static TArray<FName> GetFlagNames();
};

UCLASS(EditInlineNew, Abstract, BlueprintType, CollapseCategories, MinimalAPI)
class UStereoLayerShape : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API virtual void ApplyShape(IStereoLayers::FLayerDesc& LayerDesc);
#if WITH_EDITOR
	ENGINE_API virtual void DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI);
#endif

protected:
	ENGINE_API void MarkStereoLayerDirty();
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UMaterial> DebugTextureMaterial = nullptr;
#endif
};

UCLASS(meta = (DisplayName = "Quad Layer"), MinimalAPI)
class UStereoLayerShapeQuad : public UStereoLayerShape
{
	GENERATED_BODY()
public:
	ENGINE_API virtual void ApplyShape(IStereoLayers::FLayerDesc& LayerDesc) override;
#if WITH_EDITOR
	ENGINE_API virtual void DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI) override;
#endif
};

UCLASS(meta = (DisplayName = "Cylinder Layer"), MinimalAPI)
class UStereoLayerShapeCylinder : public UStereoLayerShape
{
	GENERATED_BODY()

public:
	UStereoLayerShapeCylinder()
		: Radius(100)
		, OverlayArc(100)
		, Height(50)
	{}

	/** Radial size of the rendered stereo layer cylinder **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "Cylinder Properties")
	float Radius;

	/** Arc angle for the stereo layer cylinder **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "Cylinder Properties")
	float OverlayArc;

	/** Height of the stereo layer cylinder **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "Cylinder Properties")
	int Height;

	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer")
	ENGINE_API void SetRadius(float InRadius);
	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer")
	ENGINE_API void SetOverlayArc(float InOverlayArc);
	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer")
	ENGINE_API void SetHeight(int InHeight);

	ENGINE_API virtual void ApplyShape(IStereoLayers::FLayerDesc& LayerDesc) override;
#if WITH_EDITOR
	ENGINE_API virtual void DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI) override;
#endif
};

UCLASS(meta = (DisplayName = "Cubemap Layer"), MinimalAPI)
class UStereoLayerShapeCubemap : public UStereoLayerShape
{
	GENERATED_BODY()
public:
	ENGINE_API virtual void ApplyShape(IStereoLayers::FLayerDesc& LayerDesc) override;
};

/** Properties for equirect layers */
USTRUCT(BlueprintType)
struct FEquirectProps
{
	GENERATED_USTRUCT_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Left source texture UVRect, specifying portion of input texture corresponding to left eye. */
	FBox2D LeftUVRect;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Right source texture UVRect, specifying portion of input texture corresponding to right eye. */
	FBox2D RightUVRect;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Left eye's texture coordinate scale after mapping to 2D. */
	FVector2D LeftScale;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Right eye's texture coordinate scale after mapping to 2D. */
	FVector2D RightScale;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Left eye's texture coordinate bias after mapping to 2D. */
	FVector2D LeftBias;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Right eye's texture coordinate bias after mapping to 2D. */
	FVector2D RightBias;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category = "StereoLayer | Equirect Layer Properties")
	/** Sphere radius. As of UE 5.3, equirect layers are supported only by the Oculus OpenXR runtime and only with a radius of 0 (infinite sphere).*/
	float Radius;

public:

	FEquirectProps()
		: LeftUVRect(FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f)))
		, RightUVRect(FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f)))
		, LeftScale(FVector2D(1.0f, 1.0f))
		, RightScale(FVector2D(1.0f, 1.0f))
		, LeftBias(FVector2D(0.0f, 0.0f))
		, RightBias(FVector2D(0.0f, 0.0f))
		, Radius(0.0f)
	{}

	FEquirectProps(FBox2D InLeftUVRect, FBox2D InRightUVRect, FVector2D InLeftScale, FVector2D InRightScale, FVector2D InLeftBias, FVector2D InRightBias, float Radius)
		: LeftUVRect(InLeftUVRect)
		, RightUVRect(InRightUVRect)
		, LeftScale(InLeftScale)
		, RightScale(InRightScale)
		, LeftBias(InLeftBias)
		, RightBias(InRightBias)
		, Radius(Radius)
	{ }

	/**
	 * Compares two FEquirectProps for equality.
	 *
	 * @param Other The other FEquirectProps to compare with.
	 * @return true if the are equal, false otherwise.
	 */
	bool operator==(const FEquirectProps& Other) const;

	/**
	 * Compares FEquirectProps with an UStereoLayerShapeEquirect
	 *
	 * @param Other The UStereoLayerShapeEquirect to compare with.
	 * @return true if the are equal, false otherwise.
	 */
	bool operator==(const class UStereoLayerShapeEquirect& Other) const;
};

UCLASS(meta = (DisplayName = "Equirect Layer"), MinimalAPI)
class UStereoLayerShapeEquirect : public UStereoLayerShape
{
	GENERATED_BODY()

public:
	UStereoLayerShapeEquirect()
		: LeftUVRect(FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f)))
		, RightUVRect(FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f)))
		, LeftScale(FVector2D(1.0f, 1.0f))
		, RightScale(FVector2D(1.0f, 1.0f))
		, LeftBias(FVector2D(0.0f, 0.0f))
		, RightBias(FVector2D(0.0f, 0.0f))
		, Radius(0.0)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Left source texture UVRect, specifying portion of input texture corresponding to left eye. */
	FBox2D LeftUVRect;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Right source texture UVRect, specifying portion of input texture corresponding to right eye. */
	FBox2D RightUVRect;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Left eye's texture coordinate scale after mapping to 2D. */
	FVector2D LeftScale;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Right eye's texture coordinate scale after mapping to 2D. */
	FVector2D RightScale;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Left eye's texture coordinate bias after mapping to 2D. */
	FVector2D LeftBias;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Right eye's texture coordinate bias after mapping to 2D. */
	FVector2D RightBias;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Equirect Properties")
	/** Sphere radius. As of UE 5.3, equirect layers are supported only by the Oculus OpenXR runtime and only with a radius of 0 (infinite sphere).*/
	float Radius;

	/**
	 * Set Equirect layer properties: UVRect, Scale, and Bias
	 * @param	LeftScale: Scale for left eye
	 * @param	LeftBias: Bias for left eye
	 * @param	RightScale: Scale for right eye
	 * @param	RightBias: Bias for right eye
	 * @param	Radius: Sphere radius
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer")
	ENGINE_API void SetEquirectProps(FEquirectProps InScaleBiases);

	ENGINE_API virtual void ApplyShape(IStereoLayers::FLayerDesc& LayerDesc) override;
#if WITH_EDITOR
	ENGINE_API virtual void DrawShapeVisualization(const class FSceneView* View, class FPrimitiveDrawInterface* PDI) override;
#endif
};

/** 
 * A geometry layer within the stereo rendered viewport.
 */
UCLASS(ClassGroup="HeadMountedDisplay", hidecategories=(Object,LOD,Lighting,TextureStreaming), editinlinenew, meta=(DisplayName="Stereo Layer", BlueprintSpawnableComponent), MinimalAPI)
class UStereoLayerComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()
    friend class FStereoLayerComponentVisualizer;
    
public:

	//~ Begin UObject Interface
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface

	/** 
	 * Change the texture displayed on the stereo layer. 
	 *
	 * If stereoscopic layer textures are supported on the platform and LeftTexture is set, this property controls the texture for the right eye.
	 * @param	InTexture: new Texture2D
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void SetTexture(UTexture* InTexture);

	/** 
	 * Change the texture displayed on the stereo layer for left eye, if stereoscopic layer textures are supported on the platform.
	 * @param	InTexture: new Texture2D
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void SetLeftTexture(UTexture* InTexture);

	// @return the texture mapped to the stereo layer.
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	UTexture* GetTexture() const { return Texture; }

	// @return the texture mapped to the stereo layer for left eye, if stereoscopic layer textures are supported on the platform.
	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer")
	UTexture* GetLeftTexture() const { return LeftTexture; }

	/** 
	 * Change the quad size. This is the unscaled height and width, before component scale is applied.
	 * @param	InQuadSize: new quad size.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void SetQuadSize(FVector2D InQuadSize);

	// @return the height and width of the rendered quad
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	FVector2D GetQuadSize() const { return QuadSize; }

	/** 
	 * Change the UV coordinates mapped to the quad face
	 * @param	InUVRect: Min and Max UV coordinates
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void SetUVRect(FBox2D InUVRect);

	// @return the UV coordinates mapped to the quad face
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	FBox2D GetUVRect() const { return UVRect; }

	/**
	 * Set Equirect layer properties: UVRect, Scale, Bias and Radius.
	 * @param	LeftScale: Scale for left eye
	 * @param	LeftBias: Bias for left eye
	 * @param	RightScale: Scale for right eye
	 * @param	RightBias: Bias for right eye
	 * @param	Radius: sphere radius
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|Stereo Layer", meta = (DeprecatedFunction, DeprecationMessage = "Use UStereoLayerShapeEquirect::SetEquirectProps() instead."))
	ENGINE_API void SetEquirectProps(FEquirectProps InEquirectProps);

	/** 
	 * Change the layer's render priority, higher priorities render on top of lower priorities
	 * @param	InPriority: Priority value
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void SetPriority(int32 InPriority);

	// @return the render priority
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	int32 GetPriority() const { return Priority; }

	// Manually mark the stereo layer texture for updating
	UFUNCTION(BlueprintCallable, Category="Components|Stereo Layer")
	ENGINE_API void MarkTextureForUpdate();

	/** True if the stereo layer texture needs to update itself every frame(scene capture, video, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= "StereoLayer")
	uint32 bLiveTexture:1;

	/** True if the stereo layer needs to support depth intersections with the scene geometry, if available on the platform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StereoLayer")
	uint32 bSupportsDepth : 1;

	/** True if the texture should not use its own alpha channel (1.0 will be substituted) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= "StereoLayer")
	uint32 bNoAlphaChannel:1;

	ENGINE_API void MarkStereoLayerDirty();
	
	EStereoLayerType GetStereoLayerType() const { return StereoLayerType; }
	
protected:
	/** Texture displayed on the stereo layer (if stereoscopic textures are supported on the platform and more than one texture is provided, this will be the right eye) **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category= "StereoLayer")
	TObjectPtr<class UTexture> Texture;

	/** Texture displayed on the stereo layer for left eye, if stereoscopic textures are supported on the platform and by the layer shape **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Stereoscopic Properties")
	TObjectPtr<class UTexture> LeftTexture;

public:
	/** True if the quad should internally set it's Y value based on the set texture's dimensions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StereoLayer")
	uint32 bQuadPreserveTextureRatio : 1;

	/** Additional flags not included in IStereoLayers::ELayerFlags */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "StereoLayer", Meta=(GetOptions= "EditorFlagCollector.GetFlagNames"))
	TArray<FName> AdditionalFlags;

protected:
	/** Size of the rendered stereo layer quad **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category="StereoLayer")
	FVector2D QuadSize;

	/** UV coordinates mapped to the quad face **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category="StereoLayer")
	FBox2D UVRect;

	/** Specifies how and where the quad is rendered to the screen **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category="StereoLayer")
    TEnumAsByte<enum EStereoLayerType> StereoLayerType;

	/** Specifies which shape of layer it is.  Note that some shapes will be supported only on certain platforms! **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NoClear, Instanced, Category = "StereoLayer", DisplayName="Stereo Layer Shape")
	TObjectPtr<UStereoLayerShape> Shape;

	/** Render priority among all stereo layers. Layers with higher priorities will be rendered on top of layers with lower priorities. **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, export, Category= "StereoLayer")
	int32 Priority;

	/** IStereoLayer id, 0 is unassigned **/
	uint32 LayerId;
	
	// returns if the layer requires a texture to operate
	virtual bool LayerRequiresTexture() { return true; };

private:
	/** Dirty state determines whether the stereo layer needs updating **/
	bool bIsDirty;

	/** Texture needs to be marked for update **/
	bool bTextureNeedsUpdate;

	/** Last transform is cached to determine if the new frames transform has changed **/
	FTransform LastTransform;

	/** Last frames visiblity state **/
	bool bLastVisible;
};

