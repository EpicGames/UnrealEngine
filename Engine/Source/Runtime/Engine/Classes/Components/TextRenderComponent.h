// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Components/PrimitiveComponent.h"
#include "TextRenderComponent.generated.h"

class FPrimitiveSceneProxy;
class UFont;
class UMaterialInterface;

UENUM()
enum EHorizTextAligment : int
{
	/**  */
	EHTA_Left UMETA(DisplayName="Left"),
	/**  */
	EHTA_Center UMETA(DisplayName="Center"),
	/**  */
	EHTA_Right UMETA(DisplayName="Right"),
};

UENUM()
enum EVerticalTextAligment : int
{
	/**  */
	EVRTA_TextTop UMETA(DisplayName="Text Top"),
	/**  */
	EVRTA_TextCenter UMETA(DisplayName="Text Center"),
	/**  */
	EVRTA_TextBottom UMETA(DisplayName="Text Bottom"),
	/**  */
	EVRTA_QuadTop UMETA(DisplayName="Quad Top"),
};

/**
 * Renders text in the world with given font. Contains usual font related attributes such as Scale, Alignment, Color etc.
 */
UCLASS(Blueprintable, ClassGroup=Rendering, hidecategories=(Object,LOD,Physics,TextureStreaming,Activation,"Components|Activation",Collision), editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UTextRenderComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Text content, can be multi line using <br> as line separator */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text, meta=(MultiLine=true))
	FText Text;

	/** Text material */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	TObjectPtr<class UMaterialInterface> TextMaterial;
	
	/** Text font */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	TObjectPtr<class UFont> Font;

	/** Horizontal text alignment */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text, meta=(DisplayName = "Horizontal Alignment"))
	TEnumAsByte<enum EHorizTextAligment> HorizontalAlignment;

	/** Vertical text alignment */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text, meta=(DisplayName = "Vertical Alignment"))
	TEnumAsByte<enum EVerticalTextAligment> VerticalAlignment;

	/** Color of the text, can be accessed as vertex color */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	FColor TextRenderColor;

	/** Horizontal scale, default is 1.0 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	float XScale;

	/** Vertical scale, default is 1.0 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	float YScale;

	/** Vertical size of the fonts largest character in world units. Transform, XScale and YScale will affect final size. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Text)
	float WorldSize;
	
	/** The inverse of the Font's character height. */
	UPROPERTY()
	float InvDefaultSize;

	/** Horizontal adjustment per character, default is 0.0 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Text)
	float HorizSpacingAdjust;

	/** Vertical adjustment per character, default is 0.0 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Text)
	float VertSpacingAdjust;

	/** Allows text to draw unmodified when using debug visualization modes. **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Rendering)
	uint32 bAlwaysRenderAsText:1;

	// -----------------------------
	
	/** Change the text value and signal the primitives to be rebuilt */
	UFUNCTION()
	ENGINE_API void SetText(const FText& Value);

	/** Change the text value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender", meta=(DisplayName="Set Text", ScriptName="SetText"))
	ENGINE_API void K2_SetText(const FText& Value);

	/** Change the text material and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetTextMaterial(UMaterialInterface* Material);

	/** Change the font and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetFont(UFont* Value);

	/** Change the horizontal alignment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetHorizontalAlignment(EHorizTextAligment Value);

	/** Change the vertical alignment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetVerticalAlignment(EVerticalTextAligment Value);

	/** Change the text render color and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetTextRenderColor(FColor Value);

	/** Change the text X scale and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetXScale(float Value);

	/** Change the text Y scale and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetYScale(float Value);

	/** Change the text horizontal spacing adjustment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetHorizSpacingAdjust(float Value);

	/** Change the text vertical spacing adjustment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetVertSpacingAdjust(float Value);

	/** Change the world size of the text and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API void SetWorldSize(float Value);

	/** Get local size of text */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API FVector GetTextLocalSize() const;

	/** Get world space size of text */
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|TextRender")
	ENGINE_API FVector GetTextWorldSize() const;

	// -----------------------------

	//~ Begin UPrimitiveComponent Interface.
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual void GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false ) const override;
	ENGINE_API virtual int32 GetNumMaterials() const override;
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	ENGINE_API virtual bool ShouldRecreateProxyOnUpdateTransform() const override;
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial) override;
	ENGINE_API virtual FMatrix GetRenderMatrix() const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	// Note: this is required because GetRenderMatrix is overridden, and therefore the updated world-space bounds must be based on that (not the component transform)
	ENGINE_API virtual void UpdateBounds() override;
	//~ End USceneComponent Interface.

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual bool RequiresGameThreadEndOfFrameUpdates() const override;
	ENGINE_API virtual void PrecachePSOs() override;
	//~ End UActorComponent Interface.

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject interface.

	static ENGINE_API void InitializeMIDCache();
	static ENGINE_API void ShutdownMIDCache();
};





