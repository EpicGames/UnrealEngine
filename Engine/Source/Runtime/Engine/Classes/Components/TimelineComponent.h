// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "TimelineComponent.generated.h"

class UCurveFloat;
class UCurveLinearColor;
class UCurveVector;

/** Signature of function to handle a timeline 'event' */
DECLARE_DYNAMIC_DELEGATE( FOnTimelineEvent );
/** Signature of function to handle timeline float track */
DECLARE_DYNAMIC_DELEGATE_OneParam( FOnTimelineFloat, float, Output );
/** Signature of function to handle timeline vector track */
DECLARE_DYNAMIC_DELEGATE_OneParam( FOnTimelineVector, FVector, Output );
/** Signature of function to handle linear color track */
DECLARE_DYNAMIC_DELEGATE_OneParam( FOnTimelineLinearColor, FLinearColor, Output );

/** Static version of delegate to handle a timeline 'event' */
DECLARE_DELEGATE( FOnTimelineEventStatic );
/** Static version of timeline delegate for a float track */
DECLARE_DELEGATE_OneParam( FOnTimelineFloatStatic, float );
/** Static version of timeline delegate for a vector track */
DECLARE_DELEGATE_OneParam( FOnTimelineVectorStatic, FVector );
/** Static version of timeline delegate for a linear color track */
DECLARE_DELEGATE_OneParam( FOnTimelineLinearColorStatic, FLinearColor );

/** Whether or not the timeline should be finished after the specified length, or the last keyframe in the tracks */
UENUM()
enum ETimelineLengthMode : int
{
	TL_TimelineLength,
	TL_LastKeyFrame
};

/** Does timeline play or reverse ? */
UENUM(BlueprintType)
namespace ETimelineDirection
{
	enum Type : int
	{
		Forward,
		Backward,
	};
}

/** Struct that contains one entry for an 'event' during the timeline */
USTRUCT()
struct FTimelineEventEntry
{
	GENERATED_USTRUCT_BODY()

	/** Time at which event should fire */
	UPROPERTY()
	float Time;

	/** Function to execute when Time is reached */
	UPROPERTY()
	FOnTimelineEvent EventFunc;


	FTimelineEventEntry()
		: Time(0)
	{
	}

};

/** Struct that contains one entry for each vector interpolation performed by the timeline */
USTRUCT()
struct FTimelineVectorTrack
{
	GENERATED_USTRUCT_BODY()

	/** Vector curve to be evaluated */
	UPROPERTY()
	TObjectPtr<class UCurveVector> VectorCurve;

	/** Function that the output from ValueCurve will be passed to */
	UPROPERTY()
	FOnTimelineVector InterpFunc;

	/** Name of track, usually set in Timeline Editor. Used by SetInterpVectorCurve function. */
	UPROPERTY()
	FName TrackName;

	/** Name of property that we should update from this curve */
	UPROPERTY()
	FName VectorPropertyName;

	/** Cached vector struct property pointer */
	FStructProperty* VectorProperty;

	/** Static version of FOnTimelineVector, for use with non-UObjects  */
	FOnTimelineVectorStatic InterpFuncStatic;

	FTimelineVectorTrack()
		: VectorCurve(NULL)
		, VectorPropertyName(NAME_None)
		, VectorProperty(NULL)
	{
	}

};

/** Struct that contains one entry for each vector interpolation performed by the timeline */
USTRUCT()
struct FTimelineFloatTrack
{
	GENERATED_USTRUCT_BODY()

	/** Float curve to be evaluated */
	UPROPERTY()
	TObjectPtr<class UCurveFloat> FloatCurve;

	/** Function that the output from ValueCurve will be passed to */
	UPROPERTY()
	FOnTimelineFloat InterpFunc;

	/** Name of track, usually set in Timeline Editor. Used by SetInterpFloatCurve function. */
	UPROPERTY()
	FName TrackName;

	/** Name of property that we should update from this curve */
	UPROPERTY()
	FName FloatPropertyName;

	/** Cached float property pointer */
	FFloatProperty* FloatProperty;

	/** Static version of FOnTimelineFloat, for use with non-UObjects */
	FOnTimelineFloatStatic InterpFuncStatic;

	FTimelineFloatTrack()
		: FloatCurve(NULL)
		, FloatPropertyName(NAME_None)
		, FloatProperty(NULL)
	{
	}

};


/** Struct that contains one entry for each linear color interpolation performed by the timeline */
USTRUCT()
struct FTimelineLinearColorTrack
{
	GENERATED_USTRUCT_BODY()

	/** Float curve to be evaluated */
	UPROPERTY()
	TObjectPtr<class UCurveLinearColor> LinearColorCurve;

	/** Function that the output from ValueCurve will be passed to */
	UPROPERTY()
	FOnTimelineLinearColor InterpFunc;

	/** Name of track, usually set in Timeline Editor. Used by SetInterpLinearColorCurve function. */
	UPROPERTY()
	FName TrackName;

	/** Name of property that we should update from this curve */
	UPROPERTY()
	FName LinearColorPropertyName;

	/** Cached linear color struct property pointer */
	FStructProperty* LinearColorProperty;

	/** Static version of FOnTimelineFloat, for use with non-UObjects */
	FOnTimelineLinearColorStatic InterpFuncStatic;

	FTimelineLinearColorTrack()
		: LinearColorCurve(NULL)
		, LinearColorPropertyName(NAME_None)
		, LinearColorProperty(NULL)
	{
	}

};

USTRUCT()
struct FTimeline
{
	GENERATED_BODY()
	
private:
	/** Specified how the timeline determines its own length (e.g. specified length, last keyframe) */
	UPROPERTY(NotReplicated)
	TEnumAsByte<ETimelineLengthMode> LengthMode;

	/** Whether timeline should loop when it reaches the end, or stop */
	UPROPERTY()
	uint8 bLooping:1;

	/** If playback should move the current position backwards instead of forwards */
	UPROPERTY()
	uint8 bReversePlayback:1;
	
	UE_DEPRECATED(5.5, "Use PlayingStateTracker instead")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use PlayingStateTracker instead"))
	uint8 bPlaying_DEPRECATED:1;
	
	/**
	* If the first bit is set to 1 (PlayingStateTracker & 0x01 == 1), then we are playing
	* 
	* The rest of the bits in the uint8 are reserved for keeping track of the "dirty" state,
	* being incremented when our state is modified. This ensures that the value is replicated
	* if it changes multiple times in one frame, such as calling "Play From Start" in the resulting
	* "Finished" delegate.
	*
	* You should modify this value using the "ChangeMarkPlayingState" function.
	*/
	UPROPERTY()
	uint8 PlayingStateTracker;

	/** How long the timeline is, will stop or loop at the end */
 	UPROPERTY(NotReplicated)
 	float Length;

	/** How fast we should play through the timeline */
	UPROPERTY()
	float PlayRate;

	/** Current position in the timeline */
	UPROPERTY()
	float Position;

	/** Array of events that are fired at various times during the timeline */
	UPROPERTY(NotReplicated)
	TArray<struct FTimelineEventEntry> Events;

	/** Array of vector interpolations performed during the timeline */
	UPROPERTY(NotReplicated)
	TArray<struct FTimelineVectorTrack> InterpVectors;

	/** Array of float interpolations performed during the timeline */
	UPROPERTY(NotReplicated)
	TArray<struct FTimelineFloatTrack> InterpFloats;

	/** Array of linear color interpolations performed during the timeline */
	UPROPERTY(NotReplicated)
	TArray<struct FTimelineLinearColorTrack> InterpLinearColors;

	/** Called whenever this timeline is playing and updates - done after all delegates are executed and variables updated  */
	UPROPERTY(NotReplicated)
	FOnTimelineEvent TimelinePostUpdateFunc;

	/** Called whenever this timeline is finished. Is not called if 'stop' is used to terminate timeline early  */
	UPROPERTY(NotReplicated)
	FOnTimelineEvent TimelineFinishedFunc;

	/**	Optional. If set, Timeline will also set float/vector properties on this object using the PropertyName set in the tracks. */
	UPROPERTY(NotReplicated)
	TWeakObjectPtr<UObject> PropertySetObject;

	/**	Optional. If set, Timeline will also set ETimelineDirection property on PropertySetObject using the name. */
	UPROPERTY(NotReplicated)
	FName DirectionPropertyName;

	/** Called whenever this timeline is finished. Is not called if 'stop' is used to terminate timeline early  */
	FOnTimelineEventStatic TimelineFinishFuncStatic;

	/** Cached property pointer for setting timeline direction */
	FProperty* DirectionProperty;

public:
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	FTimeline()
	: LengthMode( TL_LastKeyFrame )
	, bLooping( false )
	, bReversePlayback( false )
	, bPlaying_DEPRECATED ( false )
	, PlayingStateTracker( 2 )
	, Length( 5.f )
	, PlayRate( 1.f )
	, Position( 0.0f )	
	, PropertySetObject(nullptr)
	, DirectionProperty(nullptr)
	{
	}

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	~FTimeline() = default;
	FTimeline(const FTimeline&) = default;
	FTimeline(FTimeline&&) = default;
	FTimeline& operator=(const FTimeline&) = default;
	FTimeline& operator=(FTimeline&&) = default;
	
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Helper function to get to the timeline direction enum */
	ENGINE_API static UEnum* GetTimelineDirectionEnum();

	/** Start playback of timeline */
	ENGINE_API void Play();

	/** Start playback of timeline from the start */
	ENGINE_API void PlayFromStart();

	/** Start playback of timeline in reverse */
	ENGINE_API void Reverse();

	/** Start playback of timeline in reverse from the end */
	ENGINE_API void ReverseFromEnd();

	/** Stop playback of timeline */
	ENGINE_API void Stop();

	/** Get whether this timeline is playing or not. */
	ENGINE_API bool IsPlaying() const;

	/** Get whether we are reversing or not */
	ENGINE_API bool IsReversing() const;

	/** Jump to a position in the timeline. If bFireEvents is true, event functions will fire, otherwise will not. */
	ENGINE_API void SetPlaybackPosition(float NewPosition, bool bFireEvents, bool bFireUpdate = true);

	/** Get the current playback position of the Timeline */
	ENGINE_API float GetPlaybackPosition() const;

	/** true means we whould loop, false means we should not. */
	ENGINE_API void SetLooping(bool bNewLooping);

	/** Get whether we are looping or not */
	ENGINE_API bool IsLooping() const;

	/** Sets the new play rate for this timeline */
	ENGINE_API void SetPlayRate(float NewRate);

	/** Get the current play rate for this timeline */
	ENGINE_API float GetPlayRate() const;

	/** Set the new playback position time to use */
	ENGINE_API void SetNewTime(float NewTime);

	/** Get length of the timeline */
	ENGINE_API float GetTimelineLength() const;

	/** Get length of the timeline divided by the play rate */
	ENGINE_API float GetScaledTimelineLength() const;

	/** Sets the timeline length mode */
	ENGINE_API void SetTimelineLengthMode(ETimelineLengthMode NewMode);

	/** Set the length of the timeline */
	ENGINE_API void SetTimelineLength(float NewLength);

	/** Update a certain float track's curve */
	ENGINE_API void SetFloatCurve(UCurveFloat* NewFloatCurve, FName FloatTrackName);

	/** Update a certain vector track's curve */
	ENGINE_API void SetVectorCurve(UCurveVector* NewVectorCurve, FName VectorTrackName);

	/** Update a certain linear color track's curve */
	ENGINE_API void SetLinearColorCurve(UCurveLinearColor* NewLinearColorCurve, FName LinearColorTrackName);

	/** Optionally provide an object to automatically update properties on */
	ENGINE_API void SetPropertySetObject(UObject* NewPropertySetObject);

	/** Set the delegate to call after each timeline tick */
	ENGINE_API void SetTimelinePostUpdateFunc(FOnTimelineEvent NewTimelinePostUpdateFunc);

	/** Set the delegate to call when timeline is finished */
	ENGINE_API void SetTimelineFinishedFunc(FOnTimelineEvent NewTimelineFinishedFunc);

	/** Set the static delegate to call when timeline is finished */
	ENGINE_API void SetTimelineFinishedFunc(FOnTimelineEventStatic NewTimelineFinishedFunc);

	/** Add a callback event to the timeline */
	ENGINE_API void AddEvent(float Time, FOnTimelineEvent EventFunc);

	/** Add a vector interpolation to the timeline */
	ENGINE_API void AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVector InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);

	/** Add a vector interpolation to the timeline. Use a non-serializeable delegate. */
	ENGINE_API void AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVectorStatic InterpFunc);

	/** Add a float interpolation to the timeline */
	ENGINE_API void AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloat InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);

	/** Add a float interpolation to the timeline. Use a non-serializeable delegate. */
	ENGINE_API void AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloatStatic InterpFunc );

	/** Add a linear color interpolation to the timeline */
	ENGINE_API void AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColor InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);

	/** Add a linear color interpolation to the timeline. Use a non-serializeable delegate. */
	ENGINE_API void AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColorStatic InterpFunc);


	/** Advance the timeline, if playing, firing delegates */
	ENGINE_API void TickTimeline(float DeltaTime);

	/** Set the delegate to call when timeline is finished */
	ENGINE_API void SetDirectionPropertyName(FName InDirectionPropertyName);

	/** Get all curves used by the Timeline */
	void GetAllCurves(TSet<class UCurveBase*>& InOutCurves) const;

	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);
	
private:
	/** Returns the time value of the last keyframe in any of the timeline's curves */
	float GetLastKeyframeTime() const;

	/**
	 * Mark the playing state as the given value
	 *
	 * @param bIsPlaying	True if the timeline should be playing, false if it should be stopped.
	 */
	void ChangeMarkPlayingState(const bool bIsPlaying);
};

// Need to customize serialization to safely update the old bPlaying flag to the newer play state tracer
template<> struct TStructOpsTypeTraits<FTimeline> : public TStructOpsTypeTraitsBase2<FTimeline>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

/** 
 * TimelineComponent holds a series of events, floats, vectors or colors with associated keyframes.
 * Events can be triggered at keyframes along the timeline. 
 * Floats, vectors, and colors are interpolated between keyframes along the timeline.
 */
UCLASS(MinimalAPI)
class UTimelineComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

private:
	
	/** The actual timeline structure */
	UPROPERTY(ReplicatedUsing=OnRep_Timeline)
	FTimeline	TheTimeline;
	
	/** True if global time dilation should be ignored by this timeline, false otherwise. */
	UPROPERTY()
	uint32 bIgnoreTimeDilation : 1;

public:

	/** Start playback of timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void Play();

	/** Start playback of timeline from the start */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void PlayFromStart();

	/** Start playback of timeline in reverse */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void Reverse();

	/** Start playback of timeline in reverse from the end */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void ReverseFromEnd();

	/** Stop playback of timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void Stop();

	/** Get whether this timeline is playing or not. */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API bool IsPlaying() const;

	/** Get whether we are reversing or not */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API bool IsReversing() const;

	/** Jump to a position in the timeline. 
	  * @param bFireEvents If true, event functions that are between current position and new playback position will fire. 
	  * @param bFireUpdate If true, the update output exec will fire after setting the new playback position.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline", meta=(AdvancedDisplay="bFireUpdate"))
	ENGINE_API void SetPlaybackPosition(float NewPosition, bool bFireEvents, bool bFireUpdate = true);

	/** Get the current playback position of the Timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API float GetPlaybackPosition() const;

	/** true means we would loop, false means we should not. */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void SetLooping(bool bNewLooping);

	/** Get whether we are looping or not */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API bool IsLooping() const;

	/** Sets the new play rate for this timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void SetPlayRate(float NewRate);

	/** Get the current play rate for this timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API float GetPlayRate() const;

	/** Set the new playback position time to use */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void SetNewTime(float NewTime);

	/** Get length of the timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API float GetTimelineLength() const;

	/** Get length of the timeline divided by the play rate */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API float GetScaledTimelineLength() const;

	/** Set length of the timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void SetTimelineLength(float NewLength);

	/** Sets the length mode of the timeline */
	UFUNCTION(BlueprintCallable, Category="Components|Timeline")
	ENGINE_API void SetTimelineLengthMode(ETimelineLengthMode NewLengthMode);

	/** Set whether to ignore time dilation. */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetIgnoreTimeDilation(bool bNewIgnoreTimeDilation);

	/** Get whether to ignore time dilation. */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API bool GetIgnoreTimeDilation() const;

	/** Update a certain float track's curve */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetFloatCurve(UCurveFloat* NewFloatCurve, FName FloatTrackName);

	/** Update a certain vector track's curve */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetVectorCurve(UCurveVector* NewVectorCurve, FName VectorTrackName);

	/** Update a certain linear color track's curve */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetLinearColorCurve(UCurveLinearColor* NewLinearColorCurve, FName LinearColorTrackName);

	UFUNCTION()
	void OnRep_Timeline(FTimeline& OldTimeline);

	//~ Begin ActorComponent Interface.
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	virtual void Activate(bool bReset=false) override;
	virtual void Deactivate() override;
	virtual bool IsReadyForOwnerToAutoDestroy() const override;
	//~ End ActorComponent Interface.

	//~ Begin UObject Interface. 
	virtual bool IsPostLoadThreadSafe() const override;
	//~ End UObject Interface

	/** Get the signature function for Timeline event functions */
	ENGINE_API static UFunction* GetTimelineEventSignature();
	/** Get the signature function for Timeline float functions */
	ENGINE_API static UFunction* GetTimelineFloatSignature();
	/** Get the signature function for Timeline vector functions */
	ENGINE_API static UFunction* GetTimelineVectorSignature();
	/** Get the signature function for Timeline linear color functions */
	ENGINE_API static UFunction* GetTimelineLinearColorSignature();

	/** Get the signature type for a specified function */
	static ETimelineSigType GetTimelineSignatureForFunction(const UFunction* InFunc);

	/** Add a callback event to the timeline */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void AddEvent(float Time, FOnTimelineEvent EventFunc);
	
	/** Add a vector interpolation to the timeline */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVector InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);
	ENGINE_API void AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVectorStatic InterpFunc);
	
	/** Add a float interpolation to the timeline */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloat InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);
	ENGINE_API void AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloatStatic InterpFunc);

	/** Add a linear color interpolation to the timeline */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColor InterpFunc, FName PropertyName = NAME_None, FName TrackName = NAME_None);
	ENGINE_API void AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColorStatic InterpFunc);

	/** Optionally provide an object to automatically update properties on */
	ENGINE_API void SetPropertySetObject(UObject* NewPropertySetObject);

	/** Set the delegate to call after each timeline tick */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetTimelinePostUpdateFunc(FOnTimelineEvent NewTimelinePostUpdateFunc);

	/** Set the delegate to call when timeline is finished */
	UFUNCTION(BlueprintCallable, Category = "Components|Timeline")
	ENGINE_API void SetTimelineFinishedFunc(FOnTimelineEvent NewTimelineFinishedFunc);
	/** Set the static delegate to call when timeline is finished */
	ENGINE_API void SetTimelineFinishedFunc(FOnTimelineEventStatic NewTimelineFinishedFunc);

	/** Set the delegate to call when timeline is finished */
	ENGINE_API void SetDirectionPropertyName(FName DirectionPropertyName);

	/** Get all curves used by the Timeline */
	ENGINE_API void GetAllCurves(TSet<class UCurveBase*>& InOutCurves) const;
};
