// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timeout.h"

class UWorld;

class FUnregisterComponentContext
{
public:
	FUnregisterComponentContext(UWorld* InWorld)
		: World(InWorld)
		, Timeout(UE::FTimeout::AlwaysExpired())
	{}

	FUnregisterComponentContext& SetIncrementalRemoveTimeout(const UE::FTimeout& InTimeout)
	{
		Timeout = InTimeout;
		return *this;
	}
	
	const UE::FTimeout& GetIncrementalRemoveTimeout() const
	{
		return Timeout;
	}

private:
	UWorld* World;
	UE::FTimeout Timeout;
};
