// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "VectorFieldComponent.generated.h"

class FPrimitiveSceneProxy;

/** 
 * A Component referencing a vector field.
 */
UCLASS(ClassGroup=Rendering, hidecategories=Object, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UVectorFieldComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** The vector field asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VectorFieldComponent)
	TObjectPtr<class UVectorField> VectorField;

	/** The intensity at which the vector field is applied. */
	UPROPERTY(interp, Category=VectorFieldComponent)
	float Intensity;

	/** How tightly particles follow the vector field. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VectorFieldComponent)
	float Tightness;

	/** If true, the vector field is only used for preview visualizations. */
	UPROPERTY(transient)
	uint32 bPreviewVectorField:1;

	/**
	 * Set the intensity of the vector field.
	 * @param NewIntensity - The new intensity of the vector field.
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|VectorField")
	virtual void SetIntensity(float NewIntensity);


public:

	/** The FX system with which this vector field is associated. */
	class FFXSystemInterface* FXSystem;
	/** The instance of this vector field registered with the FX system. */
	class FVectorFieldInstance* VectorFieldInstance;


	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UActorComponent Interface.
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	virtual void SendRenderTransform_Concurrent() override;
	//~ End UActorComponent Interface.

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.
};



