// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "EngineDefines.h"
#include "GameFramework/Info.h"
#include "Misc/Guid.h"
#include "RenderResource.h"

#include "VolumetricCloudComponent.generated.h"


class FVolumetricCloudSceneProxy;

UENUM()
enum class EVolumetricCloudTracingMaxDistanceMode : uint8
{
	DistanceFromCloudLayerEntryPoint = 0,
	DistanceFromPointOfView = 1,
};

/**
 * A component that represents a participating media material around a planet, e.g. clouds.
 */
UCLASS(ClassGroup = Rendering, collapsecategories, hidecategories = (Object, Mobility, Activation, "Components|Activation"), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UVolumetricCloudComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	~UVolumetricCloudComponent();

	/** The altitude at which the cloud layer starts. (kilometers above the ground) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer", meta = (UIMin = 0.0f, UIMax = 20.0f, SliderExponent = 2.0))
	float LayerBottomAltitude;

	/** The height of the the cloud layer. (kilometers above the layer bottom altitude) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer", meta = (UIMin = 0.1f, UIMax = 20.0f, ClampMin = 0.1, SliderExponent = 2.0))
	float LayerHeight;

	/** The maximum distance of the volumetric surface, i.e. cloud layer upper and lower bound, before which we will accept to start tracing. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer", meta = (UIMin = 100.0f, UIMax = 500.0f, ClampMin = 1.0f, SliderExponent = 2.0))
	float TracingStartMaxDistance;

	/** The distance from which the tracing will start. This is useful when the camera for instance is inside the layer of cloud. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer", meta = (UIMin = 0.0f, UIMax = 100.0f, ClampMin = 0.0f, SliderExponent = 3.0))
	float TracingStartDistanceFromCamera;

	/** Mode to select how the tracing max distance should be interpreted. DistanceFromPointOfView is useful to avoid the top of the cloud layer to be clipped when TracingMaxDistance is shorten for performance. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer")
	EVolumetricCloudTracingMaxDistanceMode TracingMaxDistanceMode;

	/** The maximum distance that will be traced inside the cloud layer. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Layer", meta = (UIMin = 1.0f, UIMax = 500.0f, ClampMin = 0.1f, SliderExponent = 2.0))
	float TracingMaxDistance;

	/** The planet radius used when there is not SkyAtmosphere component present in the scene. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Planet", meta = (UIMin = 100.0f, UIMax = 7000.0f, ClampMin = 0.1, ClampMax = 10000.0f))
	float PlanetRadius;

	/** 
	 * The ground albedo used to light the cloud from below with respect to the sun light and sky atmosphere. 
	 * This is only used by the cloud material when the 'Volumetric Advanced' node have GroundContribution enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Planet", meta = (HideAlphaChannel))
	FColor GroundAlbedo;

	/** The material describing the cloud volume. It must be a Volume domain material. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Material")
	TSoftObjectPtr<UMaterialInterface> Material;

	/** Whether to apply atmosphere transmittance per sample, instead of using the light global transmittance. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing")
	uint32 bUsePerSampleAtmosphericLightTransmittance : 1; 
	// bUsePerSampleAtmosphericLightTransmittance is there on the cloud component and not on the light because otherwise we would need optimization permutations of the cloud shader.
	// And this for the two atmospheric lights ON or OFF. Keeping it simple for now because this changes the look of the cloud, so it is an art/look decision.

	/** Occlude the sky light contribution at the bottom of the cloud layer. This is a fast approximation to sky lighting being occluded by cloud without having to trace rays or sample AO texture. Ignored if the cloud material explicitely sets the ambient occlusion value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f))
	float SkyLightCloudBottomOcclusion;

	/**
	 * Scale the tracing sample count in primary views. Quality level scalability CVARs affect the maximum range.
	 * The sample count resolution is still clamped according to scalability setting to 'r.VolumetricCloud.ViewRaySampleCountMax'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.05", UIMax = "8", ClampMin = "0.05", SliderExponent = 1.0))
	float ViewSampleCountScale;
	/**
	 * Scale the tracing sample count in reflection views. Quality level scalability CVARs affect the maximum range.
	 * The sample count resolution is still clamped according to scalability setting to 'r.VolumetricCloud.ReflectionRaySampleMaxCount'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.05", UIMax = "8", ClampMin = "0.05", SliderExponent = 1.0))
	float ReflectionViewSampleCountScaleValue;
	UPROPERTY()
	float ReflectionViewSampleCountScale_DEPRECATED;
	UPROPERTY()
	float ReflectionSampleCountScale_DEPRECATED;

	/**
	 * Scale the shadow tracing sample count in primary views, only used with Advanced Output ray marched shadows. Quality level scalability CVARs affect the maximum range.
	 * The sample count resolution is still clamped according to scalability setting to 'r.VolumetricCloud.Shadow.ViewRaySampleMaxCount'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.05", UIMax = "8", ClampMin = "0.05", SliderExponent = 1.0))
	float ShadowViewSampleCountScale;
	/**
	 * Scale the shadow tracing sample count in reflection views, only used with Advanced Output ray marched shadows. Quality level scalability CVARs affect the maximum range.
	 * The sample count resolution is still clamped according to scalability setting to 'r.VolumetricCloud.Shadow.ReflectionRaySampleMaxCount'.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.05", UIMax = "8", ClampMin = "0.05", SliderExponent = 1.0))
	float ShadowReflectionViewSampleCountScaleValue;
	UPROPERTY()
	float ShadowReflectionViewSampleCountScale_DEPRECATED;
	UPROPERTY()
	float ShadowReflectionSampleCountScale_DEPRECATED;

	/**
	 * The shadow tracing distance in kilometers, only used with Advanced Output ray marched shadows.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.1", UIMax = "50", ClampMin = "0.01", SliderExponent = 3.0))
	float ShadowTracingDistance;

	/**
	 * When the mean transmittance is below this threshold, we stop tracing. This is a good way to reduce the ray marched sample count, and thus to increase performance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cloud Tracing", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", SliderExponent = 5.0))
	float StopTracingTransmittanceThreshold;
	
	/** Specify the aerial perspective start distance on cloud for Rayleigh scattering only. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.0, UIMax = 100.0, ClampMin = 0.0, SliderExponent = 2.0))
	float AerialPespectiveRayleighScatteringStartDistance;
	/** Specify the distance over which the Rayleigh scattering will linearly ramp up to full effect. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.0, UIMax = 100.0, ClampMin = 0.0, SliderExponent = 2.0))
	float AerialPespectiveRayleighScatteringFadeDistance;
	/** Specify the aerial perspective start distance on cloud for Mie scattering only. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.0, UIMax = 100.0, ClampMin = 0.0, SliderExponent = 2.0))
	float AerialPespectiveMieScatteringStartDistance;
	/** Specify the distance over which the Rayleigh scattering will linearly ramp up to full effect. (kilometers) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Art Direction", meta = (UIMin = 0.0, UIMax = 100.0, ClampMin = 0.0, SliderExponent = 2.0))
	float AerialPespectiveMieScatteringFadeDistance;

	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, Interp)
	uint8 bHoldout : 1;

	/** If true, this component will be rendered in the main pass (basepass, transparency) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bRenderInMainPass : 1;

	/** If true, this component will be visible in real-time sky light reflection captures. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	uint8 bVisibleInRealTimeSkyCaptures : 1;


	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetLayerBottomAltitude(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetLayerHeight(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetTracingStartMaxDistance(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetTracingStartDistanceFromCamera(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetTracingMaxDistance(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetPlanetRadius(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetGroundAlbedo(FColor NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DisplayName = "Set Use Per Sample Atmospheric Light Transmittance"))
	ENGINE_API void SetbUsePerSampleAtmosphericLightTransmittance(bool NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetSkyLightCloudBottomOcclusion(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetViewSampleCountScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetReflectionViewSampleCountScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetShadowViewSampleCountScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetShadowReflectionViewSampleCountScale(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetShadowTracingDistance(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetStopTracingTransmittanceThreshold(float NewValue);
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetMaterial(UMaterialInterface* NewValue);

	UMaterialInterface* GetMaterial() const { return MaterialLifetimePtr.Get(); }

	// Deprecated functions but still valid because they forward data correctly.
	UE_DEPRECATED(5.0, "This function has been replaced by SetReflectionViewSampleCountScale.")
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DeprecatedFunction, DeprecationMessage = "This function has been replaced by SetReflectionViewSampleCountScale."))
	ENGINE_API void SetReflectionSampleCountScale(float NewValue);
	UE_DEPRECATED(5.0, "This function has been replaced by SetShadowReflectionViewSampleCountScale.")
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DeprecatedFunction, DeprecationMessage = "This function has been replaced by SetShadowReflectionViewSampleCountScale."))
	ENGINE_API void SetShadowReflectionSampleCountScale(float NewValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetHoldout(bool bNewHoldout);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetRenderInMainPass(bool bValue);

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetVisibleInRealTimeSkyCaptures(bool bValue);


protected:
	//~ Begin UActorComponent Interface.
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual void OnRegister() override;
	virtual void PrecachePSOs() override;
	//~ End UActorComponent Interface.

public:

	//~ Begin UObject Interface. 
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UActorComponent Interface.
#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif // WITH_EDITOR
	//~ End UActorComponent Interface.


	// Those values should never be changed wihtout data conversion, that in order to maintain performance in case default values are used.
	static constexpr float BaseViewRaySampleCount = 96.0f;
	static constexpr float BaseShadowRaySampleCount = 10.0f;
	// Those values are part of a data conversion and should never be changed. CVars and component sample count controls should be enough.
	static constexpr float OldToNewReflectionViewRaySampleCount = 10.0f / BaseViewRaySampleCount;
	static constexpr float OldToNewReflectionShadowRaySampleCount = 3.0f / BaseShadowRaySampleCount;

private:

	FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy;

	// The material is assigned to a regular pointer so that the garbage collector to not get ride of it.
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MaterialLifetimePtr;

};


/**
 * A placeable actor that represents a participating media material around a planet, e.g. clouds.
 * @see TODO address to the documentation.
 */
UCLASS(showcategories = (Movement, Rendering, Transformation, DataLayers, "Input|MouseInput", "Input|TouchInput"), ClassGroup = Fog, hidecategories = (Info, Object, Input), MinimalAPI)
class AVolumetricCloud : public AInfo
{
	GENERATED_UCLASS_BODY()

private:

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Atmosphere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UVolumetricCloudComponent> VolumetricCloudComponent;

#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

};
