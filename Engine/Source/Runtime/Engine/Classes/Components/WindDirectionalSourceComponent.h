// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "WindDirectionalSourceComponent.generated.h"

UENUM()
enum class EWindSourceType : uint8
{
	Directional,
	Point,
};

class FWindData;
class FWindSourceSceneProxy;

/** Component that provides a directional wind source. Only affects SpeedTree assets. */
UCLASS(collapsecategories, hidecategories=(Object, Mobility), editinlinenew, MinimalAPI)
class UWindDirectionalSourceComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Interp, Category=WindDirectionalSourceComponent)
	float Strength;

	UPROPERTY(Interp, Category=WindDirectionalSourceComponent)
	float Speed;

	UPROPERTY(Interp, BlueprintReadOnly, Category = WindDirectionalSourceComponent)
	float MinGustAmount;

	UPROPERTY(Interp, BlueprintReadOnly, Category = WindDirectionalSourceComponent)
	float MaxGustAmount;

	UPROPERTY(Interp, Category = WindDirectionalSourceComponent, meta = (editcondition = "bSimulatePhysics", ClampMin = "0.1", UIMin = "0.1"))
	float Radius;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = WindDirectionalSourceComponent)
	uint32 bPointWind : 1;

public:
	FWindSourceSceneProxy* SceneProxy;

	/**
	 * Because the actual data used to query wind is stored on the render thread in
	 * an instance of FWindSourceSceneProxy all of our properties are read only.
	 * The data can be manipulated with the following functions which will queue 
	 * a render thread update for this component
	 */

	/** Sets the strength of the generated wind */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetStrength(float InNewStrength);

	/** Sets the windspeed of the generated wind */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetSpeed(float InNewSpeed);

	/** Set minimum deviation for wind gusts */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetMinimumGustAmount(float InNewMinGust);

	/** Set maximum deviation for wind gusts */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetMaximumGustAmount(float InNewMaxGust);

	/** Set the effect radius for point wind */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetRadius(float InNewRadius);

	/** Set the type of wind generator to use */
	UFUNCTION(BlueprintCallable, Category = Wind)
	ENGINE_API void SetWindType(EWindSourceType InNewType);

	/** Calculate wind parameters from the data on this component, safe to call on game thread */
	ENGINE_API bool GetWindParameters(const FVector& EvaluatePosition, FWindData& OutData, float& Weight) const;

protected:
	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void Activate(bool bReset) override;
	ENGINE_API virtual void Deactivate() override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void SendRenderDynamicData_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	//~ End UActorComponent Interface.

public:
	/**
	 * Creates a proxy to represent the primitive to the scene manager in the rendering thread.
	 * @return The proxy object.
	 */
	ENGINE_API virtual FWindSourceSceneProxy* CreateSceneProxy() const;
};
