// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartitionStreamingSourceComponent.generated.h"

class FSceneView;
class FPrimitiveDrawInterface;

UCLASS(Meta = (BlueprintSpawnableComponent), HideCategories = (Tags, Sockets, ComponentTick, ComponentReplication, Activation, Cooking, Events, AssetUserData, Collision, Navigation), MinimalAPI)
class UWorldPartitionStreamingSourceComponent : public UActorComponent, public IWorldPartitionStreamingSourceProvider
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

	/** Enable the component */
	UFUNCTION(BlueprintCallable, Category = "Streaming")
	void EnableStreamingSource() { bStreamingSourceEnabled = true; }

	/** Disable the component */
	UFUNCTION(BlueprintCallable, Category = "Streaming")
	void DisableStreamingSource() { bStreamingSourceEnabled = false; }

	/** Returns true if the component is active. */
	UFUNCTION(BlueprintPure, Category = "Streaming")
	bool IsStreamingSourceEnabled() const { return bStreamingSourceEnabled; }

	// IWorldPartitionStreamingSourceProvider interface
	ENGINE_API virtual bool GetStreamingSource(FWorldPartitionStreamingSource& OutStreamingSource) const override;
	virtual const UObject* GetStreamingSourceOwner() const override { return this; }

	/** Returns true if streaming is completed for this streaming source component. */
	UFUNCTION(BlueprintCallable, Category = "Streaming")
	ENGINE_API bool IsStreamingCompleted() const;

	/** Displays a debug visualizer of the streaming source. Useful when using Shapes. */
	ENGINE_API void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const;

#if WITH_EDITORONLY_DATA
	/** Value used by debug visualizer when grid loading range is chosen. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	float DefaultVisualizerLoadingRange;
#endif

	/** When TargetGrids or TargetHLODLayers are specified, this indicates the behavior. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Streaming")
	EStreamingSourceTargetBehavior TargetBehavior;
			
	/** Optional target grids affected by streaming source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Streaming")
	TArray<FName> TargetGrids;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use TargetGrids instead."))
	FName TargetGrid_DEPRECATED;

	/** Color used for debugging. */
	UPROPERTY(EditAnywhere, Category = "Streaming")
	FColor DebugColor;

	/** Optional target HLODLayers affected by the streaming source. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use TargetGrids instead."))
	TArray<TObjectPtr<const UHLODLayer>> TargetHLODLayers_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use TargetHLODLayers instead."))
	TObjectPtr<const UHLODLayer> TargetHLODLayer_DEPRECATED;

	/** Optional aggregated shape list used to build a custom shape for the streaming source. When empty, fallbacks sphere shape with a radius equal to grid's loading range. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Streaming")
	TArray<FStreamingSourceShape> Shapes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Streaming")
	EStreamingSourcePriority Priority;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Streaming")
	EStreamingSourceTargetState TargetState;

private:
	/** Whether this component is enabled or not */
	UPROPERTY(EditAnywhere, Interp, Category = "Streaming")
	bool bStreamingSourceEnabled;
};
