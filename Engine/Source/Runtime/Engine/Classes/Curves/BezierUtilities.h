// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/Box.h"
#include "Math/UnrealMathUtility.h"

namespace UE
{
namespace CubicBezier
{

/** Evaluates cubic bezier value at specified time */
inline FVector Eval(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float t)
{
	// Interpolate using De Casteljau algorithm.
	const FVector P01 = FMath::Lerp(P0, P1, t);
	const FVector P12 = FMath::Lerp(P1, P2, t);
	const FVector P23 = FMath::Lerp(P2, P3, t);
	const FVector P012 = FMath::Lerp(P01, P12, t);
	const FVector P123 = FMath::Lerp(P12, P23, t);
	return FMath::Lerp(P012, P123, t);
}

/** Evaluates cubic bezier derivative at specified time */
inline FVector EvalDerivate(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float t)
{
	const FVector P01 = FMath::Lerp(P0, P1, t);
	const FVector P12 = FMath::Lerp(P1, P2, t);
	const FVector P23 = FMath::Lerp(P2, P3, t);
	return 3.0f * (FMath::Lerp(P12, P23, t) - FMath::Lerp(P01, P12, t));
}

/** Splits cubic bezier curve at specified time, returns new points.
  *	The new curves are (P0, Result[0], Result[1], Result[2]) and (P0, Result[2], Result[3], Result[4], P3) */
ENGINE_API void SplitAt(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float t, FVector OutResult[5]);

/** Returns bounding box of cubic bezier curve. */
ENGINE_API FBox CalcBounds(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3);

/** Returns closest point to cubic bezier segment. The curve is split to up to Steps linear segments for approximate calculation. */
ENGINE_API void ClosestPointApproximate(const FVector& FromPoint, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, FVector& OutClosestPoint, float& OutClosestT, const int Steps = 16);

/** Returns closest point between a linear segment and cubic bezier segment. The curve is split to up to Steps linear segments for approximate calculation. */
ENGINE_API void SegmentClosestPointApproximate(const FVector& SegStart, const FVector& SegEnd, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, FVector& OutClosestPoint, float& OutClosestT, const int Steps = 16);

/** Returns approximate arc length of cubic bezier. The length is calculated based on control point hull with one subdivision. */
ENGINE_API float ArcLengthApproximate(const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3);

/** Recursively tessellate a bezier curve until the segments are within Tolerance from the curve, or until max recursion level is reached.
  * Output will contain points describing linear segments of the flattened curve.
  * Note that the points are appended, and that the first point is omitted so that the function can be used to flatten a bezier shape. */
ENGINE_API void Tessellate(TArray<FVector>& Output, const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const float Tolerance, const int MaxLevel = 6);

} // namespace CubicBezier
} // namespace UE
