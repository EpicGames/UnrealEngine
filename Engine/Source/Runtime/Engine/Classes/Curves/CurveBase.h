// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Curves/CurveOwnerInterface.h"
#include "CurveBase.generated.h"

/**
 * Defines a curve of interpolated points to evaluate over a given range
 */
UCLASS(abstract, BlueprintType, MinimalAPI)
class UCurveBase
	: public UObject
	, public FCurveOwnerInterface
{
	GENERATED_UCLASS_BODY()

	/** Get the time range across all curves */
	UFUNCTION(BlueprintCallable, Category="Math|Curves")
	ENGINE_API void GetTimeRange(float& MinTime, float& MaxTime) const;
	ENGINE_API void GetTimeRange(double& MinTime, double& MaxTime) const;		// LWC_TODO: Native workaround for FVector2D components

	/** Get the value range across all curves */
	UFUNCTION(BlueprintCallable, Category="Math|Curves")
	ENGINE_API void GetValueRange(float& MinValue, float& MaxValue) const;
	ENGINE_API void GetValueRange(double& MinValue, double& MaxValue) const;		// LWC_TODO: Native workaround for FVector2D components
	/** 
	 *	Create curve from CSV style comma-separated string.
	 *
	 * @param InString The input string to parse.
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateCurveFromCSVString(const FString& InString);

	/** 
	 * Imports the curve data from a JSON string.
	 *
	 * @param InString The input string to parse.
	 * @param OutProblems Set of problems encountered while processing input
	 */
	ENGINE_API virtual void ImportFromJSONString(const FString& InString, TArray<FString>& OutProblems);
	
	/**
	 * Exports the curve as a JSON string
	 *
	 * @return The curve data as a JSON string
	 */
	ENGINE_API FString ExportAsJSONString() const;
	
	/** Reset all curve data */
	ENGINE_API void ResetCurve();

public:

	//~ FCurveOwnerInterface interface

	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override
	{
		TArray<FRichCurveEditInfoConst> Curves;
		return Curves;
	}

	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override
	{
		// @todo: remove this entire implementation, since the default behavior should be to not return anything,
		// once the GetCurves 5.6 deprecation gets removed
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArray<FRichCurveEditInfoConst> TempCurves = GetCurves();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const FRichCurveEditInfoConst& TempCurve : TempCurves)
		{
			Curves.Add(TempCurve);
		}
	}

	virtual TArray<FRichCurveEditInfo> GetCurves() override
	{
		TArray<FRichCurveEditInfo> Curves;
		return Curves;
	}

	ENGINE_API virtual void ModifyOwner() override;
	ENGINE_API virtual TArray<const UObject*> GetOwners() const override;
	ENGINE_API virtual void MakeTransactional() override;
	ENGINE_API virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	ENGINE_API virtual void SetOnCurveChangedIsInteractive(bool bInteractive) override;

	virtual bool RepointCurveOwner(const FPackageReloadedEvent& InPackageReloadedEvent, FCurveOwnerInterface*& OutNewCurveOwner) const override
	{
		return RepointCurveOwnerAsset(InPackageReloadedEvent, this, OutNewCurveOwner);
	}

	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override
	{
		return false;
	};

public:

	//~ UObject interface

#if WITH_EDITORONLY_DATA

	/** Override to ensure we write out the asset import data */
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

public:

	/** The filename imported to create this object. Relative to this object's package, BaseDir() or absolute */
	UPROPERTY()
	FString ImportPath_DEPRECATED;

	// Delegate called whenever the curve data is updated
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateCurve, UCurveBase* /*Curve*/, EPropertyChangeType::Type /*ChangeType*/);
	FOnUpdateCurve OnUpdateCurve;

	bool bOnCurveUpdateInteractive = false;
#endif
};
