// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CurveEdPresetCurve
// A preset curve data object
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CurveEdPresetCurve.generated.h"

UCLASS(deprecated)
class UDEPRECATED_CurveEdPresetCurve : public UObject
{
	GENERATED_UCLASS_BODY()
};

