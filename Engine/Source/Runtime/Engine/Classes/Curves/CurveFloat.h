// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RichCurve.h"
#include "Curves/CurveBase.h"
#include "CurveFloat.generated.h"

USTRUCT(BlueprintType)
struct FRuntimeFloatCurve
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FRichCurve EditorCurveData;

	UPROPERTY(EditAnywhere,Category=RuntimeFloatCurve)
	TObjectPtr<class UCurveFloat> ExternalCurve;

	ENGINE_API FRuntimeFloatCurve();

	/** Get the current curve struct */
	ENGINE_API FRichCurve* GetRichCurve();
	ENGINE_API const FRichCurve* GetRichCurveConst() const;
};

UCLASS(BlueprintType, MinimalAPI)
class UCurveFloat : public UCurveBase
{
	GENERATED_UCLASS_BODY()

	/** Keyframe data */
	UPROPERTY()
	FRichCurve FloatCurve;

	/** Flag to represent event curve */
	UPROPERTY()
	bool	bIsEventCurve;

	/** Evaluate this float curve at the specified time */
	UFUNCTION(BlueprintCallable, Category="Math|Curves")
	ENGINE_API float GetFloatValue(float InTime) const;

	// Begin FCurveOwnerInterface
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	ENGINE_API virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	ENGINE_API virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	ENGINE_API virtual TArray<FRichCurveEditInfo> GetCurves() override;
	ENGINE_API virtual bool IsValidCurve( FRichCurveEditInfo CurveInfo ) override;

	/** Determine if Curve is the same */
	ENGINE_API bool operator == (const UCurveFloat& Curve) const;
};

