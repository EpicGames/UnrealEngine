// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RichCurve.h"
#include "Curves/CurveBase.h"
#include "CurveLinearColor.generated.h"

class UCurveLinearColor;

USTRUCT(BlueprintType)
struct FRuntimeCurveLinearColor
{
	GENERATED_USTRUCT_BODY()
public:
	UPROPERTY()
	FRichCurve ColorCurves[4];

	UPROPERTY(EditAnywhere, Category = RuntimeFloatCurve)
	TObjectPtr<class UCurveLinearColor> ExternalCurve = nullptr;

	ENGINE_API FLinearColor GetLinearColorValue(float InTime) const;
};

UCLASS(BlueprintType, collapsecategories, hidecategories = (FilePath), MinimalAPI)
class UCurveLinearColor : public UCurveBase
{
	GENERATED_UCLASS_BODY()

	/** Keyframe data, one curve for red, green, blue, and alpha */
	UPROPERTY()
	FRichCurve FloatCurves[4];

	// Begin FCurveOwnerInterface
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	ENGINE_API virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	ENGINE_API virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	ENGINE_API virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual bool IsLinearColorCurve() const override { return true; }

	// GetLinearColorValue allows RGB > 1 for HDR
	//	 if the input curves are LDR (<= 1) then the output is clamped to stay LDR, even if Adjustments would have changed it
	UFUNCTION(BlueprintCallable, Category="Math|Curves")
	ENGINE_API virtual FLinearColor GetLinearColorValue(float InTime) const override;

	// GetClampedLinearColorValue always clamps RGB in [0,1] , eg. returns LDR colors
	UFUNCTION(BlueprintCallable, Category = "Math|Curves")
	ENGINE_API virtual FLinearColor GetClampedLinearColorValue(float InTime) const override;

	// GetUnadjustedLinearColorValue returns the raw curve values without color adjustments
	//	also does NOT clamp in [0,1] , beware how the RGBA is used, you may want clamping, at least >= 0
	UFUNCTION(BlueprintCallable, Category = "Math|Curves")
	ENGINE_API FLinearColor GetUnadjustedLinearColorValue(float InTime) const;

	bool HasAnyAlphaKeys() const override { return FloatCurves[3].GetNumKeys() > 0; }

	ENGINE_API virtual bool IsValidCurve( FRichCurveEditInfo CurveInfo ) override;
	// End FCurveOwnerInterface

	/** Determine if Curve is the same */
	ENGINE_API bool operator == (const UCurveLinearColor& Curve) const;

public:
#if WITH_EDITOR

	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	ENGINE_API void DrawThumbnail(class FCanvas* Canvas, FVector2D StartXY, FVector2D SizeXY);

	ENGINE_API void PushToSourceData(TArrayView<FFloat16Color> &SrcData, int32 Start, int32 Width);

	ENGINE_API void PushUnadjustedToSourceData(TArrayView<FFloat16Color>& SrcData, int32 Start, int32 Width);

	ENGINE_API virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
#endif
	ENGINE_API virtual void PostLoad() override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

public:
	// Properties for adjusting the color of the gradient
	UPROPERTY(EditAnywhere, Category="Color", meta = (ClampMin = "0.0", ClampMax = "359.0"))
	float AdjustHue;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustSaturation;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustBrightness;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustBrightnessCurve;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustVibrance;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustMinAlpha;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AdjustMaxAlpha;

private:

	// if bClampOver1Always is true, clamping in [0,1] is always done
	// if bClampOver1Always is false, clamp in [0,1] is still done if the source RGB is in [0,1] , but NOT if source RGB is > 1
	FLinearColor GetAdjustedColorValue(float InTime, bool bClampOver1Always) const;
};

