// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "Engine/Texture2D.h"
#include "CurveLinearColorAtlas.generated.h"


static FName NAME_GradientTexture = FName("GradientTexture");
static FName NAME_GradientBias = FName("GradientBias");
static FName NAME_GradientScale = FName("GradientScale");
static FName NAME_GradientCount = FName("GradientCount");

class UCurveLinearColor;
class UCurveBase;

USTRUCT()
struct FCurveAtlasColorAdjustments
{
	GENERATED_USTRUCT_BODY()

	FCurveAtlasColorAdjustments()
		: bChromaKeyTexture(false)
		, AdjustBrightness(1.0f)
		, AdjustBrightnessCurve(1.0f)
		, AdjustVibrance(0.0f)
		, AdjustSaturation(1.0f)
		, AdjustRGBCurve(1.0f)
		, AdjustHue(0.0f)
		, AdjustMinAlpha(0.0f)
		, AdjustMaxAlpha(1.0f)
	{}

	UPROPERTY()
	uint32 bChromaKeyTexture : 1;

	UPROPERTY()
	float AdjustBrightness;

	UPROPERTY()
	float AdjustBrightnessCurve;

	UPROPERTY()
	float AdjustVibrance;

	UPROPERTY()
	float AdjustSaturation;

	UPROPERTY()
	float AdjustRGBCurve;

	UPROPERTY()
	float AdjustHue;

	UPROPERTY()
	float AdjustMinAlpha;

	UPROPERTY()
	float AdjustMaxAlpha;
};

/**
*  Manages gradient LUT textures for registered actors and assigns them to the corresponding materials on the actor
*/
UCLASS(MinimalAPI)
class UCurveLinearColorAtlas : public UTexture2D
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR

	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// Immediately render a new material to the specified slot index(SlotIndex must be within this section's range)
	ENGINE_API void OnCurveUpdated(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);

	// Re-render all texture groups
	ENGINE_API void UpdateTextures();

#endif

#if WITH_EDITORONLY_DATA

	ENGINE_API virtual FGuid BuildLightingGuid() override;

#endif

	ENGINE_API virtual void PostLoad() override;

	ENGINE_API bool GetCurveIndex(UCurveLinearColor* InCurve, int32& Index);

	UFUNCTION(BlueprintCallable, Category = "Math|Curves")
	ENGINE_API bool GetCurvePosition(UCurveLinearColor* InCurve, float& Position);

	UPROPERTY(EditAnywhere, Category = "Curves", DisplayName = "Texture Width")
	uint32	TextureSize;						// Width of the lookup textures

	UPROPERTY(EditAnywhere, Category = "Curves")
	TArray<TObjectPtr<UCurveLinearColor>> GradientCurves;

#if WITH_EDITORONLY_DATA
	/** Disable all color adjustments to preserve negative values in curves. Color adjustments clamp to 0 when enabled. */
	UPROPERTY(EditAnywhere, Category = "Curves")
	uint32 bDisableAllAdjustments : 1;

	UPROPERTY(Transient)
	uint32 bHasCachedColorAdjustments : 1;

	UPROPERTY(Transient)
	FCurveAtlasColorAdjustments CachedColorAdjustments;
#endif

protected:

#if WITH_EDITOR
	ENGINE_API void CacheAndResetColorAdjustments();
	ENGINE_API void RestoreCachedColorAdjustments();
#endif

};
