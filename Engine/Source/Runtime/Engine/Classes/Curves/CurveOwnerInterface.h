// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/RichCurve.h"
#include "UObject/PackageReload.h"
#include "Containers/AdderRef.h"

/**
 * Interface you implement if you want the CurveEditor to be able to edit curves on you.
 */
class FCurveOwnerInterface
{
public:

	virtual ~FCurveOwnerInterface() = default;

	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const = 0;

	/** Returns set of curves to edit. Must not release the curves while being edited. */
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const = 0;

	/** Returns set of curves to query. Must not release the curves while being edited. */
	virtual TArray<FRichCurveEditInfo> GetCurves() = 0;

	/** Called to modify the owner of the curve */
	virtual void ModifyOwner() = 0;

	/** Called to modify the owner of the curve during interaction/while being edited. */
	virtual void ModifyOwnerChange() { ModifyOwner(); }

	/** Returns the owner(s) of the curve */
	virtual TArray<const UObject*> GetOwners() const = 0;

	/** Called to make curve owner transactional */
	virtual void MakeTransactional() = 0;

	/** Called when any of the curves have been changed */
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) = 0;

	/** Tell the curve owner that calls to OnCurveChanged are from an interactive drag **/
	virtual void SetOnCurveChangedIsInteractive(bool bInteractive) { };

	/** Whether the curve represents a linear color */
	virtual bool IsLinearColorCurve() const
	{
		return false;
	}

	/** Whether the curves returned by GetCurves are rich or simple curves */
	virtual bool HasRichCurves() const
	{
		return true;
	}

	/** Evaluate this color curve at the specified time */
	virtual FLinearColor GetLinearColorValue(float InTime) const
	{
		return FLinearColor::Black;
	}

	/** Evaluate this color curve at the specified time */
	virtual FLinearColor GetClampedLinearColorValue(float InTime) const
	{
		return FLinearColor::Black;
	}

	/** @return True if the curve has any alpha keys */
	virtual bool HasAnyAlphaKeys() const
	{
		return false;
	}

	/** Validates that a previously retrieved curve is still valid for editing. */
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) = 0;

	/** @return Color for this curve */
	ENGINE_API virtual FLinearColor GetCurveColor(FRichCurveEditInfo CurveInfo) const;

	/** Called during package reload to repoint a curve interface asset */
	virtual bool RepointCurveOwner(const FPackageReloadedEvent& InPackageReloadedEvent, FCurveOwnerInterface*& OutNewCurveOwner) const
	{
		return false;
	}
	
protected:
	/** Default implementation of RepointCurveOwner that can be used with UObject based types that inherit FCurveOwnerInterface */
	template <typename AssetType>
	static bool RepointCurveOwnerAsset(const FPackageReloadedEvent& InPackageReloadedEvent, const AssetType* InCurveOwnerAsset, FCurveOwnerInterface*& OutNewCurveOwner)
	{
		AssetType* NewCurveOwnerAsset = nullptr;
		const bool bDidRepoint = InPackageReloadedEvent.GetRepointedObject(InCurveOwnerAsset, NewCurveOwnerAsset);
		OutNewCurveOwner = NewCurveOwnerAsset;
		return bDidRepoint;
	}
};
