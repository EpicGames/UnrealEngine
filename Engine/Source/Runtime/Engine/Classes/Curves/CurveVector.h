// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RichCurve.h"
#include "Curves/CurveBase.h"
#include "CurveVector.generated.h"

USTRUCT(BlueprintType)
struct FRuntimeVectorCurve
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FRichCurve VectorCurves[3];

	UPROPERTY(EditAnywhere, Category = RuntimeFloatCurve)
	TObjectPtr<class UCurveVector> ExternalCurve = nullptr;

	ENGINE_API FVector GetValue(float InTime) const;
	
	/** Get the current curve struct */
    ENGINE_API FRichCurve* GetRichCurve(int32 Index);
	ENGINE_API const FRichCurve* GetRichCurveConst(int32 Index) const;
};

UCLASS(BlueprintType, MinimalAPI)
class UCurveVector : public UCurveBase
{
	GENERATED_UCLASS_BODY()

	/** Keyframe data, one curve for X, Y and Z */
	UPROPERTY()
	FRichCurve FloatCurves[3];

	/** Evaluate this float curve at the specified time */
	UFUNCTION(BlueprintCallable, Category="Math|Curves")
	ENGINE_API FVector GetVectorValue(float InTime) const;

	// Begin FCurveOwnerInterface
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	ENGINE_API virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	ENGINE_API virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;

	/** Determine if Curve is the same */
	ENGINE_API bool operator == (const UCurveVector& Curve) const;

	virtual bool IsValidCurve( FRichCurveEditInfo CurveInfo ) override;
};
