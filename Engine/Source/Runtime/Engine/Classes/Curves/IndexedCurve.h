// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "IndexedCurve.generated.h"


/**
 * A curve base class which enables key handles to index lookups.
 *
 * @todo sequencer: Some heavy refactoring can be done here. Much more stuff can go in this base class.
 */
USTRUCT()
struct FIndexedCurve
{
	GENERATED_USTRUCT_BODY()

public:

	/** Default constructor. */
	FIndexedCurve() { }

	virtual ~FIndexedCurve() = default;

public:

	/** Gets the index of a handle, checks if the key handle is valid first. */
	ENGINE_API int32 GetIndexSafe(FKeyHandle KeyHandle) const;

	/** Const iterator for the handles. */
	TArray<FKeyHandle>::TConstIterator GetKeyHandleIterator() const
	{
		EnsureAllIndicesHaveHandles();
		return KeyHandlesToIndices.CreateConstIterator();
	}

	/** Get number of keys in curve. */
	virtual int32 GetNumKeys() const PURE_VIRTUAL(FIndexedCurve::GetNumKeys, return 0;);

	/** Move a key to a new time. This may change the index of the key, so the new key index is returned. */
	virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) PURE_VIRTUAL(FIndexedCurve::SetKeyTime, );

	/** Get the time for the Key with the specified index. */
	virtual float GetKeyTime(FKeyHandle KeyHandle) const PURE_VIRTUAL(FIndexedCurve::GetKeyTime, return 0.f;);
	
	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const PURE_VIRTUAL(FIndexedCurve::Duplicate, return nullptr;);

	/** Checks to see if the key handle is valid for this curve. */
	ENGINE_API bool IsKeyHandleValid(FKeyHandle KeyHandle) const;

	/** Gets the key handle for the first key in the curve */
	FKeyHandle GetFirstKeyHandle() const { return (GetNumKeys() > 0 ? GetKeyHandle(0) : FKeyHandle::Invalid()); }

	/** Gets the key handle for the last key in the curve */
	FKeyHandle GetLastKeyHandle() const { const int32 NumKeys = GetNumKeys(); return (NumKeys > 0 ? GetKeyHandle(NumKeys-1) : FKeyHandle::Invalid()); }

	/** Get the next key given the key handle */
	ENGINE_API FKeyHandle GetNextKey(FKeyHandle KeyHandle) const;

	/** Get the previous key given the key handle */
	ENGINE_API FKeyHandle GetPreviousKey(FKeyHandle KeyHandle) const;

	/** Shifts all keys forwards or backwards in time by an even amount, preserving order */
	ENGINE_API void ShiftCurve(float DeltaTime);
	ENGINE_API void ShiftCurve(float DeltaTime, const TSet<FKeyHandle>& KeyHandles);

	/** Scales all keys about an origin, preserving order */
	ENGINE_API void ScaleCurve(float ScaleOrigin, float ScaleFactor);
	ENGINE_API void ScaleCurve(float ScaleOrigin, float ScaleFactor, const TSet<FKeyHandle>& KeyHandles);

protected:

	/** Makes sure our handles are all valid and correct. */
	void EnsureAllIndicesHaveHandles() const { EnsureAllIndicesHaveHandles_Internal(GetNumKeys()); }
	void EnsureIndexHasAHandle(int32 KeyIndex) const { KeyHandlesToIndices.EnsureIndexHasAHandle(KeyIndex); }

private:
	ENGINE_API void EnsureAllIndicesHaveHandles_Internal(int32 NumKeys) const;

protected:

	/** Gets the index of a handle . */
	const int32* FindIndex(FKeyHandle KeyHandle) const { return KeyHandlesToIndices.Find(KeyHandle); }

	/** Gets the index of a handle. */
	ENGINE_API int32 GetIndex(FKeyHandle KeyHandle) const;

	/** Internal tool to get a handle from an index. */
	ENGINE_API FKeyHandle GetKeyHandle(int32 KeyIndex) const;

protected:

	/** Map of which key handles go to which indices. */
	UPROPERTY(transient)
	mutable FKeyHandleMap KeyHandlesToIndices;
};

template<>
struct TStructOpsTypeTraits<FIndexedCurve> : public TStructOpsTypeTraitsBase2<FIndexedCurve>
{
	enum
	{
		WithPureVirtual = true,
	};
};
