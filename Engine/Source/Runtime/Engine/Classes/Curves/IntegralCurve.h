// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "IntegralCurve.generated.h"


/** An integral key, which holds the key time and the key value */
USTRUCT()
struct FIntegralKey
{
	GENERATED_USTRUCT_BODY()
public:

	FIntegralKey(float InTime = 0.f, int32 InValue = 0)
		: Time(InTime)
		, Value(InValue)
	{ }
	
	/** The keyed time */
	UPROPERTY(EditAnywhere, Category=Key)
	float Time;

	/** The keyed integral value */
	UPROPERTY(EditAnywhere, Category=Key)
	int32 Value;
};


/** An integral curve, which holds the key time and the key value */
USTRUCT()
struct FIntegralCurve
	: public FIndexedCurve
{
	GENERATED_USTRUCT_BODY()

public:

	/** Default constructor. */
	FIntegralCurve() 
		: FIndexedCurve()
		, DefaultValue(MAX_int32)
		, bUseDefaultValueBeforeFirstKey(false)
	{ }

	/** Virtual destructor. */
	virtual ~FIntegralCurve() = default;
	
	/** Get number of keys in curve. */
	virtual int32 GetNumKeys() const override final { return Keys.Num(); }

	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const final { return new FIntegralCurve(*this); }

	/** Evaluates the value of an array of keys at a time */
	ENGINE_API int32 Evaluate(float Time, int32 InDefaultValue = 0) const;

	/**
	 * Check whether this curve has any data or not
	 */
	bool HasAnyData() const
	{
		return DefaultValue != MAX_int32 || Keys.Num();
	}

	/** Const iterator for the keys, so the indices and handles stay valid */
	ENGINE_API TArray<FIntegralKey>::TConstIterator GetKeyIterator() const;
	
	/**
	  * Add a new key to the curve with the supplied Time and Value.
	  * 
	  * @param KeyHandle Optionally can specify what handle this new key should have, otherwise, it'll make a new one
	  * @return The handle of the new key
	  */
	ENGINE_API FKeyHandle AddKey( float InTime, int32 InValue, FKeyHandle KeyHandle = FKeyHandle() );
	
	/** Remove the specified key from the curve.*/
	ENGINE_API void DeleteKey(FKeyHandle KeyHandle);
	
	/** Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	ENGINE_API FKeyHandle UpdateOrAddKey( float InTime, int32 Value, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER );
	
	/** Move a key to a new time. */
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) override final;

	/** Get the time for the Key with the specified index. */
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const override final;

	/** Set the value of the key with the specified index. */
	ENGINE_API void SetKeyValue(FKeyHandle KeyHandle, int32 NewValue);

	/** Get the value for the Key with the specified index. */
	ENGINE_API int32 GetKeyValue(FKeyHandle KeyHandle) const;
	
	/** Set the default value for the curve */
	void SetDefaultValue(int32 InDefaultValue) { DefaultValue = InDefaultValue; }

	/** Get the default value for the curve */
	int32 GetDefaultValue() const { return DefaultValue; }
	
	/** Removes the default value for this curve. */
	void ClearDefaultValue() { DefaultValue = MAX_int32; }

	/** Sets whether or not the default value should be used for evaluation for time values before the first key. */
	void SetUseDefaultValueBeforeFirstKey(bool InbUseDefaultValueBeforeFirstKey) { bUseDefaultValueBeforeFirstKey = InbUseDefaultValueBeforeFirstKey; }
	
	/** Gets whether or not the default value should be used for evaluation for time values before the first key. */
	bool GetUseDefaultValueBeforeFirstKey() const { return bUseDefaultValueBeforeFirstKey; }

	/** Functions for getting keys based on handles */
	ENGINE_API FIntegralKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API FIntegralKey GetKey(FKeyHandle KeyHandle) const;

	ENGINE_API FKeyHandle FindKey(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Gets the handle for the last key which is at or before the time requested.  If there are no keys at or before the requested time, an invalid handle is returned. */
	ENGINE_API FKeyHandle FindKeyBeforeOrAt(float KeyTime) const;

	/** Tries to reduce the number of keys required for accurate evaluation (zero error threshold) */
	ENGINE_API void RemoveRedundantKeys();
private:

	/** The keys, ordered by time */
	UPROPERTY(EditAnywhere, Category="Curve")
	TArray<FIntegralKey> Keys;

	/** Default value */
	UPROPERTY(EditAnywhere, Category="Curve")
	int32 DefaultValue;

	UPROPERTY()
	bool bUseDefaultValueBeforeFirstKey;
};
