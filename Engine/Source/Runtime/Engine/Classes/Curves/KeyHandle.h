// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "KeyHandle.generated.h"

/**
 * Key handles are used to keep a handle to a key. They are completely transient.
 */
struct FKeyHandle
{
	ENGINE_API FKeyHandle();

	static ENGINE_API FKeyHandle Invalid();

	ENGINE_API bool IsValid() const;

	operator bool() const { return IsValid(); }

	bool operator ==(const FKeyHandle& Other) const
	{
		return Index == Other.Index;
	}
	
	bool operator !=(const FKeyHandle& Other) const
	{
		return Index != Other.Index;
	}

	friend bool operator<(FKeyHandle A, FKeyHandle B)
	{
		return A.Index < B.Index;
	}

	friend bool operator>(FKeyHandle A, FKeyHandle B)
	{
		return A.Index > B.Index;
	}

	friend uint32 GetTypeHash(const FKeyHandle& Handle)
	{
		return GetTypeHash(Handle.Index);
	}

	friend FArchive& operator<<(FArchive& Ar,FKeyHandle& Handle)
	{
		Ar << Handle.Index;
		return Ar;
	}

private:

	/** Private constructor from a specific index - only for use in FKeyHandle::Invalid to avoid allocating new handles unnecessarily */
	explicit FKeyHandle(uint32 SpecificIndex);

	uint32 Index;
};


/**
 * Represents a mapping of key handles to key index that may be serialized 
 */
USTRUCT()
struct FKeyHandleMap
{
	GENERATED_USTRUCT_BODY()
public:
	FKeyHandleMap() {}

	// This struct is not copyable.  This must be public or because derived classes are allowed to be copied
	FKeyHandleMap( const FKeyHandleMap& Other ) {}
	void operator=(const FKeyHandleMap& Other) {}

	// Quickly initializes this map by clearing it and filling with KeyHandles in O(n) time, instead of O(n^2) if Add() was used in a loop.
	void Initialize(TArrayView<const FKeyHandle> InKeyHandles);

	/** TMap functionality */
	void Add( const FKeyHandle& InHandle, int32 InIndex );
	void Empty(int32 ExpectedNumElements = 0);
	void Remove( const FKeyHandle& InHandle );
	void Reserve(int32 NumElements);
	const int32* Find(const FKeyHandle& InHandle) const { return KeyHandlesToIndices.Find(InHandle); }
	const FKeyHandle* FindKey( int32 KeyIndex ) const;
	int32 Num() const { return KeyHandlesToIndices.Num(); }
	TArray<FKeyHandle>::TConstIterator CreateConstIterator() const { return KeyHandles.CreateConstIterator(); }
	const TMap<FKeyHandle, int32>& GetMap() const { return KeyHandlesToIndices; }

	void SetKeyHandles(int32 Num);

	/** ICPPStructOps implementation */
	bool Serialize(FArchive& Ar);
	bool operator==(const FKeyHandleMap& Other) const { return KeyHandles == Other.KeyHandles; }
	bool operator!=(const FKeyHandleMap& Other) const { return !(*this==Other); }

	friend FArchive& operator<<(FArchive& Ar,FKeyHandleMap& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// Ensures that all indices have a valid handle and that there are no handles left to invalid indices
	void EnsureAllIndicesHaveHandles(int32 NumIndices);
	
	// Ensures a handle exists for the specified Index
	void EnsureIndexHasAHandle(int32 KeyIndex);

private:

	TMap<FKeyHandle, int32> KeyHandlesToIndices;
	TArray<FKeyHandle> KeyHandles;
};


template<>
struct TStructOpsTypeTraits<FKeyHandleMap>
	: public TStructOpsTypeTraitsBase2<FKeyHandleMap>
{
	enum 
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/**
 * Lookup table that maps key handles to indices in an external data structure
 * Maintains a map of key handle to last known index,
 * and an array of optional key handles that's used to validate map entries.
 */
USTRUCT()
struct FKeyHandleLookupTable
{
	GENERATED_BODY()
public:
	/**
	 * Get the index that corresponds to the specified key handle
	 *
	 * @param KeyHandle		The key handle to retrieve an up to date index for
	 * @return The index that corresponds to this key handle, or INDEX_NONE if the key handle is invalid
	 */
	ENGINE_API int32 GetIndex(FKeyHandle KeyHandle);

	/**
	 * Attempt to find the handle for the specified index, or allocate a new one if it doesn't have one
	 *
	 * @param Index			The index to find or allocate a handle for
	 * @return A valid key handle for the specified index
	 */
	ENGINE_API FKeyHandle FindOrAddKeyHandle(int32 Index);

	/**
	 * Move the specified handle from its previous index, to its new index
	 *
	 * @param OldIndex		The previous index of the handle
	 * @param NewIndex		The new index of the handle
	 */
	ENGINE_API void MoveHandle(int32 OldIndex, int32 NewIndex);

	/**
	 * Allocate a new handle for the specified index
	 *
	 * @param Index			The index to associate a handle with
	 * @return A new key handle that corresponds to the specified index
	 */
	ENGINE_API FKeyHandle AllocateHandle(int32 Index);

	/**
	 * Deallocate the specified index
	 *
	 * @param Index			The index to invalidate a handle for
	 */
	ENGINE_API void DeallocateHandle(int32 Index);

	/**
	 *  Replaces the key handle at Index with HandleToReplace.
	 *  
	 * @param Index				The index at which to replace the handle
	 * @param HandleToReplace	The key handle to set at the index. Fails if this handle already is in the table.
	 */
	ENGINE_API void ReplaceHandle(int32 Index, FKeyHandle HandleToReplace);

	/**
	 * Reset this lookup table, forgetting all key handles and indices
	 */
	ENGINE_API void Reset();

	/** ICPPStructOps implementation */
	ENGINE_API bool Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FKeyHandleLookupTable& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

private:

	/** Relocate key handles and indices proceeding the specified index by shifting them forwards or backwards */
	void RelocateKeyHandles(int32 StartAtIndex, int32 DeltaIndex);

private:
	/** Array of optional key handles that reside in corresponding indices for an externally owned serial data structure */
	TSparseArray<FKeyHandle> KeyHandles;

	/** Map of which key handles go to which indices. Indices may point to incorrect indices. Check against entry in KeyHandles array to verify. */
	TMap<FKeyHandle, int32> KeyHandlesToIndices;

};

template<>
struct TStructOpsTypeTraits<FKeyHandleLookupTable>
	: public TStructOpsTypeTraitsBase2<FKeyHandleLookupTable>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};