// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Curves/RichCurve.h"
#include "Curves/CurveOwnerInterface.h"

#include "LinearColorRamp.generated.h"

#define UE_API ENGINE_API

USTRUCT()
struct FLinearColorRamp 
#if CPP
	: public FCurveOwnerInterface
#endif
{
	GENERATED_USTRUCT_BODY()

public:
	UE_API void SetColorAtTime(float Time, const FLinearColor& Color, bool bOnlyRBG);
	UE_API bool IsEmpty() const;
	UE_API void Reset();
	UE_API void SetFrom(const FLinearColorRamp& Other);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnColorCurveChanged, TArray<FRichCurve*> /* ChangedCurves */);
	FOnColorCurveChanged OnColorCurveChangedDelegate;

	//~ Begin FCurveOwnerInterface interface
	UE_API virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	UE_API virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	UE_API virtual TArray<FRichCurveEditInfo> GetCurves() override;
	UE_API virtual void ModifyOwner() override;
	UE_API virtual TArray<const UObject*> GetOwners() const override;
	UE_API virtual void MakeTransactional() override;
	UE_API virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	UE_API virtual bool IsLinearColorCurve() const override;
	UE_API virtual FLinearColor GetLinearColorValue(float InTime) const override;
	UE_API virtual bool HasAnyAlphaKeys() const override;
	UE_API virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;
	UE_API virtual FLinearColor GetCurveColor(FRichCurveEditInfo CurveInfo) const override;
	//~ End FCurveOwnerInterface interface

private:
	UPROPERTY()
	FRichCurve RichCurves[4];
};

#undef UE_API
