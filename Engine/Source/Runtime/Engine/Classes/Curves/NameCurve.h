// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "NameCurve.generated.h"

/**
 * One key in a curve of FNames.
 */
USTRUCT()
struct FNameCurveKey
{
	GENERATED_USTRUCT_BODY()

	/** Time at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	float Time;

	/** Value at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	FName Value;

	/** Default constructor. */
	FNameCurveKey()
		: Time(0.0f)
		, Value(NAME_None)
	{ }

	/** Creates and initializes a new instance. */
	FNameCurveKey(float InTime, const FName& InValue)
		: Time(InTime)
		, Value(InValue)
	{ }

public:

	// TStructOpsTypeTraits interface

	ENGINE_API bool operator==(const FNameCurveKey& Other) const;
	ENGINE_API bool operator!=(const FNameCurveKey& Other) const;
	ENGINE_API bool Serialize(FArchive& Ar);

	/** Serializes a name curve key from or into an archive. */
	friend FArchive& operator<<(FArchive& Ar, FNameCurveKey& Key)
	{
		Key.Serialize(Ar);
		return Ar;
	}
};


template<>
struct TIsPODType<FNameCurveKey>
{
	enum { Value = true };
};


template<>
struct TStructOpsTypeTraits<FNameCurveKey>
	: public TStructOpsTypeTraitsBase2<FNameCurveKey>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


/**
 * Implements a curve of FNames.
 */
USTRUCT()
struct FNameCurve
	: public FIndexedCurve
{
	GENERATED_USTRUCT_BODY()

	/** Virtual destructor. */
	virtual ~FNameCurve() = default;

public:

	/**
	  * Add a new key to the curve with the supplied Time and Value. Returns the handle of the new key.
	  * 
	  * @param InTime The time at which to add the key.
	  * @param InValue The value of the key.
	  * @param KeyHandle Optional handle for the new key.
	  */
	ENGINE_API FKeyHandle AddKey(float InTime, const FName& InValue, FKeyHandle KeyHandle = FKeyHandle());

	/**
	 * Remove the specified key from the curve.
	 *
	 * @param KeyHandle Handle to the key to remove.
	 */
	ENGINE_API void DeleteKey(FKeyHandle KeyHandle);

	/**
	 * Finds a key a the specified time.
	 *
	 * @param KeyTime The time at which to find the key.
	 * @param KeyTimeTolerance The key time tolerance to use for equality.
	 * @return A handle to the key, or invalid key handle if not found.
	 */
	ENGINE_API FKeyHandle FindKey(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Get a key.
	 *
	 * @param KeyHandle The handle of the key to get.
	 * @return The key.
	 */
	ENGINE_API FNameCurveKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API FNameCurveKey GetKey(FKeyHandle KeyHandle) const;

	/**
	 * Read-only access to the key collection.
	 *
	 * @return Collection of keys.
	 */
	const TArray<FNameCurveKey>& GetKeys() const
	{
		return Keys;
	}

	/**
	 * Get the time for the Key with the specified index.
	 *
	 * @param KeyHandle Handle to the key whose time to get.
	 * @return The key's time.
	 */
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const override final;

	/**
	 * Move a key to a new time.
	 *
	 * @param KeyHandle The handle of the key to change.
	 * @param NewTime The new time to set on the key.
	 */
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) override final;

	/**
	 * Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time.
	 *
	 * @param InTime The time at which the key should be added or updated.
	 * @param InValue The value of the key.
	 * @param KeyTimeTolerance The tolerance used for key time equality.
	 */
	ENGINE_API FKeyHandle UpdateOrAddKey(float InTime, const FName& InValue, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER);

public:

	// FIndexedCurve interface

	virtual int32 GetNumKeys() const override final { return Keys.Num(); }

	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const final { return new FNameCurve(*this); }

public:

	/** Sorted array of keys */
	UPROPERTY(EditAnywhere, EditFixedSize, Category="Curve")
	TArray<FNameCurveKey> Keys;
};
