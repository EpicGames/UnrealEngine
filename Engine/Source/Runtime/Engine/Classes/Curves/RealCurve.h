// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/IndexedCurve.h"
#include "Misc/FrameRate.h"
#include "RealCurve.generated.h"

/** Method of interpolation between this key and the next. */
UENUM(BlueprintType)
enum ERichCurveInterpMode : int
{
	/** Use linear interpolation between values. */
	RCIM_Linear UMETA(DisplayName = "Linear"),
	/** Use a constant value. Represents stepped values. */
	RCIM_Constant UMETA(DisplayName = "Constant"),
	/** Cubic interpolation. See TangentMode for different cubic interpolation options. */
	RCIM_Cubic UMETA(DisplayName = "Cubic"),
	/** No interpolation. */
	RCIM_None UMETA(Hidden)
};

/** Enumerates extrapolation options. */
UENUM(BlueprintType)
enum ERichCurveExtrapolation : int
{
	/** Repeat the curve without an offset. */
	RCCE_Cycle UMETA(DisplayName = "Cycle"),
	/** Repeat the curve with an offset relative to the first or last key's value. */
	RCCE_CycleWithOffset UMETA(DisplayName = "CycleWithOffset"),
	/** Sinusoidally extrapolate. */
	RCCE_Oscillate UMETA(DisplayName = "Oscillate"),
	/** Use a linearly increasing value for extrapolation.*/
	RCCE_Linear UMETA(DisplayName = "Linear"),
	/** Use a constant value for extrapolation */
	RCCE_Constant UMETA(DisplayName = "Constant"),
	/** No Extrapolation */
	RCCE_None UMETA(DisplayName = "None")
};


/**
 * Structure allowing external curve data to express extended attributes
 */
struct FCurveAttributes
{
	FCurveAttributes()
	{
		bHasPreExtrapolation  = 0;
		bHasPostExtrapolation = 0;
		PreExtrapolation = ERichCurveExtrapolation::RCCE_None;
		PostExtrapolation = ERichCurveExtrapolation::RCCE_None;
	}

	/**
	 * Check whether this curve has the specified properties
	 */
	bool HasPreExtrapolation() const  { return bHasPreExtrapolation;  }
	bool HasPostExtrapolation() const { return bHasPostExtrapolation; }

	/**
	 * Access the extended properties of this curve. Must check whether the curve has such properties first
	 */
	ERichCurveExtrapolation GetPreExtrapolation() const  { check(bHasPreExtrapolation);  return PreExtrapolation;  }
	ERichCurveExtrapolation GetPostExtrapolation() const { check(bHasPostExtrapolation); return PostExtrapolation; }

	/**
	 * Set the extended properties of this curve
	 */
	FCurveAttributes& SetPreExtrapolation(ERichCurveExtrapolation InPreExtrapolation)   { bHasPreExtrapolation = 1;  PreExtrapolation = InPreExtrapolation;   return *this; }
	FCurveAttributes& SetPostExtrapolation(ERichCurveExtrapolation InPostExtrapolation) { bHasPostExtrapolation = 1; PostExtrapolation = InPostExtrapolation; return *this; }
	/**
	 * Reset the extended properties of this curve, implying it does not support such properties
	 */
	void UnsetPreExtrapolation()  { bHasPreExtrapolation = 0;  }
	void UnsetPostExtrapolation() { bHasPostExtrapolation = 0; }

	/**
	 * Generate a new set of attributes that contains only those attributes common to both A and B
	 */
	static FCurveAttributes MaskCommon(const FCurveAttributes& A, const FCurveAttributes& B)
	{
		FCurveAttributes NewAttributes;

		if (A.bHasPreExtrapolation && B.bHasPreExtrapolation && A.PreExtrapolation == B.PreExtrapolation)
		{
			NewAttributes.SetPreExtrapolation(A.PreExtrapolation);
		}

		if (A.bHasPostExtrapolation && B.bHasPostExtrapolation && A.PostExtrapolation == B.PostExtrapolation)
		{
			NewAttributes.SetPostExtrapolation(A.PostExtrapolation);
		}

		return NewAttributes;
	}

	friend bool operator==(const FCurveAttributes& Left, const FCurveAttributes& Right)
	{
		return Left.bHasPreExtrapolation == Right.bHasPreExtrapolation
			&& Left.bHasPostExtrapolation == Right.bHasPostExtrapolation
			&& (!Left.bHasPreExtrapolation || Left.PreExtrapolation == Right.PreExtrapolation)
			&& (!Left.bHasPostExtrapolation || Left.PostExtrapolation == Right.PostExtrapolation);
	}
	friend bool operator!=(const FCurveAttributes& Left, const FCurveAttributes& Right) { return !(Left == Right); }

private:

	/** true if the curve can express pre-extrapolation modes */
	uint8 bHasPreExtrapolation : 1;
	/** true if the curve can express post-extrapolation modes */
	uint8 bHasPostExtrapolation : 1;

	/** This curve's pre-extrapolation mode. Only valid to read if bHasPreExtrapolation is true */
	ERichCurveExtrapolation PreExtrapolation;
	/** This curve's post-extrapolation mode. Only valid to read if bHasPostExtrapolation is true */
	ERichCurveExtrapolation PostExtrapolation;
};

/** A rich, editable float curve */
USTRUCT()
struct FRealCurve
	: public FIndexedCurve
{
	GENERATED_USTRUCT_BODY()

	FRealCurve() 
		: FIndexedCurve()
		, DefaultValue(MAX_flt)
		, PreInfinityExtrap(RCCE_Constant)
		, PostInfinityExtrap(RCCE_Constant)
	{ }

public:

	/**
	 * Check whether this curve has any data or not
	 */
	bool HasAnyData() const
	{
		return DefaultValue != MAX_flt || GetNumKeys();
	}

	/**
	  * Add a new key to the curve with the supplied Time and Value. Returns the handle of the new key.
	  * 
	  * @param	bUnwindRotation		When true, the value will be treated like a rotation value in degrees, and will automatically be unwound to prevent flipping 360 degrees from the previous key 
	  * @param  KeyHandle			Optionally can specify what handle this new key should have, otherwise, it'll make a new one
	  */
	virtual FKeyHandle AddKey(float InTime, float InValue, const bool bUnwindRotation = false, FKeyHandle KeyHandle = FKeyHandle()) PURE_VIRTUAL(FRealCurve::AddKey, return FKeyHandle::Invalid(););

	/**
	 *  Remove the specified key from the curve.
	 *
	 * @param KeyHandle The handle of the key to remove.
	 * @see AddKey, SetKeys
	 */
	virtual void DeleteKey(FKeyHandle KeyHandle) PURE_VIRTUAL(FRealCurve::DeleteKey,);

	/** Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	virtual FKeyHandle UpdateOrAddKey(float InTime, float InValue, const bool bUnwindRotation = false, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) PURE_VIRTUAL(FRealCurve::UpdateOrAddKey, return FKeyHandle::Invalid(););

	/** Finds a key a the specified time */
	ENGINE_API FKeyHandle FindKey(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/** True if a key exists already, false otherwise */
	ENGINE_API bool KeyExistsAtTime(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Set the value of the specified key */
	virtual void SetKeyValue(FKeyHandle KeyHandle, float NewValue, bool bAutoSetTangents = true) PURE_VIRTUAL(FRealCurve::SetKeyValue,);

	/** Returns the value of the specified key */
	virtual float GetKeyValue(FKeyHandle KeyHandle) const PURE_VIRTUAL(FRealCurve::GetKeyValue, return 0.f;);

	/** Returns a <Time, Value> pair for the specified key */
	virtual TPair<float, float> GetKeyTimeValuePair(FKeyHandle KeyHandle) const PURE_VIRTUAL(FRealCurve::GetKeyTimeValuePair, return (TPair<float,float>(0.f,0.f)););

	virtual void SetKeyInterpMode(FKeyHandle KeyHandle, ERichCurveInterpMode NewInterpMode) PURE_VIRTUAL(FRealCurve::SetKeyInterpMode,);

	/** Set the default value of the curve */
	void SetDefaultValue(float InDefaultValue) { DefaultValue = InDefaultValue; }

	/** Get the default value for the curve */
	float GetDefaultValue() const { return DefaultValue; }

	/** Removes the default value for this curve. */
	void ClearDefaultValue() { DefaultValue = MAX_flt; }

	virtual ERichCurveInterpMode GetKeyInterpMode(FKeyHandle KeyHandle) const PURE_VIRTUAL(FRealCurve::GetTimeRange, return RCIM_None; );

	/** Get range of input time values. Outside this region curve continues constantly the start/end values. */
	virtual void GetTimeRange(float& MinTime, float& MaxTime) const PURE_VIRTUAL(FRealCurve::GetTimeRange, );

	/** Get range of output values. */
	virtual void GetValueRange(float& MinValue, float& MaxValue) const PURE_VIRTUAL(FRealCurve::GetValueRange, );

	/** Clear all keys. */
	virtual void Reset() PURE_VIRTUAL(FRealCurve::Reset, );

	/** Remap InTime based on pre and post infinity extrapolation values */
	virtual void RemapTimeValue(float& InTime, float& CycleValueOffset) const PURE_VIRTUAL(FRealCurve::RemapTimeValue, );

	/** Evaluate this curve at the specified time */
	virtual float Eval(float InTime, float InDefaultValue = 0.0f) const PURE_VIRTUAL(FRealCurve::Eval, return 0.f;);

	/** Resize curve length to the [MinTimeRange, MaxTimeRange] */
	virtual void ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime) PURE_VIRTUAL(FRealCurve::ReadjustTimeRange, );

	/** Bake curve given the sample rate */
	virtual void BakeCurve(float SampleRate) PURE_VIRTUAL(FRealCurve::BakeCurve, );
	virtual void BakeCurve(float SampleRate, float FirstKeyTime, float LastKeyTime) PURE_VIRTUAL(FRealCurve::BakeCurve, );

	/** Remove redundant keys, comparing against Tolerance (and optional use of sample-rate for additional testing) */
	virtual void RemoveRedundantKeys(float Tolerance, FFrameRate SampleRate = FFrameRate(0,0)) PURE_VIRTUAL(FRealCurve::RemoveRedundantKeys, );
	virtual void RemoveRedundantKeys(float Tolerance, float FirstKeyTime, float LastKeyTime, FFrameRate SampleRate = FFrameRate(0,0)) PURE_VIRTUAL(FRealCurve::RemoveRedundantKeys, );

protected:
	static ENGINE_API void CycleTime(float MinTime, float MaxTime, float& InTime, int& CycleCount);
	virtual int32 GetKeyIndex(float KeyTime, float KeyTimeTolerance) const PURE_VIRTUAL(FRealCurve::GetKeyIndex, return INDEX_NONE;);

public:

	/** Default value */
	UPROPERTY(EditAnywhere, Category = "Curve")
	float DefaultValue;

	/** Pre-infinity extrapolation state */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;

	/** Post-infinity extrapolation state */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;
};

template<>
struct TStructOpsTypeTraits<FRealCurve> : public TStructOpsTypeTraitsBase2<FRealCurve>
{
	enum
	{
		WithPureVirtual = true,
	};
};
