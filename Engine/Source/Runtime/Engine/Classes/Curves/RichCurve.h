// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Chaos/Curve.h"
#include "RichCurve.generated.h"

/** If using RCIM_Cubic, this enum describes how the tangents should be controlled in editor. */
UENUM(BlueprintType)
enum ERichCurveTangentMode : int
{
	/** Automatically calculates tangents to create smooth curves between values. */
	RCTM_Auto UMETA(DisplayName="Auto"),
	/** User specifies the tangent as a unified tangent where the two tangents are locked to each other, presenting a consistent curve before and after. */
	RCTM_User UMETA(DisplayName="User"),
	/** User specifies the tangent as two separate broken tangents on each side of the key which can allow a sharp change in evaluation before or after. */
	RCTM_Break UMETA(DisplayName="Break"),
	/** No tangents. */
	RCTM_None UMETA(Hidden),
	/** New Auto tangent that creates smoother curves than Auto. */
	RCTM_SmartAuto UMETA(DisplayName = "Smart Auto"),
};


/** Enumerates tangent weight modes. */
UENUM(BlueprintType)
enum ERichCurveTangentWeightMode : int
{
	/** Don't take tangent weights into account. */
	RCTWM_WeightedNone UMETA(DisplayName="None"),
	/** Only take the arrival tangent weight into account for evaluation. */
	RCTWM_WeightedArrive UMETA(DisplayName="Arrive"),
	/** Only take the leaving tangent weight into account for evaluation. */
	RCTWM_WeightedLeave UMETA(DisplayName="Leave"),
	/** Take both the arrival and leaving tangent weights into account for evaluation. */
	RCTWM_WeightedBoth UMETA(DisplayName="Both")
};

/** Enumerates curve compression options. */
UENUM()
enum ERichCurveCompressionFormat : int
{
	/** No keys are present */
	RCCF_Empty UMETA(DisplayName = "Empty"),

	/** All keys use constant interpolation */
	RCCF_Constant UMETA(DisplayName = "Constant"),

	/** All keys use linear interpolation */
	RCCF_Linear UMETA(DisplayName = "Linear"),

	/** All keys use cubic interpolation */
	RCCF_Cubic UMETA(DisplayName = "Cubic"),

	/** Keys use mixed interpolation modes */
	RCCF_Mixed UMETA(DisplayName = "Mixed"),

	/** Keys use weighted interpolation modes */
	RCCF_Weighted UMETA(DisplayName = "Weighted"),
};

/** Enumerates key time compression options. */
UENUM()
enum ERichCurveKeyTimeCompressionFormat : int
{
	/** Key time is quantized to 16 bits */
	RCKTCF_uint16 UMETA(DisplayName = "uint16"),

	/** Key time uses full precision */
	RCKTCF_float32 UMETA(DisplayName = "float32"),
};

/** One key in a rich, editable float curve */
USTRUCT(BlueprintType)
struct FRichCurveKey
{
	GENERATED_USTRUCT_BODY()

	/** Interpolation mode between this key and the next */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Key")
	TEnumAsByte<ERichCurveInterpMode> InterpMode;

	/** Mode for tangents at this key */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Key")
	TEnumAsByte<ERichCurveTangentMode> TangentMode;

	/** If either tangent at this key is 'weighted' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Key")
	TEnumAsByte<ERichCurveTangentWeightMode> TangentWeightMode;

	/** Time at this key */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Key")
	float Time;

	/** Value at this key */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Key")
	float Value;

	/** If RCIM_Cubic, the arriving tangent at this key */
	UPROPERTY()
	float ArriveTangent;

	/** If RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the left tangent */
	UPROPERTY()
	float ArriveTangentWeight;

	/** If RCIM_Cubic, the leaving tangent at this key */
	UPROPERTY()
	float LeaveTangent;

	/** If RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the right tangent */
	UPROPERTY()
	float LeaveTangentWeight;

	FRichCurveKey()
		: InterpMode(RCIM_Linear)
		, TangentMode(RCTM_Auto)
		, TangentWeightMode(RCTWM_WeightedNone)
		, Time(0.f)
		, Value(0.f)
		, ArriveTangent(0.f)
		, ArriveTangentWeight(0.f)
		, LeaveTangent(0.f)
		, LeaveTangentWeight(0.f)
	{ }

	FRichCurveKey(float InTime, float InValue)
		: InterpMode(RCIM_Linear)
		, TangentMode(RCTM_Auto)
		, TangentWeightMode(RCTWM_WeightedNone)
		, Time(InTime)
		, Value(InValue)
		, ArriveTangent(0.f)
		, ArriveTangentWeight(0.f)
		, LeaveTangent(0.f)
		, LeaveTangentWeight(0.f)
	{ }

	FRichCurveKey(float InTime, float InValue, float InArriveTangent, const float InLeaveTangent, ERichCurveInterpMode InInterpMode)
		: InterpMode(InInterpMode)
		, TangentMode(RCTM_Auto)
		, TangentWeightMode(RCTWM_WeightedNone)
		, Time(InTime)
		, Value(InValue)
		, ArriveTangent(InArriveTangent)
		, ArriveTangentWeight(0.f)
		, LeaveTangent(InLeaveTangent)
		, LeaveTangentWeight(0.f)
	{ }

	/** Conversion constructor */
	ENGINE_API FRichCurveKey(const FInterpCurvePoint<float>& InPoint);
	ENGINE_API FRichCurveKey(const FInterpCurvePoint<FVector2D>& InPoint, int32 ComponentIndex);
	ENGINE_API FRichCurveKey(const FInterpCurvePoint<FVector>& InPoint, int32 ComponentIndex);
	ENGINE_API FRichCurveKey(const FInterpCurvePoint<FTwoVectors>& InPoint, int32 ComponentIndex);

	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API bool operator==(const FRichCurveKey& Other) const;
	ENGINE_API bool operator!=(const FRichCurveKey& Other) const;

	friend FArchive& operator<<(FArchive& Ar, FRichCurveKey& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};


template<>
struct TIsPODType<FRichCurveKey>
{
	enum { Value = true };
};


template<>
struct TStructOpsTypeTraits<FRichCurveKey>
	: public TStructOpsTypeTraitsBase2<FRichCurveKey>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


/** A rich, editable float curve */
USTRUCT()
struct FRichCurve
	: public FRealCurve
{
	GENERATED_USTRUCT_BODY()

public:

	/** Gets a copy of the keys, so indices and handles can't be meddled with */
	ENGINE_API TArray<FRichCurveKey> GetCopyOfKeys() const;

	/** Gets a const reference of the keys, so indices and handles can't be meddled with */
	ENGINE_API const TArray<FRichCurveKey>& GetConstRefOfKeys() const;

	/** Const iterator for the keys, so the indices and handles stay valid */
	ENGINE_API TArray<FRichCurveKey>::TConstIterator GetKeyIterator() const;
	
	/** Functions for getting keys based on handles */
	ENGINE_API FRichCurveKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API FRichCurveKey GetKey(FKeyHandle KeyHandle) const;
	ENGINE_API const FRichCurveKey& GetKeyRef(FKeyHandle KeyHandle) const;
	
	/** Quick accessors for the first and last keys */
	ENGINE_API FRichCurveKey GetFirstKey() const;
	ENGINE_API FRichCurveKey GetLastKey() const;

	/** Get the first key that matches any of the given key handles. */
	ENGINE_API FRichCurveKey* GetFirstMatchingKey(const TArray<FKeyHandle>& KeyHandles);

	/**
	  * Add a new key to the curve with the supplied Time and Value. Returns the handle of the new key.
	  * 
	  * @param	bUnwindRotation		When true, the value will be treated like a rotation value in degrees, and will automatically be unwound to prevent flipping 360 degrees from the previous key 
	  * @param  KeyHandle			Optionally can specify what handle this new key should have, otherwise, it'll make a new one
	  */
	ENGINE_API virtual FKeyHandle AddKey(float InTime, float InValue, const bool bUnwindRotation = false, FKeyHandle KeyHandle = FKeyHandle()) final override;

	/**
	* Reserves keys to be added by AddKey.
	* 
	* @see AddKey
	*/
	ENGINE_API void ReserveKeys(const int32 Number);

	/**
	 * Sets the keys with the keys.
	 *
	 * Expects that the keys are already sorted.
	 *
	 * @see AddKey, DeleteKey
	 */
	ENGINE_API void SetKeys(const TArray<FRichCurveKey>& InKeys);

	/**
	 *  Remove the specified key from the curve.
	 *
	 * @param KeyHandle The handle of the key to remove.
	 * @see AddKey, SetKeys
	 */
	ENGINE_API void DeleteKey(FKeyHandle KeyHandle) final override;

	/** Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	ENGINE_API virtual FKeyHandle UpdateOrAddKey(float InTime, float InValue, const bool bUnwindRotation = false, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) final override;

	/** Move a key to a new time. */
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) final override;

	/** Get the time for the Key with the specified index. */
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const final override;

	/** Set the value of the specified key */
	ENGINE_API virtual void SetKeyValue(FKeyHandle KeyHandle, float NewValue, bool bAutoSetTangents = true) final override;

	/** Returns the value of the specified key */
	ENGINE_API virtual float GetKeyValue(FKeyHandle KeyHandle) const final override;

	/** Returns a <Time, Value> pair for the specified key */
	ENGINE_API virtual TPair<float, float> GetKeyTimeValuePair(FKeyHandle KeyHandle) const final override;

	/** Returns whether the curve is constant or not */
	ENGINE_API bool IsConstant(float Tolerance = UE_SMALL_NUMBER) const;

	/** Returns whether the curve is empty or not */
	bool IsEmpty() const { return Keys.Num() == 0; }

	/** Set the interp mode of the specified key */
	ENGINE_API virtual void SetKeyInterpMode(FKeyHandle KeyHandle, ERichCurveInterpMode NewInterpMode) final override;
	ENGINE_API void SetKeyInterpMode(FKeyHandle KeyHandle, ERichCurveInterpMode NewInterpMode, bool bAutoSetTangents);

	/** Set the tangent mode of the specified key */
	ENGINE_API void SetKeyTangentMode(FKeyHandle KeyHandle, ERichCurveTangentMode NewTangentMode, bool bAutoSetTangents = true);

	/** Set the tangent weight mode of the specified key */
	ENGINE_API void SetKeyTangentWeightMode(FKeyHandle KeyHandle, ERichCurveTangentWeightMode NewTangentWeightMode, bool bAutoSetTangents = true);

	/** Get the interp mode of the specified key */
	ENGINE_API virtual ERichCurveInterpMode GetKeyInterpMode(FKeyHandle KeyHandle) const final override;

	/** Get the tangent mode of the specified key */
	ENGINE_API ERichCurveTangentMode GetKeyTangentMode(FKeyHandle KeyHandle) const;

	/** Get range of input time values. Outside this region curve continues constantly the start/end values. */
	ENGINE_API virtual void GetTimeRange(float& MinTime, float& MaxTime) const final override;

	/** Get range of output values. */
	ENGINE_API virtual void GetValueRange(float& MinValue, float& MaxValue) const final override;

	/** Clear all keys. */
	ENGINE_API virtual void Reset() final override;

	/** Remap InTime based on pre and post infinity extrapolation values */
	ENGINE_API virtual void RemapTimeValue(float& InTime, float& CycleValueOffset) const final override;

	/** Evaluate this rich curve at the specified time */
	ENGINE_API virtual float Eval(float InTime, float InDefaultValue = 0.0f) const final override;

	/** Auto set tangents for any 'auto' keys in curve */
	ENGINE_API void AutoSetTangents(float Tension = 0.f);

	/** Resize curve length to the [MinTimeRange, MaxTimeRange] */
	ENGINE_API virtual void ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime) final override;

	/** Determine if two RichCurves are the same */
	ENGINE_API bool operator == (const FRichCurve& Curve) const;

	/** Bake curve given the sample rate */
	ENGINE_API virtual void BakeCurve(float SampleRate) final override;
	ENGINE_API virtual void BakeCurve(float SampleRate, float FirstKeyTime, float LastKeyTime) final override;

	/** Remove redundant keys, comparing against Tolerance */
	ENGINE_API void RemoveRedundantAutoTangentKeys(float Tolerance);
	ENGINE_API virtual void RemoveRedundantKeys(float Tolerance, FFrameRate SampleRate) final override;

	ENGINE_API void RemoveRedundantAutoTangentKeys(float Tolerance, float FirstKeyTime, float LastKeyTime);
	ENGINE_API virtual void RemoveRedundantKeys(float Tolerance, float FirstKeyTime, float LastKeyTime, FFrameRate SampleRate) final override;

	/** Compresses a rich curve for efficient runtime storage and evaluation */
	ENGINE_API void CompressCurve(struct FCompressedRichCurve& OutCurve, float ErrorThreshold = 0.0001f, float SampleRate = 120.0) const;

	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const final { return new FRichCurve(*this); }

	/** Convert a rich curve to Chaos FLinearCurve */
	ENGINE_API void ConvertToChaosCurve(Chaos::FLinearCurve& OutCurve) const;

private:
	ENGINE_API void RemoveRedundantKeysInternal(float Tolerance, int32 InStartKeepKey, int32 InEndKeepKey, FFrameRate SampleRate);
	ENGINE_API virtual int32 GetKeyIndex(float KeyTime, float KeyTimeTolerance) const override final;

public:

	// FIndexedCurve interface

	virtual int32 GetNumKeys() const final override { return Keys.Num(); }

public:

	/** Sorted array of keys */
	UPROPERTY(EditAnywhere, EditFixedSize, Category="Curve", meta=(EditFixedOrder))
	TArray<FRichCurveKey> Keys;
};

/**
 * A runtime optimized representation of a FRichCurve. It consumes less memory and evaluates faster.
 */
USTRUCT()
struct FCompressedRichCurve
{
	GENERATED_USTRUCT_BODY()

	/** Compression format used by CompressedKeys */
	TEnumAsByte<ERichCurveCompressionFormat> CompressionFormat;

	/** Compression format used to pack the key time */
	TEnumAsByte<ERichCurveKeyTimeCompressionFormat> KeyTimeCompressionFormat;

	/** Pre-infinity extrapolation state */
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;

	/** Post-infinity extrapolation state */
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;

	union TConstantValueNumKeys
	{
		float ConstantValue;
		int32 NumKeys;

		TConstantValueNumKeys() : NumKeys(0) {}
	};

	/**
	* If the compression format is constant, this is the value returned
	* Inline here to reduce the likelihood of accessing the compressed keys data for the common case of constant/zero/empty curves
	* When a curve is linear/cubic/mixed, the constant float value isn't used and instead we use the number of keys
	*/
	TConstantValueNumKeys ConstantValueNumKeys;

	/** Compressed keys, used only outside of the editor */
	TArray<uint8> CompressedKeys;

	FCompressedRichCurve()
		: CompressionFormat(RCCF_Empty)
		, KeyTimeCompressionFormat(RCKTCF_float32)
		, PreInfinityExtrap(RCCE_None)
		, PostInfinityExtrap(RCCE_None)
		, ConstantValueNumKeys()
		, CompressedKeys()
	{}

	/** Evaluate this rich curve at the specified time */
	ENGINE_API float Eval(float InTime, float InDefaultValue = 0.0f) const;

	/** Populate RichCurve with decompressed key-data */
	ENGINE_API void PopulateCurve(FRichCurve& OutCurve) const;

	// Encapsulates the StaticEval arguments
	struct FStaticEvalArgs
	{
		ERichCurveCompressionFormat CompressionFormat = RCCF_Empty;
		ERichCurveKeyTimeCompressionFormat KeyTimeCompressionFormat = RCKTCF_float32;

		ERichCurveExtrapolation PreInfinityExtrap = RCCE_None;
		ERichCurveExtrapolation PostInfinityExtrap = RCCE_None;

		TConstantValueNumKeys ConstantValueNumKeys;

		const uint8* CompressedKeys = nullptr;

		float Time = 0.0f;

		float DefaultValue = 0.0f;
	};

	/** Evaluate this rich curve at the specified time */
	static ENGINE_API float StaticEval(const FStaticEvalArgs& Args);

	/** Evaluate this rich curve at the specified time */
	UE_DEPRECATED(5.6, "Please use StaticEval with the packed arguments structure as it has improved performance")
	static ENGINE_API float StaticEval(ERichCurveCompressionFormat CompressionFormat, ERichCurveKeyTimeCompressionFormat KeyTimeCompressionFormat, ERichCurveExtrapolation PreInfinityExtrap, ERichCurveExtrapolation PostInfinityExtrap, TConstantValueNumKeys ConstantValueNumKeys, const uint8* CompressedKeys, float InTime, float InDefaultValue = 0.0f);

	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API bool operator==(const FCompressedRichCurve& Other) const;
	bool operator!=(const FCompressedRichCurve& Other) const { return !(*this == Other); }

	friend FArchive& operator<<(FArchive& Ar, FCompressedRichCurve& Curve)
	{
		Curve.Serialize(Ar);
		return Ar;
	}
};

/*
 * Override serialization for compressed rich curves to handle the union
 */
template<>
struct TStructOpsTypeTraits<FCompressedRichCurve>
	: public TStructOpsTypeTraitsBase2<FCompressedRichCurve>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/**
 * Info about a curve to be edited.
 */
template<class T>
struct FRichCurveEditInfoTemplate
{
	/** Name of curve, used when displaying in editor. Can include comma's to allow tree expansion in editor */
	FName CurveName;

	/** Pointer to curves to be edited */
	T CurveToEdit;

	FRichCurveEditInfoTemplate()
		: CurveName(NAME_None)
		, CurveToEdit(nullptr)
	{ }

	FRichCurveEditInfoTemplate(T InCurveToEdit)
		: CurveName(NAME_None)
		, CurveToEdit(InCurveToEdit)
	{ }

	FRichCurveEditInfoTemplate(T InCurveToEdit, FName InCurveName)
		: CurveName(InCurveName)
		, CurveToEdit(InCurveToEdit)
	{ }

	inline bool operator==(const FRichCurveEditInfoTemplate<T>& Other) const
	{
		return Other.CurveName.IsEqual(CurveName) && Other.CurveToEdit == CurveToEdit;
	}

	uint32 GetTypeHash() const
	{
		return HashCombine(GetTypeHashHelper(CurveName), PointerHash(CurveToEdit));
	}

	friend inline uint32 GetTypeHash(const FRichCurveEditInfoTemplate<T>& RichCurveEditInfo)
	{
		return RichCurveEditInfo.GetTypeHash();
	}
};


// Rename and deprecate
typedef FRichCurveEditInfoTemplate<FRealCurve*>			FRichCurveEditInfo;
typedef FRichCurveEditInfoTemplate<const FRealCurve*>	FRichCurveEditInfoConst;
