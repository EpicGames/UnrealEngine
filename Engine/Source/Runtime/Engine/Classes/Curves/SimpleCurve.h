// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/RealCurve.h"
#include "SimpleCurve.generated.h"

/** One key in a rich, editable float curve */
USTRUCT(BlueprintType)
struct FSimpleCurveKey
{
	GENERATED_USTRUCT_BODY()

	/** Time at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	float Time;

	/** Value at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	float Value;

	FSimpleCurveKey()
		: Time(0.f)
		, Value(0.f)
	{ }

	FSimpleCurveKey(float InTime, float InValue)
		: Time(InTime)
		, Value(InValue)
	{ }

	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API bool operator==(const FSimpleCurveKey& Other) const;
	ENGINE_API bool operator!=(const FSimpleCurveKey& Other) const;

	friend FArchive& operator<<(FArchive& Ar, FSimpleCurveKey& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};


template<>
struct TIsPODType<FSimpleCurveKey>
{
	enum { Value = true };
};


template<>
struct TStructOpsTypeTraits<FSimpleCurveKey>
	: public TStructOpsTypeTraitsBase2<FSimpleCurveKey>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


/** A rich, editable float curve */
USTRUCT()
struct FSimpleCurve
	: public FRealCurve
{
	GENERATED_USTRUCT_BODY()

	FSimpleCurve() 
		: FRealCurve()
		, InterpMode(RCIM_Linear)
	{ }

public:

	/** Gets a copy of the keys, so indices and handles can't be meddled with */
	ENGINE_API TArray<FSimpleCurveKey> GetCopyOfKeys() const;

	/** Gets a const reference of the keys, so indices and handles can't be meddled with */
	ENGINE_API const TArray<FSimpleCurveKey>& GetConstRefOfKeys() const;

	/** Const iterator for the keys, so the indices and handles stay valid */
	ENGINE_API TArray<FSimpleCurveKey>::TConstIterator GetKeyIterator() const;
	
	/** Functions for getting keys based on handles */
	ENGINE_API FSimpleCurveKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API FSimpleCurveKey GetKey(FKeyHandle KeyHandle) const;
	
	/** Quick accessors for the first and last keys */
	ENGINE_API FSimpleCurveKey GetFirstKey() const;
	ENGINE_API FSimpleCurveKey GetLastKey() const;

	/** Get the first key that matches any of the given key handles. */
	ENGINE_API FSimpleCurveKey* GetFirstMatchingKey(const TArray<FKeyHandle>& KeyHandles);

	/**
	  * Add a new key to the curve with the supplied Time and Value. Returns the handle of the new key.
	  * 
	  * @param	bUnwindRotation		When true, the value will be treated like a rotation value in degrees, and will automatically be unwound to prevent flipping 360 degrees from the previous key 
	  * @param  KeyHandle			Optionally can specify what handle this new key should have, otherwise, it'll make a new one
	  */
	ENGINE_API FKeyHandle AddKey(float InTime, float InValue, const bool bUnwindRotation = false, FKeyHandle KeyHandle = FKeyHandle()) final override;

	/**
	 * Sets the keys with the keys.
	 *
	 * Expects that the keys are already sorted.
	 *
	 * @see AddKey, DeleteKey
	 */
	ENGINE_API void SetKeys(const TArray<FSimpleCurveKey>& InKeys);

	/**
	 *  Remove the specified key from the curve.
	 *
	 * @param KeyHandle The handle of the key to remove.
	 * @see AddKey, SetKeys
	 */
	ENGINE_API virtual void DeleteKey(FKeyHandle KeyHandle) final override;

	/** Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time */
	ENGINE_API virtual FKeyHandle UpdateOrAddKey(float InTime, float InValue, const bool bUnwindRotation = false, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) final override;

	/** Move a key to a new time. */
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) final override;

	/** Get the time for the Key with the specified index. */
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const final override;

	/** Set the value of the specified key */
	ENGINE_API virtual void SetKeyValue(FKeyHandle KeyHandle, float NewValue, bool bAutoSetTangents = true) final override;

	/** Returns the value of the specified key */
	ENGINE_API virtual float GetKeyValue(FKeyHandle KeyHandle) const final override;

	/** Returns a <Time, Value> pair for the specified key */
	ENGINE_API virtual TPair<float, float> GetKeyTimeValuePair(FKeyHandle KeyHandle) const final override;

	/** Set the interp mode used for keys in this curve */
	virtual void SetKeyInterpMode(FKeyHandle, ERichCurveInterpMode NewInterpMode) final override { SetKeyInterpMode(NewInterpMode); }

	void SetKeyInterpMode(ERichCurveInterpMode NewInterpMode)
	{ 
		if (ensureMsgf(NewInterpMode != RCIM_Cubic, TEXT("SimpleCurves cannot use cubic interpolation")))
		{
			InterpMode = NewInterpMode;
		}
	}

	/** Get the interp mode of the specified key */
	virtual ERichCurveInterpMode GetKeyInterpMode(FKeyHandle KeyHandle) const final override { return GetKeyInterpMode(); }

	/** Get the interp mode used for keys in this curve */
	ERichCurveInterpMode GetKeyInterpMode() const { return InterpMode; }

	/** Get range of input time values. Outside this region curve continues constantly the start/end values. */
	ENGINE_API virtual void GetTimeRange(float& MinTime, float& MaxTime) const final override;

	/** Get range of output values. */
	ENGINE_API virtual void GetValueRange(float& MinValue, float& MaxValue) const final override;

	/** Clear all keys. */
	ENGINE_API virtual void Reset() final override;

	/** Remap InTime based on pre and post infinity extrapolation values */
	ENGINE_API virtual void RemapTimeValue(float& InTime, float& CycleValueOffset) const final override;

	/** Evaluate this curve at the specified time */
	ENGINE_API virtual float Eval(float InTime, float InDefaultValue = 0.0f) const final override;

	/** Resize curve length to the [MinTimeRange, MaxTimeRange] */
	ENGINE_API virtual void ReadjustTimeRange(float NewMinTimeRange, float NewMaxTimeRange, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime) final override;

	/** Determine if two SimpleCurves are the same */
	ENGINE_API bool operator == (const FSimpleCurve& Curve) const;

	/** Bake curve given the sample rate */
	ENGINE_API virtual void BakeCurve(float SampleRate) final override;
	ENGINE_API virtual void BakeCurve(float SampleRate, float FirstKeyTime, float LastKeyTime) final override;

	/** Remove redundant keys, comparing against Tolerance */
	ENGINE_API virtual void RemoveRedundantKeys(float Tolerance, FFrameRate SampleRate = FFrameRate(0,0)) final override;
	ENGINE_API virtual void RemoveRedundantKeys(float Tolerance, float FirstKeyTime, float LastKeyTime, FFrameRate SampleRate = FFrameRate(0,0)) final override;

	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const final { return new FSimpleCurve(*this); }

protected:
	ENGINE_API virtual int32 GetKeyIndex(float KeyTime, float KeyTimeTolerance) const override final;

private:
	ENGINE_API void RemoveRedundantKeysInternal(float Tolerance, int32 InStartKeepKey, int32 InEndKeepKey);

	ENGINE_API float EvalForTwoKeys(const FSimpleCurveKey& Key1, const FSimpleCurveKey& Key2, const float InTime) const;

public:

	// FIndexedCurve interface

	virtual int32 GetNumKeys() const final override { return Keys.Num(); }

public:

	/** Interpolation mode between this key and the next */
	UPROPERTY()
	TEnumAsByte<ERichCurveInterpMode> InterpMode;

	/** Sorted array of keys */
	UPROPERTY(EditAnywhere, EditFixedSize, Category="Curve", meta=(EditFixedOrder))
	TArray<FSimpleCurveKey> Keys;
};
