// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Splines/SplineInterfaces.h"
#include "Curves/Splines/TangentSpline.h"	// only for FLegacyTangentSpline, move that to its own header.

#include "Spline.generated.h"

struct UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") FSpline;

namespace UE::Spline
{
enum class ESplineType : uint8
{
	Unimplemented = 0,
	LegacyTangent = 1,
	Tangent = 2,
};

template<ESplineType Impl>
struct TSplineTypeMap;

template<>
struct TSplineTypeMap<ESplineType::LegacyTangent>
{
	using Type = FLegacyTangentSpline;
};

template<>
struct TSplineTypeMap<ESplineType::Tangent>
{
	using Type = FTangentSpline;
};
} // UE::Spline

/**
 * A general purpose, reflected spline.
 * The implementation can be configured at runtime.
 */
USTRUCT()
struct FSpline
{
	GENERATED_BODY()

	ENGINE_API FSpline();
	ENGINE_API explicit FSpline(UE::Spline::ESplineType InType);

	ENGINE_API FSpline(const FSpline& Other);
	ENGINE_API FSpline& operator=(const FSpline& Other);

	FSpline(FSpline&& Other) = default;
	FSpline& operator=(FSpline&& Other) = default;

	ENGINE_API FSpline& operator=(const FSplineCurves& Other);

	ENGINE_API FVector Evaluate(float Param) const;

	ENGINE_API FVector EvaluateDerivative(float Param) const;

	ENGINE_API UE::Geometry::FInterval1f GetParameterSpace() const;

	ENGINE_API int32 GetNumberOfSegments() const;

	/** Reset the spline to an empty spline. Preserves the current implementation. */
	ENGINE_API void Reset();

	/**
	 * Fetch the underlying concrete spline implementation.
	 *
	 * Pointers returned by Get<Impl>() are invalidated by:
	 * - Serialize(FArchive&) when Ar.IsLoading() is true
	 * - ImportTextItem(...)
	 * - operator=(...)
	 * - SetCurrentImplementation(...)
	 * - destruction of this FSpline
	 *
	 * @return A valid pointer if Impl == GetCurrentImplementation(), otherwise nullptr.
	 */
	template<UE::Spline::ESplineType Impl>
	typename UE::Spline::TSplineTypeMap<Impl>::Type* Get();

	/** A const version of Get<Impl> */
	template<UE::Spline::ESplineType Impl>
	const typename UE::Spline::TSplineTypeMap<Impl>::Type* Get() const;
		
	ENGINE_API bool operator==(const FSpline& Other) const;
	ENGINE_API bool operator!=(const FSpline& Other) const;
	
	friend FArchive& operator<<(FArchive& Ar, FSpline& Spline)
	{ 
		Spline.Serialize(Ar);
		return Ar;
	}

	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API bool ExportTextItem(FString& ValueStr, FSpline const& DefaultValue, class UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const;
	ENGINE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	ENGINE_API UE::Spline::ESplineType GetCurrentImplementation() const;

#if WITH_EDITOR
	/** Returns the implementation of this spline when when most recently loaded. */
	ENGINE_API UE::Spline::ESplineType GetPreviousImplementation() const;
#endif

	/** Returns the default implementation for new spline. */
	ENGINE_API static UE::Spline::ESplineType GetDefaultImplementation();

private:
	
	/** Sets the current implementation and performs any necessary allocation. */
	void SetCurrentImplementation(UE::Spline::ESplineType Implementation);

	void SerializeLoad(FArchive& Ar);
	void SerializeSave(FArchive& Ar) const;

private:

	UE::Spline::ESplineType CurrentImplementation;

#if WITH_EDITOR
	UE::Spline::ESplineType PreviousImplementation;
#endif

	// Valid when CurrentImplementation is ESplineType::LegacyTangent
	TUniquePtr<FLegacyTangentSpline> LegacySplineImpl;

	// Valid when CurrentImplementation is not ESplineType::Unimplemented and not ESplineType::LegacyTangent;
	TUniquePtr<UE::Geometry::Spline::TSplineInterface<FVector>> SplineImpl;
};

template<UE::Spline::ESplineType Impl>
typename UE::Spline::TSplineTypeMap<Impl>::Type* FSpline::Get()
{
	if (Impl != CurrentImplementation)
	{
		return nullptr;
	}

	if constexpr (Impl == UE::Spline::ESplineType::LegacyTangent)
	{
		return LegacySplineImpl.Get();
	}
	else
	{
		return static_cast<typename UE::Spline::TSplineTypeMap<Impl>::Type*>(SplineImpl.Get());
	}
}

template<UE::Spline::ESplineType Impl>
const typename UE::Spline::TSplineTypeMap<Impl>::Type* FSpline::Get() const
{
	if (Impl != CurrentImplementation)
	{
		return nullptr;
	}

	if constexpr (Impl == UE::Spline::ESplineType::LegacyTangent)
	{
		return LegacySplineImpl.Get();
	}
	else
	{
		return static_cast<const typename UE::Spline::TSplineTypeMap<Impl>::Type*>(SplineImpl.Get());
	}
}

template<>
struct TStructOpsTypeTraits<FSpline> : public TStructOpsTypeTraitsBase2<FSpline>
{
	enum
	{
		WithSerializer				= true, // Enables the use of a custom Serialize method.
		WithIdenticalViaEquality	= true, // Enables the use of a custom equality operator.
		WithExportTextItem			= true, // Enables the use of a custom ExportTextItem method.
		WithImportTextItem			= true, // Enables the use of a custom ImportTextItem method.
	};
};
