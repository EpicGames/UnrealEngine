// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TransactionallySafeMutex.h"
#include "BoxTypes.h"
#include "CoreMinimal.h"
#include "Math/InterpCurve.h"
#include "Misc/TransactionallySafeRWLock.h"

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
#include "Splines/TangentBezierSpline.h"
#include "Splines/MultiSpline.h"
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

#define UE_API ENGINE_API

class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") FTangentSpline;
class UE_INTERNAL FLegacyTangentSpline;

struct FSplineCurves;
struct FSplinePoint;

/**
 * FTangentSpline Definition
 *
 * A spline that provides tangent-based control over curve shape while using
 * piecewise Bezier curves internally for evaluation. Supports both manual
 * tangent control and automatic tangent computation.
 */

class FTangentSpline : public UE::Geometry::Spline::TMultiSpline<UE::Geometry::Spline::FTangentBezierSpline3d>
{
	using FTangentBezierControlPoint = UE::Geometry::Spline::TTangentBezierControlPoint<FVector3d>;

public:

	UE_API FTangentSpline();
	UE_API FTangentSpline(const FTangentSpline& Other);
	UE_API FTangentSpline(const FLegacyTangentSpline& Other);
	UE_API FTangentSpline(const FSplineCurves& Other);
	virtual ~FTangentSpline() override = default;
	UE_API FTangentSpline& operator=(const FTangentSpline& Other);

	UE_API bool operator!=(const FTangentSpline& Other) const;

	UE_API bool IsNearlyEqual(const FTangentSpline& Other, const float RelativeTolerance, const float AbsoluteTolerance) const;

	UE_API virtual bool Serialize(FArchive& Ar) override;
	friend FArchive& operator<<(FArchive& Ar, FTangentSpline& Spline)
	{
		Spline.Serialize(Ar);
		return Ar;
	}

	UE_API float GetSegmentLength(const int32 Index, const float Param, const FVector& Scale3D = FVector(1.0f)) const;
	UE_API float GetSplineLength() const;

	UE_API virtual void SetClosedLoop(bool bInClosedLoop) override;
	UE_API void SetClosedLoop(bool bInClosedLoop, bool bUpdateSpline);

	UE_API void Reset();
	UE_API void ResetRotation();
	UE_API void ResetScale();

	UE_API void AddPoint(const FSplinePoint& Point);
	UE_API void InsertPoint(const FSplinePoint& Point, int32 Index);
	UE_API FSplinePoint GetPoint(int32 Index) const;
	UE_API void RemovePoint(int32 Index);
	UE_API int32 GetNumberOfPoints() const;

	UE_API void SetLocation(int32 Index, const FVector& InLocation);
	UE_API FVector GetLocation(const int32 Index) const;

	UE_API void SetInTangent(const int32 Index, const FVector& InTangent);
	UE_API FVector GetInTangent(const int32 Index) const;

	UE_API void SetOutTangent(const int32 Index, const FVector& OutTangent);
	UE_API FVector GetOutTangent(const int32 Index) const;

	UE_API void SetRotation(int32 Index, const FQuat& InRotation);
	UE_API FQuat GetRotation(const int32 Index) const;

	UE_API void SetScale(int32 Index, const FVector& InScale);
	UE_API FVector GetScale(const int32 Index) const;

	UE_API void SetSplinePointType(int32 Index, EInterpCurveMode Type);
	UE_API EInterpCurveMode GetSplinePointType(int32 Index) const;

	// Sets the parameters of all spline points according to the default policy (defined by the CVar 'TangentSpline.ParameterizationPolicy')
	UE_API void Reparameterize();

	// Sets the parameters of all spline points according to the specified policy.
	UE_API void Reparameterize(UE::Geometry::Spline::EParameterizationPolicy ParameterizationPolicy);

	/**
	 * Sets the parameter of a spline point.
	 * Currently, this function disallows knot vector reordering, meaning the provided parameter
	 * must not be greater than the parameter at Index + 1 or less than the parameter at Index - 1.
	 * 
	 * @param Index The index of the point to update.
	 * @param Parameter The new parameter.
	 * @return true if the operation was successful, otherwise false.
	 */
	UE_API bool SetParameterAtIndex(int32 Index, float Parameter);

	/**
	 * Sets the parameters of all spline points.
	 * InParameters must have exactly as many points as this spline.
	 *
	 * @param InParameters The list of new parameter values, 1:1 with the number of spline points.
	 * @return true if the operation was successful, otherwise false.
	 */
	UE_API bool SetParameters(const TArray<float>& InParameters);

	UE_API float GetParameterAtIndex(int32 Index) const;
	UE_API float GetParameterAtDistance(float Distance) const;
	UE_API float GetDistanceAtParameter(float Parameter) const;
	UE_API int32 GetIndexAtParameter(float Parameter) const;

	UE_API FQuat GetOrientation(int32 Index) const;
	UE_API FQuat GetOrientation(float Param) const;
	UE_API void SetOrientation(int32 Index, const FQuat& InOrientation);

	UE_API virtual float FindNearest(const FVector& InLocation, float& OutSquaredDistance) const override;
	UE_API float FindNearestOnSegment(const FVector& InLocation, int32 SegmentIndex, float& OutSquaredDistance) const;

	/** Parameter based evaluation API. */
	UE_API FVector EvaluatePosition(float Parameter) const;	// Not named Evaluate because we would be shadowing non-virtual base, this maps parameters
	UE_API FVector EvaluateDerivative(float Parameter) const;
	UE_API FQuat EvaluateRotation(float Parameter) const;
	UE_API FVector EvaluateScale(float Parameter) const;

	template <typename AttrType> AttrType EvaluateAttribute(const FName& Name, float Parameter) const;

	// Templated Channel Creation/Query functions
	template <typename AttrType> int32 NumAttributeValues(FName Name) const;

	// Templated Attribute Interaction by Index functions
	template <typename AttrType> AttrType GetAttributeValue(const FName& Name, int32 Index) const;
	template <typename AttrType> void SetAttributeValue(FName Name, const AttrType& Value, int32 Index);
	template <typename AttrType> void RemoveAttributeValue(FName Name, int32 Index);
	template <typename AttrType> int32 SetAttributeParameter(const FName& Name, int32 Index, float ParentSpaceParameter);
	template <typename AttrType> float GetAttributeParameter(const FName& Name, int32 Index) const;

	// Templated Attribute Interaction by Parameter functions
	template <typename AttrType> int32 AddAttributeValue(FName Name, const AttrType& Value, float Parameter);

	struct FUpdateSplineParams
	{
		bool bClosedLoop = false;
		bool bStationaryEndpoints = false;
		int32 ReparamStepsPerSegment = 10;
		bool bLoopPositionOverride = false;
		float LoopPosition = 0.0f;
		FVector Scale3D = FVector(1.0f);
	};

	/** Updates the spline using the current configuration. */
	UE_API void UpdateSpline();

	/** Updates the spline configuration, then updates the spline. */
	UE_API void UpdateSpline(const FUpdateSplineParams& Params);

	const FInterpCurveVector& GetSplinePointsPosition() const { return PositionCurve; }
	const FInterpCurveQuat& GetSplinePointsRotation() const { return RotationCurve; }
	const FInterpCurveVector& GetSplinePointsScale() const { return ScaleCurve; }

	uint32 GetVersion() const { return Version; }
	void SetVersion(uint32 InVersion) { Version = InVersion; }

private:

	/** Inspired by FDynamicMesh3::FChangeStamp. */
	struct FChangeStamp
	{
		FChangeStamp()
		{
		}

		UE_NONCOPYABLE(FChangeStamp);

		/** Updates the change stamp in an thread-safe, transactionally-safe way. */
		void Increment()
		{
			Mutex.Lock();
			++Value;
			Mutex.Unlock();
		}

		/** Returns the current change value in a thread-safe, transactionally-safe way. */
		uint32 GetValue() const
		{
			Mutex.Lock();
			uint32 Result = Value;
			Mutex.Unlock();

			return Result;
		}

	private:

		/** Guards `Value`. */
		mutable UE::FTransactionallySafeMutex Mutex;

		/** The change stamp is incremented when modifications occur. It's guarded by `Mutex`. */
		uint32 Value = 1;
	};

	mutable FInterpCurveFloat ReparamTable;
	FChangeStamp ReparamTableNextVersion;
	mutable uint32 ReparamTableVersion = 0;
	mutable FTransactionallySafeRWLock ReparamTableRWLock;

	static inline const FName RotationAttrName = FName("Rotation");
	static inline const FName ScaleAttrName = FName("Scale");

	/** Legacy Curves: */
	mutable FInterpCurveVector PositionCurve;
	mutable FInterpCurveQuat RotationCurve;
	mutable FInterpCurveVector ScaleCurve;
	FChangeStamp LegacyCurvesNextVersion;
	mutable uint32 LegacyCurvesVersion = 0;

	int32 ReparamStepsPerSegment = 10;

	FUpdateSplineParams CachedUpdateSplineParams;

	uint32 Version = 0xffffffff;

private:

	void UpdateTangents();

	void MarkReparamTableDirty();

	/** Clears and repopulates reparameterization attribute channels. */
	void UpdateReparamTable() const;

	void MarkLegacyCurvesDirty();

	/** Clears and repopulates PositionCurve, RotationCurve, and ScaleCurve */
	void RebuildLegacyCurves() const;

	/** Converts FSplinePoint to FTangentBezierControlPoint */
	FTangentBezierControlPoint ConvertToTangentBezierControlPoint(const FSplinePoint& Point) const;

	/**
	 * Updates the attributes of a point in the spline
	 * @param Point
	 * @param PointIndex
	 */
	void UpdatePointAttributes(const FSplinePoint& Point, int32 PointIndex);

	/**
	 * Converts an index (possibly fractional) to a parameter value in internal spline space.
	 */
	float ConvertIndexToInternalParameter(int32 Index, float Fraction = 0.0f) const;

	/**
	 * Converts an internal spline space parameter value to the closest index
	 */
	int32 ConvertInternalParameterToNearestPointIndex(float Parameter) const;

	void ValidateRotScale() const;

	// Helper functions for keeping attribute channels stable when reparameterizing.
	template <typename AttrType> TArray<AttrType> CacheChannelKnots(const FName& InName) const;
	template <typename AttrType> void RestoreChannelKnots(const FName& InName, const TArray<AttrType>& InKnots);
};

template <typename AttrType>
AttrType FTangentSpline::EvaluateAttribute(const FName& Name, float Parameter) const
{
	return UE::Geometry::Spline::TMultiSpline<UE::Geometry::Spline::FTangentBezierSpline3d>::EvaluateAttribute<AttrType>(Name, Parameter);
}

template <typename AttrType>
int32 FTangentSpline::AddAttributeValue(FName Name, const AttrType& Value, float Parameter)
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;
	using ControlPointType = UE::Geometry::Spline::TTangentBezierControlPoint<AttrType>;
	using UE::Geometry::FInterval1f;
	
	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		ON_SCOPE_EXIT
		{
			Channel->UpdateTangents();
		};
		ControlPointType ControlPoint = ControlPointType(Value);
		int32 NumPoints = Channel->GetNumPoints();

		float ParentSpaceParameter = Parameter;
		FInterval1f MappedChildSpace = GetMappedChildSpace(Name);

		// Special case for empty spline
		if (NumPoints == 0)
		{
			Channel->AppendPoint(ControlPoint, 0.f);
			SetAttributeChannelRange(Name, FInterval1f(ParentSpaceParameter, ParentSpaceParameter), EMappingRangeSpace::Parent);
			return 0;
		}

		if (NumPoints == 1)
		{
			if (ParentSpaceParameter > MappedChildSpace.Min)
			{
				Channel->AppendPoint(ControlPoint, UE::Geometry::Spline::EParameterizationPolicy::Uniform);
				SetAttributeChannelRange(Name, FInterval1f(MappedChildSpace.Min, ParentSpaceParameter), EMappingRangeSpace::Parent);
				return 1;
			}
			else
			{
				Channel->PrependPoint(ControlPoint, UE::Geometry::Spline::EParameterizationPolicy::Uniform);
				SetAttributeChannelRange(Name, FInterval1f(ParentSpaceParameter, MappedChildSpace.Min), EMappingRangeSpace::Parent);
				return 0;
			}
		}

		// append case
		if (ParentSpaceParameter > MappedChildSpace.Max)
		{
			// It is important to compute ChildSpaceParameter before the AppendPoint call, otherwise it will not be correct.
			float ChildSpaceParameter = MapParameterToChildSpace(Name, ParentSpaceParameter);
			Channel->AppendPoint(ControlPoint, ChildSpaceParameter);

			// By growing the child space and the mapped parent range proportionally, we keep the internal points stable in parent space.
			SetAttributeChannelRange(Name, FInterval1f(MappedChildSpace.Min, ParentSpaceParameter), EMappingRangeSpace::Parent);

			return Channel->GetNumPoints() - 1;
		}

		// prepend case
		if (ParentSpaceParameter < MappedChildSpace.Min)
		{
			// It is important to compute ChildSpaceParameter before the AppendPoint call, otherwise it will not be correct.
			float ChildSpaceParameter = MapParameterToChildSpace(Name, ParentSpaceParameter);
			Channel->PrependPoint(ControlPoint, ChildSpaceParameter);

			// By growing the child space and the mapped parent range proportionally, we keep the internal points stable in parent space.
			SetAttributeChannelRange(Name, FInterval1f(ParentSpaceParameter, MappedChildSpace.Max), EMappingRangeSpace::Parent);

			return 0;
		}

		float ChildSpaceParameter = MapParameterToChildSpace(Name, ParentSpaceParameter);
		int NewPointIndex = Channel->InsertPoint(ControlPoint, ChildSpaceParameter);
		return NewPointIndex;
	}

	return INDEX_NONE;
}

template <typename AttrType>
void FTangentSpline::SetAttributeValue(FName Name, const AttrType& Value, int32 Index)
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		Channel->SetValue(Index, Value);
		Channel->UpdateTangents();
	}
}

template <typename AttrType>
void FTangentSpline::RemoveAttributeValue(FName Name, int32 Index)
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;
	using UE::Geometry::FInterval1f;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		if (Channel->GetNumPoints() > 1)
		{
			if (Index == 0)
			{
				FInterval1f NewMappingRange = GetMappedChildSpace(Name);
				NewMappingRange.Min = GetAttributeParameter<AttrType>(Name, Index + 1);
				SetAttributeChannelRange(Name, NewMappingRange, EMappingRangeSpace::Parent);
			}
			else if (Index == Channel->GetNumPoints() - 1)
			{
				FInterval1f NewMappingRange = GetMappedChildSpace(Name);
				NewMappingRange.Max = GetAttributeParameter<AttrType>(Name, Index - 1);
				SetAttributeChannelRange(Name, NewMappingRange, EMappingRangeSpace::Parent);
			}
		}

		Channel->RemovePoint(Index);
	}
}

template <typename AttrType>
AttrType FTangentSpline::GetAttributeValue(const FName& Name, int32 Index) const
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		return Channel->GetValue(Index);
	}

	return AttrType();
}

template <typename AttrType>
int32 FTangentSpline::NumAttributeValues(FName Name) const
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		return Channel->GetNumPoints();
	}

	return 0;
}

template <typename AttrType>
int32 FTangentSpline::SetAttributeParameter(const FName& Name, int32 Index, float Parameter)
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;
	using UE::Geometry::FInterval1f;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		ON_SCOPE_EXIT
		{
			Channel->UpdateTangents();
		};
		float ParentSpaceParameter = Parameter;

		// Prevent collapse of the channel space and keep it starting at 0.
		auto SanitizeChannelSpace = [Channel]()
		{
			const FInterval1f ChannelSpace = Channel->GetParameterSpace();

			if (!ensureAlwaysMsgf(ChannelSpace.Min < ChannelSpace.Max, TEXT("Degenerate attribute channel parameter space detected, cannot sanitize.")))
			{
				return;
			}

			const float KnotOffset = -1.f * ChannelSpace.Min;
			const float KnotScaleFactor = 1.f / (ChannelSpace.Max - ChannelSpace.Min);
			TArray<UE::Geometry::Spline::FKnot> Knots = Channel->GetKnotVector();
			for (UE::Geometry::Spline::FKnot& Knot : Knots)
			{
				Knot.Value += KnotOffset;
				Knot.Value *= KnotScaleFactor;
			}
			Channel->SetKnotVector(Knots);
		};

		// Helper to prevent code duplication.
		// Uses current value of Index and ParentSpaceParameter at the time of calling to update the attribute parameter.
		auto SetParentSpaceParameter = [this, &ParentSpaceParameter, Name, Channel, &Index]() -> int32
			{
				const float CurrentChildSpaceParameter = MapParameterToChildSpace(Name, GetAttributeParameter<AttrType>(Name, Index));
				const float DesiredChildSpaceParameter = MapParameterToChildSpace(Name, ParentSpaceParameter);

				constexpr float MinStep = 2.f * UE_KINDA_SMALL_NUMBER;
				float NewChildSpaceParameter = DesiredChildSpaceParameter > CurrentChildSpaceParameter
					? FMath::Max(DesiredChildSpaceParameter, CurrentChildSpaceParameter + MinStep)
					: FMath::Min(DesiredChildSpaceParameter, CurrentChildSpaceParameter - MinStep);

				return Channel->SetParameter(Index, NewChildSpaceParameter);
			};

		/**
		 * Cases we handle below:
		 * 1) Moving the only existing attribute.
		 * 2) Moving an endpoint for a 2 point channel.
		 * 3) Moving the first endpoint for a 3+ point channel.
		 * 4) Moving the last endpoint for a 3+ point channel.
		 * 5) Moving an internal point.
		 */

		if (Channel->GetNumPoints() == 1)						// Case 1: Moving the only attribute
		{
			SetAttributeChannelRange(Name, FInterval1f(ParentSpaceParameter, ParentSpaceParameter), EMappingRangeSpace::Parent);
			return Index;
		}
		else if (Channel->GetNumPoints() == 2)					// Case 2: Moving an end point while no internal points exist.
		{
			FInterval1f MappedRange = GetMappedChildSpace(Name);
			float& RangeBound = Index == 0 ? MappedRange.Min : MappedRange.Max;
			RangeBound = ParentSpaceParameter;

			if (MappedRange.Min > MappedRange.Max)
			{
				// The mapping range will flip, swap end points and un-flip.
				Channel->SetParameter(1, Channel->GetParameter(0) - 1.f);	// It doesn't actually matter what we do here, as long as we get the ordering & mapping range correct.
				SetAttributeChannelRange(Name, FInterval1f(MappedRange.Max, MappedRange.Min), EMappingRangeSpace::Parent);
				Index = Index == 0 ? 1 : 0;
			}
			else
			{
				SetAttributeChannelRange(Name, MappedRange, EMappingRangeSpace::Parent);
			}

			return Index;
		}
		else if (Index == 0)									// Case 3: Moving the first point (which has exactly 1 neighbor)
		{
			// Save internal attribute point parameters for later re-parameterization after changing the child space mapping.
			TArray<float> InternalParameters = CacheChannelKnots<float>(Name);

			float ParentSpaceUpperBound = GetMappedChildSpace(Name).Max;
			const float NeighborParentSpaceParameter = GetAttributeParameter<AttrType>(Name, Index + 1);
			const bool bAttributesWillReorder = ParentSpaceParameter > NeighborParentSpaceParameter;

			// If the end point is passing its neighbor, we need to actually reshuffle the points.
			if (bAttributesWillReorder)
			{
				Index = SetParentSpaceParameter();

				// Shift values left up to the last invalidated index.
				const int32 InvalidatedInternalParameterIdx = FMath::Clamp(Index - 1, 0, InternalParameters.Num() - 1);
				for (int InternalParameterIdx = 1; InternalParameterIdx <= InvalidatedInternalParameterIdx; ++InternalParameterIdx)
				{
					InternalParameters[InternalParameterIdx - 1] = InternalParameters[InternalParameterIdx];
				}

				// Do one of 2 things to the actual invalidated index:
				if (Index == Channel->GetNumPoints() - 1)
				{
					// Old upper end point is now an internal point and needs to be re-parameterized.
					InternalParameters[InvalidatedInternalParameterIdx] = ParentSpaceUpperBound;
				}
				else
				{
					// Old lower end point is now an internal point and needs to be re-parameterized.
					InternalParameters[InvalidatedInternalParameterIdx] = ParentSpaceParameter;
				}
			}

			// Updates the lower bound of the mapping range.
			ParentSpaceUpperBound = Index == Channel->GetNumPoints() - 1 ? ParentSpaceParameter : ParentSpaceUpperBound;
			const float ParentSpaceLowerBound = bAttributesWillReorder ? NeighborParentSpaceParameter : ParentSpaceParameter;
			SetAttributeChannelRange(Name, FInterval1f(ParentSpaceLowerBound, ParentSpaceUpperBound), EMappingRangeSpace::Parent);

			// Re-parameterize the internal points so that they do not move in parent space after the mapping range changed
			RestoreChannelKnots(Name, InternalParameters);

			if (bAttributesWillReorder)
			{
				SanitizeChannelSpace();
			}

			return Index;
		}
		else if (Index == Channel->GetNumPoints() - 1)			// Case 4: Moving the last point (which has exactly 1 neighbor)
		{
			// Save internal attribute point parameters for later re-parameterization after changing the child space mapping.
			TArray<float> InternalParameters = CacheChannelKnots<float>(Name);

			float ParentSpaceLowerBound = GetMappedChildSpace(Name).Min;
			const float NeighborParentSpaceParameter = GetAttributeParameter<AttrType>(Name, Index - 1);
			const bool bAttributesWillReorder = ParentSpaceParameter < NeighborParentSpaceParameter;

			// If the end point is passing its neighbor, we need to actually reshuffle the points.
			if (bAttributesWillReorder)
			{
				Index = SetParentSpaceParameter();

				// Shift values right up to the last invalidated index.
				const int32 InvalidatedInternalParameterIdx = FMath::Clamp(Index - 1, 0, InternalParameters.Num() - 1);
				for (int InternalParameterIdx = InternalParameters.Num() - 2; InternalParameterIdx >= InvalidatedInternalParameterIdx; --InternalParameterIdx)
				{
					InternalParameters[InternalParameterIdx + 1] = InternalParameters[InternalParameterIdx];
				}

				// Do one of 2 things to the actual invalidated index:
				if (Index == 0)
				{
					// Old upper end point is now an internal point and needs to be re-parameterized.
					InternalParameters[InvalidatedInternalParameterIdx] = ParentSpaceLowerBound;
				}
				else
				{
					// Old lower end point is now an internal point and needs to be re-parameterized.
					InternalParameters[InvalidatedInternalParameterIdx] = ParentSpaceParameter;
				}
			}

			// Update the child space mapping
			ParentSpaceLowerBound = Index == 0 ? ParentSpaceParameter : ParentSpaceLowerBound;
			float ParentSpaceUpperBound = bAttributesWillReorder ? NeighborParentSpaceParameter : ParentSpaceParameter; // Prevent over-shrinking if we reshuffle.
			SetAttributeChannelRange(Name, FInterval1f(ParentSpaceLowerBound, ParentSpaceUpperBound), EMappingRangeSpace::Parent);

			// Re-parameterize the internal points so that they do not move in parent space after the mapping range changed
			RestoreChannelKnots(Name, InternalParameters);

			if (bAttributesWillReorder)
			{
				SanitizeChannelSpace();
			}

			return Index;
		}
		else													// Case 5: Moving an internal point (which has exactly 2 neighbors)
		{
			FInterval1f MappedRange = GetMappedChildSpace(Name);
			Index = SetParentSpaceParameter();

			if (Index == 0)
			{
				// The internal point is now the lower end point
				MappedRange.Min = ParentSpaceParameter;
			}
			else if (Index == Channel->GetNumPoints() - 1)
			{
				// The internal point is now the upper end point
				MappedRange.Max = ParentSpaceParameter;
			}

			SetAttributeChannelRange(Name, MappedRange, EMappingRangeSpace::Parent);

			return Index;
		}
	}

	return INDEX_NONE;
}

template <typename AttrType>
float FTangentSpline::GetAttributeParameter(const FName& Name, int32 Index) const
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(Name))
	{
		return MapParameterFromChildSpace(Name, Channel->GetParameter(Index));
	}
	return 0.0f;
}

template <typename AttrType>
TArray<AttrType> FTangentSpline::CacheChannelKnots(const FName& InName) const
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	TArray<float> InternalParameters;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(InName))
	{
		for (int InternalIdx = 1; InternalIdx < Channel->GetNumPoints() - 1; ++InternalIdx)
		{
			InternalParameters.Add(MapParameterFromChildSpace(InName, Channel->GetParameter(InternalIdx)));
		}
	}

	return InternalParameters;
}

template <typename AttrType>
void FTangentSpline::RestoreChannelKnots(const FName& InName, const TArray<AttrType>& InKnots)
{
	using AttrSplineType = UE::Geometry::Spline::TTangentBezierSpline<AttrType>;

	if (InKnots.Num() == 0) return;

	if (AttrSplineType* Channel = GetTypedAttributeChannel<AttrSplineType>(InName))
	{
		// Provided knots are for internal points, bail out if we don't have the correct number of knots.
		if (ensureAlwaysMsgf(InKnots.Num() == Channel->GetNumPoints() - 2, TEXT("Invalid number of knots!")))
		{
			for (int InternalIdx = 1; InternalIdx < Channel->GetNumPoints() - 1; ++InternalIdx)
			{
				Channel->SetParameter(InternalIdx, MapParameterToChildSpace(InName, InKnots[InternalIdx - 1]));
			}
		}
	}
}

/** FLegacyTangentSpline Definition */

class FLegacyTangentSpline
{
public:

	FInterpCurveVector PositionCurve;
	FInterpCurveQuat RotationCurve;
	FInterpCurveVector ScaleCurve;
	FInterpCurveFloat ReparamTable;

	friend FArchive& operator<<(FArchive& Ar, FLegacyTangentSpline& Spline)
	{
		Spline.Serialize(Ar);
		return Ar;
	}

	UE_API bool Serialize(FArchive& Ar);
};

#undef UE_API