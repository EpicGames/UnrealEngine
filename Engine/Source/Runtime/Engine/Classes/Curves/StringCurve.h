// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "StringCurve.generated.h"

/**
 * One key in a curve of FStrings.
 */
USTRUCT()
struct FStringCurveKey
{
	GENERATED_USTRUCT_BODY()

	/** Time at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	float Time;

	/** Value at this key */
	UPROPERTY(EditAnywhere, Category="Key")
	FString Value;

	/** Default constructor. */
	FStringCurveKey()
		: Time(0.0f)
	{ }

	/** Creates and initializes a new instance. */
	FStringCurveKey(float InTime, const FString& InValue)
		: Time(InTime)
		, Value(InValue)
	{ }

public:

	//~ TStructOpsTypeTraits interface

	ENGINE_API bool operator==(const FStringCurveKey& Other) const;
	ENGINE_API bool operator!=(const FStringCurveKey& Other) const;
	ENGINE_API bool Serialize(FArchive& Ar);

	/** Serializes a name curve key from or into an archive. */
	friend FArchive& operator<<(FArchive& Ar, FStringCurveKey& Key)
	{
		Key.Serialize(Ar);
		return Ar;
	}
};


template<>
struct TStructOpsTypeTraits<FStringCurveKey>
	: public TStructOpsTypeTraitsBase2<FStringCurveKey>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


/**
 * Implements a curve of FStrings.
 */
USTRUCT()
struct FStringCurve
	: public FIndexedCurve
{
	GENERATED_USTRUCT_BODY()

	/** Virtual destructor. */
	virtual ~FStringCurve() = default;

public:

	/**
	 * Check whether this curve has any data or not
	 */
	bool HasAnyData() const
	{
		return DefaultValue.Len() || Keys.Num();
	}

	/**
	  * Add a new key to the curve with the supplied Time and Value. Returns the handle of the new key.
	  * 
	  * @param InTime The time at which to add the key.
	  * @param InValue The value of the key.
	  * @param KeyHandle Optional handle for the new key.
	  */
	ENGINE_API FKeyHandle AddKey(float InTime, const FString& InValue, FKeyHandle KeyHandle = FKeyHandle());

	/**
	 * Remove the specified key from the curve.
	 *
	 * @param KeyHandle Handle to the key to remove.
	 */
	ENGINE_API void DeleteKey(FKeyHandle KeyHandle);

	/**
	 * Evaluate the curve at the specified time.
	 *
	 * @param Time The time to evaluate.
	 * @param InDefaultValue The default value if no key exists.
	 * @return The evaluated string.
	 */
	ENGINE_API FString Eval(float Time, const FString& InDefaultValue) const;

	/**
	 * Finds a key a the specified time.
	 *
	 * @param KeyTime The time at which to find the key.
	 * @param KeyTimeTolerance The key time tolerance to use for equality.
	 * @return A handle to the key, or invalid key handle if not found.
	 */
	ENGINE_API FKeyHandle FindKey(float KeyTime, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Get the default value for the curve */
	FString GetDefaultValue() const
	{
		return DefaultValue;
	}

	/**
	 * Get a key.
	 *
	 * @param KeyHandle The handle of the key to get.
	 * @return The key.
	 */
	ENGINE_API FStringCurveKey& GetKey(FKeyHandle KeyHandle);
	ENGINE_API FStringCurveKey GetKey(FKeyHandle KeyHandle) const;

	/**
	 * Read-only access to the key collection.
	 *
	 * @return Collection of keys.
	 */
	const TArray<FStringCurveKey>& GetKeys() const
	{
		return Keys;
	}

	/**
	 * Get the time for the Key with the specified index.
	 *
	 * @param KeyHandle Handle to the key whose time to get.
	 * @return The key's time.
	 */
	ENGINE_API virtual float GetKeyTime(FKeyHandle KeyHandle) const override final;

	/**
	* Get the value for the Key with the specified index.
	*
	* @param KeyHandle Handle to the key whose value to get.
	* @return The key's value.
	*/
	ENGINE_API FString GetKeyValue(FKeyHandle KeyHandle) const;

	/**
	 * Set the default value of the curve.
	 *
	 * @param InDefaultValue The value to set.
	 */
	void SetDefaultValue(const FString& InDefaultValue)
	{
		DefaultValue = InDefaultValue;
	}

	/**
	 * Clears the default value for this curve if it has been set. 
	 */
	void ClearDefaultValue()
	{
		DefaultValue.Empty();
	}

	/**
	 * Move a key to a new time.
	 *
	 * @param KeyHandle The handle of the key to change.
	 * @param NewTime The new time to set on the key.
	 */
	ENGINE_API virtual void SetKeyTime(FKeyHandle KeyHandle, float NewTime) override final;

	/**
	* Assign a new value to a key.
	*
	* @param KeyHandle The handle of the key to change.
	* @param NewTime The new value to set on the key.
	*/
	ENGINE_API void SetKeyValue(FKeyHandle KeyHandle, FString NewValue);

	/**
	 * Finds the key at InTime, and updates its value. If it can't find the key within the KeyTimeTolerance, it adds one at that time.
	 *
	 * @param InTime The time at which the key should be added or updated.
	 * @param InValue The value of the key.
	 * @param KeyTimeTolerance The tolerance used for key time equality.
	 */
	ENGINE_API FKeyHandle UpdateOrAddKey(float InTime, const FString& InValue, float KeyTimeTolerance = UE_KINDA_SMALL_NUMBER);

	/** Tries to reduce the number of keys required for accurate evaluation (removing duplicates) */
	ENGINE_API void RemoveRedundantKeys();
public:

	//~ FIndexedCurve interface

	virtual int32 GetNumKeys() const override final { return Keys.Num(); }

	/** Allocates a duplicate of the curve */
	virtual FIndexedCurve* Duplicate() const final { return new FStringCurve(*this); }

public:

	/** Default value */
	UPROPERTY(EditAnywhere, Category="Curve")
	FString DefaultValue;

	/** Sorted array of keys */
	UPROPERTY(EditAnywhere, EditFixedSize, Category="Curve")
	TArray<FStringCurveKey> Keys;
};
