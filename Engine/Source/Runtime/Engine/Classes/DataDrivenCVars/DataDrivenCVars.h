// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "HAL/IConsoleManager.h"
#include "Subsystems/EngineSubsystem.h"
#include "Engine/DeveloperSettings.h"
#include "DataDrivenCVars.generated.h"

class UUserDefinedStruct;

/**
 *
 */
UCLASS(DisplayName = "DataDrivenCVars", MinimalAPI)
class UDataDrivenCVarEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:

	UDataDrivenCVarEngineSubsystem() {}

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnDataDrivenCVarChanged, FString, CVarName);

	UPROPERTY(BlueprintAssignable, Category = "DataDrivenCVar")
	FOnDataDrivenCVarChanged OnDataDrivenCVarDelegate;
};

/**
 *
 */
UENUM(BlueprintType)
enum class FDataDrivenCVarType : uint8
{
	CVarFloat,
	CVarInt,
	CVarBool
};

USTRUCT(BlueprintType)
struct FDataDrivenConsoleVariable
{
	GENERATED_BODY()

	FDataDrivenConsoleVariable() {}
	ENGINE_API ~FDataDrivenConsoleVariable();

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar", meta = (InlineCategoryProperty))
	FDataDrivenCVarType Type = FDataDrivenCVarType::CVarInt;

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar")
	FString Name;

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar")
	FString ToolTip;

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar", meta = (EditCondition = "Type == FDataDrivenCVarType::CVarFloat", EditConditionHides))
	float DefaultValueFloat = 0.0f;

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar", meta = (EditCondition = "Type == FDataDrivenCVarType::CVarInt", EditConditionHides))
	int32 DefaultValueInt = 0;

	UPROPERTY(EditAnywhere, config, Category = "DataDrivenCVar", meta = (EditCondition = "Type == FDataDrivenCVarType::CVarBool", EditConditionHides))
	bool DefaultValueBool = false;

#if WITH_EDITOR
	ENGINE_API void Refresh();
#endif

	ENGINE_API void Register();
	ENGINE_API void UnRegister(bool bUseShadowName = false);

	bool bIsRegistered = false;

	FString ShadowName;
	FString ShadowToolTip;
	FDataDrivenCVarType ShadowType;
};

/**
 * 
 */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Data Driven CVars"), MinimalAPI)
class UDataDrivenConsoleVariableSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	DECLARE_EVENT_OneParam(UDataDrivenConsoleVariableSettings, FOnStageChanged, const FString&);
	FOnStageChanged OnStageChanged;

	UPROPERTY(config, EditAnywhere, Category = "DataDrivenCVar")
	TArray<FDataDrivenConsoleVariable> CVarsArray;

	static ENGINE_API void OnDataDrivenChange(IConsoleVariable* CVar);

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	ENGINE_API virtual FName GetCategoryName() const;

protected:

	TArray<FString> ShadowCVars;
};
