// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/PrimitiveComponent.h"
#include "DebugRenderSceneProxy.h"
#include "DebugDrawComponent.generated.h"

/**
 * Helper class to derive from to use debug draw delegate functionalities (i.e. DrawDebugLabels)
 * The class will take care of registering a delegate to the UDebugDrawService and draw all FText3d provided by the scene proxy.
 * This functionality only requires the derived classes to override `CreateDebugSceneProxy`.
 *
 * It is also possible to add text from other sources of data from the scene proxy but that requires a few extra steps:
 *  + create a class that inherits from `FDebugDrawDelegateHelper`
 *		- override `DrawDebugLabels`
 *		- add a method to be able to populate the source data from the scene proxy
 *
 *		ex: class FMyDelegateHelper : public FDebugDrawDelegateHelper
 *			{
 *				public:
 *					void SetupFromProxy(const FMySceneProxy* InSceneProxy) { <collect data> };
 *				protected:
 *					virtual void DrawDebugLabels(UCanvas* Canvas, APlayerController*) override { <draw data> };
 *				private:
 *					TArray<FVector> SomeData;
 *			};
 *
 *  + Within the component inheriting from `UDebugDrawComponent`
 *		- add a property of that new helper type in the component inheriting from `UDebugDrawComponent`
 *		- override `GetDebugDrawDelegateHelper` to return that new member as the new delegate helper to use
 *		- and then use it in `CreateDebugSceneProxy` before returning the created proxy.
 *
 *		ex: class MyDebugDrawComponent : public UDebugDrawComponent
 *			{
 *				protected:
 *					virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override
 *					{
 *						FMySceneProxy* Proxy = new FMySceneProxy(this);
 *						MyDelegateHelper.SetupFromProxy(Proxy);
 *						return Proxy;
 *					}
 *
 *					virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return MyDelegateHelper; }
 *				private:
 *					FMyDelegateHelper MyDelegateHelper;
 *			}
 */
UCLASS(Abstract, HideCategories = (Activation, AssetUserData, Collision, Cooking, HLOD, Lighting, LOD, Mobile, Navigation, Physics, RayTracing, Rendering, Tags, TextureStreaming), MinimalAPI)
class UDebugDrawComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

protected:
	/* Method overriden and marked as final since derived class should override `CreateDebugSceneProxy` */
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override final;

	/* Method that derived class should override to create the scene proxy and customize a custom delegate helper (if any) */
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() { return nullptr; }

	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;

	/** Method that should be overriden when subclass uses a custom delegate helper. */
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() { return DebugDrawDelegateHelper; }

	FDebugDrawDelegateHelper DebugDrawDelegateHelper;
};
