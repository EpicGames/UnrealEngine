// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ShowFlags.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "DebugDrawService.generated.h"

class APlayerController;
class FCanvas;
class FSceneView;
class FViewport;
class UCanvas;

/** 
 * Delegate called when rendering a viewport and the associated engine flag is set.
 */
DECLARE_DELEGATE_TwoParams(FDebugDrawDelegate, UCanvas*, APlayerController*);

UCLASS(config=Engine, MinimalAPI)
class UDebugDrawService : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	static ENGINE_API FDelegateHandle Register(const TCHAR* Name, const FDebugDrawDelegate& NewDelegate);
	static ENGINE_API void Unregister(FDelegateHandle HandleToRemove);

	/** Draws debug canvas that has already been initialized to a viewport */
	static ENGINE_API void Draw(const FEngineShowFlags Flags, UCanvas* Canvas, APlayerController* PlayerController = nullptr);

	/** Initialize a debug canvas object then calls above draw. If CanvasObject is null it will find/create it for you */
	static ENGINE_API void Draw(const FEngineShowFlags Flags, FViewport* Viewport, FSceneView* View, FCanvas* Canvas, UCanvas* CanvasObject = nullptr, APlayerController* PlayerController = nullptr);
private:

	/**
	 * Synchronization object for delegate registration since it can happen from multiple threads.
	 * e.g.
	 *		primitives added in batch through
	 *			ParallelFor(AddPrimitiveBatches.Num(), [&](int32 Index)
	 *			{
	 *				FTaskTagScope Scope(ETaskTag::EParallelGameThread);
	 *				Scene->AddPrimitive(...);
	 *			}
	 *		or
	 *			RecreateRenderState_Concurrent that will unregister and register delegates from multiple threads.
	 */
	static FTransactionallySafeCriticalSection DelegatesLock;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FDebugDrawMulticastDelegate, UCanvas*, APlayerController*);

	static TArray<FDebugDrawMulticastDelegate> Delegates;
	static FEngineShowFlags ObservedFlags;
};
