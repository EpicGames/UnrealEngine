// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ReporterBase.generated.h"

class UCanvas;

/** Draw styles for lines. */
UENUM()
namespace EReporterLineStyle
{
	enum Type : int
	{
		Line,
		Dash,
	};
}


UCLASS(Abstract)
class UReporterBase : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	
	/** Main draw call for the Reporter.
	 * @param Canvas The canvas to draw to
	 */
	virtual void Draw(UCanvas* Canvas) { }

	/** Is this reporter visible */
	bool bVisible;
protected:
	/** Helper to convert vectors from normalized into screen space
	 * @param InVector Normalized screen space coordinates to convert to screen space
	 * @param Canvas The canvas we wish to use during the conversion
	 * @return The screen space coordinates based on the size of the canvas
	 */
	virtual FVector2D ToScreenSpace(const FVector2D& InVector, UCanvas* Canvas);

	/** Helper to draw a line from normalized to screen space
	 * @param Canvas The canvas to draw to
	 * @param StartPos The start position of the line, in normalized coordinates
	 * @param EndPos The end position of the line, in normalized coordinates
	 * @param Color The color to draw the line
	 * @param LineStyle The style to draw the line
	 */
	void DrawLine(UCanvas* Canvas, const FVector2D& StartPos, const FVector2D& EndPos, const FLinearColor& Color, EReporterLineStyle::Type LineStyle = EReporterLineStyle::Line);

	/** Helper to draw a triangle from normalized to screen space
	 * @param Canvas The canvas to draw to
	 * @param Vertex0 The position of the first vertex that makes up the triangle, in normalized coordinates
	 * @param Vertex1 The position of the second vertex that makes up the triangle, in normalized coordinates
	 * @param Vertex2 The position of the third vertex that makes up the triangle, in normalized coordinates
	 * @param Color The color to draw the triangle
	 */
	void DrawTriangle(UCanvas* Canvas, const FVector2D& Vertex1, const FVector2D& Vertex2, const FVector2D& Vertex3, const FLinearColor& Color);
};
