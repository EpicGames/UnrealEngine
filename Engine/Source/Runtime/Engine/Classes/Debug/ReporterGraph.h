// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Debug/ReporterBase.h"
#include "ReporterGraph.generated.h"

class UCanvas;
class UFont;

/** Draw styles for axes. */
UENUM()
namespace EGraphAxisStyle
{
	enum Type : int
	{
		Lines,
		Notches,
		Grid,
	};
}

/** Draw styles for data. */
UENUM()
namespace EGraphDataStyle
{
	enum Type : int
	{
		Lines,
		Filled,
	};
}

UENUM()
namespace ELegendPosition
{
	enum Type : int
	{
		Outside,
		Inside
	};
}

/** Helper rectangle struct. */
struct FRect
{
	FVector2D Min;
	FVector2D Max;
};

/** Graph Line data. */
struct FGraphThreshold
{
	/** The threshold amount. */
	float Threshold;

	/** The color of the threshold. */
	FLinearColor Color;

	/** The Threshold name. */
	FString ThresholdName;

	FGraphThreshold()
		: Threshold(0.0f),
		Color(FLinearColor::White),
		ThresholdName("UNDEFINED")
	{

	}

	FGraphThreshold(float InThreshold, const FLinearColor& InColor, FString InName)
		: Threshold(InThreshold),
		Color(InColor),
		ThresholdName(InName)
	{

	}
};

/** Graph Line data. */
struct FGraphLine
{
	/** The list of data to plot. */
	TArray<FVector2D> Data;

	/** The color to graph this data with. */
	FLinearColor Color;

	/** Left extreme value. */
	FVector2D LeftExtreme;

	/** Right extreme value. */
	FVector2D RightExtreme;

	/** The event name. */
	FString LineName;
};

UCLASS(MinimalAPI)
class UReporterGraph : public UReporterBase
{
	GENERATED_UCLASS_BODY()

public:

	/** Set the styles of the axes.
	 * @param NewAxisStyle The new style to draw the axes with.
	 * @param NewDataStyle The new style to draw the data with.
	 */
	inline void SetStyles(EGraphAxisStyle::Type NewAxisStyle, EGraphDataStyle::Type NewDataStyle) { AxisStyle = NewAxisStyle; DataStyle = NewDataStyle; }

	/** Set the color of the axes.
	 * @param NewAxesColor The colour to set the axes to.
	*/
	inline void SetAxesColor(FLinearColor NewAxesColor) { AxesColor = NewAxesColor; }

	/** Set The size of the graph.
	 * @param MinX The normalized minimum X extent of the graph.
	 * @param MaxX The normalized maximum X extent of the graph.
	 * @param MinY The normalized minimum Y extent of the graph.
	 * @param MaxY The normalized maximum Y extent of the graph.
	 */
	ENGINE_API void SetGraphScreenSize(float MinX, float MaxX, float MinY, float MaxY);
	
	/** Set The size of the graph.
	 * @param Min The normalized minimum extent of the graph.
	 * @param Max The normalized maximum extent of the graph.
	 */
	ENGINE_API void SetGraphScreenSize(const FVector2D& Min, const FVector2D& Max);

	/** Main draw call for the Graph.
	 * @param Canvas The canvas to draw this graph to.
	 */
	ENGINE_API virtual void Draw(UCanvas* Canvas) override;

	/** Set the axis min and max data for both axes.
	 * @param MinX The normalized minimum X data.
	 * @param MaxX The normalized maximum X data.
	 * @param MinY The normalized minimum Y data.
	 * @param MaxY The normalized maximum Y data.
	 */
	ENGINE_API void SetAxesMinMax(float MinX, float MaxX, float MinY, float MaxY);

	/** Set the axis min and max data for both axes.
	 * @param MinX The normalized minimum X data.
	 * @param MinY The normalized minimum Y data.
	 */
	ENGINE_API void SetAxesMinMax(const FVector2D& Min, const FVector2D& Max);

	/** Set notches per axis.
	 * @param NewNumXNotches The number of notches/divisions to display on the graph on the X axis.
	 * @param NewNumYNotches The number of notches/divisions to display on the graph on the Y axis.
	 */
	inline void SetNotchesPerAxis(int NewNumXNotches, int NewNumYNotches) { NumXNotches = NewNumXNotches; NumYNotches = NewNumYNotches; }

	/** Set the number of graph lines.
	 * @param NumDataLines The number of lines to plot on this graph.
	 */
	ENGINE_API void SetNumGraphLines(int32 NumDataLines);

	/** Get a pointer to a graph line.
	 * @param LineIndex Index of the graph line.
	 * @return Pointer to the graph line. Do not cache. May return NULL.
	 */
	ENGINE_API FGraphLine* GetGraphLine(int32 LineIndex);

	/** Set the number of Thresholds to display on this graph.
	 * @param NumThresholds The number of Thresholds.
	 */
	ENGINE_API void SetNumThresholds(int32 NumThresholds);

	/** Get a pointer to a threshold.
	 * @param ThresholdIndex Index of the threshold.
	 * @return Pointer to the threshold. Do not cache. May return NULL.
	 */
	ENGINE_API FGraphThreshold* GetThreshold(int32 ThresholdIndex);

	/** Sets background color.
	 * @param Color Color to set as background.
	 */
	ENGINE_API void SetBackgroundColor(FColor Color);

	/** Sets position where to draw legend.
	 * @param Position Actual position for legend.
	 */
	ENGINE_API void SetLegendPosition(ELegendPosition::Type Position);

	/** Enables small offset for data sets to make it easier to read.
	 * @Param Enable - set to true to enable offsets.
	 */
	void OffsetDataSets(bool Enable) { bOffsetDataSets = Enable; }

	/** Checks if we have enabled offset for data sets on graph. */
	bool IsOffsetForDataSetsEnabled() { return !!bOffsetDataSets; }

	/** Sets cursor location on line graph to show value at specific place. */
	void SetCursorLocation(float InValue) { CursorLocation = InValue; }

	/** setter to force tiny font instead of small font. */
	void UseTinyFont(bool InUseTinyFont) { bUseTinyFont = InUseTinyFont;  }

	/** setter to enable or disable cursor for line graphs. */
	void DrawCursorOnGraph(bool InDrawCursorOnGraph) { bDrawCursorOnGraph = InDrawCursorOnGraph;  }

	void DrawExtremesOnGraph(bool InDrawExtremes) { bDrawExtremes = InDrawExtremes; }

	/** Draw background under graph.
	* @param Canvas The canvas to draw on.
	*/
	ENGINE_API void DrawBackground(UCanvas* Canvas);

	/** Draw the Legend.
	 * @param Canvas The canvas to draw on.
	 */
	ENGINE_API void DrawLegend(UCanvas* Canvas);

	/** Draw the Axes.
	 * @param Canvas The canvas to draw on.
	 */
	ENGINE_API void DrawAxes(UCanvas* Canvas);

	/** Draw an individual axis.
	 * @param Canvas The canvas to draw on.
	 * @param Start Start position of the axis.
	 * @param End End position of the axis.
	 * @param NumNotches The number of notches to draw on this axis.
	 * @param bIsVerticalAxis Is this the vertical axis of the graph?
	 */
	ENGINE_API void DrawAxis(UCanvas* Canvas, FVector2D Start, FVector2D End, float NumNotches, bool bIsVerticalAxis);

	/** Draw the thresholds.
	 * @param Canvas The canvas to draw on.
	 */
	ENGINE_API void DrawThresholds(UCanvas* Canvas);

	/** Draw the data.
	 * @param Canvas The canvas to draw on.
	 */
	ENGINE_API void DrawData(UCanvas* Canvas);

	/** Helper to convert vectors from normalized into screen space.
	 * @param InVector Normalized screen space coordinates to convert to screen space.
	 * @param Canvas The canvas we wish to use during the conversion.
	 * @return The screen space coordinates based on the size of the canvas.
	 */
	ENGINE_API virtual FVector2D ToScreenSpace(const FVector2D& InVector, UCanvas* Canvas) override;

	/** Helper to convert data from raw data into screen space.
	 * @param InVector 2D plot point in data space.
	 * @return The data space coordinate converted to normalized screen coordinates.
	*/
	ENGINE_API FVector2D DataToNormalized(const FVector2D& InVector);

	/** Returns default font used to print texts. */
	ENGINE_API UFont* GetDefaultFont();

	/** The screen size of the graph. */
	FRect GraphScreenSize;

	/** The minimum and maximum for the graph data. */
	FRect GraphMinMaxData;

	TArray<FGraphThreshold> Thresholds;

	/** The data displayed on the graph. */
	TArray<FGraphLine> CurrentData;

	/** The color of the axes. */
	FLinearColor AxesColor;

	/** The number of notches on the X axis. **/
	int NumXNotches;
		
	/** The number of notches on the Y axis. **/
	int NumYNotches;

	/** The axis style. */
	EGraphAxisStyle::Type AxisStyle;

	/** The data style. */
	EGraphDataStyle::Type DataStyle;

	/** Current legend position. */
	ELegendPosition::Type LegendPosition;

	/** The maximum length of the legend names. */
	float LegendWidth;

	/** Background color to draw under graph. */
	FColor BackgroundColor;

	/** Current location for cursor on line graphs. */
	float CursorLocation;

	/** If set, enables small offset for graphs to better visualize overlapping data sets. */
	int32 bOffsetDataSets : 1;

	/** If set, forces tiny font for texts. */
	int32 bUseTinyFont : 1;

	/** If set, enables cursor for line graphs. */
	int32 bDrawCursorOnGraph : 1;

	/** If set, draws extremes on vertical axes. */
	int32 bDrawExtremes : 1;
};

