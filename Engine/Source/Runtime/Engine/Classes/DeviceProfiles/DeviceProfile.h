// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeviceProfile.h: Declares the UDeviceProfile class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TextureLODSettings.h"
#include "DeviceProfiles/DeviceProfileMatching.h"
#include "DeviceProfile.generated.h"

struct FPropertyChangedEvent;

DECLARE_DELEGATE(FOnCVarsUpdated);


UCLASS(config=DeviceProfiles, perObjectConfig, MinimalAPI)
class UDeviceProfile : public UTextureLODSettings
{
	/** do NOT cast to UTextureLODSettings !
	use GetTextureLODSettings() instead !
	@todo CBTLDP : remove the public inheritance, hold TextureLODSettings as member instead?
	**/

	GENERATED_UCLASS_BODY()

	/** The type of this profile, I.e. IOS, Windows, PS4 etc */
	UPROPERTY(VisibleAnywhere, config, Category=DeviceSettings)
	FString DeviceType;

	/** The name of the parent profile of this object */
	UPROPERTY(EditAnywhere, config, Category=DeviceSettings)
	FString BaseProfileName;

	/** Some asset types can reference Device Profiles, is this profile visible to those assets. */
	UPROPERTY(EditAnywhere, config, Category = DeviceSettings)
	uint32 bIsVisibleForAssets : 1;

	/** The parent object of this profile, it is the object matching this DeviceType with the BaseProfileName */
	UPROPERTY()
	TObjectPtr<UDeviceProfile> Parent;

	/** Flag used in the editor to determine whether the profile is visible in the property matrix */
	bool bVisible;

	/** This is not a property, it shouldn't be set by the editor */
	FString ConfigPlatform;

	/** A collection of UDeviceProfileFragment names, which can contain predefined sets of cvars */
	TArray<FString> FragmentIncludes;

public:

	/* Need to add missing entries in TextureLODGroups to match enum TextureGroup when the device profile is reloaded*/
	virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override
	{
		Super::PostReloadConfig(PropertyThatWasLoaded);

		ValidateTextureLODGroups();
	}

	/** The collection of CVars which is set from this profile */
	UPROPERTY(EditAnywhere, config, Category=ConsoleVariables)
	TArray<FString> CVars;

	/** An array of conditions to test against and fragment names to select. */
	UPROPERTY(EditAnywhere, config, Category = "DeviceProfile Matching Rules")
	TArray<FDPMatchingRulestruct> MatchingRules;

	/** Prefer to load the DP from its platform's hierarchy */
	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return ConfigPlatform.Len() ? *ConfigPlatform : Super::GetConfigOverridePlatform();
	}

	/** 
	 * Get the collection of Console Variables that this profile inherits from its' parents
	 *
	 * @param CVarInformation - The list of inherited CVars in the format of<key:CVarName,value:CVarCompleteString>
	 */
	ENGINE_API void GatherParentCVarInformationRecursively(OUT TMap<FString, FString>& CVarInformation) const;

	/** 
	 * Accessor to the delegate object fired when there has been any changes to the console variables 
	 */
	FOnCVarsUpdated& OnCVarsUpdated()
	{
		return CVarsUpdatedDelegate;
	}

	/** 
	 * Accessor to a fragment by tag.
	 * @param FragmentTag the name of the tag to find
	 * @return FSelectedFragmentProperties ptr or null if FragmentTag not found.
	 */	
	ENGINE_API const FSelectedFragmentProperties* GetFragmentByTag(FName& FragmentTag) const;

	bool IsVisibleForAssets()const { return bIsVisibleForAssets; }
public:
	/** 
	 * Access to the device profiles Texture LOD Settings
	 */
	ENGINE_API UTextureLODSettings* GetTextureLODSettings() const;

	/**
	 * Returns the parent device profile, optionally including the default object
	 */
	ENGINE_API UDeviceProfile* GetParentProfile(bool bIncludeDefaultObject = false) const;
	
private:
	// Make sure our TextureLODGroups array is sorted correctly and complete
	ENGINE_API void ValidateTextureLODGroups();
	/** Delegate object fired when there has been any changes to the console variables */
	FOnCVarsUpdated CVarsUpdatedDelegate;

public:
	// The selected result after running the MatchingRules process.
	TArray<FSelectedFragmentProperties> SelectedFragments;

	/* ValidateProfile()
	* Validate the Profile after changes by loading it's config (.ini)
	*/
	ENGINE_API void ValidateProfile();

	//~ Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

#if WITH_EDITOR
	//~ Begin UObject Interface
	ENGINE_API virtual void PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) override;
	//~ End UObject Interface

	ENGINE_API bool ModifyCVarValue(const FString& CVarName, const FString& CVarValue, bool bAddIfNonExistant = false);
	ENGINE_API FString GetCVarValue(const FString& CVarName) const;

	/** Lazily generate a consolidated list of CVars, recursing up the device profile hierarchy 
	 *  This will not include any cvars from the device's selected fragments.
	 *  This is _not_ thread-safe (for thread-safety use ForEachConsolidatedCVar or GetConsolidatedCVarValue API instead)
	*/
	UE_DEPRECATED(5.6, "GetConsolidatedCVars is no longer supported due to being non-thread safe, use ForEachConsolidatedCvar or GetConsolidatedCVarValue instead")
	ENGINE_API const TMap<FString, FString>& GetConsolidatedCVars() const;

	/** 
	 * Iterates over all consolidated string key-value CVar pairs held in this device profile, or in any parent device profile.
	 * @param	FuncRef		Iteration function (returning false will early out and stop subsequent calls)
	 */
	ENGINE_API void ForEachConsolidatedCVar(const TFunctionRef<bool(const FString&, const FString&)>& FuncRef) const;

	/** 
	 * Get the string value of a CVar that is held in this device profile, or in any parent device profile.
	 * @param	CVarName		The name of the CVar to find
	 * @param	OutString		The string value of the CVar, if found
	 * @param	bCheckDefaults	Whether to also check the IConsoleManager for the global default value for the CVar
	 * @return true if the CVar was found in this device profile
	 */
	ENGINE_API bool GetConsolidatedCVarValue(const TCHAR* CVarName, FString& OutString, bool bCheckDefaults = false) const;

	/** 
	 * Get the int32 value of a CVar that is held in this device profile, or in any parent device profile.
	 * @param	CVarName		The name of the CVar to find
	 * @param	OutString		The int32 value of the CVar, if found
	 * @param	bCheckDefaults	Whether to also check the IConsoleManager for the global default value for the CVar
	 * @return true if the CVar was found in this device profile
	 */
	ENGINE_API bool GetConsolidatedCVarValue(const TCHAR* CVarName, int32& OutValue, bool bCheckDefaults = false) const;

	/** 
	 * Get the float value of a CVar that is held in this device profile, or in any parent device profile.
	 * @param	CVarName		The name of the CVar to find
	 * @param	OutString		The float value of the CVar, if found
	 * @param	bCheckDefaults	Whether to also check the IConsoleManager for the global default value for the CVar
	 * @return true if the CVar was found in this device profile
	 */
	ENGINE_API bool GetConsolidatedCVarValue(const TCHAR* CVarName, float& OutValue, bool bCheckDefaults = false) const;

private:
	/** Helper function to broadcast when CVars change and clear consolidated map */
	ENGINE_API void HandleCVarsChanged();

private:
	/** Consolidated CVars, lazy initialized - access via GetConsolidatedCVars */
	mutable TMap<FString, FString> ConsolidatedCVars;
	mutable FCriticalSection CVarsLock;

#endif

#if ALLOW_OTHER_PLATFORM_CONFIG
public:
	ENGINE_API const TMap<FString, FString>& GetAllExpandedCVars();
	ENGINE_API const TMap<FString, FString>& GetAllPreviewCVars();
	ENGINE_API void ClearAllExpandedCVars();
	/** Set the memory size bucket to be used when previewing this DP, changing this will reset the expanded cvars. */
	ENGINE_API void SetPreviewMemorySizeBucket(EPlatformMemorySizeBucket PreviewMemorySizeBucketIn);
	ENGINE_API EPlatformMemorySizeBucket GetPreviewMemorySizeBucket() const;
	
	ENGINE_API void SetPreviewTextureGroupRedirect(const UTextureLODSettings* InPreviewTextureGroupRedirect);

	const UTextureLODSettings* GetPreviewTextureGroupRedirect() const
	{
		return PreviewTextureGroupRedirect;
	}

private:
	
	// calculate the cvars for another platform's deviceprofile
	ENGINE_API void ExpandDeviceProfileCVars();
		
	/** Resolved CVars, including expanded scalability cvars used to properly emulate one platform on another */
	TMap<FString, FString> AllExpandedCVars;

	/** The set of cvars that can be previewed (a subset of AllExpandedCVars) */
	TMap<FString, FString> AllPreviewCVars;

	/** The EPlatformMemorySizeBucket to use when processing the device profile for previewing. */
	EPlatformMemorySizeBucket PreviewMemorySizeBucket = EPlatformMemorySizeBucket::Default;
	
	UTextureLODSettings const * PreviewTextureGroupRedirect = nullptr;  // @todo CBTLDP : why is this a member here and not a static or global ?

#endif
};
