// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeviceProfileFragment.h: Declares the UDeviceProfileFragment class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DeviceProfileFragment.generated.h"

UCLASS(config=DeviceProfiles, perObjectConfig, MinimalAPI)
class UDeviceProfileFragment : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY()
	TArray<FString> CVars;
};
