// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeviceProfileManager.h: Declares the FDeviceProfileManager class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DeviceProfileMatching.h"
#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "DeviceProfileManager.generated.h"

class UDeviceProfile;

// Delegate used to refresh the UI when the profiles change
DECLARE_MULTICAST_DELEGATE( FOnDeviceProfileManagerUpdated );

// Delegate used to notify systems when the active device profile changes
DECLARE_MULTICAST_DELEGATE( FOnActiveDeviceProfileChanged );

#if ALLOW_OTHER_PLATFORM_CONFIG
// Delegate used to notify systems when the preview device profile changes
DECLARE_MULTICAST_DELEGATE( FOnPreviewDeviceProfileChanged );
#endif

// Delegate used to notify systems pre reapply device profiles
DECLARE_MULTICAST_DELEGATE( FWillReapplyDeviceProfiles );
// Delegate used to notify systems post reapplied device profiles
DECLARE_MULTICAST_DELEGATE( FOnReapplyDeviceProfiles );

struct FPushedCVarSetting
{
	FPushedCVarSetting() : SetBy(ECVF_Default)	{}
	FPushedCVarSetting(const FString& InValue, EConsoleVariableFlags InFlags)
		: Value(InValue)
		, SetBy(EConsoleVariableFlags(InFlags & ECVF_SetByMask))
	{}

	FString Value;
	EConsoleVariableFlags SetBy;
};

/**
 * Implements a helper class that manages all profiles in the Device
 */
 UCLASS( config=DeviceProfiles, transient , MinimalAPI)
class UDeviceProfileManager : public UObject
{
public:

	GENERATED_BODY()

	/**
	 * Startup and select the active device profile
	 * Then Init the CVars from this profile and it's Device profile parent tree.
	 */
	static ENGINE_API void InitializeCVarsForActiveDeviceProfile(bool bPushSettings = false, bool bIsDeviceProfilePreview = false, bool bForceReload = false);

	/**
	 * Reapplies the device profile. Useful when configs have changed (i.e. hotfix)
	 * Applies base and then any overridden device profile. If ModifiedCVars is specified, then only the given CVars will be updated
	 */
	ENGINE_API void ReapplyDeviceProfile(bool bForceReload = false, TSet<FString>* ModifiedCVars=nullptr);

	/**
	 * Examine the currently active or overridden profile for references to entries in DeviceProfilesToQuery
	 * @param DeviceProfilesToQuery - Collection of device profiles to check.
	 * 
	 * @return true if any profiles contained in DeviceProfilesToQuery are referenced by active or overridden profile.
	 */
	ENGINE_API bool DoActiveProfilesReference(const TSet<FString>& DeviceProfilesToQuery);

	/**
	 * Create a copy of a device profile from a copy.
	 *
	 * @param ProfileName - The profile name.
 	 * @param ProfileToCopy - The profile to copy name.
	 *
	 * @return the created profile.
	 */
	ENGINE_API UDeviceProfile* CreateProfile(const FString& ProfileName, const FString& ProfileType, const FString& ParentName=TEXT(""), const TCHAR* ConfigPlatform=nullptr);

	/**
	 * Tests to see if a named device profile is available to call CreateProfile with.
	 *
	 * @param ProfileName - The profile name.
	 * @param ProfileToCopy - The profile to copy name.
	 *
	 * @return the created profile.
	 */
	ENGINE_API bool HasLoadableProfileName(const FString& ProfileName, FName OptionalPlatformName = FName());

	/**
	 * Get a list of all a named device profiles that are available to call CreateProfile with.
	 *
	 * @param OptionalPlatformName - The platform name to use for loading.
	 *
	 * @return matching profiles.
	 */
	ENGINE_API TArray<FString> GetLoadableProfileNames(FName OptionalPlatformName = FName()) const;

	/**
	 * Delete a profile.
	 *
	 * @param Profile - The profile to delete.
	 */
	ENGINE_API void DeleteProfile( UDeviceProfile* Profile );

	/**
	 * Find a profile based on the name.
	 *
	 * @param ProfileName - The profile name to find.
	 * @param bCreateProfileOnFail - Whether to create the profile from config if the object doesn't exist yet.
	 * @param OptionalPlatformName - The platform name to use for loading.
	 * @return The found profile.
	 *
	 * NOTE: when finding a profile for a TargetPlatform, use TargetPlatform->CookingDeviceProfileName()
	 *   do NOT use TargetPlatform->PlatformName() or ->IniPlatformName()
	 */
	ENGINE_API UDeviceProfile* FindProfile(const FString& ProfileName, bool bCreateProfileOnFail = true, FName OptionalPlatformName = FName());

	/**
	* Overrides the device profile. The original profile can be restored with RestoreDefaultDeviceProfile
	*/
	ENGINE_API void SetOverrideDeviceProfile(UDeviceProfile* DeviceProfile);

	/**
	* Restore the device profile to the default for this device
	*/
	ENGINE_API void RestoreDefaultDeviceProfile();

#if ALLOW_OTHER_PLATFORM_CONFIG
	/**
	* Applies the requested device profile's cvars onto the currently active DP.
	* It does not change the actual DP. 
	* Use RestorePreviewDeviceProfile to revert.
	*/
	ENGINE_API void SetPreviewDeviceProfile(UDeviceProfile* DeviceProfile, FName PreviewModeTag=NAME_None);

	/**
	* Revert the preview state.
	*/
	ENGINE_API void RestorePreviewDeviceProfile(FName PreviewModeTag=NAME_None);

	/**
	*	Invoked when the preview deviceprofile changes
	*/
	FOnPreviewDeviceProfileChanged& OnPreviewDeviceProfileChanged() { return PreviewDeviceProfileChangedDelegate; }
#endif

	/**
	 * Load the device profiles from the config file.
	 */
	ENGINE_API void LoadProfiles();

	/**
	 * Returns a delegate that is invoked when manager is updated.
	 *
	 * @return The delegate.
	 */
	ENGINE_API FOnDeviceProfileManagerUpdated& OnManagerUpdated();

	/**
	 * Returns a delegate that is invoked when the active device profile changes
	 *
	 * @return The delegate.
	 */
	ENGINE_API FOnActiveDeviceProfileChanged& OnActiveDeviceProfileChanged();

	/**
	 * Returns a delegate that is invoked pre reapply device profiles
	 *
	 * @return The delegate.
	 */
	ENGINE_API FWillReapplyDeviceProfiles& WillReapplyDeviceProfiles();

	/**
	 * Returns a delegate that is invoked post reapply device profiles
	 *
	 * @return The delegate.
	 */
	ENGINE_API FOnReapplyDeviceProfiles& OnReapplyDeviceProfiles();

	/**
	 * Returns the config files for all loaded device profiles, this will include platform-specific ones 
	 */
	ENGINE_API void GetProfileConfigFiles(OUT TArray<FString>& OutConfigFiles);

	/**
	 * Save the device profiles.
	 */
	ENGINE_API void SaveProfiles(bool bSaveToDefaults = false);

	/**
	 * Get the selected device profile
	 *
	 * @return The selected profile.
	 */
	ENGINE_API UDeviceProfile* GetActiveProfile() const;

	/**
	 * Get the currently previewing device profile. Can be null.
	 *
	 * @return The preview profile.
	 */
	ENGINE_API UDeviceProfile* GetPreviewDeviceProfile() const;

	/**
	* Get a list of all possible parent profiles for a given device profile
	*
	* @param ChildProfile				- The profile we are looking for potential parents
	* @param PossibleParentProfiles	- The list of profiles which would be suitable as a parent for the given profile
	*/
	ENGINE_API void GetAllPossibleParentProfiles(const UDeviceProfile* ChildProfile, OUT TArray<UDeviceProfile*>& PossibleParentProfiles) const;

	/**
	* Get the current active profile name.
	*
	* @return The selected profile.
	*/
	ENGINE_API const FString GetActiveDeviceProfileName();

	/**
	* Get a string containing the current matched fragment list.
	*
	* e.g. "Fragment1,Fragment2,[tag]Fragment3"
	* 
	* @param bEnabledOnly				- Only Enabled fragments will be present in the returned string.
	* @param bIncludeTags				- If true the Fragment string will include the tag.
	* @param bAlphaSort					- If true the Fragments will be in alphabetical order as opposed to application order.
	* @return csv string of the current matched fragments.
	*/
	ENGINE_API const FString GetActiveDeviceProfileMatchedFragmentsString(bool bEnabledOnly, bool bIncludeTags, bool bAlphaSort);

	/**
	* Get the selected device profile name, either the platform name, or the name
	* provided by a Device Profile Selector Module.
	*
	* @return The selected profile.
	*/
	UE_DEPRECATED(4.25, "Use either GetActiveDeviceProfileName to have the current active device profile or GetPlatformDeviceProfileName to have the default one. Note, GetActiveDeviceProfileName will fallback on GetPlatformDeviceProfileName, if there is no active device profile ")
	static ENGINE_API const FString GetActiveProfileName();

	/**
	* Get the selected device profile name, either the platform name, or the name
	* provided by a Device Profile Selector Module.
	*
	* @return The selected profile.
	*/
	static ENGINE_API const FString GetPlatformDeviceProfileName();
	
	/** Retrieves the value of a scalability group cvar if it was set by the active device profile. */
	static ENGINE_API bool GetScalabilityCVar(const FString& CvarName, int32& OutValue);
	static ENGINE_API bool GetScalabilityCVar(const FString& CvarName, float& OutValue);

	/**
	* Enable/Disable a tagged fragment of the active device profile.
	* This unsets the entire cvar DP state, then re-sets the new DP+fragment state.
	*/
	ENGINE_API void ChangeTaggedFragmentState(FName FragmentTag, bool bNewState);

	/**
	* Return the selected fragment property from the currently active device profile.
	* null if the tag is not found.
	*/
	ENGINE_API const FSelectedFragmentProperties* GetActiveDeviceProfileFragmentByTag(FName& FragmentTag) const;


	enum class EDeviceProfileMode : uint8
	{
		DPM_SetCVars,
		DPM_CacheValues,
		DPM_CacheValuesIgnoreMatchingRules, // GatherDeviceProfileCVars will not return any matching rules cvars. Used when entire matching rules state is applied later (see GetAllReferencedDeviceProfileCVars).
		DPM_SetCVarsForSingleDP,
		DPM_SetCVarsForSingleFragment,
	};

	/**
	 * Walk the device profile/fragment chain to get the final set ot CVars in a unified way
	 */
	static ENGINE_API TMap<FName, FString> GatherDeviceProfileCVars(const FString& DeviceProfileName, EDeviceProfileMode GatherMode);

	/**
	 * Gather all the cvars from the static device profile and then add all of the possible cvars+values from the dynamic/matched rule fragments.
	 */
	static ENGINE_API TMap<FName, TSet<FString>> GetAllReferencedDeviceProfileCVars(UDeviceProfile* DeviceProfile);

	/**
	 * Gets the parent chain and fragment that make up the final DP settings
	 * @return a list of pairs of DP/fragment name, along with the section name to look up in ConfigSystem
	 */
	TArray<TTuple<FName, FString>> GetDPChain(const FString& DeviceProfileName, FConfigCacheIni* ConfigSystem);

private:
	/**
	 * Set the active device profile - set via the device profile blueprint.
	 *
	 * @param DeviceProfileName - The profile name.
	 */
	ENGINE_API void SetActiveDeviceProfile( UDeviceProfile* DeviceProfile );

	/**
	* Override CVar value change callback
	*/
	ENGINE_API void HandleDeviceProfileOverrideChange();

	/** Sees if two profiles are considered identical for saving */
	ENGINE_API bool AreProfilesTheSame(UDeviceProfile* Profile1, UDeviceProfile* Profile2) const;

	/** Sees if the texture settings are the same between two profiles */
	ENGINE_API bool AreTextureGroupsTheSame(UDeviceProfile* Profile1, UDeviceProfile* Profile2) const;

	/**
	 * Perform the processing of ini sections, going up to parents, etc. Depending on runtime vs editor processing of another platform,
	 * the Mode will control how the settings are handled
	 */
	static ENGINE_API void SetDeviceProfileCVars(const FString& DeviceProfileName);


	/** Read and process all of the fragment matching rules. Returns an array containing the names of fragments selected. */
	static ENGINE_API TArray<FSelectedFragmentProperties> FindMatchingFragments(const FString& ParentDP, class FConfigCacheIni* PreviewConfigSystem);

	/** Read and discover all of the possible fragments referenced by the matching rules. Returns an array containing the names of fragments selected. */
	static ENGINE_API TArray<FString> FindAllReferencedFragmentsFromMatchedRules(const FString& ParentDP, FConfigCacheIni* ConfigSystem);

	/** Convert a FSelectedFragmentProperties array to a string */
	static ENGINE_API const FString FragmentPropertyArrayToFragmentString(const TArray<FSelectedFragmentProperties>& FragmentProperties, bool bEnabledOnly, bool bIncludeTags, bool bAlphaSort);

	/** Get the current platform's the selector module. Can return null */
	static ENGINE_API class IDeviceProfileSelectorModule* GetDeviceProfileSelectorModule();

#if ALLOW_OTHER_PLATFORM_CONFIG && WITH_EDITOR
	/** Get another platform's selector module. Can return null */
	static ENGINE_API class IDeviceProfileSelectorModule* GetPreviewDeviceProfileSelectorModule(class FConfigCacheIni* PreviewConfigSystem);
#endif
public:

	static ENGINE_API class UDeviceProfileManager* DeviceProfileManagerSingleton;
	static ENGINE_API UDeviceProfileManager& Get(bool bFromPostCDOContruct = false);

	virtual void PostCDOContruct() override
	{
		Get(true); // get this taken care of now
	}


public:

	// Holds the collection of managed profiles.
	UPROPERTY( EditAnywhere, Category=Properties )
	TArray< TObjectPtr<UDeviceProfile> > Profiles;

private:
	// Cached copy of profiles at load
	UPROPERTY()
	TArray< TObjectPtr<UDeviceProfile> > BackupProfiles;

	// Holds a delegate to be invoked profiles are updated.
	FOnDeviceProfileManagerUpdated ManagerUpdatedDelegate;

	// Holds a delegate to be invoked when the active deviceprofile changes
	FOnActiveDeviceProfileChanged ActiveDeviceProfileChangedDelegate;

#if ALLOW_OTHER_PLATFORM_CONFIG
	// Holds a delegate to be invoked when the preview deviceprofile changes
	FOnPreviewDeviceProfileChanged PreviewDeviceProfileChangedDelegate;
#endif

	// Holds a delegate to be invoked pre reapply device profiles
	FWillReapplyDeviceProfiles PreReapplyDeviceProfilesDelegate;
	// Holds a delegate to be invoked post reapply device profiles
	FOnReapplyDeviceProfiles PostReapplyDeviceProfilesDelegate;

	// Holds the selected device profile
	UDeviceProfile* ActiveDeviceProfile;

	// Add to profile to get load time backup
	static ENGINE_API FString BackupSuffix;

	// Original values of all the CVars modified by the DP.
	// Used to undo the DP before applying new state.
	static ENGINE_API TMap<FString, FPushedCVarSetting> PushedSettings;

	// Holds the device profile that has been overridden, null no override active.
	UDeviceProfile* BaseDeviceProfile = nullptr;

	// Holds the device profile that we are previewing.
	UDeviceProfile* PreviewDeviceProfile = nullptr;

	// Stores any scalability group settings set by the active device profile.
	static ENGINE_API TMap<FString, FString> DeviceProfileScalabilityCVars;

	// Stores any scalability group settings set by the preview device profile.
	static ENGINE_API TMap<FString, FString> PreviewDeviceProfileScalabilityCVars;
	
	// The list of fragments that have been selected by the active profile.
	static ENGINE_API TArray<FSelectedFragmentProperties> PlatformFragmentsSelected;
};
