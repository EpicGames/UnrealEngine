// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeviceProfileFragment.h: Declares the UDeviceProfileFragment class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DeviceProfileMatching.generated.h"

USTRUCT()
struct FSelectedFragmentProperties
{
	GENERATED_USTRUCT_BODY()

	bool operator == (const FSelectedFragmentProperties& rhs) const
	{
		return Tag == rhs.Tag && Fragment == rhs.Fragment && bEnabled == rhs.bEnabled;
	}

	bool operator != (const FSelectedFragmentProperties& rhs) const
	{
		return !(*this == rhs);
	}

	// user defined 'Tag' for this fragment.
	UPROPERTY()
	FName Tag = NAME_None;

	// Actual name of the fragment to select
	UPROPERTY()
	FString Fragment;

	// whether the fragment's cvars will be applied
	UPROPERTY()
	bool bEnabled = true;
};

USTRUCT()
struct FDPMatchingIfCondition
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Operator;

	UPROPERTY()
	FString Arg1;

	UPROPERTY()
	FString Arg2;

	bool operator==(const FDPMatchingIfCondition& Other) const
	{
		return Operator == Other.Operator
			&& Arg1.Equals(Other.Arg1, ESearchCase::CaseSensitive)
			&& Arg2.Equals(Other.Arg2, ESearchCase::CaseSensitive);
	}
};

USTRUCT()
struct FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()
	virtual ~FDPMatchingRulestructBase() = default;

	UPROPERTY()
	FString RuleName;

	UPROPERTY()
	TArray<FDPMatchingIfCondition> IfConditions;

	UPROPERTY()
	FString AppendFragments;

	UPROPERTY()
	FString SetUserVar;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const { return nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const { return nullptr; };

	bool operator==(const FDPMatchingRulestructBase& Other) const
	{
		return IfConditions == Other.IfConditions
			&& RuleName.Equals(Other.RuleName, ESearchCase::CaseSensitive)
			&& AppendFragments.Equals(Other.AppendFragments, ESearchCase::CaseSensitive)
			&& SetUserVar.Equals(Other.SetUserVar, ESearchCase::CaseSensitive);
	}
};

USTRUCT()
	struct FDPMatchingRulestructA : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructBase> OnTrue;

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructBase> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};

USTRUCT()
	struct FDPMatchingRulestructB : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()
	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructA> OnTrue;

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructA> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};

USTRUCT()
	struct FDPMatchingRulestructC : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructB> OnTrue;
	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructB> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};


USTRUCT()
	struct FDPMatchingRulestructD : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructC> OnTrue;

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructC> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};

USTRUCT()
	struct FDPMatchingRulestructE : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructD> OnTrue;

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructD> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};

USTRUCT()
struct FDPMatchingRulestruct : public FDPMatchingRulestructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructE> OnTrue;

	UPROPERTY(config)
	TArray<struct FDPMatchingRulestructE> OnFalse;

	virtual const FDPMatchingRulestructBase* GetOnTrue() const override { return OnTrue.Num() ? &OnTrue[0] : nullptr; };
	virtual const FDPMatchingRulestructBase* GetOnFalse() const override { return OnFalse.Num() ? &OnFalse[0] : nullptr; };
};
