// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Distributions.h"
#include "Distribution.generated.h"

extern ENGINE_API uint32 GDistributionType;

UENUM()
enum DistributionParamMode : int
{
	DPM_Normal,
	DPM_Abs,
	DPM_Direct,
	DPM_MAX,
};

#if !CPP      //noexport struct
/** Lookup table for distributions. */
USTRUCT(noexport)
struct FDistributionLookupTable
{
	UPROPERTY()
	float TimeScale;

	UPROPERTY()
	float TimeBias;

	UPROPERTY()
	TArray<float> Values;

	UPROPERTY()
	uint8 Op;

	UPROPERTY()
	uint8 EntryCount;

	UPROPERTY()
	uint8 EntryStride;

	UPROPERTY()
	uint8 SubEntryStride;

	UPROPERTY()
	uint8 LockFlag;
};
#endif

#if !CPP      //noexport struct
// Base class for raw (baked out) Distribution type
USTRUCT(noexport)
struct FRawDistribution
{
	UPROPERTY()
	FDistributionLookupTable Table;
};
#endif

UCLASS(DefaultToInstanced, collapsecategories, hidecategories=Object, editinlinenew, abstract,MinimalAPI)
class UDistribution : public UObject, public FCurveEdInterface
{
	GENERATED_UCLASS_BODY()

	/** Default value for initializing and checking correct values on UDistributions. */
	static const float DefaultValue;

	//~ Begin UObject Interface. 
	virtual bool IsPostLoadThreadSafe() const override;
	//~ End UObject Interface
};

