// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "DistributionFloatConstantCurve.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionFloatConstantCurve : public UDistributionFloat
{
	GENERATED_UCLASS_BODY()

	/** Keyframe data for how output constant varies over time. */
	UPROPERTY(EditAnywhere, Category=DistributionFloatConstantCurve)
	FInterpCurveFloat ConstantCurve;


	//~ Begin UDistributionFloat Interface
	ENGINE_API virtual float GetValue( float F = 0.f, UObject* Data = NULL, struct FRandomStream* InRandomStream = NULL ) const override;
	//~ End UDistributionFloat Interface

	//~ Begin FCurveEdInterface Interface
	ENGINE_API virtual int32		GetNumKeys() const override;
	ENGINE_API virtual int32		GetNumSubCurves() const override;
	ENGINE_API virtual float	GetKeyIn(int32 KeyIndex) override;
	ENGINE_API virtual float	GetKeyOut(int32 SubIndex, int32 KeyIndex) override;
	ENGINE_API virtual FColor	GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor) override;
	ENGINE_API virtual void	GetInRange(float& MinIn, float& MaxIn) const override;
	ENGINE_API virtual void	GetOutRange(float& MinOut, float& MaxOut) const override;
	ENGINE_API virtual EInterpCurveMode	GetKeyInterpMode(int32 KeyIndex) const override;
	ENGINE_API virtual void	GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const override;
	ENGINE_API virtual float	EvalSub(int32 SubIndex, float InVal) override;
	ENGINE_API virtual int32		CreateNewKey(float KeyIn) override;
	ENGINE_API virtual void	DeleteKey(int32 KeyIndex) override;
	ENGINE_API virtual int32		SetKeyIn(int32 KeyIndex, float NewInVal) override;
	ENGINE_API virtual void	SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) override;
	ENGINE_API virtual void	SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) override;
	ENGINE_API virtual void	SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent) override;
	//~ End FCurveEdInterface Interface
};

