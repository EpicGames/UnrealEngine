// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloatConstant.h"
#include "DistributionFloatParameterBase.generated.h"

UCLASS(abstract, collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionFloatParameterBase : public UDistributionFloatConstant
{
	GENERATED_UCLASS_BODY()

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	FName ParameterName;

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	float MinInput;

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	float MaxInput;

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	float MinOutput;

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	float MaxOutput;

	/** todo document */
	UPROPERTY(EditAnywhere, Category=DistributionFloatParameterBase)
	TEnumAsByte<enum DistributionParamMode> ParamMode;


	//~ Begin UDistributionFloat Interface
	ENGINE_API virtual float GetValue( float F = 0.f, UObject* Data = NULL, struct FRandomStream* InRandomStream = NULL ) const override;
	//~ End UDistributionFloat Interface
	
	
	/** todo document */
	virtual bool GetParamValue(UObject* Data, FName ParamName, float& OutFloat) const { return false; }

	
	// Begin Distribution Float
	virtual bool CanBeBaked() const override { return false; }
};



