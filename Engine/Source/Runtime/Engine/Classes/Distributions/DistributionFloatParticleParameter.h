// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloatParameterBase.h"
#include "DistributionFloatParticleParameter.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionFloatParticleParameter : public UDistributionFloatParameterBase
{
	GENERATED_UCLASS_BODY()


	//~ Begin UDistributionFloatParameterBase Interface
	ENGINE_API virtual bool GetParamValue(UObject* Data, FName ParamName, float& OutFloat) const override;
	//~ End UDistributionFloatParameterBase Interface
};

