// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Distributions.h"
#include "Distributions/Distribution.h"
#include "DistributionVector.generated.h"

struct FPropertyChangedEvent;

UENUM()
enum EDistributionVectorLockFlags : int
{
	EDVLF_None UMETA(DisplayName="None"),
	EDVLF_XY UMETA(DisplayName="XY"),
	EDVLF_XZ UMETA(DisplayName="XZ"),
	EDVLF_YZ UMETA(DisplayName="YZ"),
	EDVLF_XYZ UMETA(DisplayName="XYZ"),
	EDVLF_MAX,
};

UENUM()
enum EDistributionVectorMirrorFlags : int
{
	EDVMF_Same UMETA(DisplayName="Same"),
	EDVMF_Different UMETA(DisplayName="Different"),
	EDVMF_Mirror UMETA(DisplayName="Mirror"),
	EDVMF_MAX,
};

#if !CPP      //noexport struct
/** Type-safe vector distribution. */
USTRUCT(noexport)
struct FVectorDistribution
{
	UPROPERTY()
	FDistributionLookupTable Table;

};
#endif

#if !CPP      //noexport struct
/** Type-safe 4-vector distribution. */
USTRUCT(noexport)
struct FVector4Distribution
{
	UPROPERTY()
	FDistributionLookupTable Table;

};
#endif

USTRUCT()
struct FRawDistributionVector : public FRawDistribution
{
	GENERATED_USTRUCT_BODY()

private:
	UPROPERTY()
	float MinValue;

	UPROPERTY()
	float MaxValue;

	UPROPERTY()
	FVector MinValueVec;

	UPROPERTY()
	FVector MaxValueVec;

public:
	UPROPERTY(EditAnywhere, export, noclear, Category=RawDistributionVector)
	TObjectPtr<class UDistributionVector> Distribution;

	/** Whether the distribution data has been cooked or the object itself is available */
	ENGINE_API bool IsCreated();

	FRawDistributionVector()
		: MinValue(0)
		, MaxValue(0)
		, MinValueVec(FVector::ZeroVector)
		, MaxValueVec(FVector::ZeroVector)
		, Distribution(NULL)
	{
	}


#if WITH_EDITOR
	/**
	* Initialize a raw distribution from the original Unreal distribution
	*/
	ENGINE_API void Initialize();
#endif

	/**
	* Gets a pointer to the raw distribution if you can just call FRawDistribution::GetValue3 on it, otherwise NULL 
	*/
	ENGINE_API const FRawDistribution *GetFastRawDistribution();

	/**
	* Get the value at the specified F
	*/
	ENGINE_API FVector GetValue(float F=0.0f, UObject* Data=NULL, int32 LastExtreme=0, struct FRandomStream* InRandomStream = NULL);

	/**
	* Get the min and max values
	*/
	ENGINE_API void GetOutRange(float& MinOut, float& MaxOut);

	/**
	* Get the min and max values
	*/
	ENGINE_API void GetRange(FVector& MinOut, FVector& MaxOut);

	/**
	* Is this distribution a uniform type? (ie, does it have two values per entry?)
	*/
	inline bool IsUniform() { return LookupTable.SubEntryStride != 0; }

	ENGINE_API void InitLookupTable();

	inline bool HasLookupTable(bool bInitializeIfNeeded = true)
	{
#if WITH_EDITOR
		if(bInitializeIfNeeded)
		{
			InitLookupTable();
		}
#endif
		return GDistributionType != 0 && !LookupTable.IsEmpty();
	}

	inline bool OkForParallel()
	{
		HasLookupTable(); // initialize if required
		return true; // even if they stay distributions, this should probably be ok as long as nobody is changing them at runtime
		//return !Distribution || HasLookupTable();
	}
};


UCLASS(abstract, customconstructor, MinimalAPI)
class UDistributionVector : public UDistribution
{
	GENERATED_UCLASS_BODY()

	/** Can this variable be baked out to a FRawDistribution? Should be true 99% of the time*/
	UPROPERTY(EditAnywhere, Category=Baked)
	uint8 bCanBeBaked:1;

	/** Set internally when the distribution is updated so that that FRawDistribution can know to update itself*/
	UPROPERTY()
	uint8 bIsDirty:1;
protected:
	UPROPERTY()
	uint8 bBakedDataSuccesfully:1;	//It's possible that even though we want to bake we are not able to because of content or code.
public:

	/** Script-accessible way to query a FVector distribution */
	ENGINE_API virtual FVector GetVectorValue(float F = 0);


	UDistributionVector(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	:	Super(ObjectInitializer)
	,   bCanBeBaked(true)
	,   bIsDirty(true) // make sure the FRawDistribution is initialized
	{
	}

	//@todo.CONSOLE: Currently, consoles need this? At least until we have some sort of cooking/packaging step!
	/**
	 * Return the operation used at runtime to calculate the final value
	 */
	virtual ERawDistributionOperation GetOperation() const { return RDO_None; }

	/**
	 * Returns the lock axes flag used at runtime to swizzle random stream values. 
	 */
	virtual uint8 GetLockFlag() const { return 0; }
	
	/**
	 * Fill out an array of vectors and return the number of elements in the entry
	 *
	 * @param Time The time to evaluate the distribution
	 * @param Values An array of values to be filled out, guaranteed to be big enough for 2 vectors
	 * @return The number of elements (values) set in the array
	 */
	ENGINE_API virtual uint32 InitializeRawEntry(float Time, float* Values) const;

	ENGINE_API virtual FVector	GetValue( float F = 0.f, UObject* Data = NULL, int32 LastExtreme = 0, struct FRandomStream* InRandomStream = NULL ) const;

	//~ Begin FCurveEdInterface Interface
	ENGINE_API virtual void GetInRange(float& MinIn, float& MaxIn) const override;
	ENGINE_API virtual void GetOutRange(float& MinOut, float& MaxOut) const override;
	ENGINE_API virtual	void GetRange(FVector& OutMin, FVector& OutMax) const;
	//~ End FCurveEdInterface Interface

	/** @return true of this distribution can be baked into a FRawDistribution lookup table, otherwise false */
	virtual bool CanBeBaked() const 
	{
		return bCanBeBaked; 
	}

	bool HasBakedSuccesfully() const
	{
		return bBakedDataSuccesfully;
	}

	/**
	 * Returns the number of values in the distribution. 3 for vector.
	 */
	int32 GetValueCount() const
	{
		return 3;
	}

	/** Begin UObject interface */
#if	WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual bool NeedsLoadForClient() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
	ENGINE_API virtual bool NeedsLoadForEditorGame() const override;
	ENGINE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	/** End UObject interface */

};

