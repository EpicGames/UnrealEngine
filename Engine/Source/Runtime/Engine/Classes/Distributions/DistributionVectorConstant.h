// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "DistributionVectorConstant.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionVectorConstant : public UDistributionVector
{
	GENERATED_UCLASS_BODY()

	/** This FVector will be returned for all input times. */
	UPROPERTY(EditAnywhere, Category=DistributionVectorConstant)
	FVector Constant;

	/** If true, X == Y == Z ie. only one degree of freedom. If false, each axis is picked independently. */
	UPROPERTY()
	uint32 bLockAxes:1;

	UPROPERTY(EditAnywhere, Category=DistributionVectorConstant)
	TEnumAsByte<enum EDistributionVectorLockFlags> LockedAxes;


	//Begin UDistributionVector Interface
	ENGINE_API virtual FVector	GetValue( float F = 0.f, UObject* Data = NULL, int32 LastExtreme = 0, struct FRandomStream* InRandomStream = NULL ) const override;
	ENGINE_API virtual	void	GetRange(FVector& OutMin, FVector& OutMax) const override;
	//End UDistributionVector Interface

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin FCurveEdInterface Interface
	ENGINE_API virtual int32		GetNumKeys() const override;
	ENGINE_API virtual int32		GetNumSubCurves() const override;
	ENGINE_API virtual FColor	GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const override;
	ENGINE_API virtual float	GetKeyIn(int32 KeyIndex) override;
	ENGINE_API virtual float	GetKeyOut(int32 SubIndex, int32 KeyIndex) override;
	ENGINE_API virtual FColor	GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor) override;
	ENGINE_API virtual void	GetInRange(float& MinIn, float& MaxIn) const override;
	ENGINE_API virtual void	GetOutRange(float& MinOut, float& MaxOut) const override;
	ENGINE_API virtual EInterpCurveMode	GetKeyInterpMode(int32 KeyIndex) const override;
	ENGINE_API virtual void	GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const override;
	ENGINE_API virtual float	EvalSub(int32 SubIndex, float InVal) override;
	ENGINE_API virtual int32		CreateNewKey(float KeyIn) override;
	ENGINE_API virtual void	DeleteKey(int32 KeyIndex) override;
	ENGINE_API virtual int32		SetKeyIn(int32 KeyIndex, float NewInVal) override;
	ENGINE_API virtual void	SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) override;
	ENGINE_API virtual void	SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) override;
	ENGINE_API virtual void	SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent) override;
	//~ End FCurveEdInterface Interface

};

