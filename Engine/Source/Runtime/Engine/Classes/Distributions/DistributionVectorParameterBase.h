// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVectorConstant.h"
#include "DistributionVectorParameterBase.generated.h"

UCLASS(abstract, collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionVectorParameterBase : public UDistributionVectorConstant
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	FName ParameterName;

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	FVector MinInput;

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	FVector MaxInput;

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	FVector MinOutput;

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	FVector MaxOutput;

	UPROPERTY(EditAnywhere, Category=DistributionVectorParameterBase)
	TEnumAsByte<DistributionParamMode> ParamModes[3];


	//Begin UDistributionVector Interface
	ENGINE_API virtual FVector GetValue(float F = 0.f, UObject* Data = NULL, int32 Extreme = 0, struct FRandomStream* InRandomStream = NULL) const override;
	virtual bool CanBeBaked() const override { return false; }
	//End UDistributionVector Interface
	
	/** todo document */
	virtual bool GetParamValue(UObject* Data, FName ParamName, FVector& OutVector) const { return false; }

	/**
	 * Return whether or not this distribution can be baked into a FRawDistribution lookup table
	 */
};



