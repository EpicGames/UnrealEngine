// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVectorParameterBase.h"
#include "DistributionVectorParticleParameter.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionVectorParticleParameter : public UDistributionVectorParameterBase
{
	GENERATED_UCLASS_BODY()


	//~ Begin UDistributionVectorParameterBase Interface
	ENGINE_API virtual bool GetParamValue(UObject* Data, FName ParamName, FVector& OutVector) const override;
	//~ End UDistributionVectorParameterBase Interface
};

