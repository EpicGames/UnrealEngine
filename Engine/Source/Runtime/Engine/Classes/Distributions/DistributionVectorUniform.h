// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "DistributionVectorUniform.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionVectorUniform : public UDistributionVector
{
	GENERATED_UCLASS_BODY()

	/** Upper end of FVector magnitude range. */
	UPROPERTY(EditAnywhere, Category=DistributionVectorUniform)
	FVector Max;

	/** Lower end of FVector magnitude range. */
	UPROPERTY(EditAnywhere, Category=DistributionVectorUniform)
	FVector Min;

	/** If true, X == Y == Z ie. only one degree of freedom. If false, each axis is picked independently. */
	UPROPERTY()
	uint32 bLockAxes:1;

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniform)
	TEnumAsByte<enum EDistributionVectorLockFlags> LockedAxes;

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniform)
	TEnumAsByte<enum EDistributionVectorMirrorFlags> MirrorFlags[3];

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniform)
	uint32 bUseExtremes:1;


	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	ENGINE_API virtual FVector	GetValue( float F = 0.f, UObject* Data = NULL, int32 LastExtreme = 0, struct FRandomStream* InRandomStream = NULL ) const override;

	//Begin UDistributionVector Interface

	//@todo.CONSOLE: Currently, consoles need this? At least until we have some sort of cooking/packaging step!
	ENGINE_API virtual ERawDistributionOperation GetOperation() const override;
	ENGINE_API virtual uint8 GetLockFlag() const override;
	ENGINE_API virtual uint32 InitializeRawEntry(float Time, float* Values) const override;
	ENGINE_API virtual	void	GetRange(FVector& OutMin, FVector& OutMax) const override;
	//End UDistributionVector Interface

	/** These two functions will retrieve the Min/Max values respecting the Locked and Mirror flags. */
	ENGINE_API virtual FVector GetMinValue() const;
	ENGINE_API virtual FVector GetMaxValue() const;

	// We have 6 subs, 3 mins and three maxes. They are assigned as:
	// 0,1 = min/max x
	// 2,3 = min/max y
	// 4,5 = min/max z

	//~ Begin FCurveEdInterface Interface
	ENGINE_API virtual int32		GetNumKeys() const override;
	ENGINE_API virtual int32		GetNumSubCurves() const override;
	ENGINE_API virtual FColor	GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const override;
	ENGINE_API virtual float	GetKeyIn(int32 KeyIndex) override;
	ENGINE_API virtual float	GetKeyOut(int32 SubIndex, int32 KeyIndex) override;
	ENGINE_API virtual FColor	GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor) override;
	ENGINE_API virtual void	GetInRange(float& MinIn, float& MaxIn) const override;
	ENGINE_API virtual void	GetOutRange(float& MinOut, float& MaxOut) const override;
	ENGINE_API virtual EInterpCurveMode	GetKeyInterpMode(int32 KeyIndex) const override;
	ENGINE_API virtual void	GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const override;
	ENGINE_API virtual float	EvalSub(int32 SubIndex, float InVal) override;
	ENGINE_API virtual int32		CreateNewKey(float KeyIn) override;
	ENGINE_API virtual void	DeleteKey(int32 KeyIndex) override;
	ENGINE_API virtual int32		SetKeyIn(int32 KeyIndex, float NewInVal) override;
	ENGINE_API virtual void	SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) override;
	ENGINE_API virtual void	SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) override;
	ENGINE_API virtual void	SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent) override;
	//~ Begin FCurveEdInterface Interface

};



