// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "DistributionVectorUniformCurve.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UDistributionVectorUniformCurve : public UDistributionVector
{
	GENERATED_UCLASS_BODY()

	/** Keyframe data for how output constant varies over time. */
	UPROPERTY(EditAnywhere, Category=DistributionVectorUniformCurve)
	FInterpCurveTwoVectors ConstantCurve;

	/** If true, X == Y == Z ie. only one degree of freedom. If false, each axis is picked independently. */
	UPROPERTY()
	uint32 bLockAxes1:1;

	UPROPERTY()
	uint32 bLockAxes2:1;

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniformCurve)
	TEnumAsByte<enum EDistributionVectorLockFlags> LockedAxes[2];

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniformCurve)
	TEnumAsByte<enum EDistributionVectorMirrorFlags> MirrorFlags[3];

	UPROPERTY(EditAnywhere, Category=DistributionVectorUniformCurve)
	uint32 bUseExtremes:1;

	ENGINE_API virtual FVector	GetValue( float F = 0.f, UObject* Data = NULL, int32 LastExtreme = 0, struct FRandomStream* InRandomStream = NULL ) const override;

	//Begin UDistributionVector Interface
	//@todo.CONSOLE: Currently, consoles need this? At least until we have some sort of cooking/packaging step!
	ENGINE_API virtual ERawDistributionOperation GetOperation() const override;
	ENGINE_API virtual uint32 InitializeRawEntry(float Time, float* Values) const override;
	ENGINE_API virtual	void	GetRange(FVector& OutMin, FVector& OutMax) const override;
	//End UDistributionVector Interface

	/** These two functions will retrieve the Min/Max values respecting the Locked and Mirror flags. */
	ENGINE_API virtual FVector GetMinValue() const;
	ENGINE_API virtual FVector GetMaxValue() const;

	/**
	 *	This function will retrieve the max and min values at the given time.
	 */
	ENGINE_API virtual FTwoVectors GetMinMaxValue(float F = 0.f, UObject* Data = NULL) const;

	//~ Begin FCurveEdInterface Interface
	ENGINE_API virtual int32		GetNumKeys() const override;
	ENGINE_API virtual int32		GetNumSubCurves() const override;
	ENGINE_API virtual FColor	GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const override;
	ENGINE_API virtual float	GetKeyIn(int32 KeyIndex) override;
	ENGINE_API virtual float	GetKeyOut(int32 SubIndex, int32 KeyIndex) override;
	ENGINE_API virtual FColor	GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor) override;
	ENGINE_API virtual void	GetInRange(float& MinIn, float& MaxIn) const override;
	ENGINE_API virtual void	GetOutRange(float& MinOut, float& MaxOut) const override;
	ENGINE_API virtual EInterpCurveMode	GetKeyInterpMode(int32 KeyIndex) const override;
	ENGINE_API virtual void	GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const override;
	ENGINE_API virtual float	EvalSub(int32 SubIndex, float InVal) override;
	ENGINE_API virtual int32		CreateNewKey(float KeyIn) override;
	ENGINE_API virtual void	DeleteKey(int32 KeyIndex) override;
	ENGINE_API virtual int32		SetKeyIn(int32 KeyIndex, float NewInVal) override;
	ENGINE_API virtual void	SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) override;
	ENGINE_API virtual void	SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) override;
	ENGINE_API virtual void	SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent) override;
	ENGINE_API virtual void	LockAndMirror(FTwoVectors& Val) const;
	//~ Begin FCurveEdInterface Interface

};



