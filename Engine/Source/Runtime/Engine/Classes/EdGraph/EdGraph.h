// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Templates/Casts.h"
#include "EdGraph/EdGraphNode.h"
#include "BlueprintUtilities.h"
#include "EdGraph.generated.h"

class UEdGraph;
struct FEdGraphEditAction;
struct FPropertyChangedEvent;

USTRUCT()
struct FGraphReference
{
	GENERATED_USTRUCT_BODY()
protected:
	// Reference to the actual graph
	UPROPERTY()
	mutable TObjectPtr<class UEdGraph> MacroGraph;

	// The blueprint the graph is contained within
	UPROPERTY()
	TObjectPtr<class UBlueprint> GraphBlueprint;

	// The graph GUID so we can refind it if it has been renamed
	UPROPERTY()
	FGuid GraphGuid;

public:
	FGraphReference()
		: MacroGraph(NULL)
		, GraphBlueprint(NULL)
	{
	}

	ENGINE_API void PostSerialize(const FArchive& Ar);

	class UBlueprint* GetBlueprint() const
	{
		return GraphBlueprint;
	}

#if WITH_EDITORONLY_DATA
	ENGINE_API void SetGraph(UEdGraph* InGraph);
	ENGINE_API UEdGraph* GetGraph() const;
#endif
};

template<>
struct TStructOpsTypeTraits<FGraphReference> : public TStructOpsTypeTraitsBase2<FGraphReference>
{
	enum 
	{
		WithPostSerialize = true,
	};
};

UCLASS(MinimalAPI)
class UEdGraph : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** The schema that this graph obeys */
	UPROPERTY()
	TSubclassOf<class UEdGraphSchema>  Schema;

	/** Set of all nodes in this graph */
	UPROPERTY()
	TArray<TObjectPtr<class UEdGraphNode>> Nodes;

	/** If true, graph can be edited by the user */
	UPROPERTY()
	uint32 bEditable:1;

	/** 
		If true, graph can be deleted from the whatever container it is in. For FunctionGraphs
		this flag is reset to false on load (unless the function is the construction script or
		AnimGraph)
	*/
	UPROPERTY()
	uint32 bAllowDeletion:1;

	/** If true, graph can be renamed; Note: Graph can also be renamed if bAllowDeletion is true currently */
	UPROPERTY()
	uint32 bAllowRenaming:1;

#if WITH_EDITORONLY_DATA
	/** Child graphs that are a part of this graph; the separation is purely visual */
	UPROPERTY()
	TArray<TObjectPtr<class UEdGraph>> SubGraphs;

	/** Guid for this graph */
	UPROPERTY()
	FGuid GraphGuid;

	/** Guid of interface graph this graph comes from (used for conforming) */
	UPROPERTY()
	FGuid InterfaceGuid;
#endif // WITH_EDITORONLY_DATA

public:
	template <typename NodeType> friend struct FGraphNodeCreator;

	/** Get the schema associated with this graph */
	ENGINE_API const class UEdGraphSchema* GetSchema() const;

	/** Add a listener for OnGraphChanged events */
	ENGINE_API FDelegateHandle AddOnGraphChangedHandler( const FOnGraphChanged::FDelegate& InHandler );

	/** Remove a listener for OnGraphChanged events */
	ENGINE_API void RemoveOnGraphChangedHandler( FDelegateHandle Handle );

	//~ Begin UObject interface
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void BuildSubobjectMapping(UObject* OtherObject, TMap<UObject*, UObject*>& ObjectMapping) const override;
	ENGINE_API virtual void Serialize(FStructuredArchiveRecord Record) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface
#endif

public:

	template <typename NodeClass>
	NodeClass* CreateIntermediateNode()
	{
		NodeClass* Node = (NodeClass*)CreateNode(NodeClass::StaticClass());
		FSetAsIntermediateNode SetAsIntermediate(Node);
		return Node;
	}

	/** 
	 * Add a node to the graph 
	 * @param NodeToAdd	A graph node to add to the graph
	 * @param bUserAction	true if the node was added as the result of a direct user action
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 */
	ENGINE_API virtual void AddNode( UEdGraphNode* NodeToAdd, bool bUserAction = false, bool bSelectNewNode = true );

	/**
	 * Queues up a select operation for a series of nodes in this graph.
	 * 
	 * @param  NodeSelection	The group of nodes you want selected
	 * @param  bFromUI			True if the node was added as the result of a direct user action.
	 */
	ENGINE_API void SelectNodeSet(TSet<const UEdGraphNode*> NodeSelection, bool bFromUI = false);

	/** 
	* Remove a node from this graph
	* 
	* @param NodeToRemove		The node to remove from this graph
	* @param bBreakAllLinks		If true, all links will be broken on the given node. Editor only. Useful for moving nodes to a different graph.
	* @param bAlwaysMarkDirty	(Optional) If true, marks graph as dirtied upon removal.
	* 
	* @return True if the node has been removed from the graph
	*/
	ENGINE_API bool RemoveNode( UEdGraphNode* NodeToRemove, bool bBreakAllLinks = true, bool bAlwaysMarkDirty = true );

	/** Signal to listeners that the graph has changed - prefer to use NotifyNodeChanged when updating a single node */
	ENGINE_API virtual void NotifyGraphChanged();

	/** Signal to listeners that a node has changed in the graph - commonly used to get UI up to date with a data change */
	ENGINE_API void NotifyNodeChanged(const UEdGraphNode* Node);

	/** 
	 * Move all nodes from this graph to another graph
	 * @param DestinationGraph	The graph to move the nodes too 
	 * @param bIsLoading		If true, the node move is occurring during a blueprint load
	 * @param bInIsCompiling	TRUE if the function is being called during compilation, this will eliminate some nodes that will not be compiled
	 */
	ENGINE_API void MoveNodesToAnotherGraph(UEdGraph* DestinationGraph, bool bIsLoading, bool bInIsCompiling);

	/** Finds all the nodes of a given minimum type in the graph */
	template<class MinRequiredType, class ArrayElementType>
	inline void GetNodesOfClassEx(TArray<ArrayElementType*>& OutNodes) const
	{
		for (int32 i = 0; i < Nodes.Num(); i++)
		{
			UEdGraphNode* Node = Nodes[i];
			if (MinRequiredType* TypedNode = Cast<MinRequiredType>(Node))
			{
				OutNodes.Add(TypedNode);
			}
		}
	}

	/** Gets all the nodes in the graph of a given type */
	template<class MinRequiredType>
	inline void GetNodesOfClass(TArray<MinRequiredType*>& OutNodes) const
	{
		GetNodesOfClassEx<MinRequiredType, MinRequiredType>(OutNodes);
	}

	/** Get all children graphs in the specified graph */
	ENGINE_API void GetAllChildrenGraphs(TArray<UEdGraph*>& Graphs) const;

	/** Get parent outer graph, if it exists */
	static ENGINE_API UEdGraph* GetOuterGraph(UObject* Obj);

	/** Util to find a good place for a new node */
	ENGINE_API UE::Slate::FDeprecateVector2DResult GetGoodPlaceForNewNode();

#if WITH_EDITOR
	/** Notify the graph and its associated listeners that a property is about to change  */
	ENGINE_API void NotifyPreChange( const FString& PropertyName );

	/** Notify the graph and associated listeners that a property has changed */
	ENGINE_API void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, const FString& PropertyName );

	/** Add a delegate listening for property change notifications */
	ENGINE_API FDelegateHandle AddPropertyChangedNotifier(const FOnPropertyChanged::FDelegate& InDelegate );

	/** Remove a delegate listening for property changed notifications */
	ENGINE_API void RemovePropertyChangedNotifier(FDelegateHandle InHandle );
#endif

protected:
	ENGINE_API virtual void NotifyGraphChanged( const FEdGraphEditAction& Action );

	/** 
	 * Creates an empty node in this graph. Use FGraphNodeCreator above
	 *
	 * @param NewNodeClass		The node class to create
	 * @param bFromUI			Whether or not the node was created by the user via the UI
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 *
	 * @return A new graph node of the given type
	 */
	ENGINE_API UEdGraphNode* CreateNode( TSubclassOf<UEdGraphNode> NewNodeClass, bool bFromUI, bool bSelectNewNode );

	UEdGraphNode* CreateNode(TSubclassOf<UEdGraphNode> NewNodeClass, bool bSelectNewNode = true)
	{
		return CreateNode( NewNodeClass, false, bSelectNewNode );
	}

	UEdGraphNode* CreateUserInvokedNode(TSubclassOf<UEdGraphNode> NewNodeClass, bool bSelectNewNode = true)
	{
		return CreateNode(NewNodeClass, true, bSelectNewNode);
	}

private:
	/** A delegate that broadcasts a notification whenever the graph has changed. */
	FOnGraphChanged OnGraphChanged;

#if WITH_EDITORONLY_DATA
	/** Delegate to call when a graph's property has changed */
	FOnPropertyChanged		PropertyChangedNotifiers;
#endif
};



/** 
 * Helper object to ensure a graph node is correctly constructed
 *
 * Typical use pattern is:
 * FGraphNodeCreator<NodeType> NodeCreator(Graph);
 * NodeType* Node = NodeCreator.CreateNode();
 * // calls to build out node 
 * Node->MemberVar = ...
 * NodeCreator.Finalize();
 */
template <typename NodeType>
struct FGraphNodeCreator
{
public:
	FGraphNodeCreator(UEdGraph& InGraph)
		: Node(NULL), Graph(InGraph), bPlaced(false)
	{
	}

	/** Create an empty placeable graph node */
	NodeType* CreateNode(bool bSelectNewNode = true, TSubclassOf<NodeType> NodeClass = NodeType::StaticClass())
	{
		Node = (NodeType*)Graph.CreateNode(NodeClass, bSelectNewNode);
		return Node;
	} 

	/** Create an empty placeable graph node */
	NodeType* CreateUserInvokedNode(bool bSelectNewNode = true, TSubclassOf<NodeType> NodeClass = NodeType::StaticClass())
	{
		Node = (NodeType*)Graph.CreateUserInvokedNode(NodeClass, bSelectNewNode);
		return Node;
	}

	/** Call to finalize the node's construction */
	void Finalize()
	{
		check(!bPlaced);
		Node->CreateNewGuid();
		Node->PostPlacedNewNode();
		bPlaced = true;
		if (Node->Pins.Num() == 0)
		{
			Node->AllocateDefaultPins();
		}
	}

	/** Destructor. Ensures that finalized was called */
	~FGraphNodeCreator()
	{
		checkf(bPlaced, TEXT("Created node was not finalized in a FGraphNodeCreator<%s>"), *NodeType::StaticClass()->GetName());
	}

private:
	// Hide copy and assignment operator
	FGraphNodeCreator(const FGraphNodeCreator& rhs);
	FGraphNodeCreator* operator= (const FGraphNodeCreator& rhs);

	/** The created node */
	NodeType* Node;
	/** Graph reference we're creating the node in */
	UEdGraph& Graph;
	/** If the node has placed and finalized */
	bool bPlaced;
};

