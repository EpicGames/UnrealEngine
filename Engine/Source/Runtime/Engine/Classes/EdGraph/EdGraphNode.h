// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/WeakObjectPtr.h"
#include "Types/SlateVector2.h"
#include "EdGraphNode.generated.h"

class INameValidatorInterface;
class SGraphNode;
class SWidget;
class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;
struct FEdGraphPinType;
struct FPropertyChangedEvent;
struct FSlateIcon;
struct FDiffResults;
struct FDiffSingleResult;

class FPropertyLocalizationDataGatherer;
enum class EPropertyLocalizationGathererTextFlags : uint8;

/**
  * Struct used to define information for terminal types, e.g. types that can be contained
  * by a container. Currently can represent strong/weak references to a type (only UObjects), 
  * a structure, or a primitive. Support for "Container of Containers" is done by wrapping 
  * a structure, rather than implicitly defining names for containers.
  */
USTRUCT()
struct FEdGraphTerminalType
{
	GENERATED_USTRUCT_BODY()

	FEdGraphTerminalType()
		: TerminalCategory()
		, TerminalSubCategory()
		, TerminalSubCategoryObject(nullptr)
		, bTerminalIsConst(false)
		, bTerminalIsWeakPointer(false)
		, bTerminalIsUObjectWrapper(false)
	{
	}

	/** Category */
	UPROPERTY()
	FName TerminalCategory;

	/** Sub-category */
	UPROPERTY()
	FName TerminalSubCategory;

	/** Sub-category object */
	UPROPERTY()
	TWeakObjectPtr<UObject> TerminalSubCategoryObject;

	/** Whether or not this pin is a immutable const value */
	UPROPERTY()
	bool bTerminalIsConst;

	/** Whether or not this is a weak reference */
	UPROPERTY()
	bool bTerminalIsWeakPointer;

	/** Whether or not this is a "wrapped" Unreal object ptr type (e.g. TSubclassOf<T> instead of UClass*) */
	UPROPERTY()
	bool bTerminalIsUObjectWrapper;

	/** Creates a TerminalType from the primary portion of the PinType */
	static ENGINE_API FEdGraphTerminalType FromPinType(const FEdGraphPinType& PinType);

	friend FArchive& operator<<(FArchive& Ar, FEdGraphTerminalType& P);

	friend inline bool operator!= (const FEdGraphTerminalType& A, const FEdGraphTerminalType& B)
	{
		return A.TerminalCategory != B.TerminalCategory
			|| A.TerminalSubCategory != B.TerminalSubCategory
			|| A.TerminalSubCategoryObject != B.TerminalSubCategoryObject
			|| A.bTerminalIsConst != B.bTerminalIsConst
			|| A.bTerminalIsWeakPointer != B.bTerminalIsWeakPointer;
	}

	friend inline bool operator==(const FEdGraphTerminalType& A, const FEdGraphTerminalType& B)
	{
		return !(A != B);
	}
};

/** Enum used to define which way data flows into or out of this pin. */
UENUM()
enum EEdGraphPinDirection : int
{
	EGPD_Input,
	EGPD_Output,
	EGPD_MAX,
};

inline const TCHAR* const LexToString(const EEdGraphPinDirection State)
{
	switch (State)
	{
	case EEdGraphPinDirection::EGPD_Input:
		return TEXT("Input");
	case EEdGraphPinDirection::EGPD_Output:
		return TEXT("Output");
	case EEdGraphPinDirection::EGPD_MAX:
	default:
		break;
	}

	checkf(false, TEXT("Missing EEdGraphPinDirection Type: %d"), static_cast<const int32>(State));
	return TEXT("");
}

/** Enum used to define what container type a pin represents. */
UENUM()
enum class EPinContainerType : uint8
{
	None,
	Array,
	Set,
	Map
};

/** Enum to indicate what sort of title we want. */
UENUM()
namespace ENodeTitleType
{
	enum Type : int
	{
		/** The full title, may be multiple lines. */
		FullTitle,
		/** More concise, single line title. */
		ListView,
		/** Returns the editable title (which might not be a title at all). */
		EditableTitle,
		/** Menu Title for context menus to be displayed in context menus referencing the node. */
		MenuTitle,

		MAX_TitleTypes,
	};
}

/** Enum to indicate if a node has advanced-display-pins, and whether they are shown. */
UENUM()
namespace ENodeAdvancedPins
{
	enum Type : int
	{
		/** No advanced pins. */
		NoPins,
		/** There are some advanced pins, and they are shown. */
		Shown,
		/** There are some advanced pins, and they are hidden. */
		Hidden
	};
}

/** Enum to indicate a node's enabled state. */
UENUM()
enum class ENodeEnabledState : uint8
{
	/** Node is enabled. */
	Enabled,
	/** Node is disabled. */
	Disabled,
	/** Node is enabled for development only. */
	DevelopmentOnly
};

inline const TCHAR* const LexToString(const ENodeEnabledState State)
{
	switch (State)
	{
	case ENodeEnabledState::Enabled:
		return TEXT("Enabled");
	case ENodeEnabledState::Disabled:
		return TEXT("Disabled");
	case ENodeEnabledState::DevelopmentOnly:
		return TEXT("DevelopmentOnly");
	default:
		break;
	}

	checkf(false, TEXT("Missing ENodeEnabledState Type: %d"), static_cast<const int32>(State));
	return TEXT("");
}

/** Enum that defines what kind of orphaned pins should be retained. */
enum class ESaveOrphanPinMode : uint8
{
	SaveNone,
	SaveAll,
	SaveAllButExec
};

/** Holds metadata keys, so as to discourage text duplication throughout the engine. */
struct FNodeMetadata
{
	/** Identifies nodes that are added to populate new graphs by default (helps determine if a graph has any user-placed nodes). */
	static ENGINE_API const FName DefaultGraphNode;
private: 
	FNodeMetadata() {}
};

/** This is the context for GetContextMenuActions and GetNodeContextMenuActions calls. */
UCLASS(MinimalAPI)
class UGraphNodeContextMenuContext : public UObject
{
	GENERATED_BODY()

public:

	ENGINE_API UGraphNodeContextMenuContext();

	ENGINE_API void Init(const UEdGraph* InGraph, const UEdGraphNode* InNode, const UEdGraphPin* InPin, bool bInDebuggingMode);

	/** The blueprint associated with this context; may be NULL for non-Kismet related graphs. */
	UPROPERTY()
	TObjectPtr<const UBlueprint> Blueprint;

	/** The graph associated with this context. */
	UPROPERTY()
	TObjectPtr<const UEdGraph> Graph;

	/** The node associated with this context. */
	UPROPERTY()
	TObjectPtr<const UEdGraphNode> Node;

	/** The pin associated with this context; may be NULL when over a node. */
	const UEdGraphPin* Pin;

	/** Whether the graph editor is currently part of a debugging session (any non-debugging commands should be disabled). */
	UPROPERTY()
	bool bIsDebugging;
};

/** Deprecation types for node response. */
enum class EEdGraphNodeDeprecationType
{
	/** The node type is deprecated. */
	NodeTypeIsDeprecated,
	/** The node references a deprecated member or type (e.g. variable or function). */
	NodeHasDeprecatedReference
};

/** Deprecation response message types. */
enum class EEdGraphNodeDeprecationMessageType
{
	/** No message. The Blueprint will compile successfully. */
	None,
	/** Emit the message as a note at compile time. This will appear as a note on the node and in the compiler log. */
	Note,
	/** Emit the message as a Blueprint compiler warning. This will appear as a warning on the node and in the compiler log. */
	Warning
};

/** Deprecation response data. */
struct FEdGraphNodeDeprecationResponse
{
	/** Message type. */
	EEdGraphNodeDeprecationMessageType MessageType = EEdGraphNodeDeprecationMessageType::None;

	/** Message text to display on the node and/or emit to the compile log. */
	FText MessageText;
};

/** Options when getting a Find References search string from a node. */
enum class EGetFindReferenceSearchStringFlags : int8
{
	/** No options. */
	None = 0,
	/** Use search behavior from before there were different Find References options. */
	Legacy = 1 << 0,
	/** Use search syntax to do exact search. */
	UseSearchSyntax = 1 << 1,
};

ENUM_CLASS_FLAGS(EGetFindReferenceSearchStringFlags);

UCLASS(MinimalAPI)
class UEdGraphNode : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	TArray<UEdGraphPin*> Pins;

	/** List of connector pins */
	UPROPERTY()
	TArray<TObjectPtr<class UEdGraphPin_Deprecated>> DeprecatedPins;

	/** X position of node in the editor */
	UPROPERTY()
	int32 NodePosX;

	/** Y position of node in the editor */
	UPROPERTY()
	int32 NodePosY;

	/** Width of node in the editor; only used when the node can be resized */
	UPROPERTY()
	int32 NodeWidth;

	/** Height of node in the editor; only used when the node can be resized */
	UPROPERTY()
	int32 NodeHeight;

	/** Enum to indicate if a node has advanced-display-pins, and if they are shown */
	UPROPERTY()
	TEnumAsByte<ENodeAdvancedPins::Type> AdvancedPinDisplay;

private:
	/** Indicates in what state the node is enabled, which may eliminate it from being compiled */
	UPROPERTY()
	ENodeEnabledState EnabledState;

protected:
	/** When reconstructing a node should the orphaned pins be retained and transfered to the new pin list. */
	ESaveOrphanPinMode OrphanedPinSaveMode;

public:
	/** When true, overrides whatever OrphanedPinSaveMode specifies and behaves as if it were SaveNone. */
	uint8 bDisableOrphanPinSaving:1;

private:
	UPROPERTY()
	uint8 bDisplayAsDisabled:1;

	/** Indicates whether or not the user explicitly set the enabled state */
	UPROPERTY()
	uint8 bUserSetEnabledState:1;

#if WITH_EDITORONLY_DATA
	/** (DEPRECATED) FALSE if the node is a disabled, which eliminates it from being compiled */
	UPROPERTY()
	uint8 bIsNodeEnabled_DEPRECATED:1;

public:

	/** If true, this node can be resized and should be drawn with a resize handle */
	UPROPERTY()
	uint8 bCanResizeNode:1;

#endif // WITH_EDITORONLY_DATA

private:
	/** Whether the node was created as part of an expansion step */
	UPROPERTY()
	uint8 bIsIntermediateNode : 1;

#if WITH_EDITORONLY_DATA
	/** Whether this node is unrelated to the selected nodes or not */
	uint8 bUnrelated : 1;

#endif

public:

	/** Flag to check for compile error/warning */
	UPROPERTY()
	uint8 bHasCompilerMessage:1;

#if WITH_EDITORONLY_DATA
	/** Comment bubble pinned state */
	UPROPERTY()
	uint8 bCommentBubblePinned : 1;

	/** Comment bubble visibility */
	UPROPERTY()
	uint8 bCommentBubbleVisible : 1;

	/** Make comment bubble visible */
	UPROPERTY(Transient)
	uint8 bCommentBubbleMakeVisible : 1;

	/** If true, this node can be renamed in the editor */
	UPROPERTY()
	uint8 bCanRenameNode:1;

	/** Note for a node that lingers until saved */
	UPROPERTY(Transient)
	FText NodeUpgradeMessage;
#endif // WITH_EDITORONLY_DATA

	/** Comment string that is drawn on the node */
	UPROPERTY()
	FString NodeComment;

	/** Flag to store node specific compile error/warning*/
	UPROPERTY()
	int32 ErrorType;
	
	/** Error/Warning description */
	UPROPERTY(BlueprintReadOnly, Category="EdGraphNode")
	FString ErrorMsg;
	
	/** GUID to uniquely identify this node, to facilitate diffing versions of this graph */
	UPROPERTY()
	FGuid NodeGuid;

public:
	/** Is the node actually enabled */
	bool IsNodeEnabled() const
	{
		return (EnabledState == ENodeEnabledState::Enabled)	|| ((EnabledState == ENodeEnabledState::DevelopmentOnly) && IsInDevelopmentMode());
	}

	/** If true, this node can be renamed in the editor */
	ENGINE_API virtual bool GetCanRenameNode() const;

	/** Returns the specific sort of enable state this node wants */
	ENodeEnabledState GetDesiredEnabledState() const
	{
		return EnabledState;
	}

	/** Set the enabled state of the node to a new value */
	void SetEnabledState(ENodeEnabledState NewState, bool bUserAction = true)
	{
		EnabledState = NewState;
		bUserSetEnabledState = bUserAction;
	}

	/** Set whether or not this node should be forced to display as disabled */
	void SetForceDisplayAsDisabled(const bool bInNewDisplayState)
	{
		bDisplayAsDisabled = bInNewDisplayState;
	}

	bool IsDisplayAsDisabledForced() const
	{
		return bDisplayAsDisabled;
	}

	/** Has the user set the enabled state or is it still using the automatic settings? */
	bool HasUserSetTheEnabledState() const
	{
		return bUserSetEnabledState;
	}

#if WITH_EDITOR
	/** Set this node unrelated or not. */
	inline void SetNodeUnrelated(bool bNodeUnrelated)
	{
		bUnrelated = bNodeUnrelated;
	}

	/** Determines whether this node is unrelated to the selected nodes or not. */
	inline bool IsNodeUnrelated() const
	{
		return bUnrelated;
	}
#endif

	/** Determines whether or not the node will compile in development mode. */
	ENGINE_API virtual bool IsInDevelopmentMode() const;

	/** Returns true if this is a disabled automatically placed ghost node (see the DefaultEventNodes ini section) */
	ENGINE_API bool IsAutomaticallyPlacedGhostNode() const;

	/** Marks this node as an automatically placed ghost node (see the DefaultEventNodes ini section) */
	ENGINE_API void MakeAutomaticallyPlacedGhostNode();

	// UObject interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass);
#endif
	// End of UObject interface

#if WITH_EDITOR

private:
	static ENGINE_API TArray<UEdGraphPin*> PooledPins;

public:
	// UObject interface
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	ENGINE_API virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	ENGINE_API virtual void BeginDestroy() override;
	// End of UObject interface

	// Helper conversions for the editor which typically expects the Node positions and sizes in floats instead of int32
	float GetWidth() const { return static_cast<float>(NodeWidth); }
	void SetWidth(float InWidth) { NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(InWidth); }
	float GetHeight() const { return static_cast<float>(NodeHeight); }
	void SetHeight(float InHeight) { NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(InHeight); }
	float GetNodePosX() const { return static_cast<float>(NodePosX); }
	void SetNodePosX(float InX) { NodePosX = UE::LWC::FloatToIntCastChecked<int32>(InX); }
	float GetNodePosY() const { return static_cast<float>(NodePosY); }
	void SetNodePosY(float InY) { NodePosY = UE::LWC::FloatToIntCastChecked<int32>(InY); }
	FVector2f GetSize() const { return FVector2f(GetWidth(), GetHeight()); }
	void SetSize(const FVector2f& InSize) { SetWidth(InSize.X); SetHeight(InSize.Y); }
	FVector2f GetPosition() const { return FVector2f(GetNodePosX(), GetNodePosY()); }
	void SetPosition(const FVector2f& InPosition) { SetNodePosX(InPosition.X); SetNodePosY(InPosition.Y); }

	/** widget representing this node if it exists; Note: This is not safe to use in general and will be removed in the future, as there is no guarantee that only one graph editor/panel is viewing a given graph */
	TWeakPtr<SGraphNode> DEPRECATED_NodeWidget;

	/** Get all pins this node owns */
	const TArray<UEdGraphPin*>& GetAllPins() const { return Pins; }

	struct FNameParameterHelper
	{
		FNameParameterHelper(const FName InNameParameter) : NameParameter(InNameParameter) { }
		FNameParameterHelper(const FString& InNameParameter) : NameParameter(*InNameParameter) { }
		FNameParameterHelper(const TCHAR* InNameParameter) : NameParameter(InNameParameter) { }

		FName operator*() const { return NameParameter; }

	private:
		FName NameParameter;
	};

	/** Parameter struct of less common options for CreatePin */
	struct FCreatePinParams
	{
		FCreatePinParams()
			: ContainerType(EPinContainerType::None)
			, bIsReference(false)
			, bIsConst(false)
			, Index(INDEX_NONE)
		{
		}

		ENGINE_API FCreatePinParams(const FEdGraphPinType& PinType);

		EPinContainerType ContainerType;
		bool bIsReference;
		bool bIsConst;
		int32 Index;
		FEdGraphTerminalType ValueTerminalType;
	};

	/** Create a new pin on this node using the supplied info, and return the new pin */
	UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FName PinCategory, const FName PinName, const FCreatePinParams& PinParams = FCreatePinParams())
	{
		return CreatePin(Dir, PinCategory, NAME_None, nullptr, PinName, PinParams);
	}

	UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FName PinCategory, const FName PinSubCategory, const FName PinName, const FCreatePinParams& PinParams = FCreatePinParams())
	{
		return CreatePin(Dir, PinCategory, PinSubCategory, nullptr, PinName, PinParams);
	}

	UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FName PinCategory, UObject* PinSubCategoryObject, const FName PinName, const FCreatePinParams& PinParams = FCreatePinParams())
	{
		return CreatePin(Dir, PinCategory, NAME_None, PinSubCategoryObject, PinName, PinParams);
	}

	ENGINE_API UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FName PinCategory, const FName PinSubCategory, UObject* PinSubCategoryObject, const FName PinName, const FCreatePinParams& PinParams = FCreatePinParams());

	/** Create a new pin on this node using the supplied pin type, and return the new pin */
	ENGINE_API UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FEdGraphPinType& InPinType, const FName PinName, int32 Index = INDEX_NONE);


	/** Create a new pin on this node using the supplied pin type, and return the new pin */
	//UE_DEPRECATED(4.19, "Remove when removing FString version. Exists just to resolve ambiguity")
	UEdGraphPin* CreatePin(EEdGraphPinDirection Dir, const FEdGraphPinType& InPinType, const TCHAR* PinName, int32 Index = INDEX_NONE)
	{
		return CreatePin(Dir, InPinType, FName(PinName), Index);
	}

	/** Destroys the specified pin, does not modify its owning pin's Pins list */
	static ENGINE_API void DestroyPin(UEdGraphPin* Pin);

	/** Find a pin on this node with the supplied name and optional direction */
	ENGINE_API UEdGraphPin* FindPin(const FName PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const;

	/** Find a pin on this node with the supplied name and optional direction and assert if it is not present */
	UEdGraphPin* FindPinChecked(const FName PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const
	{
		UEdGraphPin* Result = FindPin(PinName, Direction);
		check(Result);
		return Result;
	}

	/** Find a pin on this node with the supplied name and optional direction */
	UEdGraphPin* FindPin(const FString& PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const
	{
		return FindPin(*PinName, Direction);
	}

	/** Find a pin on this node with the supplied name and optional direction and assert if it is not present */
	UEdGraphPin* FindPinChecked(const FString& PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const
	{
		return FindPinChecked(*PinName, Direction);
	}

	/** Find a pin on this node with the supplied name and optional direction */
	ENGINE_API UEdGraphPin* FindPin(const TCHAR* PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const;

	/** Find a pin on this node with the supplied name and optional direction and assert if it is not present */
	UEdGraphPin* FindPinChecked(const TCHAR* PinName, const EEdGraphPinDirection Direction = EGPD_MAX) const
	{
		UEdGraphPin* Result = FindPin(PinName, Direction);
		check(Result);
		return Result;
	}

	/** Find the pin on this node with the supplied guid */
	ENGINE_API UEdGraphPin* FindPinById(const FGuid PinId) const;

	/** Find the pin on this node with the supplied guid and assert if it is not present */
	ENGINE_API UEdGraphPin* FindPinByIdChecked(const FGuid PinId) const;

	/** Find a pin using a user-defined predicate */
	ENGINE_API UEdGraphPin* FindPinByPredicate(TFunctionRef<bool(UEdGraphPin* InPin)> InFunction) const;
	
	/** Find a pin on this node with the supplied name and remove it, returns TRUE if successful */
	ENGINE_API bool RemovePin(UEdGraphPin* Pin);

	/** Returns whether the node was created by UEdGraph::CreateIntermediateNode. */
	bool IsIntermediateNode() const { return bIsIntermediateNode; }

	/** Whether or not this node should be given the chance to override pin names.  If this returns true, then GetPinNameOverride() will be called for each pin, each frame */
	virtual bool ShouldOverridePinNames() const { return false; }

	/** Whether or not struct pins belonging to this node should be allowed to be split or not. */
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const { return false; }

	/** Gets the overridden name for the specified pin, if any */
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const { return FText::GetEmpty(); }

	/** Gets the display name for a pin */
	ENGINE_API virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const;

	/**
	 * Fetch the hover text for a pin when the graph is being edited.
	 * 
	 * @param   Pin				The pin to fetch hover text for (should belong to this node)
	 * @param   HoverTextOut	This will get filled out with the requested text
	 */
	ENGINE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const;

	/** Gets the index for a pin */
	ENGINE_API int32 GetPinIndex(UEdGraphPin* Pin) const;

	/** Gets the pin at a given index 
	* @param Index The zero-based index of the pin to access.
	* @return The pin found at this location or nullptr if invalid index.
	*/
	ENGINE_API UEdGraphPin* GetPinAt(int32 Index) const;

	/** Gets the pin with the given direction, at the given index. Pins of each direction are indexed separately for the purposes of this method */
	ENGINE_API UEdGraphPin* GetPinWithDirectionAt(int32 Index, EEdGraphPinDirection PinDirection) const;

	/** Break all links on this node */
	ENGINE_API void BreakAllNodeLinks(bool bAlwaysMarkDirty = true);

	/** Snap this node to a specified grid size */
	ENGINE_API void SnapToGrid(uint32 GridSnapSize);

	/** Clear error flag */
	void ClearCompilerMessage()
	{
		bHasCompilerMessage = false;
	}

	/** If true, this node whill show the Visual Warning message */
	ENGINE_API virtual bool ShowVisualWarning() const;

	/** Visual Warning tooltip message to show */
	ENGINE_API virtual FText GetVisualWarningTooltipText() const;

	/** Generate a unique pin name, trying to stick close to a passed in name */
	virtual FName CreateUniquePinName(FName SourcePinName) const
	{
		FName PinName(SourcePinName);
		
		int32 Index = 1;
		while (FindPin(PinName) != nullptr)
		{
			++Index;
			PinName = *FString::Printf(TEXT("%s%d"),*SourcePinName.ToString(),Index);
		}

		return PinName;
	}

	/** Returns the graph that contains this node */
	ENGINE_API class UEdGraph* GetGraph() const;

	/** @returns any sub graphs (graphs that have this node as an outer) that this node might contain (e.g. composite, animation state machine etc.).*/
	virtual TArray<UEdGraph*> GetSubGraphs() const { return TArray<UEdGraph*>(); }

	/**
	 * Allocate default pins for a given node, based only the NodeType, which should already be filled in.
	 *
	 * @return	true if the pin creation succeeds, false if there was a problem (such a failure to find a function when the node is a function call).
	 */
	virtual void AllocateDefaultPins() {}

	/** Destroy the specified node */
	ENGINE_API virtual void DestroyNode();

	/**
	 * Refresh the connectors on a node, preserving as many connections as it can.
	 */
	virtual void ReconstructNode() {}

	/**
	 * Removes the specified pin from the node, preserving remaining pin ordering.
	 */
	ENGINE_API virtual void RemovePinAt(const int32 PinIndex, const EEdGraphPinDirection PinDirection);

	/**
	 * Perform any steps necessary prior to copying a node into the paste buffer
	 */
	virtual void PrepareForCopying() {}

	/**
	 * Determine if this node can live in the specified graph
	 */
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const { return IsCompatibleWithGraph(TargetGraph); }

	/**
	 * Determine if this node can be created under the specified schema
     */
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const { return true; }
	
	/**
	 * Determine if a node of this type can be created for the specified graph.
     */
	ENGINE_API virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const;

	/**
	 * Perform any fixups (deep copies of associated data, etc...) necessary after a node has been pasted in the editor
	 */
	virtual void PostPasteNode() {}

	/** Gets the name of this node, shown in title bar */
	ENGINE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const;

	/** Gets the search string to find references to this node */
	UE_DEPRECATED(5.3, "GetFindReferenceSearchString() signature changed. Call the version with flags instead and override the _Impl version.")
	ENGINE_API virtual FString GetFindReferenceSearchString() const;

	/**
	 * Gets the search string to find references to this node
	 * @param InFlags Options for the requested search string, such as whether to use Find-in-Blueprints class member search syntax
	 */
	ENGINE_API FString GetFindReferenceSearchString(EGetFindReferenceSearchStringFlags InFlags) const;

protected:
	virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetFindReferenceSearchString();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

public:
	/** 
	 * Gets the draw color of a node's title bar
	 */
	ENGINE_API virtual FLinearColor GetNodeTitleColor() const;

	/**
	 * Get the draw color for a node's comment popup
	 */
	ENGINE_API virtual FLinearColor GetNodeCommentColor() const;

	/**
	 * Gets the  color of a node's title text
	 */
	ENGINE_API virtual FLinearColor GetNodeTitleTextColor() const;

	/**
	 * Gets the draw color of a node's body tine
	 */
	ENGINE_API virtual FLinearColor GetNodeBodyTintColor() const;

	/**
	 * Gets the tooltip to display when over the node
	 */
	ENGINE_API virtual FText GetTooltipText() const;

	/**
	 * Returns the keywords that should be used when searching for this node
	 *
	 * @TODO: Should search keywords be localized? Probably.
	 */
	ENGINE_API virtual FText GetKeywords() const;
	 
	/**
	 * Returns the link used for external documentation for the graph node
	 */
	virtual FString GetDocumentationLink() const { return FString(); }

	/**
	 * Returns the name of the excerpt to display from the specified external documentation link for the graph node
	 * Default behavior is to return the class name (including prefix)
	 */
	ENGINE_API virtual FString GetDocumentationExcerptName() const;

	/** @return Icon to use in menu or on node */
	ENGINE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const;

	/** Should we show the Palette Icon for this node on the node title */
	virtual bool ShowPaletteIconOnNode() const { return false; }

	/**
	 * Autowire a newly created node.
	 *
	 * @param	FromPin	The source pin that caused the new node to be created (typically a drag-release context menu creation).
	 */
	virtual void AutowireNewNode(UEdGraphPin* FromPin) {}

	// A chance to initialize a new node; called just once when a new node is created, before AutowireNewNode or AllocateDefaultPins is called.
	// This method is not called when a node is reconstructed, etc...
	virtual void PostPlacedNewNode() {}

	/** Called when the DefaultValue of one of the pins of this node is changed in the editor */
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) {}

	/** Called when the connection list of one of the pins of this node is changed in the editor */
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) {}

	/** Called when one of the pins of this node has had its' pin type changed from an external source (like the SPinTypeSelector in the case of kismet) */
	virtual void PinTypeChanged(UEdGraphPin* Pin) {}

	/**
	 * Called when something external to this node has changed the connection list of any of the pins in the node
	 *   - Different from PinConnectionListChanged as this is called outside of any loops iterating over our pins allowing
	 *     us to do things like reconstruct the node safely without trashing pins we are already iterating on
	 *   - Typically called after a user induced action like making a pin connection or a pin break
	 */
	virtual void NodeConnectionListChanged() {}

	/** Shorthand way to access the schema of the graph that owns this node */
	ENGINE_API const UEdGraphSchema* GetSchema() const;

	/** Whether or not this node can be safely duplicated (via copy/paste, etc...) in the graph */
	ENGINE_API virtual bool CanDuplicateNode() const;

	/** Whether or not this node can be deleted by user action */
	ENGINE_API virtual bool CanUserDeleteNode() const;

	/** Whether or not this node allows users to edit the advanced view flag of pins (actually edit the property, not the same as show/hide advanced pins). */
	virtual bool CanUserEditPinAdvancedViewFlag() const { return false; }

	/** Tries to come up with a descriptive name for the compiled output */
	ENGINE_API virtual FString GetDescriptiveCompiledName() const;

	/** Update node size to new value */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f")
	virtual void ResizeNode(const FVector2D& NewSize) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION {}
	ENGINE_API virtual void ResizeNode(const FVector2f& NewSize);

	/**
	 * Returns whether or not this node has dependencies on an external structure
	 * If OptionalOutput isn't null, it should be filled with the known dependencies objects (Classes, Structures, Functions, etc).
	 */
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput = nullptr) const { return false; }

	// Returns true if this node is deprecated
	ENGINE_API virtual bool IsDeprecated() const;

	// Returns true if this node references a deprecated type or member
	virtual bool HasDeprecatedReference() const { return false; }

	// Returns the response to use when reporting a deprecation
	ENGINE_API virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const;

	// Returns the object that should be focused when double-clicking on this node
	// (the object can be an actor, which selects it in the world, or a node/graph/pin)
	ENGINE_API virtual UObject* GetJumpTargetForDoubleClick() const;

	// Returns true if it is possible to jump to the definition of this node (e.g., if it's a variable get or a function call)
	ENGINE_API virtual bool CanJumpToDefinition() const;

	// Jump to the definition of this node (should only be called if CanJumpToDefinition() return true)
	ENGINE_API virtual void JumpToDefinition() const;

	/** Create a new unique Guid for this node */
	ENGINE_API void CreateNewGuid();
#if WITH_EDITORONLY_DATA
	/** Create a new unique Guid for this node deterministically, provided the node's pathname is deterministic */
	ENGINE_API void CreateDeterministicGuid();
#endif

	/** Gets a list of actions that can be done to this particular node */
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const {}

	/** Does the node context menu inherit parent class's menu */
	virtual bool IncludeParentNodeContextMenu() const { return false; }

	// Gives each visual node a chance to do final validation before it's node is harvested for use at runtime
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const {}

	/** Gives the node the option to customize how diffs are discovered within it.  */
	ENGINE_API virtual void FindDiffs(class UEdGraphNode* OtherNode, FDiffResults& Results);

	// This function gets menu items that can be created using this node given the specified context
	virtual void GetMenuEntries(struct FGraphContextMenuBuilder& ContextMenuBuilder) const {}

	// create a name validator for this node
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const { return NULL; }

	// called when this node is being renamed after a successful name validation
	virtual void OnRenameNode(const FString& NewName) {}

	// called to replace this nodes comment text
	ENGINE_API virtual void OnUpdateCommentText( const FString& NewComment );

	// returns true if this node supports comment bubbles
	virtual bool SupportsCommentBubble() const { return true; }

	// called when the node's comment bubble is toggled
	virtual void OnCommentBubbleToggled( bool bInCommentBubbleVisible ) {}

	// called when a pin is removed
	virtual void OnPinRemoved( UEdGraphPin* InRemovedPin ) {}

	/** 
	* Returns whether to draw this node as a control point only (knot/reroute node). Note that this means that the node should only have on input and output pin.
	* @param OutInputPinIndex The index in the pins array associated with the control point input pin.
	* @param OutOutputPinIndex The index in the pins array associated with the control point output pin.
	* @return Whether or not to draw this node as a control point.
	*/
	ENGINE_API virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const;

	/**
	 * Add's node data to the search metadata, override to collect more data that may be desirable to search for
	 *
	 * @param OutTaggedMetaData		Built array of tagged meta data for the node
	 */
	ENGINE_API virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const;

	/**
	 * Adds node pin data to the search metadata, override to collect more data that may be desirable to search for
	 *
	 * @param Pin					The pin for which to gather search meta data
	 * @param OutTaggedMetaData		Built array of tagged meta data for the given pin
	 */
	ENGINE_API virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const;

	/** Return the requested metadata for the pin if there is any */
	virtual FString GetPinMetaData(FName InPinName, FName InKey) { return FString(); }

	/** Return false if the node and any expansion will isolate itself during compile */
	virtual bool IsCompilerRelevant() const { return true; }

	/** Return the matching "pass-through" pin for the given pin (if supported by this node) */
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const { return nullptr; }

	/** If the node has a subgraph, should they be merged into the main graph? */
	virtual bool ShouldMergeChildGraphs() const { return true; }

	/** Create a visual widget to represent this node in a graph editor or graph panel.  If not implemented, the default node factory will be used. */
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() { return TSharedPtr<SGraphNode>(); }

	/** Create the background image for the widget representing this node */
	virtual TSharedPtr<SWidget> CreateNodeImage() const { return TSharedPtr<SWidget>(); }

	/** Adds an upgrade note to this node */
	ENGINE_API void AddNodeUpgradeNote(FText InUpgradeNote);

	/** If the comment bubble needs to be made visible immediately */
	ENGINE_API bool ShouldMakeCommentBubbleVisible() const;

	/** Sets a flag if the comment bubble needs to be made visible immediately */
	ENGINE_API void SetMakeCommentBubbleVisible(bool MakeVisible);

	/** Execute a provided function once for each node that is directly connected to this node, will not include the node itself */
	ENGINE_API void ForEachNodeDirectlyConnected(TFunctionRef<void(UEdGraphNode*)> Func);
	
	/** 
	 * Often we are only interested in a subset of our connections (e.g. only output pins, or only output pins except our exec pin)
	 * This function provides the ability to execute a provided function once for each node that is directly connected to this node, 
	 * but first filters out which of this node's pins to consider:
	 */
	ENGINE_API void ForEachNodeDirectlyConnectedIf(TFunctionRef<bool(const UEdGraphPin* Pin)> Filter, TFunctionRef<void(UEdGraphNode*)> Func);

	/** 
	 * Execute a provided function once for each node that is directly connected to this node's input pins, will not include the node itself
	 * Implementation provides an example usage of ForEachNodeDirectlyConnectedIf.
	 */
	ENGINE_API void ForEachNodeDirectlyConnectedToInputs(TFunctionRef<void(UEdGraphNode*)> Func);

	/** 
	 * Execute a provided function once for each node that is directly connected to this node's output pins, will not include the node itself
	 * Implementation provides an example usage of ForEachNodeDirectlyConnectedIf.
	 */
	ENGINE_API void ForEachNodeDirectlyConnectedToOutputs(TFunctionRef<void(UEdGraphNode*)> Func);
	
	// snapping
	ENGINE_API bool IsSnappedToAny(const TSet<TObjectPtr<UObject>>& NodeSet ) const;
	ENGINE_API UEdGraphNode* GetRootSnappedParent(const TSet<TObjectPtr<UObject>>& NodeSet) const;
	// snapping
protected:
#if WITH_EDITORONLY_DATA
	/** Internal function used to gather pins from a graph node for localization */
	friend void GatherGraphNodeForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	ENGINE_API virtual void GatherForLocalization(FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const;
#endif

	/**
	 * Finds the difference in properties of node instance, for subobjects
	 *
	 * @param StructA The struct of the class we are looking at LHS
	 * @param StructB The struct of the class we are looking at RHS
	 * @param DataA The raw data for the UObject we are comparing LHS
	 * @param DataB The raw data for the UObject we are comparing RHS
	 * @param Results The Results where differences are stored
	 * @param Diff The single result with default parameters setup
	 */
	ENGINE_API virtual void DiffProperties(UClass* StructA, UClass* StructB, UObject* DataA, UObject* DataB, FDiffResults& Results, FDiffSingleResult& Diff) const;

	/**
	 * Finds the difference in properties of node instance, for arbitrary UStructs
	 *
	 * @param StructA The struct we are looking at LHS
	 * @param StructB The struct we are looking at RHS
	 * @param DataA The raw data we are comparing LHS
	 * @param DataB The raw data we are comparing RHS
	 * @param Results The Results where differences are stored
	 * @param Diff The single result with default parameters setup
	 */
	ENGINE_API virtual void DiffProperties(UStruct* StructA, UStruct* StructB, uint8* DataA, uint8* DataB, FDiffResults& Results, FDiffSingleResult& Diff) const;

	// Returns a human-friendly description of the property in the form "PropertyName: Value"
	ENGINE_API virtual FString GetPropertyNameAndValueForDiff(const FProperty* Prop, const uint8* PropertyAddr) const;

#endif // WITH_EDITOR

	friend struct FSetAsIntermediateNode;
};

struct FSetAsIntermediateNode
{
	friend UEdGraph;

private:
	FSetAsIntermediateNode(UEdGraphNode* GraphNode)
	{
		GraphNode->bIsIntermediateNode = true;
	}
};


