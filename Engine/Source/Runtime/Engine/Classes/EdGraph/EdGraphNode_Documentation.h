// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "Textures/SlateIcon.h"
#include "EdGraphNode_Documentation.generated.h"

class INameValidatorInterface;
class UEdGraphPin;
struct Rect;

UCLASS(MinimalAPI)
class UEdGraphNode_Documentation : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** Documentation Link */
	UPROPERTY(meta=(DisplayName="Documentation Link"))
	FString Link;

	/** Documentation Excerpt */
	UPROPERTY(meta=(DisplayName="Documentation Excerpt"))
	FString Excerpt;

public:

#if WITH_EDITOR
	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override {}
	virtual bool ShouldOverridePinNames() const override { return true; }
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor( 0.2f, 1.f, 0.2f ); }
	ENGINE_API virtual FText GetTooltipText() const override;
	ENGINE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	ENGINE_API virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	ENGINE_API virtual void ResizeNode(const FVector2f& NewSize) override;
	ENGINE_API virtual void PostPlacedNewNode() override;
	ENGINE_API virtual void OnRenameNode(const FString& NewName) override;
	ENGINE_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual FString GetDocumentationLink() const override { return Link; }
	virtual FString GetDocumentationExcerptName() const override { return Excerpt; }
	ENGINE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	/** Set the Bounds for the comment node */
	ENGINE_API void SetBounds(const class FSlateRect& Rect);

#endif
};

