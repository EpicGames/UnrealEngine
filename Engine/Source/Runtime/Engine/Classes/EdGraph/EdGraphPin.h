// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphPin.generated.h"

class UEdGraphPin;
enum class EPinResolveType : uint8;

enum class EPinInsertPosition
{
	Before,
	After,
};

USTRUCT()
struct FSimpleMemberReference
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Most often the Class that this member is defined in. Could be a UPackage 
	 * if it is a native delegate signature function (declared globally).
	 */
	UPROPERTY()
	TObjectPtr<UObject> MemberParent;

	/** Name of the member */
	UPROPERTY()
	FName MemberName;

	/** The Guid of the member */
	UPROPERTY()
	FGuid MemberGuid;

	FSimpleMemberReference() : MemberParent(nullptr) {}

	void Reset()
	{
		operator=(FSimpleMemberReference());
	}

	bool operator==(const FSimpleMemberReference& Other) const
	{
		return (MemberParent == Other.MemberParent)
			&& (MemberName == Other.MemberName)
			&& (MemberGuid == Other.MemberGuid);
	}

	/* For backwards compatibility (when MemberParent used to exclusively be a class) */
	UClass* GetMemberParentClass() const
	{
		return Cast<UClass>(MemberParent);
	}
};

inline FArchive& operator<<(FArchive& Ar, FSimpleMemberReference& Data)
{
	Ar << Data.MemberParent;
	Ar << Data.MemberName;
	Ar << Data.MemberGuid;
	return Ar;
}

/** Struct used to define the type of information carried on this pin */
USTRUCT(BlueprintType)
struct FEdGraphPinType
{
	GENERATED_USTRUCT_BODY()

	/** Category of pin type */
	UPROPERTY()
	FName PinCategory;

	/** Sub-category of pin type */
	UPROPERTY()
	FName PinSubCategory;

	/** Sub-category object */
	UPROPERTY()
	TWeakObjectPtr<UObject> PinSubCategoryObject;

	/** Sub-category member reference */
	UPROPERTY()
	FSimpleMemberReference PinSubCategoryMemberReference;

	/** Data used to determine value types when bIsMap is true */
	UPROPERTY()
	FEdGraphTerminalType PinValueType;

	UPROPERTY()
	EPinContainerType ContainerType;

private:
	/** UE_DEPRECATED(4.17) Whether or not this pin represents an array of values */
	UPROPERTY()
	uint8 bIsArray_DEPRECATED:1;

public:
	/** Whether or not this pin is a value passed by reference or not */
	UPROPERTY()
	uint8 bIsReference:1;

	/** Whether or not this pin is a immutable const value */
	UPROPERTY()
	uint8 bIsConst:1;

	/** Whether or not this is a weak reference */
	UPROPERTY()
	uint8 bIsWeakPointer:1;

	/** Whether or not this is a "wrapped" Unreal object ptr type (e.g. TSubclassOf<T> instead of UClass*) */
	UPROPERTY()
	uint8 bIsUObjectWrapper:1;

	/** Set to true if the type was serialized prior to BlueprintPinsUseRealNumbers */
	UPROPERTY()
	uint8 bSerializeAsSinglePrecisionFloat:1;

	inline bool IsContainer() const { return (ContainerType != EPinContainerType::None); }
	inline bool IsArray() const { return (ContainerType == EPinContainerType::Array); }
	inline bool IsSet() const { return (ContainerType == EPinContainerType::Set); }
	inline bool IsMap() const { return (ContainerType == EPinContainerType::Map); }

public:
	FEdGraphPinType() 
		: PinSubCategoryObject(nullptr)
		, ContainerType(EPinContainerType::None)
		, bIsArray_DEPRECATED(false)
		, bIsReference(false)
		, bIsConst(false)
		, bIsWeakPointer(false)
		, bIsUObjectWrapper(false)
		, bSerializeAsSinglePrecisionFloat(false)
	{
	}

	FEdGraphPinType(FName InPinCategory, FName InPinSubCategory, UObject* InPinSubCategoryObject, EPinContainerType InPinContainerType, bool bInIsReference, const FEdGraphTerminalType& InValueTerminalType )
		: PinCategory(InPinCategory)
		, PinSubCategory(InPinSubCategory)
		, PinSubCategoryObject(InPinSubCategoryObject)
		, PinValueType(InValueTerminalType)
		, ContainerType(InPinContainerType)
		, bIsArray_DEPRECATED(false)
		, bIsReference(bInIsReference)
		, bIsConst(false)
		, bIsWeakPointer(false)
		, bIsUObjectWrapper(false)
		, bSerializeAsSinglePrecisionFloat(false)
	{
	}

	bool operator == ( const FEdGraphPinType& Other ) const
	{
		return (PinCategory == Other.PinCategory)
			&& (PinSubCategory == Other.PinSubCategory)
			&& (PinSubCategoryObject == Other.PinSubCategoryObject)
			&& (PinValueType == Other.PinValueType)
			&& (ContainerType == Other.ContainerType)
			&& (bIsReference == Other.bIsReference)
			&& (bIsWeakPointer == Other.bIsWeakPointer)
			&& (PinSubCategoryMemberReference == Other.PinSubCategoryMemberReference)
			&& (bIsConst == Other.bIsConst);
	}
	bool operator != ( const FEdGraphPinType& Other ) const
	{
		return !(*this == Other);
	}

	void ResetToDefaults()
	{
		PinCategory = NAME_None;
		PinSubCategory = NAME_None;
		PinSubCategoryObject = nullptr;
		PinValueType = FEdGraphTerminalType();
		PinSubCategoryMemberReference.Reset();
		ContainerType = EPinContainerType::None;
		bIsReference = false;
		bIsWeakPointer = false;
		bIsUObjectWrapper = false;
		bSerializeAsSinglePrecisionFloat = false;
		bIsConst = false;
	}

	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareCustomVersions(FArchive& Ar);
#endif

	static ENGINE_API FEdGraphPinType GetPinTypeForTerminalType( const FEdGraphTerminalType& TerminalType );
	static ENGINE_API FEdGraphPinType GetTerminalTypeForContainer( const FEdGraphPinType& ContainerType );

	static EPinContainerType ToPinContainerType(const bool bInIsArray, const bool bInIsSet, const bool bInIsMap)
	{
		EPinContainerType ContainerType = EPinContainerType::None;

		if (bInIsArray)
		{
			check(!bInIsSet && !bInIsMap);
			ContainerType = EPinContainerType::Array;
		}
		else if (bInIsSet)
		{
			check(!bInIsMap);
			ContainerType = EPinContainerType::Set;
		}
		else if (bInIsMap)
		{
			ContainerType = EPinContainerType::Map;
		}

		return ContainerType;
	}

};

template<>
struct TStructOpsTypeTraits< FEdGraphPinType > : public TStructOpsTypeTraitsBase2< FEdGraphPinType >
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Weak;
};

UENUM()
enum EBlueprintPinStyleType : int
{
	BPST_Original UMETA(DisplayName="Circles, Grid, Diamond"),
	BPST_VariantA UMETA(DisplayName="Directional Circles")
};

USTRUCT()
struct FEdGraphPinReference
{
	GENERATED_USTRUCT_BODY()

	/** Constructors */
	FEdGraphPinReference() : OwningNode(nullptr) {}
	FEdGraphPinReference(UEdGraphPin* InPin) : OwningNode(nullptr) { SetPin(InPin); }
	FEdGraphPinReference(const UEdGraphPin* InPin) : OwningNode(nullptr) { SetPin(InPin); }

	/** Sets the pin referred to by this struct */
	ENGINE_API void SetPin(const UEdGraphPin* NewPin);

	/** Gets the pin referred to by this struct */
	ENGINE_API UEdGraphPin* Get() const;

	friend uint32 GetTypeHash(const FEdGraphPinReference& EdGraphPinReference)
	{
		return GetTypeHash(EdGraphPinReference.PinId);
	}

	bool operator==(const FEdGraphPinReference& Other) const
	{
		return PinId == Other.PinId && OwningNode == Other.OwningNode;
	}

private:
	/** The node that owns the pin referred to by this struct. Updated at Set and Save time. */
	UPROPERTY()
	TWeakObjectPtr<UEdGraphNode> OwningNode;

	/** The pin's unique ID. Updated at Set and Save time. */
	UPROPERTY()
	FGuid PinId;
};

#if WITH_EDITORONLY_DATA
struct FSnappedEdGraphPin
{
	UEdGraphPin* ParentPin = nullptr;
	UEdGraphPin* ChildPin = nullptr;

	friend auto operator<=>(const FSnappedEdGraphPin&, const FSnappedEdGraphPin&) = default;
};
#endif

enum class EPinResolveType : uint8;

class UEdGraphPin
{
	friend struct FEdGraphPinReference;
private:
	/** The node that owns this pin. */
	UEdGraphNode* OwningNode;


public:
	/** The pin's unique ID. */
	FGuid PinId;

	/** Name of this pin. */
	FName PinName;

	/** Index of the pin in the source data structure represented by the graph, INDEX_NONE if not used */
	int32 SourceIndex;

	/** Direction of flow of this pin (input or output) */
	TEnumAsByte<enum EEdGraphPinDirection> Direction;

#if WITH_EDITORONLY_DATA
	////////////////////////////////////////////////////////////////////////////////////
	// ONLY PUT BITFIELD PROPERTIES AFTER THIS TO ENSURE GOOD MEMORY ALIGNMENT
	////////////////////////////////////////////////////////////////////////////////////

	/** If true, this connector is currently hidden. */
	uint8 bHidden:1;

	/** If true, this connector is unconnectable, and present only to allow the editing of the default text. */
	uint8 bNotConnectable:1;

	/** If true, the default value of this connector is fixed and cannot be modified by the user (it's visible for reference only). */
	uint8 bDefaultValueIsReadOnly:1;

	/** If true, the default value on this pin is ignored and should not be set. */
	uint8 bDefaultValueIsIgnored:1;
	
	/** If true, the pin may be hidden by user. */
	uint8 bAdvancedView:1;

	/** If true, the pin is deprecated. */
	uint8 bDeprecated:1;

	/** If true, the pin is displayed as ref. This is transient. */
	uint8 bDisplayAsMutableRef:1;

	/** If true, the pin's name will be allowed to be made friendly by the editor. */
	uint8 bAllowFriendlyName:1;

	/** 
	* If true, this pin existed on an older version of the owning node, but when the node was reconstructed a matching pin was not found.
	* This pin must be linked to other pins or have a non-default value and will be removed if disconnected, reset to default, or the node is refreshed.
	**/
	uint8 bOrphanedPin:1;

	bool ShouldSavePinIfOrphaned() const { return bSavePinIfOrphaned; }
	ENGINE_API void SetSavePinIfOrphaned(bool bShouldSave);

private:
	/** If true, this pin will be retained when reconstructing a node if there is no matching pin on the new version of the pin. This is transient. */
	uint8 bSavePinIfOrphaned:1;

public:
	/** Older content sometimes had an empty autogenerated default value string in cases where that does not mean the property default value (0, none, false, etc.) */
	uint8 bUseBackwardsCompatForEmptyAutogeneratedValue:1;

private:
	/** True if we have a snapped child pin in our LinkedTo list, corresponding pin will have bHasSnappedParent set */
	uint8 bHasSnappedChild:1;
	
	/** True if we are snapped within a parent pin in our LinkedTo list, corresponding pin will have bHasSnappedChild set */
	uint8 bHasSnappedParent:1;
public:
#endif // WITH_EDITORONLY_DATA

	/** True when InvalidateAndTrash was called. This pin is intended to be discarded and destroyed. */
	uint8 bWasTrashed:1;	/** Used as the display name if set. */

#if WITH_EDITORONLY_DATA
	FText PinFriendlyName;

	/** The deprecated message of the pin. Used when the pin is deprecated. */
	FString DeprecationMessage;
#endif

	/** The tool-tip describing this pin's purpose */
	FString PinToolTip;

	/** The type of information carried on this pin */
	FEdGraphPinType PinType;

	/** Default value for this pin (used if the pin has no connections), stored as a string */
	FString DefaultValue;

	/** Initial default value (the autogenerated value, to identify if the user has modified the value), stored as a string */
	FString AutogeneratedDefaultValue;

	/** If the default value for this pin should be an object, we store a pointer to it */
	TObjectPtr<class UObject> DefaultObject;

	/** If the default value for this pin should be an FText, it is stored here. */
	FText DefaultTextValue;

	/** Set of pins that we are linked to */
	TArray<UEdGraphPin*> LinkedTo;

	/** The pins created when a pin is split and hidden */ 
	TArray<UEdGraphPin*> SubPins;

	/** The pin that was split and generated this pin */
	UEdGraphPin* ParentPin;

	/** Pin that this pin uses for passing through reference connection */
	UEdGraphPin* ReferencePassThroughConnection;

#if WITH_EDITORONLY_DATA
	/** Pin name could be changed, so whenever possible it's good to have a persistent GUID identifying Pin to reconstruct Node seamlessly */
	FGuid PersistentGuid;
#endif

public:

	/** Creates a new pin. Should be called from the OwningNode so it can be immediately added to the Pins array. */
	ENGINE_API static UEdGraphPin* CreatePin(UEdGraphNode* InOwningNode);

	/** Create a link. Note, this does not check that schema allows it, and will not break any existing connections */
	ENGINE_API void MakeLinkTo(UEdGraphPin* ToPin, bool bAlwaysMarkDirty = true);

#if WITH_EDITORONLY_DATA
	ENGINE_API void SetSnappedChild(UEdGraphPin* SnappedChild);
	ENGINE_API void ClearSnappedChild();
	ENGINE_API bool IsSnapped() const;
	ENGINE_API bool IsSnappedParentOf(const UEdGraphNode* Node) const;
	ENGINE_API FSnappedEdGraphPin GetSnapDescription() const;
	ENGINE_API UEdGraphNode* GetSnappedChildNode() const;
	ENGINE_API UEdGraphPin* GetSnappedChildPin() const;
	ENGINE_API UEdGraphPin* GetSnappedParentPin() const;
#endif

	/** Break a link to the specified pin (if present) */
	ENGINE_API void BreakLinkTo(UEdGraphPin* ToPin, bool bAlwaysMarkDirty = true);

	/** Break all links from this pin */
	ENGINE_API void BreakAllPinLinks(bool bNotifyNodes = false, bool bAlwaysMarkDirty = true);

private:
	enum class ETransferPersistentDataMode : uint8
	{
		Move,
		Copy
	};

	template<class T>
	static void TransferPersistentDataFromOldPin(UEdGraphPin& DestPin, T& SourcePin, const ETransferPersistentDataMode TransferMode);

public:
	/**
	* Moves the persistent data (across a node refresh) from the SourcePin.
	*
	* @param	SourcePin	Source pin.
	*/
	ENGINE_API void MovePersistentDataFromOldPin(UEdGraphPin& SourcePin);

	/**
	 * Copies the persistent data (across a node refresh) from the SourcePin.
	 *
	 * @param	SourcePin	Source pin.
	 */
	ENGINE_API void CopyPersistentDataFromOldPin(const UEdGraphPin& SourcePin);

	/** Connects the two pins as by-ref pass-through, allowing the input to auto-forward to the output pin */
	ENGINE_API void AssignByRefPassThroughConnection(UEdGraphPin* InTargetPin);

	/** Returns the node that owns this pin */
	inline class UEdGraphNode* GetOwningNode() const { check(OwningNode); return OwningNode; }
	inline class UEdGraphNode* GetOwningNodeUnchecked() const { return OwningNode; }

	/** Shorthand way to access the schema of the graph that owns the node that owns this pin */
	ENGINE_API const class UEdGraphSchema* GetSchema() const;

	/** Returns true if there are entries in the pin's LinkedTo array, or if it has SubPins, one of their SubPins arrays: */
	ENGINE_API bool HasAnyConnections() const;

	/** Direction flipping utility; returns the complementary direction */
	static EEdGraphPinDirection GetComplementaryDirection(EEdGraphPinDirection InDirection)
	{
		return (InDirection == EGPD_Input) ? EGPD_Output : EGPD_Input;
	}

	/** Helper to safely set a pin's bHidden property only if it has no sub-pins that are influencing it to be hidden */
	void SafeSetHidden(bool bIsHidden)
	{
#if WITH_EDITORONLY_DATA
		 if (SubPins.Num() == 0)
		 {
			 bHidden = bIsHidden;
		 }
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 * Returns an internal string representation of the string/object/text default value, that is compatible with AutogeneratedDefaultValue and ImportText
	 * @note Do not directly compare this value with another to test for a default value. Instead use DoesDefaultValueMatch or DoesDefaultValueMatchAutogenerated on the schema.
	 */
	ENGINE_API FString GetDefaultAsString() const;

	/**
	 * Returns true if GetDefaultAsString will return an empty string.
	 */
	ENGINE_API bool IsDefaultAsStringEmpty() const;

	/** Returns a human readable FText representation of the string/object/text default value */
	ENGINE_API FText GetDefaultAsText() const;

	/** Returns true if the current default value matches the autogenerated default value */
	ENGINE_API bool DoesDefaultValueMatchAutogenerated() const;

#if WITH_EDITORONLY_DATA
	/** Returns how the name of the pin should be displayed in the UI */
	ENGINE_API FText GetDisplayName() const;
#endif // WITH_EDITORONLY_DATA

	/**
	 * Generate a string detailing the link this pin has to another pin.
	 * 
	 * @Param	InFunctionName	String with function name requesting the info
	 * @Param	InInfoData		String detailing the info (EG. Is Not linked to)
	 * @Param	InToPin			The relevant pin
	 */
	const FString GetLinkInfoString( const FString& InFunctionName, const FString& InInfoData, const UEdGraphPin* InToPin ) const;

	/** Reset default values to empty. This should not be called when AutogeneratedDefaultValue needs to be respected! */
	void ResetDefaultValue()
	{
		DefaultValue.Empty();
		DefaultObject = nullptr;
		DefaultTextValue = FText::GetEmpty();
	}

	/** Resets node to default constructor state */
	void ResetToDefaults()
	{
		check(LinkedTo.Num() == 0);

		PinType.ResetToDefaults();

		PinName = NAME_None;
		SourceIndex = INDEX_NONE;
#if WITH_EDITORONLY_DATA
		PinFriendlyName = FText::GetEmpty();
		DeprecationMessage.Reset();
#endif // WITH_EDITORONLY_DATA
		AutogeneratedDefaultValue.Reset();
		ResetDefaultValue();

#if WITH_EDITORONLY_DATA
		bHidden = false;
		bNotConnectable = false;
		bDefaultValueIsReadOnly = false;
		bDefaultValueIsIgnored = false;
		bDeprecated = false;
		bOrphanedPin = false;
		bSavePinIfOrphaned = true;
		bAllowFriendlyName = true;
#endif // WITH_EDITORONLY_DATA
	}

	/** Provides a reference collector with all object references this pin has. Should only be called by the owning node. */
	void AddStructReferencedObjects(class FReferenceCollector& Collector);

	/** Serializes an array of pins as the owner. Only the OwningNode should call this function. */
	static void SerializeAsOwningNode(FArchive& Ar, TArray<UEdGraphPin*>& ArrayRef);
#if WITH_EDITORONLY_DATA
	static void DeclarePinCustomVersions(FArchive& Ar);
#endif

	/** Marks the owning node as modified. */
	ENGINE_API bool Modify(bool bAlwaysMarkDirty = true);

	/** Changes the owning node. This will remove the pin from the old owning node's pin list and add itself to the new node's pin list. */
	ENGINE_API void SetOwningNode(UEdGraphNode* NewOwningNode);

	/** Marks the pin as 'trashed'. *Does not* remove the pin from the Owning Node's Pins list */
	ENGINE_API void MarkAsGarbage();

	/** Returns true if InvalidateAndTrash was ever called on this pin. */
	inline bool WasTrashed() const { return bWasTrashed; }

	/** Transition support for UEdGraphPins */
	ENGINE_API static UEdGraphPin* CreatePinFromDeprecatedPin(class UEdGraphPin_Deprecated* DeprecatedPin);
	ENGINE_API static UEdGraphPin* FindPinCreatedFromDeprecatedPin(class UEdGraphPin_Deprecated* DeprecatedPin);

	/** ExportText/ImportText */
	ENGINE_API bool ExportTextItem(FString& ValueStr, int32 PortFlags) const;
	ENGINE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, class UObject* Parent, FOutputDevice* ErrorText);

	const FString GetName() const { return PinName.ToString(); }
	const FName GetFName() const { return PinName; }
	UEdGraphNode* GetOuter() const { return GetOwningNodeUnchecked(); }
	bool IsPendingKill() const {	return bWasTrashed; }
	ENGINE_API FEdGraphTerminalType GetPrimaryTerminalType() const;

	/** Verification that all pins have been destroyed after shutting down */
	ENGINE_API static void ShutdownVerification();

	ENGINE_API static void Purge();

	/** Destructor */
	~UEdGraphPin();

	/** This needs to be called if you want to use pin data within PostEditUndo */
	ENGINE_API static void ResolveAllPinReferences();

	ENGINE_API static bool AreOrphanPinsEnabled();

	ENGINE_API static void SanitizePinsPostUndoRedo();
private:
	/** Private Constructor. Create pins using CreatePin since all pin instances are managed by TSharedPtr. */
	UEdGraphPin(UEdGraphNode* InOwningNode, const FGuid& PinGuid);

	/** Backward compatibility code to populate this pin with data from the supplied deprecated UEdGraphPin. */
	void InitFromDeprecatedPin(class UEdGraphPin_Deprecated* DeprecatedPin);

	/** Helper function for common destruction logic */
	void DestroyImpl(bool bClearLinks);

	bool Serialize(FArchive& Ar);

	// Helper functions
	static void ConvertConnectedGhostNodesToRealNodes(UEdGraphNode* InNode, bool bAlwaysMarkDirty = true);
	static void ResolveReferencesToPin(UEdGraphPin* Pin, bool bStrictValidation = true);
	static void SerializePinArray(FArchive& Ar, TArray<UEdGraphPin*>& ArrayRef, UEdGraphPin* RequestingPin, EPinResolveType ResolveType);
	static bool SerializePin(FArchive& Ar, UEdGraphPin*& PinRef, int32 ArrayIdx, UEdGraphPin* RequestingPin, EPinResolveType ResolveType, TArray<UEdGraphPin*>& OldPins);
	static FString ExportText_PinReference(const UEdGraphPin* Pin);
	static FString ExportText_PinArray(const TArray<UEdGraphPin*>& PinArray);
	static bool ImportText_PinArray(const TCHAR*& Buffer, TArray<UEdGraphPin*>& ArrayRef, UEdGraphPin* RequestingPin, EPinResolveType ResolveType);
	
#if WITH_EDITORONLY_DATA
	static FString bHasSnappedChildName;
	static FString bHasSnappedParentName;
#endif
};

UCLASS(MinimalAPI)
class UEdGraphPin_Deprecated : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Name of this pin */
	UPROPERTY()
	FString PinName;

#if WITH_EDITORONLY_DATA
	/** Used as the display name if set */
	UPROPERTY()
	FText PinFriendlyName;
#endif

	/** The tool-tip describing this pin's purpose */
	UPROPERTY()
	FString PinToolTip;

	/** Direction of flow of this pin (input or output) */
	UPROPERTY()
	TEnumAsByte<enum EEdGraphPinDirection> Direction;

	/** The type of information carried on this pin */
	UPROPERTY()
	struct FEdGraphPinType PinType;

	/** Default value for this pin (used if the pin has no connections), stored as a string */
	UPROPERTY()
	FString DefaultValue;

	/** Initial default value (the autogenerated value, to identify if the user has modified the value), stored as a string */
	UPROPERTY()
	FString AutogeneratedDefaultValue;

	/** If the default value for this pin should be an object, we store a pointer to it */
	UPROPERTY()
	TObjectPtr<class UObject> DefaultObject;

	/** If the default value for this pin should be an FText, it is stored here. */
	UPROPERTY()
	FText DefaultTextValue;

	/** Set of pins that we are linked to */
	UPROPERTY()
	TArray<TObjectPtr<class UEdGraphPin_Deprecated>> LinkedTo;

	/** The pins created when a pin is split and hidden */ 
	UPROPERTY()
	TArray<TObjectPtr<class UEdGraphPin_Deprecated>> SubPins;

	/** The pin that was split and generated this pin */
	UPROPERTY()
	TObjectPtr<UEdGraphPin_Deprecated> ParentPin;

	/** Pin that this pin uses for passing through reference connection */
	UPROPERTY()
	TObjectPtr<UEdGraphPin_Deprecated> ReferencePassThroughConnection;

#if WITH_EDITORONLY_DATA
	/** If true, this connector is currently hidden. */
	UPROPERTY()
	uint32 bHidden:1;

	/** If true, this connector is unconnectable, and present only to allow the editing of the default text. */
	UPROPERTY()
	uint32 bNotConnectable:1;

	/** If true, the default value of this connector is fixed and cannot be modified by the user (it's visible for reference only) */
	UPROPERTY()
	uint32 bDefaultValueIsReadOnly:1;

	/** If true, the default value on this pin is ignored and should not be set */
	UPROPERTY()
	uint32 bDefaultValueIsIgnored:1;

	/** If true, the pin may be hidden by user */
	UPROPERTY()
	uint32 bAdvancedView:1;

	/** If true, the pin is displayed as ref */
	UPROPERTY(transient)
	uint32 bDisplayAsMutableRef : 1;

	/** Pin name could be changed, so whenever possible it's good to have a persistent GUID identifying Pin to reconstruct Node seamlessly */
	UPROPERTY()
	FGuid PersistentGuid;
#endif // WITH_EDITORONLY_DATA

public:
	// UObject interface
	virtual bool IsSafeForRootSet() const override { return false; }
	// End UObject interface

	/** Legacy fix up for a bug in older EdGraphPins */
	void FixupDefaultValue();
};


