// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ObjectKey.h"
#include "Input/Reply.h"
#include "EdGraphSchema.generated.h"

class FSlateRect;
struct FSlateBrush;
class UEdGraph;
struct FBPVariableDescription;

/** Distinguishes between different graph types. Graphs can have different properties; for example: functions have one entry point, ubergraphs can have multiples. */
UENUM()
enum EGraphType : int
{
	GT_Function,
	GT_Ubergraph,
	GT_Macro,
	GT_Animation,
	GT_StateMachine,
	GT_MAX,
};

/** This is the type of response the graph editor should take when making a connection */
UENUM()
enum ECanCreateConnectionResponse : int
{
	/** Make the connection; there are no issues (message string is displayed if not empty). */
	CONNECT_RESPONSE_MAKE,

	/** Cannot make this connection; display the message string as an error. */
	CONNECT_RESPONSE_DISALLOW,

	/** Break all existing connections on A and make the new connection (it's exclusive); display the message string as a warning/notice. */
	CONNECT_RESPONSE_BREAK_OTHERS_A,

	/** Break all existing connections on B and make the new connection (it's exclusive); display the message string as a warning/notice. */
	CONNECT_RESPONSE_BREAK_OTHERS_B,

	/** Break all existing connections on A and B, and make the new connection (it's exclusive); display the message string as a warning/notice. */
	CONNECT_RESPONSE_BREAK_OTHERS_AB,

	/** Make the connection via an intermediate cast node, or some other conversion node. */
	CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE,

	/** Make the connection by promoting a lower type to a higher type. Ex: Connecting a Float -> Double, float should become a double */
	CONNECT_RESPONSE_MAKE_WITH_PROMOTION,

	CONNECT_RESPONSE_MAX,
};

// Used to opaquely verify that two different persistent entries backing actions are part of the same section/category (e.g., both are variables in the same Blueprint)
struct FEdGraphSchemaActionDefiningObject
{
	FEdGraphSchemaActionDefiningObject(UObject* InObject, void* AdditionalPointer = nullptr, FName AdditionalName = NAME_None)
		: DefiningObject(InObject)
		, DefiningPointer(AdditionalPointer)
		, DefiningName(AdditionalName)
		, bIsEditable((InObject != nullptr) ? !InObject->IsNative() : false)
	{
	}

	friend bool operator==(const FEdGraphSchemaActionDefiningObject& A, const FEdGraphSchemaActionDefiningObject& B)
	{
		return (A.DefiningObject == B.DefiningObject) && (A.DefiningPointer == B.DefiningPointer) && (A.DefiningName == B.DefiningName) && (A.bIsEditable == B.bIsEditable);
	}

	friend bool operator!=(const FEdGraphSchemaActionDefiningObject& A, const FEdGraphSchemaActionDefiningObject& B)
	{
		return !(A == B);
	}

	bool IsPotentiallyEditable() const
	{
		return bIsEditable;
	}
private:
	FObjectKey DefiningObject;
	void* DefiningPointer;
	FName DefiningName;
	bool bIsEditable;
};

/** This structure represents a context dependent action, with sufficient information for the schema to perform it. */
USTRUCT()
struct FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	/** Simple type info. */
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction"); return Type;}
	virtual FName GetTypeId() const { return StaticGetTypeId(); }

private:
	/** The menu text that should be displayed for this node in the creation menu. */
	UPROPERTY()
	FText MenuDescription;

	/** The tooltip text that should be displayed for this node in the creation menu. */
	UPROPERTY()
	FText TooltipDescription;

	/** This is the UI centric category the action fits in (e.g., Functions, Variables). Use this instead of the NodeType.NodeCategory because multiple NodeCategories might visually belong together. */
	UPROPERTY()
	FText Category;

	/** This is just an arbitrary dump of extra text that search will match on, in addition to the description and tooltip, e.g., Add might have the keyword Math. */
	UPROPERTY()
	FText Keywords;
	
#if WITH_EDITORONLY_DATA
	/** Cached chain of pipe separated categories */
	UPROPERTY()
	TArray<FString> CategoryChain;
#endif
public:
	/** This is a priority number for overriding alphabetical order in the action list (higher value  == higher in the list). */
	UPROPERTY()
	int32 Grouping;

	/** Section ID of the action list in which this action belongs. */
	UPROPERTY()
	int32 SectionID;

	UPROPERTY()
	TArray<FString> MenuDescriptionArray;

	UPROPERTY()
	TArray<FString> FullSearchTitlesArray;

	UPROPERTY()
	TArray<FString>  FullSearchKeywordsArray;

	UPROPERTY()
	TArray<FString>  FullSearchCategoryArray;

	UPROPERTY()
	TArray<FString> LocalizedMenuDescriptionArray;

	UPROPERTY()
	TArray<FString> LocalizedFullSearchTitlesArray;

	UPROPERTY()
	TArray<FString>  LocalizedFullSearchKeywordsArray;

	UPROPERTY()
	TArray<FString>  LocalizedFullSearchCategoryArray;

	UPROPERTY()
	FString SearchText;
	FEdGraphSchemaAction() 
		: Grouping(0)
		, SectionID(0)
	{}
	
	virtual ~FEdGraphSchemaAction() = default;

	FEdGraphSchemaAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords = FText(), int32 InSectionID = 0)
		: Grouping(InGrouping)
		, SectionID(InSectionID)
	{
		UpdateSearchData(MoveTemp(InMenuDesc), MoveTemp(InToolTip), MoveTemp(InNodeCategory), MoveTemp(InKeywords));
	}

	/** Whether or not this action can be parented to other actions of the same type. */
	virtual bool IsParentable() const { return false; }

	/** Execute this action, given the graph and schema, and possibly a pin that we were dragged from. Returns a node that was created by this action (if any). */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, bool bSelectNewNode = true) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		return nullptr;
	}
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true)
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PerformAction(ParentGraph, FromPin, FVector2D(Location), bSelectNewNode);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Execute this action, given the graph and schema, and possibly a pin that we were dragged from. Returns a node that was created by this action (if any). */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2D Location, bool bSelectNewNode = true) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		UEdGraphNode* NewNode = nullptr;
		if (FromPins.Num() > 0)
		{
			NewNode = PerformAction(ParentGraph, FromPins[0], UE::Slate::CastToVector2f(Location), bSelectNewNode);
		}
		else
		{
			NewNode = PerformAction(ParentGraph, nullptr, UE::Slate::CastToVector2f(Location), bSelectNewNode);
		}

		return NewNode;
	}
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true)
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PerformAction(ParentGraph, FromPins, FVector2D(Location), bSelectNewNode);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Performs a double click on the action */
	ENGINE_API virtual FReply OnDoubleClick(UBlueprint* InBlueprint);

	// Updates the category of the *action* and refreshes the search text; does not change the persistent backing item
	// (e.g., it will not actually move a user added variable or function to a new category)
	ENGINE_API void CosmeticUpdateCategory(FText NewCategory);

#if WITH_EDITOR
	/** Updates the root category of the action, making any existing categories a subcategory e.g. NewRootCategory|OldRootCategory */
	ENGINE_API void CosmeticUpdateRootCategory(FText NewRootCategory);
#endif

	ENGINE_API void UpdateSearchData(FText NewMenuDescription, FText NewToolTipDescription, FText NewCategory, FText NewKeywords);

	int32 GetSectionID() const
	{
		return SectionID;
	}

	int32 GetGrouping() const 
	{
		return Grouping;
	}

	const FText& GetMenuDescription() const
	{
		return MenuDescription;
	}

	const FText& GetTooltipDescription() const
	{
		return TooltipDescription;
	}

	const FText& GetCategory() const
	{
		return Category;
	}

	const FText& GetKeywords() const
	{
		return Keywords;
	}

	const TArray<FString>& GetMenuDescriptionArray() const
	{
		return MenuDescriptionArray;
	}

	/** Retrieves the full searchable title for this action. */
	const TArray<FString>& GetSearchTitleArray() const
	{
		return FullSearchTitlesArray;
	}

	/** Retrieves the full searchable keywords for this action. */
	const TArray<FString>& GetSearchKeywordsArray() const
	{
		return FullSearchKeywordsArray;
	}

	/** Retrieves the full searchable categories for this action. */
	const TArray<FString>& GetSearchCategoryArray() const
	{
		return FullSearchCategoryArray;
	}

	const TArray<FString>& GetLocalizedMenuDescriptionArray() const
	{
		return LocalizedMenuDescriptionArray;
	}

	/** Retrieves the localized full searchable title for this action. */
	const TArray<FString>& GetLocalizedSearchTitleArray() const
	{
		return LocalizedFullSearchTitlesArray;
	}

	/** Retrieves the localized full searchable keywords for this action. */
	const TArray<FString>& GetLocalizedSearchKeywordsArray() const
	{
		return LocalizedFullSearchKeywordsArray;
	}

	/** Retrieves the localized full searchable categories for this action. */
	const TArray<FString>& GetLocalizedSearchCategoryArray() const
	{
		return LocalizedFullSearchCategoryArray;
	}

	const FString& GetFullSearchText() const
	{
		return SearchText;
	}

	// GC.
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}

	// Moves the item backing this action to the specified category if it is possible (does nothing for native-introduced variables/functions/etc...)
	virtual void MovePersistentItemToCategory(const FText& NewCategoryName) {}

	// Returns the ordering index of this action in the parent container (if the item cannot be reordered then this will return INDEX_NONE)
	virtual int32 GetReorderIndexInContainer() const { return INDEX_NONE; }

	// Reorders this action to be before the other item in the parent container (returns false if they are not in the same container or cannot be reordered)
	virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) { return false; }

	// Returns an opaque handle that can be used to confirm that two different persistent entries backing actions are part of the same section/category
	// (e.g., both are variables in the same Blueprint)
	virtual FEdGraphSchemaActionDefiningObject GetPersistentItemDefiningObject() const { return FEdGraphSchemaActionDefiningObject(nullptr); }

	// Returns true if the action is of the given type.
	virtual bool IsA(const FName& InType) const
	{
		return InType == GetTypeId();
	}

	// Returns true if the action refers to a member or local variable
	virtual bool IsAVariable() const { return false; }

	// Returns true if the action can be renamed
	virtual bool CanBeRenamed() const { return true; }

	// Returns true if the action can be deleted
	virtual bool CanBeDeleted() const { return false; }

	// Can be used to override the icon of the action in the palette
	virtual FSlateBrush const* GetPaletteIcon() const { return nullptr; }

	// Can be used to override the tooltip shown in the palette
	virtual FText GetPaletteToolTip() const { return FText(); }

#if WITH_EDITOR
	// returns the full list of categories, after any pipe (|) separators have been parsed
	ENGINE_API const TArray<FString>& GetCategoryChain() const;
#endif
private:
	ENGINE_API void UpdateSearchText();
};

/** Action to add a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_NewNode"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UEdGraphNode> NodeTemplate;


	FEdGraphSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
	{}

	FEdGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodeTemplate(nullptr)
	{}

	// FEdGraphSchemaAction interface
	ENGINE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	ENGINE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FEdGraphSchemaAction interface

	template <typename NodeType>
	static NodeType* SpawnNodeFromTemplate(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode = true)
	{
		FEdGraphSchemaAction_NewNode Action;
		Action.NodeTemplate = InTemplateNode;

		return Cast<NodeType>(Action.PerformAction(ParentGraph, nullptr, Location, bSelectNewNode));
	}

	static ENGINE_API UEdGraphNode* CreateNode(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& Location, class UEdGraphNode* InNodeTemplate);
};

/** Dummy action, useful for putting messages in the menu */
struct FEdGraphSchemaAction_Dummy : public FEdGraphSchemaAction
{
	static FName StaticGetTypeId() { static FName Type("FEdGraphSchemaAction_Dummy"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FEdGraphSchemaAction_Dummy()
	: FEdGraphSchemaAction()
	{}

	FEdGraphSchemaAction_Dummy(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords = FText(), int32 InSectionID = 0)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InKeywords, InSectionID)
	{}

	virtual bool CanBeRenamed() const override
	{
		return false;
	}
};

/** This is a response from CanCreateConnection, indicating if the connecting action is legal and what the result will be */
struct FPinConnectionResponse
{
public:
	FText Message;

	TEnumAsByte<enum ECanCreateConnectionResponse> Response;

public:
	FPinConnectionResponse()
		: Message(FText::GetEmpty())
		, Response(CONNECT_RESPONSE_MAKE)
		, bIsFatal(false)
	{
	}

	FPinConnectionResponse(const ECanCreateConnectionResponse InResponse, FString InMessage)
		: Message(FText::FromString(MoveTemp(InMessage)))
		, Response(InResponse)
		, bIsFatal(false)
	{
	}

	FPinConnectionResponse(const ECanCreateConnectionResponse InResponse, const ANSICHAR* InMessage)
		: Message(FText::FromString(InMessage))
		, Response(InResponse)
		, bIsFatal(false)
	{
	}

	FPinConnectionResponse(const ECanCreateConnectionResponse InResponse, const WIDECHAR* InMessage)
		: Message(FText::FromString(InMessage))
		, Response(InResponse)
		, bIsFatal(false)
	{
	}

	FPinConnectionResponse(const ECanCreateConnectionResponse InResponse, FText InMessage)
		: Message(MoveTemp(InMessage))
		, Response(InResponse)
		, bIsFatal(false)
	{
	}

	friend bool operator==(const FPinConnectionResponse& A, const FPinConnectionResponse& B)
	{
		return (A.Message.ToString() == B.Message.ToString()) && (A.Response == B.Response) && (A.bIsFatal == B.bIsFatal);
	}	

	/** If a connection can be made without breaking existing connections */
	bool CanSafeConnect() const
	{
		return (Response == CONNECT_RESPONSE_MAKE  || Response == CONNECT_RESPONSE_MAKE_WITH_PROMOTION);
	}

	bool IsFatal() const
	{
		return (Response == CONNECT_RESPONSE_DISALLOW) && bIsFatal;
	}

	void SetFatal()
	{
		Response = CONNECT_RESPONSE_DISALLOW;
		bIsFatal = true;
	}

private:
	bool bIsFatal:1;
};


// This object is a base class helper used when building a list of actions for some menu or palette
struct FGraphActionListBuilderBase
{
public:
	/** A single entry in the list - can contain multiple actions */
	class UE_DEPRECATED(5.5, "ActionGroup has been deprecated - operate only on TSharedPtr<FEdGraphSchemaAction> or const FEdGraphSchemaAction& as appropriate") 
	ActionGroup
	{
	public:
		/** Constructor accepting a single action */
		ENGINE_API ActionGroup( TSharedPtr<FEdGraphSchemaAction> InAction, FString RootCategory = FString());

		/** Constructor accepting multiple actions */
		ENGINE_API ActionGroup( const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, FString RootCategory = FString());

		/** Move constructor and move assignment operator */
		ENGINE_API ActionGroup(ActionGroup && Other);
		ENGINE_API ActionGroup& operator=(ActionGroup && Other);

		/** Copy constructor and assignment operator */
		ENGINE_API ActionGroup(const ActionGroup&);
		ENGINE_API ActionGroup& operator=(const ActionGroup&);

		ENGINE_API ~ActionGroup();
		/**
		 * @return  A reference to the array of strings that represent the category chain
		 */
		 ENGINE_API const TArray<FString>& GetCategoryChain() const;

		/**
		 * Goes through all actions and calls PerformAction on them individually
		 * @param ParentGraph The graph we are operating in the context of.
		 * @param FromPins Optional pins that the action was dragged from.
		 * @param Location The position on the graph to place new nodes.
		 */
		ENGINE_API void PerformAction( class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const UE::Slate::FDeprecateVector2DParameter& Location );
		
		/**
		 * Returns a the string that should be used when searching for matching actions. Looks only at the first action.
		 */
		const FString& GetSearchTextForFirstAction() const { return Actions[0]->GetFullSearchText(); }

		/** Returns the SearchKeywordsArray */
		const TArray<FString>& GetSearchKeywordsArrayForFirstAction() const { return Actions[0]->GetSearchKeywordsArray(); }
		/** Returns the MenuDescriptionArray */
		const TArray<FString>& GetMenuDescriptionArrayForFirstAction() const { return Actions[0]->GetMenuDescriptionArray(); }
		/** Returns the SearchTitleArray */
		const TArray<FString>& GetSearchTitleArrayForFirstAction() const { return Actions[0]->GetSearchTitleArray(); }
		/** Returns the SearchCategoryArray */
		const TArray<FString>& GetSearchCategoryArrayForFirstAction() const { return Actions[0]->GetSearchCategoryArray(); }

		/** Returns the localized SearchKeywordsArray */
		const TArray<FString>& GetLocalizedSearchKeywordsArrayForFirstAction() const { return Actions[0]->GetLocalizedSearchKeywordsArray(); }
		/** Returns the localized MenuDescriptionArray */
		const TArray<FString>& GetLocalizedMenuDescriptionArrayForFirstAction() const { return Actions[0]->GetLocalizedMenuDescriptionArray(); }
		/** Returns the localized SearchTitleArray */
		const TArray<FString>& GetLocalizedSearchTitleArrayForFirstAction() const { return Actions[0]->GetLocalizedSearchTitleArray(); }
		/** Returns the localized SearchCategoryArray */
		const TArray<FString>& GetLocalizedSearchCategoryArrayForFirstAction() const { return Actions[0]->GetLocalizedSearchCategoryArray(); }

		/** All of the actions this entry contains */
		TArray< TSharedPtr<FEdGraphSchemaAction> > Actions;

	private:
		void Move(ActionGroup& Other);
		void Copy(const ActionGroup& Other);

		/**
		 * Concatenates RootCategory with the first action's category (RootCategory
		 * coming first, as a prefix, and the splits the category hierarchy apart
		 * into separate entries.
		 */
		void InitCategoryChain();

		/**
		 * Initializes the search text.
		 */
		void InitSearchText();

		/** The category to list this entry under (could be left empty, as it gets concatenated with the first sub-action's category) */
		FString RootCategory;
		/** The chain of categories */
		TArray<FString> CategoryChain;
	};
private:

	/** All of the action entries */
	TArray< TSharedPtr<FEdGraphSchemaAction> > Entries;

public:

	/** Virtual destructor */
	virtual ~FGraphActionListBuilderBase() = default;

	/** Adds an action entry containing a single action */
	ENGINE_API virtual void AddAction( const TSharedPtr<FEdGraphSchemaAction>& NewAction);
	UE_DEPRECATED(5.5, "Overriding the schema action category is no longer supported, set the category via CosmeticUpdateRootCategory or CosmeticUpdateCategory")
	ENGINE_API virtual void AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction, FString const& Category);

	/** Adds an action entry containing multiple actions */
	UE_DEPRECATED(5.5, "Collections of Action lists were ambiguous and unused - make a compound FEdGraphSchemaAction and Add it")
	ENGINE_API virtual void AddActionList( const TArray<TSharedPtr<FEdGraphSchemaAction> >& NewActions, FString const& Category = FString() );

	/** Appends all the action entries from a different graph action builder */
	ENGINE_API void Append( FGraphActionListBuilderBase& Other );

	/** Returns the current number of entries */
	ENGINE_API int32 GetNumActions() const;

	/** Returns the specified entry */
	ENGINE_API TSharedPtr<FEdGraphSchemaAction>& GetSchemaAction(int32 Index);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup has been deprecated, access TSharedPtr<FEdGraphSchemaAction> directly using GetSchemaAction")
	ENGINE_API ActionGroup GetAction(int32 Index);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Clears the action entries */
	ENGINE_API virtual void Empty();

	// The temporary graph outer to store any template nodes created
	UEdGraph* OwnerOfTemporaries;

public:
	template<typename NodeType>
	NodeType* CreateTemplateNode(UClass* Class=NodeType::StaticClass())
	{
		return NewObject<NodeType>(OwnerOfTemporaries, Class);
	}

	FGraphActionListBuilderBase()
		: OwnerOfTemporaries(nullptr)
	{
	}
};

/** Used to nest all added action under one root category */
struct FCategorizedGraphActionListBuilder : public FGraphActionListBuilderBase
{
public:
	ENGINE_API FCategorizedGraphActionListBuilder(FString Category = FString());

	// FGraphActionListBuilderBase Interface
	ENGINE_API virtual void AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction) override;
	UE_DEPRECATED(5.5, "Overriding the schema action category is no longer supported, specify category via FCategorizedGraphActionListBuilder constructor, or on the FEdGraphSchemaAction")
	ENGINE_API virtual void AddAction(const TSharedPtr<FEdGraphSchemaAction>& NewAction, FString const& Category) override;
	UE_DEPRECATED(5.5, "Collections of Action lists were ambiguous and unused - make a compound FEdGraphSchemaAction and Add it")
	ENGINE_API virtual void AddActionList(const TArray<TSharedPtr<FEdGraphSchemaAction> >& NewActions, FString const& Category = FString()) override;
	// End of FGraphActionListBuilderBase Interface

private:
	/** An additional category that we want all actions listed under (ok if left empty) */
	FString Category;
};

// This context is used when building a list of actions that can be done in the current blueprint
struct FGraphActionMenuBuilder : public FGraphActionListBuilderBase
{
public:
	const UEdGraphPin* FromPin;
public:
	FGraphActionMenuBuilder() : FromPin(nullptr) {}
};

// This context is used when building a list of actions that can be done in the current context
struct FGraphContextMenuBuilder : public FGraphActionMenuBuilder
{
public:
	// The current graph (will never be NULL)
	const UEdGraph* CurrentGraph;

	// The selected objects
	TArray<UObject*> SelectedObjects;
public:
	ENGINE_API FGraphContextMenuBuilder(const UEdGraph* InGraph);
};

/** This is a response from GetGraphDisplayInformation */
struct FGraphDisplayInfo
{
public:
	/** Plain name for this graph */
	FText PlainName;
	/** Friendly name to display for this graph */
	FText DisplayName;
	/** Text to show as tooltip for this graph */
	FText Tooltip;
	/** Optional link to big tooltip documentation for this graph */
	FString DocLink;
	/** Excerpt within doc for big tooltip */
	FString DocExcerptName;

	TArray<FString> Notes;
public:
	FGraphDisplayInfo() 
	{}

	ENGINE_API FString GetNotesAsString() const;
};

#if WITH_EDITORONLY_DATA
struct FGraphSchemaSearchWeightModifiers
{
	float NodeTitleWeight = 0.0f;
	float KeywordWeight = 0.0f;
	float DescriptionWeight = 0.0f;
	float CategoryWeight = 0.0f;
	float WholeMatchLocalizedWeightMultiplier = 0.0f;
	float WholeMatchWeightMultiplier = 0.0f;
	float StartsWithBonusWeightMultiplier = 0.0f;
	float PercentageMatchWeightMultiplier = 0.0f;
	float ShorterMatchWeight = 0.0f;
};

// Helper struct storing the search text array with its weight info
struct FGraphSchemaSearchTextWeightInfo
{
	FGraphSchemaSearchTextWeightInfo(const TArray< FString >* InArray, float InWeightModifier, float* OutDebugWeight)
		: Array(InArray), WeightModifier(InWeightModifier), DebugWeight(OutDebugWeight)
	{}

	const TArray< FString >* Array = nullptr;
	float WeightModifier = 0.0f;
	float* DebugWeight = nullptr;
};

// Helper struct storing the breakdown of the weights assigned to the search text
struct FGraphSchemaSearchTextDebugInfo
{
	virtual ~FGraphSchemaSearchTextDebugInfo() = default;
	
	float TotalWeight = 0.0f;			// Overall weight

	float NodeTitleWeight = 0.0f;		// Weight for the node's title
	float KeywordWeight = 0.0f;			// Weight for the node's keywords
	float DescriptionWeight = 0.0f;		// Weight for the node's description
	float CategoryWeight = 0.0f;		// Weight for the category

	float PercentMatch = 0.0f;			// The calculated whole match percentage
	float PercentMatchWeight = 0.0f;	// Weight for the whole match percentage
	float ShorterMatchWeight = 0.0f;	// Weight for the shorter matched words

	/** Print out the debug info about this weight info to the console */
	ENGINE_API virtual void Print(const TArray<FString>& SearchForKeywords, const FEdGraphSchemaAction& Action) const;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup weight calculation was always performed on the first action, provide that if scoring a FGraphActionListBuilderBase::ActionGroup, or just provide the FEdGraphSchemaAction")
	ENGINE_API virtual void Print(const TArray<FString>& SearchForKeywords, const FGraphActionListBuilderBase::ActionGroup& Action) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
#endif // WITH_EDITORONLY_DATA

UCLASS(abstract, MinimalAPI)
class UEdGraphSchema : public UObject
{
	GENERATED_UCLASS_BODY()


	/**
	 * Get all actions that can be performed when right clicking on a graph or drag-releasing on a graph from a pin
	 *
	 * @param [in,out]	ContextMenuBuilder	The context (graph, dragged pin, etc...) and output menu builder.
	 */
	 ENGINE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const;

	/** Returns context menu name */
	ENGINE_API FName GetContextMenuName() const;

	/** Returns parent context menu name */
	ENGINE_API virtual FName GetParentContextMenuName() const;

	/** Returns context menu name for a given class */
	static ENGINE_API FName GetContextMenuName(UClass* InClass);

	/**
	 * Gets actions that should be added to the right-click context menu for a node or pin
	 * 
	 * @param	Menu				The menu to append actions to.
	 * @param	Context				The menu's context.
	 */
	ENGINE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const;
	
	/**
	 * Determine if a connection can be created between two pins.
	 *
	 * @param	A	The first pin.
	 * @param	B	The second pin.
	 *
	 * @return	An empty string if the connection is legal, otherwise a message describing why the connection would fail.
	 */
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Not implemented by this schema"));
	}

	/**
	 * Determine if two nodes can be merged
	 *
	 * @param	A	The first node.
	 * @param	B	The second node.
	 *
	 * @return	An empty string if the merge is legal, otherwise a message describing why the merge would fail.
	 */
	virtual const FPinConnectionResponse CanMergeNodes(const UEdGraphNode* A, const UEdGraphNode* B) const 
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Not implemented by this schema"));
	}


	// Categorizes two pins into an input pin and an output pin.  Returns true if successful or false if they don't make sense as such (two inputs or two outputs)
	template<typename PinType>
	static bool CategorizePinsByDirection(PinType* PinA, PinType* PinB, /*out*/ PinType*& InputPin, /*out*/ PinType*& OutputPin)
	{
		InputPin = nullptr;
		OutputPin = nullptr;

		bool bResult = false;
		if (ensure(PinA && PinB))
		{
			if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Output))
			{
				InputPin = PinA;
				OutputPin = PinB;
				bResult = true;
			}
			else if ((PinB->Direction == EGPD_Input) && (PinA->Direction == EGPD_Output))
			{
				InputPin = PinB;
				OutputPin = PinA;
				bResult = true;
			}
		}
		return bResult;
	}

	/**
	 * Try to make a connection between two pins.
	 *
	 * @param	A	The first pin.
	 * @param	B	The second pin.
	 *
	 * @return	True if a connection was made/broken (graph was modified); false if the connection failed and had no side effects.
	 */
	ENGINE_API virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const;

	/** Is this schema supporting connection relinking for the given pin? */
	ENGINE_API virtual bool IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const;

	/**
	 * Determine if a connection can be relinked to the given pin.
	 * @param[in] OldSourcePin The current source pin of the connection.
	 * @param[in] TargetPinCandidate The target pin of the relink.
	 * @return A message describing if the operation can succeed or why the relink operation would fail.
	 */
	ENGINE_API virtual const FPinConnectionResponse CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const;

	/** Try relinking the connection starting at the old source and target pins and relink it to the new target pin. */
	ENGINE_API virtual bool TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const;

	/**
	 * Try to create an automatic cast or other conversion node node to facilitate a connection between two pins.
	 * It makes the cast node, a connection between A and the cast node, and a connection from the cast node to B.two 
	 * This method is called when a connection is made where CanCreateConnection returned bCanAutoConvert.
	 *
	 *
	 * @param	A	The first pin.
	 * @param	B	The second pin.
	 *
	 * @return	True if a cast node and connection were made; false if the connection failed and had no side effects.
	 */
	ENGINE_API virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* A, UEdGraphPin* B) const;

	/**
	* Try to create a promotion from one type to another in order to make a connection between two pins.
	* 
	* @param	A	The first pin.
	* @param	B	The second pin.
	*
	* @return	True if the promotion and connection were successful; False if the connection failed.
	*/
	ENGINE_API virtual bool CreatePromotedConnection(UEdGraphPin* A, UEdGraphPin* B) const;
	
#if WITH_EDITORONLY_DATA
	/**
	 * @return	True if the schema would like for failed connections to show the node panel menu so the user can pick an appropriate node
	 *
	 * Implementaion notes: Consider returning true if your node context menu handles muliple context pins for node indexing and creation.
	 * When the user attempts to connect two incompatible pins this will allow them to find a node which fullfills the requirements.
	 */
	ENGINE_API bool ShouldShowPanelContextMenuForIncompatibleConnections() const;
#endif

	/**
	 * Determine if the supplied pin default values would be valid.
	 *
	 * @param	Pin			   	The pin to check the default value on.
	 *
	 * @return	An empty string if the new value is legal, otherwise a message describing why it is invalid.
	 */
	virtual FString IsPinDefaultValid(const UEdGraphPin* Pin, const FString& NewDefaultValue, TObjectPtr<UObject> NewDefaultObject, const FText& InNewDefaultText) const  { return TEXT("Not implemented by this schema"); }

	/** 
	 *	Determine whether the current pin default values are valid
	 *	@see IsPinDefaultValid
	 */
	ENGINE_API FString IsCurrentPinDefaultValid(const UEdGraphPin* Pin) const;

	/**
	 * An easy way to check to see if the current graph system supports pin watching.
	 * 
	 * @return True if the schema supports pin watching, false if not.
	 */
	virtual bool DoesSupportPinWatching() const	{ return false; }

	/**
	 * Checks to see if the specified pin is being watched by the graph's debug system.
	 * 
	 * @param  Pin	The pin you want to check for.
	 * @return True if the supplied pin is currently being watched, false if not.
	 */
	virtual bool IsPinBeingWatched(UEdGraphPin const* Pin) const { return false; }

	/**
	 * If the specified pin is currently being watched, then this will clear the 
	 * watch from the graph's debug system.
	 * 
	 * @param  Pin	The pin you wish to no longer watch.
	 */
	virtual void ClearPinWatch(UEdGraphPin const* Pin) const {}

	/**
	 * Checks to see if a pin supports Pin Value Inspection Tooltips
	 *
	 * @param	Pin The pin to check
	 * @return	true if it supports data tooltips
	 */
	virtual bool CanShowDataTooltipForPin(const UEdGraphPin& Pin) const { return false; }

	/**
	 * Sets the string to the specified pin; even if it is invalid it is still set.
	 *
	 * @param	Pin			   	The pin on which to set the default value.
	 * @param	NewDefaultValue	The new default value.
	 * @param   bMarkAsModified Marks the container of the value as modified
	 */
	ENGINE_API virtual void TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified = true) const;

	/** Sets the object to the specified pin */
	ENGINE_API virtual void TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified = true) const;

	/** Sets the text to the specified pin */
	ENGINE_API virtual void TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified = true) const;

	/** Returns if the pin's value matches the given value */
	ENGINE_API virtual bool DoesDefaultValueMatch(const UEdGraphPin& InPin, const FString& InValue) const;

	/** Returns if the pin's value matches what the true (autogenerated) default value for that pin would be */
	ENGINE_API virtual bool DoesDefaultValueMatchAutogenerated(const UEdGraphPin& InPin) const;

	/** Resets a pin back to it's autogenerated default value, optionally calling the default value change callbacks */
	virtual void ResetPinToAutogeneratedDefaultValue(UEdGraphPin* Pin, bool bCallModifyCallbacks = true) const { }

	/** If we should disallow viewing and editing of the supplied pin */
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const { return false; }

	/** Should the Pin in question display an asset picker */
	virtual bool ShouldShowAssetPickerForPin(UEdGraphPin* Pin) const { return true; }

	/** Returns true if the schema supports the pin type through the schema action */
	virtual bool SupportsPinType(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType) const { return true; }

	/** Returns true if the schema supports the pin type through the schema action */
	virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const { return true; }

	/**
	 * Gets the draw color of a pin based on it's type.
	 *
	 * @param	PinType	The type to convert into a color.
	 *
	 * @return	The color representing the passed in type.
	 */
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const { return FLinearColor::Black; }

	/**
	 * Gets the draw color of a pin based on the UEdGraphPin.
	 * Will return GetPinTypeColor by default
	 *
	 * @param	InPin The pin to convert into a color.
	 *
	 * @return	The color representing the passed in pin
	 */
	virtual FLinearColor GetPinColor(const UEdGraphPin* InPin) const { return InPin ? GetPinTypeColor(InPin->PinType) : FLinearColor::Black; }

	virtual FLinearColor GetSecondaryPinTypeColor(const FEdGraphPinType& PinType) const { return FLinearColor::White; };

#if WITH_EDITORONLY_DATA
	/** Get the name to show in the editor */
	ENGINE_API virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const;

	/**
	 * Returns true if this schema is using fuzzy string matching for searches 
	 * 
	 * @param InCurrentAction				The current graph action to calculate the weight of.
	 */
	virtual bool AllowsFuzzyActionFiltering(const FEdGraphSchemaAction& InCurrentAction) const { return false; }

	/**
	 * Calculate the weight priority of a given action for the context menu. 
	 * 
	 * @param InCurrentAction				The current graph action to calculate the weight of. Higher weight = higher preference
	 * @param InFilterTerms					Filter terms that the user has entered into the search box
	 * @param InSanitizedFilterTerms		Sanitized search filters in all caps with no symbols or spaces
	 * @param DraggedFromPins				Any pins that this action was dragged off of
	 */
	ENGINE_API virtual float GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup weight calculation was always performed on the first action, provide that if scoring a FGraphActionListBuilderBase::ActionGroup, or just provide the FEdGraphSchemaAction")
	ENGINE_API virtual float GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Get the weight modifiers from the console variable settings */
	ENGINE_API virtual FGraphSchemaSearchWeightModifiers GetSearchWeightModifiers() const;
#endif // WITH_EDITORONLY_DATA

	/**
	 * Takes the PinDescription and tacks on any other data important to the 
	 * schema (things like the pin's type, etc.). The base one here just spits 
	 * back the PinDescription.
	 * 
	 * @param   Pin				The pin you want a tool-tip generated for
	 * @param   PinDescription	A detailed description, describing the pin's purpose
	 * @param   TooltipOut		The constructed tool-tip (out)
	 */
	ENGINE_API virtual void ConstructBasicPinTooltip(UEdGraphPin const& Pin, FText const& PinDescription, FString& TooltipOut) const;

	/** @return     The type of graph (function vs. ubergraph) that this that TestEdGraph is. */
	//@TODO: This is too K2-specific to be included in EdGraphSchema and should be refactored
	virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const { return GT_Function; }

	/**
	 * Query if the passed in pin is a title bar pin.
	 *
	 * @param	Pin	The pin to check.
	 *
	 * @return	true if the pin should appear in the title bar.
	 */
	virtual bool IsTitleBarPin(const UEdGraphPin& Pin) const { return false; }

	/**
	 * Breaks all links from/to a single node
	 *
	 * @param	TargetNode	The node to break links on
	 */
	ENGINE_API virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const;

	/** */
	static ENGINE_API bool SetNodeMetaData(UEdGraphNode* Node, FName const& KeyValue);

	/**
	 * Breaks all links from/to a single pin
	 *
	 * @param	TargetPin	The pin to break links on
	 * @param	bSendsNodeNotifcation	whether to send a notification to the node post pin connection change
	 */
	ENGINE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const;

	/**
	 * Breaks the link between two nodes.
	 *
	 * @param	SourcePin	The pin where the link begins.
	 * @param	TargetLink	The pin where the link ends.
	 */
	ENGINE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const;

	/** Split a pin in to subelements */
	virtual void SplitPin(UEdGraphPin* Pin, bool bNotify = true) const { };

	/** Collapses a pin and its siblings back in to the original pin */
	virtual void RecombinePin(UEdGraphPin* Pin) const { };

	/** Handles double-clicking on a pin<->pin connection */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2D& GraphPosition) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION { }
	virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OnPinConnectionDoubleCicked(PinA, PinB, FVector2D(GraphPosition));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Break links on this pin and create links instead on MoveToPin
	 * 
	 * @param	MoveFromPin			Pin we are breaking links from
	 * @param	MoveToPin			Pin we are copying links to
	 * @param	bIsIntermediateMove	Allows linking to transient pins, should only be true when called from utility functions
	 * @param	bNotifyLinkedNodes	If true, it will notify linked nodes if it fails to move connection, this allows type fixup
	 */
	ENGINE_API virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove = false, bool bNotifyLinkedNodes = false) const;
	 
	/** Copies pin links from one pin to another without breaking the original links */
	ENGINE_API virtual FPinConnectionResponse CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy = false) const;

	/** Is self pin type? */
	virtual bool IsSelfPin(const UEdGraphPin& Pin) const   {return false;}

	/** Is given string a delegate category name ? */
	virtual bool IsDelegateCategory(const FName Category) const   {return false;}

	/** 
	 * Populate new graph with any default nodes
	 * 
	 * @param	Graph			Graph to add the default nodes to
	 * @param	ContextClass	If specified, the graph terminators will use this class to search for context for signatures (i.e. interface functions)
	 */
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const {}

	/**
	 * Reconstructs a node
	 *
	 * @param	TargetNode	The node to reconstruct
	 * @param	bIsBatchRequest	If true, this reconstruct node is part of a batch.  Allows subclasses to defer marking classes as dirty until they are all done
	 */
	ENGINE_API virtual void ReconstructNode(UEdGraphNode& TargetNode, bool bIsBatchRequest=false) const;

	/**
	 * Attempts to construct a substitute node that is unique within its graph. If this call returns non-null node, it is expected for the caller to destroy the node that was passed in.
	 *
	 * @param	Node			The node to replace
	 * @param	Graph			The destination graph
	 * @param	InstanceGraph	Object instancing graph
	 * @param	InOutExtraNames	List of extra names that are in-use from the substitution should be added to this list to prevent other substitutions from attempting to use them
	 * @return					NULL if a substitute node cannot be created; otherwise, the substitute node instance
	 */
	virtual UEdGraphNode* CreateSubstituteNode(UEdGraphNode* Node, const UEdGraph* Graph, FObjectInstancingGraph* InstanceGraph, TSet<FName>& InOutExtraNames) const { return nullptr; }

	/**
	 * Sets a node's position.
	 *
	 * @param	Node			The node to set
	 * @param	Position		The target position
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	ENGINE_API virtual void SetNodePosition(UEdGraphNode* Node, const FVector2D& Position) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	ENGINE_API virtual void SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const;

	/**
	 * Returns the currently selected graph node count
	 *
	 * @param	Graph			The active graph to find the selection count for
	 */
	virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const { return 0; }

	/** Returns schema action to create comment from implemention */
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const { return nullptr; }

	/**
	 * Handle a graph being removed by the user (potentially removing associated bound nodes, etc...)
	 */
	virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const {}

	/*
	 * Try to delete the graph through the schema, return true if successful
	 */
	virtual bool TryDeleteGraph(UEdGraph* GraphToDelete) const { return false; }

	/*
	 * Try to rename a graph through the schema, return true if successful
	 */
	virtual bool TryRenameGraph(UEdGraph* GraphToRename, const FName& InNewName) const { return false; }

	/*
	 * Try to retrieve the event child actions for a given graph
	 */
	virtual bool TryToGetChildEvents(const UEdGraph* Graph, const int32 SectionId, TArray<TSharedPtr<FEdGraphSchemaAction>>& Actions, const FText& ParentCategory) const { return false; }

	/**
	 * Can TestNode be encapsulated into a child graph?
	 */
	virtual bool CanEncapuslateNode(UEdGraphNode const& TestNode) const { return true; }

	/*
	 * Can the function graph be dropped into another graph
	 */
	virtual bool CanGraphBeDropped(TSharedPtr<FEdGraphSchemaAction> InAction) const { return false; }

	/*
	 * Returns a custom reference string for searching within the blueprint based on a given action
	 */
	virtual FString GetFindReferenceSearchTerm(const FEdGraphSchemaAction* InGraphAction) const { return FString(); }

	/*
	 * Begins a drag and drop action to drag a graph action into another graph
	 */
	UE_DEPRECATED(5.0, "Use version that takes FPointerEvent instead.")
	ENGINE_API virtual FReply BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction) const;

	/*
	* Begins a drag and drop action to drag a graph action into another graph
	*/
	ENGINE_API virtual FReply BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent) const;

	/**
	 * Gets display information for a graph
	 *
	 * @param	Graph				Graph to get information on
	 * @param	[out] DisplayInfo	Appropriate display info for Graph
	 */
	ENGINE_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const;

	/**
	 * Returns an optional category for a graph
	 *
	 * @param	InGraph				Graph to get the category for
	 */
	virtual FText GetGraphCategory(const UEdGraph* InGraph) const { return FText(); }

	/**
	 * Tentatively sets the category for a given graph
	 *
	 * @param	InGraph				Graph to set the category for
	 * @param	InCategory			Pipe "|" separated category for the graph.
	 */
	ENGINE_API virtual FReply TrySetGraphCategory(const UEdGraph* InGraph, const FText& InCategory) const;

	/** Called when asset(s) are dropped onto a graph background. */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2D& GraphPosition, UEdGraph* Graph) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION {}
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DroppedAssetsOnGraph(Assets, FVector2D(GraphPosition), Graph);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Called when asset(s) are dropped onto the specified node */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2D& GraphPosition, UEdGraphNode* Node) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION {}
	virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DroppedAssetsOnNode(Assets, FVector2D(GraphPosition), Node);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Called when asset(s) are dropped onto the specified pin */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual void DroppedAssetsOnPin(const TArray<struct FAssetData>& Assets, const FVector2D& GraphPosition, UEdGraphPin* Pin) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION {}
	virtual void DroppedAssetsOnPin(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DroppedAssetsOnPin(Assets, FVector2D(GraphPosition), Pin);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Allows schema to generate a tooltip (icon & message) when the specified asset(s) are dragged over the specified node */
	virtual void GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const 
	{ 
		OutTooltipText = FString();
		OutOkIcon = false;
	}

	/** Allows schema to generate a tooltip (icon & message) when the specified asset(s) are dragged over the specified pin */
	virtual void GetAssetsPinHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const 
	{ 
		OutTooltipText = FString();
		OutOkIcon = false;
	}

	/** Allows schema to generate a tooltip (icon & message) when the specified asset(s) are dragged over the specified graph */
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
	{
		OutTooltipText = FString();
		OutOkIcon = false;
	}

	/* Can this graph type be duplicated? */
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const { return true; }

	/* Duplicate a given graph return the duplicate graph */
	virtual UEdGraph* DuplicateGraph(UEdGraph* GraphToDuplicate) const { return nullptr;}

	/* Indicates whether variants of functions are allowed */
	virtual bool AllowsFunctionVariants() const { return false; }

	/* returns new FConnectionDrawingPolicy from this schema */
	virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const { return nullptr; }

	/* When dragging off a pin, we want to duck the alpha of some nodes */
	virtual bool FadeNodeWhenDraggingOffPin(const UEdGraphNode* Node, const UEdGraphPin* Pin) const { return false; }

	virtual void BackwardCompatibilityNodeConversion(UEdGraph* Graph, bool bOnlySafeChanges) const { }

	/* When a node is removed, this method determines whether we should remove it immediately or use the old (slower) code path that results in all node being recreated: */
	virtual bool ShouldAlwaysPurgeOnModification() const { return true; }

	/** 
	 * Perform any logic necessary to safely remove this node from the graph.  
	 * @param Graph		Type of pin to drop onto the node
	 * @param Node		Direction of the source pin
	 * @return			Whether or not the node was successfully deleted.
    */
	virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const { return false; }

	/*
	 * Some schemas have nodes that support the user dynamically adding pins when dropping a connection on the node
	 *
	 * @param InTargetNode					Node to check for pin adding support
	 * @param InSourcePinName				Name of the pin being dropped, a new pin of similar name will be constructed
	 * @param InSourcePinType				Type of pin to drop onto the node
	 * @param InSourcePinDirection			Direction of the source pin
	 * @return								Returns the new pin if created
	 */
	virtual UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const { return nullptr; }

	/**
	 * Checks if the node supports dropping a pin on it
	 *
	 * @param InTargetNode					Node to check for pin adding support
	 * @param InSourcePinType				Type of pin to drop onto the node
	 * @param InSourcePinDirection			Direction of the source pin
	 * @param OutErrorMessage				Only filled with an error if there is pin add support but there is an error with the pin type
	 * @return								Returns TRUE if there is support for dropping the pin on the node
	 */
	virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const { return false; }

	/**
	 * Let's the schema know about the next pin being dropped
	 *
	 * @param InSourcePin					The pin which is about to be dropped
	 */
	virtual void SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const {}

	/**
	 * Checks if a CacheRefreshID is out of date
	 *
	 * @param InVisualizationCacheID	The current refresh ID to check if out of date
	 * @return							TRUE if dirty
	 */
	virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const { return false; }

	/** Returns the current cache title refresh ID that is appropriate for the passed node */
	virtual int32 GetCurrentVisualizationCacheID() const { return 0; }

	/** Forces cached visualization data to refresh */
	virtual void ForceVisualizationCacheClear() const {};

	/** 
	 * Check whether new nodes can be user-created (by dragging off pins etc.) 
	 * @param	InSourcePin	The pin we dragged off
	 * @return the response to making a new connection
	 */
	virtual FPinConnectionResponse CanCreateNewNodes(UEdGraphPin* InSourcePin) const { return FPinConnectionResponse(); }

	/**
	 * Check whether variables can be dropped onto the graph
	 * @param	InGraph The graph the drop is subject of
	 * @param	InVariableToDrop The variable we want to drop
	 * @return the response to rejection or allowing the drop
	 */
	virtual bool CanVariableBeDropped(UEdGraph* InGraph, FProperty* InVariableToDrop) const { return false; }

	/**
	 * Request to drop a variable on a panel
	 * @param	InGraph The graph the drop is subject of
	 * @param	InVariableToDrop The variable we want to drop
	 * @param	InDropPosition The position inside of the graph
	 * @param	InScreenPosition The position inside of the screen
	 * @return the response to rejection or performing the drop
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual bool RequestVariableDropOnPanel(UEdGraph* InGraph, FProperty* InVariableToDrop, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		return false;
	}
	virtual bool RequestVariableDropOnPanel(UEdGraph* InGraph, FProperty* InVariableToDrop, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RequestVariableDropOnPanel(InGraph, InVariableToDrop, FVector2D(InDropPosition), FVector2D(InScreenPosition));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Request to drop a variable on a node
	 * @param	InGraph The graph the drop is subject of
	 * @param	InVariableToDrop The variable we want to drop
	 * @param	InNode The node we want to drop onto
	 * @param	InDropPosition The position inside of the graph
	 * @param	InScreenPosition The position inside of the screen
	 * @return the response to rejection or performing the drop
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual bool RequestVariableDropOnNode(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphNode* InNode, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		return false;
	}
	virtual bool RequestVariableDropOnNode(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphNode* InNode, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RequestVariableDropOnNode(InGraph, InVariableToDrop, InNode, FVector2D(InDropPosition), FVector2D(InScreenPosition));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Request to drop a variable on a pin
	 * @param	InGraph The graph the drop is subject of
	 * @param	InVariableToDrop The variable we want to drop
	 * @param	InPin The pin we want to drop onto
	 * @param	InDropPosition The position inside of the graph
	 * @param	InScreenPosition The position inside of the screen
	 * @return the response to rejection or performing the drop
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this API no longer interfaces directly with double-precision scalars and vectors.")
	virtual bool RequestVariableDropOnPin(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphPin* InPin, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		return false;
	}
	virtual bool RequestVariableDropOnPin(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphPin* InPin, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RequestVariableDropOnPin(InGraph, InVariableToDrop, InPin, FVector2D(InDropPosition), FVector2D(InScreenPosition));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Returns true if the types and directions of two pins are schema compatible. Handles
	 * outputting a more derived type to an input pin expecting a less derived type.
	 *
	 * @param	PinA		  	The pin a.
	 * @param	PinB		  	The pin b.
	 * @param	CallingContext	(optional) The calling context (required to properly evaluate pins of type Self)
	 * @param	bIgnoreArray	(optional) Whether or not to ignore differences between array and non-array types
	 *
	 * @return	true if the pin types and directions are compatible.
	 */
	virtual bool ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext = nullptr, bool bIgnoreArray = false) const { return true; }

	/**
	 * Returns true if the types are schema Equivalent. 
	 *
	 * @param	PinA		  	The type of Pin A.
	 * @param	PinB		  	The type of Pin B.
	 *
	 * @return	true if the pin types and directions are compatible.
	 */
	virtual bool ArePinTypesEquivalent(const FEdGraphPinType& PinA, const FEdGraphPinType& PinB) const { return true; }

	/**
	 * Returns true if the schema wants to overdrive the behaviour of dirtying the blueprint on new node creation.
	 *
	 * @param   InBlueprint    The blueprint to dirty or not
	 * @param   InEdGraphNode  The node that was just added and caused the request
	 * 
	 * @return  true if the blueprint marking has been taken care off.
	 */
	virtual bool MarkBlueprintDirtyFromNewNode(UBlueprint* InBlueprint, UEdGraphNode* InEdGraphNode) const { return false; }

	/**
	* Returns the local variables related to the graph.
	*
	* @param   InGraph    The graph where to look for local variables
	* @param   OutLocalVariables    The local variables found in the graph
	* 
	* @return  true if the graph can contain local variables (even if it has no local variables)
	*/
	virtual bool GetLocalVariables(const UEdGraph* InGraph, TArray<FBPVariableDescription>& OutLocalVariables) const { return false; }

	/**
	* Generates a graph schema action from a graph and a variable description.
	*
	* @param   InGraph    The graph where the variable is located
	* @param   VariableDescription    The description of the variable from which to generate the action
	* 
	* @return  a shared pointer to the newly created action.
	*/
	virtual TSharedPtr<FEdGraphSchemaAction> MakeActionFromVariableDescription(const UEdGraph* InEdGraph, const FBPVariableDescription& VariableDescription) const { return nullptr; }

	/**
	 * Insert additional actions into the blueprint action menu
	 * @param InBlueprints List of all blueprints you want actions for.
	 * @param InGraphs A list of graphs you want compatible actions for.
	 * @param InPins A list of pins you want compatible actions for.
	 * @param OutAllActions Resulting compatible actions
	 */
	virtual void InsertAdditionalActions(TArray<UBlueprint*> InBlueprints, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins, FGraphActionListBuilderBase& OutAllActions) const {}

#if WITH_EDITOR
	/**
	 * Returns a name validator appropiate for the schema and object that is being named
	 * @param InBlueprintObj The blueprint where the object being named lives.
	 * @param InOriginalName The original name of the object.
	 * @param InValidationScope The scope where the named object lives.
	 * @param InActionTypeId The type of object that is being named.
	 * @param NameValidator The name validator to use when naming this object.
	 */
	ENGINE_API virtual TSharedPtr<INameValidatorInterface> GetNameValidator(const UBlueprint* InBlueprintObj, const FName& InOriginalName, const UStruct* InValidationScope, const FName& InActionTypeId) const;
#endif
	
#if WITH_EDITORONLY_DATA
private:
	virtual bool ShouldShowPanelContextMenuForIncompatibleConnectionsImpl() const { return false; }
protected:
	/** Build an array containing all search types, return the index of the first non-localized entry. */
	ENGINE_API int32 CollectSearchTextWeightInfo(const FEdGraphSchemaAction& InCurrentAction, const FGraphSchemaSearchWeightModifiers& InWeightModifiers,
		TArray<FGraphSchemaSearchTextWeightInfo>& OutWeightedArrayList, FGraphSchemaSearchTextDebugInfo* InDebugInfo) const;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup weight calculation was always performed on the first action, provide that if scoring a FGraphActionListBuilderBase::ActionGroup, or just provide the FEdGraphSchemaAction")
	ENGINE_API int32 CollectSearchTextWeightInfo(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const FGraphSchemaSearchWeightModifiers& InWeightModifiers,
		TArray<FGraphSchemaSearchTextWeightInfo>& OutWeightedArrayList, FGraphSchemaSearchTextDebugInfo* InDebugInfo) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ENGINE_API void PrintSearchTextDebugInfo(const TArray<FString>& InFilterTerms, const FEdGraphSchemaAction& InCurrentAction, const FGraphSchemaSearchTextDebugInfo* InDebugInfo) const;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Only the first entry in the action group has ever been used, provide it to PrintSearchTextDebugInfo")
	ENGINE_API void PrintSearchTextDebugInfo(const TArray<FString>& InFilterTerms, const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const FGraphSchemaSearchTextDebugInfo* InDebugInfo) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};
