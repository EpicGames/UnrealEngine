// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/SecureHash.h"
#include "AssetImportData.generated.h"

/** Struct that is used to store an array of asset import data as an asset registry tag */
USTRUCT()
struct FAssetImportInfo
{
	GENERATED_BODY()
	
#if WITH_EDITORONLY_DATA

	FAssetImportInfo() {}

	FAssetImportInfo(const FAssetImportInfo& In) : SourceFiles(In.SourceFiles) { }
	FAssetImportInfo& operator=(const FAssetImportInfo& In) { SourceFiles = In.SourceFiles; return *this; }

	FAssetImportInfo(FAssetImportInfo&& In) : SourceFiles(MoveTemp(In.SourceFiles)) { }
	FAssetImportInfo& operator=(FAssetImportInfo&& In) { SourceFiles = MoveTemp(In.SourceFiles); return *this; }

	struct FSourceFile
	{
		FSourceFile() = default;

		FSourceFile(FString InRelativeFilename, const FDateTime& InTimestamp = 0, const FMD5Hash& InFileHash = FMD5Hash(), const FString& InDisplayLabelName = FString())
			: RelativeFilename(MoveTemp(InRelativeFilename))
			, Timestamp(InTimestamp)
			, FileHash(InFileHash)
			, DisplayLabelName(InDisplayLabelName)
		{}

		/** The path to the file that this asset was imported from. Relative to either the asset's package, BaseDir(), or absolute */
		FString RelativeFilename;

		/** The timestamp of the file when it was imported (as UTC). 0 when unknown. */
		FDateTime Timestamp;

		/** The MD5 hash of the file when it was imported. Invalid when unknown. */
		FMD5Hash FileHash;

		/** The Label use to display this source file in the property editor. */
		FString DisplayLabelName;
	};

	/** Convert this import information to JSON */
	ENGINE_API FString ToJson() const;

	/** Attempt to parse an asset import structure from the specified json string. */
	static ENGINE_API TOptional<FAssetImportInfo> FromJson(FString InJsonString);

	/** Insert information pertaining to a new source file to this structure */
	void Insert(const FSourceFile& InSourceFile) { SourceFiles.Add(InSourceFile); }

	/** Array of information pertaining to the source files that this asset was imported from */
	TArray<FSourceFile> SourceFiles;

#endif // WITH_EDITORONLY_DATA
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnImportDataChanged, const FAssetImportInfo& /*OldData*/, const class UAssetImportData* /* NewData */);

/* todo: Make this class better suited to multiple import paths - maybe have FAssetImportInfo use a map rather than array? */
UCLASS(EditInlineNew, MinimalAPI)
class UAssetImportData : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	/** Only valid in the editor */
	virtual bool IsEditorOnly() const override { return true; }

#if WITH_EDITOR
	/**
	 * Add or update a filename at the specified index. If the index is greater then the number of source file,
	 * it will add empty filenames to fill up to the specified index. The timespan and MD5 will be computed.
	 *
	 * @Param InPath: The filename we want to set at the specified index.
	 * @Param Index: This specify the source file index in case you have many source file for an imported asset
	 * @Param SourceFileLabel: Optional, can be empty string, the label we want to see in the UI when displaying the source file. (useful for multi source)
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetImportData")
	ENGINE_API void ScriptedAddFilename(const FString& InPath, int32 Index, FString SourceFileLabel);
#endif //WITH_EDITOR


#if WITH_EDITORONLY_DATA

	/** Path to the resource used to construct this static mesh. Relative to the object's package, BaseDir() or absolute */
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	/** Date/Time-stamp of the file from the last import */
	UPROPERTY()
	FString SourceFileTimestamp_DEPRECATED;

	/** Source file data describing the files that were used to import this asset. */
	UPROPERTY(VisibleAnywhere, Category="File Path")
	FAssetImportInfo SourceData;

public:

	/** Static event that is broadcast whenever any asset has updated its import data */
	static ENGINE_API FOnImportDataChanged OnImportDataChanged;

	/** Update this import data using the specified file. Called when an asset has been imported from a file. */
	ENGINE_API void Update(const FString& AbsoluteFilename, FMD5Hash *Md5Hash = nullptr);

	//@third party BEGIN SIMPLYGON
	/** Update this import data using the specified filename and Precomputed Hash. */
	ENGINE_API void Update(const FString& AbsoluteFileName, const FMD5Hash PreComputedHash);
	//@third party END SIMPLYGON

	/** Update this import data using the specified filename. Will not update the imported timestamp or MD5 (so we can update files when they move). */
	ENGINE_API void UpdateFilenameOnly(const FString& InPath);

	/** Update the filename at the specific specified index. Will not update the imported timestamp or MD5 (so we can update files when they move). */
	ENGINE_API void UpdateFilenameOnly(const FString& InPath, int32 Index);

	/** Add a filename at the specific index. Will update the imported timespan and MD5. It will also add in between with empty filenames*/
	ENGINE_API void AddFileName(const FString& InPath, int32 Index, FString SourceFileLabel = FString());

	/** Replace the source files with the one provided. The MD5 hashes will be computed if they aren't set */
	ENGINE_API void SetSourceFiles(TArray<FAssetImportInfo::FSourceFile>&& SourceFiles);

#if WITH_EDITOR
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) {}
	/** If your asset import data flavor need to add some asset registry tag, override this function. */
	virtual void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) {}

	/** Helper function to return the first filename stored in this data. The resulting filename will be absolute (ie, not relative to the asset).  */
	UFUNCTION(BlueprintCallable, meta=(DisplayName="GetFirstFilename", ScriptName="GetFirstFilename"), Category="AssetImportData")
	ENGINE_API FString K2_GetFirstFilename() const;
#endif
	ENGINE_API FString GetFirstFilename() const;

	/** Const access to the source file data */
	const FAssetImportInfo& GetSourceData() const { return SourceData; }

	/** Extract all the (resolved) filenames from this data  */
#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, meta=(DisplayName="ExtractFilenames", ScriptName="ExtractFilenames"), Category="AssetImportData")
	ENGINE_API TArray<FString> K2_ExtractFilenames() const;
#endif
	ENGINE_API TArray<FString> ExtractFilenames() const;
	ENGINE_API void ExtractFilenames(TArray<FString>& AbsoluteFilenames) const;
	
	ENGINE_API void ExtractDisplayLabels(TArray<FString>& FileDisplayLabels) const;

	int32 GetSourceFileCount() const { return SourceData.SourceFiles.Num(); }

	/** Resolve a filename that is relative to either the specified package, BaseDir() or absolute */
	static ENGINE_API FString ResolveImportFilename(const FString& InRelativePath, const UPackage* Outermost);
	static ENGINE_API FString ResolveImportFilename(FStringView InRelativePath, FStringView OutermostPath);

	/** Convert an absolute import path so that it's relative to either this object's package, BaseDir() or leave it absolute */
	static ENGINE_API FString SanitizeImportFilename(const FString& InPath, const UPackage* Outermost);
	static ENGINE_API FString SanitizeImportFilename(const FString& InPath, const FString& PackagePath);
#if WITH_EDITOR
	/**
	 * This function is call when we duplicate asset import data and assign the new copy to a new asset type.
	 * 
	 * @note - A use case is when we convert static mesh to skeletal mesh, Interchange need to change the pipeline context
	 * for the new asset type so we show the correct properties in the re-import dialog.
	 */
	virtual bool ConvertAssetImportDataToNewOwner(UObject* Owner)
	{
		return false;
	}
#endif

	ENGINE_API virtual void PostLoad() override;

	/** Convert an absolute import path so that it's relative to either this object's package, BaseDir() or leave it absolute */
	ENGINE_API FString SanitizeImportFilename(const FString& InPath) const;

protected:

	/** Resolve a filename that is relative to either this object's package, BaseDir() or absolute */
	ENGINE_API FString ResolveImportFilename(const FString& InRelativePath) const;

	/** Overridden serialize function to write out the underlying data as json */
	ENGINE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	
#endif		// WITH_EDITORONLY_DATA
};


