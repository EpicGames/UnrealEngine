// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * A base class for the helper object that holds thumbnail information an asset.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ThumbnailInfo.generated.h"

UCLASS(MinimalAPI)
class UThumbnailInfo : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const { return true; }
};



