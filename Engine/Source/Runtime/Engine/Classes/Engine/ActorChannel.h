// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/NetworkGuid.h"
#include "Engine/Channel.h"
#include "Net/DataReplication.h"
#include "Containers/StaticBitArray.h"
#include "ActorChannel.generated.h"

#ifndef NET_ENABLE_SUBOBJECT_REPKEYS
#define NET_ENABLE_SUBOBJECT_REPKEYS 1
#endif // NET_ENABLE_SUBOBJECT_REPKEYS

// Enable the code allowing to validate replicate subobjects when converting from the legacy method to the explicit registration list
#ifndef SUBOBJECT_TRANSITION_VALIDATION
	#define SUBOBJECT_TRANSITION_VALIDATION !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

class UActorComponent;
class AActor;
class FInBunch;
class FNetFieldExportGroup;
class FOutBunch;
class UNetConnection;
struct FObjectKey;

namespace UE::Net
{
	struct FSubObjectRegistry;
	class FNetConditionGroupManager;
}

enum class ESetChannelActorFlags : uint32
{
	None					= 0,
	SkipReplicatorCreation	= (1 << 0),
	SkipMarkActive			= (1 << 1),
};

ENUM_CLASS_FLAGS(ESetChannelActorFlags);

/**
 * A channel for exchanging actor and its subobject's properties and RPCs.
 *
 * ActorChannel manages the creation and lifetime of a replicated actor. Actual replication of properties and RPCs
 * actually happens in FObjectReplicator now (see DataReplication.h).
 *
 * An ActorChannel bunch looks like this:
 *
 * +----------------------+---------------------------------------------------------------------------+
 * | SpawnInfo            | (Spawn Info) Initial bunch only                                           |
 * |  -Actor Class        |   -Created by ActorChannel                                                |
 * |  -Spawn Loc/Rot      |                                                                           |
 * | NetGUID assigns      |                                                                           |
 * |  -Actor NetGUID      |                                                                           |
 * |  -Component NetGUIDs |                                                                           |
 * +----------------------+---------------------------------------------------------------------------+
 * |                      |                                                                           |
 * +----------------------+---------------------------------------------------------------------------+
 * | NetGUID ObjRef       | (Content chunks) x number of replicating objects (Actor + any components) |
 * |                      |   -Each chunk created by its own FObjectReplicator instance.              |
 * +----------------------+---------------------------------------------------------------------------+
 * |                      |                                                                           |
 * | Properties...        |                                                                           |
 * |                      |                                                                           |
 * | RPCs...              |                                                                           |
 * |                      |                                                                           |
 * +----------------------+---------------------------------------------------------------------------+
 * | </End Tag>           |                                                                           |
 * +----------------------+---------------------------------------------------------------------------+
 */
UCLASS(transient, customConstructor, MinimalAPI)
class UActorChannel : public UChannel
{
	GENERATED_BODY()

public:
	friend class FObjectReplicator;

	// Variables.
	UPROPERTY()
	TObjectPtr<AActor> Actor;					// Actor this corresponds to.

	FNetworkGUID	ActorNetGUID;		// Actor GUID (useful when we don't have the actor resolved yet). Currently only valid on clients.
	float			CustomTimeDilation;

	// Variables.
	double	RelevantTime;			// Last time this actor was relevant to client.
	double	LastUpdateTime;			// Last time this actor was replicated.
	uint32  SpawnAcked:1;			// Whether spawn has been acknowledged.
	uint32  bForceCompareProperties:1;	// Force this actor to compare all properties for a single frame
	uint32  bIsReplicatingActor:1;	// true when in this channel's ReplicateActor() to avoid recursion as that can cause invalid data to be sent
	
	/** whether we should nullptr references to this channel's Actor in other channels' Recent data when this channel is closed
	 * set to false in cases where the Actor can't become relevant again (e.g. destruction) as it's unnecessary in that case
	 */
	uint32	bClearRecentActorRefs:1;
	uint32	bHoldQueuedExportBunchesAndGUIDs:1;	// Don't export QueuedExportBunches or QueuedMustBeMappedGuidsInLastBunch if this is true

#if !UE_BUILD_SHIPPING
	/** Whether or not to block sending of NMT_ActorChannelFailure (for NetcodeUnitTest) */
	uint32	bBlockChannelFailure:1;
#endif

private:
	uint32	bSkipRoleSwap:1;		// true if we should not swap the role and remote role of this actor when properties are received

	/** Tracks whether or not our actor has been seen as pending kill. */
	uint32 bActorIsPendingKill : 1;

	/**
	 * Whether or not our NetDriver detected a hitch somewhere else in the engine.
	 * This is used by ProcessQueuedBunches to prevent erroneous log spam.
	 */
	uint32 bSuppressQueuedBunchWarningsDueToHitches : 1;

	/** Set to true if SerializeActor is called due to an RPC forcing the channel open */
	uint32 bIsForcedSerializeFromRPC:1;

    uint16 ChannelSubObjectDirtyCount = 0;

public:
	bool GetSkipRoleSwap() const { return !!bSkipRoleSwap; }
	void SetSkipRoleSwap(const bool bShouldSkip) { bSkipRoleSwap = bShouldSkip; }

	TSharedPtr<FObjectReplicator> ActorReplicator;

	TMap< UObject*, TSharedRef< FObjectReplicator > > ReplicationMap;

	// Async networking loading support state
	TArray< class FInBunch * >			QueuedBunches;			// Queued bunches waiting on pending guids to resolve
	double								QueuedBunchStartTime;	// Time when since queued bunches was last empty

	TSet<FNetworkGUID> PendingGuidResolves;	// These guids are waiting for their resolves, we need to queue up bunches until these are resolved

	UE_DEPRECATED(5.1, "The CreateSubObjects array will be made private in future versions. Use GetCreatedSubObjects() instead")
	UPROPERTY()
	TArray< TObjectPtr<UObject> >					CreateSubObjects;		// Any sub-object we created on this channel

	inline const TArray< TObjectPtr<UObject> >& GetCreatedSubObjects() const 
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CreateSubObjects;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TArray< FNetworkGUID >				QueuedMustBeMappedGuidsInLastBunch;	// Array of guids that will async load on client. This list is used for queued RPC's.
	TArray< class FOutBunch * >			QueuedExportBunches;				// Bunches that need to be appended to the export list on the next SendBunch call. This list is used for queued RPC's.


	EChannelCloseReason QueuedCloseReason;

	/**
	 * Default constructor
	 */
	ENGINE_API UActorChannel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API ~UActorChannel();

public:

	// UChannel interface.

	ENGINE_API virtual void Init( UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags ) override;
	ENGINE_API virtual void SetClosingFlag() override;
	ENGINE_API virtual void ReceivedBunch( FInBunch& Bunch ) override;
	ENGINE_API virtual void Tick() override;
	ENGINE_API virtual bool CanStopTicking() const override;

	ENGINE_API void ProcessBunch( FInBunch & Bunch );
	ENGINE_API bool ProcessQueuedBunches();

	ENGINE_API virtual void ReceivedNak( int32 NakPacketId ) override;
	ENGINE_API virtual int64 Close(EChannelCloseReason Reason) override;
	ENGINE_API virtual FString Describe() override;

	/** Release any references this channel is holding to UObjects and object replicators and mark it as broken. */
	ENGINE_API void BreakAndReleaseReferences();

	ENGINE_API void ReleaseReferences(bool bKeepReplicators);

	/** UActorChannel interface and accessors. */
	AActor* GetActor() const { return Actor; }

	/** Replicate this channel's actor differences. Returns how many bits were replicated (does not include non-bunch packet overhead) */
	ENGINE_API int64 ReplicateActor();

	/** Tells if the actor is ready to be replicated since he is BeginPlay or inside BeginPlay */
	ENGINE_API bool IsActorReadyForReplication() const;

	/**
	 * Set this channel's actor to the given actor.
	 * It's expected that InActor is either null (releasing the channel's reference) or
	 * a valid actor that is not PendingKill or PendingKillPending.
	 */
	ENGINE_API void SetChannelActor(AActor* InActor, ESetChannelActorFlags Flags);

	ENGINE_API virtual void NotifyActorChannelOpen(AActor* InActor, FInBunch& InBunch);

	/** Append any export bunches */
	UE_DEPRECATED(5.5, "Use GetAdditionalRequiredBunches")
	ENGINE_API virtual void AppendExportBunches( TArray< FOutBunch* >& OutExportBunches ) override;

	/** Returns any additional bunches (such as exports) needed by the passed-in OutgoingBunch */
	ENGINE_API virtual TArray<FOutBunch*> GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags) override;

	/** Append any "must be mapped" guids to front of bunch. These are guids that the client will wait on before processing this bunch. */
	ENGINE_API virtual void AppendMustBeMappedGuids( FOutBunch* Bunch ) override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Queue a function bunch for this channel to be sent on the next property update. */
	ENGINE_API void QueueRemoteFunctionBunch( UObject* CallTarget, UFunction* Func, FOutBunch &Bunch );

	/** If not queueing the RPC, prepare the channel for replicating the call.  */
	ENGINE_API void PrepareForRemoteFunction(UObject* TargetObj);
	
	/** Returns true if channel is ready to go dormant (e.g., all outstanding property updates have been ACK'd) */
	ENGINE_API virtual bool ReadyForDormancy(bool debug=false) override;
	
	/** Puts the channel in a state to start becoming dormant. It will not become dormant until ReadyForDormancy returns true in Tick */
	ENGINE_API virtual void StartBecomingDormant() override;

	/** Cleans up replicators and clears references to the actor class this channel was associated with.*/
	ENGINE_API void CleanupReplicators( const bool bKeepReplicators = false );

	/** Writes the header for a content block of properties / RPCs for the given object (either the actor a subobject of the actor) */
	ENGINE_API void WriteContentBlockHeader( UObject* Obj, FNetBitWriter &Bunch, const bool bHasRepLayout );

	/** Writes the header for a content block specifically for deleting sub-objects */
	UE_DEPRECATED(5.2, "This function will be made private in the future." )
	ENGINE_API void WriteContentBlockForSubObjectDelete( FOutBunch & Bunch, FNetworkGUID & GuidToDelete);

	/** Writes header and payload of content block */
	ENGINE_API int32 WriteContentBlockPayload( UObject* Obj, FNetBitWriter &Bunch, const bool bHasRepLayout, FNetBitWriter& Payload );

	/** Reads the header of the content block and instantiates the subobject if necessary */
	ENGINE_API UObject* ReadContentBlockHeader( FInBunch& Bunch, bool& bObjectDeleted, bool& bOutHasRepLayout );

	/** Reads content block header and payload */
	ENGINE_API UObject* ReadContentBlockPayload( FInBunch &Bunch, FNetBitReader& OutPayload, bool& bOutHasRepLayout );

	/** Writes property/function header and data blob to network stream */
	ENGINE_API int32 WriteFieldHeaderAndPayload( FNetBitWriter& Bunch, const FClassNetCache* ClassCache, const FFieldNetCache* FieldCache, FNetFieldExportGroup* NetFieldExportGroup, FNetBitWriter& Payload, bool bIgnoreInternalAck=false );

	/** Reads property/function header and data blob from network stream */
	ENGINE_API bool ReadFieldHeaderAndPayload( UObject* Object, const FClassNetCache* ClassCache, FNetFieldExportGroup* NetFieldExportGroup, FNetBitReader& Bunch, const FFieldNetCache** OutField, FNetBitReader& OutPayload ) const;

	/** Finds the net field export group for a class net cache, if not found, creates one */
	ENGINE_API FNetFieldExportGroup* GetNetFieldExportGroupForClassNetCache( UClass* ObjectClass );
		
	/** Finds (or creates) the net field export group for a class net cache, if not found, creates one */
	ENGINE_API FNetFieldExportGroup* GetOrCreateNetFieldExportGroupForClassNetCache( const UObject* Object );

	/** Returns the replicator for the actor associated with this channel. Guaranteed to exist. */
	ENGINE_API FObjectReplicator & GetActorReplicationData();

	// --------------------------------
	// Subobject Replication state
	//
	//	Concepts: 
	//		ObjID  - this is an arbitrary identifier given to us by the game code.
	//		RepKey - this is an idenifier for the current replicated state. 
	//
	//	ObjID should be constant per object or "category". Its up to the game code. For example the game code could use 0 to determine if an entire array is dirty,
	//	then usen 1-N for each subobject in that list. Or it could have 5 arrays using 0-4, and then use 100*ArrayNum + idx for the items in the array.
	//
	//	RepKey should change as the subobject changes. Each time a subobject is marked dirty, its RepKey should change.
	//
	//	GameCode should call ::KeyNeedsToReplicate(ObjID, RepKey) to determine if it needs to replicate. For example:
	//
	//
	/*

	bool AMyActorClass::ReplicateSubobjects(UActorChannel *Channel, FOutBunch *Bunch, FReplicationFlags *RepFlags)
	{
		bool WroteSomething = false;

		if (Channel->KeyNeedsToReplicate(0, ReplicatedArrayKey) )	// Does the array need to replicate?
		{
			for (int32 idx = 0; idx < ReplicatedSubobjects.Num(); ++idx )
			{
				UMyActorSubobjClass *Obj = ReplicatedSubObjects[idx];
				if (Channel->KeyNeedsToReplicate(1 + idx, Obj->RepKey))
				{								
					WroteSomething |= Channel->ReplicateSubobject<UMyActorSubobjClass>(Obj, *Bunch, *RepFlags);
				}
			}
		}

		return WroteSomething;
	}

	void UMyActorSubobjClass::MarkDirtyForReplication()
	{
		this->RepKey++;
		MyOwningActor->ReplicatedArrayKey++;
	}

	*/
	//	
	// --------------------------------

    /** 
    * Sets the owner of the next replicated subobjects that will be passed to ReplicateSubObject.
    * The ActorComponent version will choose not to replicate the future subobjects if the component opted to use the registered list.
    */
	static ENGINE_API void SetCurrentSubObjectOwner(AActor* SubObjectOwner);
	static ENGINE_API void SetCurrentSubObjectOwner(UActorComponent* SubObjectOwner);

#if SUBOBJECT_TRANSITION_VALIDATION
	/** Only returns true when calling ReplicateSubobjects() while we are trying to detect bad usage of the legacy function. */
	static ENGINE_API bool CanIgnoreDeprecatedReplicateSubObjects();
#endif

	/** Replicates given subobject on this actor channel */
	ENGINE_API bool ReplicateSubobject(UObject* Obj, FOutBunch& Bunch, FReplicationFlags RepFlags);

	/** Replicates an ActorComponent subobject. Used to catch ActorChannels who are using the registration list while their Owner is not flagged to use it */
	ENGINE_API bool ReplicateSubobject(UActorComponent* ActorChannel, FOutBunch& Bunch, FReplicationFlags RepFlags);
	
	/** Custom implementation for ReplicateSubobject when RepFlags.bUseCustomSubobjectReplication is true */
	virtual bool ReplicateSubobjectCustom(UObject* Obj, FOutBunch& Bunch, const FReplicationFlags& RepFlags) { return true;  }

	/** utility template for replicating list of replicated subobjects */
	template<typename Type>
	UE_DEPRECATED(5.1, "This function will be deleted. Register your subobjects using AddReplicatedSubObject instead.")
	bool ReplicateSubobjectList(TArray<Type*> &ObjectList, FOutBunch &Bunch, const FReplicationFlags &RepFlags)
	{
		bool WroteSomething = false;
		for (auto It = ObjectList.CreateIterator(); It; ++It)
		{
			Type* Obj = *It;
			WroteSomething |= ReplicateSubobject(Obj, Bunch, RepFlags);
		}

		return WroteSomething;
	}

	void SetForcedSerializeFromRPC(bool bInFromRPC) { bIsForcedSerializeFromRPC = bInFromRPC; }

#if NET_ENABLE_SUBOBJECT_REPKEYS
	// Static size for SubobjectRepKeyMap. Allows us to resuse arrays and avoid dyanmic memory allocations
	static const int32 SubobjectRepKeyBufferSize = 64;

	struct FPacketRepKeyInfo
	{
		FPacketRepKeyInfo() : PacketID(INDEX_NONE) { }

		int32			PacketID;
		TArray<int32>	ObjKeys;
	};

	// Maps ObjID to the current RepKey
	TMap<int32, int32>		SubobjectRepKeyMap;
	
	// Maps packetId to keys in Subobject
	TMap<int32, FPacketRepKeyInfo >		SubobjectNakMap;

	// Keys pending in this bunch
	TArray<int32> PendingObjKeys;
	
	// Returns true if the given ObjID is not up to date with RepKey
	// this implicitly 'writes' the RepKey to the current out bunch.
	ENGINE_API bool KeyNeedsToReplicate(int32 ObjID, int32 RepKey);
#endif // NET_ENABLE_SUBOBJECT_REPKEYS
	
	// --------------------------------

	ENGINE_API virtual void AddedToChannelPool() override;

	static ENGINE_API bool CanSubObjectReplicateToClient(
		const APlayerController* PlayerController,
		ELifetimeCondition NetCondition,
		FObjectKey SubObjectKey,
		const TStaticBitArray<COND_Max>& ConditionMap,
		const UE::Net::FNetConditionGroupManager& ConditionGroupManager);

protected:

	/** Attempts to find a valid, non-dormant replicator for the given object. */
	ENGINE_API TSharedRef<FObjectReplicator>* FindReplicator(UObject* Obj);

	/**
	* Attempts to find a valid, non-dormant replicator for the given object.
	*
	* @param Obj				The object whose replicator to find.
	* @param bOutFoundInvalid	Indicates we found a replicator, but it was invalid.
	*
	* @return A replicator, if one was found.
	*/
	UE_DEPRECATED(5.2, "This function has been deprecated in favor of the one with a single parameter")
	ENGINE_API TSharedRef<FObjectReplicator>* FindReplicator(UObject* Obj, bool* bOutFoundInvalid);

	/** 
	* Creates a new object replicator or reuses a replicator if it was stored for dormancy in the Connection.
	*/
	 ENGINE_API TSharedRef<FObjectReplicator>& CreateReplicator(UObject* Obj);

	/**
	 * Creates a new object replicator.
	 *
	 * This will replace any existing entries in the ReplicatorMap, so this should
	 * always be preceeded by a call to FindReplicator.
	 *
	 * @param Obj						The object to create a replicator for.
	 * @param bCheckDormantReplicators	When true, we will search the DormantReplicator map before actually
	 * 									creating a new replicator. Even in this case, we will treat the
	 * 									replicator as newly created.
	 *
	 * @return The newly created replicator.
	 */
	UE_DEPRECATED(5.2, "This function has been deprecated in favor of the one with a single parameter")
	ENGINE_API TSharedRef<FObjectReplicator>& CreateReplicator(UObject* Obj, bool bCheckDormantReplicators);

	/**
	 * Convenience method for finding a replicator, and creating one if necessary, all at once.
	 *
	 * @param Obj			The object to find / create a replicator for.
	 * @param bOutCreated	Whether or not the replicator was found or created.
	 *
	 * @return The found or created replicator.
	 */
	ENGINE_API TSharedRef<FObjectReplicator>& FindOrCreateReplicator(UObject* Obj, bool* bOutCreated=nullptr);

	ENGINE_API bool ObjectHasReplicator(const TWeakObjectPtr<UObject>& Obj) const;	// returns whether we have already created a replicator for this object or not

	/** Unmap all references to this object, so that if later we receive this object again, we can remap the original references */
	ENGINE_API void MoveMappedObjectToUnmapped(const UObject* Object);

	ENGINE_API void DestroyActorAndComponents(EChannelCloseReason Reason = EChannelCloseReason::Destroyed);

	ENGINE_API virtual bool CleanUp( const bool bForDestroy, EChannelCloseReason CloseReason ) override;

	/** Closes the actor channel but with a 'dormant' flag set so it can be reopened */
	ENGINE_API virtual void BecomeDormant() override;

	/** Handle the replication of subobjects for this actor. Returns true if data was written into the Bunch. */
	ENGINE_API bool DoSubObjectReplication(FOutBunch& Bunch, FReplicationFlags& OutRepFlags);

	enum class ESubObjectDeleteFlag : uint8
	{
		Destroyed, // Delete operation that occurs when we detect that the original object on the authority became invalid.
		TearOff,      // The client's actor channel will remove references to this subobject
		ForceDelete,  // The subobject needs to be deleted on the client even if the original on the authority was not.
	};

	/** Writes the header for a content block specifically for deleting sub-objects */
	ENGINE_API void WriteContentBlockForSubObjectDelete(FOutBunch& Bunch, FNetworkGUID& GuidToDelete, ESubObjectDeleteFlag DeleteFlag);

	ENGINE_API const TCHAR* ToString(UActorChannel::ESubObjectDeleteFlag DeleteFlag);

private:

	/** Replicate Subobjects using the actor's registered list and its replicated actor component list */
	ENGINE_API bool ReplicateRegisteredSubObjects(FOutBunch& Bunch, FReplicationFlags RepFlags);

    /** Write the replicated bits into the bunch data */
	ENGINE_API bool WriteSubObjectInBunch(UObject* Obj, FOutBunch& Bunch, FReplicationFlags RepFlags);

	/** Find the replicated subobjects of the component and write them into the bunch */
	ENGINE_API bool WriteComponentSubObjects(UActorComponent* Component, FOutBunch& Bunch, FReplicationFlags RepFlags, const TStaticBitArray<COND_Max>& ConditionMap);

	/** Replicate a list of subobjects */
	ENGINE_API bool WriteSubObjects(UObject* SubObjectOwner, const UE::Net::FSubObjectRegistry& SubObjectList, FOutBunch& Bunch, FReplicationFlags RepFlags, const TStaticBitArray<COND_Max>& ConditionMap);

	ENGINE_API bool ValidateReplicatedSubObjects();

	ENGINE_API void TestLegacyReplicateSubObjects(UActorComponent* ReplicatedComponent, FOutBunch& Bunch, FReplicationFlags RepFlags);
	ENGINE_API void TestLegacyReplicateSubObjects(FOutBunch& Bunch, FReplicationFlags RepFlags);

	ENGINE_API bool UpdateDeletedSubObjects(FOutBunch& Bunch);

	inline TArray< TObjectPtr<UObject> >& GetCreatedSubObjects()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CreateSubObjects;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Used internally by ProcessQueuedBunches to check time limits */
	bool ShouldProcessAllQueuedBunches(float CurrentTimeSeconds);

	/** Handles the internal logic for processing bunches */
	void ProcessBunchInternal(FInBunch& Bunch);

private:

	// TODO: It would be nice to merge the tracking of these with PendingGuidResolves, to not duplicate memory,
	// especially since both of these sets should be empty most of the time for most channels.
	TSet<TSharedRef<struct FQueuedBunchObjectReference>> QueuedBunchObjectReferences;

	static ENGINE_API const FString ClassNetCacheSuffix;
};
