// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakInterfacePtr.h"
#include "ActorInstanceHandle.generated.h"

class USceneComponent;
class AActor;
class UActorInstanceManager;
class IActorInstanceManagerInterface;
class ULevel;

using FActorInstanceManagerInterface = TWeakInterfacePtr<IActorInstanceManagerInterface>;

/**
 * Handle to a unique object. This may specify a full weigh actor or it may only specify the actor instance that represents the same object.
 * 
 * @note The handle has game thread constraints related to UObjects and should be used carefully from other threads.
 * 
 *	Can only be used on the game thread
 *	-	all constructors 
 *	-	all getters (GetXYZ, FetchActor, IsActorValid, DoesRepresent) *	
 *	-	comparison operators against live AActor pointer
 *	
 *	Can be used on any thread
 *	-	MakeActorHandleToResolve to create a handle that will be lazily resolved on the game thread
 *		since it only stores a weak object ptr without any access to the live object
 *	-	handle validity and comparison operators against another handle (i.e. IsValid(), operator==|!=(const FActorInstanceHandle& Other))
 */
USTRUCT(BlueprintType)
struct FActorInstanceHandle
{
	GENERATED_BODY()

	FActorInstanceHandle() = default;

	ENGINE_API explicit FActorInstanceHandle(AActor* InActor);
	ENGINE_API FActorInstanceHandle(const UPrimitiveComponent* RelevantComponent, int32 CollisionInstanceIndex);
	ENGINE_API FActorInstanceHandle(AActor* InActor, const UPrimitiveComponent* RelevantComponent, int32 CollisionInstanceIndex);
	ENGINE_API FActorInstanceHandle(FActorInstanceManagerInterface InManagerInterface, int32 InstanceIndex);
	ENGINE_API FActorInstanceHandle(const FActorInstanceHandle& Other);

	/** 
	 * A path dedicated to creation of handles while converting actor to a dehydrated representation. This path ensures
	 * an actor won't be spawned as a side effect of looking for the actor given Manager/Index represents
	 */
	static ENGINE_API FActorInstanceHandle MakeDehydratedActorHandle(UObject& Manager, int32 InInstanceIndex);

	/** 
	 * A path dedicated to creation of handles from any threads.
	 * This path marks the handle as need resolving once it gets accessed from the game thread.
	 */
	static ENGINE_API FActorInstanceHandle MakeActorHandleToResolve(const TWeakObjectPtr<UPrimitiveComponent>& WeakComponent, int32 CollisionInstanceIndex);

	ENGINE_API bool IsValid() const;

	UE_DEPRECATED(5.8, "Detecting the ResolutionStatus of the pointer will be removed in a future release.")
	ENGINE_API bool IsUnresolvedDehydratedActor() const;

	ENGINE_API bool DoesRepresentClass(const UClass* OtherClass) const;

	template<typename T>
	bool DoesRepresent() const;

	ENGINE_API UClass* GetRepresentedClass() const;
	ENGINE_API ULevel* GetLevel() const;
	ENGINE_API FVector GetLocation() const;
	ENGINE_API FRotator GetRotation() const;
	ENGINE_API FTransform GetTransform() const;

	ENGINE_API FName GetFName() const;
	ENGINE_API FString GetName() const;

	/** If this handle has a valid actor, return it; otherwise return the actor responsible for managing the instances. */
	ENGINE_API AActor* GetManagingActor() const;

	/** Returns either the actor's root component or the root component for the manager associated with the handle */
	ENGINE_API USceneComponent* GetRootComponent() const;

	/** Returns the actor specified by this handle. This may require loading and creating the actor object. */
	ENGINE_API AActor* FetchActor() const;

	template <typename T>
	T* FetchActor() const;

	ENGINE_API AActor* GetCachedActor() const;
	ENGINE_API void SetCachedActor(AActor* InActor) const;

	/* Returns the index used internally by the manager */
	inline int32 GetInstanceIndex() const { return InstanceIndex; }

	FActorInstanceHandle& operator=(const FActorInstanceHandle& Other) = default;
	FActorInstanceHandle& operator=(FActorInstanceHandle&& Other) = default;
	ENGINE_API FActorInstanceHandle& operator=(AActor* OtherActor);

	ENGINE_API bool operator==(const FActorInstanceHandle& Other) const;
	ENGINE_API bool operator!=(const FActorInstanceHandle& Other) const;

	ENGINE_API bool operator==(const AActor* OtherActor) const;
	ENGINE_API bool operator!=(const AActor* OtherActor) const;

	explicit operator bool() const { return IsValid(); }

	friend ENGINE_API uint32 GetTypeHash(const FActorInstanceHandle& Handle);

	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FActorInstanceHandle& Handle);

	FActorInstanceManagerInterface GetManagerInterface() const { return ManagerInterface; }

	template<typename T>
	T* GetManager() const 
	{
		return Cast<T>(ManagerInterface.GetObject());
	}

private:
	friend struct FActorInstanceHandleInternalHelper;

	/**
	 * helper functions that let us treat the actor pointer as a UObject in templated functions
	 * these do NOT fetch the actor so they will return nullptr if we don't have a full actor representation
	 */
	UObject* GetActorAsUObject();
	const UObject* GetActorAsUObject() const;

	/** Returns true if Actor is not null and not pending kill */
	bool IsActorValid() const;

	void ResolveHandle() const;
	
	/**
	 * Weak UObject pointer used for two purposes:
	 *  - a resolved handle uses it to store the AActor
	 *  - a handle to be resolved uses it to store the UPrimitiveComponent provided by MakeActorHandleToResolve 
	 */
	UPROPERTY()
	mutable TWeakObjectPtr<UObject> ReferenceObject;

	/** Identifies the actor instance manager to use */
	FActorInstanceManagerInterface ManagerInterface;

	/** Identifies the instance within the manager */
	int32 InstanceIndex = INDEX_NONE;

	/**
	 * Enum to keep track of the resolution status of the handle.
	 * It is only possible to safely resolve the handle on the game thread so
	 * other threads should use MakeActorHandleToResolve to safely create one. 
	 */
	enum class EResolutionStatus : uint8
	{
		Invalid,
		Resolved, /* ManagerInterface and InstanceIndex are set or an Actor pointer is stored in ReferenceObject */
		NeedsResolving /* Component pointer is stored in ReferenceObject and InstanceIndex might hold a CollisionInstanceIndex */
	};

	/** Indicates if the handle is resolved, needs to be resolved (extract Actor|ManagerInterface) */
	EResolutionStatus ResolutionStatus = EResolutionStatus::Invalid;
};

template<typename T>
bool FActorInstanceHandle::DoesRepresent() const
{
	if (const UClass* RepresentedClass = GetRepresentedClass())
	{
		if constexpr (TIsIInterface<T>::Value)
		{
			return RepresentedClass->ImplementsInterface(T::UClassType::StaticClass());
		}
		else
		{
			return RepresentedClass->IsChildOf(T::StaticClass());
		}
	}
	return false;
}

template <typename T>
T* FActorInstanceHandle::FetchActor() const
{
	if (DoesRepresent<T>())
	{
		return CastChecked<T>(FetchActor(), ECastCheckedType::NullAllowed);
	}
	return nullptr;
}
