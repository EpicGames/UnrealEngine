// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Chaos/Transform.h"
#include "ActorInstanceManagerInterface.generated.h"


class ULevel;
class UClass;
class AActor;
class UPrimitiveComponent;
struct FActorInstanceHandle;

UINTERFACE(MinimalAPI)
class UActorInstanceManagerInterface : public UInterface
{
	GENERATED_BODY()
};

class IActorInstanceManagerInterface
{
	GENERATED_BODY()

public:
	// Returns the index used internally by the actor instance manager that is associated with the instance 
	// referred to by InIndex used by collision and rendering
	virtual int32 ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const = 0;

	// Returns the actor associated with Handle if one exists
	virtual AActor* FindActor(const FActorInstanceHandle& Handle) = 0;

	// Returns the actor associated with Handle. If one does not exist the system is expected to spawn one instantly. 
	virtual AActor* FindOrCreateActor(const FActorInstanceHandle& Handle) = 0;

	// Returns the specific class InstanceIndex represents
	virtual UClass* GetRepresentedClass(int32 InstanceIndex) const = 0;

	// Returns the ULevel given instance is represented in. Potentially null if InstanceIndex doesn't correspond to an actor
	virtual ULevel* GetLevelForInstance(int32 InstanceIndex) const = 0;

	// Returns the transform of the instance specified by Handle
	virtual FTransform GetTransform(const FActorInstanceHandle& Handle) const = 0;
};
