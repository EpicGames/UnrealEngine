// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetManagerTypes.h"
#include "Misc/AssetRegistryInterface.h"
#include "StreamableManager.h"
#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "ContentEncryptionConfig.h"
#include "Misc/SourceLocation.h"
#include "Misc/SourceLocationUtils.h"

#include "AssetManager.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAssetManager, Log, All);

class IAssetRegistry;
class ITargetPlatform;
class FAssetManagerEditorModule;
class FTimerManager;
class UAssetManagerSettings;
namespace UE::AssetRegistry { struct FShouldSetManagerContext; }
namespace UE::Cook { class ICookInfo; }
struct FCompiledAssetManagerSearchRules;
struct FPrimaryAssetTypeData;
struct FPrimaryAssetData;
struct FPrimaryAssetRulesCustomOverride;

/** Delegate called when acquiring resources/chunks for assets, first parameter will be true if all resources were acquired, false if any failed.
	Second parameter will contain a list of chunks not yet downloaded */
DECLARE_DELEGATE_TwoParams(FAssetManagerAcquireResourceDelegateEx, bool /* bSuccess */, const TArray<int32>& /* MissingChunks */);

/** Delegate called when acquiring resources/chunks for assets, parameter will be true if all resources were acquired, false if any failed */
DECLARE_DELEGATE_OneParam(FAssetManagerAcquireResourceDelegate, bool /* bSuccess */);

/** Delegate called when new asset search root is registered due to runtime asset mounting, path will not have a trailing slash */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddedAssetSearchRoot, const FString&);

/** EXPERIMENTAL -  Additional parameters for downloading packages 
 * Asset Manager will automatically create a FStreamableDownloadCachePin for each primary asset requested with download params.
 * Cache pins can be retrieved with FStreamableHandle::GetDownloadCachePins()
 * Cache pins will be released along with the primary asset or when bundle state changes.
 */
struct FAssetManagerDownloadParams
{
	/** Download priority */
	FStreamableDownloadPriority Priority = UE::StreamableManager::Private::DownloadDefaultPriority;
	/** If set, any soft references will also be installed */
	bool bInstallSoftReferences = false;

	bool operator==(const FAssetManagerDownloadParams& Right) const = default;
};

struct FAssetManagerLoadParams
{
	/** [optional] Delegate to call when load finishes. Will be called on the next tick if asset is already loaded, or many seconds later */
	FStreamableDelegateWithHandle OnComplete;
	/** [optional] Delegate to call when the load is canceled. Will be called on the next tick or many seconds later */
	FStreamableDelegateWithHandle OnCancel;
	/** [optional] delegate that is called periodically as request updates */
	FStreamableUpdateDelegate OnUpdate;
	/** [optional] Priority to pass to the streaming system, higher priority will be loaded first */
	TAsyncLoadPriority Priority = UE::StreamableManager::Private::DefaultAsyncLoadPriority;
	/** [optional] EXPERIMENTAL - If present, will attempt to download necessary packages with IoStoreOnDemand */
	TOptional<FAssetManagerDownloadParams> DownloadParams;
};

enum class EUpdateManagementDatabaseFlags : uint32
{
	None = 0,							// No flags
	BuildChunkMap = (1 << 0),			// if the CachedChunkMap should be built or not.
	ForceRefresh = (1 << 1),			// When set the database will be forcibly refreshed.
};
ENUM_CLASS_FLAGS(EUpdateManagementDatabaseFlags);

/** 
 * A singleton UObject that is responsible for loading and unloading PrimaryAssets, and maintaining game-specific asset references
 * Games should override this class and change the class reference
 */
UCLASS(MinimalAPI)
class UAssetManager : public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	ENGINE_API UAssetManager();

	UE_DEPRECATED(5.3, "AssetManager is now always constructed during UEngine::InitializeObjectReferences. Call IsInitialized instead if you need to check whether it has not yet been initialized.")
	static ENGINE_API bool IsValid();
	/** Returns true if the global singleton AssetManager has been constructed. */
	static ENGINE_API bool IsInitialized();

	/** Returns the current AssetManager object */
	static ENGINE_API UAssetManager& Get();

	UE_DEPRECATED(5.3, "AssetManager is now always constructed during UEngine::InitializeObjectReferences. Call GetIfInitialized instead if you need to check whether it has not yet been initialized.")
	static ENGINE_API UAssetManager* GetIfValid();
	/** Returns the current global singleton AssetManager if it has been constructed, null otherwise */
	static ENGINE_API UAssetManager* GetIfInitialized();

	/** Accesses the StreamableManager used by this Asset Manager. Static for easy access */
	static FStreamableManager& GetStreamableManager() { return Get().StreamableManager; }

	/** Asset Type of UWorld assets */
	static ENGINE_API const FPrimaryAssetType MapType;

	/** Asset Type of Label used to tag other assets */
	static ENGINE_API const FPrimaryAssetType PrimaryAssetLabelType;

	/** Type representing a packaging chunk, this is a virtual type that is never loaded off disk */
	static ENGINE_API const FPrimaryAssetType PackageChunkType;

	/**
	 * Asset Type used in return values to indicate that a specific asset of a given PrimaryAssetType should be suppressed
	 * and not considered a PrimaryAsset.
	 */
	static ENGINE_API const FPrimaryAssetType SuppressionType;

	/** Virtual path $AssetSearchRoots, replaced with all roots including defaults like /Game */
	static ENGINE_API const FString AssetSearchRootsVirtualPath;

	/** Virtual path $DynamicSearchRoots, replaced with dynamically added asset roots */
	static ENGINE_API const FString DynamicSearchRootsVirtualPath;

	/** Get the asset registry tag name for encryption key data */
	static ENGINE_API FName GetEncryptionKeyAssetTagName();

	/** Creates a PrimaryAssetId from a chunk id */
	static ENGINE_API FPrimaryAssetId CreatePrimaryAssetIdFromChunkId(int32 ChunkId);

	/** Extracts a chunk id from a primary asset id, returns INDEX_NONE if it is not PackageChunkType */
	static ENGINE_API int32 ExtractChunkIdFromPrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId);

	// BUILDING ASSET DIRECTORY

	/** 
	 * Scans a list of paths and reads asset data for all primary assets of a specific type.
	 * If done in the editor it will load the data off disk, in cooked games it will load out of the asset registry cache
	 *
	 * @param PrimaryAssetType	Type of asset to look for. If the scanned asset matches GetPrimaryAssetType with this it will be added to directory
	 * @param Paths				List of file system paths to scan
	 * @param BaseClass			Base class of all loaded assets, if the scanned asset isn't a child of this class it will be skipped
	 * @param bHasBlueprintClasses	If true, the assets are blueprints that subclass BaseClass. If false they are base UObject assets
	 * @param bIsEditorOnly		If true, assets will only be scanned in editor builds, and will not be stored into the asset registry
	 * @param bForceSynchronousScan If true will scan the disk synchronously, otherwise will wait for asset registry scan to complete
	 * @return					Number of primary assets found
	 */
	ENGINE_API virtual int32 ScanPathsForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const TArray<FString>& Paths, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly = false, bool bForceSynchronousScan = true);

	/** Single path wrapper */
	ENGINE_API virtual int32 ScanPathForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const FString& Path, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly = false, bool bForceSynchronousScan = true);

	/** Call before many calls to ScanPaths to improve load performance. Match each call with PopBulkScanning(). */
	ENGINE_API void PushBulkScanning();
	ENGINE_API void PopBulkScanning();

	/** 
	 * Removes scan paths. 
	 * Used by the GameFeaturePlugins system to reset the database to the previous state when plugin unregisters.
	 * @param PrimaryAssetType Type of asset to look for
	 * @param Paths				List of file system paths to remove
	 * @param BaseClass			Base class of all loaded assets, must match the type that were previously added
	 * @param bHasBlueprintClasses	Flag, must match what was previously added
	 * @param bIsEditorOnly		Flag, must match what was previously added
	 */
	ENGINE_API virtual void RemoveScanPathsForPrimaryAssets(FPrimaryAssetType PrimaryAssetType, const TArray<FString>& Paths, UClass* BaseClass, bool bHasBlueprintClasses, bool bIsEditorOnly = false);

	/** 
	 * Removes the whole primary asset type. 
	 * Used by GameFeaturePlugins system to remove types, that are exclusive to the plugin itself.
	 * @param PrimaryAssetType Type of asset to look for
	 */
	ENGINE_API virtual void RemovePrimaryAssetType(FPrimaryAssetType PrimaryAssetType);

	/** 
	 * Removes tracking of primary assets from the management database.
	 * Used by GameFeaturePlugins system to unregister plugin's primary assets.
	 * @param PrimaryAssetType Type of asset to look for
	 * @param RootPath Mount point path, containing leading and tail path separators "/".
	 */
	ENGINE_API virtual void RemovePrimaryAssetsForTypeInMountPoint(FPrimaryAssetType PrimaryAssetType, FStringView RootPath);

protected:
	/** Should only be called from PushBulkScanning() and override */
	ENGINE_API virtual void StartBulkScanning();
	/** Should only be called from PopBulkScanning() and override */
	ENGINE_API virtual void StopBulkScanning();
	bool IsBulkScanning() const { return NumBulkScanRequests > 0 ; }

public:
	/** 
	 * Adds or updates a Dynamic asset, which is a runtime-specified asset that has no on disk representation, so has no FAssetData. But it can have bundle state and a path.
	 *
	 * @param FPrimaryAssetId	Type/Name of the asset. The type info will be created if it doesn't already exist
	 * @param AssetPath			Path to the object representing this asset, this is optional for dynamic assets
	 * @param BundleData		List of Name->asset paths that represent the possible bundle states for this asset
	 * @return					True if added
	 */
	ENGINE_API virtual bool AddDynamicAsset(const FPrimaryAssetId& PrimaryAssetId, const FSoftObjectPath& AssetPath, const FAssetBundleData& BundleData);

	/** This will expand out references in the passed in AssetBundleData that are pointing to other primary assets with bundles. This is useful to preload entire webs of assets */
	ENGINE_API virtual void RecursivelyExpandBundleData(FAssetBundleData& BundleData) const;

	/** Register a delegate to call when all types are scanned at startup, if this has already happened call immediately */
	static ENGINE_API void CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate&& Delegate);

	/** Register a delegate to call when the asset manager singleton is spawned, if this has already happened call immediately */
	static ENGINE_API void CallOrRegister_OnAssetManagerCreated(FSimpleMulticastDelegate::FDelegate&& Delegate);

	/** Returns true if initial scan has completed, this can be pretty late in editor builds */
	ENGINE_API virtual bool HasInitialScanCompleted() const;

	/** Call to register a callback executed when a new asset search root is added, can be used to scan for new assets */
	ENGINE_API FDelegateHandle Register_OnAddedAssetSearchRoot(FOnAddedAssetSearchRoot::FDelegate&& Delegate);

	/** Unregister previously added callback */
	ENGINE_API void Unregister_OnAddedAssetSearchRoot(FDelegateHandle DelegateHandle);

	/** Expands a list of paths that potentially use virtual paths into real directory and package paths. Returns true if any changes were made */
	ENGINE_API virtual bool ExpandVirtualPaths(TArray<FString>& InOutPaths) const;

	/** Register a new asset search root of the form /AssetRoot, will notify other systems about change */
	ENGINE_API virtual void AddAssetSearchRoot(const FString& NewRootPath);

	/** Returns all the game asset roots, includes /Game by default and any dynamic ones */
	ENGINE_API const TArray<FString>& GetAssetSearchRoots(bool bIncludeDefaultRoots = true) const;


	// ACCESSING ASSET DIRECTORY

	/** Gets the FAssetData for a primary asset with the specified type/name, will only work for once that have been scanned for already. Returns true if it found a valid data */
	ENGINE_API virtual bool GetPrimaryAssetData(const FPrimaryAssetId& PrimaryAssetId, FAssetData& AssetData) const;

	/** Gets list of all FAssetData for a primary asset type, returns true if any were found */
	ENGINE_API virtual bool GetPrimaryAssetDataList(FPrimaryAssetType PrimaryAssetType, TArray<FAssetData>& AssetDataList) const;

	/** Gets the in-memory UObject for a primary asset id, returning nullptr if it's not in memory. Will return blueprint class for blueprint assets. This works even if the asset wasn't loaded explicitly */
	ENGINE_API virtual UObject* GetPrimaryAssetObject(const FPrimaryAssetId& PrimaryAssetId) const;

	/** Templated versions of above */
	template<class AssetType> 
	inline AssetType* GetPrimaryAssetObject(const FPrimaryAssetId& PrimaryAssetId) const
	{
		UObject* ObjectReturn = GetPrimaryAssetObject(PrimaryAssetId);
		return Cast<AssetType>(ObjectReturn);
	}

	template<class AssetType>
	inline TSubclassOf<AssetType> GetPrimaryAssetObjectClass(const FPrimaryAssetId& PrimaryAssetId) const
	{
		TSubclassOf<AssetType> ReturnClass;
		ReturnClass = Cast<UClass>(GetPrimaryAssetObject(PrimaryAssetId));
		return ReturnClass;
	}

	/** Gets list of all loaded objects for a primary asset type, returns true if any were found. Will return blueprint class for blueprint assets. This works even if the asset wasn't loaded explicitly */
	ENGINE_API virtual bool GetPrimaryAssetObjectList(FPrimaryAssetType PrimaryAssetType, TArray<UObject*>& ObjectList) const;

	/** Gets the FSoftObjectPath for a primary asset type and name, returns invalid if not found */
	ENGINE_API virtual FSoftObjectPath GetPrimaryAssetPath(const FPrimaryAssetId& PrimaryAssetId) const;

	/** Gets the list of all FSoftObjectPaths for a given type, returns true if any found */
	ENGINE_API virtual bool GetPrimaryAssetPathList(FPrimaryAssetType PrimaryAssetType, TArray<FSoftObjectPath>& AssetPathList) const;

	/** Sees if the passed in object is a registered primary asset, if so return it. Returns invalid Identifier if not found */
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetIdForObject(UObject* Object) const;

	/** Sees if the passed in object path is a registered primary asset, if so return it. Returns invalid Identifier if not found */
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetIdForPath(const FSoftObjectPath& ObjectPath) const;

	/** Sees if the package has a primary asset, useful if only the package name is available */
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetIdForPackage(FName PackagePath) const;

	/** Returns the primary asset Id for the given FAssetData, only works if in directory */
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetIdForData(const FAssetData& AssetData) const;

	/** Gets list of all FPrimaryAssetId for a primary asset type, returns true if any were found */
	ENGINE_API virtual bool GetPrimaryAssetIdList(FPrimaryAssetType PrimaryAssetType, TArray<FPrimaryAssetId>& PrimaryAssetIdList, EAssetManagerFilter Filter = EAssetManagerFilter::Default) const;

	/** Gets metadata for a specific asset type, returns false if not found */
	ENGINE_API virtual bool GetPrimaryAssetTypeInfo(FPrimaryAssetType PrimaryAssetType, FPrimaryAssetTypeInfo& AssetTypeInfo) const;

	/** Gets list of all primary asset types infos */
	ENGINE_API virtual void GetPrimaryAssetTypeInfoList(TArray<FPrimaryAssetTypeInfo>& AssetTypeInfoList) const;

	/** Gets list of all FPrimaryAssetId for a primary asset type beginning with a specific path */
	ENGINE_API virtual void GetPrimaryAssetsInPath(FPrimaryAssetType PrimaryAssetType, FStringView Path, TArray<FPrimaryAssetId>& PrimaryAssetIdList) const;

	// ASYNC LOADING PRIMARY ASSETS

	/** 
	 * Loads a list of Primary Assets. This will start an async load of those assets, calling callback on completion.
	 * These assets will stay in memory until explicitly unloaded.
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and delegate will get called.
	 * Prefer the overloads with Param structs for new code.
	 *
	 * @param AssetsToLoad		List of primary assets to load
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param DelegateToCall	Delegate that will be called on completion
	 * @param Priority			Async loading priority for this request
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	virtual TSharedPtr<FStreamableHandle> LoadPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToLoad,
		const TArray<FName>& LoadBundles = TArray<FName>(),
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/**
	 * Loads a list of Primary Assets. This will start an async load of those assets, calling callback on completion.
	 * These assets will stay in memory until explicitly unloaded.
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and complete delegate will get called.
	 *
	 * @param AssetsToLoad		List of primary assets to load
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> LoadPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToLoad,
		const TArray<FName>& LoadBundles,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** Single asset wrapper, see LoadPrimaryAssets */
	virtual TSharedPtr<FStreamableHandle> LoadPrimaryAsset(
		const FPrimaryAssetId& AssetToLoad,
		const TArray<FName>& LoadBundles = TArray<FName>(),
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/** Single asset wrapper, see LoadPrimaryAssets */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> LoadPrimaryAsset(
		const FPrimaryAssetId& AssetToLoad,
		const TArray<FName>& LoadBundles,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** Loads all assets of a given type, useful for cooking */
	virtual TSharedPtr<FStreamableHandle> LoadPrimaryAssetsWithType(
		FPrimaryAssetType PrimaryAssetType,
		const TArray<FName>& LoadBundles = TArray<FName>(),
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/** Loads all assets of a given type, useful for cooking */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> LoadPrimaryAssetsWithType(
		FPrimaryAssetType PrimaryAssetType,
		const TArray<FName>& LoadBundles,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Unloads a list of Primary Assets that were previously Loaded.
	 * If the only thing keeping these assets in memory was a prior Load call, they will be freed.
	 *
	 * @param AssetsToUnload	List of primary assets to load
	 * @return					Number of assets unloaded
	 */
	ENGINE_API virtual int32 UnloadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToUnload);

	/** Single asset wrapper */
	ENGINE_API virtual int32 UnloadPrimaryAsset(const FPrimaryAssetId& AssetToUnload);

	/** Loads all assets of a given type, useful for cooking */
	ENGINE_API virtual int32 UnloadPrimaryAssetsWithType(FPrimaryAssetType PrimaryAssetType);

	/** 
	 * Changes the bundle state of a set of loaded primary assets.
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and delegate will get called.
	 * Prefer the overloads with Param structs for new code.
	 *
	 * @param AssetsToChange	List of primary assets to change state of
	 * @param AddBundles		List of bundles to add
	 * @param RemoveBundles		Explicit list of bundles to remove
	 * @param RemoveAllBundles	[optional] If true, remove all existing bundles even if not in remove list
	 * @param DelegateToCall	[optional] Delegate that will be called on completion
	 * @param Priority			[optional] Async loading priority for this request
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	virtual TSharedPtr<FStreamableHandle> ChangeBundleStateForPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToChange,
		const TArray<FName>& AddBundles,
		const TArray<FName>& RemoveBundles,
		bool bRemoveAllBundles = false,
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/**
	 * Changes the bundle state of a set of loaded primary assets.
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and delegate will get called.
	 *
	 * @param AssetsToChange	List of primary assets to change state of
	 * @param AddBundles		List of bundles to add
	 * @param RemoveBundles		Explicit list of bundles to remove
	 * @param RemoveAllBundles	If true, remove all existing bundles even if not in remove list
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> ChangeBundleStateForPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToChange,
		const TArray<FName>& AddBundles,
		const TArray<FName>& RemoveBundles,
		bool bRemoveAllBundles,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Changes the bundle state of all loaded primary assets. Only assets matching OldBundles will be modified
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and delegate will get called.
	 * Prefer the overloads with Param structs for new code.
	 *
	 * @param NewBundles		New bundle state for the assets that are changed
	 * @param OldBundles		Old bundle state, it will remove these bundles and replace with NewBundles
	 * @param DelegateToCall	[optional] Delegate that will be called on completion
	 * @param Priority			[optional] Async loading priority for this request
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	virtual TSharedPtr<FStreamableHandle> ChangeBundleStateForMatchingPrimaryAssets(
		const TArray<FName>& NewBundles,
		const TArray<FName>& OldBundles,
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/**
	 * Changes the bundle state of all loaded primary assets. Only assets matching OldBundles will be modified
	 * You can wait on the returned streamable request or poll as needed.
	 * If there is no work to do, returned handle will be null and delegate will get called.
	 *
	 * @param NewBundles		New bundle state for the assets that are changed
	 * @param OldBundles		Old bundle state, it will remove these bundles and replace with NewBundles
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that can be used to poll or wait. You do not need to keep this handle to stop the assets from being unloaded
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> ChangeBundleStateForMatchingPrimaryAssets(
		const TArray<FName>& NewBundles,
		const TArray<FName>& OldBundles,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Returns the loading handle associated with the primary asset, it can then be checked for progress or waited on
	 *
	 * @param PrimaryAssetId	Asset to get handle for
	 * @param bForceCurrent		If true, returns the current handle. If false, will return pending if active, or current if not
	 * @param Bundles			If not null, will fill in with a list of the requested bundle state
	 * @return					Streamable Handle that can be used to poll or wait
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> GetPrimaryAssetHandle(const FPrimaryAssetId& PrimaryAssetId, bool bForceCurrent = false, TArray<FName>* Bundles = nullptr) const ;

	/** 
	 * Returns a list of primary assets that are in the given bundle state. Only assets that are loaded or being loaded are valid
	 *
	 * @param PrimaryAssetList	Any valid assets are added to this list
	 * @param ValidTypes		List of types that are allowed. If empty, all types allowed
	 * @param RequiredBundles	Adds to list if the bundle state has all of these bundles. If empty will return all loaded
	 * @param ExcludedBundles	Doesn't add if the bundle state has any of these bundles
	 * @param bForceCurrent		If true, only use the current state. If false, use the current or pending
	 * @return					True if any found
	 */
	ENGINE_API bool GetPrimaryAssetsWithBundleState(TArray<FPrimaryAssetId>& PrimaryAssetList, const TArray<FPrimaryAssetType>& ValidTypes, const TArray<FName>& RequiredBundles, const TArray<FName>& ExcludedBundles = TArray<FName>(), bool bForceCurrent = false) const;

	/** Fills in a TMap with the pending/active loading state of every asset */
	ENGINE_API void GetPrimaryAssetBundleStateMap(TMap<FPrimaryAssetId, TArray<FName>>& BundleStateMap, bool bForceCurrent = false) const;

	/**
	 * Fills in a set of object paths with the assets that need to be loaded, for a given Primary Asset and bundle list.
	 * Prefer to use GetPrimaryAssetLoadList instead, because this overload creates additional allocation and is here just for backward compatibility.
	 *
	 * @param OutAssetLoadSet	Set that will have asset paths added to it
	 * @param PrimaryAssetId	Asset that would be loaded
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param bLoadRecursive	If true, this will call RecursivelyExpandBundleData and recurse into sub bundles of other primary assets loaded by a bundle reference
	 * @return					True if primary asset id was found
	 */
	ENGINE_API bool GetPrimaryAssetLoadSet(TSet<FSoftObjectPath>& OutAssetLoadSet, const FPrimaryAssetId& PrimaryAssetId, const TArray<FName>& LoadBundles, bool bLoadRecursive) const;

	/**
	 * Fills in a array of unique object paths with the assets that need to be loaded, for a given Primary Asset and bundle list
	 *
	 * @param OutAssetLoadList	TArray that will have asset paths added to it
	 * @param PrimaryAssetId	Asset that would be loaded
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param bLoadRecursive	If true, this will call RecursivelyExpandBundleData and recurse into sub bundles of other primary assets loaded by a bundle reference
	 * @param bEnsureUniqueness Only unique items will be inserted in OutAssetLoadList
	 * @return					True if primary asset id was found
	 */
	ENGINE_API bool GetPrimaryAssetLoadList(TArray<FSoftObjectPath>& OutAssetLoadList, const FPrimaryAssetId& PrimaryAssetId, const TArray<FName>& LoadBundles, bool bLoadRecursive, bool bEnsureUniqueness = true) const;

	/**
	 * Preloads data for a set of assets in a specific bundle state, and returns a handle you must keep active.
	 * These assets are not officially Loaded, so Unload/ChangeBundleState will not affect them and if you release the handle without otherwise loading the assets they will be freed.
	 * Prefer the overloads with Param structs for new code.
	 *
	 * @param AssetsToLoad		List of primary assets to load
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param bLoadRecursive	If true, this will call RecursivelyExpandBundleData and recurse into sub bundles of other primary assets loaded by a bundle reference
	 * @param DelegateToCall	[optional] Delegate that will be called on completion
	 * @param Priority			[optional] Async loading priority for this request
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	virtual TSharedPtr<FStreamableHandle> PreloadPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToLoad,
		const TArray<FName>& LoadBundles,
		bool bLoadRecursive,
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());
	/**
	 * Preloads data for a set of assets in a specific bundle state, and returns a handle you must keep active.
	 * These assets are not officially Loaded, so Unload/ChangeBundleState will not affect them and if you release the handle without otherwise loading the assets they will be freed.
	 *
	 * @param AssetsToLoad		List of primary assets to load
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param bLoadRecursive	If true, this will call RecursivelyExpandBundleData and recurse into sub bundles of other primary assets loaded by a bundle reference
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> PreloadPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToLoad,
		const TArray<FName>& LoadBundles,
		bool bLoadRecursive,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/**
	 * Downloads data for a set of assets in a specific bundle state, and returns a handle with a FStreamableDownloadCachePin.
	 * The FStreamableDownloadCachePin must be kept active to hold the data in the download cache.
	 * Does not load any assets.
	 *
	 * @param AssetsToLoad		List of primary assets to load
	 * @param LoadBundles		List of bundles to load for those assets
	 * @param bLoadRecursive	If true, this will call RecursivelyExpandBundleData and recurse into sub bundles of other primary assets loaded by a bundle reference
	 * @param LoadParams		Contains callbacks and download params. Download params are required.
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> PredownloadPrimaryAssets(
		const TArray<FPrimaryAssetId>& AssetsToLoad,
		const TArray<FName>& LoadBundles,
		bool bLoadRecursive,
		FAssetManagerLoadParams&& LoadParams,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/**
	 * Load non primary assets with the primary streamable manager.
	 * This will not auto release the handle, release it if needed.
	 * Prefer the overloads with Param structs for new code.
	 * 
	 * @param AssetList			List of non primary assets to load
	 * @param DelegateToCall	[optional] Delegate that will be called on completion
	 * @param Priority			[optional] Async loading priority for this request
	 * @param DebugName			[optional] Name of this handle, either FString or anything that can construct FString, will be reported in debug tools, will report Source Location if DebugName is not specified explicitly
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	TSharedPtr<FStreamableHandle> LoadAssetList(
		const TArray<FSoftObjectPath>& AssetList,
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		FString&& DebugName = FString("LoadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());
	/** rvalue reference overload for Asset List */
	TSharedPtr<FStreamableHandle> LoadAssetList(
		TArray<FSoftObjectPath>&& AssetList,
		FStreamableDelegate DelegateToCall = FStreamableDelegate(),
		TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
		FString&& DebugName = FString("LoadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());
	/**
	 * lvalue reference overload to load non primary assets with the primary streamable manager.
	 * This will not auto release the handle, release it if needed.
	 * Prefer using rvalue reference version as it will not create an extra copy of your input array.
	 * 
	 * @param AssetList			List of non primary assets to load
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param DebugName			[optional] Name of this handle, either FString or anything that can construct FString, will be reported in debug tools, will report Source Location if DebugName is not specified explicitly
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	TSharedPtr<FStreamableHandle> LoadAssetList(
		const TArray<FSoftObjectPath>& AssetList,
		FAssetManagerLoadParams&& LoadParams,
		FString&& DebugName = FString("LoadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());

	/**
	 * Load non primary assets with the primary streamable manager.
	 * This will not auto release the handle, release it if needed.
	 * This is a entry point for the set of overloads. Overide it to have your custom behavior.
	 * 
	 * @param AssetList			List of non primary assets to load
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param DebugName			[optional] Name of this handle, either FString or anything that can construct FString, will be reported in debug tools, will report Source Location if DebugName is not specified explicitly
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	ENGINE_API virtual TSharedPtr<FStreamableHandle> LoadAssetList(
		TArray<FSoftObjectPath>&& AssetList,
		FAssetManagerLoadParams&& LoadParams,
		FString&& DebugName = FString("LoadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());

	/**
	 * Download non primary assets with the primary streamable manager.
	 * Does not load assets after downloading.
	 * This will not auto release the handle, release it if needed.
	 * 
	 * @param AssetList			List of non primary assets to load
	 * @param LoadParams		Contains callbacks and async load priority
	 * @param DebugName			[optional] Name of this handle, either FString or anything that can construct FString, will be reported in debug tools, will report Source Location if DebugName is not specified explicitly
	 * @param Location			[optional] Is not intended for direct use, the parameter catches call site source location and passes it down to the streaming manager. See FStreamingManager.
	 * @return					Streamable Handle that must be stored to keep the preloaded assets from being freed
	 */
	TSharedPtr<FStreamableHandle> DownloadAssetList(
		const TArray<FSoftObjectPath>& AssetList,
		FAssetManagerLoadParams&& LoadParams,
		FString&& DebugName = FString("DownloadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());
	/** rvalue reference overload for Asset List */
	ENGINE_API TSharedPtr<FStreamableHandle> DownloadAssetList(
		TArray<FSoftObjectPath>&& AssetList,
		FAssetManagerLoadParams&& LoadParams,
		FString&& DebugName = FString("DownloadAssetList"),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current());

	/** Returns a single AssetBundleInfo, matching Scope and Name */
	ENGINE_API virtual FAssetBundleEntry GetAssetBundleEntry(const FPrimaryAssetId& BundleScope, FName BundleName) const;

	/** Appends all AssetBundleInfos inside a given scope */
	ENGINE_API virtual bool GetAssetBundleEntries(const FPrimaryAssetId& BundleScope, TArray<FAssetBundleEntry>& OutEntries) const;

	/** 
	 * Returns the list of Chunks that are not currently mounted, and are required to load the referenced assets. Returns true if any chunks are missing
	 *
	 * @param AssetList				Asset Paths to check chunks for
	 * @param OutMissingChunkList	Chunks that are known about but not yet installed
	 * @param OutErrorChunkList		Chunks that do not exist at all and are not installable
	 */
	ENGINE_API virtual bool FindMissingChunkList(const TArray<FSoftObjectPath>& AssetList, TArray<int32>& OutMissingChunkList, TArray<int32>& OutErrorChunkList) const;

	/** 
	 * Acquires a set of chunks using the platform chunk layer, then calls the passed in callback
	 *
	 * @param AssetList				Asset Paths to get resources for
	 * @param CompleteDelegate		Delegate called when chunks have been acquired or failed. If any chunks fail the entire operation is considered a failure
	 * @param Priority				Priority to use when acquiring chunks
	 */
	ENGINE_API virtual void AcquireResourcesForAssetList(const TArray<FSoftObjectPath>& AssetList, FAssetManagerAcquireResourceDelegateEx CompleteDelegate, EChunkPriority::Type Priority = EChunkPriority::Immediate);
	ENGINE_API virtual void AcquireResourcesForAssetList(const TArray<FSoftObjectPath>& AssetList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority = EChunkPriority::Immediate);

	/** 
	 * Acquires a set of chunks using the platform chunk layer, then calls the passed in callback. This will download all bundles of a primary asset
	 *
	 * @param PrimaryAssetList		Primary assets to get chunks for
	 * @param CompleteDelegate		Delegate called when chunks have been acquired or failed. If any chunks fail the entire operation is considered a failure
	 * @param Priority				Priority to use when acquiring chunks
	 */
	ENGINE_API virtual void AcquireResourcesForPrimaryAssetList(const TArray<FPrimaryAssetId>& PrimaryAssetList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority = EChunkPriority::Immediate);
	
	/** Returns the chunk download/install progress. AcquiredCount is the number of chunks that were requested and have already been insatlled, Requested includes both installed and pending */
	ENGINE_API virtual bool GetResourceAcquireProgress(int32& OutAcquiredCount, int32& OutRequestedCount) const;

	// FUNCTIONS FOR MANAGEMENT/COOK RULES

	/** Changes the default management rules for a specified type */
	ENGINE_API virtual void SetPrimaryAssetTypeRules(FPrimaryAssetType PrimaryAssetType, const FPrimaryAssetRules& Rules);

	/** Changes the management rules for a specific asset, this overrides the type rules. If passed in Rules is default, delete override */
	ENGINE_API virtual void SetPrimaryAssetRules(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRules& Rules);
	ENGINE_API virtual void SetPrimaryAssetRulesExplicitly(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRulesExplicitOverride& ExplicitRules);

	/** Gets the management rules for a specific asset, this will merge the type and individual values */
	ENGINE_API virtual FPrimaryAssetRules GetPrimaryAssetRules(FPrimaryAssetId PrimaryAssetId) const;

	/** Returns list of asset packages managed by primary asset */
	ENGINE_API virtual bool GetManagedPackageList(FPrimaryAssetId PrimaryAssetId, TArray<FName>& AssetPackageList) const;

	/**
	 * Returns PrimaryAssetIds that manage whether a package is included in the cook, with property describing the
	 * reference (direct or indirect).
	 */
	ENGINE_API virtual bool GetPackageCookRuleManagers(FName PackageName, bool bRecurseToParents,
		TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const;

	/**
	 * Returns PrimaryAssetIds that manage a package in any way, either chunk assignment or inclusion in the cook.
	 * Chunk assignments are a superset of inclusion in the cook assignments, so this is a supserset of
	 * GetPackageCookRuleManagers.
	 * Includes property describing the reference (direct or indirect, and cookrule or not).
	 */
	ENGINE_API virtual bool GetPackageManagers(FName PackageName, bool bRecurseToParents,
		TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const;

	/** Returns just the keys of the TMap returned from GetPackageManagers that takes a TMap&. */
	ENGINE_API virtual bool GetPackageManagers(FName PackageName, bool bRecurseToParents, TSet<FPrimaryAssetId>& ManagerSet) const;

protected:
	ENGINE_API bool GetPackageManagersInternal(FName PackageName,
		bool bUseCookRuleReferences, bool bRecurseToParents,
		TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers) const;
public:

	// GENERAL ASSET UTILITY FUNCTIONS

	/** Parses AssetData to extract the primary type/name from it. This works even if it isn't in the directory */
	ENGINE_API virtual FPrimaryAssetId ExtractPrimaryAssetIdFromData(const FAssetData& AssetData, FPrimaryAssetType SuggestedType = FPrimaryAssetType()) const;

	/** Gets the FAssetData at a specific path, handles redirectors and blueprint classes correctly. Returns true if it found a valid data */
	ENGINE_API virtual bool GetAssetDataForPath(const FSoftObjectPath& ObjectPath, FAssetData& AssetData) const;

	/** Checks to see if the given asset data is a blueprint with a base class in the ClassNameSet. This checks the parent asset tag */
	ENGINE_API virtual bool IsAssetDataBlueprintOfClassSet(const FAssetData& AssetData, const TSet<FTopLevelAssetPath>& ClassNameSet) const;

	/** Turns an FAssetData into FSoftObjectPath, handles adding _C as necessary */
	ENGINE_API virtual FSoftObjectPath GetAssetPathForData(const FAssetData& AssetData) const;

	/** Tries to redirect a Primary Asset Id, using list in AssetManagerSettings */
	ENGINE_API virtual FPrimaryAssetId GetRedirectedPrimaryAssetId(const FPrimaryAssetId& OldId) const;

	/** Reads redirector list and gets a list of the redirected previous names for a Primary Asset Id */
	ENGINE_API virtual void GetPreviousPrimaryAssetIds(const FPrimaryAssetId& NewId, TArray<FPrimaryAssetId>& OutOldIds) const;

	/** If bShouldManagerDetermineTypeAndName is true in settings, this function is used to determine the primary asset id for any object that does not have it's own implementation. Games can override the behavior here or call it from other places */
	ENGINE_API virtual FPrimaryAssetId DeterminePrimaryAssetIdForObject(const UObject* Object) const;

	/** Reads AssetManagerSettings for specifically redirected asset paths. This is useful if you need to convert older saved data */
	ENGINE_API virtual FSoftObjectPath GetRedirectedAssetPath(const FSoftObjectPath& OldPath) const;

	/** Extracts all FSoftObjectPaths from a Class/Struct */
	ENGINE_API virtual void ExtractSoftObjectPaths(const UStruct* Struct, const void* StructValue, TArray<FSoftObjectPath>& FoundAssetReferences, const TArray<FName>& PropertiesToSkip = TArray<FName>()) const;

	/** Helper function to search the asset registry for AssetData matching search rules */
	ENGINE_API virtual int32 SearchAssetRegistryPaths(TArray<FAssetData>& OutAssetDataList, const FAssetManagerSearchRules& Rules) const;

	/** Helper function to check if a single asset passes restrictions in SearchRules, this can be used when an asset is manually registered */
	ENGINE_API virtual bool DoesAssetMatchSearchRules(const FAssetData& AssetData, const FAssetManagerSearchRules& Rules) const;

	/** Returns true if the specified TypeInfo should be scanned. Can be overridden by the game */
	ENGINE_API virtual bool ShouldScanPrimaryAssetType(FPrimaryAssetTypeInfo& TypeInfo) const;

	/** Manually register a new or updated primary asset, returns true if it was successful */
	ENGINE_API virtual bool RegisterSpecificPrimaryAsset(const FPrimaryAssetId& PrimaryAssetId, const FAssetData& NewAssetData);

	/** Helper function which requests the asset registery scan a list of directories/assets */
	ENGINE_API virtual void ScanPathsSynchronous(const TArray<FString>& PathsToScan) const;

	/**
	 * Called when a new asset registry becomes available
	 * @param InName Logical name for this asset registry. Must match the name returned by GetUniqueAssetRegistryName
	 *
	 * @returns TRUE if new data was loaded and added into the master asset registry
	 */
	ENGINE_API virtual bool OnAssetRegistryAvailableAfterInitialization(FName InName, FAssetRegistryState& OutNewState);

	/** Returns the root path for the package name or path (i.e. /Game/MyPackage would return /Game/ ). This works even if the root is not yet mounted */
	static ENGINE_API bool GetContentRootPathFromPackageName(const FString& PackageName, FString& OutContentRootPath);

	/** Normalize a package path for use in asset manager, will remove duplicate // and add or remove a final slash as desired */
	static ENGINE_API void NormalizePackagePath(FString& InOutPath, bool bIncludeFinalSlash);
	static ENGINE_API FString GetNormalizedPackagePath(const FString& InPath, bool bIncludeFinalSlash);

	/** Dumps out summary of managed types to log */
	static ENGINE_API void DumpAssetTypeSummary();

	/** Dumps out list of loaded asset bundles to log */
	static ENGINE_API void DumpLoadedAssetState();

	/** Shows a list of all bundles for the specified primary asset by primary asset id (i.e. Map:Entry) */
	static ENGINE_API void DumpBundlesForAsset(const TArray<FString>& Args);

	/** Dumps information about the Asset Registry to log */
	static ENGINE_API void DumpAssetRegistryInfo();

	/** Dumps out list of primary asset -> managed assets to log */
	static ENGINE_API void DumpReferencersForPackage(const TArray< FString >& PackageNames);

	/** Finds all the referencers for a set of packages. Recursively will get all the referencers up to the max depth */
	static ENGINE_API void GetAllReferencersForPackage(TSet<FAssetData>& OutFoundAssets, const TArray<FName>& InPackageNames, int32 MaxDepth);

	/** Starts initial load, gets called from InitializeObjectReferences */
	ENGINE_API virtual void StartInitialLoading();

	/** Finishes initial loading, gets called from end of Engine::Init() */
	ENGINE_API virtual void FinishInitialLoading();

	/** Accessor for asset registry */
	ENGINE_API IAssetRegistry& GetAssetRegistry() const;

	/** Return settings object */
	ENGINE_API const UAssetManagerSettings& GetSettings() const;

	/** Returns a timer manager that is safe to use for asset loading actions. This will either be the editor or game instance one, or null during very early startup */
	ENGINE_API FTimerManager* GetTimerManager() const;

	// Overrides
	ENGINE_API virtual void BeginDestroy() override;

	ENGINE_API virtual void PostInitProperties() override;

	/** Get the encryption key guid attached to this primary asset. Can be invalid if the asset is not encrypted */
	ENGINE_API virtual void GetCachedPrimaryAssetEncryptionKeyGuid(FPrimaryAssetId InPrimaryAssetId, FGuid& OutGuid);

	/** Loads the redirector maps */
	ENGINE_API virtual void LoadRedirectorMaps();

	/** Refresh the entire set of asset data, can call from editor when things have changed dramatically. Will only refresh if force is true or it thinks something has changed */
	ENGINE_API virtual void RefreshPrimaryAssetDirectory(bool bForceRefresh = false);

	/** Invalidate cached asset data so it knows to rescan when needed */
	ENGINE_API virtual void InvalidatePrimaryAssetDirectory();

	/** Warn about this primary asset id being missing, but only if this is the first time this session */
	ENGINE_API virtual void WarnAboutInvalidPrimaryAsset(const FPrimaryAssetId& PrimaryAssetId, const FString& Message) const;

	/** Helper function to write out asset reports */
	ENGINE_API virtual bool WriteCustomReport(FString FileName, TArray<FString>& FileLines) const;

	/** Apply a single custom primary asset rule, calls function below */
	ENGINE_API virtual void ApplyCustomPrimaryAssetRulesOverride(const FPrimaryAssetRulesCustomOverride& CustomOverride);

#if WITH_EDITOR
	// EDITOR ONLY FUNCTIONALITY

	/** Gets package names to add to the cook, and packages to never cook even if in startup set memory or referenced */
	ENGINE_API virtual void ModifyCook(TConstArrayView<const ITargetPlatform*> TargetPlatforms, TArray<FName>& PackagesToCook,
		TArray<FName>& PackagesToNeverCook);

	/** Gets package names to add to a DLC cook*/
	ENGINE_API virtual void ModifyDLCCook(const FString& DLCName, TConstArrayView<const ITargetPlatform*> TargetPlatforms,
		TArray<FName>& PackagesToCook, TArray<FName>& PackagesToNeverCook);

	/**
	 * Allows for game code to modify the base packages that have been read in from the DevelopmentAssetRegistry when performing a DLC cook.
	 * Can be used to modify which packages should be considered to be already cooked.
	 * Any packages within the PackagesToClearResults will have their cook results cleared and be cooked again if requested by the cooker.
	 */
	virtual void ModifyDLCBasePackages(const ITargetPlatform* TargetPlatform, TArray<FName>& PlatformBasedPackages, TSet<FName>& PackagesToClearResults) const {};

	/**
	 * If the given package contains a primary asset, get the packages referenced by its AssetBundleEntries.
	 * Used to inform the cook of should-be-cooked dependencies of PrimaryAssets for PrimaryAssets that
	 * are recorded in the AssetManager but have cooktype Unknown and so are not returned from ModifyCook.
	 */
	ENGINE_API virtual void ModifyCookReferences(FName PackageName, TArray<FName>& PackagesToCook);

	/** Returns whether or not a specific UPackage should be cooked for the provied TargetPlatform */
	ENGINE_API virtual bool ShouldCookForPlatform(const UPackage* Package, const ITargetPlatform* TargetPlatform);

	/** Returns cook rule for a package name using Management rules, games should override this to take into account their individual workflows */
	ENGINE_API virtual EPrimaryAssetCookRule GetPackageCookRule(FName PackageName) const;

	/**
	 * Helper function for GetPackageCookRule. Given the list of Managers that manage a package, calculate the unioned cookrule for the
	 * package. @see FPrimaryAssetCookRuleUnion::UnionWith. If two managers are in conflict (e.g. one is CookAlways, the other is
	 * ProductionNeverCook), the higher-priority will win. If they have the same priority, the NeverCook rule will win, and the managers
	 * will be reported in the OutConflictIds field.
	 */
	ENGINE_API EPrimaryAssetCookRule CalculateCookRuleUnion(const TMap<FPrimaryAssetId, UE::AssetRegistry::EDependencyProperty>& Managers,
		TOptional<TPair<FPrimaryAssetId, FPrimaryAssetId>>* OutConflictIds) const;

	/** Returns true if the specified asset package can be cooked, will error and return false if it is disallowed */
	ENGINE_API virtual bool VerifyCanCookPackage(UE::Cook::ICookInfo* CookInfo, FName PackageName, bool bLogError = true) const;

	/** 
	 * For a given package and platform, return what Chunks it should be assigned to, games can override this as needed. Returns false if no information found 
	 * @param PackageName Package to check chunks for
	 * @param TargetPlatform Can be used to do platform-specific chunking, this is null when previewing in the editor
	 * @param ExistingChunkList List of chunks that the asset was previously assigned to, may be empty
	 * @param OutChunkList List of chunks to actually assign this to
	 * @param OutOverrideChunkList List of chunks that were added due to override rules, not just normal primary asset rules. Tools use this for dependency checking
	 */
	ENGINE_API virtual bool GetPackageChunkIds(FName PackageName, const ITargetPlatform* TargetPlatform, TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList, TArray<int32>* OutOverrideChunkList = nullptr) const;

	/** 
	 * Retrieve the encryption key guid for a given chunk ID
	 *
	 * @param InChunkIndex - The chunk ID for which we are querying the encryption key guid
	 * @return If chunk ID is valid and references and encrypted chunk, returns the key guid. Otherwise, returns an empty guid 
	 */
	virtual FGuid GetChunkEncryptionKeyGuid(int32 InChunkId) const { return FGuid(); }

	/** Get the encrypted chunk assignments for a give package. Only valid between PreSaveAssetRegistry and PostSaveAssetRegistry */
	virtual TSet<int32> GetEncryptedChunkIDsForPackage(FName InPackageName) const { return TSet<int32>(); }

	/**
	 * Determine if we should separate the asset registry for this chunk out into its own file and return the unique name that identifies it
	 * @param InChunkIndex Chunk index to check
	 * @returns The logical name of the chunk's asset registry, or NAME_None if it isn't required
	 */
	virtual FName GetUniqueAssetRegistryName(int32 InChunkIndex) const { return NAME_None; }

	/**
	 * For a given content encryption group name (as defined in the content encryption config that the project provides, return the relevant chunk ID
	 */
	virtual int32 GetContentEncryptionGroupChunkID(FName InGroupName) const { return INDEX_NONE; }

	/** Returns the list of chunks assigned to the list of primary assets, which is usually a manager list. This is called by GetPackageChunkIds */
	ENGINE_API virtual bool GetPrimaryAssetSetChunkIds(const TSet<FPrimaryAssetId>& PrimaryAssetSet, const ITargetPlatform* TargetPlatform,
		TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList) const;
	ENGINE_API virtual bool GetPrimaryAssetSetChunkIds(TConstArrayView<FPrimaryAssetId> PrimaryAssets, const ITargetPlatform* TargetPlatform,
		TArrayView<const int32> ExistingChunkList, TArray<int32>& OutChunkList) const;

	/** Resets all asset manager data, called in the editor to reinitialize the config */
	ENGINE_API virtual void ReinitializeFromConfig();

	/** Updates the asset management database if needed */
	UE_DEPRECATED(5.6, "UpdateManagementDatabase as been updated to take InFlags instead.")
	ENGINE_API virtual void UpdateManagementDatabase(bool bForceRefresh);
	ENGINE_API virtual void UpdateManagementDatabase(EUpdateManagementDatabaseFlags InFlags = EUpdateManagementDatabaseFlags::BuildChunkMap);

	/** Returns the chunk information map computed during UpdateManagementDatabase */
	ENGINE_API const TMap<int32, FAssetManagerChunkInfo>& GetChunkManagementMap() const;

	/**
	 * Called post cook to build the chunk map only with the packages which have been cooked.
	 * Provides the StartupPackages which will go into chunk 0.
	 *
	 * It's expected this function will set the value of CachedChunkMap which can be used for package chunk assignment.
	 */
	UE_INTERNAL ENGINE_API void UpdateCachedChunkMapAfterCook(const TSet<FName>& AllCookedPackages, const TSet<FName>& StartupPackages);

	/** Handles applying Asset Labels and should be overridden to do any game-specific labelling */
	ENGINE_API virtual void ApplyPrimaryAssetLabels();

	/** Refreshes cache of asset data for in memory object */
	ENGINE_API virtual void RefreshAssetData(UObject* ChangedObject);

	/** 
	 * Initializes asset bundle data from a passed in struct or class, this will read the AssetBundles metadata off the UProperties. As an example this property definition:
	 *
	 * UPROPERTY(EditDefaultsOnly, Category=Data, meta = (AssetBundles = "Client,Server"))
	 * TSoftObjectPtr<UCurveTable> CurveTableReference;
	 *
	 * Would add the value of CurveTableReference to both the Client and Server asset bundles
	 *
	 * @param Struct		UScriptStruct or UClass representing the property hierarchy
	 * @param StructValue	Location in memory of Struct or Object
	 * @param AssetBundle	Bundle that will be filled out
	 */
	ENGINE_API virtual void InitializeAssetBundlesFromMetadata(const UStruct* Struct, const void* StructValue, FAssetBundleData& AssetBundle, FName DebugName = NAME_None) const;

	/** UObject wrapper */
	virtual void InitializeAssetBundlesFromMetadata(const UObject* Object, FAssetBundleData& AssetBundle) const
	{
		InitializeAssetBundlesFromMetadata(Object->GetClass(), Object, AssetBundle, Object->GetFName());
	}

	/** 
	  * Called immediately before saving the asset registry during cooking
	  */
	virtual void PreSaveAssetRegistry(const ITargetPlatform* TargetPlatform, const TSet<FName>& InCookedPackages) {}

	/** 
	  * Called immediately after saving the asset registry during cooking
	  */
	virtual void PostSaveAssetRegistry() {}

	/**
	  * Gathers information about which assets the game wishes to encrypt into named groups
	  */
	virtual void GetContentEncryptionConfig(FContentEncryptionConfig& OutContentEncryptionConfig) {}

	/** 
	  * Processes a command token as part of cooking.
	 *
	 * @param Token - The token to process.
	 * @return True if the token was processed, false otherwise.
	 */
	virtual bool HandleCookCommand(FStringView Token)
	{
		return false;
	}
#endif

	/**
	 * Allows the asset manager to override the chunkID to be a string rather than integer value.
	 * Used at cook time and runtime to determine how to map chunkids to the potential string override.
	 * This does not update the internal asset registry number only the generated pak chunk file name.
	 * The AssetRegistryGenerator will request this mapping during Chunk Generation time.
	 * All strings must be be alpha numeric with no spaces.
	 *
	 * When overriding it's recommended to setup PlatformChunk mapping via the InstallBundle rules.
	 * When a pak chunk does not have a parseable number the default platform chunk will be 0 unless InstallBundle rules override that.
	 *
	 * 
	 * @param ChunkIdStringOverride - Out parameter for the override information of a chunkID to string.
	 */
	virtual void GetPakChunkIdToStringMapping(TMap<int32, FString>& ChunkIdStringOverride) {}
	virtual void GetPakChunkIdToStringMapping(const FString& PlatformName, TMap<int32, FString>& ChunkIdStringOverride) {}

	/**
	 * Helper to make it easy to get the ChunkId based on the string override for a pakchunk.
	 * see "GetPakChunkIdToStringMapping" for more information
	 *
	 * @param ChunkIdStringOverride - Out parameter for getting the chunkID based on the string override.
	 */
	ENGINE_API void GetPakStringToChunkIdMapping(TMap<FString, int32>& StringOverrideToChunkId);

protected:
	friend FAssetManagerEditorModule;

	/** Internal helper function that attempts to get asset data from the specified path; Accounts for possibility of blueprint classes ending in _C */
	ENGINE_API virtual void GetAssetDataForPathInternal(IAssetRegistry& AssetRegistry, const FString& AssetPath, OUT FAssetData& OutAssetData) const;

	/** Updates the asset data cached on the name data; returns false if the asset is not a valid primary asset. */
	ENGINE_API virtual bool TryUpdateCachedAssetData(const FPrimaryAssetId& PrimaryAssetId, const FAssetData& NewAssetData, bool bAllowDuplicates);

	/** Returns the NameData for a specific type/name pair */
	ENGINE_API FPrimaryAssetData* GetNameData(const FPrimaryAssetId& PrimaryAssetId, bool bCheckRedirector = true);
	ENGINE_API const FPrimaryAssetData* GetNameData(const FPrimaryAssetId& PrimaryAssetId, bool bCheckRedirector = true) const;

	ENGINE_API virtual void OnObjectReferenceListInvalidated();

	ENGINE_API void CallPreGarbageCollect();
	ENGINE_API virtual void PreGarbageCollect();

	/** Called when an internal load handle finishes, handles setting to pending state */
	UE_DEPRECATED(5.6, "Use NotifyOnAssetStateChangeCompleted instead")
	virtual void OnAssetStateChangeCompleted(FPrimaryAssetId PrimaryAssetId, TSharedPtr<FStreamableHandle> BoundHandle, FStreamableDelegate WrappedDelegate) { checkf(false, TEXT("Deprecated")); }

	/** Called when an internal load handle finishes */
	virtual void NotifyOnAssetStateChangeCompleted(const FPrimaryAssetId& PrimaryAssetId) {}

	/** Called when an internal load handle finishes, handles setting to pending state */
	ENGINE_API void OnAssetStateChangeCompleted(TSharedPtr<FStreamableHandle> BoundHandle, FPrimaryAssetId PrimaryAssetId);

	/** Scans all asset types specified in DefaultGame */
	ENGINE_API virtual void ScanPrimaryAssetTypesFromConfig();

	/** Called to apply the primary asset rule overrides from config */
	ENGINE_API virtual void ScanPrimaryAssetRulesFromConfig();

	/** Sees if a specific primary asset passes the custom override filter, subclass this to handle FilterString */
	ENGINE_API virtual bool DoesPrimaryAssetMatchCustomOverride(FPrimaryAssetId PrimaryAssetId, const FPrimaryAssetRulesCustomOverride& CustomOverride) const;

	/** Called after scanning is complete, either from FinishInitialLoading or after the AssetRegistry finishes */
	ENGINE_API virtual void PostInitialAssetScan();

	/** Returns true if path should be excluded from primary asset scans, called from ShouldIncludeInAssetSearch and in the editor */
	ENGINE_API virtual bool IsPathExcludedFromScan(const FString& Path) const;

	/** Returns true if we're in the middle of handling the initial config, false if this is being called from something else like a plugin */
	ENGINE_API bool IsScanningFromInitialConfig() const;

	/** Filter function that is called from SearchAssetRegistryPaths, returns true if asset data should be included in search results */
	ENGINE_API virtual bool ShouldIncludeInAssetSearch(const FAssetData& AssetData, const FAssetManagerSearchRules& SearchRules) const;

	/** Call to start acquiring a list of chunks */
	ENGINE_API virtual void AcquireChunkList(const TArray<int32>& ChunkList, FAssetManagerAcquireResourceDelegate CompleteDelegate, EChunkPriority::Type Priority, TSharedPtr<FStreamableHandle> StalledHandle);

	/** Called when a new chunk has been downloaded */
	ENGINE_API virtual void OnChunkDownloaded(uint32 ChunkId, bool bSuccess);

	/** Called to load asset list */
	UE_DEPRECATED(5.6, "Function was renamed to LoadAssetList that accepts FAssetManagerLoadParams struct.")
	virtual TSharedPtr<FStreamableHandle> LoadAssetListInternal(TArray<FSoftObjectPath>&& AssetList, FStreamableDelegateWithHandle&& DelegateToCall, TAsyncLoadPriority Priority, FString&& DebugName)
	{
		checkf(false, TEXT("Deprecated"));
		return nullptr;
	}

#if WITH_EDITOR
	/** Function used during creating Management references to decide when to recurse and set references */
	ENGINE_API virtual EAssetSetManagerResult::Type ShouldSetManager(UE::AssetRegistry::FShouldSetManagerContext& Context) const;
	UE_DEPRECATED(5.6, "Override the version that takes an FShouldSetManagerContext instead.")
	virtual EAssetSetManagerResult::Type ShouldSetManager(const FAssetIdentifier& Manager, const FAssetIdentifier& Source, const FAssetIdentifier& Target,
		UE::AssetRegistry::EDependencyCategory Category, UE::AssetRegistry::EDependencyProperty Properties, EAssetSetManagerFlags::Type Flags) const
	{
		return EAssetSetManagerResult::DoNotSet;
	}

	/** Called when asset registry is done loading off disk, will finish any deferred loads */
	ENGINE_API virtual void OnAssetRegistryFilesLoaded();

	/** Handles updating manager when a new asset is created */
	ENGINE_API virtual void OnInMemoryAssetCreated(UObject *Object);

	/** Handles updating manager if deleted object is relevant*/
	ENGINE_API virtual void OnInMemoryAssetDeleted(UObject *Object);

	/** Called when object is saved */
	ENGINE_API virtual void OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext);

	/** When asset is renamed */
	ENGINE_API virtual void OnAssetRenamed(const FAssetData& NewData, const FString& OldPath);

	/** When an asset is removed */
	ENGINE_API virtual void OnAssetRemoved(const FAssetData& Data);

	/** Try to remove an old asset identifier when it has been deleted/renamed */
	ENGINE_API virtual void RemovePrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId);

	/** Scans the respective PackagePath for public assets to include in PackagesToCook */
	ENGINE_API virtual void GatherPublicAssetsForPackage(FName PackagePath, TArray<FName>& PackagesToCook) const;

	/** Scans the respective PackagePath for public assets to include in AssetsToCook */
	ENGINE_API virtual void GatherPublicAssetsForPackage(FName PackagePath, TSet<FSoftObjectPath>& AssetsToCook) const;

	/** Called right before PIE starts, will refresh asset directory and can be overriden to preload assets */
	ENGINE_API virtual void PreBeginPIE(bool bStartSimulate);

	/** Called after PIE ends, resets loading state */
	ENGINE_API virtual void EndPIE(bool bStartSimulate);

	/**
	 * Used by UpdateManagementDatabase to build the CachedChunkMap.
	 * Should return a new cachedChunkMap.
	 * 
	 * @param PackagesToUpdateChunksFor - the packages needing chunk assignment as part of the management database update.
	 * @return a new CachedChunkMap to be used.
	 */
	UE_DEPRECATED(5.6, "BuildChunkMap now takes 2 additional arguments")
	ENGINE_API virtual TMap<int32, FAssetManagerChunkInfo> BuildChunkMap(const TSet<FName>& PackagesToUpdateChunksFor) const;

	/**
	 * Used by UpdateManagementDatabase and at the end of the cook to build the CachedChunkMap.
	 * Should return a new cachedChunkMap.
	 *
	 * @param PackagesToUpdateChunksFor - the packages needing chunk assignment as part of the management database update.
	 * @param StartupPackages - All the known startup packages. When called from UpdateManagementDatabase, this will be empty.
	 * @param PackageThatWereCooked - All the known packages that were cooked. When called from UpdateManagementDatabase, this will be empty.
	 * @return a new CachedChunkMap to be used.
	 */
	ENGINE_API virtual TMap<int32, FAssetManagerChunkInfo> BuildChunkMap(const TSet<FName>& PackagesToUpdateChunksFor, const TSet<FName>& StartupPackages, const TSet<FName>& PackageThatWereCooked) const;

	/** Copy of the asset state before PIE was entered, return to that when PIE completes */
	TMap<FPrimaryAssetId, TArray<FName>> PrimaryAssetStateBeforePIE;

	/** Cached map of chunk ids to lists of assets in that chunk */
	TMap<int32, FAssetManagerChunkInfo> CachedChunkMap;
#endif // WITH_EDITOR

	/** Map from object path to Primary Asset Id */
	TMap<FSoftObjectPath, FPrimaryAssetId> AssetPathMap;

	/** Overridden asset management data for specific types */
	TMap<FPrimaryAssetId, FPrimaryAssetRulesExplicitOverride> AssetRuleOverrides;

	/** Map from PrimaryAssetId to list of PrimaryAssetIds that are the parent of this one, for determining chunking */
	TMap<FPrimaryAssetId, TArray<FPrimaryAssetId>> ManagementParentMap;
	/** Map from PrimaryAssetId to list of PrimaryAssetIds that are the parent of this one, for determining what gets cooked */
	TMap<FPrimaryAssetId, TArray<FPrimaryAssetId>> CookRuleManagementParentMap;

	/** Cached map of asset bundles, global and per primary asset */
	TMap<FPrimaryAssetId, TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>> CachedAssetBundles;

	/** List of assets we have warned about being missing */
	mutable TSet<FPrimaryAssetId> WarningInvalidAssets;

	/** List of directories that have already been synchronously scanned */
	mutable TArray<FString> AlreadyScannedDirectories;

	/** All asset search roots including startup ones */
	TArray<FString> AllAssetSearchRoots;

	/** List of dynamic asset search roots added past startup */
	TArray<FString> AddedAssetSearchRoots;

	/** The streamable manager used for all primary asset loading */
	FStreamableManager StreamableManager;

	/** Defines a set of chunk installs that are waiting */
	struct FPendingChunkInstall
	{
		/** Chunks we originally requested */
		TArray<int32> RequestedChunks;

		/** Chunks we are still waiting for */
		TArray<int32> PendingChunks;

		/** Stalled streamable handle waiting for this install, may be null */
		TSharedPtr<FStreamableHandle> StalledStreamableHandle;

		/** Delegate to call on completion, may be empty */
		FAssetManagerAcquireResourceDelegate ManualCallback;
	};

	/** List of chunk installs that are being waited for */
	TArray<FPendingChunkInstall> PendingChunkInstalls;

	/** Cache of encryption keys used by each primary asset */
	TMap<FPrimaryAssetId, FGuid> PrimaryAssetEncryptionKeyCache;

	/** List of UObjects that are being kept from being GCd, derived from the asset type map. Arrays are currently more efficient than Sets */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjectReferenceList;

	/** True if we are running a build that is already scanning assets globally so we can perhaps avoid scanning paths synchronously */
	UPROPERTY()
	bool bIsGlobalAsyncScanEnvironment;

	/** True if PrimaryAssetType/Name will be implied for loading assets that don't have it saved on disk. Won't work for all projects */
	UPROPERTY()
	bool bShouldGuessTypeAndName;

	/** True if we should always use synchronous loads, this speeds up cooking */
	UPROPERTY()
	bool bShouldUseSynchronousLoad;

	/** True if we are loading from pak files */
	UPROPERTY()
	bool bIsLoadingFromPakFiles;

	/** True if the chunk install interface should be queries before loading assets */
	UPROPERTY()
	bool bShouldAcquireMissingChunksOnLoad;

	/** If true, DevelopmentCook assets will error when they are cooked */
	UPROPERTY()
	bool bOnlyCookProductionAssets;

	/** Suppresses bOnlyCookProductionAssets based on the AllowsDevelopmentObjects() property of the TargetPlatforms being cooked. */
	bool bTargetPlatformsAllowDevelopmentObjects;

	bool bObjectReferenceListDirty = true;

	/** >0 if we are currently in bulk scanning mode */
	UPROPERTY()
	int32 NumBulkScanRequests;

	/** True if asset data is current, if false it will need to rescan before PIE */
	UPROPERTY()
	bool bIsPrimaryAssetDirectoryCurrent;

	/** True if the asset management database is up to date */
	UPROPERTY()
	bool bIsManagementDatabaseCurrent;

	/** True if the asset management database should be updated after scan completes */
	UPROPERTY()
	bool bUpdateManagementDatabaseAfterScan;

	/** True if only on-disk assets should be searched by the asset registry */
	UPROPERTY()
	bool bIncludeOnlyOnDiskAssets;

	/** True if we have passed the initial asset registry/type scan */
	UPROPERTY()
	bool bHasCompletedInitialScan;

	/** Number of notifications seen in this update */
	UPROPERTY()
	int32 NumberOfSpawnedNotifications;

	/** Redirector maps loaded out of AssetMigrations.ini */
	TMap<FName, FName> PrimaryAssetTypeRedirects;
	TMap<FString, FString> PrimaryAssetIdRedirects;
	TMap<FSoftObjectPath, FSoftObjectPath> AssetPathRedirects;

	/** Delegate called when initial span finishes */
	static ENGINE_API FSimpleMulticastDelegate OnCompletedInitialScanDelegate;

	/** Delegate called when the asset manager singleton is created */
	static ENGINE_API FSimpleMulticastDelegate OnAssetManagerCreatedDelegate;

	/** Delegate called when a new asset search root is registered */
	FOnAddedAssetSearchRoot OnAddedAssetSearchRootDelegate;

	/** Delegate bound to chunk install */
	FDelegateHandle ChunkInstallDelegateHandle;

private:
	/** Provide proper reentrancy for AssetRegistry temporary caching */
	bool bOldTemporaryCachingMode = false;

	/** True if we're doing an initial scan, private because this may be replaced by a different data structure */
	bool bScanningFromInitialConfig = false;

	void InternalAddAssetScanPath(FPrimaryAssetTypeData& TypeData, const FString& AssetScanPath);

#if WITH_EDITOR
	/** Recursive handler for InitializeAssetBundlesFromMetadata */
	ENGINE_API virtual void InitializeAssetBundlesFromMetadata_Recursive(const UStruct* Struct, const void* StructValue, FAssetBundleData& AssetBundle, FName DebugName, TSet<const void*>& AllVisitedStructValues) const;
#endif

	/** Per-type asset information, cannot be accessed by children as it is defined in CPP file */
	TMap<FName, TSharedRef<FPrimaryAssetTypeData>> AssetTypeMap;

#if WITH_EDITOR
	/**
	 * Map from the PackageName of a PrimaryAsset to the packages that should be added to the cook
	 * based on the PrimaryAsset's AssetBundleData, if the PrimaryAsset is ever referenced.
	 */
	TMap<FName, TArray<FTopLevelAssetPath>> AssetBundlePathsForPackage;
#endif

	mutable IAssetRegistry* CachedAssetRegistry;
	mutable const UAssetManagerSettings* CachedSettings;

	friend FCompiledAssetManagerSearchRules;
};

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(DelegateToCall);
	LoadParams.Priority = Priority;

	return LoadPrimaryAssets(AssetsToLoad, LoadBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAsset(const FPrimaryAssetId& AssetToLoad, const TArray<FName>& LoadBundles, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(DelegateToCall);
	LoadParams.Priority = Priority;

	return LoadPrimaryAsset(AssetToLoad, LoadBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadPrimaryAssetsWithType(FPrimaryAssetType PrimaryAssetType, const TArray<FName>& LoadBundles, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(DelegateToCall);
	LoadParams.Priority = Priority;

	return LoadPrimaryAssetsWithType(PrimaryAssetType, LoadBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::ChangeBundleStateForPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToChange, const TArray<FName>& AddBundles, const TArray<FName>& RemoveBundles, bool bRemoveAllBundles, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(MoveTemp(DelegateToCall));
	LoadParams.Priority = Priority;

	return ChangeBundleStateForPrimaryAssets(AssetsToChange, AddBundles, RemoveBundles, bRemoveAllBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::ChangeBundleStateForMatchingPrimaryAssets(const TArray<FName>& NewBundles, const TArray<FName>& OldBundles, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(DelegateToCall);
	LoadParams.Priority = Priority;

	return ChangeBundleStateForMatchingPrimaryAssets(NewBundles, OldBundles, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::PreloadPrimaryAssets(const TArray<FPrimaryAssetId>& AssetsToLoad, const TArray<FName>& LoadBundles, bool bLoadRecursive, FStreamableDelegate DelegateToCall, TAsyncLoadPriority Priority, UE::FSourceLocation Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(MoveTemp(DelegateToCall));
	LoadParams.Priority = Priority;

	return PreloadPrimaryAssets(AssetsToLoad, LoadBundles, bLoadRecursive, MoveTemp(LoadParams), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadAssetList(
	const TArray<FSoftObjectPath>& AssetList,
	FStreamableDelegate DelegateToCall,
	TAsyncLoadPriority Priority,
	FString&& DebugName,
	UE::FSourceLocation&& Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(MoveTemp(DelegateToCall));
	LoadParams.Priority = Priority;

	return LoadAssetList(CopyTemp(AssetList), MoveTemp(LoadParams), MoveTemp(DebugName), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadAssetList(
	TArray<FSoftObjectPath>&& AssetList,
	FStreamableDelegate DelegateToCall,
	TAsyncLoadPriority Priority,
	FString&& DebugName,
	UE::FSourceLocation&& Location)
{
	FAssetManagerLoadParams LoadParams;
	LoadParams.OnComplete = UE::StreamableManager::Private::WrapDelegate(MoveTemp(DelegateToCall));
	LoadParams.Priority = Priority;

	return LoadAssetList(MoveTemp(AssetList), MoveTemp(LoadParams), MoveTemp(DebugName), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::LoadAssetList(
	const TArray<FSoftObjectPath>& AssetList,
	FAssetManagerLoadParams&& LoadParams,
	FString&& DebugName,
	UE::FSourceLocation&& Location)
{
	return LoadAssetList(CopyTemp(AssetList), MoveTemp(LoadParams), MoveTemp(DebugName), MoveTemp(Location));
}

inline TSharedPtr<FStreamableHandle> UAssetManager::DownloadAssetList(
	const TArray<FSoftObjectPath>& AssetList,
	FAssetManagerLoadParams&& LoadParams,
	FString&& DebugName,
	UE::FSourceLocation&& Location)
{
	return DownloadAssetList(CopyTemp(AssetList), MoveTemp(LoadParams), MoveTemp(DebugName), MoveTemp(Location));
}
