// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetManagerTypes.h"
#include "Engine/DeveloperSettings.h"
#include "AssetManagerSettings.generated.h"

/** Simple structure for redirecting an old asset name/path to a new one */
USTRUCT()
struct FAssetManagerRedirect
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = AssetRedirect)
	FString Old;

	UPROPERTY(EditAnywhere, Category = AssetRedirect)
	FString New;

	friend inline bool operator==(const FAssetManagerRedirect& A, const FAssetManagerRedirect& B)
	{
		return A.Old == B.Old && A.New == B.New;
	}
};

/** Simple structure to allow overriding asset rules for a specific primary asset. This can be used to set chunks */
USTRUCT()
struct FPrimaryAssetRulesOverride
{
	GENERATED_BODY()

	/** Which primary asset to override the rules for */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules)
	FPrimaryAssetId PrimaryAssetId;	

	/** What to overrides the rules with */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules, meta = (ShowOnlyInnerProperties))
	FPrimaryAssetRules Rules;
};

/** Apply primary asset rules to groups of primary assets, using type + filter directory or string */
USTRUCT()
struct FPrimaryAssetRulesCustomOverride
{
	GENERATED_BODY()

	/** Which type to apply rules for */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules)
	FPrimaryAssetType PrimaryAssetType;

	/** Will only apply to files in this directory */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules, meta = (RelativeToGameContentDir, LongPackageName))
	FDirectoryPath FilterDirectory;

	/** Game-specific string defining which assets to apply this to */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules)
	FString FilterString;

	/** What to overrides the rules with */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetRules, meta = (ShowOnlyInnerProperties))
	FPrimaryAssetRules Rules;
};

/** Settings for the Asset Management framework, which can be used to discover, load, and audit game-specific asset types */
UCLASS(config = Game, defaultconfig, meta = (DisplayName = "Asset Manager"), MinimalAPI)
class UAssetManagerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAssetManagerSettings() 
	: bOnlyCookProductionAssets(false)
	, bShouldGuessTypeAndNameInEditor(true)
	, bShouldAcquireMissingChunksOnLoad(false) 
	, bShouldWarnAboutInvalidAssets(true)
	{}

	/** List of asset types to scan at startup */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager", meta = (TitleProperty = "PrimaryAssetType"))
	TArray<FPrimaryAssetTypeInfo> PrimaryAssetTypesToScan;

	/** List of directories to exclude from scanning for Primary Assets, useful to exclude test assets */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager", meta = (RelativeToGameContentDir, LongPackageName))
	TArray<FDirectoryPath> DirectoriesToExclude;

	/** List of specific asset rule overrides */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	TArray<FPrimaryAssetRulesOverride> PrimaryAssetRules;

	/** List of game-specific asset rule overrides for types, this will not do anything by default */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	TArray<FPrimaryAssetRulesCustomOverride> CustomPrimaryAssetRules;

	/** If true, DevelopmentCook assets will error when they are cooked, you should enable this on production branches */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	bool bOnlyCookProductionAssets;

	/**
	 * If true, the asset manager will determine the type and name for Primary Assets that do not implement GetPrimaryAssetId, by calling DeterminePrimaryAssetIdForObject and using the ini settings.
	 * This works in both cooked and uncooked builds but is slower than directly implementing GetPrimaryAssetId on the native asset
	 */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	bool bShouldManagerDetermineTypeAndName;

	/**
	 * If true, PrimaryAsset Type/Name will be implied for assets in the editor even if bShouldManagerDetermineTypeAndName is false.
	 * This guesses the correct id for content that hasn't been resaved after GetPrimaryAssetId was implemented
	 */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	bool bShouldGuessTypeAndNameInEditor;

	/** If true, this will query the platform chunk install interface to request missing chunks for any requested primary asset loads */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	bool bShouldAcquireMissingChunksOnLoad;

	/** If true, the asset manager will warn when it is told to load or do something with assets it does not know about */
	UPROPERTY(config, EditAnywhere, Category = "Asset Manager")
	bool bShouldWarnAboutInvalidAssets;

	/** Redirect from Type:Name to Type:NameNew */
	UPROPERTY(config, EditAnywhere, Category = "Redirects")
	TArray<FAssetManagerRedirect> PrimaryAssetIdRedirects;

	/** Redirect from Type to TypeNew */
	UPROPERTY(config, EditAnywhere, Category = "Redirects")
	TArray<FAssetManagerRedirect> PrimaryAssetTypeRedirects;

	/** Redirect from /game/assetpath to /game/assetpathnew */
	UPROPERTY(config, EditAnywhere, Category = "Redirects")
	TArray<FAssetManagerRedirect> AssetPathRedirects;

	/** The metadata tags to be transferred to the Asset Registry. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Asset Registry", DisplayName = "Metadata Tags For Asset Registry")
	TSet<FName> MetaDataTagsForAssetRegistry;

	ENGINE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;

#if WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	ENGINE_API void ApplyMetaDataTagsSettings();
	ENGINE_API void ClearMetaDataTagsSettings();
#endif
};
