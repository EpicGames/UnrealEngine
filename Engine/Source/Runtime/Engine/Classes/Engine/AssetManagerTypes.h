// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetIdentifier.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/SoftObjectPtr.h"

#include "AssetManagerTypes.generated.h"

class FCbWriter;

/** Rule about when to cook/ship a primary asset */
UENUM()
enum class EPrimaryAssetCookRule : uint8
{
	/** Nothing is known about this asset specifically. It will cook in both Development and Production if something else depends on it. */
	Unknown,

	/** Asset should never be cooked/shipped in any situation. An error will be generated if something depends on it. */
	NeverCook,

	/** Asset will be cooked in development if something else depends on it, but will never be cooked in a production build. */
	ProductionNeverCook,
	/** Legacy name equivalent to Production Never Cook */
	DevelopmentCook=ProductionNeverCook,

	/** Asset will always be cooked in development, but should never be cooked in a production build. */
	DevelopmentAlwaysProductionNeverCook,
	/** Legacy name equivalent to DevelopmentAlwaysProductionNeverCook */
	DevelopmentAlwaysCook = DevelopmentAlwaysProductionNeverCook,

	/**
	 * Asset will always be cooked in development; nothing is known about whether it should cook in Production. It will cook
	 * in production if something else depends on it.
	 */
	DevelopmentAlwaysProductionUnknownCook,

	/** Asset will always be cooked, in both production and development. */
	AlwaysCook,
};

/** The production levels referenced by values of EPrimaryAssetCookRule. */
enum class EPrimaryAssetProductionLevel
{
	Development = 0,
	Production,
	Count
};

/** Structure defining rules for what to do with assets, this is defined per type and can be overridden per asset */
USTRUCT()
struct FPrimaryAssetRules
{
	GENERATED_BODY()
	
	/** Primary Assets with a higher priority will take precedence over lower priorities when assigning management for referenced assets. If priorities match, both will manage the same Secondary Asset. */
	UPROPERTY(EditAnywhere, Category = Rules)
	int32 Priority;

	/** Assets will be put into this Chunk ID specifically, if set to something other than -1. The default Chunk is Chunk 0. */
	UPROPERTY(EditAnywhere, Category = Rules, meta = (DisplayName = "Chunk ID"))
	int32 ChunkId;

	/** If true, this rule will apply to all referenced Secondary Assets recursively, as long as they are not managed by a higher-priority Primary Asset. */
	UPROPERTY(EditAnywhere, Category = Rules)
	bool bApplyRecursively;

	/** Rule describing when this asset should be cooked. */
	UPROPERTY(EditAnywhere, Category = Rules)
	EPrimaryAssetCookRule CookRule;

	FPrimaryAssetRules() 
		: Priority(-1), ChunkId(-1), bApplyRecursively(true), CookRule(EPrimaryAssetCookRule::Unknown)
	{
	}

	bool operator==(const FPrimaryAssetRules& Other) const
	{
		return Priority == Other.Priority
			&& bApplyRecursively == Other.bApplyRecursively
			&& ChunkId == Other.ChunkId
			&& CookRule == Other.CookRule;
	}

	/** Checks if all rules are the same as the default. If so this will be ignored. */
	ENGINE_API bool IsDefault() const;

	/** Override non-default rules from an override struct. */
	ENGINE_API void OverrideRules(const FPrimaryAssetRules& OverrideRules);

	/** Propagate cook rules from parent to child, won't override non-default values. */
	ENGINE_API void PropagateCookRules(const FPrimaryAssetRules& ParentRules);
};

/** Structure defining overrides to rules */
struct FPrimaryAssetRulesExplicitOverride
{
	FPrimaryAssetRules Rules;
	uint8 bOverridePriority:1;
	uint8 bOverrideApplyRecursively:1;
	uint8 bOverrideChunkId:1;
	uint8 bOverrideCookRule:1;

	FPrimaryAssetRulesExplicitOverride()
		: bOverridePriority(false), bOverrideApplyRecursively(false), bOverrideChunkId(false), bOverrideCookRule(false)
	{
	}

	bool HasAnyOverride() const { return bOverridePriority || bOverrideApplyRecursively || bOverrideChunkId || bOverrideCookRule; }

	/** Override non-default rules from an override struct. */
	ENGINE_API void OverrideRulesExplicitly(FPrimaryAssetRules& RulesToOverride) const;
};

/** Structure with publicly exposed information about an asset type. These can be loaded out of a config file or constructed at runtime */
USTRUCT()
struct FPrimaryAssetTypeInfo
{
	GENERATED_BODY()

	/** The logical name for this type of Primary Asset */
	UPROPERTY(EditAnywhere, Category = AssetType)
	FName PrimaryAssetType;

private:
	// Use accessors below to modify config data

	/** Base Class of all assets of this type */
	UPROPERTY(EditAnywhere, Category = AssetType, meta = (AllowAbstract))
	TSoftClassPtr<UObject> AssetBaseClass;

public:
	/** Runtime cached copy of asset base class, this will only be correct if FillRuntimeData has been called */
	UPROPERTY(Transient)
	TObjectPtr<UClass> AssetBaseClassLoaded;

	/** True if the assets loaded are blueprints classes, false if they are normal UObjects */
	UPROPERTY(EditAnywhere, Category = AssetType)
	bool bHasBlueprintClasses;

	/**
	 * If true this type will not cause anything to be cooked; the AssetManager will use instances of this type to
	 * define chunk assignments and NeverCook rules, but will ignore AlwaysCook rules. Assets labeled by instances
	 * of this type will need to be reference by another PrimaryAsset, or by something outside the AssetManager,
	 * to be cooked.
	 */
	UPROPERTY(EditAnywhere, Category = AssetType)
	bool bIsEditorOnly;

private:
	// Use accessors below to modify config data

	/** Directories to search for this asset type */
	UPROPERTY(EditAnywhere, Category = AssetType, meta = (RelativeToGameContentDir, LongPackageName))
	TArray<FDirectoryPath> Directories;

	/** Individual assets to scan */
	UPROPERTY(EditAnywhere, Category = AssetType)
	TArray<FSoftObjectPath> SpecificAssets;

public:
	/** Default management rules for this type, individual assets can be overridden */
	UPROPERTY(EditAnywhere, Category = Rules, meta = (ShowOnlyInnerProperties))
	FPrimaryAssetRules Rules;

	/** Combination of directories and individual assets to search for this asset type. Will have the Directories and Assets added to it but may include virtual paths */
	UPROPERTY(Transient)
	TArray<FString> AssetScanPaths;

	/** True if this is an asset created at runtime that has no on disk representation. Cannot be set in config */
	UPROPERTY(Transient)
	bool bIsDynamicAsset;

	/** Number of tracked assets of that type */
	UPROPERTY(Transient)
	int32 NumberOfAssets;

	FPrimaryAssetTypeInfo() : AssetBaseClass(UObject::StaticClass()), AssetBaseClassLoaded(UObject::StaticClass()), bHasBlueprintClasses(false), bIsEditorOnly(false),  bIsDynamicAsset(false), NumberOfAssets(0) {}

	/** Initializes a runtime version of the struct */
	FPrimaryAssetTypeInfo(FName InPrimaryAssetType, UClass* InAssetBaseClass, bool bInHasBlueprintClasses, bool bInIsEditorOnly)
		: PrimaryAssetType(InPrimaryAssetType), AssetBaseClass(InAssetBaseClass), AssetBaseClassLoaded(InAssetBaseClass), bHasBlueprintClasses(bInHasBlueprintClasses), bIsEditorOnly(bInIsEditorOnly), bIsDynamicAsset(false), NumberOfAssets(0)
	{
	}

	FPrimaryAssetTypeInfo(FName InPrimaryAssetType, UClass* InAssetBaseClass, bool bInHasBlueprintClasses, bool bInIsEditorOnly, TArray<FDirectoryPath>&& InDirectories, TArray<FSoftObjectPath>&& InSpecificAssets)
		: PrimaryAssetType(InPrimaryAssetType)
		, AssetBaseClass(InAssetBaseClass)
		, AssetBaseClassLoaded(InAssetBaseClass)
		, bHasBlueprintClasses(bInHasBlueprintClasses)
		, bIsEditorOnly(bInIsEditorOnly)
		, Directories(MoveTemp(InDirectories))
		, SpecificAssets(MoveTemp(InSpecificAssets))
		, bIsDynamicAsset(false)
		, NumberOfAssets(0)
	{
	}

	/** Gets the config version of the asset base class */
	const TSoftClassPtr<UObject>& GetAssetBaseClass() const
	{
		return AssetBaseClass;
	}

	/** Set the base class, only possible before runtime data is filled in */
	void SetAssetBaseClass(const TSoftClassPtr<UObject>& InAssetBaseClass)
	{
		if (ensure(CanModifyConfigData()))
		{
			AssetBaseClass = InAssetBaseClass;
		}
	}

	/** Access the config version of directories to scan, may be empty for runtime-added types */
	const TArray<FDirectoryPath>& GetDirectories() const 
	{
		return Directories;
	}

	/** Modify the config version of directories to scan, only possible before runtime data is filled in */
	TArray<FDirectoryPath>& GetDirectories()
	{
		ensure(CanModifyConfigData());
		return Directories; 
	}

	/** Access the config version of specific assets to scan, may be empty for runtime-added types */
	const TArray<FSoftObjectPath>& GetSpecificAssets() const 
	{
		return SpecificAssets;
	}
	
	/** Modify the config version of specific assets to scan, only possible before runtime data is filled in */
	TArray<FSoftObjectPath>& GetSpecificAssets() 
	{
		ensure(CanModifyConfigData());
		return SpecificAssets;
	}

	/** Returns true if this has valid config data that can be converted into runtime data as needed, this may be false for runtime only info */
	ENGINE_API bool HasValidConfigData() const;

	/** Returns true if this has valid runtime data because FillRuntimeData was previously called */
	ENGINE_API bool HasValidRuntimeData() const;

	/** Returns true if it is safe to modify config data as it has not been turned into runtime data yet */
	ENGINE_API bool CanModifyConfigData() const;

	/** Fills out transient variables based on parsed ones. Sets status bools saying rather data is valid, and rather it had to synchronously load the base class */
	ENGINE_API void FillRuntimeData(bool& bIsValid, bool& bBaseClassWasLoaded);
};

/** Information about a package chunk, computed by the asset manager or read out of the cooked asset registry */
struct FAssetManagerChunkInfo
{
	/** Packages/PrimaryAssets that were explicitly added to a chunk */
	TSet<FAssetIdentifier> ExplicitAssets;

	/** All packages/Primary Assets in a chunk, includes everything in Explicit plus recursively added ones */
	TSet<FAssetIdentifier> AllAssets;
};

/** Filter options that can be use to restrict the types of asset processed in various asset manager functionality */
enum class EAssetManagerFilter : int32
{
	// Default filter, process everything
	Default			= 0,

	// Only process assets that are unloaded (have no active or pending bundle assignments)
	UnloadedOnly	= 0x00000001
};

ENUM_CLASS_FLAGS(EAssetManagerFilter);

/** Delegate that can be used to do extra filtering on asset registry searches. Return true if it should be included */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FAssetManagerShouldIncludeDelegate, const struct FAssetData&, const struct FAssetManagerSearchRules&);

/** Rules for how to scan the asset registry for assets matching path and type descriptions */
USTRUCT(BlueprintType)
struct FAssetManagerSearchRules
{
	GENERATED_BODY()

	/** List of top-level directories and specific assets to search, must be paths starting with /, directories should not have a trailing / */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	TArray<FString> AssetScanPaths;

	/** Optional list of include wildcard patterns using * that will get matched against full package path. If there are any at least one of these must match */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	TArray<FString> IncludePatterns;

	/** Optional list of exclude wildcard patterns that can use *, if any of these match it will be excluded */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	TArray<FString> ExcludePatterns;

	/** Assets must inherit from this class, for blueprints this should be the instance base class */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	TObjectPtr<UClass> AssetBaseClass = nullptr;

	/** True if scanning for blueprints, false for all other assets */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	bool bHasBlueprintClasses = false;

	/** True if this should force a synchronous scan of the disk even if an async scan is in progress */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	bool bForceSynchronousScan = false;
	
	/** True if AssetScanPaths are real paths that do not need expansion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	bool bSkipVirtualPathExpansion = false;

	/** True if this test should skip the ShouldIncludeInAssetSearch function on AssetManager */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Default)
	bool bSkipManagerIncludeCheck = false;

	/** Native filter delegate to call on asset data, if bound it should return true if asset should be included in results */
	FAssetManagerShouldIncludeDelegate ShouldIncludeDelegate;

	/** Returns true if there are any rules set that need to be verified */
	ENGINE_API bool AreRulesSet() const;
};

/** Merges CookRules from multiple managers to decide the final CookRule for an asset they manage. */
struct FPrimaryAssetCookRuleUnion
{
	/**
	 * Add the information from another manager to the Union.
	 *
	 * @param CookRule The CookRule in the manager's FPrimaryAssetRules. CookRules that add references (AlwaysCook,
	 * DevelopmentAlwaysCook) apply an inclusion for their production level (Production or Development) and all lower
	 * levels. (e.g. AlwaysCooked applies an inclusion for both Production and Development). CookRules that add
	 * exclusions (NeverCook, ProductionNeverCook, DevelopmentAlwaysProductionNeverCook) apply an exclusion for their
	 * production level and all higher levels (e.g. NeverCooked applies an exclusion for both Production and
	 * Development.) Inclusions for each level are unioned, and exclusions for each level are unioned.
	 *
	 * @param bDirectReference Whether the manager's reference to the asset being managed is direct. Inclusions
	 * (AlwaysCook, DevelopmentAlwaysCook) apply transitively - both direct and indirect references apply the
	 * inclusion. Exclusions (NeverCook, ProductionNeverCook, DevelopmentAlwaysProductionNeverCook) apply only to
	 * direct references. e.g. 
	 * PrimaryAssetId A has DevelopmentAlwaysProductionNeverCook
	 * A -> B
	 * B -> C
	 * B == DevelopmentAlwaysProductionNeverCook
	 * C == DevelopmentAlwaysProductionUnknownCook
	 *
	 * @param Id Record of the Manager that has the reference to the asset, used for error feedback if there is
	 * a conflict.
	 *
	 * @param Priority The Manager's Priority. In the case of a conflict between two managers (one excluding and one
	 * including at the same level), if the priorities are different this is not an error, and the exclusion or
	 * inclusion directive from the higher-priority mangager will be kept. If the priorities are equal, the exclusion
	 * will be kept and a conflict result will be returned from GetRule.
	 */
	ENGINE_API void UnionWith(EPrimaryAssetCookRule CookRule, bool bDirectReference, const FPrimaryAssetId& Id, int32 Priority);

	/**
	 * Get the result of the unions up to this point. If OutConflictIds is non-null, it will be cleared if there
	 * are no conflicts, and will be set to the Ids that have a conflict between inclusion and exclusion.
	 */
	ENGINE_API EPrimaryAssetCookRule GetRule(TOptional<TTuple<FPrimaryAssetId, FPrimaryAssetId>>* OutConflictIds);

private:
	struct FAssignmentInfo
	{
		FPrimaryAssetId Id;
		int32 Priority = -1;
		bool bSet = false;
	};

	FAssignmentInfo InclusionByLevel[(int32)EPrimaryAssetProductionLevel::Count];
	FAssignmentInfo ExclusionByLevel[(int32)EPrimaryAssetProductionLevel::Count];

	friend void SerializeForLog(FCbWriter& Writer, const FPrimaryAssetRules& Value);
};
