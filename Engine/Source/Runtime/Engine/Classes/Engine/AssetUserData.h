// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetUserData.generated.h"

/**
 * Object that can be subclassed to store custom data on Unreal asset objects.
 */
UCLASS(DefaultToInstanced, abstract, editinlinenew, MinimalAPI)
class UAssetUserData
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** used for debugging UAssetUserData data in editor */
	virtual void Draw(class FPrimitiveDrawInterface* PDI, const class FSceneView* View) const {}

	/** used for debugging UAssetUserData data in editor */
	virtual void DrawCanvas(class FCanvas& Canvas, class FSceneView& View) const {}
	
	/** Called when the owner object is modified */

	UE_DEPRECATED(5.6, "PostEditChangeOwner has been deprecated, please override signature with FPropertyChangedEvent parameter instead" )
	virtual void PostEditChangeOwner() {}

	virtual void PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent) { PRAGMA_DISABLE_DEPRECATION_WARNINGS PostEditChangeOwner(); PRAGMA_ENABLE_DEPRECATION_WARNINGS }
	
	/** Merge this Asset User Data with the given one. Used when merging Skeletal Meshes.
	* 
	* @return true if the merge has completed successfully. */
	virtual bool Merge(const TNotNull<UAssetUserData*> Other)
	{
		return false;
	}
};
