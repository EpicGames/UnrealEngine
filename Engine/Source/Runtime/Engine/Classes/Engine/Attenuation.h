// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Engine/EngineTypes.h"
#include "Curves/CurveFloat.h"
#include "Attenuation.generated.h"

struct FGlobalFocusSettings;

UENUM(BlueprintType)
enum class EAttenuationDistanceModel : uint8
{
	Linear,
	Logarithmic,
	Inverse,
	LogReverse,
	NaturalSound,
	Custom,
};

UENUM(BlueprintType)
namespace EAttenuationShape
{
	enum Type : int
	{
		Sphere,
		Capsule,
		Box,
		Cone
	};
}

UENUM(BlueprintType)
enum class ENaturalSoundFalloffMode : uint8
{
	// (Default) Continues attenuating pass falloff max using volume value
	// specified at the max falloff distance's bounds
	Continues,

	// Sound goes silent upon leaving the shape
	Silent,

	// Holds the volume value specified at the shapes falloff bounds
	Hold,
};

/*
* Base class for attenuation settings.
*/
USTRUCT(BlueprintType)
struct FBaseAttenuationSettings
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API virtual ~FBaseAttenuationSettings();

	/* The type of attenuation as a function of distance to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance, meta = (DisplayName = "Attenuation Function"))
	EAttenuationDistanceModel DistanceAlgorithm;

	/* The shape of the non-custom attenuation method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance)
	TEnumAsByte<enum EAttenuationShape::Type> AttenuationShape;

	// Whether to continue attenuating, go silent, or hold last volume value when beyond falloff bounds and 
	// 'Attenuation At Max (dB)' is set to a value greater than -60dB.
	// (Only for 'Natural Sound' Distance Algorithm). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance)
	ENaturalSoundFalloffMode FalloffMode;

	/* The attenuation volume at the falloff distance in decibels (Only for 'Natural Sound' Distance Algorithm). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance, meta=(DisplayName = "Attenuation At Max (dB)", ClampMin = "-60", ClampMax = "0"))
	float dBAttenuationAtMax;

	/* The dimensions to use for the attenuation shape. Interpretation of the values differ per shape.
	   Sphere  - X is Sphere Radius. Y and Z are unused
	   Capsule - X is Capsule Half Height, Y is Capsule Radius, Z is unused
	   Box     - X, Y, and Z are the Box's dimensions
	   Cone    - X is Cone Radius, Y is Cone Angle, Z is Cone Falloff Angle
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance)
	FVector AttenuationShapeExtents;

	/* The distance back from the sound's origin to begin the cone when using the cone attenuation shape. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance, meta=(ClampMin = "0"))
	float ConeOffset;

	/* The distance over which volume attenuation occurs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance, meta=(ClampMin = "0"))
	float FalloffDistance;

	/* An optional attenuation radius (sphere) that extends from the cone origin. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationDistance, meta = (ClampMin = "0"))
	float ConeSphereRadius;

	/* The distance over which volume attenuation occurs for the optional sphere shape. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationDistance, meta = (ClampMin = "0"))
	float ConeSphereFalloffDistance;

	/* The custom volume attenuation curve to use. Value clamped at 1.0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= AttenuationDistance)
	FRuntimeFloatCurve CustomAttenuationCurve;

	ENGINE_API FBaseAttenuationSettings();

	struct AttenuationShapeDetails
	{
		FVector Extents;
		float Falloff;
		float ConeOffset;
		float ConeSphereRadius;
		float ConeSphereFalloff;
	};

	ENGINE_API virtual void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, AttenuationShapeDetails>& ShapeDetailsMap) const;
	ENGINE_API float GetMaxDimension() const;

	ENGINE_API float GetMaxFalloffDistance() const;

	ENGINE_API float Evaluate(const FTransform& Origin, FVector Location, float DistanceScale = 1.f) const;

	ENGINE_API float AttenuationEval(float Distance, float Falloff, float DistanceScale = 1.f) const;
	ENGINE_API float AttenuationEvalBox(const FTransform& Origin, FVector Location, float DistanceScale = 1.f) const;
	ENGINE_API float AttenuationEvalCapsule(const FTransform& Origin, FVector Location, float DistanceScale = 1.f) const;
	ENGINE_API float AttenuationEvalCone(const FTransform& Origin, FVector Location, float DistanceScale = 1.f) const;
};
