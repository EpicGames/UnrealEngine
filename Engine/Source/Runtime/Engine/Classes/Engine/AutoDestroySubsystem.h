// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Stats/Stats.h"
#include "Engine/EngineTypes.h"

#include "AutoDestroySubsystem.generated.h"

class AActor;
struct FLatentActionManager;

/**
* The Auto destroy subsystem manages actors who have bAutoDestroyWhenFinished
* set as true. This ensures that even actors who do not have Tick enabled 
* get properly destroyed, as well as decouple this behavior from AActor::Tick
*/
UCLASS()
class UAutoDestroySubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:

	/**
	* Registers the given actor if it has the bAutoDestroyWhenFinished flag set
	*
	* @return True if actor registered
	*/
	bool RegisterActor(AActor* ActorToRegister);

	/**
	* Remove this actor from the array of actors to poll with this subsystem
	* 
	* @return	True if this actor was removed
	*/
	bool UnregisterActor(AActor* ActorToRemove);

protected:

	//~FTickableGameObject interface

	ETickableTickType GetTickableTickType() const override;

	/** 
	* Update any Actors that have bAutoDestroyWhenFinished set to true. 
	* If the actor should be destroyed, then destroy it and it's components
	*/
	void Tick(float DeltaTime) override;

	TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UAutoDestroySubsystem, STATGROUP_Tickables); }

	//~End of FTickableGameObject interface


	//~USubsystem interface
	void Deinitialize() override;
	//~End of USubsystem interface

private:
	
	/** Callback for a registered actor's End Play so we can remove it from our known actors */
	UFUNCTION()
	void OnActorEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason);

	/** 
	* Returns true if there are no latent actions on the given actor or the actor cannot have latent actions
	*/
	static bool CheckLatentActionsOnActor(FLatentActionManager& LatentActionManager, AActor* ActorToCheck, float WorldDeltaTime);

	/** 
	* Check each component on the given actor and if they are all ready for auto destroy, return true
	*/
	static bool ActorComponentsAreReadyForDestroy(AActor* const ActorToCheck);

	/** Actors to check if they should auto destroy or not */
	UPROPERTY()
	TArray<TObjectPtr<AActor>> ActorsToPoll;
};