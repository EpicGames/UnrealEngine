// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "BlendableInterface.generated.h"

class UMaterialInstanceDynamic;
class UMaterialInterface;

/** Where to place a post process material in the post processing chain. */
UENUM()
enum EBlendableLocation : int
{
	/** Post process material location to modify the scene color, between translucency distortion and DOF.
	 * Always run at rendering resolution.
	 * Inputs and output always in linear color space.
	 *
	 * Input0:former pass scene color, excluding AfterDOF translucency
	 * Input1:AfterDOF translucency.
	 */
	BL_SceneColorBeforeDOF = 2 UMETA(DisplayName="Scene Color Before DOF"),

	/** Post process material location to modify the scene color, between DOF and AfterDOF translucency.
	 * Always run at rendering resolution.
	 * Inputs and output always in linear color space.
	 *
	 * Input0:former pass scene color, excluding AfterDOF translucency
	 * Input1:AfterDOF translucency.
	 */
	BL_SceneColorAfterDOF = 1 UMETA(DisplayName = "Scene Color After DOF"),

	/** Post process material location to modify the AfterDOF translucency, before composition into the scene color.
	 * Always run at rendering resolution.
	 * Inputs and output always in linear color space.
	 *
	 * Input0:scene color without translucency, after DOF,
	 * Input1:AfterDOF translucency.
	 */
	BL_TranslucencyAfterDOF = 5 UMETA(DisplayName = "Translucency After DOF"),

	/** Post process material location to compose a backplate into SSR, between TSR/TAA and next frame's SSR.
	 * Runs at display resolution with TSR or TAAU, rendering resolution otherwise.
	 * Inputs and output always in linear color space.
	 *
	 * Input0:TAA/TSR output,
	 */
	BL_SSRInput = 4 UMETA(DisplayName = "SSR Input"),

	/** Post process material location to modify the scene color, before bloom.
	 * Runs at display resolution with TSR or TAAU, rendering resolution otherwise.
	 * Inputs and output always in linear color space.
	 *
	 * Input0:former pass scene color,
	 */
	BL_SceneColorBeforeBloom = 6 UMETA(DisplayName = "Scene Color Before Bloom"),

	/** Post process material replacing the tone mapper, to modify the scene color.
	 * Runs at display resolution with TSR or TAAU, rendering resolution otherwise.
	 * Inputs are always linear color space.
	 *
	 * Input0:former pass scene color,
	 * Input1:AfterDOF translucency.
	 */
	BL_ReplacingTonemapper = 3 UMETA(DisplayName="Replacing the Tonemapper"),

	/** Post process material location to modify the scene color, after tone mapper.
	 * Runs at display resolution with TSR or TAAU, rendering resolution otherwise.
	 * Inputs and output in different color spaces, based rendering settings for instance (sRGB/Rec709, HDR or even Linear Color).
	 *
	 * Input0:former pass scene color,
	 * Input1:AfterDOF translucency.
	 */
	BL_SceneColorAfterTonemapping = 0 UMETA(DisplayName = "Scene Color After Tonemapping"),

	BL_MAX = 7 UMETA(Hidden),

	// Olds names that needs to be kept forever to ensure asset serialization to work correctly when UENUM() switched from serializing int to names.
	BL_BeforeTranslucency UE_DEPRECATED(5.4, "Renamed to BL_SceneColorBeforeDOF")        = BL_SceneColorBeforeDOF UMETA(Hidden),
	BL_BeforeTonemapping  UE_DEPRECATED(5.4, "Renamed to BL_SceneColorAfterDOF")         = BL_SceneColorAfterDOF UMETA(Hidden),
	BL_AfterTonemapping   UE_DEPRECATED(5.4, "Renamed to BL_SceneColorAfterTonemapping") = BL_SceneColorAfterTonemapping UMETA(Hidden),
};

/** Dummy class needed to support Cast<IBlendableInterface>(Object). */
UINTERFACE(MinimalAPI)
class UBlendableInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Derive from this class if you want to be blended by the PostProcess blending e.g. PostproceessVolume
 */
class IBlendableInterface
{
	GENERATED_IINTERFACE_BODY()
		
	/** @param Weight 0..1, excluding 0, 1=fully take the values from this object, crash if outside the valid range. */
	virtual void OverrideBlendableSettings(class FSceneView& View, float Weight) const = 0;
};

struct FPostProcessMaterialNode
{
	// Default constructor
	FPostProcessMaterialNode()
		: MaterialInterface(0), bIsMID(false), Location(BL_MAX), Priority(0)
	{
		check(!IsValid());
	}

	// Constructor
	FPostProcessMaterialNode(UMaterialInterface* InMaterialInterface, EBlendableLocation InLocation, int32 InPriority, bool InbIsBlendable)
		: MaterialInterface(InMaterialInterface), bIsMID(false), Location(InLocation), Priority(InPriority), bIsBlendable(InbIsBlendable)
	{
		check(IsValid());
	}

	// Constructor
	FPostProcessMaterialNode(UMaterialInstanceDynamic* InMID, EBlendableLocation InLocation, int32 InPriority, bool InbIsBlendable)
		: MaterialInterface((UMaterialInterface*)InMID), bIsMID(true), Location(InLocation), Priority(InPriority), bIsBlendable(InbIsBlendable)
	{
		check(IsValid());
	}

	UMaterialInterface* GetMaterialInterface() const { return MaterialInterface; }
	/** Only call if you are sure it's an MID. */
	UMaterialInstanceDynamic* GetMID() const { check(bIsMID); return (UMaterialInstanceDynamic*)MaterialInterface; }

	/** For type safety in FBlendableManager. */
	static const FName& GetFName()
	{
		static const FName Name = FName("FPostProcessMaterialNode");

		return Name;
	}

	struct FCompare
	{
		inline bool operator()(const FPostProcessMaterialNode& P1, const FPostProcessMaterialNode& P2) const
		{
			if(P1.Location < P2.Location) return true;
			if(P1.Location > P2.Location) return false;

			if(P1.Priority < P2.Priority) return true;
			if(P1.Priority > P2.Priority) return false;

			return false;
		}
	};

	EBlendableLocation GetLocation() const { return Location; }
	int32 GetPriority() const { return Priority; }
	bool GetIsBlendable() const { return bIsBlendable; }

	bool IsValid() const { return MaterialInterface != 0; }

private:
	UMaterialInterface* MaterialInterface;

	/** if MaterialInterface is an MID, needed for GetMID(). */
	bool bIsMID;

	EBlendableLocation Location;

	/** Default is 0. */
	int32 Priority;

	/** Flag for whether the material should be blendable */
	bool bIsBlendable;
};
