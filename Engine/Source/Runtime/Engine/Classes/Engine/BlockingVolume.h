// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "BlockingVolume.generated.h"

/** An invisible volume used to block other actors. */
UCLASS(MinimalAPI)
class ABlockingVolume
	: public AVolume
{
	GENERATED_UCLASS_BODY()

public:

	// UObject interface.

#if WITH_EDITOR
	virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR	
};
