// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/BlueprintCore.h"
#include "Blueprint/BlueprintPropertyGuidProvider.h"
#include "UObject/SoftObjectPath.h"
#include "Blueprint/BlueprintSupport.h"

#if WITH_EDITOR
#include "EngineLogs.h"
#endif

#include "Blueprint.generated.h"

enum class EBlueprintCompileOptions;
class FCompilerResultsLog;
class ITargetPlatform;
class UActorComponent;
class UEdGraph;
class FKismetCompilerContext;
class UInheritableComponentHandler;
class UBlueprintExtension;
class UBlueprintGeneratedClass;
class FBlueprintActionDatabaseRegistrar;
struct FBPComponentClassOverride;
struct FDiffResults;

/**
 * Enumerates states a blueprint can be in.
 */
UENUM()
enum EBlueprintStatus : int
{
	/** Blueprint is in an unknown state. */
	BS_Unknown,
	/** Blueprint has been modified but not recompiled. */
	BS_Dirty,
	/** Blueprint tried but failed to be compiled. */
	BS_Error,
	/** Blueprint has been compiled since it was last modified. */
	BS_UpToDate,
	/** Blueprint is in the process of being created for the first time. */
	BS_BeingCreated,
	/** Blueprint has been compiled since it was last modified. There are warnings. */
	BS_UpToDateWithWarnings,
	BS_MAX,
};


/** Enumerates types of blueprints. */
UENUM()
enum EBlueprintType : int
{
	/** Normal blueprint. */
	BPTYPE_Normal				UMETA(DisplayName="Blueprint Class"),
	/** Blueprint that is const during execution (no state graph and methods cannot modify member variables). */
	BPTYPE_Const				UMETA(DisplayName="Const Blueprint Class"),
	/** Blueprint that serves as a container for macros to be used in other blueprints. */
	BPTYPE_MacroLibrary			UMETA(DisplayName="Blueprint Macro Library"),
	/** Blueprint that serves as an interface to be implemented by other blueprints. */
	BPTYPE_Interface			UMETA(DisplayName="Blueprint Interface"),
	/** Blueprint that handles level scripting. */
	BPTYPE_LevelScript			UMETA(DisplayName="Level Blueprint"),
	/** Blueprint that serves as a container for functions to be used in other blueprints. */
	BPTYPE_FunctionLibrary		UMETA(DisplayName="Blueprint Function Library"),

	BPTYPE_MAX,
};


/** Type of compilation. */
namespace EKismetCompileType
{
	enum Type
	{
		SkeletonOnly,
		Full,
		StubAfterFailure, 
		BytecodeOnly,
		// Cpp type was removed with BP nativization
	};
};

/** Breakpoints have been moved to Engine/Source/Editor/UnrealEd/Public/Kismet2/Breakpoint.h,
*   renamed to FBlueprintBreakpoint, and are now UStructs */
UCLASS(deprecated)
class UDEPRECATED_Breakpoint : public UObject
{
	GENERATED_BODY()
};

/** Compile modes. */
UENUM()
enum class EBlueprintCompileMode : uint8
{
	Default UMETA(DisplayName="Use Default", ToolTip="Use the default setting."),
	Development UMETA(ToolTip="Always compile in development mode (even when cooking)."),
	FinalRelease UMETA(ToolTip="Always compile in final release mode.")
};

/** Cached 'cosmetic' information about a macro graph (this is transient and is computed at load) */
USTRUCT()
struct FBlueprintMacroCosmeticInfo
{
	GENERATED_BODY()

	// Does this macro contain one or more latent nodes?
	bool bContainsLatentNodes;

	FBlueprintMacroCosmeticInfo()
		: bContainsLatentNodes(false)
	{
	}
};

/** Options used for a specific invication of the blueprint compiler */
struct FKismetCompilerOptions
{
public:
	/** The compile type to perform (full compile, skeleton pass only, etc) */
	EKismetCompileType::Type	CompileType;

	/** Whether or not to save intermediate build products (temporary graphs and expanded macros) for debugging */
	bool bSaveIntermediateProducts;

	/** Whether to regenerate the skeleton first, when compiling on load we don't need to regenerate the skeleton. */
	bool bRegenerateSkelton;

	/** Whether or not this compile is for a duplicated blueprint */
	bool bIsDuplicationInstigated;

	/** Whether or not to reinstance and stub if the blueprint fails to compile */
	bool bReinstanceAndStubOnFailure;

	/** Whether or not to skip class default object validation */
	bool bSkipDefaultObjectValidation;

	/** Whether or not to update Find-in-Blueprint search metadata */
	bool bSkipFiBSearchMetaUpdate;

	/** Whether or not to use Delta Serialization when copying unrelated objects */
	bool bUseDeltaSerializationDuringReinstancing;

	/** Whether or not to skip new variable defaults detection */
	bool bSkipNewVariableDefaultsDetection;

	TSharedPtr<FString> OutHeaderSourceCode;
	TSharedPtr<FString> OutCppSourceCode;

	bool DoesRequireBytecodeGeneration() const
	{
		return (CompileType == EKismetCompileType::Full) 
			|| (CompileType == EKismetCompileType::BytecodeOnly);
	}

	FKismetCompilerOptions()
		: CompileType(EKismetCompileType::Full)
		, bSaveIntermediateProducts(false)
		, bRegenerateSkelton(true)
		, bIsDuplicationInstigated(false)
		, bReinstanceAndStubOnFailure(true)
		, bSkipDefaultObjectValidation(false)
		, bSkipFiBSearchMetaUpdate(false)
	{
	};
};

/** One metadata entry for a variable */
USTRUCT()
struct FBPVariableMetaDataEntry
{
	GENERATED_USTRUCT_BODY()

	/** Name of metadata key */
	UPROPERTY(EditAnywhere, Category=BPVariableMetaDataEntry)
	FName DataKey;

	/** Name of metadata value */
	UPROPERTY(EditAnywhere, Category=BPVariableMetaDataEntry)
	FString DataValue;

	FBPVariableMetaDataEntry() {}
	FBPVariableMetaDataEntry(const FName InKey, FString InValue)
		: DataKey(InKey)
		, DataValue(MoveTemp(InValue))
	{}
};


/** Struct indicating a variable in the generated class */
USTRUCT()
struct FBPVariableDescription
{
	GENERATED_USTRUCT_BODY()

	/** Name of the variable */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	FName VarName;

	/** A Guid that will remain constant even if the VarName changes */
	UPROPERTY()
	FGuid VarGuid;

	/** Type of the variable */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	struct FEdGraphPinType VarType;

	/** Friendly name of the variable */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	FString FriendlyName;

	/** Category this variable should be in */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	FText Category;

	/** Property flags for this variable - Changed from int32 to uint64*/
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	uint64 PropertyFlags;

	UPROPERTY(EditAnywhere, Category=BPVariableRepNotify)
	FName RepNotifyFunc;

	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	TEnumAsByte<ELifetimeCondition> ReplicationCondition;

	/** Metadata information for this variable */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	TArray<struct FBPVariableMetaDataEntry> MetaDataArray;

	/** Optional new default value stored as string*/
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	FString DefaultValue;

	ENGINE_API FBPVariableDescription();

	/** Set a metadata value on the variable */
	ENGINE_API void SetMetaData(FName Key, FString Value);
	/** Gets a metadata value on the variable; asserts if the value isn't present.  Check for validiy using FindMetaDataEntryIndexForKey. */
	ENGINE_API const FString& GetMetaData(FName Key) const;
	/** Clear metadata value on the variable */
	ENGINE_API void RemoveMetaData(FName Key);
	/** Find the index in the array of a metadata entry */
	ENGINE_API int32 FindMetaDataEntryIndexForKey(FName Key) const;
	/** Checks if there is metadata for a key */
	ENGINE_API bool HasMetaData(FName Key) const;
	
};


/** Struct containing information about what interfaces are implemented in this blueprint */
USTRUCT()
struct FBPInterfaceDescription
{
	GENERATED_USTRUCT_BODY()

	/** Reference to the interface class we're adding to this blueprint */
	UPROPERTY()
	TSubclassOf<class UInterface>  Interface;

	/** References to the graphs associated with the required functions for this interface */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> Graphs;


	FBPInterfaceDescription()
		: Interface(nullptr)
	{ }
};


USTRUCT()
struct FEditedDocumentInfo
{
	GENERATED_USTRUCT_BODY()

	/** Edited object */
	UPROPERTY()
	FSoftObjectPath EditedObjectPath;

	/** Saved view position */
	UPROPERTY()
	FDeprecateSlateVector2D SavedViewOffset;

	/** Saved zoom amount */
	UPROPERTY()
	float SavedZoomAmount;

	FEditedDocumentInfo()
		: SavedViewOffset(0.0f, 0.0f)
		, SavedZoomAmount(-1.0f)
		, EditedObject_DEPRECATED(nullptr)
	{ }

	FEditedDocumentInfo(UObject* InEditedObject)
		: EditedObjectPath(InEditedObject)
		, SavedViewOffset(0.0f, 0.0f)
		, SavedZoomAmount(-1.0f)
		, EditedObject_DEPRECATED(nullptr)
	{ }

	FEditedDocumentInfo(UObject* InEditedObject, const UE::Slate::FDeprecateVector2DParameter& InSavedViewOffset, float InSavedZoomAmount)
		: EditedObjectPath(InEditedObject)
		, SavedViewOffset(InSavedViewOffset)
		, SavedZoomAmount(InSavedZoomAmount)
		, EditedObject_DEPRECATED(nullptr)
	{ }

	void PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading() && EditedObject_DEPRECATED)
		{
			// Convert hard to soft reference.
			EditedObjectPath = EditedObject_DEPRECATED;
			EditedObject_DEPRECATED = nullptr;
		}
	}

	friend bool operator==(const FEditedDocumentInfo& LHS, const FEditedDocumentInfo& RHS)
	{
		return LHS.EditedObjectPath == RHS.EditedObjectPath && LHS.SavedViewOffset == RHS.SavedViewOffset && LHS.SavedZoomAmount == RHS.SavedZoomAmount;
	}

private:
	// Legacy hard reference is now serialized as a soft reference (see above).
	UPROPERTY()
	TObjectPtr<UObject> EditedObject_DEPRECATED;
};

template<>
struct TStructOpsTypeTraits<FEditedDocumentInfo> : public TStructOpsTypeTraitsBase2<FEditedDocumentInfo>
{
	enum
	{
		WithPostSerialize = true
	};
};

/** Bookmark node info */
USTRUCT()
struct FBPEditorBookmarkNode
{
	GENERATED_USTRUCT_BODY()

	/** Node ID */
	UPROPERTY()
	FGuid NodeGuid;

	/** Parent ID */
	UPROPERTY()
	FGuid ParentGuid;

	/** Display name */
	UPROPERTY()
	FText DisplayName;

	friend bool operator==(const FBPEditorBookmarkNode& LHS, const FBPEditorBookmarkNode& RHS)
	{
		return LHS.NodeGuid == RHS.NodeGuid;
	}
};

UENUM()
enum class EShouldCookBlueprintPropertyGuids
{
	/** Don't cook the property GUIDs for this Blueprint */
	No,
	/** Cook the property GUIDs for this Blueprint (see UCookerSettings::BlueprintPropertyGuidsCookingMethod) */
	Yes,
	/** Inherit whether to cook the property GUIDs for this Blueprint from the parent Blueprint (behaves like 'No' if there is no parent Blueprint) */
	Inherit,
};

#if WITH_EDITOR
/** Control flags for current object/world accessor methods */
enum class EGetObjectOrWorldBeingDebuggedFlags
{
	/** Use normal weak ptr semantics when accessing the referenced object. */
	None = 0,
	/** Return a valid ptr even if the PendingKill flag is set on the referenced object. */
	IgnorePendingKill = 1 << 0
};

ENUM_CLASS_FLAGS(EGetObjectOrWorldBeingDebuggedFlags);
#endif


/**
 * Blueprints are special assets that provide an intuitive, node-based interface that can be used to create new types of Actors
 * and script level events; giving designers and gameplay programmers the tools to quickly create and iterate gameplay from
 * within Unreal Editor without ever needing to write a line of code.
 */
UCLASS(config=Engine, MinimalAPI)
class UBlueprint : public UBlueprintCore, public IBlueprintPropertyGuidProvider
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Pointer to the parent class that the generated class should derive from. This *can* be null under rare circumstances, 
	 * one such case can be created by creating a blueprint (A) based on another blueprint (B), shutting down the editor, and
	 * deleting the parent blueprint. Exported as Alphabetical in GetAssetRegistryTags
	 */
	UPROPERTY(meta=(NoResetToDefault))
	TSubclassOf<UObject> ParentClass;

	/** The type of this blueprint */
	UPROPERTY(AssetRegistrySearchable)
	TEnumAsByte<enum EBlueprintType> BlueprintType;

	/** Whether or not this blueprint should recompile itself on load */
	UPROPERTY(config)
	uint8 bRecompileOnLoad:1;

	/** When the class generated by this blueprint is loaded, it will be recompiled the first time.  After that initial recompile, subsequent loads will skip the regeneration step */
	UPROPERTY(transient)
	uint8 bHasBeenRegenerated:1;

	/** State flag to indicate whether or not the Blueprint is currently being regenerated on load */
	UPROPERTY(transient)
	uint8 bIsRegeneratingOnLoad:1;

#if WITH_EDITORONLY_DATA

	/** The blueprint is currently compiled */
	UPROPERTY(transient)
	uint8 bBeingCompiled:1;

	/** Whether or not this blueprint is newly created, and hasn't been opened in an editor yet */
	UPROPERTY(transient)
	uint8 bIsNewlyCreated : 1;

	/** Whether to force opening the full (non data-only) editor for this blueprint. */
	UPROPERTY(transient)
	uint8 bForceFullEditor : 1;

	UPROPERTY(transient)
	uint8 bQueuedForCompilation : 1 ;

	/** If set to true, prevent MarkBlueprintAsStructurallyModified to do its job. Use that only when you will call MarkBlueprintAsStructurallyModified or equivalent later. */
	UPROPERTY(transient)
	uint8 bSuppressStructurallyModified :1 = false;

	/**whether or not you want to continuously rerun the construction script for an actor as you drag it in the editor, or only when the drag operation is complete*/
	UPROPERTY(EditAnywhere, Category=BlueprintOptions)
	uint8 bRunConstructionScriptOnDrag : 1;

	/**whether or not you want to continuously rerun the construction script for an actor in sequencer*/
	UPROPERTY(EditAnywhere, Category=BlueprintOptions)
	uint8 bRunConstructionScriptInSequencer : 1;

	/** Whether or not this blueprint's class is a const class or not.  Should set CLASS_Const in the KismetCompiler. */
	UPROPERTY(EditAnywhere, Category=ClassOptions, AdvancedDisplay)
	uint8 bGenerateConstClass : 1;

	/** Whether or not this blueprint's class is a abstract class or not.  Should set CLASS_Abstract in the KismetCompiler. */
	UPROPERTY(EditAnywhere, Category = ClassOptions, AdvancedDisplay)
	uint8 bGenerateAbstractClass : 1;

	/** TRUE to show a warning when attempting to start in PIE and there is a compiler error on this Blueprint */
	UPROPERTY(transient)
	uint8 bDisplayCompilePIEWarning:1;

	/** Deprecates the Blueprint, marking the generated class with the CLASS_Deprecated flag */
	UPROPERTY(EditAnywhere, Category=ClassOptions, AdvancedDisplay)
	uint8 bDeprecate:1;

	/** 
	 * Flag indicating that a read only duplicate of this blueprint is being created, used to disable logic in ::PostDuplicate,
	 *
	 * This flag needs to be copied on duplication (because it's the duplicated object that we're disabling on PostDuplicate),
	 * but we don't *need* to serialize it for permanent objects.
	 *
	 * Without setting this flag a blueprint will be marked dirty when it is duplicated and if saved while in this dirty
	 * state you will not be able to open the blueprint. More specifically, UClass::Rename (called by DestroyGeneratedClass)
	 * sets a dirty flag on the package. Once saved the package will fail to open because some unnamed objects are present in
	 * the pacakge.
	 *
	 * This flag can be used to avoid the package being marked as dirty in the first place. Ideally PostDuplicateObject
	 * would not rename classes that are still in use by the original object.
	 */
	UPROPERTY()
	mutable uint8 bDuplicatingReadOnly:1;

	/**
	 * Whether to include the property GUIDs for the generated class in a cooked build.
	 * @note This option may slightly increase memory usage in a cooked build, but can avoid needing to add CoreRedirect data for Blueprint classes stored within SaveGame archives.
	 */
	UPROPERTY(EditAnywhere, Category=ClassOptions, AdvancedDisplay, meta=(DisplayName="Should Cook Property Guids?"))
	EShouldCookBlueprintPropertyGuids ShouldCookPropertyGuidsValue = EShouldCookBlueprintPropertyGuids::Inherit;

	ENGINE_API bool ShouldCookPropertyGuids() const;

public:
	/** The mode that will be used when compiling this class. */
	UPROPERTY(EditAnywhere, Category=ClassOptions, AdvancedDisplay)
	EBlueprintCompileMode CompileMode;

	/** The current status of this blueprint */
	UPROPERTY(transient, Category=BlueprintOptions, BlueprintReadOnly)
	TEnumAsByte<enum EBlueprintStatus> Status;

	/** Overrides the BP's display name in the editor UI */
	UPROPERTY(EditAnywhere, Category=BlueprintOptions, DuplicateTransient)
	FString BlueprintDisplayName;

	/** Shows up in the content browser tooltip when the blueprint is hovered */
	UPROPERTY(EditAnywhere, Category=BlueprintOptions, meta=(MultiLine=true), DuplicateTransient)
	FString BlueprintDescription;

	/** The namespace of this blueprint (if set, the Blueprint will be treated differently for the context menu) */
	UPROPERTY(EditAnywhere, Category = BlueprintOptions, AssetRegistrySearchable)
	FString BlueprintNamespace;

	/** The category of the Blueprint, used to organize this Blueprint class when displayed in palette windows */
	UPROPERTY(EditAnywhere, Category=BlueprintOptions)
	FString BlueprintCategory;

	/** Additional HideCategories. These are added to HideCategories from parent. */
	UPROPERTY(EditAnywhere, Category=BlueprintOptions)
	TArray<FString> HideCategories;

#endif //WITH_EDITORONLY_DATA

	/** The version of the blueprint system that was used to  create this blueprint */
	UPROPERTY()
	int32 BlueprintSystemVersion;

	/** 'Simple' construction script - graph of components to instance */
	UPROPERTY()
	TObjectPtr<class USimpleConstructionScript> SimpleConstructionScript;

#if WITH_EDITORONLY_DATA
	/** Set of pages that combine into a single uber-graph */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> UbergraphPages;

	/** Set of functions implemented for this class graphically */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> FunctionGraphs;

	/** Graphs of signatures for delegates */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> DelegateSignatureGraphs;

	/** Set of macros implemented for this class */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> MacroGraphs;

	/** Set of functions actually compiled for this class */
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<UEdGraph>> IntermediateGeneratedGraphs;

	/** Set of functions actually compiled for this class */
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<UEdGraph>> EventGraphs;

	/** Cached cosmetic information about macro graphs, use GetCosmeticInfoForMacro() to access */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UEdGraph>, FBlueprintMacroCosmeticInfo> PRIVATE_CachedMacroInfo;
#endif // WITH_EDITORONLY_DATA

	/** Array of component template objects, used by AddComponent function */
	UPROPERTY()
	TArray<TObjectPtr<class UActorComponent>> ComponentTemplates;

	/** Array of templates for timelines that should be created */
	UPROPERTY()
	TArray<TObjectPtr<class UTimelineTemplate>> Timelines;

	/** Array of blueprint overrides of component classes in parent classes */
	UPROPERTY()
	TArray<FBPComponentClassOverride> ComponentClassOverrides;

	/** Stores data to override (in children classes) components (created by SCS) from parent classes */
	UPROPERTY()
	TObjectPtr<class UInheritableComponentHandler> InheritableComponentHandler;

#if WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExtensionAdded, UBlueprintExtension*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExtensionRemoved, UBlueprintExtension*);

	/** Array of new variables to be added to generated class */
	UPROPERTY()
	TArray<FBPVariableDescription> NewVariables;

	/** Array of generated variables to be added to generated class */
	UPROPERTY()
	TArray<FBPVariableDescription> GeneratedVariables;

	/** Array of user sorted categories */
	UPROPERTY()
	TArray<FName> CategorySorting;

	/** Namespaces imported by this blueprint */
	UPROPERTY(AssetRegistrySearchable)
	TSet<FString> ImportedNamespaces;

	/** Array of info about the interfaces we implement in this blueprint */
	UPROPERTY(AssetRegistrySearchable)
	TArray<struct FBPInterfaceDescription> ImplementedInterfaces;
	
	/** Set of documents that were being edited in this blueprint, so we can open them right away */
	UPROPERTY()
	TArray<struct FEditedDocumentInfo> LastEditedDocuments;

	/** Bookmark data */
	UPROPERTY()
	TMap<FGuid, struct FEditedDocumentInfo> Bookmarks;

	/** Bookmark nodes (for display) */
	UPROPERTY()
	TArray<FBPEditorBookmarkNode> BookmarkNodes;

	// moved to FPerBlueprintSettings
	UPROPERTY()
	TArray<TObjectPtr<class UDEPRECATED_Breakpoint>> Breakpoints_DEPRECATED;

	// moved to FPerBlueprintSettings
	UPROPERTY()
	TArray<FEdGraphPinReference> WatchedPins_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<class UEdGraphPin_Deprecated>> DeprecatedPinWatches;

	/** Index map for component template names */
	UPROPERTY()
	TMap<FName, int32> ComponentTemplateNameIndex;

	/** Maps old to new component template names */
	UPROPERTY(transient)
	TMap<FName, FName> OldToNewComponentTemplateNames;

private:
	/** Array of extensions for this blueprint. To access use: UBlueprint::GetExtensions / UBlueprint::AddExtension / UBlueprint::RemoveExtension[At] */
	UPROPERTY()
	TArray<TObjectPtr<UBlueprintExtension>> Extensions;

public:
	/** Fires whenever BP extension added */
	FOnExtensionAdded OnExtensionAdded;

	/** Fires whenever BP extension removed */
	FOnExtensionRemoved OnExtensionRemoved;
#endif // WITH_EDITORONLY_DATA

public:

#if WITH_EDITOR
	/** Broadcasts a notification whenever the blueprint has changed. */
	DECLARE_EVENT_OneParam( UBlueprint, FChangedEvent, class UBlueprint* );
	FChangedEvent& OnChanged() { return ChangedEvent; }

	/**	This should NOT be public */
	ENGINE_API void BroadcastChanged();

	/** Broadcasts a notification whenever the blueprint has changed. */
	DECLARE_EVENT_OneParam(UBlueprint, FCompiledEvent, class UBlueprint*);
	FCompiledEvent& OnCompiled() { return CompiledEvent; }
	void BroadcastCompiled() { CompiledEvent.Broadcast(this); }

	/** Gives const access to extensions. */
	ENGINE_API TArrayView<const TObjectPtr<UBlueprintExtension>> GetExtensions() const;

	/** Adds given extension, broadcasting on add. */
	ENGINE_API int32 AddExtension(const TObjectPtr<UBlueprintExtension>& InExtension);

	/** Removes given extension, broadcasting on remove. */
	ENGINE_API int32 RemoveExtension(const TObjectPtr<UBlueprintExtension>& InExtension);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Removes all extensions matching the predicate, broadcasting on remove if one exists. */
	template <class PREDICATE_CLASS>
	int32 RemoveAllExtension(const PREDICATE_CLASS& Predicate)
	{
		auto BroadcastRemovePredicate = [&Predicate, this](UBlueprintExtension* InExtension)
		{
			bool NotMatch = !::Invoke(Predicate, InExtension); // use a ! to guarantee it can't be anything other than zero or one

			if (!NotMatch)
			{
				OnExtensionRemoved.Broadcast(InExtension);
			}

			return !NotMatch;
		};

		return Extensions.RemoveAll(BroadcastRemovePredicate);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITOR

	/** Whether or not this blueprint can be considered for a bytecode only compile */
	virtual bool IsValidForBytecodeOnlyRecompile() const { return true; }
	
	virtual bool ShouldAutomaticallyRegisterInputOnConstruction() const { return false; }

#if WITH_EDITORONLY_DATA
	/** Delegate called when the debug object is set */
	DECLARE_EVENT_OneParam(UBlueprint, FOnSetObjectBeingDebugged, UObject* /*InDebugObj*/);
	FOnSetObjectBeingDebugged& OnSetObjectBeingDebugged() { return OnSetObjectBeingDebuggedDelegate; }

protected:
	/** Current object being debugged for this blueprint */
	TWeakObjectPtr< UObject > CurrentObjectBeingDebugged;

	/** Raw path of object to be debugged, this might have been spawned inside a specific PIE level so is not stored as an object path type */
	FString ObjectPathToDebug;

	/** Current world being debugged for this blueprint */
	TWeakObjectPtr< class UWorld > CurrentWorldBeingDebugged;

	/** Delegate called when the debug object is set */
	FOnSetObjectBeingDebugged OnSetObjectBeingDebuggedDelegate;

public:

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category=Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	/** CRC for CDO calculated right after the latest compilation used by Reinstancer to check if default values were changed */
	UPROPERTY(transient, duplicatetransient)
	uint32 CrcLastCompiledCDO;

	UPROPERTY(transient, duplicatetransient)
	uint32 CrcLastCompiledSignature;

	/**
	 * Transient flag that indicates whether or not the internal dependency
	 * cache needs to be updated. This is not carried forward by duplication so
	 * that the post-duplicate compile path is forced to reinitialize the cache.
	 */
	UPROPERTY(transient, duplicatetransient)
	bool bCachedDependenciesUpToDate;

	/**
	 * Set of blueprints that we reference - i.e. blueprints that we have
	 * some kind of reference to (variable of that blueprints type or function
	 * call
	 *
	 * We need this to be serializable so that its references can be collected.
	 *
	 * This is intentionally marked 'duplicatetransient' so that it won't carry
	 * over to a duplicated Blueprint object. The post-duplicate compile path
	 * will instead regenerate this set to be relative to the duplicated object.
	 */
	UPROPERTY(transient, duplicatetransient)
	TSet<TWeakObjectPtr<UBlueprint>> CachedDependencies;

	/**
	 * Transient cache of dependent blueprints - i.e. blueprints that call
	 * functions declared in this blueprint. Used to speed up compilation checks
	 *
	 * This is intentionally marked 'duplicatetransient' so that it won't carry
	 * over to a duplicated Blueprint object. However, the post-duplicate compile
	 * path will not regenerate this set, since it is populated by each dependent
	 * Blueprint's compile. In this case, a duplicated Blueprint equates to a
	 * new Blueprint, so it won't initially have any dependents to include here.
	 */
	UPROPERTY(transient, duplicatetransient)
	TSet<TWeakObjectPtr<UBlueprint>> CachedDependents;

	/**
	 * User Defined Structures the blueprint depends on
	 *
	 * This is intentionally marked 'duplicatetransient' so that it won't carry
	 * over to a duplicated Blueprint object. The post-duplicate compile path
	 * will instead regenerate this set to be relative to the duplicated object.
	 */
	UPROPERTY(transient, duplicatetransient)
	TSet<TWeakObjectPtr<UStruct>> CachedUDSDependencies;

	// If this BP is just a duplicate created for a specific compilation, the reference to original GeneratedClass is needed
	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<UClass> OriginalClass;

	bool IsUpToDate() const
	{
		return BS_UpToDate == Status || BS_UpToDateWithWarnings == Status;
	}

	bool IsPossiblyDirty() const
	{
		return (BS_Dirty == Status) || (BS_Unknown == Status);
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual bool RequiresForceLoadMembers(UObject* InObject) const { return true; }

	static ENGINE_API bool ForceLoad(UObject* Obj);

	static ENGINE_API void ForceLoadMembers(UObject* InObject);

	static ENGINE_API void ForceLoadMembers(UObject* InObject, const UBlueprint* InBlueprint);

	static ENGINE_API bool ValidateGeneratedClass(const UClass* InClass);

	/** Find the object in the TemplateObjects array with the supplied name */
	ENGINE_API UActorComponent* FindTemplateByName(const FName& TemplateName) const;

	/** Find a timeline by name */
	ENGINE_API class UTimelineTemplate* FindTimelineTemplateByVariableName(const FName& TimelineName);	

	/** Find a timeline by name */
	ENGINE_API const class UTimelineTemplate* FindTimelineTemplateByVariableName(const FName& TimelineName) const;	

	void GetBlueprintClassNames(FName& GeneratedClassName, FName& SkeletonClassName, FName NameOverride = NAME_None) const
	{
		FName NameToUse = (NameOverride != NAME_None) ? NameOverride : GetFName();

		const FString GeneratedClassNameString = FString::Printf(TEXT("%s_C"), *NameToUse.ToString());
		GeneratedClassName = FName(*GeneratedClassNameString);

		const FString SkeletonClassNameString = FString::Printf(TEXT("SKEL_%s_C"), *NameToUse.ToString());
		SkeletonClassName = FName(*SkeletonClassNameString);
	}
	
	void GetBlueprintCDONames(FName& GeneratedClassName, FName& SkeletonClassName, FName NameOverride = NAME_None) const
	{
		FName NameToUse = (NameOverride != NAME_None) ? NameOverride : GetFName();

		const FString GeneratedClassNameString = FString::Printf(TEXT("Default__%s_C"), *NameToUse.ToString());
		GeneratedClassName = FName(*GeneratedClassNameString);

		const FString SkeletonClassNameString = FString::Printf(TEXT("Default__SKEL_%s_C"), *NameToUse.ToString());
		SkeletonClassName = FName(*SkeletonClassNameString);
	}

	/** Gets the class generated when this blueprint is compiled. */
	ENGINE_API virtual UClass* GetBlueprintClass() const;

	// Should the generic blueprint factory work for this blueprint?
	virtual bool SupportedByDefaultBlueprintFactory() const
	{
		return true;
	}

	// Compile this prefab, overriding this methods give a chance to derived classes to specify extra default compilation flags
	ENGINE_API virtual EBlueprintCompileOptions GetDefaultCompileOptions() const;

	/** Sets the current object being debugged */
	ENGINE_API virtual void SetObjectBeingDebugged(UObject* NewObject);

	/** Clears the current object being debugged because it is gone, but do not reset the saved information */
	ENGINE_API virtual void UnregisterObjectBeingDebugged();

	ENGINE_API virtual void SetWorldBeingDebugged(UWorld* NewWorld);

	ENGINE_API virtual void GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const;

	/**
	* Allows derived blueprints to require compilation on load, otherwise they may get treated as data only and not compiled on load.
	*/
	virtual bool AlwaysCompileOnLoad() const { return false; }

	/**
	 * Some Blueprints (and classes) can recompile while we are debugging a live session (play in editor).
	 * This function controls whether this can always occur.
	 * There are also editor preferences and project settings that can be used to opt-in other classes even
	 * when this returns false
	 */
	ENGINE_API virtual bool CanAlwaysRecompileWhilePlayingInEditor() const;

private:

	/** Sets the current object being debugged */
	ENGINE_API void DebuggingWorldRegistrationHelper(UObject* ObjectProvidingWorld, UObject* ValueToRegister);
	
	static void ForceLoadGraph(UObject* Object);
public:

	/** @return the current object being debugged, which can be nullptr */
	UObject* GetObjectBeingDebugged(EGetObjectOrWorldBeingDebuggedFlags InFlags = EGetObjectOrWorldBeingDebuggedFlags::None) const
	{
		const bool bEvenIfPendingKill = EnumHasAnyFlags(InFlags, EGetObjectOrWorldBeingDebuggedFlags::IgnorePendingKill);
		return CurrentObjectBeingDebugged.Get(bEvenIfPendingKill);
	}

	/** @return path to object that should be debugged next, may be inside a nonexistent world */
	const FString& GetObjectPathToDebug() const
	{
		return ObjectPathToDebug;
	}

	/** @return the current world being debugged, which can be nullptr */
	class UWorld* GetWorldBeingDebugged(EGetObjectOrWorldBeingDebuggedFlags InFlags = EGetObjectOrWorldBeingDebuggedFlags::None) const
	{
		const bool bEvenIfPendingKill = EnumHasAnyFlags(InFlags, EGetObjectOrWorldBeingDebuggedFlags::IgnorePendingKill);
		return CurrentWorldBeingDebugged.Get(bEvenIfPendingKill);
	}

	/** Renames only the generated classes. Should only be used internally or when testing for rename. */
	ENGINE_API virtual bool RenameGeneratedClasses(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None);

	//~ Begin UObject Interface (WITH_EDITOR)
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;
	ENGINE_API virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
protected:
	ENGINE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const override;
public:
	UE_DEPRECATED(5.5, "This function is no longer called. See UObject::PostLoadAssetRegistryTags for more information.")
	static ENGINE_API void PostLoadBlueprintAssetRegistryTags(const FAssetData& InAssetData, TArray<FAssetRegistryTag>& OutTagsAndValuesToUpdate);
	
	static ENGINE_API void PostLoadBlueprintAssetRegistryTags(FPostLoadAssetRegistryTagsContext& Context);
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetId() const override;
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	/** Removes any child redirectors from the root set and marks them as transient */
	ENGINE_API void RemoveChildRedirectors();

	/** Consigns the GeneratedClass and the SkeletonGeneratedClass to oblivion, and nulls their references */
	ENGINE_API void RemoveGeneratedClasses();

	/** @return the user-friendly name of the blueprint */
	ENGINE_API virtual FString GetFriendlyName() const;

	/** @return true if the blueprint supports event binding for multicast delegates */
	ENGINE_API virtual bool AllowsDynamicBinding() const;

	/** @return true if the blueprint supports event binding for input events */
	ENGINE_API virtual bool SupportsInputEvents() const;

	ENGINE_API bool ChangeOwnerOfTemplates();

	ENGINE_API UInheritableComponentHandler* GetInheritableComponentHandler(bool bCreateIfNecessary);

	/** Collect blueprints that depend on this blueprint. */
	ENGINE_API virtual void GatherDependencies(TSet<TWeakObjectPtr<UBlueprint>>& InDependencies) const;

	/** Checks all nodes in all graphs to see if they should be replaced by other nodes */
	ENGINE_API virtual void ReplaceDeprecatedNodes();

	/** Clears out any editor data regarding a blueprint class, this can be called when you want to unload a blueprint */
	ENGINE_API virtual void ClearEditorReferences();

	/** Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object. */
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	/** 
	 * Fills in a list of differences between this blueprint and another blueprint.
	 * Default blueprints are handled by SBlueprintDiff, this should be overridden for specific blueprint types.
	 *
	 * @param OtherBlueprint	Other blueprint to compare this to, should be the same type
	 * @param Results			List of diff results to fill in with type-specific differences
	 * @return					True if these blueprints were checked for specific differences, false if they are not comparable
	 */
	ENGINE_API virtual bool FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const;

	ENGINE_API void ConformNativeComponents();
#endif	//#if WITH_EDITOR

	//~ Begin IBlueprintPropertyGuidProvider interface
	ENGINE_API virtual FName FindBlueprintPropertyNameFromGuid(const FGuid& PropertyGuid) const override final;
	ENGINE_API virtual FGuid FindBlueprintPropertyGuidFromName(const FName PropertyName) const override final;
	//~ End IBlueprintPropertyGuidProvider interface

	//~ Begin UObject Interface
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#endif // WITH_EDITORONLY_DATA
#if WITH_EDITOR
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual FString GetDesc(void) override;
	ENGINE_API virtual void TagSubobjects(EObjectFlags NewFlags) override;
	ENGINE_API virtual bool NeedsLoadForClient() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
	ENGINE_API virtual bool NeedsLoadForEditorGame() const override;
	ENGINE_API virtual bool HasNonEditorOnlyReferences() const override;
	//~ End UObject Interface

#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void GeneratedClassPreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext);
	ENGINE_API virtual void GeneratedClassPostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext);

	/** Get the Blueprint object that generated the supplied class */
	static ENGINE_API UBlueprint* GetBlueprintFromClass(const UClass* InClass);

	/** 
	 * Gets an array of all blueprints used to generate this class and its parents.  0th elements is the BP used to generate InClass
	 *
	 * @param InClass				The class to get the blueprint lineage for
	 * @param OutBlueprintParents	Array with the blueprints used to generate this class and its parents.  0th = this, Nth = least derived BP-based parent
	 * @return						true if there were no status errors in any of the parent blueprints, otherwise false
	 */
	static ENGINE_API bool GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<UBlueprint*>& OutBlueprintParents);
#endif

	/**
	 * Gets an array of all BPGCs used to generate this class and its parents.  0th elements is the BPGC used to generate InClass
	 *
	 * @param InClass				The class to get the blueprint lineage for
	 * @param OutBlueprintParents	Array of BPGCs used to generate this class and its parents.  0th = this, Nth = least derived BP-based parent
	 * @return						true if there were no status errors in any of the parent blueprints, otherwise false
	 */
	static ENGINE_API bool GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<UBlueprintGeneratedClass*>& OutBlueprintParents);

private:
	/**
	 * Gets an array of all IBlueprintPropertyGuidProviders for this class and its parents.  0th elements is the IBlueprintPropertyGuidProvider for InClass
	 *
	 * @param InClass				The class to get the blueprint lineage for
	 * @param OutBlueprintParents	Array of IBlueprintPropertyGuidProviders for this class and its parents.  0th = this, Nth = least derived BP-based parent
	 * @return						true if there were no status errors in any of the parent blueprints, otherwise false
	 */
	static ENGINE_API bool GetBlueprintHierarchyFromClass(const UClass* InClass, TArray<IBlueprintPropertyGuidProvider*>& OutBlueprintParents);

public:
#if WITH_EDITOR
	/** returns true if the class hierarchy is error free */
	static ENGINE_API bool IsBlueprintHierarchyErrorFree(const UClass* InClass);
#endif

	template<class TFieldType>
	static FName GetFieldNameFromClassByGuid(const UClass* InClass, const FGuid VarGuid)
	{
		FProperty* AssertPropertyType = (TFieldType*)0;

		TArray<IBlueprintPropertyGuidProvider*> BlueprintPropertyGuidProviders;
		UBlueprint::GetBlueprintHierarchyFromClass(InClass, BlueprintPropertyGuidProviders);

		for (IBlueprintPropertyGuidProvider* BlueprintPropertyGuidProvider : BlueprintPropertyGuidProviders)
		{
			const FName FoundPropertyName = BlueprintPropertyGuidProvider->FindBlueprintPropertyNameFromGuid(VarGuid);
			if (FoundPropertyName != NAME_None)
			{
				return FoundPropertyName;
			}
		}

		return NAME_None;
	}

	template<class TFieldType>
	static bool GetGuidFromClassByFieldName(const UClass* InClass, const FName VarName, FGuid& VarGuid)
	{
		FProperty* AssertPropertyType = (TFieldType*)0;

		TArray<IBlueprintPropertyGuidProvider*> BlueprintPropertyGuidProviders;
		UBlueprint::GetBlueprintHierarchyFromClass(InClass, BlueprintPropertyGuidProviders);

		for (IBlueprintPropertyGuidProvider* BlueprintPropertyGuidProvider : BlueprintPropertyGuidProviders)
		{
			const FGuid FoundPropertyGuid = BlueprintPropertyGuidProvider->FindBlueprintPropertyGuidFromName(VarName);
			if (FoundPropertyGuid.IsValid())
			{
				VarGuid = FoundPropertyGuid;
				return true;
			}
		}

		return false;
	}
	
#if WITH_EDITOR
	static ENGINE_API FName GetFunctionNameFromClassByGuid(const UClass* InClass, const FGuid FunctionGuid);
	static ENGINE_API bool GetFunctionGuidFromClassByFieldName(const UClass* InClass, const FName FunctionName, FGuid& FunctionGuid);

	/**
	 * Gets the last edited uber graph.  If no graph was found in the last edited document set, the first
	 * ubergraph is returned.  If there are no ubergraphs nullptr is returned.
	 */
	ENGINE_API UEdGraph* GetLastEditedUberGraph() const;

	/* Notify the blueprint when a graph is renamed to allow for additional fixups. */
	virtual void NotifyGraphRenamed(class UEdGraph* Graph, FName OldName, FName NewName) { }
#endif

#if WITH_EDITOR
	static ENGINE_API UClass* GetBlueprintParentClassFromAssetTags(const FAssetData& BlueprintAsset);
#endif

	/** Find a function given its name and optionally an object property name within this Blueprint */
	ENGINE_API ETimelineSigType GetTimelineSignatureForFunctionByName(const FName& FunctionName, const FName& ObjectPropertyName);

	/** Gets the current blueprint system version. Note- incrementing this version will invalidate ALL existing blueprints! */
	static int32 GetCurrentBlueprintSystemVersion()
	{
		return 2;
	}

	/** Get all graphs in this blueprint */
	ENGINE_API void GetAllGraphs(TArray<UEdGraph*>& Graphs) const;

	/**
	* Allow each blueprint type (AnimBlueprint or ControlRigBlueprint) to add specific
	* UBlueprintNodeSpawners pertaining to the sub-class type. Serves as an
	* extensible way for new nodes, and game module nodes to add themselves to
	* context menus.
	*
	* @param  ActionRegistrar	BlueprintActionDataBaseRetistrar 
	*/
	virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const {}

	/**
	* Allow each blueprint instance to add specific 
	* UBlueprintNodeSpawners pertaining to the sub-class type. Serves as an
	* extensible way for new nodes, and game module nodes to add themselves to
	* context menus.
	*
	* @param  ActionRegistrar	BlueprintActionDataBaseRetistrar
	*/
	virtual void GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const {}

	/**
	 * Returns true if this blueprint should be marked dirty upon a transaction
	 */
	virtual bool ShouldBeMarkedDirtyUponTransaction() const { return true; }

#if WITH_EDITOR
private:

	/** Broadcasts a notification whenever the blueprint has changed. */
	FChangedEvent ChangedEvent;

	/** Broadcasts a notification whenever the blueprint is compiled. */
	FCompiledEvent CompiledEvent;

public:
	/** If this blueprint is currently being compiled, the CurrentMessageLog will be the log currently being used to send logs to. */
	FCompilerResultsLog* CurrentMessageLog;

	/** Message log for storing upgrade notes that were generated within the Blueprint, will be displayed to the compiler results each compiler and will remain until saving */
	TSharedPtr<FCompilerResultsLog> UpgradeNotesLog;

	/** Message log for storing pre-compile errors/notes/warnings that will only last until the next Blueprint compile */
	TSharedPtr<FCompilerResultsLog> PreCompileLog;

	/** 
	 * Sends a message to the CurrentMessageLog, if there is one available.  Otherwise, defaults to logging to the normal channels.
	 * Should use this for node and blueprint actions that happen during compilation!
	 * Note, needs to include BlueprintImpl.h in UnrealEd since code is using UnrealEd and this class is in Engine
	 */
	template<typename... ArgTypes>
	void Message_Note(const FString& MessageToLog, ArgTypes... Args);

	template<typename... ArgTypes>
	void Message_Warn(const FString& MessageToLog, ArgTypes... Args);

	template<typename... ArgTypes>
	void Message_Error(const FString& MessageToLog, ArgTypes... Args);
#endif
	
#if WITH_EDITORONLY_DATA
protected:
	/** 
	 * Blueprint can choose to load specific modules for compilation. Children are expected to call base implementation.
	 */
	ENGINE_API virtual void LoadModulesRequiredForCompilation();
#endif

#if WITH_EDITOR

public:
	/**
	 * Returns true if this blueprint supports global variables
	 */
	virtual bool SupportsGlobalVariables() const { return true; }

	/**
	 * Returns true if this blueprint supports global variables
	 */
	virtual bool SupportsLocalVariables() const { return true; }

	/**
	 * Returns true if this blueprint supports functions
	 */
	virtual bool SupportsFunctions() const { return true; }

	/**
	 * Returns true if this blueprints allows the given function to be overridden
	 */
	virtual bool AllowFunctionOverride(const UFunction* const InFunction) const { return true; }
		
	/**
	 * Returns true if this blueprint supports macros
	 */
	virtual bool SupportsMacros() const { return true; }

	/**
	 * Returns true if this blueprint supports delegates
	 */
	virtual bool SupportsDelegates() const { return true; }

	/**
	 * Returns true if this blueprint supports event graphs
	 */
	virtual bool SupportsEventGraphs() const { return true; }

	/**
	 * Returns true if this blueprint supports animation layers
	 */
	virtual bool SupportsAnimLayers() const { return false; }

	/**
	 * Copies a given graph into a text buffer. Returns true if successful.
	 */
	virtual bool ExportGraphToText(UEdGraph* InEdGraph, FString& OutText) { return false; }

	/**
	 * Returns true if the blueprint can import a given InClipboardText.
	 * If this return false the default BP functionality will be used.
	 */
	virtual bool CanImportGraphFromText(const FString& InClipboardText) { return false; }

	/**
	 * Returns a new ed graph if the blueprint's specialization imported a graph based on 
	 * a clipboard text content an nullptr if that's not successful.
	 */
	virtual bool TryImportGraphFromText(const FString& InClipboardText, UEdGraph** OutGraphPtr = nullptr) { return false; }

#endif
};


template<>
inline FName UBlueprint::GetFieldNameFromClassByGuid<UFunction>(const UClass* InClass, const FGuid FunctionGuid)
{
#if WITH_EDITOR
	return GetFunctionNameFromClassByGuid(InClass, FunctionGuid);
#else	// WITH_EDITOR
	return NAME_None;
#endif	// WITH_EDITOR
}

template<>
inline bool UBlueprint::GetGuidFromClassByFieldName<UFunction>(const UClass* InClass, const FName FunctionName, FGuid& FunctionGuid)
{
#if WITH_EDITOR
	return GetFunctionGuidFromClassByFieldName(InClass, FunctionName, FunctionGuid);
#else	// WITH_EDITOR
	return false;
#endif	// WITH_EDITOR
}
