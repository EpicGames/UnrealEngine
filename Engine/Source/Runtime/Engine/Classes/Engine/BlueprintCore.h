// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "BlueprintCore.generated.h"

UCLASS(MinimalAPI)
class UBlueprintCore
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Pointer to the skeleton class; this is regenerated any time a member variable or function is added but  
	is usually incomplete (no code or hidden autogenerated variables are added to it) */
	UPROPERTY(nontransactional, transient)
	TSubclassOf<class UObject>  SkeletonGeneratedClass;

	/** Pointer to the 'most recent' fully generated class */
	UPROPERTY(nontransactional)
	TSubclassOf<class UObject>	GeneratedClass;

	/** BackCompat:  Whether or not we need to purge references in this blueprint to the skeleton generated during compile-on-load  */
	UPROPERTY()
	bool bLegacyNeedToPurgeSkelRefs;

private:
	/** Blueprint Guid */
	UPROPERTY()
	FGuid BlueprintGuid;

public:

	ENGINE_API virtual void Serialize( FArchive& Ar ) override;

	/** Generates a new blueprint Guid, used when creating new blueprints */
	void GenerateNewGuid()
	{
		BlueprintGuid = FGuid::NewGuid();
	}

	/** Gets the Blueprint Guid */
	const FGuid& GetBlueprintGuid() const { return BlueprintGuid; }

private:
	
	/** Generates a new deterministic guid based on blueprint properties */
	ENGINE_API void GenerateDeterministicGuid();
};
