// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintPropertyGuidProvider.h"
#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "FieldNotificationId.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphPin.h"
#include "Containers/SortedMap.h"

#include "BlueprintGeneratedClass.generated.h"

class AActor;
class UActorComponent;
class UDynamicBlueprintBinding;
class UInheritableComponentHandler;
class UTimelineTemplate;
class UClassCookedMetaData;
class UEdGraphPin;

DECLARE_MEMORY_STAT_EXTERN(TEXT("Persistent Uber Graph Frame memory"), STAT_PersistentUberGraphFrameMemory, STATGROUP_Memory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BPComp Instancing Fast Path memory"), STAT_BPCompInstancingFastPathMemory, STATGROUP_Memory, );

USTRUCT()
struct FNodeToCodeAssociation
{
	GENERATED_USTRUCT_BODY()

public:
	TWeakObjectPtr<UEdGraphNode> Node;
	TWeakObjectPtr<UFunction> Scope;
	int32 Offset;
public:
	FNodeToCodeAssociation()
	: Node(nullptr)
	, Scope(nullptr)
	, Offset(0)
	{
	}

	FNodeToCodeAssociation(UEdGraphNode* InNode, UFunction* InFunction, int32 InOffset)
		: Node(InNode)
		, Scope(InFunction)
		, Offset(InOffset)
	{
	}
};


USTRUCT()
struct FDebuggingInfoForSingleFunction
{
	GENERATED_USTRUCT_BODY()

public:
	// Reverse map from code offset to source node
	TMap< int32, TWeakObjectPtr<UEdGraphNode> > LineNumberToSourceNodeMap;

	// Reverse map from code offset to source pin
	TMap< int32, FEdGraphPinReference > LineNumberToSourcePinMap;

	// Reverse map from source pin to mapped code offset(s)
	TMultiMap< FEdGraphPinReference, int32 > SourcePinToLineNumbersMap;

	// Map from source node (impure) to pure node script code range
	TMap< TWeakObjectPtr<UEdGraphNode>, FInt32Range > PureNodeScriptCodeRangeMap;

	// Reverse map from code offset to outer expansion (tunnel instance) source nodes
	TMap< int32, TArray<TWeakObjectPtr<UEdGraphNode> > > LineNumberToTunnelInstanceSourceNodesMap;

public:
	FDebuggingInfoForSingleFunction()
	{
	}
};


USTRUCT()
struct FPointerToUberGraphFrame
{
	GENERATED_USTRUCT_BODY()

public:
	uint8* RawPointer;
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	uint32 UberGraphFunctionKey;
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	// WARNING: This struct layout definition repeated in ScriptCore.cpp as
	// FPointerToUberGraphFrameCoreUObject to work around reflection generation issues
	//////////////////////////////////////////////////////////////////////////

	FPointerToUberGraphFrame()
		: RawPointer(nullptr)
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
		, UberGraphFunctionKey(0)
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	{}

	~FPointerToUberGraphFrame() = default;
};


template<>
struct TStructOpsTypeTraits<FPointerToUberGraphFrame> : public TStructOpsTypeTraitsBase2<FPointerToUberGraphFrame>
{
	enum
	{
		WithZeroConstructor = true,
		WithCopy = false,
	};
};


//////////////////////////////////////////////////////////////////////////
// TSimpleRingBuffer

template<typename ElementType>
class TSimpleRingBuffer
{
public:
	TSimpleRingBuffer(int32 MaxItems)
		: WriteIndex(0)
	{
		Storage.Empty(MaxItems);
	}

	int32 Num() const
	{
		return Storage.Num();
	}

	ElementType& operator()(int32 i)
	{
		// First element is at WriteIndex-1, second at 2, etc...
		const int32 RingIndex = (WriteIndex - 1 - i);
		const int32 WrappedRingIndex = (RingIndex < 0) ? (RingIndex + ArrayMax()) : RingIndex;

		return Storage[WrappedRingIndex];
	}

	const ElementType& operator()(int32 i) const
	{
		// First element is at WriteIndex-1, second at 2, etc...
		const int32 RingIndex = (WriteIndex - 1 - i);
		const int32 WrappedRingIndex = (RingIndex < 0) ? (RingIndex + ArrayMax()) : RingIndex;

		return Storage[WrappedRingIndex];
	}

	int32 ArrayMax() const
	{
		return Storage.GetSlack() + Storage.Num();
	}

	// Note: Doesn't call the constructor on objects stored in it; make sure to properly initialize the memory returned by WriteNewElementUninitialized
	ElementType& WriteNewElementUninitialized()
	{
		const int32 OldWriteIndex = WriteIndex;
		WriteIndex = (WriteIndex + 1) % ArrayMax();

		if (Storage.GetSlack())
		{
			Storage.AddUninitialized(1);
		}
		return Storage[OldWriteIndex];
	}

	ElementType& WriteNewElementInitialized()
	{
		const int32 OldWriteIndex = WriteIndex;
		WriteIndex = (WriteIndex + 1) % ArrayMax();

		if (Storage.GetSlack())
		{
			Storage.Add(ElementType());
		}
		else
		{
			Storage[OldWriteIndex] = ElementType();
		}
		return Storage[OldWriteIndex];
	}

private:
	TArray<ElementType> Storage;
	int32 WriteIndex;
};

USTRUCT()
struct FBlueprintDebugData
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FBlueprintDebugData();
	ENGINE_API FBlueprintDebugData(FBlueprintDebugData&&);
	ENGINE_API FBlueprintDebugData(const FBlueprintDebugData&);
	ENGINE_API FBlueprintDebugData& operator=(FBlueprintDebugData&&);
	ENGINE_API FBlueprintDebugData& operator=(const FBlueprintDebugData&);
	ENGINE_API ~FBlueprintDebugData();

#if WITH_EDITORONLY_DATA

protected:
	// Lookup table from UUID to nodes that were allocated that UUID
	TMap<int32, TWeakObjectPtr<UEdGraphNode> > DebugNodesAllocatedUniqueIDsMap;

	// Lookup table from impure node to entry in DebugNodeLineNumbers
	TMultiMap<TWeakObjectPtr<UEdGraphNode>, int32> DebugNodeIndexLookup;

	// List of debug site information for each node that ended up contributing to codegen
	//   This contains a tracepoint for each impure node after all pure antecedent logic has executed but before the impure function call
	//   It does *not* contain the wire tracepoint placed after the impure function call
	TArray<struct FNodeToCodeAssociation> DebugNodeLineNumbers;

	// List of entry points that contributed to the ubergraph
	TMap<int32, FName> EntryPoints;

	// Acceleration structure for execution wire highlighting at runtime
	TMap<TWeakObjectPtr<UFunction>, FDebuggingInfoForSingleFunction> PerFunctionLineNumbers;

	// Map from objects to class properties they created
	TMap<TWeakObjectPtr<UObject>, TFieldPath<FProperty> > DebugObjectToPropertyMap;

	// Map from pins or nodes to class properties they created
	TMap<FEdGraphPinReference, TFieldPath<FProperty> > DebugPinToPropertyMap;

public:

	// Returns the UEdGraphNode associated with the UUID, or nullptr if there isn't one.
	ENGINE_API UEdGraphNode* FindNodeFromUUID(int32 UUID) const;

	ENGINE_API bool IsValid() const;

	// Finds the UEdGraphNode associated with the code location Function+CodeOffset, or nullptr if there isn't one
	ENGINE_API UEdGraphNode* FindSourceNodeFromCodeLocation(const UFunction* Function, int32 CodeOffset, bool bAllowImpreciseHit) const;

	// Finds the source pin associated with the code location Function+CodeOffset, or nullptr if there isn't one
	ENGINE_API UEdGraphPin* FindSourcePinFromCodeLocation(UFunction* Function, int32 CodeOffset) const;

	// Finds all code locations (Function+CodeOffset) associated with the source pin
	ENGINE_API void FindAllCodeLocationsFromSourcePin(UEdGraphPin const* SourcePin, UFunction* InFunction, TArray<int32>& OutPinToCodeAssociations) const;

	// Finds the first code location (Function+CodeOffset) associated with the source pin within the given range, or INDEX_NONE if there isn't one
	ENGINE_API int32 FindCodeLocationFromSourcePin(UEdGraphPin const* SourcePin, UFunction* InFunction, FInt32Range InRange = FInt32Range()) const;

	// Finds all code locations (Function+CodeOffset) associated with the source node
	ENGINE_API void FindAllCodeLocationsFromSourceNode(UEdGraphNode* SourceNode, UFunction* InFunction, TArray<int32>& OutNodeToCodeAssociations) const;

	// Finds the pure node script code range associated with the [impure] source node, or FInt32Range(INDEX_NONE) if there is no existing association
	ENGINE_API FInt32Range FindPureNodeScriptCodeRangeFromSourceNode(const UEdGraphNode* SourceNode, UFunction* InFunction) const;

	ENGINE_API const TArray<TWeakObjectPtr<UEdGraphNode> >* FindExpansionSourceNodesFromCodeLocation(UFunction* Function, int32 CodeOffset) const;

	// Finds the breakpoint injection site(s) in bytecode if any were associated with the given node
	ENGINE_API void FindBreakpointInjectionSites(UEdGraphNode* Node, TArray<uint8*>& InstallSites) const;

	// Looks thru the debugging data for any class variables associated with the node
	ENGINE_API FProperty* FindClassPropertyForPin(const UEdGraphPin* Pin) const;

	// Looks thru the debugging data for any class variables associated with the node (e.g., temporary variables or timelines)
	ENGINE_API FProperty* FindClassPropertyForNode(const UEdGraphNode* Node) const;

	// Adds a debug record for a source node and destination in the bytecode of a specified function
	ENGINE_API void RegisterNodeToCodeAssociation(UEdGraphNode* SourceNode, const TArray<TWeakObjectPtr<UEdGraphNode> >& ExpansionSourceNodes, UFunction* InFunction, int32 CodeOffset, bool bBreakpointSite);

	ENGINE_API void RegisterPureNodeScriptCodeRange(UEdGraphNode* SourceNode, UFunction* InFunction, FInt32Range InPureNodeScriptCodeRange);

	ENGINE_API void RegisterPinToCodeAssociation(UEdGraphPin const* SourcePin, UFunction* InFunction, int32 CodeOffset);

	ENGINE_API const TMap<int32, FName>& GetEntryPoints() const;

	ENGINE_API bool IsValidEntryPoint(const int32 LinkId) const;

	ENGINE_API void RegisterEntryPoint(const int32 ScriptOffset, const FName FunctionName);

	// Registers an association between an object (pin or node typically) and an associated class member property
	ENGINE_API void RegisterClassPropertyAssociation(class UObject* TrueSourceObject, class FProperty* AssociatedProperty);

	ENGINE_API void RegisterClassPropertyAssociation(const UEdGraphPin* TrueSourcePin, class FProperty* AssociatedProperty);

	// Registers an association between a UUID and a node
	ENGINE_API void RegisterUUIDAssociation(UEdGraphNode* TrueSourceNode, int32 UUID);

	// Returns the object that caused the specified property to be created (can return nullptr if the association is unknown)
	ENGINE_API UObject* FindObjectThatCreatedProperty(class FProperty* AssociatedProperty) const;

	// Returns the pin that caused the specified property to be created (can return nullptr if the association is unknown or the association is from an object instead)
	ENGINE_API UEdGraphPin* FindPinThatCreatedProperty(class FProperty* AssociatedProperty) const;

	ENGINE_API void GenerateReversePropertyMap(TMap<FProperty*, UObject*>& PropertySourceMap);
#endif
};

USTRUCT()
struct FEventGraphFastCallPair
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UFunction> FunctionToPatch = nullptr;

	UPROPERTY()
	int32 EventGraphCallOffset = 0;
};

/** A single changed Blueprint component property. */
USTRUCT()
struct FBlueprintComponentChangedPropertyInfo
{
	GENERATED_USTRUCT_BODY()

	/** The name of the changed property. */
	UPROPERTY()
	FName PropertyName;

	/** The array index of the changed property. */
	UPROPERTY()
	int32 ArrayIndex;

	/** The parent struct (owner) of the changed property. */
	UPROPERTY()
	TObjectPtr<UStruct> PropertyScope;

	/** Default constructor. */
	FBlueprintComponentChangedPropertyInfo()
	{
		ArrayIndex = 0;
		PropertyScope = nullptr;
	}
};

/** Cooked data for a Blueprint component template. */
USTRUCT()
struct FBlueprintCookedComponentInstancingData
{
	GENERATED_USTRUCT_BODY()

	/** List of property info records with values that differ between the template and the component class CDO. This list will be generated at cook time. */
	UPROPERTY()
	TArray<struct FBlueprintComponentChangedPropertyInfo> ChangedPropertyList;

	/** Source template object class (recorded at load time and used for instancing). */
	UClass* ComponentTemplateClass;

	/** Source template object name (recorded at load time and used for instancing). */
	FName ComponentTemplateName;

	/** Source template object flags (recorded at load time and used for instancing). */
	TEnumAsByte<EObjectFlags> ComponentTemplateFlags;

	/** Flag indicating whether or not this contains valid cooked data. Note that an empty changed property list can also be a valid template data context. */
	UPROPERTY()
	bool bHasValidCookedData;

	/** Default constructor. */
	FBlueprintCookedComponentInstancingData()
	{
		bHasValidCookedData = false;
		ComponentTemplateClass = nullptr;
		ComponentTemplateFlags = RF_NoFlags;
	}

	/** Destructor. */
	ENGINE_API ~FBlueprintCookedComponentInstancingData();

	/** Builds/returns the internal property list that's used for serialization. This is a linked list of FProperty references. */
	ENGINE_API const FCustomPropertyListNode* GetCachedPropertyList() const;

	/** Called at load time to generate the internal cached property data stream from serialization of the source template object. */
	ENGINE_API void BuildCachedPropertyDataFromTemplate(UActorComponent* SourceTemplate);

	/** Returns the internal property data stream that's used for fast binary object serialization when instancing components at runtime. */
	const TArray<uint8>& GetCachedPropertyData() const { return CachedPropertyData; }

protected:
	/** Internal method used to help recursively build the cached property list for serialization. */
	ENGINE_API void BuildCachedPropertyList(FCustomPropertyListNode** CurrentNode, const UStruct* CurrentScope, int32* CurrentSourceIdx = nullptr) const;

	/** Internal method used to help recursively build a cached sub property list from an array property for serialization. */
	ENGINE_API void BuildCachedArrayPropertyList(const FArrayProperty* ArraySubPropertyNode, FCustomPropertyListNode** CurrentNode, int32* CurrentSourceIdx) const;

private:
	/** Internal property list that's used in binary object serialization at component instancing time. */
	mutable TIndirectArray<FCustomPropertyListNode> CachedPropertyListForSerialization;

	/** Internal property data stream that's used in binary object serialization at component instancing time. */
	TArray<uint8> CachedPropertyData;
};

/** Utility struct to store class overrides for components. */
USTRUCT()
struct FBPComponentClassOverride
{
	GENERATED_BODY()

	/** The component name an override is being specified for. */
	UPROPERTY()
	FName ComponentName;

	/** The class to use when constructing the component. */
	UPROPERTY()
	TObjectPtr<const UClass> ComponentClass;

	FBPComponentClassOverride()
		: ComponentClass(nullptr)
	{
	}

	FBPComponentClassOverride(FName InComponentName, const UClass* InComponentClass)
		: ComponentName(InComponentName)
		, ComponentClass(InComponentClass)
	{
	}

	bool operator==(const FName OtherComponentName) const
	{
		return (ComponentName == OtherComponentName);
	}
};

UCLASS(NeedsDeferredDependencyLoading, MinimalAPI)
class UBlueprintGeneratedClass : public UClass, public IBlueprintPropertyGuidProvider
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(AssetRegistrySearchable)
	int32	NumReplicatedProperties;

	/** Flag used to indicate if this class has data to support the component instancing fast path. */
	UPROPERTY()
	uint8 bHasCookedComponentInstancingData:1;

	/** **** Experimental ****
	 * This will enable dynamic subobject instancing, which will use default data to determine which
	 * properties need instancing at construction/serialization time rather than reflected property
	 * 'Instanced' semantics. */
	UPROPERTY()
	uint8 bSupportsDynamicInstancedReference:1;

#if WITH_EDITORONLY_DATA
	/** Used to check if this class has sparse data that can be serialized. This will be false when loading the data if it hasn't already been saved out. */
	UPROPERTY()
	uint8 bIsSparseClassDataSerializable:1;
#endif // WITH_EDITORONLY_DATA

private:
	/** Flag to make sure the custom property list has been initialized */
	uint8 bCustomPropertyListForPostConstructionInitialized:1;

#if WITH_EDITORONLY_DATA
	/** Flag to mark this generated class as an internal class, not for public consumption, such as a temporary skeleton or reinstancing class. 
	 *  Marked as a UPROPERTY so that it carries over during object duplication.
	 */
	UPROPERTY()
	uint8 bIsInternalClass:1;
#endif
	
public:
	/** Array of objects containing information for dynamically binding delegates to functions in this blueprint */
	UPROPERTY()
	TArray<TObjectPtr<class UDynamicBlueprintBinding>> DynamicBindingObjects;

	/** Array of component template objects, used by AddComponent function */
	UPROPERTY()
	TArray<TObjectPtr<class UActorComponent>> ComponentTemplates;

	/** Array of templates for timelines that should be created */
	UPROPERTY()
	TArray<TObjectPtr<class UTimelineTemplate>> Timelines;

	/** Array of blueprint overrides of component classes in parent classes */
	UPROPERTY()
	TArray<FBPComponentClassOverride> ComponentClassOverrides;

	/** The name of the properties with FieldNotify */
	UPROPERTY()
	TArray<FFieldNotificationId> FieldNotifies;

	int32 FieldNotifiesStartBitNumber = 0;

	/** 'Simple' construction script - graph of components to instance */
	UPROPERTY()
	TObjectPtr<class USimpleConstructionScript> SimpleConstructionScript;

	/** Stores data to override (in children classes) components (created by SCS) from parent classes */
	UPROPERTY()
	TObjectPtr<class UInheritableComponentHandler> InheritableComponentHandler;
	
	FStructProperty* UberGraphFramePointerProperty;

	UPROPERTY()
	TObjectPtr<UFunction> UberGraphFunction;

#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	uint32 UberGraphFunctionKey;
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

#if WITH_EDITORONLY_DATA
	// This is a list of event graph call function nodes that are simple (no argument) thunks into the event graph (typically used for animation delegates, etc...)
	// It is a deprecated list only used for backwards compatibility prior to VER_UE4_SERIALIZE_BLUEPRINT_EVENTGRAPH_FASTCALLS_IN_UFUNCTION.
	UPROPERTY()
	TArray<FEventGraphFastCallPair> FastCallPairs_DEPRECATED;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UObject> OverridenArchetypeForCDO;

	/** Property guid map */
	UPROPERTY()
	TMap<FName,FGuid> PropertyGuids;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UFunction>> CalledFunctions;
#endif //WITH_EDITORONLY_DATA

	/** Property guid map (if any), for use only when this BP is cooked */
	UPROPERTY()
	TMap<FName, FGuid> CookedPropertyGuids;

	// Mapping of changed properties & data to apply when instancing components in a cooked build (one entry per named AddComponent node template for fast lookup at runtime).
	// Note: This is not currently utilized by the editor; it is a runtime optimization for cooked builds only. It assumes that the component class structure does not change.
	UPROPERTY()
	TMap<FName, struct FBlueprintCookedComponentInstancingData> CookedComponentInstancingData;

	/** 
	 * Gets an array of all BPGeneratedClasses (including InClass as 0th element) parents of given generated class 
	 *
	 * @param InClass				The class to get the blueprint lineage for
	 * @param OutBlueprintParents	Array with the blueprints used to generate this class and its parents.  0th = this, Nth = least derived BP-based parent
	 * @return						true if there were no status errors in any of the parent blueprints, otherwise false
	 */
	static ENGINE_API bool GetGeneratedClassesHierarchy(const UClass* InClass, TArray<const UBlueprintGeneratedClass*>& OutBPGClasses);

	/**
	 * Iterate over all BPGCs used to generate this class and its parents, calling InFunc on them. First element is the BPGC used to generate InClass
	 *
	 * @param InClass				The class to get the blueprint lineage for
	 * @param InFunc				Function that will be called for each BPGC. Must return true to continue iteration, or false to stop.
	 * @return						true if there were no status errors in any of the parent blueprints, otherwise false
	 */
	static ENGINE_API bool ForEachGeneratedClassInHierarchy(const UClass* InClass, TFunctionRef<bool(const UBlueprintGeneratedClass*)> InFunc);

	ENGINE_API UInheritableComponentHandler* GetInheritableComponentHandler(const bool bCreateIfNecessary = false);

	/** Find the object in the TemplateObjects array with the supplied name */
	ENGINE_API UActorComponent* FindComponentTemplateByName(const FName& TemplateName) const;

	/** Create Timeline objects for this Actor based on the Timelines array*/
	static ENGINE_API void CreateComponentsForActor(const UClass* ThisClass, AActor* Actor);
	static ENGINE_API void CreateTimelineComponent(AActor* Actor, const UTimelineTemplate* TimelineTemplate);

	/** Called by the UE::FieldNotification::IClassDescriptor. */
	ENGINE_API void ForEachFieldNotify(TFunctionRef<bool(::UE::FieldNotification::FFieldId FieldId)> Callback, bool bIncludeSuper) const;
	ENGINE_API void InitializeFieldNotifies();

#if WITH_EDITOR
	/** Customization point for fixing up backpointers to the class or CDO after compile for instances that can be reused (e.g. no layout change) */
	ENGINE_API void ReinitializeObjectAfterCompile(UObject* Object) const;
	virtual void HandleReinitializeObjectAfterCompile(UObject* Object) const {}

	/**
	 * Called during serialization to allow the class to stash any sparse class data that needs to 
	 * be conformed against the archetype once the CDO is available.
	 */
	ENGINE_API virtual void PrepareToConformSparseClassData(UScriptStruct* SparseClassDataArchetypeStruct);

	/**
	 * Called to conform any pending sparse class data stashed by PrepareToConformSparseClassData.
	 * @note Called either prior to generating the CDO (compile-on-load) or after loading the CDO.
	 */
	ENGINE_API virtual void ConformSparseClassData(UObject* Object);
#endif

	//~ IBlueprintPropertyGuidProvider interface
	ENGINE_API virtual FName FindBlueprintPropertyNameFromGuid(const FGuid& PropertyGuid) const override final;
	ENGINE_API virtual FGuid FindBlueprintPropertyGuidFromName(const FName PropertyName) const override final;
	// End IBlueprintPropertyGuidProvider interface

	//~ UObject interface
	ENGINE_API virtual UE::Core::FVersePath GetVersePath() const override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	ENGINE_API virtual void GetAdditionalAssetDataObjectsForCook(FArchiveCookContext& CookContext, 
		TArray<UObject*>& OutObjects) const override;
protected:
	ENGINE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const;
public:
#endif //~ WITH_EDITOR
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetId() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
	ENGINE_API virtual bool NeedsLoadForClient() const override;
	ENGINE_API virtual bool NeedsLoadForEditorGame() const override;
	ENGINE_API virtual bool CanBeClusterRoot() const override;
#if WITH_EDITOR
	ENGINE_API virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) override;
	ENGINE_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	ENGINE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
#endif	//~ WITH_EDITOR
	ENGINE_API virtual bool IsAsset() const override;
	//~ End UObject interface

	//~ UClass interface
#if WITH_EDITOR
	ENGINE_API virtual UClass* GetAuthoritativeClass() override;
	ENGINE_API virtual void ConditionalRecompileClass(FUObjectSerializeContext* InLoadContext) override;
	ENGINE_API virtual void FlushCompilationQueueForLevel() override;
	ENGINE_API virtual UObject* GetArchetypeForCDO() const override;
#endif //~ WITH_EDITOR
	ENGINE_API virtual void SerializeDefaultObject(UObject* Object, FStructuredArchive::FSlot Slot) override;
	ENGINE_API virtual void PostLoadDefaultObject(UObject* Object) override;
	ENGINE_API virtual bool IsFunctionImplementedInScript(FName InFunctionName) const override;
	ENGINE_API virtual uint8* GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const override;
	ENGINE_API virtual void CreatePersistentUberGraphFrame(UObject* Obj, bool bCreateOnlyIfEmpty = false, bool bSkipSuperClass = false, UClass* OldClass = nullptr) const override;
	ENGINE_API virtual void DestroyPersistentUberGraphFrame(UObject* Obj, bool bSkipSuperClass = false) const override;
	ENGINE_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	ENGINE_API virtual void PurgeClass(bool bRecompilingOnLoad) override;
	ENGINE_API virtual void Bind() override;
	ENGINE_API virtual void GetDefaultObjectPreloadDependencies(TArray<UObject*>& OutDeps) override;
	ENGINE_API virtual UObject* FindArchetype(const UClass* ArchetypeClass, const FName ArchetypeName) const override;

	ENGINE_API virtual void InitPropertiesFromCustomList(uint8* DataPtr, const uint8* DefaultDataPtr) override;
	ENGINE_API virtual void SetupObjectInitializer(FObjectInitializer& ObjectInitializer) const override;
	ENGINE_API virtual void PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph) override;
	ENGINE_API virtual void PostLoadInstance(UObject* InObj) override;

protected:

	ENGINE_API virtual FName FindPropertyNameFromGuid(const FGuid& PropertyGuid) const override;
	ENGINE_API virtual FGuid FindPropertyGuidFromName(const FName InName) const override;
	ENGINE_API virtual bool ArePropertyGuidsAvailable() const override;
	//~ End UClass interface

	/**
	* Returns a linked list of properties with default values that differ from the parent default object. If non-NULL, only these properties will
	* be copied post-construction. Otherwise, all properties will be copied to the new instance, even if the default value matches the inherited default value.
	*/
	const FCustomPropertyListNode* GetCustomPropertyListForPostConstruction() const
	{
		return CustomPropertyListForPostConstruction.Num() > 0 ? *CustomPropertyListForPostConstruction.GetData() : nullptr;
	}

	/**
	* Helper method to assist with initializing object properties from an explicit list.
	*
	* @param	InPropertyList		only these properties will be copied from defaults
	* @param	InStruct			the current scope for which the given property list applies
	* @param	DataPtr				destination address (where to start copying values to)
	* @param	DefaultDataPtr		source address (where to start copying the defaults data from)
	*/
	static ENGINE_API void InitPropertiesFromCustomList(const FCustomPropertyListNode* InPropertyList, UStruct* InStruct, uint8* DataPtr, const uint8* DefaultDataPtr);

	/**
	* Helper method to assist with initializing from an array property with an explicit item list.
	*
	* @param	ArrayProperty		the array property for which the given property list applies
	* @param	InPropertyList		only these properties (indices) will be copied from defaults
	* @param	DataPtr				destination address (where to start copying values to)
	* @param	DefaultDataPtr		source address (where to start copying the defaults data from)
	*/
	static ENGINE_API void InitArrayPropertyFromCustomList(const FArrayProperty* ArrayProperty, const FCustomPropertyListNode* InPropertyList, uint8* DataPtr, const uint8* DefaultDataPtr);


	/**
	* Helper method to assist with initializing a property from sub property list
	*
	* @param	Property					the property to copy
	* @param	CustomPropertySubListNode	the list node of the sub property to copy
	* @param	PropertyValue				destination address (where to start copying values to)
	* @param	DefaultPropertyValue		source address (where to start copying the defaults data from)
	* @return	true if the method was able to copy the property successfully via the sub custom property list
	*/
	static ENGINE_API bool InitPropertyFromSubPropertyList(const FProperty* Property, const FCustomPropertyListNode* SubPropertyList, uint8* PropertyValue, const uint8* DefaultPropertyValue);

public:

	/** Called when the custom list of properties used during post-construct initialization needs to be rebuilt (e.g. after serialization and recompilation). */
	ENGINE_API void UpdateCustomPropertyListForPostConstruction();

	static ENGINE_API void AddReferencedObjectsInUbergraphFrame(UObject* InThis, FReferenceCollector& Collector);

	static ENGINE_API FName GetUberGraphFrameName();
	static ENGINE_API bool UsePersistentUberGraphFrame();

	/** Whether or not to use "fast path" component instancing. */
	ENGINE_API bool UseFastPathComponentInstancing();
	
	/** Returns true if this is an internal BP class, either a skeleton or a reinstancing class, which are not meant to be shown to the user.
	 *  @return true if this class is an internal BP class.
	 */
	bool IsInternalClass() const
	{
#if WITH_EDITORONLY_DATA
		return bIsInternalClass;
#else
		// Internal BP classes will never show up in non-editor builds.
		return false;
#endif
	}

#if WITH_EDITORONLY_DATA
	FBlueprintDebugData DebugData;


	FBlueprintDebugData& GetDebugData()
	{
		return DebugData;
	}

	const FBlueprintDebugData& GetDebugData() const
	{
		return DebugData;
	}
#endif

	/** Bind functions on supplied actor to delegates */
	static ENGINE_API void BindDynamicDelegates(const UClass* ThisClass, UObject* InInstance);

	// Finds the desired dynamic binding object for this blueprint generated class
	static ENGINE_API UDynamicBlueprintBinding* GetDynamicBindingObject(const UClass* ThisClass, UClass* BindingClass);

#if WITH_EDITOR
	/** Unbind functions on supplied actor from delegates */
	static ENGINE_API void UnbindDynamicDelegates(const UClass* ThisClass, UObject* InInstance);

	/** Unbind functions on supplied actor from delegates tied to a specific property */
	ENGINE_API void UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty);
#endif

	/** called to gather blueprint replicated properties */
	ENGINE_API virtual void GetLifetimeBlueprintReplicationList(TArray<class FLifetimeProperty>& OutLifetimeProps) const;

protected:
	/** Internal helper method used to recursively build the custom property list that's used for post-construct initialization. */
	ENGINE_API bool BuildCustomPropertyListForPostConstruction(FCustomPropertyListNode*& InPropertyList, UStruct* InStruct, const uint8* DataPtr, const uint8* DefaultDataPtr);

	/** Internal helper method used to recursively build a custom property list from an array property used for post-construct initialization. */
	ENGINE_API bool BuildCustomArrayPropertyListForPostConstruction(FArrayProperty* ArrayProperty, FCustomPropertyListNode*& InPropertyList, const uint8* DataPtr, const uint8* DefaultDataPtr, int32 StartIndex = 0);

private:
	/** List of native class-owned properties that differ from defaults. This is used to optimize property initialization during post-construction by minimizing the number of native class-owned property values that get copied to the new instance. */
	TIndirectArray<FCustomPropertyListNode> CustomPropertyListForPostConstruction;
	/** In some cases UObject::ConditionalPostLoad() code calls PostLoadDefaultObject() on a class that's still being serialized. */
	FTransactionallySafeCriticalSection SerializeAndPostLoadCritical;

	using FEditorTags = TSortedMap<FName, FString, FDefaultAllocator, FNameFastLess>;

#if WITH_EDITORONLY_DATA
	ENGINE_API void GetEditorTags(FEditorTags& Tags) const;

	/** Editor-only asset registry tags on cooked BPGC */
	FEditorTags CookedEditorTags;

protected:
	ENGINE_API virtual TSubclassOf<UClassCookedMetaData> GetCookedMetaDataClass() const;

private:
	ENGINE_API UClassCookedMetaData* NewCookedMetaData();
	ENGINE_API const UClassCookedMetaData* FindCookedMetaData();
	ENGINE_API void PurgeCookedMetaData();

	UPROPERTY()
	TObjectPtr<UClassCookedMetaData> CachedCookedMetaDataPtr;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	TWeakObjectPtr<UScriptStruct> SparseClassDataPendingConformStruct = nullptr;
	void* SparseClassDataPendingConform = nullptr;
#endif // WITH_EDITORONLY_DATA
	
	// Also set on UClass, but friend-ness is not an inherited attribute in C++.
	friend class FBlueprintCompileReinstancer;
	friend struct FBlueprintCompilationManagerImpl;
};
