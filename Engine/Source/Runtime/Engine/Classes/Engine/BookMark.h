// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BookmarkBase.h"
#include "BookMark.generated.h"


USTRUCT()
struct FBookmarkJumpToSettings : public FBookmarkBaseJumpToSettings
{
	GENERATED_BODY()

	bool bShouldRestorLevelVisibility = true;
};

/**
 * A camera position the current level.
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UBookMark : public UBookmarkBase
{
	GENERATED_UCLASS_BODY()

	/** Camera position */
	UPROPERTY(EditAnywhere, Category=BookMark)
	FVector Location;

	/** Camera rotation */
	UPROPERTY(EditAnywhere, Category=BookMark)
	FRotator Rotation;

	/** Array of levels that are hidden */
	UPROPERTY(EditAnywhere, Category=BookMark)
	TArray<FString> HiddenLevels;
};
