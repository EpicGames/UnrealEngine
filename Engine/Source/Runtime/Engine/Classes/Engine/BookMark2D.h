// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BookmarkBase.h"
#include "BookMark2D.generated.h"

USTRUCT()
struct FBookmark2DJumpToSettings
{
	GENERATED_BODY()
};

/**
 * Simple class to store 2D camera information.
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UBookMark2D : public UBookmarkBase
{
	GENERATED_UCLASS_BODY()

	/** Zoom of the camera */
	UPROPERTY(EditAnywhere, Category=BookMark2D)
	float Zoom2D;

	/** Location of the camera */
	UPROPERTY(EditAnywhere, Category=BookMark2D)
	FIntPoint Location;
};
