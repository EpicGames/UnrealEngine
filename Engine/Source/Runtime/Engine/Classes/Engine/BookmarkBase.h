// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "BookmarkBase.generated.h"

USTRUCT()
struct FBookmarkBaseJumpToSettings
{
	GENERATED_BODY()
};

/**
 * Bookmarks are objects that can keep track of specific locations within a level or scene, without
 * necessarily adding additional actors to that scene.
 *
 * Different implementations can be used depending on specific needs, and new implementations can
 * be generated by implementing a class Derived on UBookmarkBase.
 *
 * See UBookMark, UBookMark2D, AGameWorldSettings::DefaultBookmarkClass, IBookmarkEditorInteractor.
 */
UCLASS(hidecategories=Object, Abstract, MinimalAPI)
class UBookmarkBase : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Called when a bookmark is cleared.
	 */
	virtual void OnCleared()
	{
		Modify();
		MarkAsGarbage();
	}
};
