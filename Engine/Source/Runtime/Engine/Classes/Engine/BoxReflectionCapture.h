// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/ReflectionCapture.h"
#include "BoxReflectionCapture.generated.h"


/** 
 * Actor used to capture the scene for reflection in a box shape 
 * @see https://docs.unrealengine.com/latest/INT/Resources/ContentExamples/Reflections/1_3/index.html
 */
UCLASS(hidecategories=(Collision, Attachment, Actor), MinimalAPI)
class ABoxReflectionCapture
	: public AReflectionCapture
{
	GENERATED_UCLASS_BODY()
};
