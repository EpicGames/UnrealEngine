// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Brush.generated.h"

class UBrushBuilder;
class ULevel;

//-----------------------------------------------------------------------------
// Variables.
UENUM()
enum ECsgOper : int
{
	/** Active brush. (deprecated, do not use.) */
	CSG_Active,
	/** Add to world. (deprecated, do not use.) */
	CSG_Add,
	/** Subtract from world. (deprecated, do not use.) */
	CSG_Subtract,
	/** Form from intersection with world. */
	CSG_Intersect,
	/** Form from negative intersection with world. */
	CSG_Deintersect,
	CSG_None,
	CSG_MAX,
};


UENUM()
enum EBrushType : int
{
	/** Default/builder brush. */
	Brush_Default UMETA(Hidden),
	/** Add to world. */
	Brush_Add UMETA(DisplayName=Additive),
	/** Subtract from world. */
	Brush_Subtract UMETA(DisplayName=Subtractive),
	Brush_MAX,
};


// Selection information for geometry mode
USTRUCT()
struct FGeomSelection
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 Type;    // EGeometrySelectionType_

	UPROPERTY()
	int32 Index;    // Index into the geometry data structures

	UPROPERTY()
	int32 SelectionIndex;    // The selection index of this item


	FGeomSelection()
		: Type(0)
		, Index(0)
		, SelectionIndex(0)
	{ }

	friend FArchive& operator<<(FArchive& Ar,FGeomSelection& MyGeomSelection)
	{
		return Ar << MyGeomSelection.Type << MyGeomSelection.Index << MyGeomSelection.SelectionIndex;
	}
};


UCLASS(hidecategories=(Object, Collision, Display, Rendering, Physics, Input, Blueprint), showcategories=("Input|MouseInput", "Input|TouchInput"), NotBlueprintable, ConversionRoot, MinimalAPI)
class ABrush
	: public AActor
{
	GENERATED_UCLASS_BODY()

	/** Type of brush */
	UPROPERTY(EditAnywhere, Category=Brush, meta = (NoResetToDefault))
	TEnumAsByte<enum EBrushType> BrushType;

	// Information.
	UPROPERTY()
	FColor BrushColor;

	UPROPERTY()
	int32 PolyFlags;

	UPROPERTY()
	uint32 bColored:1;

	UPROPERTY()
	uint32 bSolidWhenSelected:1;

	/** If true, this brush class can be placed using the class browser like other simple class types */
	UPROPERTY()
	uint32 bPlaceableFromClassBrowser:1;

	/** If true, this brush is a builder or otherwise does not need to be loaded into the game */
	UPROPERTY()
	uint32 bNotForClientOrServer:1;

	UPROPERTY(Instanced)
	TObjectPtr<class UModel> Brush;

private:
	UPROPERTY(Category = Collision, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UBrushComponent> BrushComponent;
public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Instanced, Category=BrushBuilder)
	TObjectPtr<class UBrushBuilder> BrushBuilder;
	
	/** If true, display the brush with a shaded volume */
	UPROPERTY(Transient, EditAnywhere, Category=BrushSettings)
	uint32 bDisplayShadedVolume:1;

	/** Value used to set the opacity for the shaded volume, between 0-1 */
	UPROPERTY(Transient, EditAnywhere, Category=BrushSettings, meta=(ClampMin=0.0, ClampMax=1.0))
	float ShadedVolumeOpacityValue = 0.25f;
#endif

	/** Flag set when we are in a manipulation (scaling, translation, brush builder param change etc.) */
	UPROPERTY()
	uint32 bInManipulation:1;

	/**
	 * Stores selection information from geometry mode.  This is the only information that we can't
	 * regenerate by looking at the source brushes following an undo operation.
	 */
	UPROPERTY()
	TArray<struct FGeomSelection> SavedSelections;
	
#if WITH_EDITOR
	/** Delegate used for notifications when PostRegisterAllComponents is called for a Brush */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBrushRegistered, ABrush*);
	/** Function to get the 'brush registered' delegate */
	static FOnBrushRegistered& GetOnBrushRegisteredDelegate()
	{
		return OnBrushRegistered;
	}
#endif

public:
	
	// UObject interface.
#if WITH_EDITOR
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostEditMove(bool bFinished) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual FName GetCustomIconName() const override;
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif // WITH_EDITOR

	virtual bool NeedsLoadForClient() const override
	{ 
		return !IsNotForClientOrServer() && Super::NeedsLoadForClient(); 
	}

	virtual bool NeedsLoadForServer() const override
	{ 
		return !IsNotForClientOrServer() && Super::NeedsLoadForServer(); 
	}

public:
	
	// AActor interface
	ENGINE_API virtual bool IsLevelBoundsRelevant() const override;
	ENGINE_API virtual bool IsComponentRelevantForNavigation(UActorComponent* Component) const override;
	ENGINE_API virtual void RebuildNavigationData();

#if WITH_EDITOR
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual void SetIsTemporarilyHiddenInEditor( bool bIsHidden ) override;
	ENGINE_API virtual bool SetIsHiddenEdLayer(bool bIsHiddenEdLayer) override;
	ENGINE_API virtual bool SupportsLayers() const override;
	ENGINE_API virtual bool SupportsExternalPackaging() const override;
	ENGINE_API virtual bool CanDeleteSelectedActor(FText& OutReason) const override;
	ENGINE_API virtual bool CanReplaceSelectedActor(FText& OutReason) const override;
	ENGINE_API virtual bool IsActorLabelEditable() const override;
public:

	ENGINE_API virtual void InitPosRotScale();
	ENGINE_API virtual void CopyPosRotScaleFrom( ABrush* Other );

	/** @return true if its Brush Component should ever participate in BSP Rebuild */
	bool ENGINE_API CanEverAffectBSP() const;

	/** @return true if its Brush Component should be considered when building the level's static navigable geometry */
	bool ENGINE_API ShouldExportStaticNavigableGeometry() const;
	
	static void SetSuppressBSPRegeneration(bool bSuppress) { bSuppressBSPRegeneration = bSuppress; }
	static bool GetSuppressBSPRegeneration() { return bSuppressBSPRegeneration; }
private:

	/** An array to keep track of all the levels that need rebuilding. This is checked via NeedsRebuild() in the editor tick and triggers a csg rebuild. */
	static ENGINE_API TArray< TWeakObjectPtr< ULevel > > LevelsToRebuild;

	/** Delegate called when PostRegisterAllComponents is called for a Brush */
	static ENGINE_API FOnBrushRegistered OnBrushRegistered;

	/** Global bool to suppress automatic BSP regeneration */
	static ENGINE_API bool bSuppressBSPRegeneration;

public:

	/**
	 * Called to see if any of the levels need rebuilding
	 *
	 * @param	OutLevels if specified, provides a copy of the levels array
	 * @return	true if the csg needs to be rebuilt on the next editor tick.	
	 */
	static ENGINE_API bool NeedsRebuild(TArray< TWeakObjectPtr< ULevel > >* OutLevels = nullptr);

	/**
	 * Called upon finishing the csg rebuild to clear the rebuild bool.
	 */
	static void OnRebuildDone()
	{
		LevelsToRebuild.Empty();
	}

	/**
	 * Called to make not of the level that needs rebuilding
	 *
	 * @param	InLevel The level that needs rebuilding
	 */
	static ENGINE_API void SetNeedRebuild(ULevel* InLevel);
#endif//WITH_EDITOR

	/** @return true if this is a static brush */
	ENGINE_API virtual bool IsStaticBrush() const;

	/** @return false */
	virtual bool IsVolumeBrush() const { return false; }
	
	/** @return false */
	virtual bool IsBrushShape() const { return false; }

	// ABrush interface.

	/** Figures out the best color to use for this brushes wireframe drawing.	*/
	ENGINE_API virtual FColor GetWireColor() const;

	/**
	 * Return if true if this brush is not used for gameplay (i.e. builder brush)
	 * 
	 * @return	true if brush is not for client or server
	 */
	inline bool IsNotForClientOrServer() const
	{
		return bNotForClientOrServer;
	}

	/** Indicate that this brush need not be loaded on client or servers	 */
	inline void SetNotForClientOrServer()
	{
		bNotForClientOrServer = true;
	}

	/** Indicate that brush need should be loaded on client or servers	 */
	inline void ClearNotForClientOrServer()
	{
		bNotForClientOrServer = false;
	}

#if WITH_EDITORONLY_DATA
	/** @return the brush builder that created the current brush shape */
	const UBrushBuilder* GetBrushBuilder() const { return BrushBuilder; }
#endif

public:
	/** Returns BrushComponent subobject **/
	class UBrushComponent* GetBrushComponent() const { return BrushComponent; }

#if WITH_EDITOR
	/** Debug purposes only; an attempt to catch the cause of UE-36265 */
	static ENGINE_API const TCHAR* GGeometryRebuildCause;
#endif
};
