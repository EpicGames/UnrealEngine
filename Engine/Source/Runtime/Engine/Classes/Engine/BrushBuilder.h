// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BrushBuilder.generated.h"

class ABrush;

// Internal state, not accessible to script.
USTRUCT()
struct FBuilderPoly
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<int32> VertexIndices;

	UPROPERTY()
	int32 Direction;

	UPROPERTY()
	FName ItemName;

	UPROPERTY()
	int32 PolyFlags;

	FBuilderPoly()
	: Direction(0), ItemName(NAME_None), PolyFlags(0)
	{}
};


/**
 * Base class of UnrealEd brush builders.
 *
 *
 * Tips for writing brush builders:
 *
 * - Always validate the user-specified and call BadParameters function
 *   if anything is wrong, instead of actually building geometry.
 *   If you build an invalid brush due to bad user parameters, you'll
 *   cause an extraordinary amount of pain for the poor user.
 *
 * - When generating polygons with more than 3 vertices, BE SURE all the
 *   polygon's vertices are coplanar!  Out-of-plane polygons will cause
 *   geometry to be corrupted.
 */
UCLASS(abstract, hidecategories=(Object), MinimalAPI)
class UBrushBuilder
	: public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FString BitmapFilename;

	/** localized FString that will be displayed as the name of this brush builder in the editor */
	UPROPERTY()
	FString ToolTip;

	/** If false, disabled the bad param notifications */
	UPROPERTY(Transient)
	uint32 NotifyBadParams:1;

protected:

	UPROPERTY()
	TArray<FVector> Vertices;

	UPROPERTY()
	TArray<struct FBuilderPoly> Polys;

	UPROPERTY()
	FName Layer;

	UPROPERTY()
	uint32 MergeCoplanars:1;

public:

	// @ todo document all below
	virtual void BeginBrush( bool InMergeCoplanars, FName InLayer ) {}
	virtual bool EndBrush( UWorld* InWorld, ABrush* InBrush ) { return false; }
	virtual int32 GetVertexCount() const { return 0; }
	virtual FVector GetVertex( int32 i ) const { return FVector::ZeroVector; }
	virtual int32 GetPolyCount() const { return 0; }
	virtual bool BadParameters( const FText& msg ) { return false; }
	virtual int32 Vertexv( FVector v ) { return 0; }
	virtual int32 Vertex3f( float X, float Y, float Z ) { return 0; }
	virtual void Poly3i( int32 Direction, int32 i, int32 j, int32 k, FName ItemName = NAME_None, bool bIsTwoSidedNonSolid = false ) {}
	virtual void Poly4i( int32 Direction, int32 i, int32 j, int32 k, int32 l, FName ItemName = NAME_None, bool bIsTwoSidedNonSolid = false ) {}
	virtual void PolyBegin( int32 Direction, FName ItemName = NAME_None ) {}
	virtual void Polyi( int32 i ) {}
	virtual void PolyEnd() {}

	/**
	 * Builds the brush shape for the specified ABrush or the builder brush if Brush is nullptr.
	 * @param InWorld The world to operate in
	 * @param InBrush The brush to change shape, or nullptr to specify the builder brush
	 * @return true if the brush shape was updated.
	 */
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = nullptr )
	{
		return false;
	}
};
