// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Brush.h"
#include "BrushShape.generated.h"


/**
 * A brush that acts as a template for geometry mode modifiers like "Lathe".
 */
UCLASS(MinimalAPI)
class ABrushShape
	: public ABrush
{
	GENERATED_UCLASS_BODY()

public:

	// ABrush overrides

	virtual bool IsStaticBrush() const override
	{
		return false;
	}

	virtual bool IsBrushShape() const override
	{
		return true;
	}
};
