// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"
#include "CancellableAsyncAction.generated.h"

class UGameInstance;

/**
 * base class for asynchronous actions that can be spawned from UK2Node_AsyncAction or C++ code.
 * These actions register themselves with the game instance and need to be explicitly canceled or ended normally to go away.
 * The ExposedAsyncProxy metadata specifies the blueprint node will return this object for later canceling.
 */
UCLASS(Abstract, BlueprintType, meta = (ExposedAsyncProxy = AsyncAction), MinimalAPI)
class UCancellableAsyncAction : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()
public:

	/** Handle when this action is being destroyed to ensure that the action is canceled and child classes can clean up. */
	ENGINE_API virtual void BeginDestroy() override;

	/** Cancel an asynchronous action, this attempts to cancel any lower level processes and also prevents delegates from being fired */
	UFUNCTION(BlueprintCallable, Category = "Async Action")
	ENGINE_API virtual void Cancel();

	/** Returns true if this action is still active and has not completed or been cancelled */
	UFUNCTION(BlueprintCallable, Category = "Async Action")
	ENGINE_API virtual bool IsActive() const;

	/** This should be called prior to broadcasting delegates back into the event graph, this ensures the action is still valid */
	ENGINE_API virtual bool ShouldBroadcastDelegates() const;

	/** Returns true if this action is registered with a valid game instance */
	ENGINE_API bool IsRegistered() const;

	/** Wrapper function to get a timer manager for scheduling callbacks */
	ENGINE_API class FTimerManager* GetTimerManager() const;
	
};
