// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Canvas.generated.h"

class UMaterialInterface;
class UReporterGraph;
class UTexture;
class FSceneView;

/**
 * Holds texture information with UV coordinates as well.
 */
USTRUCT(BlueprintType)
struct FCanvasIcon
{
	GENERATED_USTRUCT_BODY()

	/** Source texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CanvasIcon)
	TObjectPtr<class UTexture> Texture;

	/** UV coords */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CanvasIcon)
	float U;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CanvasIcon)
	float V;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CanvasIcon)
	float UL;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CanvasIcon)
	float VL;

	FCanvasIcon()
		: Texture(nullptr)
		, U(0)
		, V(0)
		, UL(0)
		, VL(0)
	{ }

};

struct FDisplayDebugManager
{
private:
	FCanvasTextItem DebugTextItem;
	FVector2f CurrentPos;
	float NextColumXPos;
	float MaxCharHeight;
	FVector2f InitialPos;
	class UCanvas* Canvas;
	int32 ColumnCount;
public:
	FDisplayDebugManager()
		: DebugTextItem(FCanvasTextItem(FVector2D(0, 0), FText::GetEmpty(), nullptr, FLinearColor::White))
		, CurrentPos(FVector2f::ZeroVector)
		, NextColumXPos(0.f)
		, MaxCharHeight(0.f)
		, InitialPos(FVector2f::ZeroVector)
		, Canvas(nullptr)
		, ColumnCount(0)
	{
		DebugTextItem.EnableShadow(FLinearColor::Black);
	}

	void Initialize(class UCanvas* InCanvas, const UFont* NewFont, FVector2D InInitialPosition)
	{
		SetFont(NewFont);
		Canvas = InCanvas;
		InitialPos = (FVector2f)InInitialPosition;
		CurrentPos = InitialPos;
		NextColumXPos = 0.f;
		ColumnCount = 0;
	}

	void SetFont(const UFont* NewFont)
	{
		if (NewFont && (NewFont != DebugTextItem.Font))
		{
			DebugTextItem.Font = NewFont;
			MaxCharHeight = DebugTextItem.Font->GetMaxCharHeight();
		}
	}

	void SetDrawColor(const FColor& NewColor)
	{
		DebugTextItem.SetColor(NewColor.ReinterpretAsLinear());
	}

	void SetLinearDrawColor(const FLinearColor& NewColor)
	{
		DebugTextItem.SetColor(NewColor);
	}

	void ENGINE_API DrawString(const FString& InDebugString, const float& OptionalXOffset = 0.f);

	void AddColumnIfNeeded();

	float GetTextScale() const;

	float GetYStep() const
	{
		return MaxCharHeight * 1.15f * GetTextScale();
	}

	float GetXPos() const
	{
		return CurrentPos.X;
	}

	float GetYPos() const
	{
		return CurrentPos.Y;
	}

	float& GetYPosRef()
	{
		return CurrentPos.Y;
	}

	void SetYPos(const float NewYPos)
	{
		CurrentPos.Y = NewYPos;
	}

	float GetMaxCharHeight() const
	{
		return MaxCharHeight;
	}

	float& GetMaxCharHeightRef()
	{
		return MaxCharHeight;
	}

	void ShiftYDrawPosition(const float& YOffset)
	{
		CurrentPos.Y += YOffset;
		AddColumnIfNeeded();
	}
};

/**
 * A drawing canvas.
 */
UCLASS(transient, BlueprintType, MinimalAPI)
class UCanvas
	: public UObject
{
	GENERATED_UCLASS_BODY()

	// Modifiable properties.
	UPROPERTY()
	float OrgX;		// Origin for drawing in X.

	UPROPERTY()
	float OrgY;    // Origin for drawing in Y.

	UPROPERTY()
	float ClipX;	// Bottom right clipping region.

	UPROPERTY()
	float ClipY;    // Bottom right clipping region.
		
	UPROPERTY()
	FColor DrawColor;    // Color for drawing.

	UPROPERTY()
	uint32 bCenterX:1;    // Whether to center the text horizontally (about CurX)

	UPROPERTY()
	uint32 bCenterY:1;    // Whether to center the text vertically (about CurY)

	UPROPERTY()
	uint32 bNoSmooth:1;    // Don't bilinear filter.

	UPROPERTY()
	int32 SizeX;		// Zero-based actual dimensions X.

	UPROPERTY()
	int32 SizeY;		// Zero-based actual dimensions Y.

	// Internal.
	UPROPERTY()
	FPlane ColorModulate; 

	UPROPERTY()
	TObjectPtr<class UTexture2D> DefaultTexture; //Default texture to use 

	UPROPERTY()
	TObjectPtr<class UTexture2D> GradientTexture0; //Default texture to use 

	/** Helper class to render 2d graphs on canvas */
	UPROPERTY()
	TObjectPtr<class UReporterGraph> ReporterGraph;

	int32 UnsafeSizeX;   // Canvas size before safe frame adjustment
	int32 UnsafeSizeY;	// Canvas size before safe frame adjustment

	//cached data for safe zone calculation.  Some platforms have very expensive functions
	//to grab display metrics
	int32 SafeZonePadX;
	int32 SafeZonePadY;
	int32 SafeZonePadEX;
	int32 SafeZonePadEY;
	int32 CachedDisplayWidth;
	int32 CachedDisplayHeight;

	FDisplayDebugManager DisplayDebugManager;
public:
	FCanvas* Canvas;
	FSceneView* SceneView;
	FMatrix	ViewProjectionMatrix;
	FQuat HmdOrientation;

	// UCanvas interface.
	
	/** Initializes the canvas. */
	ENGINE_API void Init(int32 InSizeX, int32 InSizeY, FSceneView* InSceneView, FCanvas* InCanvas);

	ENGINE_API virtual void BeginDestroy() override;

	/** Changes the view for the canvas. */
	ENGINE_API void SetView(FSceneView* InView);

	/** Updates the canvas. */
	ENGINE_API void Update();

	/* Applies the current Platform's safe zone to the current Canvas position. Automatically called by Update. */
	ENGINE_API void ApplySafeZoneTransform();
	ENGINE_API void PopSafeZoneTransform();

	/* Updates cached SafeZone data from the device.  Call when main device is resized. */
	ENGINE_API void UpdateSafeZoneData();

	/* Function to go through all constructed canvas items and update their safe zone data. */
	static ENGINE_API void UpdateAllCanvasSafeZoneData();

	/** 
	 * Draw arbitrary aligned rectangle.
	 *
	 * @param Tex Texture to draw.
	 * @param X Position to draw X.
	 * @param Y Position to draw Y.
	 * @param XL Width of tile.
	 * @param YL Height of tile.
	 * @param U Horizontal position of the upper left corner of the portion of the texture to be shown(texels).
	 * @param V Vertical position of the upper left corner of the portion of the texture to be shown(texels).
	 * @param UL The width of the portion of the texture to be drawn(texels).
	 * @param VL The height of the portion of the texture to be drawn(texels).
	 * @param BlendMode Blending mode of texture.
	 */
	ENGINE_API void DrawTile(UTexture* Tex, float X, float Y, float XL, float YL, float U, float V, float UL, float VL, EBlendMode BlendMode=BLEND_Translucent);

	/**
	 * Calculate the length of a string.
	 *
	 * @param Font The font used.
	 * @param ScaleX Scale in X axis.
	 * @param ScaleY Scale in Y axis.
	 * @param XL out Horizontal length of string.
	 * @param YL out Vertical length of string.
	 * @param Text String to calculate for.
	 */
	static ENGINE_API void ClippedStrLen(const UFont* Font, float ScaleX, float ScaleY, int32& XL, int32& YL, FStringView Text);

	/**	
	 * Calculate the size of a string built from a font, word wrapped to a specified region.
	 */
	ENGINE_API void VARARGS WrappedStrLenf(const UFont* Font, float ScaleX, float ScaleY, int32& XL, int32& YL, const TCHAR* Fmt, ...);
		
	/**
	 * Compute size and optionally print text with word wrap.
	 */
	ENGINE_API int32 WrappedPrint(bool Draw, float X, float Y, int32& out_XL, int32& out_YL, const UFont* Font, float ScaleX, float ScaleY, bool bCenterTextX, bool bCenterTextY, FStringView Text, const FFontRenderInfo& RenderInfo) ;
	
	/**
	 * Draws a string of text to the screen.
	 *
	 * @param InFont The font to draw with.
	 * @param InText The string to be drawn.
	 * @param X Position to draw X.
	 * @param Y Position to draw Y.
	 * @param XScale Optional. The horizontal scaling to apply to the text. 
	 * @param YScale Optional. The vertical scaling to apply to the text. 
	 * @param RenderInfo Optional. The FontRenderInfo to use when drawing the text.
	 * @return The Y extent of the rendered text.
	 */
	ENGINE_API float DrawText(const UFont* InFont, FStringView InText, float X, float Y, float XScale = 1.f, float YScale = 1.f, const FFontRenderInfo& RenderInfo = FFontRenderInfo());
	ENGINE_API float DrawText(const UFont* InFont, const FText& InText, float X, float Y, float XScale = 1.f, float YScale = 1.f, const FFontRenderInfo& RenderInfo = FFontRenderInfo());

#if !PLATFORM_TCHAR_IS_UTF8CHAR

	UE_DEPRECATED(5.5, "Passing an ANSI string to DrawText has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	ENGINE_API float DrawText(const UFont* InFont, FAnsiStringView InText, float X, float Y, float XScale = 1.f, float YScale = 1.f, const FFontRenderInfo& RenderInfo = FFontRenderInfo());

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

	enum ELastCharacterIndexFormat
	{
		// The last whole character before the horizontal offset
		LastWholeCharacterBeforeOffset,
		// The character directly at the offset
		CharacterAtOffset,
		// Not used
		Unused,
	};

	/** 
	 * Measures a string, optionally stopped after the specified horizontal offset in pixels is reached.
	 * 
	 * @param	Parameters	Used for various purposes
	 *							DrawXL:		[out] will be set to the width of the string
	 *							DrawYL:		[out] will be set to the height of the string
	 *							DrawFont:	[in] specifies the font to use for retrieving the size of the characters in the string
	 *							Scale:		[in] specifies the amount of scaling to apply to the string
	 * @param	Text		The string to calculate the size for
	 * @param	StopAfterHorizontalOffset  Offset horizontally into the string to stop measuring characters after, in pixels (or INDEX_NONE)
	 * @param	CharIndexFormat  Behavior to use for StopAfterHorizontalOffset
	 * @param	OutLastCharacterIndex  The index of the last character processed (used with StopAfterHorizontalOffset)
	 *
	 */
	static ENGINE_API void MeasureStringInternal(FTextSizingParameters& Parameters, FStringView Text, const int32 StopAfterHorizontalOffset, const ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex);

	UE_DEPRECATED(5.4, "Use MeasureStringInternal with FStringView instead.")
	static ENGINE_API void MeasureStringInternal(FTextSizingParameters& Parameters, const TCHAR* const Text, const int32 TextLength, const int32 StopAfterHorizontalOffset, const ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex);

	/**
	 * Calculates the size of the specified string.
	 *
	 * @param	Parameters	Used for various purposes
	 *							DrawXL:		[out] will be set to the width of the string
	 *							DrawYL:		[out] will be set to the height of the string
	 *							DrawFont:	[in] specifies the font to use for retrieving the size of the characters in the string
	 *							Scale:		[in] specifies the amount of scaling to apply to the string
	 * @param	Text		the string to calculate the size for
	 */
	static ENGINE_API void CanvasStringSize(FTextSizingParameters& Parameters, FStringView Text);

	/**
	 * Parses a single string into an array of strings that will fit inside the specified bounding region.
	 *
	 * @param	Parameters		Used for various purposes:
	 *							DrawX:		[in] specifies the pixel location of the start of the horizontal bounding region that should be used for wrapping.
	 *							DrawY:		[in] specifies the Y origin of the bounding region.  This should normally be set to 0, as this will be
	 *										     used as the base value for DrawYL.
	 *										[out] Will be set to the Y position (+YL) of the last line, i.e. the total height of all wrapped lines relative to the start of the bounding region
	 *							DrawXL:		[in] specifies the pixel location of the end of the horizontal bounding region that should be used for wrapping
	 *							DrawYL:		[in] specifies the height of the bounding region, in pixels.  A input value of 0 indicates that
	 *										     the bounding region height should not be considered.  Once the total height of lines reaches this
	 *										     value, the function returns and no further processing occurs.
	 *							DrawFont:	[in] specifies the font to use for retrieving the size of the characters in the string
	 *							Scale:		[in] specifies the amount of scaling to apply to the string
	 * @param	CurX			specifies the pixel location to begin the wrapping; usually equal to the X pos of the bounding region, unless wrapping is initiated
	 *								in the middle of the bounding region (i.e. indentation)
	 * @param	Text			the text that should be wrapped
	 * @param	out_Lines		[out] will contain an array of strings which fit inside the bounding region specified.  Does
	 *							not clear the array first.
	 * @param	OutWrappedLineData An optional array to fill with the indices from the source string marking the begin and end points of the wrapped lines
	 */
	static ENGINE_API void WrapString(FCanvasWordWrapper& Wrapper, FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData = nullptr);

	ENGINE_API void WrapString(FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData = nullptr);

	/**
	 * Transforms a 3D world-space vector into 2D screen coordinates.
	 *
	 * @param Location			The vector to transform.
	 * @param bClampToZeroPlane	If true, 2D screen coordinates behind the viewing plane (-Z) will have Z set to 0 (leaving X and Y alone)
	 * @return The transformed vector.
	 */
	ENGINE_API FVector Project(FVector Location, bool bClampToZeroPlane = true) const;

	/** 
	 * Transforms 2D screen coordinates into a 3D world-space origin and direction.
	 *
	 * @param ScreenPos Screen coordinates in pixels.
	 * @param WorldOrigin (out) World-space origin vector.
	 * @param WorldDirection (out) World-space direction vector.
	 */
	ENGINE_API void Deproject(FVector2D ScreenPos, /*out*/ FVector& WorldOrigin, /*out*/ FVector& WorldDirection) const;

	/**
	 * Calculate the length of a string, taking text wrapping into account.
	 *
	 * @param InFont The Font use.
	 * @param InText The string to calculate for.
	 * @param XL out Horizontal length of string.
	 * @param YL out Vertical length of string.
	 * @param bDPIAware If true measures text considering the current DPI scale factor of the canvas.  Defaults to false for backwards compatibility
	 * @param InCanvas Canvas state object
	 */
	static ENGINE_API void StrLen(const UFont* InFont, FStringView InText, float& XL, float& YL, bool bDPIAware, FCanvas* InCanvas);
	static ENGINE_API void StrLen(const UFont* InFont, FStringView InText, double& XL, double& YL, bool bDPIAware, FCanvas* InCanvas);
	
#if !PLATFORM_TCHAR_IS_UTF8CHAR

	UE_DEPRECATED(5.5, "Passing an ANSI string to StrLen has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	static ENGINE_API void StrLen(const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, bool bDPIAware, FCanvas* InCanvas);
	UE_DEPRECATED(5.5, "Passing an ANSI string to StrLen has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	static ENGINE_API void StrLen(const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, bool bDPIAware, FCanvas* InCanvas);

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

	/**
	 * Calculate the length of a string, taking text wrapping into account.
	 *
	 * @param InFont The Font use.
	 * @param InText The string to calculate for.
	 * @param XL out Horizontal length of string.
	 * @param YL out Vertical length of string.
	 * @param bDPIAware If true measures text considering the current DPI scale factor of the canvas.  Defaults to false for backwards compatibility
	 */
	ENGINE_API void StrLen(const UFont* InFont, FStringView InText, float& XL, float& YL, bool bDPIAware = false);
	ENGINE_API void StrLen(const UFont* InFont, FStringView InText, double& XL, double& YL, bool bDPIAware = false);
	
#if !PLATFORM_TCHAR_IS_UTF8CHAR

	UE_DEPRECATED(5.5, "Passing an ANSI string to StrLen has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	ENGINE_API void StrLen(const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, bool bDPIAware = false);
	UE_DEPRECATED(5.5, "Passing an ANSI string to StrLen has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	ENGINE_API void StrLen(const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, bool bDPIAware = false);

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

	/** 
	 * Calculates the horizontal and vertical size of a given string. This is used for clipped text as it does not take wrapping into account.
	 *
	 * @param InFont The font to use.
	 * @param InText String to calculate for.
	 * @param XL out Horizontal length of string.
	 * @param YL out Vertical length of string.
	 * @param ScaleX Scale that the string is expected to draw at horizontally.
	 * @param ScaleY Scale that the string is expected to draw at vertically.
	 */
	ENGINE_API void TextSize( const UFont* InFont, FStringView InText, float& XL, float& YL, float ScaleX=1.f, float ScaleY=1.f);
	ENGINE_API void TextSize( const UFont* InFont, FStringView InText, double& XL, double& YL, double ScaleX=1.f, double ScaleY=1.f);
	
#if !PLATFORM_TCHAR_IS_UTF8CHAR

	UE_DEPRECATED(5.5, "Passing an ANSI string to TextSize has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	ENGINE_API void TextSize( const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, float ScaleX=1.f, float ScaleY=1.f);
	UE_DEPRECATED(5.5, "Passing an ANSI string to TextSize has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	ENGINE_API void TextSize( const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, double ScaleX=1.f, double ScaleY=1.f);

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

	/** Set DrawColor with a FLinearColor and optional opacity override */
	ENGINE_API void SetLinearDrawColor(FLinearColor InColor, float OpacityOverride=-1.f);

	/** Set draw color. (R,G,B,A) */
	ENGINE_API void SetDrawColor(uint8 R, uint8 G, uint8 B, uint8 A = 255);

	/** Set draw color. (FColor) */
	ENGINE_API void SetDrawColor(FColor const& C);

	/** constructor for FontRenderInfo */
	ENGINE_API FFontRenderInfo CreateFontRenderInfo(bool bClipText = false, bool bEnableShadow = false, FLinearColor GlowColor = FLinearColor(), FVector2D GlowOuterRadius = FVector2D(), FVector2D GlowInnerRadius = FVector2D());
	
	/** reset canvas parameters, optionally do not change the origin */
	ENGINE_API virtual void Reset(bool bKeepOrigin = false);

	/** Sets the position of the lower-left corner of the clipping region of the Canvas */
	ENGINE_API void SetClip(float X, float Y);

	/** Return X,Y for center of the draw region. */
	ENGINE_API void GetCenter(float& outX, float& outY) const;
	ENGINE_API void GetCenter(double& outX, double& outY) const;
	
	/** Fake CanvasIcon constructor.	 */
	static ENGINE_API FCanvasIcon MakeIcon(class UTexture* Texture, float U = 0.f, float V = 0.f, float UL = 0.f, float VL = 0.f);

	/** Draw a scaled CanvasIcon at the desired canvas position. */
	ENGINE_API void DrawScaledIcon(FCanvasIcon Icon, float X, float Y, FVector Scale);
	
	/** Draw a CanvasIcon at the desired canvas position.	 */
	ENGINE_API void DrawIcon(FCanvasIcon Icon, float X, float Y, float Scale = 0.f);

	/**
	 * Draws a graph comparing 2 variables.  Useful for visual debugging and tweaking.
	 *
	 * @param Title		Label to draw on the graph, or "" for none
	 * @param ValueX	X-axis value of the point to plot
	 * @param ValueY	Y-axis value of the point to plot
	 * @param UL_X		X screen coord of the upper-left corner of the graph
	 * @param UL_Y		Y screen coord of the upper-left corner of the graph
	 * @param W			Width of the graph, in pixels
	 * @param H			Height of the graph, in pixels
	 * @param RangeX	Range of values expressed by the X axis of the graph
	 * @param RangeY	Range of values expressed by the Y axis of the graph
	 */
	ENGINE_API virtual void DrawDebugGraph(const FString& Title, float ValueX, float ValueY, float UL_X, float UL_Y, float W, float H, FVector2D RangeX, FVector2D RangeY);

	/** 
	 * Draw a CanvasItem
	 *
	 * @param Item			Item to draw
	 */
	ENGINE_API void DrawItem( FCanvasItem& Item );

	/** 
	 * Draw a CanvasItem at the given coordinates
	 *
	 * @param Item			Item to draw
	 * @param InPosition	Position to draw item
	 */
	ENGINE_API void DrawItem( FCanvasItem& Item, const FVector2D& InPosition );
	
	/** 
	 * Draw a CanvasItem at the given coordinates
	 *
	 * @param Item			Item to draw
	 * @param X				X Position to draw item
	 * @param Y				Y Position to draw item
	 */
	ENGINE_API void DrawItem( FCanvasItem& Item, float X, float Y );

	/** Creates if necessary and returns ReporterGraph instance for 2d graph canvas drawing */
	ENGINE_API TWeakObjectPtr<class UReporterGraph>  GetReporterGraph();

	/** @return the current DPI scale being applied to all canvas draw elements */
	float GetDPIScale() const { return Canvas->GetDPIScale(); }

	/**
	 * Draws a line on the Canvas.
	 *
	 * @param ScreenPositionA		Starting position of the line in screen space.
	 * @param ScreenPositionB		Ending position of the line in screen space.
	 * @param Thickness				How many pixels thick this line should be.
	 * @param RenderColor			Color to render the line.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Line", ScriptName="DrawLine"))
	ENGINE_API void K2_DrawLine(FVector2D ScreenPositionA=FVector2D::ZeroVector, FVector2D ScreenPositionB=FVector2D::ZeroVector, float Thickness=1.0f, FLinearColor RenderColor=FLinearColor::White);

	/**
	 * Draws a texture on the Canvas.
	 *
	 * @param RenderTexture				Texture to use when rendering. If no texture is set then this will use the default white texture.
	 * @param ScreenPosition			Screen space position to render the texture.
	 * @param ScreenSize				Screen space size to render the texture.
	 * @param CoordinatePosition		Normalized UV starting coordinate to use when rendering the texture.
	 * @param CoordinateSize			Normalized UV size coordinate to use when rendering the texture.
	 * @param RenderColor				Color to use when rendering the texture.
	 * @param BlendMode					Blending mode to use when rendering the texture.
	 * @param Rotation					Rotation, in degrees, to render the texture.
	 * @param PivotPoint				Normalized pivot point to use when rotating the texture.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Texture", ScriptName="DrawTexture"))
	ENGINE_API void K2_DrawTexture(UTexture* RenderTexture, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize=FVector2D::UnitVector, FLinearColor RenderColor=FLinearColor::White, EBlendMode BlendMode=BLEND_Translucent, float Rotation=0.f, FVector2D PivotPoint=FVector2D(0.5f,0.5f));

	/**
	 * Draws a material on the Canvas.
	 *
	 * @param RenderMaterial			Material to use when rendering. Remember that only the emissive channel is able to be rendered as no lighting is performed when rendering to the Canvas.
	 * @param ScreenPosition			Screen space position to render the texture.
	 * @param ScreenSize				Screen space size to render the texture.
	 * @param CoordinatePosition		Normalized UV starting coordinate to use when rendering the texture.
	 * @param CoordinateSize			Normalized UV size coordinate to use when rendering the texture.
	 * @param Rotation					Rotation, in degrees, to render the texture.
	 * @param PivotPoint				Normalized pivot point to use when rotating the texture.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Material", ScriptName="DrawMaterial"))
	ENGINE_API void K2_DrawMaterial(UMaterialInterface* RenderMaterial, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize=FVector2D::UnitVector, float Rotation=0.f, FVector2D PivotPoint=FVector2D(0.5f,0.5f));

	/**
	 * Draws text on the Canvas.
	 *
	 * @param RenderFont				Font to use when rendering the text. If this is null, then a default engine font is used.
	 * @param RenderText				Text to render on the Canvas.
	 * @param ScreenPosition			Screen space position to render the text.
	 * @param RenderColor				Color to render the text.
	 * @param Kerning					Horizontal spacing adjustment to modify the spacing between each letter.
	 * @param ShadowColor				Color to render the shadow of the text.
	 * @param ShadowOffset				Pixel offset relative to the screen space position to render the shadow of the text.
	 * @param bCentreX					If true, then interpret the screen space position X coordinate as the center of the rendered text.
	 * @param bCentreY					If true, then interpret the screen space position Y coordinate as the center of the rendered text.
	 * @param bOutlined					If true, then the text should be rendered with an outline.
	 * @param OutlineColor				Color to render the outline for the text.
	 */	
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Text", ScriptName="DrawText"))
	ENGINE_API void K2_DrawText(UFont* RenderFont, const FString& RenderText, FVector2D ScreenPosition, FVector2D Scale=FVector2D(1.0f,1.0f), FLinearColor RenderColor=FLinearColor::White, float Kerning=0.0f, FLinearColor ShadowColor=FLinearColor::Black, FVector2D ShadowOffset=FVector2D::UnitVector, bool bCentreX=false, bool bCentreY=false, bool bOutlined=false, FLinearColor OutlineColor=FLinearColor::Black);
	
	/**
	 * Draws a 3x3 grid border with tiled frame and tiled interior on the Canvas.
	 *
	 * @param BorderTexture				Texture to use for border.
	 * @param BackgroundTexture			Texture to use for border background.
	 * @param LeftBorderTexture			Texture to use for the tiling left border.
	 * @param RightBorderTexture		Texture to use for the tiling right border.
	 * @param TopBorderTexture			Texture to use for the tiling top border.
	 * @param BottomBorderTexture		Texture to use for the tiling bottom border.
	 * @param ScreenPosition			Screen space position to render the texture.
	 * @param ScreenSize				Screen space size to render the texture.
	 * @param CoordinatePosition		Normalized UV starting coordinate to use when rendering the border texture.
	 * @param CoordinateSize			Normalized UV size coordinate to use when rendering the border texture.
	 * @param RenderColor				Color to tint the border.
	 * @param BorderScale				Scale of the border.
	 * @param BackgroundScale			Scale of the background.
	 * @param Rotation					Rotation, in degrees, to render the texture.
	 * @param PivotPoint				Normalized pivot point to use when rotating the texture.
	 * @param CornerSize				Frame corner size in percent of frame texture (should be < 0.5f).
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Border", ScriptName="DrawBorder"))
	ENGINE_API void K2_DrawBorder(UTexture* BorderTexture, UTexture* BackgroundTexture, UTexture* LeftBorderTexture, UTexture* RightBorderTexture, UTexture* TopBorderTexture, UTexture* BottomBorderTexture, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize=FVector2D::UnitVector, FLinearColor RenderColor=FLinearColor::White, FVector2D BorderScale=FVector2D(0.1f,0.1f), FVector2D BackgroundScale=FVector2D(0.1f,0.1f), float Rotation=0.0f, FVector2D PivotPoint=FVector2D(0.5f,0.5f), FVector2D CornerSize=FVector2D::ZeroVector);

	/**
	 * Draws an unfilled box on the Canvas.
	 *
	 * @param ScreenPosition			Screen space position to render the text.
	 * @param ScreenSize				Screen space size to render the texture.
	 * @param Thickness					How many pixels thick the box lines should be.
	 * @param RenderColor				Color to tint the box.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Box", ScriptName="DrawBox"))
	ENGINE_API void K2_DrawBox(FVector2D ScreenPosition, FVector2D ScreenSize, float Thickness=1.0f, FLinearColor RenderColor=FLinearColor::White);

	/**
	 * Draws a set of triangles on the Canvas.
	 *
	 * @param RenderTexture				Texture to use when rendering the triangles. If no texture is set, then the default white texture is used.
	 * @param Triangles					Triangles to render.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Triangles", ScriptName="DrawTriangles"))
	ENGINE_API void K2_DrawTriangle(UTexture* RenderTexture, TArray<FCanvasUVTri> Triangles);

	/**
	 * Draws a set of triangles on the Canvas.
	 *
	 * @param RenderMaterial			Material to use when rendering. Remember that only the emissive channel is able to be rendered as no lighting is performed when rendering to the Canvas.
	 * @param Triangles					Triangles to render.
	 */
	UFUNCTION(BlueprintCallable, Category = Canvas, meta = (DisplayName = "Draw Material Triangles", ScriptName = "DrawMaterialTriangles"))
	ENGINE_API void K2_DrawMaterialTriangle(UMaterialInterface* RenderMaterial, TArray<FCanvasUVTri> Triangles);
	/**
	 * Draws a polygon on the Canvas.
	 *
	 * @param RenderTexture				Texture to use when rendering the triangles. If no texture is set, then the default white texture is used.
	 * @param ScreenPosition			Screen space position to render the text.
	 * @param Radius					How large in pixels this polygon should be.
	 * @param NumberOfSides				How many sides this polygon should have. This should be above or equal to three.
	 * @param RenderColor				Color to tint the polygon.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Draw Polygon", ScriptName="DrawPolygon"))
	ENGINE_API void K2_DrawPolygon(UTexture* RenderTexture, FVector2D ScreenPosition, FVector2D Radius=FVector2D::UnitVector, int32 NumberOfSides=3, FLinearColor RenderColor=FLinearColor::White);

	/**
	 * Performs a projection of a world space coordinates using the projection matrix set up for the Canvas.
	 *
	 * @param WorldLocation				World space location to project onto the Canvas rendering plane.
	 * @return							Returns a vector where X, Y defines a screen space position representing the world space location.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Project", ScriptName="Project"))
	ENGINE_API FVector K2_Project(FVector WorldLocation);

	/**
	 * Performs a deprojection of a screen space coordinate using the projection matrix set up for the Canvas.
	 *
	 * @param ScreenPosition			Screen space position to deproject to the World.
	 * @param WorldOrigin				Vector which is the world position of the screen space position.
	 * @param WorldDirection			Vector which can be used in a trace to determine what is "behind" the screen space position. Useful for object picking.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Deproject", ScriptName="Deproject"))
	ENGINE_API void K2_Deproject(FVector2D ScreenPosition, FVector& WorldOrigin, FVector& WorldDirection);

	/**
	 * Returns the wrapped text size in screen space coordinates.
	 *
	 * @param RenderFont				Font to use when determining the size of the text. If this is null, then a default engine font is used.
	 * @param RenderText				Text to determine the size of.
	 * @return							Returns the screen space size of the text.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Wrapped Text Size", ScriptName="WrappedTextSize"))
	ENGINE_API FVector2D K2_StrLen(UFont* RenderFont, const FString& RenderText);

	/**
	 * Returns the clipped text size in screen space coordinates.
	 *
	 * @param RenderFont				Font to use when determining the size of the text. If this is null, then a default engine font is used.
	 * @param RenderText				Text to determine the size of.
	 * @param Scale						Scale of the font to use when determining the size of the text.
	 * @return							Returns the screen space size of the text.
	 */
	UFUNCTION(BlueprintCallable, Category=Canvas, meta=(DisplayName="Clipped Text Size", ScriptName="ClippedTextSize"))
	ENGINE_API FVector2D K2_TextSize(UFont* RenderFont, const FString& RenderText, FVector2D Scale=FVector2D::UnitVector);
};
