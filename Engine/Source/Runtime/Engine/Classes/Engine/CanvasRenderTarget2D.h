// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptMacros.h"
#include "Engine/TextureRenderTarget2D.h"
#include "CanvasRenderTarget2D.generated.h"

class UCanvas;

/** This delegate is assignable through Blueprint and has similar functionality to the above. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnCanvasRenderTargetUpdate, UCanvas*, Canvas, int32, Width, int32, Height);


/**
 * CanvasRenderTarget2D is 2D render target which exposes a Canvas interface to allow you to draw elements onto 
 * it directly.  Use CreateCanvasRenderTarget2D() to create a render target texture by unique name, then
 * bind a function to the OnCanvasRenderTargetUpdate delegate which will be called when the render target is
 * updated.  If you need to repaint your canvas every single frame, simply call UpdateResource() on it from a Tick
 * function.  Also, remember to hold onto your new canvas render target with a reference so that it doesn't get
 * garbage collected.
 */
UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UCanvasRenderTarget2D
	: public UTextureRenderTarget2D
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;

	/**
	 * Updates the the canvas render target texture's resource. This is where the render target will create or 
	 * find a canvas object to use.  It also calls UpdateResourceImmediate() to clear the render target texture 
	 * from the deferred rendering list, to stop the texture from being cleared the next frame. From there it
	 * will ask the rendering thread to set up the RHI viewport. The canvas is then set up for rendering and 
	 * then the user's update delegate is called.  The canvas is then flushed and the RHI resolves the 
	 * texture to make it available for rendering.
	 */
	UFUNCTION(BlueprintCallable, Category="Canvas Render Target 2D")
	void UpdateResource() { UpdateResourceWithParams(EUpdateResourceFlags::None); }

	/**
	 * Creates a new canvas render target and initializes it to the specified dimensions
	 *
	 * @param	WorldContextObject	The world where this render target will be rendered for
	 * @param	CanvasRenderTarget2DClass	Class of the render target.  Unless you want to use a special sub-class, you can simply pass UCanvasRenderTarget2D::StaticClass() here.
	 * @param	Width				Width of the render target.
	 * @param	Height				Height of the render target.
	 *
	 * @return						Returns the instanced render target.
	 */
	UFUNCTION(BlueprintCallable, Category="Canvas Render Target 2D",meta=(WorldContext="WorldContextObject" ))
	static ENGINE_API UCanvasRenderTarget2D* CreateCanvasRenderTarget2D(UObject* WorldContextObject, TSubclassOf<UCanvasRenderTarget2D> CanvasRenderTarget2DClass, int32 Width = 1024, int32 Height = 1024);
 
	/**
	 * Allows a Blueprint to implement how this Canvas Render Target 2D should be updated.
	 *
	 * @param	Canvas				Canvas object that can be used to paint to the render target
	 * @param	Width				Width of the render target.
	 * @param	Height				Height of the render target.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category="Canvas Render Target 2D")
	ENGINE_API void ReceiveUpdate(UCanvas* Canvas, int32 Width, int32 Height);

	/**
	 * Gets a specific render target's size from the global map of canvas render targets.
	 *
	 * @param	Width	Output variable for the render target's width
	 * @param	Height	Output variable for the render target's height
	 */
	UFUNCTION(BlueprintPure, Category="Canvas Render Target 2D")
	ENGINE_API void GetSize(int32& Width, int32& Height);


	/** Called when this Canvas Render Target is asked to update its texture resource. */
	UPROPERTY(BlueprintAssignable, Category="Canvas Render Target 2D")
	FOnCanvasRenderTargetUpdate OnCanvasRenderTargetUpdate;

	// UObject overrides
	ENGINE_API virtual UWorld* GetWorld() const override;

	/** Don't delete the underlying resource if it already exists */
	ENGINE_API void FastUpdateResource();

	inline bool ShouldClearRenderTargetOnReceiveUpdate() const { return bShouldClearRenderTargetOnReceiveUpdate; }
	inline void SetShouldClearRenderTargetOnReceiveUpdate(bool bInShouldClearRenderTargetOnReceiveUpdate)
	{
		bShouldClearRenderTargetOnReceiveUpdate = bInShouldClearRenderTargetOnReceiveUpdate;
	}

	UFUNCTION(BlueprintGetter)
	ENGINE_API virtual ETextureRenderTargetSampleCount GetSampleCount() const override;

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetSampleCount(ETextureRenderTargetSampleCount InSampleCount);

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:

	ENGINE_API void RepaintCanvas();

	void OnSampleCountChanged();

	/* The world this render target will be used with */
	UPROPERTY()
	TWeakObjectPtr<UWorld> World;

	/** MSAA sample count. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetSampleCount, BlueprintSetter = SetSampleCount, Category = TextureRenderTarget2D, AssetRegistrySearchable, AdvancedDisplay, meta = (AllowPrivateAccess = "true"))
	ETextureRenderTargetSampleCount SampleCount;
	
	// If true, clear the render target to green whenever OnReceiveUpdate() is called.  (Defaults to true.)
	// If false, the render target will retain whatever values it had, allowing the user to update only areas that
	// have changed.
	UPROPERTY(Transient)
	bool bShouldClearRenderTargetOnReceiveUpdate;
};
