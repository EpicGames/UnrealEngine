// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/CoreNet.h"
#include "HAL/LowLevelMemTracker.h"
#include "Channel.generated.h"

LLM_DECLARE_TAG_API(NetChannel, ENGINE_API);

class FInBunch;
class FOutBunch;
class UNetConnection;

// Constant for all buffers that are reading from the network
inline const int MAX_STRING_SERIALIZE_SIZE	= NAME_SIZE;

/**
 * Enumerates channel types.
 */
enum EChannelType
{
	CHTYPE_None			= 0,  // Invalid type.
	CHTYPE_Control		= 1,  // Connection control.
	CHTYPE_Actor  		= 2,  // Actor-update channel.

	// @todo: Remove and reassign number to CHTYPE_Voice (breaks net compatibility)
	CHTYPE_File         = 3,  // Binary file transfer.

	CHTYPE_Voice		= 4,  // VoIP data channel
	CHTYPE_MAX          = 8,  // Maximum.
};

/**
 * Flags for channel creation.
 */
enum class EChannelCreateFlags : uint32
{
	None			= (1 << 0),
	OpenedLocally	= (1 << 1)
};
ENUM_CLASS_FLAGS(EChannelCreateFlags);

enum class EChannelGetAdditionalRequiredBunchesFlags : uint32
{
	None				= 0U,
	SkipNetGUIDExports	= 1U
};
ENUM_CLASS_FLAGS(EChannelGetAdditionalRequiredBunchesFlags);

// The channel index to use for voice
#define VOICE_CHANNEL_INDEX 1


/**
 * Base class of communication channels.
 */
UCLASS(abstract, transient, MinimalAPI)
class UChannel : public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TObjectPtr<class UNetConnection>	Connection;		// Owner connection.

	// Variables.
	uint32				OpenAcked:1;		// If OpenedLocally is true, this means we have acknowledged the packet we sent the bOpen bunch on. Otherwise, it means we have received the bOpen bunch from the server.
	uint32				Closing:1;			// State of the channel.
	uint32				Dormant:1;			// Channel is going dormant (it will close but the client will not destroy
	UE_DEPRECATED(5.3, "Replication pausing is deprecated and will be removed")
	uint32				bIsReplicationPaused:1;	// Replication is being paused, but channel will not be closed
	uint32				OpenTemporary:1;	// Opened temporarily.
	uint32				Broken:1;			// Has encountered errors and is ignoring subsequent packets.
	uint32				bTornOff:1;			// Actor associated with this channel was torn off
	uint32				bPendingDormancy:1;	// Channel wants to go dormant (it will check during tick if it can go dormant)
	uint32				bIsInDormancyHysteresis:1; // Channel wants to go dormant, and is otherwise ready to become dormant, but is waiting for a timeout before doing so.
	uint32				bPausedUntilReliableACK:1; // Unreliable property replication is paused until all reliables are ack'd.
	uint32				SentClosingBunch:1;	// Set when sending closing bunch to avoid recursion in send-failure-close case.
	uint32				bPooled:1;			// Set when placed in the actor channel pool
	uint32				OpenedLocally:1;	// Whether channel was opened locally or by remote.
	uint32				bOpenedForCheckpoint:1;	// Whether channel was opened by replay checkpoint recording
	uint32				bSupportsParallelTick:1; // Indicates if this channel can be ticked in parallel with other NetConnection::Tick steps. Set this in the Init function of derived class
	int32				ChIndex;			// Index of this channel.
	FPacketIdRange		OpenPacketId;		// If OpenedLocally is true, this is the packet we sent the bOpen bunch on. Otherwise, it's the packet we received the bOpen bunch on.
	FName				ChName;				// Name of the type of this channel.
	int32				NumInRec;			// Number of packets in InRec.
	int32				NumOutRec;			// Number of packets in OutRec.
	class FInBunch*		InRec;				// Incoming data with queued dependencies.
	class FOutBunch*	OutRec;				// Outgoing reliable unacked data.
	class FInBunch*		InPartialBunch;		// Partial bunch we are receiving (incoming partial bunches are appended to this)

	// UObject overrides

	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** UChannel interface. */
	/** Initialize this channel for the given connection and index. */
	ENGINE_API virtual void Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags);

    /** Reinitialize the channel with the existing NetConnection */
	virtual void ReInit() {}

	/** Set the closing flag. */
	ENGINE_API virtual void SetClosingFlag();

	/** Close the base channel. Returns how many bits were written to the send buffer */
	ENGINE_API virtual int64 Close(EChannelCloseReason Reason);

	/** Describe the channel. */
	ENGINE_API virtual FString Describe();

	/** Handle an incoming bunch. */
	virtual void ReceivedBunch( FInBunch& Bunch ) PURE_VIRTUAL(UChannel::ReceivedBunch,);
	
	/** Positive acknowledgment processing. */
	ENGINE_API virtual void ReceivedAck( int32 AckPacketId );

	/** Negative acknowledgment processing. */
	ENGINE_API virtual void ReceivedNak( int32 NakPacketId );
	
	/** Handle time passing on this channel. */
	ENGINE_API virtual void Tick();

	/** Return true to indicate that this channel no longer needs to Tick() every frame. */
	virtual bool CanStopTicking() const { return !bPendingDormancy; }

	// General channel functions.
	/** Handle an acknowledgment on this channel, returns true if the channel should be closed and fills in the OutCloseReason leaving it to the caller to cleanup the channel. Note: Temporary channels might be closed/cleaned-up by this call. */
	ENGINE_API bool ReceivedAcks(EChannelCloseReason& OutCloseReason);

	/** Handle an acknowledgment on this channel. Note: Channel might be closed/cleaned-up by this call. */
	ENGINE_API void ReceivedAcks();
	
	/** Process a properly-sequenced bunch. */
	ENGINE_API bool ReceivedSequencedBunch( FInBunch& Bunch );
	
	/** 
	 * Process a raw, possibly out-of-sequence bunch: either queue it or dispatch it.
	 * The bunch is sure not to be discarded.
	 */
	ENGINE_API void ReceivedRawBunch( FInBunch & Bunch, bool & bOutSkipAck );

	/** Append any export bunches */
	UE_DEPRECATED(5.5, "Use GetAdditionalRequiredBunches")
	ENGINE_API virtual void AppendExportBunches(TArray< FOutBunch* >& OutExportBunches);

	/** Returns any additional bunches (such as exports) needed by the passed-in OutgoingBunch */
	ENGINE_API virtual TArray<FOutBunch*> GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags = EChannelGetAdditionalRequiredBunchesFlags::None);

	/** Append any "must be mapped" guids to front of bunch. These are guids that the client will wait on before processing this bunch. */
	ENGINE_API virtual void AppendMustBeMappedGuids( FOutBunch* Bunch );

	/** Send a bunch if it's not overflowed, and queue it if it's reliable. */
	ENGINE_API virtual FPacketIdRange SendBunch(FOutBunch* Bunch, bool Merge);
	
	/** Return whether this channel is ready for sending. */
	UE_DEPRECATED(5.6, "Replaced with a version that doesn't have a saturate option.")
	ENGINE_API int32 IsNetReady( bool Saturate );

	/** Returns true if this channel is ready to send and has not exceeded the bandwidth limit yet. */
	ENGINE_API bool IsNetReady() const;

	/** Make sure the incoming buffer is in sequence and there are no duplicates. */
	ENGINE_API void AssertInSequenced();

	/** cleans up channel if it hasn't already been */
	ENGINE_API void ConditionalCleanUp(const bool bForDestroy, EChannelCloseReason CloseReason);

	/** Returns true if channel is ready to go dormant (e.g., all outstanding property updates have been ACK'd) */
	virtual bool ReadyForDormancy(bool suppressLogs=false) { return false; }

	/** Puts the channel in a state to start becoming dormant. It will not become dormant until ReadyForDormancy returns true in Tick */
	virtual void StartBecomingDormant() { }

	UE_DEPRECATED(5.3, "Will be removed from test/shipping in the future.")
	ENGINE_API void PrintReliableBunchBuffer();

	/* Notification that this channel has been placed in a channel pool and needs to reset to its original state so it can be used again like a new channel */
	ENGINE_API virtual void AddedToChannelPool();

	/** Returns true if this channel has flushed and acknowledged pending reliable data during a graceful close (when net.EnableGracefulClose is true) */
	ENGINE_API virtual bool HasAcknowledgedAllReliableData() const;

	/** Indicates if this channel can be ticked in parallel with other NetConnection::Tick steps*/
	bool SupportsParallelTick() const
	{
		return bSupportsParallelTick;
	}
	
protected:

	/** Closes the actor channel but with a 'dormant' flag set so it can be reopened */
	virtual void BecomeDormant() { }

	/** cleans up channel structures and nulls references to the channel */
	ENGINE_API virtual bool CleanUp( const bool bForDestroy, EChannelCloseReason CloseReason );

	/** Sets whether replication is currently paused on this channel or not */
	UE_DEPRECATED(5.3, "Replication pausing is deprecated and will be removed")
	virtual void SetReplicationPaused(bool InbIsReplicationPaused) 
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsReplicationPaused = InbIsReplicationPaused;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Returns whether replication is currently paused on this channel */
	UE_DEPRECATED(5.3, "Replication pausing is deprecated and will be removed")
	virtual bool IsReplicationPaused() 
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bIsReplicationPaused; 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:

	/** Just sends the bunch out on the connection */
	ENGINE_API int32 SendRawBunch(FOutBunch* Bunch, bool Merge, const FNetTraceCollector* Collector = nullptr);

	/** Final step to prepare bunch to be sent. If reliable, adds to acknowldege list. */
	ENGINE_API FOutBunch* PrepBunch(FOutBunch* Bunch, FOutBunch* OutBunch, bool Merge);

	/** Received next bunch to process. This handles partial bunches */
	ENGINE_API bool ReceivedNextBunch( FInBunch & Bunch, bool & bOutSkipAck );
};
