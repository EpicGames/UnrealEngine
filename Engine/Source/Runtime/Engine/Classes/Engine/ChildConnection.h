// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/NetConnection.h"
#include "Net/Core/Connection/NetCloseResult.h"
#include "ChildConnection.generated.h"

namespace UE::Net
{
	enum class ECloseChildFlags
	{
		None 				= 0,
		SendCloseMessage	= (1 << 0),	/* Send the NMT_CloseChildConnection control message to the remote endpoint (client/server). */
		DestroyActors		= (1 << 1)	/* Destroy all actors associated with the child connection. */
	};
	ENUM_CLASS_FLAGS(ECloseChildFlags);
}

/**
 * Represents a secondary split screen connection that reroutes calls to the parent connection.
 */
UCLASS(MinimalAPI,transient,config=Engine)
class UChildConnection
	: public UNetConnection
{
	GENERATED_UCLASS_BODY()
	using FNetCloseResult = UE::Net::FNetCloseResult;
	UPROPERTY(transient)
	TObjectPtr<class UNetConnection> Parent;

public:

	UNetConnection* GetParentConnection() { return Parent; }

	// UNetConnection interface.

	virtual UChildConnection* GetUChildConnection() override
	{
		return this;
	}

	virtual FString LowLevelGetRemoteAddress(bool bAppendPort=false) override
	{
		return Parent->LowLevelGetRemoteAddress(bAppendPort);
	}

	virtual FString LowLevelDescribe() override
	{
		return Parent->LowLevelDescribe();
	}

	virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override
	{
	}

	virtual void InitSendBuffer() override
	{
		Parent->InitSendBuffer();
	}

	virtual void AssertValid() override
	{
		Parent->AssertValid();
	}

	virtual void FlushNet(bool bIgnoreSimulation = false) override
	{
		Parent->FlushNet(bIgnoreSimulation);
	}

	UE_DEPRECATED(5.6, "Replaced with a version that doesn't have a saturate option.")
	virtual int32 IsNetReady(bool Saturate) override
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Parent->IsNetReady(Saturate);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual bool IsNetReady() const override
	{
		return Parent->IsNetReady();
	}

	virtual bool IsEncryptionEnabled() const override
	{
		return Parent->IsEncryptionEnabled();
	}

	virtual void Tick(float DeltaSeconds) override
	{
		SetConnectionState(Parent->GetConnectionState());
	}
	ENGINE_API virtual void GracefulClose(FNetCloseResult&& CloseReason) override;
	ENGINE_API virtual void HandleClientPlayer(class APlayerController* PC, class UNetConnection* NetConnection) override;
	ENGINE_API virtual void CleanUp() override;
	
	/** Close the child connection without closing the parent and any other child connections and remove it from it's parent. */
	ENGINE_API void CloseAndRemoveChild(UE::Net::ECloseChildFlags Flags);

	virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}
	virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}

	ENGINE_API virtual void InitChildConnection(UNetDriver* InDriver, UNetConnection* InParent);

	ENGINE_API virtual void SetProxyConnection(bool bEnabled) override;
	ENGINE_API virtual bool IsProxyConnection() const override;

private:
	void AssignConnectionHandle();

protected:
	virtual void RemoveLocalPlayer(UGameInstance* GameInstance, ULocalPlayer* LocalPlayer);
};
