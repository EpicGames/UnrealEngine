// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/PlatformInterfaceBase.h"
#include "CloudStorageBase.generated.h"

/** All the types of delegate callbacks that a CloudStorage subclass may receive from C++ */
UENUM()
enum ECloudStorageDelegate : int
{
	// @todo: Fill in the result descriptions for these (and the other PI subclasses)
	CSD_KeyValueReadComplete,
	CSD_KeyValueWriteComplete,
	CSD_ValueChanged,
	CSD_DocumentQueryComplete,
	CSD_DocumentReadComplete,
	CSD_DocumentWriteComplete,
	// Data: Document index that has the conflict
	// Type: Int
	/** Desc: Called when multiple machines have updated the document, and script needs to determine which one to use, via the Resolve functions. */
	CSD_DocumentConflictDetected,
	CSD_MAX,
};


/**
 * Base class for the various platform interface classes.
 */
UCLASS()
class UCloudStorageBase
	: public UPlatformInterfaceBase
{
	GENERATED_UCLASS_BODY()

	/** When using local storage (aka "cloud emulation"), this maintains a list of the file paths. */
	UPROPERTY()
	TArray<FString> LocalCloudFiles;

	/** If true, delegate callbacks should be skipped. */
	UPROPERTY()
	uint32 bSuppressDelegateCalls:1;

	/** Performs any initialization. */
	virtual void Init();

	/**
	 * Initiate reading a key/value pair from cloud storage. A CSD_KeyValueReadComplete.
	 * delegate will be called when it completes (if this function returns true).
	 * 
	 * @param KeyName String name of the key to retrieve.
	 * @param Type Type of data to retrieve .
	 * @param SerializedObj If you are reading an object (PIDT_Object), it will de-serialize the binary blob into this object.
	 * @return true on success, false otherwise.
	 */
	virtual bool ReadKeyValue(const FString& KeyName, EPlatformInterfaceDataType Type, class UObject* SerializedObj = nullptr);

	/**
	 * Write a key/value pair to the cloud. A CSD_KeyValueWriteComplete.
	 * delegate will be called when it completes (if this function returns true).
	 *
	 * @param KeyName String name of the key to write.
	 * @param Value The type and value to write.
	 * @return true on success, false otherwise.
	 */
	virtual bool WriteKeyValue(const FString& KeyName, const FPlatformInterfaceData& Value);

	/**
	 * Kick off an async query of documents that exist in the cloud. If any documents have
	 * already been retrieved, this will flush those documents, and refresh the set. A
	 * CSD_DocumentQueryComplete delegate will be called when it completes  (if this 
	 * function returns true). Then use GetNumCloudDocuments() and GetCloudDocumentName() 
	 * to get the information about any existing documents.
	 *
	 * @return true on success, false otherwise.
	 */
	virtual bool QueryForCloudDocuments();

	/**
	 * @return the number of documents that are known to exist in the cloud
	 */
	virtual int32 GetNumCloudDocuments(bool bIsForConflict = false);

	/**
	 * @return the name of the given cloud by index (up to GetNumCloudDocuments() - 1)
	 */
	virtual FString GetCloudDocumentName(int32 Index);

	/**
	 * Create a new document in the cloud (uninitialized, unsaved, use the Write/Save functions)
	 *
	 * @param Filename Filename for the cloud document (with any extension you want).
	 * @return Index of the new document, or -1 on failure.
	 */
	virtual int32 CreateCloudDocument(const FString& Filename);

	/**
	 * Reads a document into memory (or whatever is needed so that the ParseDocumentAs* functions
	 * operate synchronously without stalling the game). A CSD_DocumentReadComplete delegate
	 * will be called (if this function returns true).
	 *
	 * @param Index index of the document to read
	 * @param True if successful
	 */
	virtual bool ReadCloudDocument(int32 Index, bool bIsForConflict = false);

	/**
	 * Once a document has been read in, use this to return a string representing the 
	 * entire document. This should only be used if SaveDocumentWithString was used to
	 * generate the document.
	 *
	 * @param Index index of the document to read
	 * @return The document as a string. It will be empty if anything went wrong.
	 */
	virtual FString ParseDocumentAsString(int32 Index, bool bIsForConflict = false);

	/**
	 * Once a document has been read in, use this to return a string representing the.
	 * entire document. This should only be used if SaveDocumentWithString was used to generate the document.
	 *
	 * @param Index index of the document to read.
	 * @param ByteData The array of bytes to be filled out. It will be empty if anything went wrong.
	 */
	virtual void ParseDocumentAsBytes(int32 Index, TArray<uint8>& ByteData, bool bIsForConflict = false);

	/**
	 * Once a document has been read in, use this to return a string representing the
	 * entire document. This should only be used if SaveDocumentWithString was used to
	 * generate the document.
	 *
	 * @param Index index of the document to read.
	 * @param ExpectedVersion Version number expected to be in the save data. If this doesn't match what's there, this function will return nullptr.
	 * @param ObjectClass The class of the object to create.
	 * @return The object deserialized from the document. It will be none if anything went wrongs.
	 */
	virtual class UObject* ParseDocumentAsObject(int32 Index, TSubclassOf<class UObject> ObjectClass, int32 ExpectedVersion, bool bIsForConflict = false);

	/**
	 * Writes a document that has been already "saved" using the SaveDocumentWith* functions.
	 * A CSD_DocumentWriteComplete delegate will be called (if this function returns true).
	 *
	 * @param Index index of the document to read.
	 * @param True if successful.
	 */
	virtual bool WriteCloudDocument(int32 Index);

	/**
	 * Prepare a document for writing to the cloud with a string as input data. This is synchronous.
	 *
	 * @param Index index of the document to save.
	 * @param StringData The data to put into the document.
	 * @param True if successful
	 */
	virtual bool SaveDocumentWithString(int32 Index, const FString& StringData);

	/**
	 * Prepare a document for writing to the cloud with an array of bytes as input data. This is synchronous.
	 *
	 * @param Index index of the document to save.
	 * @param ByteData The array of generic bytes to put into the document.
	 * @param True if successful.
	 */
	virtual bool SaveDocumentWithBytes(int32 Index, const TArray<uint8>& ByteData);

	/**
	 * Prepare a document for writing to the cloud with an object as input data. This is synchronous.
	 *
	 * @param Index index of the document to save.
	 * @param ObjectData The object to serialize to bytes and put into the document.
	 * @param true on success, false otherwise.
	 */
	virtual bool SaveDocumentWithObject(int32 Index, class UObject* ObjectData, int32 SaveVersion);

	/**
	 * If there was a conflict notification, this will simply tell the cloud interface
	 * to choose the most recently modified version, and toss any others.
	 */
	virtual bool ResolveConflictWithNewestDocument();

	/**
	 * If there was a conflict notification, this will tell the cloud interface
	 * to choose the version with a given Index to be the primary version, and to
	 * toss any others.
	 *
	 * @param Index Conflict version index.
	 */
	virtual bool ResolveConflictWithVersionIndex(int32 Index);
};
