// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"

/**
 *	This will hold all of our enums and types and such that we need to
 *	use in multiple files where the enum can't be mapped to a specific file.
 */

#include "CollisionProfile.generated.h"

struct FBodyInstance;
struct FCollisionResponseParams;
struct FPropertyChangedEvent;

/** Structure representing a collision profile name, this gets a special UI in the editor */
USTRUCT(BlueprintType)
struct FCollisionProfileName
{
	GENERATED_USTRUCT_BODY()

	FCollisionProfileName()
		: Name(NAME_None)
	{}

	FCollisionProfileName(const FName InName)
		: Name(InName)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision)
	FName Name;
};


/**
 * Structure for collision response templates.
 */
USTRUCT()
struct FCollisionResponseTemplate
{
	GENERATED_USTRUCT_BODY()

	/** Name of collision profile */
	UPROPERTY()
	FName Name;

	/** Type of collision used for things with this profile */
	UPROPERTY()
	TEnumAsByte<ECollisionEnabled::Type> CollisionEnabled;

	/** Enum indicating what type of object this should be considered as when it moves */
	// no property anymore
	TEnumAsByte<enum ECollisionChannel> ObjectType;

	/** If this profile can be modified by games */
	UPROPERTY()
	bool	bCanModify;

	/** This is result of ResponseToChannel after loaded - please note that it is not property serializable **/
	struct FCollisionResponseContainer ResponseToChannels;

	/** Collision object type to use for things with this profile */
	UPROPERTY()
	FName ObjectTypeName;

	/** Types of objects that this physics objects will collide with. */
	UPROPERTY()
	TArray<FResponseChannel>	CustomResponses;

#if WITH_EDITORONLY_DATA
	/** Help message for collision profile **/
	UPROPERTY()
	FString HelpMessage;
#endif

	/** This constructor */
	ENGINE_API FCollisionResponseTemplate();

	ENGINE_API bool IsEqual(const TEnumAsByte<ECollisionEnabled::Type> InCollisionEnabled, 
		const TEnumAsByte<enum ECollisionChannel> InCollisionObjectType, 
		const struct FCollisionResponseContainer& InResponseToChannels);

	ENGINE_API void CreateCustomResponsesFromResponseContainers();
};


/**
 * Structure for custom channel setup information.
 */
USTRUCT()
struct FCustomChannelSetup
{
	GENERATED_USTRUCT_BODY()

	/** Which channel you'd like to customize **/
	UPROPERTY()
	TEnumAsByte<enum ECollisionChannel> Channel;

	/** Default Response for the channel */
	UPROPERTY()
	TEnumAsByte<enum ECollisionResponse> DefaultResponse;

	/** Sets meta data TraceType="1" for the enum entry if true. Otherwise, this channel will be treated as object query channel, so you can query object types**/
	UPROPERTY()
	bool bTraceType;
	
	/** Specifies if this is static object. Otherwise it will be dynamic object. This is used for query all objects vs all static objects vs all dynamic objects **/
	UPROPERTY()
	bool bStaticObject;	

	/** Name used in editor and metadata to refer to this channel */
	UPROPERTY()
	FName Name;

	FCustomChannelSetup()
		: Channel(ECC_WorldStatic)
		, DefaultResponse(ECR_Block)
		, bTraceType(false)
		, bStaticObject(false)
	{ }

	bool operator==(const FCustomChannelSetup& Other) const
	{
		return (Channel == Other.Channel);
	}
};


/**
 * Structure for custom profiles.
 *
 * if you'd like to just add custom channels, not changing anything else engine defined
 * if you'd like to override all about profile, please use 
 * +Profiles=(Name=NameOfProfileYouLikeToOverwrite,....)
 */
USTRUCT()
struct FCustomProfile
{
	GENERATED_USTRUCT_BODY()
	
	/** Name of new profile to add */
	UPROPERTY()
	FName Name;

	/** Types of objects that this physics objects will collide with. */
	UPROPERTY()
	TArray<FResponseChannel>	CustomResponses;
};


/**
 * Set up and modify collision settings.
 */
UCLASS(config=Engine, defaultconfig, MinimalAPI, meta=(DisplayName="Collision"))
class UCollisionProfile : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

private:

	/** List of all profiles, engine and game-specific */
	UPROPERTY(globalconfig)
	TArray<FCollisionResponseTemplate>	Profiles;

	/** Game-specific overrides to default responses to collision channels */
	UPROPERTY(globalconfig)
	TArray<FCustomChannelSetup>			DefaultChannelResponses;

	/** Game-specific overrides to engine profiles */
	UPROPERTY(globalconfig)
	TArray<FCustomProfile>				EditProfiles;

	/** Used to handle renaming profiles */
	UPROPERTY(globalconfig)
	TArray<FRedirector>					ProfileRedirects;

	/** Used to handle renaming collision channels */
	UPROPERTY(globalconfig)
	TArray<FRedirector>					CollisionChannelRedirects;

public:

	/** default property name for no collision - this is very popular **/
	static ENGINE_API const FName NoCollision_ProfileName;
	static ENGINE_API const FName BlockAll_ProfileName;
	static ENGINE_API const FName OverlapAll_ProfileName;
	static ENGINE_API const FName PhysicsActor_ProfileName;
	static ENGINE_API const FName BlockAllDynamic_ProfileName;
	static ENGINE_API const FName Pawn_ProfileName;
	static ENGINE_API const FName Vehicle_ProfileName;
	static ENGINE_API const FName DefaultProjectile_ProfileName;

	/** Accessor and initializer **/
	ENGINE_API static UCollisionProfile* Get();

	/** Holds a delegate to be invoked when LoadProfileConfig is called. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoadProfileConfig, UCollisionProfile*);
	FOnLoadProfileConfig OnLoadProfileConfig;

	/** Begin UObject interface */
	virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;
	/** End UObject interface */

	/** Fill up the array with the profile names **/
	ENGINE_API static void GetProfileNames(TArray<TSharedPtr<FName>>& OutNameList);

	/** Get the channel and response params from the specified profile */
	ENGINE_API static bool GetChannelAndResponseParams(FName ProfileName, ECollisionChannel &CollisionChannel, FCollisionResponseParams &ResponseParams);

	/** Fill up the loaded config of the profile name to the BodyInstance **/
	bool ReadConfig(FName ProfileName, struct FBodyInstance& BodyInstance) const;

	/** Fill up the loaded config of the profile name to the BodyInstance **/
	ENGINE_API bool GetProfileTemplate(FName ProfileName, struct FCollisionResponseTemplate& ProfileData) const;

	/** Check if this profile name has been redirected **/
	ENGINE_API const FName* LookForProfileRedirect(FName ProfileName) const;

	/** Accessor for UI customization **/
	int32 GetNumOfProfiles() const { return Profiles.Num(); }

	/** Accessor for UI customization **/
	ENGINE_API const FCollisionResponseTemplate* GetProfileByIndex(int32 Index) const;

	/** 
	 * This function loads all config data to memory
	 * 
	 * 1. First it fixes the meta data for each custom channel name since those meta data is used for #2
	 * 2. Second it sets up Correct ResponseToChannel for all profiles
	 * 3. It loads profile redirect data 
	 **/
	ENGINE_API void LoadProfileConfig(bool bForceInit=false);

	/** return index of Container index from DisplayName and vice versa**/
	/** this is misnamed now since I changed to update name if the name changed **/
	int32 ReturnContainerIndexFromChannelName(FName& InOutDisplayName)  const;
	ENGINE_API FName ReturnChannelNameFromContainerIndex(int32 ContainerIndex)  const;

	/** Convert ObjectType or TraceType to CollisionChannel */
	ENGINE_API ECollisionChannel ConvertToCollisionChannel(bool TraceType, int32 Index) const;

	/** 
	 * Convert collision channel to ObjectTypeQuery. Note: performs a search of object types.
	 * @return ObjectTypeQuery_MAX if the conversion was not possible 
	 */
	ENGINE_API EObjectTypeQuery ConvertToObjectType(ECollisionChannel CollisionChannel) const;

	/** 
	 * Convert collision channel to TraceTypeQuery. Note: performs a search of object types.
	 * @return TraceTypeQuery_MAX if the conversion was not possible 
	 */
	ENGINE_API ETraceTypeQuery ConvertToTraceType(ECollisionChannel CollisionChannel) const;

	/* custom collision profile name that you can modify what you'd like */
	ENGINE_API static FName CustomCollisionProfileName;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
private:

	/** 
	 * Profile redirects - later one overrides if same one found
	 */
	TMap<FName, FName> ProfileRedirectsMap;

	/** 
	 * Collision Channel Name redirects - later one overrides if same one found
	 */
	TMap<FName, FName> CollisionChannelRedirectsMap;

	/**
	 * Display Names for each channel
	 * I don't have meta data in cooked build, so I'll need to save the list 
	 */
	TArray<FName> ChannelDisplayNames;

	/**
	 * These are the mapping table converts from ObjectType/TraceType enum that are used for blueprint
	 * index [i] to ECollisionChannel. This is faster and quicker 
	 */
	TArray<ECollisionChannel> ObjectTypeMapping;
	TArray<ECollisionChannel> TraceTypeMapping;

	/** Get Profile Data from the list given **/
	bool FindProfileData(const TArray<FCollisionResponseTemplate>& ProfileList, FName ProfileName, struct FCollisionResponseTemplate& ProfileData) const;

	/** Check redirect and see if this needs to replace
	 * 
	 * returns true if it has been replaced
	 * returns false if it hasn't 
	 */
	bool CheckRedirect(FName ProfileName, FBodyInstance& BodyInstance, struct FCollisionResponseTemplate& Template) const;

	/**
	 * Fill up ResponseToChannels for ProfileList data from config
	 */
	void FillProfileData(TArray<FCollisionResponseTemplate>& ProfileList, const UEnum* CollisionChannelEnum, TArray<FCustomProfile>& EditProfileList);

	/**
	 * Load Custom Responses to the Template. Returns true if all are found and customized. False otherwise
	*/
	int32 LoadCustomResponses(FCollisionResponseTemplate& Template, const UEnum* CollisionChannelEnum, TArray<FResponseChannel>& CustomResponses) const;
	/**
	 * Saves Template.ResponseToChannel to CustomResponses
	 */
	ENGINE_API void SaveCustomResponses(FCollisionResponseTemplate& Template) const;

	ENGINE_API void AddChannelRedirect(FName OldName, FName NewName);
	ENGINE_API void AddProfileRedirect(FName OldName, FName NewName);

	friend class FCollisionProfileDetails;
	friend struct FCollisionProfilePrivateAccessor;
};

struct FCollisionProfilePrivateAccessor
{
	static ENGINE_API bool AddProfileTemplate(FCollisionResponseTemplate& NewProfileData);
};
