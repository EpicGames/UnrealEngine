// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "ComponentDelegateBinding.generated.h"

/** Entry for a delegate to assign after a blueprint has been instanced */
USTRUCT()
struct FBlueprintComponentDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	/** Name of component property that contains delegate we want to assign to. */
	UPROPERTY()
	FName ComponentPropertyName;

	/** Name of property on the component that we want to assign to. */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Name of function that we want to bind to the delegate. */
	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintComponentDelegateBinding()
		: ComponentPropertyName(NAME_None)
		, DelegatePropertyName(NAME_None)
		, FunctionNameToBind(NAME_None)
	{ }
};


UCLASS(MinimalAPI)
class UComponentDelegateBinding
	: public UDynamicBlueprintBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintComponentDelegateBinding> ComponentDelegateBindings;

	//~ Begin DynamicBlueprintBinding Interface
	ENGINE_API virtual void BindDynamicDelegates(UObject* InInstance) const override;
	ENGINE_API virtual void UnbindDynamicDelegates(UObject* InInstance) const override;
	ENGINE_API virtual void UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty) const override;
	//~ End DynamicBlueprintBinding Interface
};
