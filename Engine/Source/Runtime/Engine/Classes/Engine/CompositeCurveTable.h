// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveTable.h"
#include "CompositeCurveTable.generated.h"

/**
 * Curve table composed of a stack of other curve tables.
 */
UCLASS(MinimalAPI)
class UCompositeCurveTable
	: public UCurveTable
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface.
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	ENGINE_API virtual void EmptyTable() override;

	// Support for runtime modification of parent tables, 
	// Be aware this can be slow and can cause hitches during gameplay
	ENGINE_API void AppendParentTables(const TArray<UCurveTable*>& NewTables);

protected:

	// Searches the parent tables to see if there are any loops.
	// Returns a pointer to the first table found that depends on itself if a loop exists. Returns nullptr if no loops are found.
	const UCompositeCurveTable* FindLoops(TArray<const UCompositeCurveTable*> AlreadySeenTables) const;

	void UpdateCachedRowMap(bool bWarnOnInvalidChildren = true);

	void OnParentTablesUpdated(EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);

	// Parent tables
	// Tables with higher indices override data in tables with lower indices
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Tables)
	TArray<TObjectPtr<UCurveTable>> ParentTables;

	// temporary copy used to detect changes so we can update delegates correctly on removal
	UPROPERTY(transient)
	TArray<TObjectPtr<UCurveTable>> OldParentTables;

	// true if this asset is currently being loaded; false otherwise
	uint8 bIsLoading : 1;

	// true if we're already in the middle of updating parent tables for this asset
	uint8 bUpdatingParentTables : 1;
};
