// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTable.h"
#include "CompositeDataTable.generated.h"


/**
 * Data table composed of a stack of other data tables.
 */
UCLASS(MinimalAPI, BlueprintType, hideCategories=(ImportOptions,ImportSource))
class UCompositeDataTable
	: public UDataTable
{
	GENERATED_UCLASS_BODY()

	friend class UCompositeDataTableFactory;

	enum class ERowState : uint8
	{
		/** Inherited from one or more of the parent tables */
		Inherited,

		/** Inherited from one or more of the parent tables but overridden by the current table */
		Overridden,

		/** Added by the current table */
		New,

		Invalid,
	};

	//~ Begin UObject Interface.
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ERowState GetRowState(FName RowName) const;
	//~ End UObject Interface
#endif // WITH_EDITORONLY_DATA

	/** Table management overrides. Composite data tables don't currently add or remove rows. */
	ENGINE_API virtual void EmptyTable() override;
	ENGINE_API virtual void RemoveRow(FName RowName) override;
	ENGINE_API virtual void AddRow(FName RowName, const FTableRowBase& RowData) override;
	ENGINE_API virtual void AddRow(FName RowName, const uint8* RowData, const UScriptStruct* RowType) override;

#if WITH_EDITOR
	ENGINE_API virtual void CleanBeforeStructChange() override;
	ENGINE_API virtual void RestoreAfterStructChange() override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	// Support for runtime modification of parent tables
	// Be aware this can be slow and can cause hitches during gameplay
	ENGINE_API void AppendParentTables(const TArray<UDataTable*>& NewTables);
	ENGINE_API void RemoveParentTables(const TArray<UDataTable*>& TablesToRemove);
	ENGINE_API void AddParentTable(const TObjectPtr<UDataTable>& TableToAdd);
	ENGINE_API void RemoveParentTable(const TObjectPtr<UDataTable>& TableToRemove);

	ENGINE_API const TArray<UDataTable*>& GetParentTables() const;

protected:

	// Searches the parent tables to see if there are any loops.
	// Returns a pointer to the first table found that depends on itself if a loop exists. Returns nullptr if no loops are found.
	const UCompositeDataTable* FindLoops(TArray<const UCompositeDataTable*> AlreadySeenTables) const;

	// Empties the table
	// if bClearParentTables is false then the row map will be cleared but the parent table array won't be changed
	void EmptyCompositeTable(bool bClearParentTables);

	void UpdateCachedRowMap(bool bWarnOnInvalidChildren = true);

	void OnParentTablesUpdated(EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);

	// Parent tables
	// Tables with higher indices override data in tables with lower indices
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Tables)
	TArray<TObjectPtr<UDataTable>> ParentTables;

	// temporary copy used to detect changes so we can update delegates correctly on removal
	UPROPERTY(transient)
	TArray<TObjectPtr<UDataTable>> OldParentTables;

#if WITH_EDITORONLY_DATA
	TMap<FName, ERowState> RowSourceMap;
#endif // WITH_EDITORONLY_DATA

	// true if this asset is currently being loaded; false otherwise
	uint8 bIsLoading : 1;

	// true if we're already in the middle of updating parent tables for this asset
	uint8 bUpdatingParentTables : 1;

	// if this is true then the parent table array will not be cleared when EmptyTable is called
	uint8 bShouldNotClearParentTablesOnEmpty : 1;
};
