// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "ConsoleSettings.h"

#include "Console.generated.h"

class FViewport;
class SWidget;
struct FAutoCompleteCommand;
class FCanvasTextItem;
class FCanvasTileItem;
class UFont;

/**
 * Node for storing an auto-complete tree based on each char in the command.
 */
USTRUCT()
struct FAutoCompleteNode
{
	GENERATED_USTRUCT_BODY()

	/** Char for node in the tree */
	UPROPERTY()
	int32 IndexChar;

	/** Indices into AutoCompleteList for commands that match to this level */
	UPROPERTY()
	TArray<int32> AutoCompleteListIndices;

	/** Children for further matching */
	TArray<FAutoCompleteNode*> ChildNodes;

	FAutoCompleteNode()
	{
		IndexChar = INDEX_NONE;
	}

	FAutoCompleteNode(int32 NewChar)
	{
		IndexChar = NewChar;
	}

	~FAutoCompleteNode()
	{
		for (int32 Idx = 0; Idx < ChildNodes.Num(); Idx++)
		{
			FAutoCompleteNode *Node = ChildNodes[Idx];
			delete Node;
		}
		ChildNodes.Empty();
	}	
};


/**
 * A basic command line console that accepts most commands.
 */
UCLASS(Within=GameViewportClient, config=Input, transient, MinimalAPI)
class UConsole
	: public UObject
	, public FOutputDevice
{
	GENERATED_UCLASS_BODY()

	/** The player which the next console command should be executed in the context of.  If nullptr, execute in the viewport. */
	UPROPERTY()
	TObjectPtr<class ULocalPlayer> ConsoleTargetPlayer;

	UPROPERTY()
	TObjectPtr<class UTexture2D> DefaultTexture_Black;

	UPROPERTY()
	TObjectPtr<class UTexture2D> DefaultTexture_White;

	/** Holds the scrollback buffer */
	TArray<FString> Scrollback;

	/** Index to the next slot to be filled in the scrollback buffer. NOTE: The most recent line is at (SBHead-1). */
	int32 SBHead;

	/** Scrollback position offset (0 = view newest entries, increases to view older entries) */
	int32 SBPos;

	/** Max number of command history entries */
	static const int32 MAX_HISTORY_ENTRIES = 50;

	/** Holds the history buffer, order is old to new */
	UPROPERTY(config)
	TArray<FString> HistoryBuffer;

	/** The command the user is currently typing. */
	FString TypedStr;

	int32 TypedStrPos;    //Current position in TypedStr

	/** The command the user would get if they autocompleted their current input. Empty if no autocompletion entries are available. */
	FString PrecompletedInputLine;

	/** The most recent input that was autocompleted during this open console session. */
	FString LastAutoCompletedCommand;

	/**
	 * Indicates that InputChar events should be captured to prevent them from being passed on to other interactions.  Reset
	 * when the another keydown event is received.
	 */
	uint32 bCaptureKeyInput:1;

	/** True while a control key is pressed. */
	uint32 bCtrl:1;

	/** True while a shift key is pressed. */
	uint32 bShift:1;

	/** Full list of auto-complete commands and info */
	TArray<FAutoCompleteCommand> AutoCompleteList;

	/** Is the current auto-complete selection locked */
	uint32 bAutoCompleteLocked:1;

	/** Currently selected auto complete index */
	int32 AutoCompleteIndex;

	/** -1: auto complete cursor is not visible, >=0 */
	int32 AutoCompleteCursor;

	/** Do we need to rebuild auto complete? */
	uint32 bIsRuntimeAutoCompleteUpToDate:1;

	// NAME_Typing, NAME_Open or NAME_None
	FName ConsoleState;

	FAutoCompleteNode AutoCompleteTree;

	/** Current list of matching commands for auto-complete, @see UpdateCompleteIndices() */
	TArray<FAutoCompleteCommand> AutoComplete;

	// Scrollback region selection mouse and state
	struct ScrollbackSelection
	{
		FVector2f MousePosDown = FVector2f(0.0f, 0.0f);
		FVector2f MousePosUp = FVector2f(0.0f, 0.0f);
		FVector2f MousePos = FVector2f(0.0f, 0.0f);
		bool bActive = false;
		bool bCapture = false;
		bool bMade = false;
		float Offset = 0.0f;
	} Selection;

	float TextW = 15.0f;
	float TextH = 15.0f;

	ENGINE_API ~UConsole();

	// UObject Interface
	ENGINE_API virtual void PostInitProperties() override;

	/** Set the input to text */
	ENGINE_API virtual void SetInputText( const FString& Text );

	/** Set cursor position for typing text */
	ENGINE_API virtual void SetCursorPos( int32 Position );
	
	/**
	 * Executes a console command.
	 * @param Command The command to execute.
	 */
	ENGINE_API virtual void ConsoleCommand(const FString& Command);
	
	/**
	 * Clears the console output buffer.
	 */
	ENGINE_API virtual void ClearOutput();
	
	/**
	 * Prints a (potentially multi-line) FString of text to the console.
	 * The text is split into separate lines and passed to OutputTextLine.
	 *
	 * @param Text Text to display on the console.
	 */
	ENGINE_API virtual void OutputText(const FString& Text);
	
	/**
	 * Opens the typing bar with text already entered.
	 *
	 * @param Text The text to enter in the typing bar.
	 */
	ENGINE_API virtual void StartTyping(const FString& Text);
	
	/**
	 * Clears out all pressed keys from the player's input object.
	 */
	ENGINE_API virtual void FlushPlayerInput();
	
	/** looks for Control key presses and the copy/paste combination that apply to both the console bar and the full open console */
	ENGINE_API virtual bool ProcessControlKey(FKey Key, EInputEvent Event);

	/** looks for Shift key presses */
	ENGINE_API virtual bool ProcessShiftKey(FKey Key, EInputEvent Event);
	
	/** appends the specified text to the string of typed text */
	ENGINE_API virtual void AppendInputText(const FString& Text);
	
	/** Build the list of auto complete console commands */
	ENGINE_API virtual void BuildRuntimeAutoCompleteList(bool bForce = false);

	/** Virtual function to allow subclasses of UConsole to add additional commands */
	ENGINE_API virtual void AugmentRuntimeAutoCompleteList(TArray<FAutoCompleteCommand>& List);
	
	/** Update the auto complete indices from the typed string, optionally using a hint about whether our search term has just gotten more specific (i.e. narrowing down the search) */
	ENGINE_API virtual void UpdateCompleteIndices(bool bIsSearchNarrowing = false);

	/**
	 * Process a character input event (typing) routed through unrealscript from another object. This method is assigned as the value for the
	 * OnReceivedNativeInputKey delegate so that native input events are routed to this unrealscript function.
	 *
	 * @param	DeviceId		the input device that generated this character input event
	 * @param	Unicode			the character that was typed
	 *
	 * @return	True to consume the character, false to pass it on.
	 */
	ENGINE_API virtual bool InputChar_Typing(FInputDeviceId DeviceId, const FString& Unicode);
	
	/**
	 * perform rendering of the console on the canvas
	 */
	ENGINE_API virtual void PostRender_Console_Typing(class UCanvas* Canvas);
	
	/** Perform actions on transition to Typing state */
	ENGINE_API virtual void BeginState_Typing(FName PreviousStateName);

	/** Perform actions on transition from Typing state */
	ENGINE_API virtual void EndState_Typing( FName NextStateName );

	/**
	 * This state is used when the console is open.
	 */
	ENGINE_API virtual bool InputChar_Open( FInputDeviceId DeviceId, const FString& Unicode );

	/**
	 * Process an input key event routed through unrealscript from another object. This method is assigned as the value for the
	 * OnReceivedNativeInputKey delegate so that native input events are routed to this unrealscript function.
	 *
	 * @param	ControllerId	the controller that generated this input key event
	 * @param	Key				the name of the key which an event occured for (KEY_Up, KEY_Down, etc.)
	 * @param	EventType		the type of event which occured (pressed, released, etc.)
	 * @param	AmountDepressed	for analog keys, the depression percent.
	 *
	 * @return	true to consume the key event, false to pass it on.
	 */
	ENGINE_API virtual bool InputKey_Open(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed = 1.f, bool bGamepad = false);
	
	/** 
	 * perform rendering of the console on the canvas
	 */
	ENGINE_API virtual void PostRender_Console_Open(class UCanvas* Canvas);

	/** Perform actions on transition to the Open state */
	ENGINE_API virtual void BeginState_Open(FName PreviousStateName);

	/** Perform actions on transition from Open state */
	ENGINE_API virtual void EndState_Open(FName NextStateName);
	
	ENGINE_API virtual bool InputChar(FInputDeviceId DeviceId, const FString& Unicode);
	
	ENGINE_API virtual bool InputKey(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed=1.f, bool bGamepad=false);
	
	virtual bool InputAxis(FInputDeviceId DevideId, FKey Key, float Delta, float DeltaTime, int32 NumSamples=1, bool bGamepad=false) { return false; };

	UE_DEPRECATED(5.8, "Use the overload that uses FTouchId instead")
	virtual bool InputTouch(const FInputDeviceId DevideId, const uint32 Handle, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force, const FDateTime DeviceTimestamp, const uint32 TouchpadIndex) final
	{
		return InputTouch(FTouchId(DevideId, static_cast<ETouchIndex::Type>(Handle)), Type, TouchLocation, Force, DeviceTimestamp.GetTicks());
	}

	/**
	 * Allow the console to process a touch event received by the viewport-client.
	 * If the console uses the event, it should return true.  If true is returned, the viewport-client will not offer it to the next candidate.
	 * 
	 * @param	TouchId - Identifier unique to this ongoing touch (i.e. device + finger).
	 * @param	Type - What kind of touch event this is (see ETouchType).
	 * @param	TouchLocation - Viewport-relative position of the touch.
	 * @param	Force - How hard the touch is.
	 * @param	Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the key event, false to pass it on.
	 */
	virtual bool InputTouch(const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force, const uint64 Timestamp)
	{
		return false;
	}

	ENGINE_API virtual void MouseMove(FViewport* Viewport, int32 X, int32 Y);
	ENGINE_API virtual void CapturedMouseMove(FViewport* InViewport, int32 X, int32 Y);

	/** render to the canvas based on the console state */
	ENGINE_API virtual void PostRender_Console(class UCanvas* Canvas);

	/** controls state transitions for the console */
	ENGINE_API virtual void FakeGotoState(FName NextStateName);

	ENGINE_API virtual bool ConsoleActive() const;

	ENGINE_API void InvalidateAutocomplete();

	/** Delegate for registering hot-reloaded classes that have been added  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FRegisterConsoleAutoCompleteEntries, TArray<FAutoCompleteCommand>&);
	static ENGINE_API FRegisterConsoleAutoCompleteEntries RegisterConsoleAutoCompleteEntries;

	/** Delegate for when the console is activated or deactivated */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConsoleActivationStateChanged, bool);
	static ENGINE_API FOnConsoleActivationStateChanged OnConsoleActivationStateChanged;

private:

	bool InputKey_InputLine(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed = 1.f, bool bGamepad = false);

	// interface FOutputDevice
	ENGINE_API virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
	// expose the base class other Serialize function (clang will give "error : 'UConsole::Serialize' hides overloaded virtual function" without this)
	using UObject::Serialize; 

	/**
	 * Prints a single line of text to the console.
	 * @param Text - A line of text to display on the console.
	 */
	void OutputTextLine(const FString& Text);

	void PostRender_InputLine(class UCanvas* Canvas, FIntPoint UserInputLinePos);

	void SetAutoCompleteFromHistory();

	void SetInputLineFromAutoComplete();

	void UpdatePrecompletedInputLine();

	void NormalizeHistoryBuffer();

	// Scrollback region selection utilities
	void DrawLine(
		UCanvas* Canvas, UFont* Font, FCanvasTileItem& TileBlack, FCanvasTileItem& TileWhite,
		FCanvasTextItem ConsoleText, const FString& Line, float PenX, float PenY, float PosLeft,
		TArray<FString>& SelectedLines);
	void GetPosInTextLine(UCanvas* Canvas, UFont* Font, const FString& Line, float MousePosX, float& PosInLine, int32& PosInString);
	float GetPosTextLineEnd(UCanvas* Canvas, UFont* Font, const FString& Line);

	// Console settings from BaseInput.ini
	const UConsoleSettings* ConsoleSettings;

	// Widget that was focused before the console was opened (focus will be restored to this if it's valid after the console closes)
	TWeakPtr<SWidget> PreviousFocusedWidget;

	// Previous mouse capture mode of the viewport before the console was opened (mouse capture mode must be restored to its previous state)
	EMouseCaptureMode PreviousMouseCaptureMode = EMouseCaptureMode::CapturePermanently;

	const bool bDPIAwareStringMeasurement = true;
};
