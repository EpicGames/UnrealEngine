// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"

/** Project specific configuration for content encryption */
class FContentEncryptionConfig
{
public:

	enum class EAllowedReferences
	{
		None,
		Soft,
		All,
	};

	enum class EGroupType
	{
		Root,
		Explicit
	};

	struct FGroup
	{
		TSet<FName> PackageNames;
		TSet<FString> NonAssetFiles;

		// Allow this group to track all assets under a specific set of mount points
		TSet<FName> MountPoints;

		EGroupType GroupType = EGroupType::Root;
		EAllowedReferences AllowedReferences = EAllowedReferences::None;
		int32 DesiredChunkId = INDEX_NONE;
	};

	typedef TMap<FName, FGroup> TGroupMap;

	void AddPackage(FName InGroupName, FName InPackageName)
	{
		PackageGroups.FindOrAdd(InGroupName).PackageNames.Add(InPackageName);
	}

	void AddNonAssetFile(FName InGroupName, const FString& InFilename)
	{
		PackageGroups.FindOrAdd(InGroupName).NonAssetFiles.Add(InFilename);
	}

	void AddMountPoint(FName InGroupName, FName InMountPoint)
	{
		PackageGroups.FindOrAdd(InGroupName).MountPoints.Add(InMountPoint);
	}

	void SetGroupType(FName InGroupName, EGroupType InGroupType)
	{
		PackageGroups.FindOrAdd(InGroupName).GroupType = InGroupType;
	}

	void SetAllowedReferences(FName InGroupName, EAllowedReferences InAllowedReferences)
	{
		PackageGroups.FindOrAdd(InGroupName).AllowedReferences = InAllowedReferences;
	}

	void SetDesiredChunkId(FName InGroupName, int32 InChunkId)
	{
		PackageGroups.FindOrAdd(InGroupName).DesiredChunkId = InChunkId;
	}

	void AddReleasedKey(FGuid InKey)
	{
		ReleasedKeys.Add(InKey);
	}

	const TGroupMap& GetPackageGroupMap() const
	{
		return PackageGroups;
	}


	const TSet<FGuid>& GetReleasedKeys() const
	{
		return ReleasedKeys;
	}

	void DissolveGroups(const TSet<FName>& InGroupsToDissolve)
	{
		for (FName GroupName : InGroupsToDissolve)
		{
			if (PackageGroups.Contains(GroupName))
			{
				PackageGroups.FindOrAdd(NAME_None).PackageNames.Append(PackageGroups.Find(GroupName)->PackageNames);
				PackageGroups.Remove(GroupName);
			}
		}
	}

private:

	TGroupMap PackageGroups;
	TSet<FGuid> ReleasedKeys;
};