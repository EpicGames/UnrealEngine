// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/CoreNet.h"
#include "Engine/Channel.h"
#include "ControlChannel.generated.h"

class FInBunch;
class FOutBunch;
class UNetConnection;
struct FActorDestructionInfo;

/**
 * A queued control channel message
 */
struct FQueuedControlMessage
{
	/** The raw message data */
	TArray<uint8> Data;

	/** The bit size of the message */
	uint32 CountBits;

public:
	/**
	 * Base constructor
	 */
	FQueuedControlMessage()
		: Data()
		, CountBits(0)
	{
	}
};

/**
 * A channel for exchanging connection control messages.
 */
UCLASS(transient, customConstructor, MinimalAPI)
class UControlChannel
	: public UChannel
{
	GENERATED_UCLASS_BODY()

	/**
	 * Used to interrogate the first packet received to determine endianess
	 * of the sending client
	 */
	bool bNeedsEndianInspection;

	/** 
	 * provides an extra buffer beyond RELIABLE_BUFFER for control channel messages
	 * as we must be able to guarantee delivery for them
	 * because they include package map updates and other info critical to client/server synchronization
	 */
	TArray<FQueuedControlMessage> QueuedMessages;

	/** maximum size of additional buffer
	 * if this is exceeded as well, we kill the connection.  @TODO FIXME temporarily huge until we figure out how to handle 1 asset/package implication on packagemap
	 */
	enum { MAX_QUEUED_CONTROL_MESSAGES = 32768 };

	/**
	 * Inspects the packet for endianess information. Validates this information
	 * against what the client sent. If anything seems wrong, the connection is
	 * closed
	 *
	 * @param Bunch the packet to inspect
	 *
	 * @return true if the packet is good, false otherwise (closes socket)
	 */
	ENGINE_API bool CheckEndianess(FInBunch& Bunch);

	/** adds the given bunch to the QueuedMessages list. Closes the connection if MAX_QUEUED_CONTROL_MESSAGES is exceeded */
	ENGINE_API void QueueMessage(const FOutBunch* Bunch);

	/**
	 * Default constructor
	 */
	UControlChannel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: UChannel(ObjectInitializer)
	{
		ChName = NAME_Control;
	}

	ENGINE_API virtual void Init( UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags ) override;

	//~ Begin UChannel Interface.
	ENGINE_API virtual FPacketIdRange SendBunch(FOutBunch* Bunch, bool Merge) override;

	ENGINE_API virtual void Tick() override;

	/** Always tick the control channel for now. */
	virtual bool CanStopTicking() const override { return false; }
	//~ End UChannel Interface.


	/** Handle an incoming bunch. */
	ENGINE_API virtual void ReceivedBunch( FInBunch& Bunch ) override;

	/** Describe the text channel. */
	ENGINE_API virtual FString Describe() override;

	/** Sends a message to destroy a specific actor without creating an actor channel. */
	ENGINE_API int64 SendDestructionInfo(FActorDestructionInfo* DestructionInfo);

	/** Handle receiving a destruction message for an actor outside of an actor channel. */
	ENGINE_API void ReceiveDestructionInfo(FInBunch& Bunch);
};
