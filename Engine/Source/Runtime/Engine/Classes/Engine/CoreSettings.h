// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"

#include "CoreSettings.generated.h"

struct FPropertyChangedEvent;

/**
 * Streaming settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Streaming"), MinimalAPI)
class UStreamingSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	ENGINE_API UStreamingSettings();

protected:

	UPROPERTY(config, EditAnywhere, Category = PackageStreaming, meta = (
		ConsoleVariable = "s.AsyncLoadingThreadEnabled", DisplayName = "Async Loading Thread Enabled",
		ToolTip = "Enables separate thread for package streaming. Requires restart to take effect."))
	uint32 AsyncLoadingThreadEnabled : 1;

	UPROPERTY(config, EditAnywhere, Category = PackageStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.WarnIfTimeLimitExceeded", DisplayName = "Warn If Time Limit Has Been Exceeded",
		ToolTip = "Enables log warning if time limit for time-sliced package streaming has been exceeded."))
	uint32 WarnIfTimeLimitExceeded : 1;

	UPROPERTY(config, EditAnywhere, Category = PackageStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.TimeLimitExceededMultiplier", DisplayName = "Time Limit Exceeded Warning Multiplier",
		ToolTip = "Multiplier for time limit exceeded warning time threshold."))
	float TimeLimitExceededMultiplier;

	UPROPERTY(config, EditAnywhere, Category = PackageStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.TimeLimitExceededMinTime", DisplayName = "Minimum Time Limit For Time Limit Exceeded Warning",
		ToolTip = "Minimum time the time limit exceeded warning will be triggered by."))
	float TimeLimitExceededMinTime;

	UPROPERTY(config, EditAnywhere, Category = PackageStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.MinBulkDataSizeForAsyncLoading", DisplayName = "Minimum Bulk Data Size For Async Loading",
		ToolTip = "Minimum time the time limit exceeded warning will be triggered by."))
	int32 MinBulkDataSizeForAsyncLoading;

	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, meta = (
		ConsoleVariable = "s.UseBackgroundLevelStreaming", DisplayName = "Use Background Level Streaming",
		ToolTip = "Whether to allow background level streaming."))
	uint32 UseBackgroundLevelStreaming:1;

	/** Whether to use the entire time limit even if blocked on I/O */
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.AsyncLoadingUseFullTimeLimit", DisplayName = "Async Loading Use Full Time Limit",
		ToolTip = "Whether to use the entire time limit even if blocked on I/O."))
	uint32 AsyncLoadingUseFullTimeLimit:1;

	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.AsyncLoadingTimeLimit", DisplayName = "Async Loading Time Limit",
		ToolTip = "Maximum amount of time to spend doing asynchronous loading (ms per frame)."))
	float AsyncLoadingTimeLimit;

	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.PriorityAsyncLoadingExtraTime", DisplayName = "Priority Async Loading Extra Time",
		ToolTip = "Additional time to spend asynchronous loading during a high priority load."))
	float PriorityAsyncLoadingExtraTime;

	/** Maximum allowed time to spend for actor registration steps during level streaming (ms per frame)*/
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.LevelStreamingActorsUpdateTimeLimit", DisplayName = "Actor Initialization Update Time Limit",
		ToolTip = "Maximum allowed time to spend for actor registration steps during level streaming (ms per frame)."))
	float LevelStreamingActorsUpdateTimeLimit;

	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.PriorityLevelStreamingActorsUpdateExtraTime", DisplayName = "Priority Actor Initialization Update Extra Time",
		ToolTip = "Additional time to spend on actor registration steps during a high priority load."))
	float PriorityLevelStreamingActorsUpdateExtraTime;

	/** Batching granularity used to register actor components during level streaming */
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.LevelStreamingComponentsRegistrationGranularity", DisplayName = "Components Registration Granularity",
		ToolTip = "Batching granularity used to register actor components during level streaming."))
	int32 LevelStreamingComponentsRegistrationGranularity;

	/** Batching granularity used to add primitives to scene in parallel when registering actor components during level streaming */
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.LevelStreamingAddPrimitiveGranularity", DisplayName = "Added Primitive Granularity",
		ToolTip = "Batching granularity used to add primitives to scene in parallel when registering actor components during level streaming."))
	int32 LevelStreamingAddPrimitiveGranularity;

	/** Maximum allowed time to spend while unregistering components during level streaming (ms per frame) */
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.UnregisterComponentsTimeLimit", DisplayName = "Component Unregister Update Time Limit",
		ToolTip = "Maximum allowed time to spend while unregistering components during level streaming (ms per frame)."))
		float LevelStreamingUnregisterComponentsTimeLimit;

	/** Batching granularity used to unregister actor components during level streaming */
	UPROPERTY(EditAnywhere, config, Category = LevelStreaming, AdvancedDisplay, meta = (
		ConsoleVariable = "s.LevelStreamingComponentsUnregistrationGranularity", DisplayName = "Components Unregistration Granularity",
		ToolTip = "Batching granularity used to unregister actor components during level streaming."))
	int32 LevelStreamingComponentsUnregistrationGranularity;

	UPROPERTY(EditAnywhere, config, Category = General, meta = (
		ConsoleVariable = "s.FlushStreamingOnExit", DisplayName = "Flush Streaming when exiting the application",
		ToolTip = "If enabled, streaming will be flushed when exiting application, otherwise it will be cancelled."))
	uint32 FlushStreamingOnExit : 1;

	UPROPERTY(config, EditAnywhere, Category = "Deprecated Settings", meta = (
		ConsoleVariable = "s.EventDrivenLoaderEnabled", DisplayName = "Use Event Driven Loader (disabling not recommended)",
		ToolTip = "Enables the event driven loader in cooked builds (default). Disabling EDL will result in using deprecated loading path."))
		uint32 EventDrivenLoaderEnabled : 1;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;

	//~ End UObject Interface
};

/** Whether to use unified TimeBudget to manage ProcessAsyncLoading and UpdateLevelStreaming */
extern ENGINE_API int32 GUseUnifiedTimeBudgetForStreaming;
/** Whether to allow background level streaming. */
extern ENGINE_API int32 GUseBackgroundLevelStreaming;
/** Maximum amount of time to spend doing asynchronous loading (ms per frame). */
extern ENGINE_API float GAsyncLoadingTimeLimit;
/** Whether to use the entire time limit even if blocked on I/O. */
extern ENGINE_API int32 GAsyncLoadingUseFullTimeLimit;
/** Additional time to spend asynchronous loading during a high priority load. */
extern ENGINE_API float GPriorityAsyncLoadingExtraTime;
/** Maximum allowed time to spend for actor registration steps during level streaming (ms per frame). */
extern ENGINE_API float GLevelStreamingActorsUpdateTimeLimit;
/** Additional time to spend on actor registration steps during a high priority load. */
extern ENGINE_API float GPriorityLevelStreamingActorsUpdateExtraTime;
/** Batching granularity used to register actor components during level streaming. */
extern ENGINE_API int32 GLevelStreamingComponentsRegistrationGranularity;
/** Batching granularity used to add primitives to scene in parallel when registering actor components during level streaming. */
extern ENGINE_API int32 GLevelStreamingAddPrimitiveGranularity;
/** Batching granularity used to unregister actor components during level streaming.  */
extern ENGINE_API int32 GLevelStreamingComponentsUnregistrationGranularity;
/** Batching granularity used to initialize actors during level streaming. If this is zero, we process all actors and stages in one pass. */
extern ENGINE_API int32 GLevelStreamingRouteActorInitializationGranularity;
/** Batching granularity used to call EndPlay on actors when removing streaming level. If this is zero, we process all actors in one pass. */
extern ENGINE_API int32 GLevelStreamingRouteActorEndPlayForRemoveFromWorldGranularity;
/** Maximum allowed time to spend for actor unregistration steps during level streaming (ms per frame). If this is 0.0 then we don't timeslice.*/
extern ENGINE_API float GLevelStreamingUnregisterComponentsTimeLimit;
/** Whether to force a GC after levels are streamed out to instantly reclaim the memory at the expensive of a hitch. */
extern ENGINE_API int32 GLevelStreamingForceGCAfterLevelStreamedOut;
/** Whether to kick off incremental GC when there are over the specified amount of levels still waiting to be purged. */
extern ENGINE_API int32 GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge;
/** Whether to kick off incremental GC when there are over the specified amount of actors (from levels) still waiting to be purged. */
extern ENGINE_API int32 GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge;
/** Enables level streaming requests while async loading (of anything) while the match is already in progress and no loading screen is up. */
extern ENGINE_API int32 GLevelStreamingAllowLevelRequestsWhileAsyncLoadingInMatch;
/** When we're already loading this many levels and actively in match, don't allow any more requests until one of those completes.  Set to zero to disable. */
extern ENGINE_API int32 GLevelStreamingMaxLevelRequestsAtOnceWhileInMatch;
/** Whether to force a verification of objects residing in a GC'ed level package (ignored in shipping builds). */
extern ENGINE_API int32 GLevelStreamingForceVerifyLevelsGotRemovedByGC;
/** Whether to force routing actor initialize phase in its own frame. */
extern ENGINE_API int32 GLevelStreamingForceRouteActorInitializeNextFrame;
/** In a low memory situation, will kick off a 'soft' GC if the number of streaming levels pending purge meets or exceeds this count. */
extern ENGINE_API int32 GLevelStreamingLowMemoryPendingPurgeCount;
/** In a low memory situation, will kick off a 'soft' GC if the number of actors (from streaming levels) pending purge meets or exceeds this count. */
extern ENGINE_API int32 GLevelStreamingLowMemoryActorsPendingPurgeCount;

/**
* Implements the settings for garbage collection.
*/
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Garbage Collection"), MinimalAPI)
class UGarbageCollectionSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	ENGINE_API UGarbageCollectionSettings();

protected:

	UPROPERTY(EditAnywhere, config, Category = General, meta = (
		ConsoleVariable = "gc.TimeBetweenPurgingPendingKillObjects", DisplayName = "Time Between Purging Pending Kill Objects",
		ToolTip = "Time in seconds (game time) we should wait between purging object references to objects that are pending kill."))
	float TimeBetweenPurgingPendingKillObjects;

	UPROPERTY(EditAnywhere, config, Category = General, meta = (
		ConsoleVariable = "gc.FlushStreamingOnGC", DisplayName = "Flush Streaming On GC",
		ToolTip = "If enabled, streaming will be flushed each time garbage collection is triggered."))
	uint32 FlushStreamingOnGC : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.AllowParallelGC", DisplayName = "Allow Parallel GC",
		ToolTip = "If enabled, garbage collection will use multiple threads."))
	uint32 AllowParallelGC : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.IncrementalBeginDestroyEnabled", DisplayName = "Incremental BeginDestroy Enabled",
		ToolTip = "If true, the engine will destroy objects incrementally using time limit each frame."))
	uint32 IncrementalBeginDestroyEnabled : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.CreateGCClusters", DisplayName = "Create Garbage Collector UObject Clusters",
		ToolTip = "If true, the engine will attempt to create clusters of objects for better garbage collection performance."))
	uint32 CreateGCClusters : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.AssetClustreringEnabled", DisplayName = "Asset Clustering Enabled",
		ToolTip = "Whether to allow asset files to create actor clusters for GC."))
	uint32 AssetClusteringEnabled : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.ActorClusteringEnabled", DisplayName = "Actor Clustering Enabled",
		ToolTip = "Whether to allow levels to create actor clusters for GC."))
	uint32 ActorClusteringEnabled : 1;

	UPROPERTY(EditAnywhere, config, Category = Debug, meta = (
		ConsoleVariable = "gc.VerifyUObjectsAreNotFGCObjects", DisplayName = "Verify UObjects Are Not FGCObjects",
		ToolTip = "If true, the engine will throw a warning when it detects a UObject-derived class which also derives from FGCObject or any of its members is derived from FGCObject."))
	uint32 VerifyUObjectsAreNotFGCObjects : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.GarbageEliminationEnabled", DisplayName = "Garbage Elimination Enabled",
		ToolTip = "If true, objects marked as Garbage will be automatically nulled and destroyed by Garbage Collector."))
	uint32 GarbageEliminationEnabled : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.DumpObjectCountsToLogWhenMaxObjectLimitExceeded", DisplayName = "Log UObject counts when UObject limit has been exceeded",
		ToolTip = "If true, the engine will dump UObject counts when maximum number of UObjects limit has been exceeded."))
	uint32 DumpObjectCountsToLogWhenMaxObjectLimitExceeded : 1;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.MinGCClusterSize", DisplayName = "Minimum GC Cluster size",
		ToolTip = "Minimum GC cluster size."))
	int32 MinGCClusterSize;

	UPROPERTY(EditAnywhere, config, Category = General, meta = (
		ConsoleVariable = "gc.NumRetriesBeforeForcingGC", DisplayName = "Number Of Retries Before Forcing GC",
		ToolTip = "Maximum number of times GC can be skipped if worker threads are currently modifying UObject state. 0 = never force GC"))
	int32 NumRetriesBeforeForcingGC;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.MaxObjectsNotConsideredByGC", DisplayName = "Maximum Object Count Not Considered By GC",
		ToolTip = "Maximum Object Count Not Considered By GC. Works only in cooked builds."))
	int32 MaxObjectsNotConsideredByGC;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.MaxObjectsInGame", DisplayName = "Maximum number of UObjects that can exist in cooked game",
		ToolTip = "Maximum number of UObjects that can exist in cooked game. Keep this as small as possible."))
	int32 MaxObjectsInGame;

	UPROPERTY(EditAnywhere, config, Category = Optimization, meta = (
		ConsoleVariable = "gc.MaxObjectsInEditor", DisplayName = "Maximum number of UObjects that can exist in the editor",
		ToolTip = "Maximum number of UObjects that can exist in the editor game. Make sure this can hold enough objects for the editor and commadlets within reasonable limit."))
	int32 MaxObjectsInEditor;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface
};
