// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "CullDistanceVolume.generated.h"

class UPrimitiveComponent;

/**
 * Helper structure containing size and cull distance pair.
 */
USTRUCT(BlueprintType)
struct FCullDistanceSizePair
{
	GENERATED_USTRUCT_BODY()

	/** Size to associate with cull distance. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CullDistanceSizePair)
	float Size;

	/** Cull distance associated with size. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CullDistanceSizePair)
	float CullDistance;


	FCullDistanceSizePair()
		: Size(0)
		, CullDistance(0)
	{ }

	FCullDistanceSizePair(float InSize, float InCullDistance)
		: Size(InSize)
		, CullDistance(InCullDistance)
	{ }
};


UCLASS(hidecategories=(Advanced, Attachment, Collision, Volume), MinimalAPI)
class ACullDistanceVolume
	: public AVolume
{
	GENERATED_UCLASS_BODY()

	/**
	 * Array of size and cull distance pairs. The code will calculate the sphere diameter of a primitive's BB and look for a best
	 * fit in this array to determine which cull distance to use.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CullDistanceVolume)
	TArray<struct FCullDistanceSizePair> CullDistances;

	/**
	 * Whether the volume is currently enabled or not.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CullDistanceVolume)
	uint32 bEnabled:1;

public:

	// UObject Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void Destroyed() override;
#endif // WITH_EDITOR

	/**
	 * Returns whether the passed in primitive can be affected by cull distance volumes.
	 *
	 * @param	PrimitiveComponent	Component to test
	 * @return	true if tested component can be affected, false otherwise
	 */
	static bool CanBeAffectedByVolumes( UPrimitiveComponent* PrimitiveComponent );

	/** 
	  * Get the set of primitives and new max draw distances defined by this volume. 
	  * Presumes only primitives that can be affected by volumes are being passed in.
	  */
	void GetPrimitiveMaxDrawDistances(TMap<UPrimitiveComponent*,float>& OutCullDistances);
};
