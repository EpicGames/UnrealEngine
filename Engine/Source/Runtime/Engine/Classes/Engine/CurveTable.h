// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/SimpleCurve.h"
#include "CurveTable.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogCurveTable, Log, All);


// forward declare JSON writer
template <class CharType>
struct TPrettyJsonPrintPolicy;
template <class CharType, class PrintPolicy>
class TJsonWriter;

/**
* Whether the curve table contains simple, rich, or no curves
*/
UENUM()
enum class ECurveTableMode : uint8
{
	Empty,
	SimpleCurves,
	RichCurves
};


/**
 * Imported spreadsheet table as curves.
 */
UCLASS(MinimalAPI, Meta = (LoadBehavior = "LazyOnDemand"))
class UCurveTable
	: public UObject
	, public FCurveOwnerInterface
{
	GENERATED_UCLASS_BODY()

	DECLARE_TS_MULTICAST_DELEGATE(FOnCurveTableChanged);

	const TMap<FName, FRealCurve*>& GetRowMap() const { return RowMap; }
	const TMap<FName, FRealCurve*>& GetRowMap() { return RowMap; }

	const TMap<FName, FRichCurve*>& GetRichCurveRowMap() const { check(CurveTableMode != ECurveTableMode::SimpleCurves); return *reinterpret_cast<const TMap<FName, FRichCurve*>*>(&RowMap); }
	const TMap<FName, FRichCurve*>& GetRichCurveRowMap() { check(CurveTableMode != ECurveTableMode::SimpleCurves); return *reinterpret_cast<TMap<FName, FRichCurve*>*>(&RowMap); }

	const TMap<FName, FSimpleCurve*>& GetSimpleCurveRowMap() const { check(CurveTableMode != ECurveTableMode::RichCurves); return *reinterpret_cast<const TMap<FName, FSimpleCurve*>*>(&RowMap); }
	const TMap<FName, FSimpleCurve*>& GetSimpleCurveRowMap() { check(CurveTableMode != ECurveTableMode::RichCurves);  return *reinterpret_cast<TMap<FName, FSimpleCurve*>*>(&RowMap); }

	ECurveTableMode GetCurveTableMode() const { return CurveTableMode; }

	/** Removes a single row from the CurveTable by name. Just returns if row is not found. */
	ENGINE_API virtual void RemoveRow(FName RowName);
	ENGINE_API FRichCurve& AddRichCurve(FName RowName);
	ENGINE_API FSimpleCurve& AddSimpleCurve(FName RowName);

	/** Move the curve to another FName in the table */
	ENGINE_API void RenameRow(const FName& CurveName, const FName& NewCurveName);

	/** Remove a curve row from the table.  Note the associated curve will be deleted. */
	ENGINE_API void DeleteRow(const FName& CurveName);

protected:
	/** 
	 * Map of name of row to row data structure. 
	 * If CurveTableMode is SimpleCurves the value type will be FSimpleCurve*
	 * If ECurveTableMode is RichCurves the value type will be FRichCurve*
	 */
	TMap<FName, FRealCurve*>	RowMap;

	static FTransactionallySafeCriticalSection& GetCurveTableChangeCriticalSection();

public:
	//~ Begin UObject Interface.
	virtual void FinishDestroy() override;
	virtual void Serialize( FArchive& Ar ) override;

#if WITH_EDITORONLY_DATA
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;

	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** The filename imported to create this object. Relative to this object's package, BaseDir() or absolute */
	UPROPERTY()
	FString ImportPath_DEPRECATED;

	/** Interpolation mode persisted for reimport. */
	UPROPERTY(VisibleAnywhere, Category=ImportSettings)
	TEnumAsByte<ERichCurveInterpMode> ImportCurveInterpMode = RCIM_None;

#endif	// WITH_EDITORONLY_DATA

	//~ End  UObject Interface

	//~ Begin FCurveOwnerInterface Interface.
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual void ModifyOwner() override;
	virtual void MakeTransactional() override;
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;
	virtual TArray<const UObject*> GetOwners() const override;
	virtual bool RepointCurveOwner(const FPackageReloadedEvent& InPackageReloadedEvent, FCurveOwnerInterface*& OutNewCurveOwner) const override
	{
		return RepointCurveOwnerAsset(InPackageReloadedEvent, this, OutNewCurveOwner);
	}
	virtual bool HasRichCurves() const override
	{
		return CurveTableMode != ECurveTableMode::SimpleCurves;
	}
	//~ End FCurveOwnerInterface Interface.

	/** Gets a multicast delegate that is called any time the curve table changes. */
	FOnCurveTableChanged& OnCurveTableChanged() { return OnCurveTableChangedDelegate; }

	//~ Begin UCurveTable Interface

	/** Function to find the row of a table given its name. */
	FRealCurve* FindCurve(FName RowName, const FString& ContextString, bool bWarnIfNotFound = true) const
	{
		if (RowName.IsNone())
		{
			UE_CLOGF(bWarnIfNotFound, LogCurveTable, Warning, "UCurveTable::FindCurve : NAME_None is invalid row name for CurveTable '%ls' (%ls).", *GetPathName(), *ContextString);
			return nullptr;
		}

		FRealCurve* const* FoundCurve = RowMap.Find(RowName);

		if (FoundCurve == nullptr)
		{
			UE_CLOGF(bWarnIfNotFound, LogCurveTable, Warning, "UCurveTable::FindCurve : Row '%ls' not found in CurveTable '%ls' (%ls).", *RowName.ToString(), *GetPathName(), *ContextString);
			return nullptr;
		}

		return *FoundCurve;
	}

	FRichCurve* FindRichCurve(FName RowName, const FString& ContextString, bool bWarnIfNotFound = true) const
	{
		if (CurveTableMode == ECurveTableMode::SimpleCurves)
		{
			UE_LOGF(LogCurveTable, Error, "UCurveTable::FindCurve : Using FindRichCurve on CurveTable '%ls' (%ls) that is storing simple curves.", *GetPathName(), *ContextString);
			return nullptr;
		}

		return (FRichCurve*)FindCurve(RowName, ContextString, bWarnIfNotFound);
	}

	FSimpleCurve* FindSimpleCurve(FName RowName, const FString& ContextString, bool bWarnIfNotFound = true) const
	{
		if (CurveTableMode == ECurveTableMode::RichCurves)
		{
			UE_LOGF(LogCurveTable, Error, "UCurveTable::FindCurve : Using FindSimpleCurve on CurveTable '%ls' (%ls) that is storing rich curves.", *GetPathName(), *ContextString);
			return nullptr;
		}

		return (FSimpleCurve*)FindCurve(RowName, ContextString, bWarnIfNotFound);
	}

	/** High performance version with no type safety */
	FRealCurve* FindCurveUnchecked(FName RowName) const
	{
		// If RowName is none, it won't be found in the map
		FRealCurve* const* FoundCurve = RowMap.Find(RowName);

		if (FoundCurve == nullptr)
		{
			return nullptr;
		}

		return *FoundCurve;
	}

	/** Output entire contents of table as a string */
	ENGINE_API FString GetTableAsString() const;

	/** Output entire contents of table as CSV */
	ENGINE_API FString GetTableAsCSV() const;

	/** Output entire contents of table as JSON */
	ENGINE_API FString GetTableAsJSON() const;

	/** Output entire contents of table as JSON. bAsArray true will write is as a JSON array, false will write it as a series of named objects*/
	template <typename CharType = TCHAR>
	bool WriteTableAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter,bool bAsArray = true) const;

	/** 
	 *	Create table from CSV style comma-separated string. 
	 *	RowCurve must be defined before calling this function. 
	 *  @param InString The string representing the CurveTable
	 *  @param InterpMode The mode of interpolation to use for the curves
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromCSVString(const FString& InString, ERichCurveInterpMode InterpMode = RCIM_Linear);

	/** 
	 *	Create table from JSON string. 
	 *	RowCurve must be defined before calling this function. 
	 *  @param InString The string representing the CurveTable
	 *  @param InterpMode The mode of interpolation to use for the curves
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromJSONString(const FString& InString, ERichCurveInterpMode InterpMode = RCIM_Linear);
	
	/** 
	 *	Create table from another Curve Table
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromOtherTable(const UCurveTable* InTable);

	/** Empty the table info (will not clear RowCurve) */
	ENGINE_API virtual void EmptyTable();

	ENGINE_API static void InvalidateAllCachedCurves();

	static int32 GetGlobalCachedCurveID()
	{
		return GlobalCachedCurveID;
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:

	/** Util that removes invalid chars and then make an FName
	 * @param InString The string to create a valid name from
	 * @return A valid name from InString
	 */
	static FName MakeValidName(const FString& InString);

	ENGINE_API static int32 GlobalCachedCurveID;


private:

	/** A multicast delegate that is called any time the curve table changes. */
	FOnCurveTableChanged OnCurveTableChangedDelegate;

protected:
	ECurveTableMode CurveTableMode;
};


/**
 * Handle to a particular row in a table.
 */
USTRUCT(BlueprintType)
struct FCurveTableRowHandle
{
	GENERATED_USTRUCT_BODY()

	FCurveTableRowHandle()
		: CurveTable(nullptr)
		, RowName(NAME_None)
	{ }

	/** Pointer to table we want a row from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CurveTableRowHandle, meta=(DisplayThumbnail="false"))
	TObjectPtr<const class UCurveTable>	CurveTable;

	/** Name of row in the table that we want */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CurveTableRowHandle)
	FName				RowName;

	/** Returns true if the curve is valid */
	bool IsValid(const FString& ContextString) const
	{
		return (GetCurve(ContextString, false) != nullptr);
	}

	/** Returns true if this handle is specifically pointing to nothing */
	bool IsNull() const
	{
		return CurveTable == nullptr && RowName.IsNone();
	}

	/** Get the curve straight from the row handle */
	ENGINE_API FRealCurve* GetCurve(const FString& ContextString, bool bWarnIfNotFound=true) const;

	/** Get the rich curve straight from the row handle */
	ENGINE_API FRichCurve* GetRichCurve(const FString& ContextString, bool bWarnIfNotFound=true) const;

	/** Get the simple curve straight from the row handle */
	ENGINE_API FSimpleCurve* GetSimpleCurve(const FString& ContextString, bool bWarnIfNotFound = true) const;

	/** Evaluate the curve if it is valid
	 * @param XValue The input X value to the curve
	 * @param ContextString A string to provide context for where this operation is being carried out
	 * @return The value of the curve if valid, 0 if not
	 */
	float Eval(float XValue,const FString& ContextString) const
	{
		float Result = 0.f;
		Eval(XValue, &Result, ContextString);
		return Result;
	}

	/** Evaluate the curve if it is valid
	 * @param XValue The input X value to the curve
	 * @param YValue The output Y value from the curve
	 * @param ContextString A string to provide context for where this operation is being carried out
	 * @return True if it filled out YValue with a valid number, false otherwise
	 */
	ENGINE_API bool Eval(float XValue, float* YValue,const FString& ContextString) const;

	ENGINE_API bool operator==(const FCurveTableRowHandle& Other) const;
	ENGINE_API bool operator!=(const FCurveTableRowHandle& Other) const;
	ENGINE_API void PostSerialize(const FArchive& Ar);

	/** Used so we can have a TMap of this struct */
	inline friend uint32 GetTypeHash(const FCurveTableRowHandle& Handle)
	{
		return HashCombine(GetTypeHash(Handle.RowName), PointerHash(Handle.CurveTable));
	}
};

template<>
struct TStructOpsTypeTraits< FCurveTableRowHandle > : public TStructOpsTypeTraitsBase2< FCurveTableRowHandle >
{
	enum
	{
		WithPostSerialize = true,
	};
};

/** Macro to call GetCurve with a correct error info. Assumed to be called within a UObject */
#define GETCURVE_REPORTERROR(Handle) Handle.GetCurve(FString::Printf(TEXT("%s.%s"), *GetPathName(), TEXT(#Handle)))

/** Macro to call GetCurve with a correct error info. */
#define GETCURVE_REPORTERROR_WITHPATHNAME(Handle, PathNameString) Handle.GetCurve(FString::Printf(TEXT("%s.%s"), *PathNameString, TEXT(#Handle)))
