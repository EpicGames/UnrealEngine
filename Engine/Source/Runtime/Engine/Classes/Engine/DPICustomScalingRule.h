// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DPICustomScalingRule.generated.h"

/**
 * Custom Scaling Rules for Slate and UMG Widgets can be implemented by sub-classing from this class
 * and setting this rule to be used in your project settings.
 */
UCLASS(Abstract, MinimalAPI)
class UDPICustomScalingRule : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Return the scale to use given the size of the viewport.
	 * @param Size The size of the viewport.
	 * @return The Scale to apply to the entire UI.
	 */
	ENGINE_API virtual float GetDPIScaleBasedOnSize(FIntPoint Size) const;
};
