// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "Engine/HitResult.h"
#include "DamageEvents.generated.h"

class UDamageType;

/** Event used by AActor::TakeDamage and related functions */
USTRUCT(BlueprintType)
struct FDamageEvent
{
	GENERATED_BODY()

public:
	/** Default constructor (no initialization). */
	FDamageEvent() { }

	FDamageEvent(FDamageEvent const& InDamageEvent)
		: DamageTypeClass(InDamageEvent.DamageTypeClass)
	{ }
	
	virtual ~FDamageEvent() = default;

	explicit FDamageEvent(TSubclassOf<UDamageType> InDamageTypeClass)
		: DamageTypeClass(InDamageTypeClass)
	{ }

	/** Optional DamageType for this event.  If nullptr, UDamageType will be assumed. */
	UPROPERTY()
	TSubclassOf<UDamageType> DamageTypeClass;

	/** ID for this class. NOTE this must be unique for all damage events. */
	static const int32 ClassID = 0;

	virtual int32 GetTypeID() const { return FDamageEvent::ClassID; }
	virtual bool IsOfType(int32 InID) const { return FDamageEvent::ClassID == InID; };

	/** This is for compatibility with old-style functions which want a unified set of hit data regardless of type of hit.  Ideally this will go away over time. */
	ENGINE_API virtual void GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const;
};

/** Damage subclass that handles damage with a single impact location and source direction */
USTRUCT()
struct FPointDamageEvent : public FDamageEvent
{
	GENERATED_BODY()

	/** Actual damage done */
	UPROPERTY()
	float Damage;
	
	/** Direction the shot came from. Should be normalized. */
	UPROPERTY()
	FVector_NetQuantizeNormal ShotDirection;
	
	/** Describes the trace/location that caused this damage */
	UPROPERTY()
	FHitResult HitInfo;

	FPointDamageEvent() : Damage(0.0f), ShotDirection(ForceInitToZero), HitInfo() {}
	FPointDamageEvent(float InDamage, const FHitResult& InHitInfo, FVector const& InShotDirection, TSubclassOf<UDamageType> InDamageTypeClass)
		: FDamageEvent(InDamageTypeClass), Damage(InDamage), ShotDirection(InShotDirection), HitInfo(InHitInfo)
	{}
	
	/** ID for this class. NOTE this must be unique for all damage events. */
	static const int32 ClassID = 1;
	
	virtual int32 GetTypeID() const override { return FPointDamageEvent::ClassID; };
	virtual bool IsOfType(int32 InID) const override { return (FPointDamageEvent::ClassID == InID) || FDamageEvent::IsOfType(InID); };

	/** Simple API for common cases where we are happy to assume a single hit is expected, even though damage event may have multiple hits. */
	ENGINE_API virtual void GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const override;
};

/** Parameters used to compute radial damage */
USTRUCT(BlueprintType)
struct FRadialDamageParams
{
	GENERATED_BODY()

	/** Max damage done */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialDamageParams)
	float BaseDamage;

	/** Damage will not fall below this if within range */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialDamageParams)
	float MinimumDamage;
	
	/** Within InnerRadius, do max damage */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialDamageParams)
	float InnerRadius;
		
	/** Outside OuterRadius, do no damage */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialDamageParams)
	float OuterRadius;
		
	/** Describes amount of exponential damage falloff */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialDamageParams)
	float DamageFalloff;

	FRadialDamageParams()
		: BaseDamage(0.f), MinimumDamage(0.f), InnerRadius(0.f), OuterRadius(0.f), DamageFalloff(1.f)
	{}
	FRadialDamageParams(float InBaseDamage, float InInnerRadius, float InOuterRadius, float InDamageFalloff)
		: BaseDamage(InBaseDamage), MinimumDamage(0.f), InnerRadius(InInnerRadius), OuterRadius(InOuterRadius), DamageFalloff(InDamageFalloff)
	{}
	FRadialDamageParams(float InBaseDamage, float InMinimumDamage, float InInnerRadius, float InOuterRadius, float InDamageFalloff)
		: BaseDamage(InBaseDamage), MinimumDamage(InMinimumDamage), InnerRadius(InInnerRadius), OuterRadius(InOuterRadius), DamageFalloff(InDamageFalloff)
	{}
	FRadialDamageParams(float InBaseDamage, float InRadius)
		: BaseDamage(InBaseDamage), MinimumDamage(0.f), InnerRadius(0.f), OuterRadius(InRadius), DamageFalloff(1.f)
	{}

	/** Returns damage done at a certain distance */
	ENGINE_API float GetDamageScale(float DistanceFromEpicenter) const;

	/** Return outermost radius of the damage area. Protects against malformed data. */
	float GetMaxRadius() const { return FMath::Max( FMath::Max(InnerRadius, OuterRadius), 0.f ); }
};

/** Damage subclass that handles damage with a source location and falloff radius */
USTRUCT()
struct FRadialDamageEvent : public FDamageEvent
{
	GENERATED_BODY()

	/** Static parameters describing damage falloff math */
	UPROPERTY()
	FRadialDamageParams Params;
	
	/** Location of origin point */
	UPROPERTY()
	FVector Origin;

	/** Hit reslts of specific impacts */
	UPROPERTY()
	TArray<FHitResult> ComponentHits;

	/** ID for this class. NOTE this must be unique for all damage events. */
	static const int32 ClassID = 2;

	virtual int32 GetTypeID() const override { return FRadialDamageEvent::ClassID; };
	virtual bool IsOfType(int32 InID) const override { return (FRadialDamageEvent::ClassID == InID) || FDamageEvent::IsOfType(InID); };

	/** Simple API for common cases where we are happy to assume a single hit is expected, even though damage event may have multiple hits. */
	ENGINE_API virtual void GetBestHitInfo(AActor const* HitActor, AActor const* HitInstigator, FHitResult& OutHitInfo, FVector& OutImpulseDir) const override;

	FRadialDamageEvent()
		: Origin(ForceInitToZero)
	{}
};


