// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetRegistry/AssetBundleData.h"
#include "Templates/SubclassOf.h"
#include "DataAsset.generated.h"

/**
 * Create a simple asset that stores data related to a particular system in an instance of this class.
 * Assets can be made in the Content Browser using any native class that inherits from this.
 * If you want data inheritance or a complicated hierarchy, Data Only Blueprint Classes should be created instead.
 */
UCLASS(abstract, MinimalAPI, Meta = (LoadBehavior = "LazyOnDemand"))
class UDataAsset : public UObject
{
	GENERATED_BODY()
public:
	// UObject interface
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void Serialize(FStructuredArchiveRecord Record) override;
#endif

#if WITH_EDITOR
	ENGINE_API virtual FText GetDisplayNameText() const;
#endif

	/** Asset registry tag NativeClass with full path in export form Class'/Script/ModuleName.ClassName' of the first found parent native class */
	static ENGINE_API const FName NativeClassTag;
};

/**
 * A DataAsset that implements GetPrimaryAssetId and has asset bundle support, which allows it to be manually loaded/unloaded from the AssetManager.
 * Instances of native subclasses can be created directly as Data Assets in the editor and will use the name of the native class as the PrimaryAssetType.
 * Or, blueprint subclasses can be created to add variables and then subclassed again by Data Only Blueprints that set those variables.
 * With blueprint subclasses, use Data Only Blueprints (and not Data Asset instances) to properly handle data inheritance and updating the parent class.
 *
 * The PrimaryAssetType will be equal to the name of the first native class going up the hierarchy, or the highest level blueprint class.
 * IE, if you have UPrimaryDataAsset -> UParentNativeClass -> UChildNativeClass -> DataOnlyBlueprintClass the type will be ChildNativeClass.
 * Whereas if you have UPrimaryDataAsset -> ParentBlueprintClass -> DataOnlyBlueprintClass the type will be ParentBlueprintClass.
 * To change this behavior, override GetPrimaryAssetId in your native class or copy those functions into a different native base class.
 */
UCLASS(abstract, MinimalAPI, Blueprintable)
class UPrimaryDataAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	// UObject interface
	ENGINE_API virtual FPrimaryAssetId GetPrimaryAssetId() const override;
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	/** This scans the class for AssetBundles metadata on asset properties and initializes the AssetBundleData with InitializeAssetBundlesFromMetadata */
	ENGINE_API virtual void UpdateAssetBundleData();

	/** Updates AssetBundleData */
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

protected:
	/** Asset Bundle data computed at save time. In cooked builds this is accessible from AssetRegistry */
	UPROPERTY()
	FAssetBundleData AssetBundleData;
#endif
};
