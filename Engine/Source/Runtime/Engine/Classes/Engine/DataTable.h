// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "DataTableUtils.h"
#include "DataTable.generated.h"

class Error;
class UDataTable;
struct FDataTableEditorUtils;
class UGameplayTagTableManager;
class FDataTableImporterCSV;
class FDataTableImporterJSON;
template <class CharType> struct TPrettyJsonPrintPolicy;

// forward declare JSON writer
template <class CharType>
struct TPrettyJsonPrintPolicy;
template <class CharType, class PrintPolicy>
class TJsonWriter;


/**
 * Base class for all table row structs to inherit from.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FTableRowBase
{
	GENERATED_USTRUCT_BODY()

	FTableRowBase() { }
	virtual ~FTableRowBase() = default;

	/** 
	 * Can be overridden by subclasses; Called whenever the owning data table is imported or re-imported.
	 * Allows for custom fix-ups, parsing, etc. after initial data is read in.
	 * 
	 * @param InDataTable					The data table that owns this row
	 * @param InRowName						The name of the row we're performing fix-up on
	 * @param OutCollectedImportProblems	List of problems accumulated during import; Can be added to via this method
	 */
	virtual void OnPostDataImport(const UDataTable* InDataTable, const FName InRowName, TArray<FString>& OutCollectedImportProblems) {}

	/**
	 * Can be overridden by subclasses; Called on every row when the owning data table is modified
	 * Allows for custom fix-ups, parsing, etc for user changes
	 * This will be called in addition to OnPostDataImport when importing
	 *
	 * @param InDataTable					The data table that owns this row
	 * @param InRowName						The name of the row we're performing fix-up on
	 */
	virtual void OnDataTableChanged(const UDataTable* InDataTable, const FName InRowName) {}

#if WITH_EDITOR
	/** Generic function to validate data table row.
	 *
	 * @param	Context	the context holding validation warnings/errors.
	 * @return	Valid if this table row has data validation rules set up for it and the data for this object is valid.
	 *			Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const { return EDataValidationResult::NotValidated; }
#endif // WITH_EDITOR
};


/**
 * Imported spreadsheet table.
 */
UCLASS(MinimalAPI, BlueprintType, AutoExpandCategories = "DataTable,ImportOptions", Meta = (LoadBehavior = "LazyOnDemand"))
class UDataTable
	: public UObject
{
	GENERATED_UCLASS_BODY()

	DECLARE_TS_MULTICAST_DELEGATE(FOnDataTableChanged);
	DECLARE_MULTICAST_DELEGATE(FOnDataTableImport);
	
	friend FDataTableEditorUtils;
	friend UGameplayTagTableManager;
	friend FDataTableImporterCSV;
	friend FDataTableImporterJSON;

	/** Structure to use for each row of the table, must inherit from FTableRowBase */
	UPROPERTY(VisibleAnywhere, Category=DataTable, meta=(DisplayThumbnail="false"))
	TObjectPtr<UScriptStruct>			RowStruct;

protected:
	/** Map of name of row to row data structure. */
	TMap<FName, uint8*>		RowMap;

	// TODO: remove this, it is temporarily here to allow DataTableEditorUtils to compile until I get around to updating functions like RemoveRow and RenameRow
	virtual TMap<FName, uint8*>& GetNonConstRowMap() { return RowMap; }

	/** Called to add rows to the data table */
	ENGINE_API virtual void AddRowInternal(FName RowName, uint8* RowDataPtr);

	/** Deletes the row memory */
	ENGINE_API virtual void RemoveRowInternal(FName RowName);
public:

	virtual const TMap<FName, uint8*>& GetRowMap() const { return RowMap; }
	virtual const TMap<FName, uint8*>& GetRowMap() { return RowMap; }

	const UScriptStruct* GetRowStruct() const { return RowStruct; }

	/** Returns true if it is valid to import multiple table rows with the same name; returns false otherwise */
	virtual bool AllowDuplicateRowsOnImport() const { return false; }

	/** Set to true to not cook this data table into client builds. Useful for sensitive tables that only servers should know about. */
	UPROPERTY(EditAnywhere, Category=DataTable)
	uint8 bStripFromClientBuilds : 1;

	/** Set to true to ignore extra fields in the import data, if false it will warn about them */
	UPROPERTY(EditAnywhere, Category=ImportOptions)
	uint8 bIgnoreExtraFields : 1;

	/** Set to true to ignore any fields that are expected but missing, if false it will warn about them */
	UPROPERTY(EditAnywhere, Category = ImportOptions)
	uint8 bIgnoreMissingFields : 1;

	/** Set to true to preserve existing values for any fields that are expected but missing in the CSV file. If false, missing fields will be populated with default values. */
	UPROPERTY(EditAnywhere, Category = ImportOptions)
	uint8 bPreserveExistingValues : 1;

	/** Explicit field in import data to use as key. If this is empty it uses Name for JSON and the first field found for CSV */
	UPROPERTY(EditAnywhere, Category=ImportOptions)
	FString ImportKeyField;
	
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
protected:
	ENGINE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const override;
public:
#endif // WITH_EDITOR

	//~ Begin UObject Interface.
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void Serialize(FStructuredArchiveRecord Record) override;
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual bool NeedsLoadForClient() const override { return bStripFromClientBuilds ? false : Super::NeedsLoadForClient(); }
	virtual bool NeedsLoadForEditorGame() const override { return bStripFromClientBuilds ? false : Super::NeedsLoadForEditorGame(); }
#if WITH_EDITORONLY_DATA
	ENGINE_API FTopLevelAssetPath GetRowStructPathName() const;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	/** The file this data table was imported from, may be empty */
	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSource)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** The filename imported to create this object. Relative to this object's package, BaseDir() or absolute */
	UPROPERTY()
	FString ImportPath_DEPRECATED;

	/** The name of the RowStruct we were using when we were last saved */
	UPROPERTY()
	FName RowStructName_DEPRECATED;

	/** The name of the RowStruct we were using when we were last saved */
	UPROPERTY()
	FTopLevelAssetPath RowStructPathName;

protected:
	/** When RowStruct is being modified, row data is stored serialized with tags */
	UPROPERTY(Transient)
	TArray<uint8> RowsSerializedWithTags;

	UPROPERTY(Transient)
	TSet<TObjectPtr<UObject>> TemporarilyReferencedObjects;
#endif	// WITH_EDITORONLY_DATA

private:
	/** A multicast delegate that is called any time the data table changes. */
	FOnDataTableChanged OnDataTableChangedDelegate;

public:
	/** Gets a multicast delegate that is called any time the data table changes. */
	FOnDataTableChanged& OnDataTableChanged() { return OnDataTableChangedDelegate; }

	/** 
	 * Call whenever the data of a table has changed, this calls the OnDataTableChanged() delegate and per-row callbacks.
	 * If ChangedRowName is not none, only that row was changed, otherwise assume all rows have changed
	 */
	ENGINE_API void HandleDataTableChanged(FName ChangedRowName = NAME_None);
	
	/** Get all of the rows in the table, regardless of name */
	template <class T>
	void GetAllRows(const TCHAR* ContextString, OUT TArray<T*>& OutRowArray) const
	{
		if (CommonTypeCheck(T::StaticStruct(), TEXT("UDataTable::GetAllRows"), ContextString))
		{
			OutRowArray.Reserve(OutRowArray.Num() + GetRowMap().Num());
			for (TMap<FName, uint8*>::TConstIterator RowMapIter(GetRowMap().CreateConstIterator()); RowMapIter; ++RowMapIter)
			{
				OutRowArray.Add(reinterpret_cast<T*>(RowMapIter.Value()));
			}
		}	
	}

	template <class T>
	void GetAllRows(const FString& ContextString, OUT TArray<T*>& OutRowArray) const
	{
		GetAllRows<T>(*ContextString, OutRowArray);
	}

	/** Function to find the row of a table given its name. */
	template <class T>
	T* FindRow(FName RowName, const TCHAR* ContextString, bool bWarnIfRowMissing = true) const
	{
		return reinterpret_cast<T*>(FindRowInternal(T::StaticStruct(), RowName, ContextString, bWarnIfRowMissing));
	}

	template <class T>
	T* FindRow(FName RowName, const FString& ContextString, bool bWarnIfRowMissing = true) const
	{
		return FindRow<T>(RowName, *ContextString, bWarnIfRowMissing);
	}

	/** Perform some operation for every row. */
	template <class T>
	void ForeachRow(const TCHAR* ContextString, TFunctionRef<void (const FName& Key, const T& Value)> Predicate) const
	{
		if (CommonTypeCheck(T::StaticStruct(), TEXT("UDataTable::ForeachRow"), ContextString))
		{
			for (TMap<FName, uint8*>::TConstIterator RowMapIter(GetRowMap().CreateConstIterator()); RowMapIter; ++RowMapIter)
			{
				T* Entry = reinterpret_cast<T*>(RowMapIter.Value());
				Predicate(RowMapIter.Key(), *Entry);
			}
		}
	}

	template <class T>
	void ForeachRow(const FString& ContextString, TFunctionRef<void (const FName& Key, const T& Value)> Predicate) const
	{
		ForeachRow<T>(*ContextString, Predicate);
	}

	/** Returns the column property where PropertyName matches the name of the column property. Returns nullptr if no match is found or the match is not a supported table property */
	ENGINE_API FProperty* FindTableProperty(const FName& PropertyName) const;

	/** High performance version with no type safety */
	uint8* FindRowUnchecked(FName RowName) const
	{
		if(RowStruct == nullptr)
		{
			return nullptr;
		}

		// If is RowName is none, it won't find anything in the map
		uint8* const* RowDataPtr = GetRowMap().Find(RowName);

		if(RowDataPtr == nullptr)
		{
			return nullptr;
		}

		return *RowDataPtr;
	}

	/** 
	 * Get the value in the datatable at the given row and column. 
	 * The provided function should get the value from the property type and the cell data
	 */
	template<typename TPropertyType, typename TReturnType>
	const TReturnType* GetCellValue(const FName& RowName, const FName& ColumnName, TFunction<const TReturnType*(const TPropertyType*, const uint8*)> GetPropertyValueFromCell)
	{
		const FProperty* Property = FindTableProperty(ColumnName);
		if (!Property)
		{
			return nullptr;
		}
		
		const TPropertyType* PropertyType = CastField<TPropertyType>(Property);
		if (!PropertyType)
		{
			return nullptr;
		}
		
		const uint8* RowData = FindRowUnchecked(RowName);
		if (!RowData)
		{
			return nullptr;
		}

		const uint8* CellData = Property->ContainerPtrToValuePtr<uint8>(RowData, 0);
		if (!CellData)
		{
			return nullptr;
		}

		return GetPropertyValueFromCell(PropertyType, CellData);
	}

	/** Empty the table info (will not clear RowStruct) */
	ENGINE_API virtual void EmptyTable();

	ENGINE_API TArray<FName> GetRowNames() const;

	/** Removes a single row from the DataTable by name. Just returns if row is not found. */
	ENGINE_API virtual void RemoveRow(FName RowName);

	/** Copies RowData into table. That is: create Row if not found and copy data into the RowMap based on RowData. This is a "copy in" operation, so changing the passed in RowData after the fact does nothing. */
	ENGINE_API virtual void AddRow(FName RowName, const FTableRowBase& RowData);
	ENGINE_API virtual void AddRow(FName RowName, const uint8* RowData, const UScriptStruct* RowType);

#if WITH_EDITOR
	ENGINE_API virtual void CleanBeforeStructChange();
	ENGINE_API virtual void RestoreAfterStructChange();

	/** Output entire contents of table as a string */
	ENGINE_API FString GetTableAsString(const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Output entire contents of table as CSV */
	ENGINE_API FString GetTableAsCSV(const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Output entire contents of table as JSON */
	ENGINE_API FString GetTableAsJSON(const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Output entire contents of table as JSON */
	template<typename CharType = TCHAR>
	bool WriteTableAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Output entire contents of table as a JSON Object*/
	template<typename CharType = TCHAR>
	bool WriteTableAsJSONObject(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Output the fields from a particular row (use RowMap to get RowData) to an existing JsonWriter */
	template<typename CharType = TCHAR>
	bool WriteRowAsJSON(const TSharedRef< TJsonWriter<CharType, TPrettyJsonPrintPolicy<CharType> > >& JsonWriter, const void* RowData, const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;

	/** Copies all the import options from another table, this does not copy row dawta */
	ENGINE_API bool CopyImportOptions(UDataTable* SourceTable);
#endif
	/** 
	 *	Create table from CSV style comma-separated string. 
	 *	RowStruct must be defined before calling this function. 
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromCSVString(const FString& InString);

	/** 
	*	Create table from JSON style string. 
	*	RowStruct must be defined before calling this function. 
	*	@return	Set of problems encountered while processing input
	*/
	ENGINE_API TArray<FString> CreateTableFromJSONString(const FString& InString);

	/** Get array of UProperties that corresponds to columns in the table */
	TArray<FProperty*> GetTablePropertyArray(const TArray<const TCHAR*>& Cells, UStruct* RowStruct, TArray<FString>& OutProblems, int32 KeyColumn = 0);
	
	/** 
	 *	Create table from another Data Table
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromOtherTable(const UDataTable* InTable);

	/**
	 *	Create table from a raw data map with a given script struct
	 *	@return	Set of problems encountered while processing input
	 */
	ENGINE_API TArray<FString> CreateTableFromRawData(TMap<FName, const uint8*>& DataMap, UScriptStruct* InRowStruct);

#if WITH_EDITOR
	/** Get an array of all the column titles, using the friendly display name from the property */
	ENGINE_API TArray<FString> GetColumnTitles() const;

	/** Get an array of all the column titles, using the unique name from the property */
	ENGINE_API TArray<FString> GetUniqueColumnTitles() const;

	/** Get array for each row in the table. The first row is the titles*/
	ENGINE_API TArray< TArray<FString> > GetTableData(const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None) const;
#endif //WITH_EDITOR

	//~ End UDataTable Interface

protected:
	void SaveStructData(FStructuredArchiveSlot Slot);
	void LoadStructData(FStructuredArchiveSlot Slot);

	/**
	 * Called whenever new data is imported into the data table via CreateTableFrom*; Alerts each imported row and gives the
	 * row struct a chance to operate on the imported data
	 * 
	 * @param OutCollectedImportProblems	[OUT] Array of strings of import problems
	 */
	void OnPostDataImported(OUT TArray<FString>& OutCollectedImportProblems);

	UScriptStruct& GetEmptyUsingStruct() const;

	/** Used to trigger the data table changed delegate. This allows us to trigger the delegate only once from more complex changes */
	struct FScopedDataTableChange
	{
		FScopedDataTableChange(UDataTable* InTable);
		FScopedDataTableChange(UDataTable* InTable,FName Row);
		~FScopedDataTableChange();

	private:
		UDataTable* Table;
		FName RowName;
		static TMap<UDataTable*, int32> ScopeCount;
		static FTransactionallySafeCriticalSection CriticalSection;
	};

private:
	ENGINE_API bool CommonTypeCheck(UScriptStruct* Type, const TCHAR* MethodName, const TCHAR* ContextString) const;
	ENGINE_API uint8* FindRowInternal(UScriptStruct* Type, FName RowName, const TCHAR* ContextString, bool bWarnIfRowMissing) const;
};


/** Handle to a particular row in a table*/
USTRUCT(BlueprintType)
struct FDataTableRowHandle
{
	GENERATED_USTRUCT_BODY()

	FDataTableRowHandle()
		: DataTable(nullptr)
		, RowName(NAME_None)
	{

	}

	/** Pointer to table we want a row from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DataTableRowHandle)
	TObjectPtr<const UDataTable>	DataTable;

	/** Name of row in the table that we want */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DataTableRowHandle)
	FName				RowName;

	/** Returns true if this handle is specifically pointing to nothing */
	bool IsNull() const
	{
		return DataTable == nullptr && RowName.IsNone();
	}

	/** Get the row straight from the row handle */
	template <class T>
	T* GetRow(const TCHAR* ContextString) const
	{
		if(DataTable == nullptr)
		{
			if (RowName != NAME_None)
			{
				UE_LOGF(LogDataTable, Warning, "FDataTableRowHandle::GetRow : No DataTable for row %ls (%ls).", *RowName.ToString(), ContextString);
			}
			return nullptr;
		}

		return DataTable->FindRow<T>(RowName, ContextString);
	}

	template <class T>
	T* GetRow(const FString& ContextString) const
	{
		return GetRow<T>(*ContextString);
	}

	FString ToDebugString(bool bUseFullPath = false) const
	{
		if (DataTable == nullptr)
		{
			return FString::Printf(TEXT("No Data Table Specified, Row: %s"), *RowName.ToString());
		}

		return FString::Printf(TEXT("Table: %s, Row: %s"), bUseFullPath ? *DataTable->GetPathName() : *DataTable->GetName(), *RowName.ToString());
	}

	ENGINE_API bool operator==(FDataTableRowHandle const& Other) const;
	ENGINE_API bool operator!=(FDataTableRowHandle const& Other) const;
	ENGINE_API void PostSerialize(const FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits< FDataTableRowHandle > : public TStructOpsTypeTraitsBase2< FDataTableRowHandle >
{
	enum
	{
		WithPostSerialize = true,
	};
};

/** Handle to a particular set of rows in a table */
USTRUCT(BlueprintType)
struct FDataTableCategoryHandle
{
	GENERATED_USTRUCT_BODY()

	/** Pointer to table we want a row from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DataTableCategoryHandle)
	TObjectPtr<const class UDataTable>	DataTable = nullptr;

	/** Name of column in the table that we want */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DataTableCategoryHandle)
	FName				ColumnName;

	/** Contents of rows in the table that we want */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DataTableCategoryHandle)
	FName				RowContents;

	/** Returns true if this handle is specifically pointing to nothing */
	bool IsNull() const
	{
		return DataTable == nullptr && ColumnName == NAME_None && RowContents == NAME_None;
	}

	/** Searches DataTable for all rows that contain entries with RowContents in the column named ColumnName and returns them. */
	template <class T>
	void GetRows(TArray<T*>& OutRows, const FString& ContextString) const
	{
		OutRows.Empty();
		if (DataTable == nullptr)
		{
			if (RowContents != NAME_None)
			{
				UE_LOGF(LogDataTable, Warning, "FDataTableCategoryHandle::GetRows : No DataTable for row %ls (%ls).", *RowContents.ToString(), *ContextString);
			}

			return;
		}

		if (ColumnName == NAME_None)
		{
			if (RowContents != NAME_None)
			{
				UE_LOGF(LogDataTable, Warning, "FDataTableCategoryHandle::GetRows : No Column selected for row %ls (%ls).", *RowContents.ToString(), *ContextString);
			}

			return;
		}

		// Find the property that matches the desired column (ColumnName)
		FProperty* Property = DataTable->FindTableProperty(ColumnName);
		if (Property == nullptr)
		{
			return;
		}

		// check each row to see if the value in the Property element is the one we're looking for (RowContents). If it is, add the row to OutRows
		uint8* RowContentsAsBinary = (uint8*)FMemory_Alloca(Property->GetSize());
		Property->InitializeValue(RowContentsAsBinary);
		if (Property->ImportText_Direct(*RowContents.ToString(), RowContentsAsBinary, nullptr, PPF_None) == nullptr)
		{
			Property->DestroyValue(RowContentsAsBinary);
			return;
		}

		for (auto RowIt = DataTable->GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
		{
			uint8* RowData = RowIt.Value();

			if (Property->Identical(Property->ContainerPtrToValuePtr<void>(RowData, 0), RowContentsAsBinary, PPF_None))
			{
				OutRows.Add((T*)RowData);
			}
		}
		Property->DestroyValue(RowContentsAsBinary);

		return;
	}

	ENGINE_API bool operator==(FDataTableCategoryHandle const& Other) const;
	ENGINE_API bool operator!=(FDataTableCategoryHandle const& Other) const;
};


/** Macro to call GetRow with a correct error info. Assumed to be called from within a UObject */
#define GETROW_REPORTERROR(Handle, Template) Handle.GetRow<Template>(FString::Printf(TEXT("%s.%s"), *GetPathName(), TEXT(#Handle)))
#define GETROWOBJECT_REPORTERROR(Object, Handle, Template) Handle.GetRow<Template>(FString::Printf(TEXT("%s.%s"), *Object->GetPathName(), TEXT(#Handle)))
