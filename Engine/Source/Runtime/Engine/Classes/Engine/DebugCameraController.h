// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/HitResult.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "FinalPostProcessSettings.h"
#include "SceneTypes.h"
#include "GameFramework/PlayerController.h"
#include "DebugCameraController.generated.h"

class ASpectatorPawn;

/**
* Camera controller that allows you to fly around a level mostly unrestricted by normal movement rules.
*
* To turn it on, please press Alt+C or both (left and right) analogs on XBox pad,
* or use the "ToggleDebugCamera" console command. Check the debug camera bindings
* in DefaultPawn.cpp for the camera controls.
*/
UCLASS(config=Game, MinimalAPI)
class ADebugCameraController
	: public APlayerController
{
	GENERATED_UCLASS_BODY()

	/** Whether to show information about the selected actor on the debug camera HUD. */
	UPROPERTY(globalconfig)
	uint32 bShowSelectedInfo:1;

	/** Saves whether the FreezeRendering console command is active */
	UPROPERTY()
	uint32 bIsFrozenRendering:1;

	/** Whether to orbit selected actor. */
	UPROPERTY()
	uint32 bIsOrbitingSelectedActor : 1;

	/** When orbiting, true if using actor center as pivot, false if using last selected hitpoint */
	UPROPERTY()
	uint32 bOrbitPivotUseCenter:1;

	/** Whether set view mode to display GBuffer visualization overview */
	UPROPERTY()
	uint32 bEnableBufferVisualization : 1;

	/** Whether set view mode to display GBuffer visualization full */
	UPROPERTY()
	uint32 bEnableBufferVisualizationFullMode : 1;

	/** Whether GBuffer visualization overview inputs are set up  */
	UPROPERTY()
	uint32 bIsBufferVisualizationInputSetup : 1;

	/** Last display enabled setting before toggling buffer visualization overview */
	UPROPERTY()
	uint32 bLastDisplayEnabled : 1;

	/** Visualizes the frustum of the camera */
	UPROPERTY()
	TObjectPtr<class UDrawFrustumComponent> DrawFrustum;
	
	/** Sets whether to show information about the selected actor on the debug camera HUD. */
	UFUNCTION(exec)
	ENGINE_API virtual void ShowDebugSelectedInfo();

	/** Selects the object the camera is aiming at. */
	ENGINE_API void SelectTargetedObject();

	/** Called when the user pressed the deselect key, just before the selected actor is cleared. */
	ENGINE_API void Unselect();

	/** Speeds up camera movement */
	ENGINE_API void IncreaseCameraSpeed();

	/** Slows down camera movement */
	ENGINE_API void DecreaseCameraSpeed();

	/** Increases camera field of vision */
	ENGINE_API void IncreaseFOV();

	/** Decreases camera field of vision */
	ENGINE_API void DecreaseFOV();

	/** Toggles the display of debug info and input commands for the Debug Camera. */
	UFUNCTION(BlueprintCallable, Category="Debug Camera")
	ENGINE_API void ToggleDisplay();

	/** Sets display of debug info and input commands. */
	ENGINE_API void SetDisplay(bool bEnabled);

	/** Returns whether debug info display is enabled */
	ENGINE_API bool IsDisplayEnabled();

	/**
	 * function called from key bindings command to save information about
	 * turning on/off FreezeRendering command.
	 */
	ENGINE_API virtual void ToggleFreezeRendering();

	/** Method called prior to processing input */
	ENGINE_API virtual void PreProcessInput(const float DeltaTime, const bool bGamePaused);

	/**
	 * Updates the rotation of player, based on ControlRotation after RotationInput has been applied.
	 * This may then be modified by the PlayerCamera, and is passed to Pawn->FaceRotation().
	 */
	ENGINE_API virtual void UpdateRotation(float DeltaTime) override;

	/** Pre process input when orbiting */
	ENGINE_API void PreProcessInputForOrbit(const float DeltaTime, const bool bGamePaused);

	/** Updates the rotation and location of player when orbiting */
	ENGINE_API void UpdateRotationForOrbit(float DeltaTime);

	/** Gets pivot to use when orbiting */
	ENGINE_API bool GetPivotForOrbit(FVector& PivotLocation) const;

	/** Toggles camera orbit */
	ENGINE_API void ToggleOrbit(bool bOrbitCenter);

	/** Toggles camera orbit center */
	ENGINE_API void ToggleOrbitCenter();

	/** Toggles camera orbit hitpoint */
	ENGINE_API void ToggleOrbitHitPoint();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	/** Whether buffer visualization option should be enabled */
	static ENGINE_API bool EnableDebugBuffers();

	/** Whether cycle viewmode option should be enabled */
	static ENGINE_API bool EnableDebugViewmodes();

	/** Cycle view mode */
	ENGINE_API void CycleViewMode();

	/** Toggle buffer visualization overview */
	ENGINE_API void ToggleBufferVisualizationOverviewMode();

	/** Buffer overview move up */
	ENGINE_API void BufferVisualizationMoveUp();

	/** Buffer overview move down */
	ENGINE_API void BufferVisualizationMoveDown();

	/** Buffer overview move right */
	ENGINE_API void BufferVisualizationMoveRight();

	/** Buffer overview move left */
	ENGINE_API void BufferVisualizationMoveLeft();

	/** Ignores axis motion */
	ENGINE_API void ConsumeAxisMotion(float Val);

	/** Toggle buffer visualization full display */
	ENGINE_API void ToggleBufferVisualizationFullMode();

	/** Set buffer visualization full mode */
	ENGINE_API void SetBufferVisualizationFullMode(bool bFullMode);

	/** Update visualize buffer post processing settings */
	ENGINE_API void UpdateVisualizeBufferPostProcessing(FFinalPostProcessSettings& InOutPostProcessingSettings);

	/** Get visualization buffer's material name used by post processing settings */
	ENGINE_API FString GetBufferMaterialName(const FString& InBuffer);

	/** Get current visualization buffer's material name */
	ENGINE_API FString GetSelectedBufferMaterialName();


#endif

public:

	/** Currently selected actor, may be invalid */
	UPROPERTY()
	TWeakObjectPtr<class AActor> SelectedActor;

	/** Returns the currently selected actor, or null if it is invalid or not set */
	UFUNCTION(BlueprintCallable, Category="Debug Camera")
	ENGINE_API AActor* GetSelectedActor() const;
	
	/** Currently selected component, may be invalid */
	UPROPERTY()
	TWeakObjectPtr<class UPrimitiveComponent> SelectedComponent;

	/** Selected hit point */
	UPROPERTY()
	FHitResult SelectedHitPoint;

	/** Controller that was active before this was spawned */
	UPROPERTY()	
	TObjectPtr<class APlayerController> OriginalControllerRef;

	/** Player object that was active before this was spawned */
	UPROPERTY()	
	TObjectPtr<class UPlayer> OriginalPlayer;

	/** Allows control over the speed of the spectator pawn. This scales the speed based on the InitialMaxSpeed. Use Set Pawn Movement Speed Scale during runtime */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Debug Camera")
	float SpeedScale;
	
	/** Sets the pawn movement speed scale. */
	UFUNCTION(BlueprintCallable, Category="Debug Camera")
	ENGINE_API void SetPawnMovementSpeedScale(float NewSpeedScale);
	
	/** Initial max speed of the spectator pawn when we start possession. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Debug Camera")
	float InitialMaxSpeed;

	/** Initial acceleration of the spectator pawn when we start possession. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Debug Camera")
	float InitialAccel;

	/** Initial deceleration of the spectator pawn when we start possession. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Debug Camera")
	float InitialDecel;

protected:

	// Adjusts movement speed limits based on SpeedScale.
	ENGINE_API virtual void ApplySpeedScale();
	ENGINE_API virtual void SetupInputComponent() override;

	/** Sets up or clears input for buffer visualization overview */
	ENGINE_API void SetupBufferVisualizationOverviewInput();

public:

	/** 
	* Function called on activation of debug camera controller.
	* @param OriginalPC The active player controller before this debug camera controller was possessed by the player.
	*/
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "OnActivate"))
	ENGINE_API void ReceiveOnActivate(class APlayerController* OriginalPC);

	/** Function called on activation debug camera controller */
	ENGINE_API virtual void OnActivate(class APlayerController* OriginalPC);
	
	/** 
	* Function called on deactivation of debug camera controller.
	* @param RestoredPC The Player Controller that the player input is being returned to.
	*/
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "OnDeactivate"))
	ENGINE_API void ReceiveOnDeactivate(class APlayerController* RestoredPC);

	/** Function called on deactivation debug camera controller */
	ENGINE_API virtual void OnDeactivate(class APlayerController* RestoredPC);

	/**
	 * Builds a list of components that are hidden based upon gameplay
	 *
	 * @param ViewLocation the view point to hide/unhide from
	 * @param HiddenComponents the list to add to/remove from
	 */
	ENGINE_API virtual void UpdateHiddenComponents(const FVector& ViewLocation,TSet<FPrimitiveComponentId>& HiddenComponents) override;

public:

	// APlayerController Interface

	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual FString ConsoleCommand(const FString& Command, bool bWriteToLog = true) override;
	ENGINE_API virtual void AddCheats(bool bForce) override;
	ENGINE_API virtual void EndSpectatingState() override;
	/** Custom spawn to spawn a default SpectatorPawn, to use as a spectator and initialize it. By default it is spawned at the PC's current location and rotation. */
	ENGINE_API virtual ASpectatorPawn* SpawnSpectatorPawn() override;

protected:

	/**
	 * Called when an actor has been selected with the primary key (e.g. left mouse button).
	 *
	 * The selection trace starts from the center of the debug camera's view.
	 *
	 * @param SelectHitLocation The exact world-space location where the selection trace hit the New Selected Actor.
	 * @param SelectHitNormal The world-space surface normal of the New Selected Actor at the hit location.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category="Debug Camera", meta=(DisplayName="OnActorSelected"))
	ENGINE_API void ReceiveOnActorSelected(AActor* NewSelectedActor, const FVector& SelectHitLocation, const FVector& SelectHitNormal, const FHitResult& Hit);
	
	/**
	 * Called when an actor has been selected with the primary key (e.g. left mouse button).
	 * @param Hit	Info struct for the selection point.
	 */
	ENGINE_API virtual void Select( FHitResult const& Hit );

	ENGINE_API virtual void SetSpectatorPawn(class ASpectatorPawn* NewSpectatorPawn) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	/** Get buffer visualization overview targets based on console var */
	ENGINE_API TArray<FString> GetBufferVisualizationOverviewTargets();

	/** Get next buffer */
	ENGINE_API void GetNextBuffer(const TArray<FString>& OverviewBuffers, int32 Step = 1);

	/** Get next buffer */
	ENGINE_API void GetNextBuffer(int32 Step = 1);

#endif

private:

	/** The normalized screen location when a drag starts */
	FVector2D LastTouchDragLocation;

	/** Last position for orbit */
	FVector LastOrbitPawnLocation;

	/** Current orbit pivot, if orbit is enabled */
	FVector OrbitPivot;

	/** Current orbit radius, if orbit is enabled */
	float OrbitRadius;

	/** Last index in settings array for cycle view modes */
	int32 LastViewModeSettingsIndex;

	/** Buffer selected in buffer visualization overview or full screen view */
	FString CurrSelectedBuffer;

	void OnTouchBegin(ETouchIndex::Type FingerIndex, FVector Location);
	void OnTouchEnd(ETouchIndex::Type FingerIndex, FVector Location);
	void OnFingerMove(ETouchIndex::Type FingerIndex, FVector Location);
};
