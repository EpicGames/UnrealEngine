// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugCameraControllerSettings.h: Declares the DebugCameraControllerSettings class.
=============================================================================*/

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "DebugCameraControllerSettings.generated.h"

USTRUCT()
struct FDebugCameraControllerSettingsViewModeIndex
{
	GENERATED_USTRUCT_BODY()

	FDebugCameraControllerSettingsViewModeIndex() { ViewModeIndex = VMI_BrushWireframe; }
	FDebugCameraControllerSettingsViewModeIndex(EViewModeIndex InViewModeIndex) : ViewModeIndex(InViewModeIndex) {}

	virtual ~FDebugCameraControllerSettingsViewModeIndex() = default;

	UPROPERTY(EditAnywhere, Category = General, meta = (DisplayName = "Cycle View Mode"))
	TEnumAsByte<EViewModeIndex> ViewModeIndex;
};

/**
 * Default debug camera controller settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Debug Camera Controller"), MinimalAPI)
class UDebugCameraControllerSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(config, EditAnywhere, Category = General)
	TArray<FDebugCameraControllerSettingsViewModeIndex> CycleViewModes;

public:
#if WITH_EDITOR

	// UObject interface.
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

protected:
	ENGINE_API void RemoveInvalidViewModes();

#endif // WITH_EDITOR

public:
	ENGINE_API TArray<EViewModeIndex> GetCycleViewModes();

	static UDebugCameraControllerSettings * Get() { return CastChecked<UDebugCameraControllerSettings>(UDebugCameraControllerSettings::StaticClass()->GetDefaultObject()); }
};
