// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/HUD.h"
#include "DebugCameraHUD.generated.h"

/**
 * HUD that displays info for the DebugCameraController view.
 */
UCLASS(config = Game, hidedropdown, MinimalAPI)
class ADebugCameraHUD
	: public AHUD
{
	GENERATED_UCLASS_BODY()

	/** @todo document */
	ENGINE_API virtual bool DisplayMaterials( float X, float& Y, float DY, class UMeshComponent* MeshComp );
	
	//~ Begin AActor Interface
	ENGINE_API virtual void PostRender() override;
	//~ End AActor Interface

};
