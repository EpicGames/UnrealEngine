// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "DebugDisplayProperty.generated.h"


/** 
 * Debug property display functionality to interact with this, use "display", "displayall", "displayclear"
 *
 * @see UGameViewportClient
 * @see FDebugDisplayProperty
 * @see DrawStatsHUD
 */
USTRUCT()
struct FDebugDisplayProperty
{
	GENERATED_USTRUCT_BODY()

	/** the object whose property to display. If this is a class, all objects of that class are drawn. */
	UPROPERTY()
	TObjectPtr<class UObject> Obj;

	/** if Obj is a class and WithinClass is not nullptr, further limit the display to objects that have an Outer of WithinClass */
	UPROPERTY()
	TSubclassOf<class UObject>  WithinClass;

	/** name of the property to display */
	FName PropertyName;

	/** whether PropertyName is a "special" value not directly mapping to a real property (e.g. state name) */
	uint32 bSpecialProperty:1;

	/** Default constructor. */
	FDebugDisplayProperty()
		: Obj(nullptr)
		, WithinClass(nullptr)
		, bSpecialProperty(false)
	{ }
};
