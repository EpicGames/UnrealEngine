// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "DecalActor.generated.h"

class UArrowComponent;
class UBillboardComponent;
class UBoxComponent;
class UDecalComponent;

/**
* DecalActor contains a DecalComponent which can be used to render material modifications on top of existing geometry.
*
* @see https://docs.unrealengine.com/latest/INT/Engine/Actors/DecalActor
* @see UDecalComponent
*/
UCLASS(hideCategories=(Collision, Attachment, Actor, Input, Replication), showCategories=("Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass, MinimalAPI)
class ADecalActor
	: public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** The decal component for this decal actor */
	UPROPERTY(Category = Decal, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Decal,Rendering|Components|Decal", AllowPrivateAccess = "true"))
	TObjectPtr<UDecalComponent> Decal;

#if WITH_EDITORONLY_DATA
	/* Reference to the editor only arrow visualization component */
	UPROPERTY()
	TObjectPtr<UArrowComponent> ArrowComponent;

	/* Reference to the billboard component */
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

	// formerly we used this component to draw a box, now we use the DecalComponentVisualizer
	UPROPERTY()
	TObjectPtr<UBoxComponent> BoxComponent_DEPRECATED;
#endif

public:

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal", meta=(DeprecatedFunction))
	ENGINE_API void SetDecalMaterial(class UMaterialInterface* NewDecalMaterial);
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal", meta=(DeprecatedFunction))
	ENGINE_API class UMaterialInterface* GetDecalMaterial() const;
	UFUNCTION(BlueprintCallable, Category="Rendering|Components|Decal", meta=(DeprecatedFunction))
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateDynamicMaterialInstance();
	// END DEPRECATED

	
#if WITH_EDITOR
	//~ Begin UObject Interface
	ENGINE_API virtual void PostEditMove(bool bFinished) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin AActor Interface.
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	ENGINE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	//~ End AActor Interface.
#endif // WITH_EDITOR

	
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

public:

	/** Returns Decal subobject **/
	UDecalComponent* GetDecal() const { return Decal; }
#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};
