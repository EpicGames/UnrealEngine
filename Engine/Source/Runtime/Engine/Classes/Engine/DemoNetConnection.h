// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/NetConnection.h"
#include "ReplayTypes.h"
#include "DemoNetConnection.generated.h"

class APlayerController;
class FObjectReplicator;
class UDemoNetDriver;

/**
 * Simulated network connection for recording and playing back game sessions.
 */
UCLASS(transient, config=Engine, MinimalAPI)
class UDemoNetConnection : public UNetConnection
{
	GENERATED_BODY()

public:
	ENGINE_API UDemoNetConnection(const FObjectInitializer& ObjectInitializer);

	// UNetConnection interface.

	ENGINE_API virtual void InitConnection( class UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed = 0, int32 InMaxPacket=0) override;
	ENGINE_API virtual FString LowLevelGetRemoteAddress( bool bAppendPort = false ) override;
	ENGINE_API virtual FString LowLevelDescribe() override;
	ENGINE_API virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override;

	UE_DEPRECATED(5.6, "Replaced with a version that doesn't have a saturate option.")
	ENGINE_API virtual int32 IsNetReady( bool Saturate ) override;
	
	virtual bool IsNetReady() const override
	{
		return true;
	}

	ENGINE_API virtual void FlushNet( bool bIgnoreSimulation = false ) override;
	ENGINE_API virtual void HandleClientPlayer( APlayerController* PC, class UNetConnection* NetConnection ) override;
	ENGINE_API virtual TSharedPtr<const FInternetAddr> GetRemoteAddr() override;
	ENGINE_API virtual bool ClientHasInitializedLevel( const ULevel* TestLevel ) const override;
	ENGINE_API virtual TSharedPtr<FObjectReplicator> CreateReplicatorForNewActorChannel(UObject* Object);
	virtual FString RemoteAddressToString() override { return TEXT("Demo"); }

	ENGINE_API virtual void NotifyActorNetGUID(UActorChannel* Channel) override;
	ENGINE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason) override;

	virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}
	virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}

public:

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** @return The DemoRecording driver object */
	inline class UDemoNetDriver* GetDriver()
	{
		return (UDemoNetDriver*)Driver;
	}

	/** @return The DemoRecording driver object */
	inline const class UDemoNetDriver* GetDriver() const
	{
		return (UDemoNetDriver*)Driver;
	}

	TMap<FNetworkGUID, UActorChannel*>& GetOpenChannelMap() { return OpenChannelMap; }

protected:
	ENGINE_API virtual void DestroyIgnoredActor(AActor* Actor) override;

	ENGINE_API void QueueNetStartupActorForRewind(AActor* Actor);

private:
	ENGINE_API void TrackSendForProfiler(const void* Data, int32 NumBytes);

	TMap<FNetworkGUID, UActorChannel*> OpenChannelMap;
};
