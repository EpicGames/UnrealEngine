// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/BitReader.h"
#include "Misc/NetworkGuid.h"
#include "Engine/EngineBaseTypes.h"
#include "GameFramework/Actor.h"
#include "Misc/EngineVersion.h"
#include "GameFramework/PlayerController.h"
#include "Engine/NetDriver.h"
#include "Engine/PackageMapClient.h"
#include "Misc/NetworkVersion.h"
#include "NetworkReplayStreaming.h"
#include "Engine/DemoNetConnection.h"
#include "Net/RepLayout.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/ReplayResult.h"
#include "Templates/Atomic.h"
#include "Net/UnrealNetwork.h"
#include "ReplayHelper.h"

#include "DemoNetDriver.generated.h"

class FNetworkNotify;
class FRepState;
class UDemoNetDriver;

DECLARE_MULTICAST_DELEGATE(FOnGotoTimeMCDelegate);
DECLARE_DELEGATE_OneParam(FOnGotoTimeDelegate, const bool /* bWasSuccessful */);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnDemoStartedDelegate, UDemoNetDriver* /* DemoNetDriver */);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.1, "No longer used")
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDemoFailedToStartDelegate, UDemoNetDriver* /* DemoNetDriver */, EDemoPlayFailure::Type /* FailureType*/);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

DECLARE_MULTICAST_DELEGATE(FOnDemoFinishPlaybackDelegate);
DECLARE_MULTICAST_DELEGATE(FOnDemoFinishRecordingDelegate);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnPauseChannelsDelegate, const bool /* bPaused */);

class FQueuedReplayTask : public TSharedFromThis<FQueuedReplayTask>
{
public:
	FQueuedReplayTask(UDemoNetDriver* InDriver) : Driver(InDriver)
	{
	}

	virtual ~FQueuedReplayTask() = default;

	virtual void StartTask() = 0;
	virtual bool Tick() = 0;
	virtual FName GetName() const = 0;
	virtual bool ShouldPausePlayback() const { return true; }

	TWeakObjectPtr<UDemoNetDriver> Driver;
};

/** Information about net startup actors that need to be rolled back by being destroyed and re-created */
USTRUCT()
struct FRollbackNetStartupActorInfo
{
	GENERATED_BODY()

	FName		Name;
	UPROPERTY()
	TObjectPtr<UObject>	Archetype = nullptr;
	FVector		Location;
	FRotator	Rotation;
	FVector		Scale3D;
	FName		LevelName;

	TSharedPtr<FRepState> RepState;
	TMap<FString, TSharedPtr<FRepState>> SubObjRepState;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjReferences;

	void CountBytes(FArchive& Ar) const
	{
		if (FRepState const * const LocalRepState = RepState.Get())
		{
			Ar.CountBytes(sizeof(FRepState), sizeof(FRepState));
			LocalRepState->CountBytes(Ar);
		}

		SubObjRepState.CountBytes(Ar);
		for (const auto& SubObjRepStatePair : SubObjRepState)
		{
			SubObjRepStatePair.Key.CountBytes(Ar);
			
			if (FRepState const * const LocalRepState = SubObjRepStatePair.Value.Get())
			{
				const SIZE_T SizeOfRepState = sizeof(FRepState);
				Ar.CountBytes(SizeOfRepState, SizeOfRepState);
				LocalRepState->CountBytes(Ar);
			}
		}

		ObjReferences.CountBytes(Ar);
	}
};

struct FDemoSavedRepObjectState
{
	ENGINE_API FDemoSavedRepObjectState(
		const TWeakObjectPtr<const UObject>& InObject,
		const TSharedRef<const FRepLayout>& InRepLayout,
		FRepStateStaticBuffer&& InPropertyData);

	ENGINE_API ~FDemoSavedRepObjectState();

	TWeakObjectPtr<const UObject> Object;
	TSharedPtr<const FRepLayout> RepLayout;
	FRepStateStaticBuffer PropertyData;

	void CountBytes(FArchive& Ar) const
	{
		// The RepLayout for this object should still be stored by the UDemoNetDriver,
		// so we don't need to count it here.

		PropertyData.CountBytes(Ar);
	}
};

typedef TArray<struct FDemoSavedRepObjectState> FDemoSavedPropertyState;

USTRUCT()
struct FMulticastRecordOptions
{
	GENERATED_BODY()

	UPROPERTY()
	FString FuncPathName;

	UPROPERTY()
	bool bServerSkip = false;

	UPROPERTY()
	bool bClientSkip = false;
};

/**
 * Simulated network driver for recording and playing back game sessions.
 */
UCLASS(transient, config=Engine, MinimalAPI)
class UDemoNetDriver : public UNetDriver
{
	GENERATED_BODY()

public:
	ENGINE_API UDemoNetDriver(const FObjectInitializer& ObjectInitializer);
	ENGINE_API UDemoNetDriver(FVTableHelper& Helper);
	ENGINE_API ~UDemoNetDriver();

	ENGINE_API virtual void SetWorld(class UWorld* InWorld) override;

	/** Current record/playback frame number */
	int32 GetDemoFrameNum() const { return ReplayHelper.DemoFrameNum; }

	bool GetChannelsArePaused() const { return bChannelsArePaused; }

	double GetCurrentLevelIndex() const { return ReplayHelper.CurrentLevelIndex; }

	void SetCurrentLevelIndex(int32 Index)
	{
		ReplayHelper.CurrentLevelIndex = Index;
	}

	APlayerController* GetSpectatorController() const { return SpectatorController; }

private:

	/** This is the main spectator controller that is used to view the demo world from */
	APlayerController* SpectatorController;

	/** Our network replay streamer */
	TSharedPtr<class INetworkReplayStreamer> ReplayStreamer;

public:
	TSharedPtr<class INetworkReplayStreamer> GetReplayStreamer() const { return ReplayStreamer; }

	uint32 GetDemoCurrentTimeInMS() const { return (uint32)((double)GetDemoCurrentTime() * 1000); }

	/** Internal debug timing/tracking */
	double AccumulatedRecordTime;
	double LastRecordAvgFlush;
	double MaxRecordTime;
	int32 RecordCountSinceFlush;

private:
	/** 
	 * Net startup actors that need to be rolled back during scrubbing by being destroyed and re-spawned 
	 * NOTE - DeletedNetStartupActors will take precedence here, and destroy the actor instead
	 */
	UPROPERTY(transient)
	TMap<FString, FRollbackNetStartupActorInfo> RollbackNetStartupActors;

public:
	double GetLastCheckpointTime() const { return ReplayHelper.LastCheckpointTime; }

	void SetLastCheckpointTime(double CheckpointTime)
	{
		ReplayHelper.LastCheckpointTime = CheckpointTime;
	}

	ENGINE_API void RespawnNecessaryNetStartupActors(TArray<AActor*>& SpawnedActors, ULevel* Level = nullptr);

private:
	ENGINE_API void RestoreComponentState(UActorComponent* ActorComp, FRollbackNetStartupActorInfo& RollbackActor);

	ENGINE_API void InitDefaults();

	ENGINE_API bool LoadCheckpoint(const FGotoResult& GotoResult);
	
	TArray<TUniquePtr<FDeltaCheckpointData>> PlaybackDeltaCheckpointData;
	
	TSharedPtr<struct FReplayPlaylistTracker> PlaylistTracker;

public:

	void SetPlayingPlaylist(TSharedPtr<struct FReplayPlaylistTracker> InPlaylistTracker)
	{
		PlaylistTracker = InPlaylistTracker;
	}

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Returns true if we're in the process of saving a checkpoint. */
	ENGINE_API bool IsSavingCheckpoint() const;
	bool IsLoadingCheckpoint() const { return ReplayHelper.bIsLoadingCheckpoint; }
	
	ENGINE_API bool IsPlayingClientReplay() const;

	/** PlaybackPackets are used to buffer packets up when we read a demo frame, which we can then process when the time is right */
	TArray<FPlaybackPacket> PlaybackPackets;

	bool IsRecordingMapChanges() const { return ReplayHelper.bRecordMapChanges; }

	ENGINE_API void RequestCheckpoint();

private:
	struct FDemoActorPriority
	{
		FActorPriority ActorPriority;
		UObject*	Level;
	};

	bool bIsFastForwarding;
	bool bIsFinalizingFastForward;
	bool bIsRestoringStartupActors;

	/** True if as have paused all of the channels */
	uint8 bChannelsArePaused : 1;
	uint8 bIsFastForwardingForCheckpoint : 1;
	uint8 bWasStartStreamingSuccessful : 1;

	/** If true, recording will prioritize replicating actors based on the value that AActor::GetReplayPriority returns. */
	uint8 bPrioritizeActors : 1;

protected:
	uint8 bIsWaitingForHeaderDownload : 1;
	uint8 bIsWaitingForStream : 1;

	TOptional<UE::Net::TNetResult<EReplayResult>> PendingRecordFailure;

private:
	TArray<FNetworkGUID> NonQueuedGUIDsForScrubbing;

	// Replay tasks
	TArray<TSharedRef<FQueuedReplayTask>>		QueuedReplayTasks;
	TSharedPtr<FQueuedReplayTask>				ActiveReplayTask;
	TSharedPtr<FQueuedReplayTask>				ActiveScrubReplayTask;

	/** Set via GotoTimeInSeconds, only fired once (at most). Called for successful or failed scrub. */
	FOnGotoTimeDelegate OnGotoTimeDelegate_Transient;
	
	/** Saved server time after loading a checkpoint, so that we can set the server time as accurately as possible after the fast-forward */
	double SavedReplicatedWorldTimeSeconds;

	/** Saved fast-forward time, used for correcting world time after the fast-forward is complete */
	float SavedSecondsToSkip;

	/** The unique identifier for the lifetime of this object. */
	FString DemoSessionID;

	/** Optional time quota for actor replication during recording. Going over this limit effectively lowers the net update frequency of the remaining actors. Negative values are considered unlimited. */
	float MaxDesiredRecordTimeMS;

	/**
	 * Maximum time allowed each frame to spend on saving a checkpoint. If 0, it will save the checkpoint in a single frame, regardless of how long it takes.
	 * See also demo.CheckpointSaveMaxMSPerFrameOverride.
	 */
	UPROPERTY(Config)
	float CheckpointSaveMaxMSPerFrame;

	/** A player controller that this driver should consider its viewpoint for actor prioritization purposes. */
	TWeakObjectPtr<APlayerController> ViewerOverride;

	/** Array of prioritized actors, used in TickDemoRecord. Stored as a member so that its storage doesn't have to be re-allocated each frame. */
	TArray<FDemoActorPriority> PrioritizedActors;

	/** Does the actual work of TickFlush, either on the main thread or in a task thread in parallel with Slate. */
	ENGINE_API void TickFlushInternal(float DeltaSeconds);

	/** Returns the last checkpoint time in integer milliseconds. */
	uint32 GetLastCheckpointTimeInMS() const { return ReplayHelper.GetLastCheckpointTimeInMS(); }

	/** Called during a normal demoFrame*/
	ENGINE_API void TickDemoRecordFrame(float DeltaSeconds);

	/** Config data for multicast RPCs we might want to skip recording. */
	UPROPERTY(config)
	TArray<FMulticastRecordOptions> MulticastRecordOptions;

public:

	// UNetDriver interface.

	ENGINE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual FString LowLevelGetNetworkNumber() override;
	ENGINE_API virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	ENGINE_API virtual bool InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error) override;
	ENGINE_API virtual void TickFlush(float DeltaSeconds) override;
	ENGINE_API virtual void PostTickFlush() override;
	ENGINE_API virtual void TickDispatch(float DeltaSeconds) override;
	ENGINE_API virtual void ProcessRemoteFunction(class AActor* Actor, class UFunction* Function, void* Parameters, struct FOutParmRec* OutParms, struct FFrame* Stack, class UObject* SubObject = nullptr) override;
	virtual bool IsAvailable() const override { return true; }
	ENGINE_API void SkipTime(const float InTimeToSkip);
	
	UE_DEPRECATED(5.3, "Internal call will be made private in the future.")
	ENGINE_API void SkipTimeInternal(const float SecondsToSkip, const bool InFastForward, const bool InIsForCheckpoint);
	UE_DEPRECATED(5.3, "Internal call will be made private in the future.")
	ENGINE_API bool InitConnectInternal(FString& Error);

	ENGINE_API virtual bool ShouldClientDestroyTearOffActors() const override;
	ENGINE_API virtual bool ShouldSkipRepNotifies() const override;
	ENGINE_API virtual bool ShouldQueueBunchesForActorGUID(FNetworkGUID InGUID) const override;
	ENGINE_API virtual bool ShouldIgnoreRPCs() const override;
	ENGINE_API virtual FNetworkGUID GetGUIDForActor(const AActor* InActor) const override;
	ENGINE_API virtual AActor* GetActorForGUID(FNetworkGUID InGUID) const override;
	ENGINE_API virtual bool ShouldReceiveRepNotifiesForObject(UObject* Object) const override;
	ENGINE_API virtual void ForceNetUpdate(AActor* Actor) override;
	ENGINE_API virtual bool IsServer() const override;
	ENGINE_API virtual bool ShouldReplicateFunction(AActor* Actor, UFunction* Function) const override;
	ENGINE_API virtual bool ShouldReplicateActor(AActor* Actor) const override;
	ENGINE_API virtual bool ShouldForwardFunction(AActor* Actor, UFunction* Function, void* Parms) const override;
	ENGINE_API virtual void NotifyActorChannelOpen(UActorChannel* Channel, AActor* Actor) override;
	ENGINE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason) override;
	ENGINE_API virtual void NotifyActorClientDormancyChanged(AActor* Actor, ENetDormancy OldDormancyState) override;
	ENGINE_API virtual void NotifyActorTornOff(AActor* Actor) override;

	virtual void ProcessLocalServerPackets() override {}
	virtual void ProcessLocalClientPackets() override {}

	ENGINE_API virtual void InitDestroyedStartupActors() override;

	ENGINE_API virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider) override;

	virtual void LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits) override {}
	virtual class ISocketSubsystem* GetSocketSubsystem() override { return nullptr; }

	virtual bool DoesSupportEncryption() const override { return false; }

protected:
	ENGINE_API virtual UChannel* InternalCreateChannelByName(const FName& ChName) override;

public:
	/** Called when we are already recording but have traveled to a new map to start recording again */
	ENGINE_API bool ContinueListen(FURL& ListenURL);

	/** 
	 * Scrubs playback to the given time. 
	 * 
	 * @param TimeInSeconds
	 * @param InOnGotoTimeDelegate		Delegate to call when finished. Will be called only once at most.
	*/
	ENGINE_API void GotoTimeInSeconds(const float TimeInSeconds, const FOnGotoTimeDelegate& InOnGotoTimeDelegate = FOnGotoTimeDelegate());

	ENGINE_API bool IsRecording() const;
	ENGINE_API bool IsPlaying() const;

	/** Total time of demo in seconds */
	float GetDemoTotalTime() const { return ReplayHelper.DemoTotalTime; }

	void SetDemoTotalTime(float TotalTime)
	{
		ReplayHelper.DemoTotalTime = TotalTime;
	}

	/** Current record/playback position in seconds */
	float GetDemoCurrentTime() const { return ReplayHelper.DemoCurrentTime; }

	void SetDemoCurrentTime(float CurrentTime)
	{
		ReplayHelper.DemoCurrentTime = CurrentTime;
	}

	FString GetDemoURL() const { return ReplayHelper.DemoURL.ToString(); }

	/** Sets the desired maximum recording time in milliseconds. */
	void SetMaxDesiredRecordTimeMS(const float InMaxDesiredRecordTimeMS) { MaxDesiredRecordTimeMS = InMaxDesiredRecordTimeMS; }

	float GetMaxDesiredRecordTimeMS() const { return MaxDesiredRecordTimeMS; }

	/** Sets the controller to use as the viewpoint for recording prioritization purposes. */
	void SetViewerOverride(APlayerController* const InViewerOverride ) { ViewerOverride = InViewerOverride; }

	/** Enable or disable prioritization of actors for recording. */
	void SetActorPrioritizationEnabled(const bool bInPrioritizeActors) { bPrioritizeActors = bInPrioritizeActors; }

	bool IsActorPrioritizationEnabled() const { return bPrioritizeActors; }

	/** Sets CheckpointSaveMaxMSPerFrame. */
	void SetCheckpointSaveMaxMSPerFrame(const float InCheckpointSaveMaxMSPerFrame)
	{ 
		CheckpointSaveMaxMSPerFrame = InCheckpointSaveMaxMSPerFrame; 
		ReplayHelper.CheckpointSaveMaxMSPerFrame = InCheckpointSaveMaxMSPerFrame;
	}

	/** Called by a task thread if the engine is doing async end of frame tasks in parallel with Slate. */
	ENGINE_API void TickFlushAsyncEndOfFrame(float DeltaSeconds);

	ENGINE_API const TArray<FLevelNameAndTime>& GetLevelNameAndTimeList();

	/** Returns the replicated state of every object on a current actor channel. Use the result to compare in DiffReplicatedProperties. */
	ENGINE_API FDemoSavedPropertyState SavePropertyState() const;

	/** Compares the values of replicated properties stored in State with the current values of the object replicators. Logs and returns true if there were any differences. */
	ENGINE_API bool ComparePropertyState(const FDemoSavedPropertyState& State) const;

public:
	/** @return true if the net resource is valid or false if it should not be used */
	virtual bool IsNetResourceValid(void) override { return true; }

	ENGINE_API void TickDemoRecord(float DeltaSeconds);
	ENGINE_API void PauseChannels(const bool bPause);
	ENGINE_API void PauseRecording(const bool bInPauseRecording);
	ENGINE_API bool IsRecordingPaused() const;

	ENGINE_API bool ConditionallyProcessPlaybackPackets();
	ENGINE_API void ProcessAllPlaybackPackets();

	ENGINE_API bool ConditionallyReadDemoFrameIntoPlaybackPackets(FArchive& Ar);

	ENGINE_API bool ProcessPacket(const uint8* Data, int32 Count);
	bool ProcessPacket(const FPlaybackPacket& PlaybackPacket)
	{
		return ShouldSkipPlaybackPacket(PlaybackPacket) ||
				ProcessPacket(PlaybackPacket.Data.GetData(), PlaybackPacket.Data.Num());
	}
	
	ENGINE_API void WriteDemoFrameFromQueuedDemoPackets(FArchive& Ar, TArray<FQueuedDemoPacket>& QueuedPackets, float FrameTime, EWriteDemoFrameFlags Flags);

	ENGINE_API void WritePacket(FArchive& Ar, uint8* Data, int32 Count);

	ENGINE_API void TickDemoPlayback(float DeltaSeconds);
	
	ENGINE_API void FinalizeFastForward(const double StartTime);
	
	ENGINE_API void SpawnDemoRecSpectator( UNetConnection* Connection, const FURL& ListenURL);

	/**
	 * Restores the given player controller so that it properly points to the given NetConnection
	 * after scrubbing when viewing a replay.
	 *
	 * @param PC			The PlayerController to set up the given NetConnection for
	 * @param NetConnection	The NetConnection to be assigned to the player controller.
	 */
	ENGINE_API void RestoreConnectionPostScrub(APlayerController* PC, UNetConnection* NetConnection);

	/**
	 * Sets the main spectator controller to be used and adds them to the spectator control array
	 *
	 * @param PC			The PlayerController to set the main controller param to.
	 */
	ENGINE_API void SetSpectatorController(APlayerController* PC);
	
	// Splitscreen demo handling

	/**
	 * Creates a new splitscreen replay viewer.
	 *
	 * @param NewPlayer		The LocalPlayer in control of this new viewer
	 * @param InWorld		The world to spawn the new viewer in.
	 *
	 * @return If the viewer was able to be created or not.
	 */
	ENGINE_API bool SpawnSplitscreenViewer(ULocalPlayer* NewPlayer, UWorld* InWorld);

	/**
	 * Removes a splitscreen demo viewer and cleans up its connection.
	 *
	 * @param RemovePlayer		The PlayerController to remove from the replay system
	 * @param bMarkOwnerForDeletion		If this function should handle deleting the given player as well.
	 *
	 * @return If the player was successfully removed from the replay.
	 */
	ENGINE_API bool RemoveSplitscreenViewer(APlayerController* RemovePlayer, bool bMarkOwnerForDeletion=false);

private:
	// Internal splitscreen management

	/** An array of all the spectator controllers (the main one and all splitscreen ones) that currently exist */
	UPROPERTY(transient)
	TArray<TObjectPtr<APlayerController>> SpectatorControllers;

	/**
	 * Removes all child connections for splitscreen viewers.
	 * This should be done before the ClientConnections or ServerConnection
	 * variables change or during most travel scenarios.
	 *
	 * @param bDeleteOwner	If the connections should delete the owning actor to the connection
	 *
	 * @return The number of splitscreen connections cleaned up.
	 */
	ENGINE_API int32 CleanUpSplitscreenConnections(bool bDeleteOwner);

public:

	ENGINE_API void ResetDemoState();
	ENGINE_API void JumpToEndOfLiveReplay();
	ENGINE_API void AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data);
	ENGINE_API void AddOrUpdateEvent(const FString& EventName, const FString& Group, const FString& Meta, const TArray<uint8>& Data);

	ENGINE_API void EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate);

	// In most cases, this is desirable over EnumerateEvents because it will explicitly use ActiveReplayName
	// instead of letting the streamer decide.
	ENGINE_API void EnumerateEventsForActiveReplay(const FString& Group, const FEnumerateEventsCallback& Delegate);
	ENGINE_API void EnumerateEventsForActiveReplay(const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate);

	ENGINE_API void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate);

	// In most cases, this is desirable over RequestEventData because it will explicitly use ActiveReplayName
	// instead of letting the streamer decide.
	ENGINE_API void RequestEventDataForActiveReplay(const FString& EventID, const FRequestEventDataCallback& Delegate);
	ENGINE_API void RequestEventDataForActiveReplay(const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate);

	/** Retrieve data for all events matching the requested group, and call the passed in delegate on completion. */
	ENGINE_API void RequestEventGroupDataForActiveReplay(const FString& Group, const FRequestEventGroupDataCallback& Delegate);
	ENGINE_API void RequestEventGroupDataForActiveReplay(const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate);

	bool IsFastForwarding() const { return bIsFastForwarding; }
	bool IsFinalizingFastForward() const { return bIsFinalizingFastForward; }
	bool IsRestoringStartupActors() const { return bIsRestoringStartupActors; }

	ENGINE_API FReplayExternalDataArray* GetExternalDataArrayForObject(UObject* Object);

	ENGINE_API bool SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits);

	ENGINE_API bool ReadDemoFrameIntoPlaybackPackets(FArchive& Ar, TArray<FPlaybackPacket>& Packets, const bool bForLevelFastForward, float* OutTime);
	bool ReadDemoFrameIntoPlaybackPackets(FArchive& Ar) { return ReadDemoFrameIntoPlaybackPackets(Ar, PlaybackPackets, false, nullptr); }

	/**
	 * Adds a join-in-progress user to the set of users associated with the currently recording replay (if any)
	 *
	 * @param UserString a string that uniquely identifies the user, usually their FUniqueNetId
	 */
	ENGINE_API void AddUserToReplay(const FString& UserString);

	ENGINE_API void StopDemo();

	ENGINE_API void ReplayStreamingReady(const FStartStreamingResult& Result);

	ENGINE_API void AddReplayTask(FQueuedReplayTask* NewTask);
	ENGINE_API bool IsAnyTaskPending() const;
	ENGINE_API void ClearReplayTasks();
	ENGINE_API bool ProcessReplayTasks();
	ENGINE_API bool IsNamedTaskInQueue(const FName& Name) const;
	ENGINE_API FName GetNextQueuedTaskName() const;

	/** If a channel is associated with Actor, adds the channel's GUID to the list of GUIDs excluded from queuing bunches during scrubbing. */
	ENGINE_API void AddNonQueuedActorForScrubbing(AActor const* Actor);
	/** Adds the channel's GUID to the list of GUIDs excluded from queuing bunches during scrubbing. */
	ENGINE_API void AddNonQueuedGUIDForScrubbing(FNetworkGUID InGUID);

	ENGINE_API virtual bool IsLevelInitializedForActor(const AActor* InActor, const UNetConnection* InConnection) const override;

	/** Called when a "go to time" operation is completed. */
	ENGINE_API void NotifyGotoTimeFinished(bool bWasSuccessful);

	ENGINE_API virtual void NotifyActorDestroyed(AActor* ThisActor, bool IsSeamlessTravel=false, EChannelCloseReason CloseReason = EChannelCloseReason::Destroyed) override;
	ENGINE_API virtual void NotifyActorLevelUnloaded(AActor* Actor) override;
	ENGINE_API virtual void NotifyStreamingLevelUnload(ULevel* InLevel) override;

	/** Call this function during playback to track net startup actors that need a hard reset when scrubbing, which is done by destroying and then re-spawning */
	ENGINE_API virtual void QueueNetStartupActorForRollbackViaDeletion(AActor* Actor);

	/** Called when seamless travel begins when recording a replay. */
	ENGINE_API void OnSeamlessTravelStartDuringRecording(const FString& LevelName);

	/** @return the unique identifier for the lifetime of this object. */
	const FString& GetDemoSessionID() const { return DemoSessionID; }

	/** Returns true if TickFlush can be called in parallel with the Slate tick. */
	ENGINE_API bool ShouldTickFlushAsyncEndOfFrame() const;

	/** Returns whether or not this replay was recorded / is playing with Level Streaming fixes. */
	bool HasLevelStreamingFixes() const
	{
		return ReplayHelper.HasLevelStreamingFixes();
	}

	/** Returns whether or not this replay was recorded / is playing with delta checkpoints. */
	bool HasDeltaCheckpoints() const 
	{
		return ReplayHelper.HasDeltaCheckpoints();
	}

	/** Returns whether or not this replay was recorded / is playing with the game specific per frame data feature. */
	bool HasGameSpecificFrameData() const
	{
		return ReplayHelper.HasGameSpecificFrameData();
	}

	/**
	 * Gets the actively recording or playback replay (stream) name.
	 * Note, this will be empty when not recording or playing back.
	 */
	const FString& GetActiveReplayName() const
	{
		return ReplayHelper.ActiveReplayName;
	}

	uint32 GetPlaybackDemoChangelist() const { return ReplayHelper.PlaybackDemoHeader.EngineVersion.GetChangelist(); }

	UE_DEPRECATED(5.2, "Will be removed in favor of custom versions, use GetPlaybackReplayVersion instead")
	uint32 GetPlaybackDemoVersion() const 
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ReplayHelper.PlaybackDemoHeader.Version; 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FReplayCustomVersion::Type GetPlaybackReplayVersion() const { return ReplayHelper.GetPlaybackReplayVersion(); }

	uint32 GetPlaybackEngineNetworkProtocolVersion() const { return ReplayHelper.PlaybackDemoHeader.GetCustomVersion(FEngineNetworkCustomVersion::Guid); }
	uint32 GetPlaybackGameNetworkProtocolVersion() const { return ReplayHelper.PlaybackDemoHeader.GetCustomVersion(FGameNetworkCustomVersion::Guid); }

	uint32 GetPlaybackCustomVersionVersion(const FGuid& VersionGuid) const { return ReplayHelper.PlaybackDemoHeader.GetCustomVersion(VersionGuid); }

	ENGINE_API FString GetDemoPath() const;

private:

	struct FLevelnterval
	{
		int32 Priority;
		int32 StartIndex;
		int32 Count;
		int32 LevelIndex;
	};

	TArray<FLevelnterval> LevelIntervals;

	ENGINE_API void BuildSortedLevelPriorityOnLevels(const TArray<FDemoActorPriority>& PrioritizedActorList, TArray<FLevelnterval>& OutLevelIntervals);

	/** Called when the downloading header request from the replay streamer completes. */
	ENGINE_API void OnRefreshHeaderCompletePrivate(const FDownloadHeaderResult& Result, int32 LevelIndex);

	ENGINE_API void CleanupOutstandingRewindActors();

	// Tracks actors that will need to be rewound during scrubbing.
	// This list should always be empty outside of scrubbing.
	TSet<FNetworkGUID> TrackedRewindActorsByGUID;

	// Time of the last packet we've processed (in seconds).
	float LastProcessedPacketTime;

	// Index into PlaybackPackets array. Used so we can process many packets in one frame and avoid removing them individually.
	int32 PlaybackPacketIndex;

	// Determines whether or not a packet should be skipped, based on it's level association.
	ENGINE_API bool ShouldSkipPlaybackPacket(const FPlaybackPacket& Packet);

	/**
	 * Replicates the given prioritized actors, so their packets can be captured for recording.
	 * This should be used for normal frame recording.
	 * @see ReplicateCheckpointActor for recording during checkpoints.
	 *
	 * @param ToReplicate	The actors to replicate.
	 * @param Params		Implementation specific params necessary to replicate the actor.
	 *
	 * @return True if there is time remaining to replicate more actors. False otherwise.
	 */
	ENGINE_API bool ReplicatePrioritizedActors(const FDemoActorPriority* ActorsToReplicate, uint32 Count, class FRepActorsParams& Params);
	ENGINE_API bool ReplicatePrioritizedActor(const FActorPriority& ActorPriority, class FRepActorsParams& Params);

	friend class FPendingTaskHelper;

	// Manages basic setup of newly visible levels, and queuing a FastForward task if necessary.
	ENGINE_API void PrepFastForwardLevels();

	// Performs the logic for actually fast-forwarding a level.
	ENGINE_API bool FastForwardLevels(const FGotoResult& GotoResult);

	ENGINE_API void OnPostLoadMapWithWorld(UWorld* World);

	// Diff Actor plus it's Components and Subobjects from given ActorChannel
	ENGINE_API void DiffActorProperties(UActorChannel* const ActorChannel);

	// Callback sent just before an actor has destroy called on itself.
	ENGINE_API void OnActorPreDestroy(AActor* DestroyedActor);

	FDelegateHandle DelegateHandleActorPreDestroy;

protected:

	ENGINE_API void ProcessSeamlessTravel(int32 LevelIndex);

	ENGINE_API bool DemoReplicateActor(AActor* Actor, UNetConnection* Connection, bool bMustReplicate);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Please use NotifyDemoPlaybackError instead")
	ENGINE_API void NotifyDemoPlaybackFailure(EDemoPlayFailure::Type FailureType);
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	void NotifyDemoPlaybackError(const UE::Net::TNetResult<EReplayResult>& Result);
	ENGINE_API void NotifyDemoRecordFailure(const UE::Net::TNetResult<EReplayResult>& Result);

	TArray<FQueuedDemoPacket> QueuedPacketsBeforeTravel;

	int64 MaxArchiveReadPos;

private:

	// Max percent of time to spend building consider lists / prioritizing actors
	// for demo recording. Only used if MaxDesiredRecordTimeMS > 0.
	float RecordBuildConsiderAndPrioritizeTimeSlice;

	// Max percent of time to spend replicating prioritized destruction infos. Only used if MaxDesiredRecordTimeMS > 0.
	float RecordDestructionInfoReplicationTimeSlice;

	ENGINE_API void AdjustConsiderTime(const float ReplicatedPercent);

	ENGINE_API bool ProcessFastForwardPackets(TArrayView<FPlaybackPacket> Packets, const TSet<int32>& LevelIndices);
	ENGINE_API void ProcessPlaybackPackets(TArrayView<FPlaybackPacket> Packets);

	virtual ECreateReplicationChangelistMgrFlags GetCreateReplicationChangelistMgrFlags() const override
	{
		return ECreateReplicationChangelistMgrFlags::SkipDeltaCustomState;
	}

	TUniquePtr<struct FDemoBudgetLogHelper> BudgetLogHelper;

//////////////////////////////////////////////////////////////////////////
// Replay frame fidelity
public:
	// Simplified rating of replay frame fidelity as percentage of actors that were replicated.
	// [0..1] where 0 means nothing was recorded this frame and 1 means full fidelity.
	// This treats all actors equally. Assuming more important actors are prioritized higher, in general actual "fidelity"
	// is expected to be higher than reported, which should be fine for detecting low-fidelity frame/intervals in replay file.
	float GetLastReplayFrameFidelity() const
	{
		return LastReplayFrameFidelity;
	}

private:
	TAtomic<float> LastReplayFrameFidelity{ 0 };

	FReplayHelper ReplayHelper;

	/** Enabled via -skipreplayrollback and causes rollback data to not be generated, with the assumption that there will be no scrubbing. */
	bool bSkipStartupActorRollback = false;

	friend class UDemoNetConnection;
	friend class FQueuedReplayTask;
};
