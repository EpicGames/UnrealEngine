// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/PendingNetGame.h"
#include "DemoPendingNetGame.generated.h"

class UEngine;
struct FWorldContext;

UCLASS(transient, config=Engine)
class UDemoPendingNetGame
	: public UPendingNetGame
{
	GENERATED_UCLASS_BODY()

	// UPendingNetGame interface.
	virtual void Tick( float DeltaTime ) override;
	virtual void SendJoin() override;
	virtual bool LoadMapCompleted(UEngine* Engine, FWorldContext& Context, bool bLoadedMapSuccessfully, const FString& LoadMapError) override;
};
