// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Light.h"
#include "DirectionalLight.generated.h"

class UArrowComponent;
class UDirectionalLightComponent;

/**
 * Implements a directional light actor.
 */
UCLASS(ClassGroup=(Lights, DirectionalLights), MinimalAPI, meta=(ChildCanTick))
class ADirectionalLight
	: public ALight
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	// Reference to editor visualization arrow
private:
	UPROPERTY()
	TObjectPtr<UArrowComponent> ArrowComponent;

	/* EditorOnly reference to the light component to allow it to be displayed in the details panel correctly */
	UPROPERTY(VisibleAnywhere, Category="Light")
	TObjectPtr<UDirectionalLightComponent> DirectionalLightComponent;
#endif

public:

	// UObject Interface
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override { OutRuntimeBounds.Init(); OutEditorBounds.Init(); }
#endif

public:

#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	UArrowComponent* GetArrowComponent() const { return ArrowComponent; }

	/** Returns SkyAtmosphereComponent subobject */
	UDirectionalLightComponent* GetComponent() const { return DirectionalLightComponent; }
#endif
};
