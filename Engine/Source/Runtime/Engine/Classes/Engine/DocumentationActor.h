// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "DocumentationActor.generated.h"

namespace EDocumentationActorType
{
	enum Type
	{
		// The link is unset
		None,
		// A UDN document link
		UDNLink,
		// A URL
		URLLink,
		// the link is invalid
		InvalidLink
	};
}


UCLASS(hidecategories = (Sprite, MaterialSprite, Actor, Transform, Tags, Materials, Rendering, Components, Blueprint, bject, Collision, Display, Rendering, Physics, Input, Lighting, Layers), MinimalAPI)
class ADocumentationActor
	: public AActor
{
	GENERATED_UCLASS_BODY()

	/* Open the document link this actor relates to */
	ENGINE_API bool OpenDocumentLink() const;
	
	/* Is the document link this actor has valid */
	ENGINE_API bool HasValidDocumentLink() const;

	// Actor interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	ENGINE_API virtual void PostLoad() override;
#endif
	// End of Actor interface

#if WITH_EDITORONLY_DATA
	/** Link to a help document. */
	UPROPERTY(Category = HelpDocumentation, EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	FString DocumentLink; 
	
private:
	UPROPERTY(Category = Sprite, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UMaterialBillboardComponent> Billboard;
#endif

public:

	/** Determine the type of link contained in the DocumentLink member */
	ENGINE_API EDocumentationActorType::Type GetLinkType() const;
	
private:

	/** Update the member variable that days what type of link we have */
	ENGINE_API void UpdateLinkType();

	EDocumentationActorType::Type	LinkType;

public:

#if WITH_EDITORONLY_DATA
	/** Returns Billboard subobject **/
	class UMaterialBillboardComponent* GetBillboard() const { return Billboard; }
#endif
};
