// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DynamicBlueprintBinding.generated.h"

/**
 * Used to tie events to delegates at runtime
 * 
 * To use, override 'K2Node::GetDynamicBindingClass' to specify a child of this class.
 * Additionally, override 'K2Node::RegisterDynamicBinding'. During compilation, your node
 * will be given an instance of the specified binding class to use for binding delegates.
 */
UCLASS(abstract, MinimalAPI)
class UDynamicBlueprintBinding
	: public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void BindDynamicDelegates(UObject* InInstance) const { }
	virtual void UnbindDynamicDelegates(UObject* Instance) const { }
	virtual void UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty) const { }
};
