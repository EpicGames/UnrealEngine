// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *	This file is for shared structs and enums that need to be declared before the rest of Engine.
 *  This includes structs used by the renderer and task execution framework.
 */

#include "Async/TaskGraphFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/CoreMisc.h"
#include "Net/Core/Connection/NetEnums.h"
#include <atomic>

#include "EngineBaseTypes.generated.h"

class UActorComponent;
struct FSlateBrush;
struct FSlateIcon;
struct FTickContext;

/** Describes the type of input event received for a specific key or button */
UENUM( BlueprintType, meta=(ScriptName="InputEventType"))
enum EInputEvent : int
{
	IE_Pressed              =0,
	IE_Released             =1,
	IE_Repeat               =2,
	IE_DoubleClick          =3,
	IE_Axis                 =4,
	IE_MAX                  =5,
};

/** Describes how the application consumes mouse input events */
UENUM()
enum class EMouseCaptureMode : uint8
{
	/** Do not capture the mouse at all */
	NoCapture,
	/** Capture the mouse permanently when the viewport is clicked, and consume the initial mouse down that caused the capture so it isn't processed by player input */
	CapturePermanently,
	/** Capture the mouse permanently when the viewport is clicked, and allow player input to process the mouse down that caused the capture */
	CapturePermanently_IncludingInitialMouseDown,
	/** Capture the mouse during a mouse down, releases on mouse up */
	CaptureDuringMouseDown,
	/** Capture only when the right mouse button is down, not any of the other mouse buttons */
	CaptureDuringRightMouseDown,
};

/** Describes the behavior of a mouse cursor when inside the viewport window */
UENUM()
enum class EMouseLockMode : uint8
{
	/** Do not lock the mouse cursor to the viewport */
	DoNotLock,
	/** Only lock the mouse cursor to the viewport when the mouse is captured */
	LockOnCapture,
	/** Always lock the mouse cursor to the viewport */
	LockAlways,
	/** Lock the cursor if we're in fullscreen */
	LockInFullscreen,
};

/** Type of tick we wish to perform on the level */
enum ELevelTick
{
	/** Update the level time only. */
	LEVELTICK_TimeOnly = 0,
	/** Update time and viewports. */
	LEVELTICK_ViewportsOnly = 1,
	/** Update all. */
	LEVELTICK_All = 2,
	/** Delta time is zero, we are paused. Components don't tick. */
	LEVELTICK_PauseTick = 3,
};

/** Determines which ticking group a tick function belongs to. */
UENUM(BlueprintType)
enum ETickingGroup : int
{
	/** Any item that needs to be executed before physics simulation starts. */
	TG_PrePhysics UMETA(DisplayName="Pre Physics"),

	/** Special tick group that starts physics simulation. */							
	TG_StartPhysics UMETA(Hidden, DisplayName="Start Physics"),

	/** Any item that can be run in parallel with our physics simulation work. */
	TG_DuringPhysics UMETA(DisplayName="During Physics"),

	/** Special tick group that ends physics simulation. */
	TG_EndPhysics UMETA(Hidden, DisplayName="End Physics"),

	/** Any item that needs rigid body and cloth simulation to be complete before being executed. */
	TG_PostPhysics UMETA(DisplayName="Post Physics"),

	/** Any item that needs to be ticked after all normal gameplay tasks. */
	TG_PostUpdateWork UMETA(DisplayName="Post Update Work"),

	/** Last group which tick functions can be delayed into because of dependencies. */
	TG_LastDemotable UMETA(Hidden, DisplayName = "Last Demotable"),

	/** Special tick group that is not actually a tick group. After every tick group this is repeatedly re-run until there are no more newly spawned items to run. */
	TG_NewlySpawned UMETA(Hidden, DisplayName="Newly Spawned"),

	TG_MAX,
};

/** This is a small structure to define how a tick function depends on other tick functions. */
USTRUCT()
struct FTickPrerequisite
{
	GENERATED_BODY()

public:
	/**
	 * Tick functions are owned by UObjects, so we need a separate weak pointer to the UObject
	 * solely for the purpose of determining if PrerequisiteTickFunction is still valid.
	 */
	TWeakObjectPtr<class UObject> PrerequisiteObject;

	/** 
	 * Pointer to the actual tick function and must be completed prior to our tick running.
	 * @warning This should not be accessed directly, use Get() to validate the PrerequisiteObject first.
	 */
	struct FTickFunction*		PrerequisiteTickFunction;

	FTickPrerequisite()
		: PrerequisiteTickFunction(nullptr)
	{
	}

	/** 
	 * Constructor
	 * @param TargetObject - UObject containing this tick function. Only used to verify that the other pointer is still usable
	 * @param TargetTickFunction - Actual tick function to use as a prerequisite
	 */
	FTickPrerequisite(UObject* TargetObject, struct FTickFunction& TargetTickFunction)
		: PrerequisiteObject(TargetObject)
		, PrerequisiteTickFunction(&TargetTickFunction)
	{
		check(PrerequisiteTickFunction);
	}

	/** Equality operator, used to prevent duplicates and allow removal by value. */
	bool operator==(const FTickPrerequisite& Other) const
	{
		return PrerequisiteObject == Other.PrerequisiteObject &&
			PrerequisiteTickFunction == Other.PrerequisiteTickFunction;
	}

	/** Return the tick function, if it is still valid. Can be null if the tick function was null or the containing UObject is invalid. */
	struct FTickFunction* Get()
	{
		if (PrerequisiteObject.IsValid(true))
		{
			return PrerequisiteTickFunction;
		}
		return nullptr;
	}
	
	const struct FTickFunction* Get() const
	{
		if (PrerequisiteObject.IsValid(true))
		{
			return PrerequisiteTickFunction;
		}
		return nullptr;
	}
};

/**
 * Abstract base class for all tick functions.
 * Registered tick functions are executed by the functions in FTickTaskManagerInterface during the engine tick frame.
 * 
 * @warning Most of these functions are not threadsafe. The tick manager stores raw pointers to these structs 
 * and accesses them on the game thread, so registered functions cannot be safely destroyed during a tick frame.
 */
USTRUCT()
struct FTickFunction
{
	GENERATED_BODY()

public:
	// The following UPROPERTYs are for configuration and inherited from the CDO/archetype/blueprint etc

	/**
	 * Defines the first tick group where this function can be executed.
	 * These groups determine the relative order of when objects tick during a frame update.
	 * If this function has prerequisites, execution may be delayed to a later group.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Tick", AdvancedDisplay)
	TEnumAsByte<enum ETickingGroup> TickGroup;

	/**
	 * Defines the tick group that this tick function must finish within. 
	 * Normal synchronous ticks do not need to set this manually.
	 * The game thread will stall at the end of this group if the function (or a spawned task) is still in progress.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Tick", AdvancedDisplay)
	TEnumAsByte<enum ETickingGroup> EndTickGroup;

public:
	/** Bool indicating that this function should execute even if the gameplay is paused. Pause ticks are very limited in capabilities. */
	UPROPERTY(EditDefaultsOnly, Category="Tick", AdvancedDisplay)
	uint8 bTickEvenWhenPaused:1;

	/** If false, this tick function will never be registered and will never tick. Only settable in defaults. */
	UPROPERTY()
	uint8 bCanEverTick:1;

	/** If true, this tick function will start enabled, but can be disabled later on. */
	UPROPERTY(EditDefaultsOnly, Category="Tick")
	uint8 bStartWithTickEnabled:1;

	/** If we allow this tick to run on a dedicated server */
	UPROPERTY(EditDefaultsOnly, Category="Tick", AdvancedDisplay)
	uint8 bAllowTickOnDedicatedServer:1;

	/** True if we allow this tick to be combined with other ticks for improved performance */
	uint8 bAllowTickBatching:1;

	/** Run this tick first within the tick group, presumably to start async tasks that must be completed with this tick group, hiding the latency. */
	uint8 bHighPriority:1;

	/** If false, this tick will only run on the game thread. If true it can run on any thread in parallel with the game thread and with other async ticks and tasks */
	uint8 bRunOnAnyThread:1;

	/** (experimental) if true, the tick function will be run transactionally */
	uint8 bRunTransactionally:1;

	/** True if this is an event that will not execute until it has been manually dispatched (and prerequisites complete) instead of being dispatched at start of the tick group */
	uint8 bDispatchManually : 1;

private:
	/** For bDispatchManually functions, if this is set and the task is valid the task has been fully triggered and will execute */
	uint8 bWasDispatchedManually:1;

	enum class ETickState : uint8
	{
		// This tick will not execute in the next/current frame.
		Disabled,
		// This tick will execute normally.
		Enabled,
		// This tick has a tick interval that is counting down before the next execution.
		CoolingDown
	};

	/** Internal tick state, set by tick manager every frame. */
	ETickState TickState : 2;

public:
	/** The time in seconds between executions of this tick function. If <= 0 then it will tick every frame. */
	UPROPERTY(EditDefaultsOnly, Category="Tick", meta=(DisplayName="Tick Interval (secs)"))
	float TickInterval;

private:
	/** List of prerequisites for this tick function, modified by calling Add/RemovePrerequisite. */
	TArray<struct FTickPrerequisite> Prerequisites;

	/** Defines the internal state of a tick function, set in TickTaskManager */
	enum class ETickTaskState : uint8
	{
		// Has not been queued yet for the current frame, or has already executed
		NotQueued,
		// In the process of being queued, but no actual task yet
		Pending,
		// Has an actual task and a completion event, may be executing
		HasTask,
		// Directly points to a completion event, probably in a batched tick
		HasCompletionEvent,
	};

	/** Internal data structure that contains members only required for a registered tick function. */
	struct FInternalData
	{
		FInternalData();

		/** Whether the tick function is registered. */
		bool bRegistered : 1;

		/** Cache whether this function was rescheduled as an interval function by the TickManager */
		bool bWasInterval:1;

		/** Internal state, this determines what TaskPointer points to */
		ETickTaskState TaskState;

		/** Internal data that indicates the tick group we actually started in (it may have been delayed due to prerequisites) */
		TEnumAsByte<enum ETickingGroup> ActualStartTickGroup;

		/** Internal data that indicates the tick group we are guaranteed to end by (it may have been delayed due to prerequisites) */
		TEnumAsByte<enum ETickingGroup> ActualEndTickGroup;
		
		/** Internal data to track if we have started visiting this tick function yet this frame, smaller than full frame counter */
		std::atomic<uint32> TickVisitedGFrameCounter;

		/** Internal data to track if we have finished visiting this tick function yet this frame, smaller than full frame counter */
		std::atomic<uint32> TickQueuedGFrameCounter;
	
		/** Pointer to a type determined by TaskState, do not access directly */
		void* TaskPointer;

		/** The next function in the cooling down list for ticks with an interval */
		FTickFunction* Next;

		/** 
		 * If TickInterval is greater than 0 and the TickState is CoolingDown, this is the time, 
		 * relative to the element ahead of it in the cooling down list, remaining until the next time this function will tick 
		 */
		float RelativeTickCooldown;

		/** 
		 * The last world game time at which we were ticked, uses GetUnpausedTimeSeconds for bTickEvenWhenPaused.
		 * Valid only if we've been ticked at least once since having a tick interval, otherwise set to -1.
		 */
		double LastIntervalTickSeconds;

		/** Back pointer to the FTickTaskLevel managing this tick function, if it is currently registered */
		class FTickTaskLevel* TickTaskLevel;
	};

	/** Lazily allocated struct that contains the necessary data for a tick function that is registered */
	TUniquePtr<FInternalData> InternalData;

public:
	/** Default constructor, initializes to disabled. */
	ENGINE_API FTickFunction();

	/** Destructor, unregisters the tick function if it is still registered. */
	ENGINE_API virtual ~FTickFunction();

	/** 
	 * Adds the tick function to the primary list of tick functions for a specific level.
	 * If called during a tick frame, this will tick in the next tick group.
	 * 
	 * @warning This function is not threadsafe and must be called from the game thread.
	 */
	ENGINE_API void RegisterTickFunction(class ULevel* Level);

	/**
	 * Removes the tick function from the list of tick functions to automatically execute.
	 * This does not cancel ticks that have already been scheduled in the current tick frame.
	 * 
	 * @warning This function is not threadsafe and must be called from the game thread.
	 */
	ENGINE_API void UnRegisterTickFunction();

	/** 
	 * See if the tick function is currently registered to a level in the tick manager.
	 * If this is true and the tick is also enabled, the tick manager will automatically schedule it for execution.
	 */
	bool IsTickFunctionRegistered() const { return (InternalData && InternalData->bRegistered); }

	/** 
	 * Enables or disables this tick function. Disabled tick functions will never execute, even if registered.
	 * Enabling during a tick frame will tick it in the next tick group if it has not already ticked this frame.
	 * Disabling during a tick frame will stop it from executing unless it is already in progress.
	 * 
	 * @warning This function is not threadsafe to call on a registered tick function and must be called from the game thread. 
	 */
	ENGINE_API void SetTickFunctionEnable(bool bInEnabled);

	/** Returns whether the tick function is currently enabled and is possible to execute. */
	bool IsTickFunctionEnabled() const { return TickState != ETickState::Disabled; }

	/** 
	 * Update tick interval in the system and overwrite the current cooldown if any.
	 * This may not affect the current tick frame.
	 *
	 * @warning This function is not threadsafe to call on a registered tick function and must be called from the game thread. 
	 */
	ENGINE_API void UpdateTickIntervalAndCoolDown(float NewTickInterval);

	/** 
	 * Returns true if GetCompletionHandle will return a valid completion handle.
	 * This is only true for functions that are registered and have not finished ticking in the current frame.
	 */
	ENGINE_API bool IsCompletionHandleValid() const;

	/**
	 * Gets the current completion handle of this tick function, so it can be delayed until a later point when some additional
	 * tasks have been completed. Only valid after StartFrame has been called and then only until the TickFunction finishes
	 * execution. This returns a reference so IsCompletionHandleValid must be called first.
	 * The returned handle can be safely accessed on any thread to coordinate with other async tasks.
	 */
	ENGINE_API FGraphEventRef GetCompletionHandle() const;

	/**
	 * Returns true if bDispatchManually is set and this has a task that can be triggered right now.
	 */
	ENGINE_API virtual bool CanDispatchManually() const;

	/** 
	 * If bDispatchManually is set, attempt to start the task based on some event during a frame.
	 * It will return true if it worked, which may have executed immediately or may be waiting for a prerequisite.
	 */
	ENGINE_API virtual bool DispatchManually();

	/**
	 * Gets the actual tick group that this function will start executing in.
	 * This is only guaranteed to be correct after StartFrame has been called through the end of the frame.
	 */
	TEnumAsByte<enum ETickingGroup> GetActualTickGroup() const
	{
		return (InternalData ? InternalData->ActualStartTickGroup : TickGroup);
	}

	/**
	 * Returns the last tick group this function could complete within.
 	 * This is only guaranteed to be correct after StartFrame has been called through the end of the frame.
	 */
	TEnumAsByte<enum ETickingGroup> GetActualEndTickGroup() const
	{
		return (InternalData ? InternalData->ActualEndTickGroup : EndTickGroup);
	}

	/** 
	 * Adds a tick function to the list of prerequisites. This adds the requirement that TargetTickFunction is called before this tick function.
	 * This prerequisite will only take effect if the TargetTickFunction is registered and enabled, and will not affect the current tick frame.
	 * @param TargetObject - UObject containing TargetTickFunction, only used to verify that the other pointer is still usable.
	 * @param TargetTickFunction - Actual tick function to use as a prerequisite.
	 */
	ENGINE_API void AddPrerequisite(UObject* TargetObject, struct FTickFunction& TargetTickFunction);

	/** 
	 * Removes a prerequisite that was previously added. This will not affect the current tick frame.
	 * @param TargetObject - UObject containing this tick function, only used to verify that the other pointer is still usable.
	 * @param TargetTickFunction - Actual tick function to use as a prerequisite.
	 */
	ENGINE_API void RemovePrerequisite(UObject* TargetObject, struct FTickFunction& TargetTickFunction);

	/**
	 * Sets this function to normal or high priority, and also modifies all prerequisites recursively.
	 *
	 * @warning This only affects current prerequisites, prerequisites that are added later will not inherit the priority.
	 */
	ENGINE_API void SetPriorityIncludingPrerequisites(bool bInHighPriority);

	/**
	 * Return a reference to the prerequisites list for this tick function.
	 *
	 * @warning Modifying this can create invalid data, use Add/RemovePrerequisite for normal use cases.
	 */
	TArray<struct FTickPrerequisite>& GetPrerequisites()
	{
		return Prerequisites;
	}

	/**
	 * Return a reference to the prerequisites list for this tick function (const).
	 */
	const TArray<struct FTickPrerequisite>& GetPrerequisites() const
	{
		return Prerequisites;
	}

	/**
	 * For actively registered tick functions with an interval, return the last time it ticked.
	 * For unregistered or non-interval functions it will return -1.
	 */
	double GetLastIntervalTickGameTime() const { return (InternalData ? InternalData->LastIntervalTickSeconds : -1.0); }

	UE_DEPRECATED(5.8, "Renamed because function only returns the correct time for functions with a TickInterval")
	double GetLastTickGameTime() const { return (InternalData ? InternalData->LastIntervalTickSeconds : -1.0); }

	/** 
	 * Correctly call ExecuteTick on a nested function. This will set the completion event properly for the duration of the tick function.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for delaying the completion of this task until child tasks are complete.
	 */
	ENGINE_API void ExecuteNestedTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

private:
	/** Returns the current internal task only if it matches required state */
	void* GetTaskPointer(ETickTaskState RequiredState) const
	{
		return (InternalData && InternalData->TaskState == RequiredState ? InternalData->TaskPointer : nullptr);
	}

	/** Directly set the task pointer and state */
	void SetTaskPointer(ETickTaskState NewState, void* InTaskPointer);

	/**
	 * Queues a tick function for execution from the game thread
	 * @param TickContext - context to tick in
	 */
	void QueueTickFunction(class FTickTaskSequencer& TTS, const FTickContext& TickContext);

	/**
	 * Queues a tick function for execution from the game thread
	 * @param TickContext - context to tick in
	 * @param StackForCycleDetection - Stack For Cycle Detection
	 */
	void QueueTickFunctionParallel(const FTickContext& TickContext, TArray<FTickFunction*, TInlineAllocator<8> >& StackForCycleDetection);

public:
	// Functions to be used by executing tasks, don't call these directly from user code

	/** Returns the delta time to use for ExecuteTick using the global delta time and world. This also updates internal tracking data. */
	ENGINE_API float CalculateDeltaTime(float DeltaTime, const class UWorld* TickingWorld);

	/** Logs the DiagnosticMessage and other function info. */
	ENGINE_API void LogTickFunction(ENamedThreads::Type CurrentThread, bool bLogPrerequisites, int32 Indent = 0);

	/** Logs the prerequisite info specifically. */
	ENGINE_API void ShowPrerequistes(int32 Indent = 1);

	/** Clear any current task information such as the completion handle. This must be called after execution. */
	ENGINE_API void ClearTaskInformation();

	/** 
	 * Override this abstract function to actually execute the tick. Batched tick managers should use ExecuteNestedTick.
	 * If this function sets bRunOnAnyThread, this function could execute on a background thread.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for delaying the completion of this task until child tasks are complete.
	 */
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) PURE_VIRTUAL(,);
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FName CachedDiagnosticContext;
	FString CachedDiagnosticMessage;
#endif

	/** Override this abstract function to return a human readable string used for debug commands. */
	virtual FString DiagnosticMessage() PURE_VIRTUAL(, return TEXT("DiagnosticMessage() not implemented"););
	
	/** Override this function to return an internal name used to group related tick functions. */
	virtual FName DiagnosticContext(bool bDetailed)
	{
		return NAME_None;
	}

	/**
	 * To properly expose nested/batched tick functions to the logging and debugging systems of the tick manager, 
	 * override this to execute InFunc for every nested tick function that would be executed as part of this tick.
	 */
	virtual void ForEachNestedTick(TFunctionRef<void(FTickFunction&)> InFunc) const {}
	
	friend class FTickTaskSequencer;
	friend class FTickTaskManager;
	friend class FTickTaskLevel;

	// It is unsafe to copy FTickFunctions and any subclasses of FTickFunction should specify the type trait WithCopy = false
	FTickFunction& operator=(const FTickFunction&) = delete;
};

template<>
struct TStructOpsTypeTraits<FTickFunction> : public TStructOpsTypeTraitsBase2<FTickFunction>
{
	enum
	{
		WithCopy = false,
		WithPureVirtual = true
	};
};

/** Tick function that calls AActor::TickActor. */
USTRUCT()
struct FActorTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** Actor that is the target of this tick **/
#if UE_WITH_REMOTE_OBJECT_HANDLE
	TObjectPtr<AActor> Target;
#else
	class AActor*	Target;
#endif

	/** 
	 * Override this function to execute an actor tick.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for delaying the completion of this task until child tasks are complete.
	 */
	ENGINE_API virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;

	/** Returns the name of the owning actor, override for other types of ticks. */
	ENGINE_API virtual FString DiagnosticMessage() override;

	/** Returns the name of the owning actor class. */
	ENGINE_API virtual FName DiagnosticContext(bool bDetailed) override;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** Exposes references to GC system */
	ENGINE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
#endif
};

template<>
struct TStructOpsTypeTraits<FActorTickFunction> : public TStructOpsTypeTraitsBase2<FActorTickFunction>
{
	enum
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WithAddStructReferencedObjects = true,
#endif
		WithCopy = false
	};
};

/** Tick function that calls UActorComponent::ConditionalTick. */
USTRUCT()
struct FActorComponentTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

#if UE_WITH_REMOTE_OBJECT_HANDLE
	TObjectPtr<class UActorComponent> Target;
#else
	/** Actor component that is the target of this tick */
	class UActorComponent*	Target;
#endif

	/** 
	 * Override this function to execute an actor component tick
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for delaying the completion of this task until child tasks are complete.
	 */
	ENGINE_API virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;

	/** Returns the name of the owning componet, override for other types of ticks. */
	ENGINE_API virtual FString DiagnosticMessage() override;

	/** Returns the name of the owning component class. */
	ENGINE_API virtual FName DiagnosticContext(bool bDetailed) override;

	/**
	 * Helper function for executing component tick functions using the same conditions as FActorTickFunction.
	 * This adds a cycle counter scope which is used for profiling.
	 * @param Target - the actor component we are ticking
	 * @param bTickInEditor - whether the target wants to tick in the editor
	 * @param DeltaTime - The time since the last tick.
	 * @param TickType - Type of tick that we are running
	 * @param ExecuteTickFunc - the lambda that ultimately calls tick on the actor component
	 */
	template <typename ExecuteTickLambda>
	static void ExecuteTickHelper(UActorComponent* Target, bool bTickInEditor, float DeltaTime, ELevelTick TickType, const ExecuteTickLambda& ExecuteTickFunc);	

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** Exposes references to GC system */
	ENGINE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
#endif
};

template<>
struct TStructOpsTypeTraits<FActorComponentTickFunction> : public TStructOpsTypeTraitsBase2<FActorComponentTickFunction>
{
	enum
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WithAddStructReferencedObjects = true,
#endif
		WithCopy = false
	};
};


UENUM()
namespace ENetworkLagState
{
	enum Type : int
	{
		/** The net driver is operating normally or it is not possible to tell if it is lagging */
		NotLagging,
		/** The net driver is in the process of timing out all of the client connections */
		Lagging
	};
}


namespace ENetworkLagState
{
	inline const TCHAR* ToString(ENetworkLagState::Type LagType)
	{
		switch (LagType)
		{
			case NotLagging:
				return TEXT("NotLagging");
			case Lagging:
				return TEXT("Lagging");
		}
		return TEXT("Unknown lag type occurred.");
	}
}

/** Types of server travel failures broadcast by the engine */
UENUM(BlueprintType)
namespace ETravelFailure
{
	enum Type : int
	{
		/** No level found in the loaded package */
		NoLevel,
		/** LoadMap failed on travel (about to Browse to default map) */
		LoadMapFailure,
		/** Invalid URL specified */
		InvalidURL,
		/** A package is missing on the client */
		PackageMissing,
		/** A package version mismatch has occurred between client and server */
		PackageVersion,
		/** A package is missing and the client is unable to download the file */
		NoDownload,
		/** General travel failure */
		TravelFailure,
		/** Cheat commands have been used disabling travel */
		CheatCommands,
		/** Failed to create the pending net game for travel */
		PendingNetGameCreateFailure,
		/** Failed to save before travel */
		CloudSaveFailure,
		/** There was an error during a server travel to a new map */
		ServerTravelFailure,
		/** There was an error during a client travel to a new map */
		ClientTravelFailure,
	};
}

namespace ETravelFailure
{
	inline const TCHAR* ToString(ETravelFailure::Type FailureType)
	{
		switch (FailureType)
		{
		case NoLevel:
			return TEXT("NoLevel");
		case LoadMapFailure:
			return TEXT("LoadMapFailure");
		case InvalidURL:
			return TEXT("InvalidURL");
		case PackageMissing:
			return TEXT("PackageMissing");
		case PackageVersion:
			return TEXT("PackageVersion");
		case NoDownload:
			return TEXT("NoDownload");
		case TravelFailure:
			return TEXT("TravelFailure");
		case CheatCommands:
			return TEXT("CheatCommands");
		case PendingNetGameCreateFailure:
			return TEXT("PendingNetGameCreateFailure");
		case ServerTravelFailure:
			return TEXT("ServerTravelFailure");
		case ClientTravelFailure:
			return TEXT("ClientTravelFailure");
		case CloudSaveFailure:
			return TEXT("CloudSaveFailure");
		}
		return TEXT("Unknown ETravelFailure error occurred.");
	}
}

// Traveling from server to server.
UENUM()
enum ETravelType : int
{
	/** Absolute URL. */
	TRAVEL_Absolute,
	/** Partial (carry name, reset server). */
	TRAVEL_Partial,
	/** Relative URL. */
	TRAVEL_Relative,
	TRAVEL_MAX,
};

/** Types of demo play failures broadcast from the engine */
UENUM(BlueprintType)
namespace EDemoPlayFailure
{
	enum UE_DEPRECATED(5.1, "No longer used in favor of EReplayResult") Type : int
	{
		/** A Generic failure. */
		Generic,
		/** Demo was not found. */
		DemoNotFound,
		/** Demo is corrupt. */
		Corrupt,
		/** Invalid version. */
		InvalidVersion,
		/** InitBase failed. */
		InitBase,
		/** Failed to process game specific header. */
		GameSpecificHeader,
		/** Replay streamer had an internal error. */
		ReplayStreamerInternal,
		/** LoadMap failed. */
		LoadMap,
		/** Error serializing data stream. */
		Serialization
	};
}

namespace EDemoPlayFailure
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "EDemoPlayFailure is now deprecated")
	inline const TCHAR* ToString(EDemoPlayFailure::Type FailureType)
	{
		switch (FailureType)
		{
		case Generic:
			return TEXT("Generic");
		case DemoNotFound:
			return TEXT("DemoNotFound");
		case Corrupt:
			return TEXT("Corrupt");
		case InvalidVersion:
			return TEXT("InvalidVersion");
		case InitBase:
			return TEXT("InitBase");
		case GameSpecificHeader:
			return TEXT("GameSpecificHeader");
		case ReplayStreamerInternal:
			return TEXT("ReplayStreamerInternal");
		case LoadMap:
			return TEXT("LoadMap");
		case Serialization:
			return TEXT("Serialization");
		}

		return TEXT("Unknown EDemoPlayFailure error occurred.");
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

//URL structure.
USTRUCT()
struct FURL
{
	GENERATED_USTRUCT_BODY()

	// Protocol, i.e. "unreal" or "http".
	UPROPERTY()
	FString Protocol;

	// Optional hostname, i.e. "204.157.115.40" or "unreal.epicgames.com", blank if local.
	UPROPERTY()
	FString Host;

	// Optional host port.
	UPROPERTY()
	int32 Port;

	UPROPERTY()
	int32 Valid;

	// Map name, i.e. "SkyCity", default is "Entry".
	UPROPERTY()
	FString Map;

	// Optional place to download Map if client does not possess it
	UPROPERTY()
	FString RedirectURL;

	// Options.
	UPROPERTY()
	TArray<FString> Op;

	// Portal to enter through, default is "".
	UPROPERTY()
	FString Portal;

	// Statics.
	static ENGINE_API FUrlConfig UrlConfig;
	static ENGINE_API bool bDefaultsInitialized;

	/**
	 * Prevent default from being generated.
	 */
	explicit FURL( ENoInit ) { }

	/**
	 * Construct a purely default, local URL from an optional filename.
	 */
	ENGINE_API FURL( const TCHAR* Filename=nullptr );

	/**
	 * Construct a URL from text and an optional relative base.
	 */
	ENGINE_API FURL( FURL* Base, const TCHAR* TextURL, ETravelType Type );

	static ENGINE_API void StaticInit();
	static ENGINE_API void StaticExit();

	/**
	 * Static: Removes any special URL characters from the specified string
	 *
	 * @param Str String to be filtered
	 */
	static ENGINE_API void FilterURLString( FString& Str );

	/**
	 * Returns whether this URL corresponds to an internal object, i.e. an Unreal
	 * level which this app can try to connect to locally or on the net. If this
	 * is false, the URL refers to an object that a remote application like Internet
	 * Explorer can execute.
	 */
	ENGINE_API bool IsInternal() const;

	/**
	 * Returns whether this URL corresponds to an internal object on this local 
	 * process. In this case, no Internet use is necessary.
	 */
	ENGINE_API bool IsLocalInternal() const;

	/**
	 * Tests if the URL contains an option string.
	 */
	ENGINE_API bool HasOption( const TCHAR* Test ) const;

	/**
	 * Returns the value associated with an option.
	 *
	 * @param Match The name of the option to get.
	 * @param Default The value to return if the option wasn't found.
	 *
	 * @return The value of the named option, or Default if the option wasn't found.
	 */
	ENGINE_API const TCHAR* GetOption( const TCHAR* Match, const TCHAR* Default ) const;

	/**
	 * Load URL from config.
	 */
	ENGINE_API void LoadURLConfig( const TCHAR* Section, const FString& Filename=GGameIni );

	/**
	 * Save URL to config.
	 */
	ENGINE_API void SaveURLConfig( const TCHAR* Section, const TCHAR* Item, const FString& Filename=GGameIni ) const;

	/**
	 * Add a unique option to the URL, replacing any existing one.
	 */
	ENGINE_API void AddOption( const TCHAR* Str );

	/**
	 * Remove an option from the URL
	 */
	ENGINE_API void RemoveOption( const TCHAR* Key, const TCHAR* Section = nullptr, const FString& Filename = GGameIni);

	/**
	 * Convert this URL to text.
	 */
	ENGINE_API FString ToString( bool FullyQualified=0 ) const;

	/**
	 * Prepares the Host and Port values into a standards compliant string
	 */
	ENGINE_API FString GetHostPortString() const;

	/**
	 * Serializes a FURL to or from an archive.
	 */
	ENGINE_API friend FArchive& operator<<( FArchive& Ar, FURL& U );

	/**
	 * Compare two URLs to see if they refer to the same exact thing.
	 */
	ENGINE_API bool operator==( const FURL& Other ) const;
};

/**
 * The network mode the game is currently running.
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Networking/Overview/
 */
enum ENetMode : int
{
	/** Standalone: a game without networking, with one or more local players. Still considered a server because it has all server functionality. */
	NM_Standalone,

	/** Dedicated server: server with no local players. */
	NM_DedicatedServer,

	/** Listen server: a server that also has a local player who is hosting the game, available to other players on the network. */
	NM_ListenServer,

	/**
	 * Network client: client connected to a remote server.
	 * Note that every mode less than this value is a kind of server, so checking NetMode < NM_Client is always some variety of server.
	 */
	NM_Client,

	NM_MAX,
};

/**
 * Define view modes to get specific show flag settings (some on, some off and some are not altered)
 * Don't change the order, the ID is serialized with the editor
 */
UENUM()
enum EViewModeIndex : int
{
	/** Wireframe w/ brushes. */
	VMI_BrushWireframe = 0 UMETA(DisplayName = "Wireframe"),
	/** Wireframe w/ BSP. */
	VMI_Wireframe = 1 UMETA(DisplayName = "CSG Wireframe"),
	/** Unlit. */
	VMI_Unlit = 2 UMETA(DisplayName = "Unlit"),
	/** Lit. */
	VMI_Lit = 3 UMETA(DisplayName = "Lit"),
	VMI_Lit_DetailLighting = 4 UMETA(DisplayName = "Detail Lighting"),
	/** Lit wo/ materials. */
	VMI_LightingOnly = 5 UMETA(DisplayName = "Lighting Only"),
	/** Colored according to light count. */
	VMI_LightComplexity = 6 UMETA(DisplayName = "Light Complexity"),
	/** Colored according to shader complexity. */
	VMI_ShaderComplexity = 8 UMETA(DisplayName = "Shader Complexity"),
	/** Colored according to world-space LightMap texture density. */
	VMI_LightmapDensity = 9 UMETA(DisplayName = "Lightmap Density"),
	/** Colored according to light count - showing lightmap texel density on texture mapped objects. */
	VMI_LitLightmapDensity = 10 UMETA(DisplayName = "Lit Lightmap Density"),
	VMI_ReflectionOverride = 11 UMETA(DisplayName = "Reflections"),
	VMI_VisualizeBuffer = 12 UMETA(DisplayName = "Buffer Visualization"),
	//	VMI_VoxelLighting = 13,

	/** Colored according to stationary light overlap. */
	VMI_StationaryLightOverlap = 14 UMETA(DisplayName = "Stationary Light Overlap"),

	VMI_CollisionPawn = 15 UMETA(DisplayName = "Player Collision"),
	VMI_CollisionVisibility = 16 UMETA(DisplayName = "Visibility Collision"),
	//VMI_UNUSED = 17,
	/** Colored according to the current LOD index. */
	VMI_LODColoration = 18 UMETA(DisplayName = "Mesh LOD Coloration"),
	/** Colored according to the quad coverage. */
	VMI_QuadOverdraw = 19 UMETA(DisplayName = "Quad Overdraw"),
	/** Visualize the accuracy of the primitive distance computed for texture streaming. */
	VMI_PrimitiveDistanceAccuracy = 20 UMETA(DisplayName = "Primitive Distance"),
	/** Visualize the accuracy of the mesh UV densities computed for texture streaming. */
	VMI_MeshUVDensityAccuracy = 21  UMETA(DisplayName = "Mesh UV Density"),
	/** Colored according to shader complexity, including quad overdraw. */
	VMI_ShaderComplexityWithQuadOverdraw = 22 UMETA(DisplayName = "Shader Complexity & Quads"),
	/** Colored according to the current HLOD index. */
	VMI_HLODColoration = 23  UMETA(DisplayName = "Hierarchical LOD Coloration"),
	/** Group item for LOD and HLOD coloration*/
	VMI_GroupLODColoration = 24  UMETA(DisplayName = "Group LOD Coloration"),
	/** Visualize the accuracy of the material texture scales used for texture streaming. */
	VMI_MaterialTextureScaleAccuracy = 25 UMETA(DisplayName = "Material Texture Scales"),
	/** Compare the required texture resolution to the actual resolution. */
	VMI_RequiredTextureResolution = 26 UMETA(DisplayName = "Required Texture Resolution"),

	// Ray tracing modes

	/** Run path tracing pipeline */
	VMI_PathTracing = 27 UMETA(DisplayName = "Path Tracing"),
	/** Run ray tracing debug pipeline */
	VMI_RayTracingDebug = 28 UMETA(DisplayName = "Ray Tracing Debug"),

	/** Visualize various aspects of Nanite */
	VMI_VisualizeNanite = 29 UMETA(DisplayName = "Nanite Visualization"),

	/** Compare the required texture resolution to the actual resolution. */
	VMI_VisualizeVirtualTexture = 30 UMETA(DisplayName = "Virtual Texture Visualization"),

	/** Visualize Lumen debug views */
	VMI_VisualizeLumen = 31 UMETA(DisplayName = "Lumen Visualization"),

	/** Visualize virtual shadow map */
	VMI_VisualizeVirtualShadowMap = 32 UMETA(DisplayName = "Virtual Shadow Map Visualization"),

	/** Visualize Skin Cache. */
	VMI_VisualizeGPUSkinCache = 33 UMETA(DisplayName = "GPU Skin Cache Visualization"),

	/** Visualize Substrate debug views */
	VMI_VisualizeSubstrate = 34 UMETA(DisplayName = "Substrate Visualization"),

	/** Visualize Groom debug views */
	VMI_VisualizeGroom = 35 UMETA(DisplayName = "Groom Visualization"),

	VMI_LWCComplexity = 36 UMETA(DisplayName = "Material LWC Function Usage"),
	
	/** Lit Wireframe. */
	VMI_Lit_Wireframe UE_DEPRECATED(5.8, "VMI_Lit_Wireframe is deprecated. Use VMI_Lit and FEngineShowFlags::SetMeshEdges") = 37 UMETA(Hidden, DisplayName = "Lit Wireframe"),

	/** Visualize Actor Coloration. */
	VMI_VisualizeActorColoration = 38 UMETA(DisplayName = "Actor Coloration Visualization"),

	/** Visualize Shadow Casters */
	VMI_ShadowCasters = 39 UMETA(DisplayName = "Shadow Casters Visualization"),

	/** Displays using clay material to evaluate surface curvature and smoothness. */
	VMI_Clay = 40 UMETA(DisplayName = "Clay"),

	/** Displays simulated light reflections using stripes to evaluate surface curvature and smoothness. */
	VMI_Zebra = 41 UMETA(DisplayName = "Zebra"),

	/** Highlights front-facing and back-facing polygons differently to evaluate face normals. */
	VMI_FrontBackFace = 42 UMETA(DisplayName = "Front/Back Face"),

	/** Assigns random colors to individual elements or components to improve visual distinction between overlapping or adjacent parts. */
	VMI_RandomColor = 43 UMETA(DisplayName = "Random"),

	/** Visualize MegaLights debug views */
	VMI_VisualizeMegaLights = 44 UMETA(DisplayName = "MegaLights Visualization"),

	/** Streaming deficit viewmodes: color primitives by how far behind the streaming target they are */
	VMI_StreamingTextureDeficit = 45 UMETA(DisplayName = "Streaming Texture Deficit"),
	VMI_StreamingTextureResidency = 46 UMETA(DisplayName = "Streaming Texture Residency"),
	VMI_StreamingMeshLODDeficit = 47 UMETA(DisplayName = "Streaming Mesh LOD Deficit"),
	VMI_StreamingMeshLODResidency = 48 UMETA(DisplayName = "Streaming Mesh LOD Residency"),

	VMI_Max UMETA(Hidden),

	// VMI_Unknown - The value assigned to VMI_Unknown must be the highest possible of any member of EViewModeIndex, or GetViewModeName might seg-fault
	VMI_Unknown = 255 UMETA(DisplayName = "Unknown"),
};

/**
 * Class containing a static util function to help with EViewModeIndex
 */
UCLASS(config = Engine, MinimalAPI)
class UViewModeUtils : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Get the display name associated with a particular EViewModeIndex
	 */
	static ENGINE_API FText GetViewModeDisplayName(const EViewModeIndex ViewModeIndex);
	static ENGINE_API const FSlateBrush* GetViewModeDisplayIcon(const EViewModeIndex ViewModeIndex);
	static ENGINE_API FSlateIcon GetViewModeDisplaySlateIcon(const EViewModeIndex InViewModeIndex);
};


/** Settings to allow designers to override the automatic expose */
USTRUCT()
struct FExposureSettings
{
	GENERATED_USTRUCT_BODY()

	FExposureSettings() : FixedEV100(1), bFixed(false)
	{
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%f,%d"), FixedEV100, bFixed ? 1 : 0);
	}

	void SetFromString(const TCHAR *In)
	{
		// set to default
		*this = FExposureSettings();

		const TCHAR* Comma = FCString::Strchr(In, TEXT(','));
		check(Comma);

		const int32 BUFFER_SIZE = 128;
		TCHAR Buffer[BUFFER_SIZE];
		check((Comma-In)+1 < BUFFER_SIZE);
		
		FCString::Strncpy(Buffer, In, UE_PTRDIFF_TO_INT32((Comma-In)+1));
		FixedEV100 = FCString::Atof(Buffer);
		bFixed = !!FCString::Atoi(Comma+1);
	}

	// EV100 settings for fixed mode
	UPROPERTY()
	float FixedEV100;
	// true: fixed exposure using the LogOffset value, false: automatic eye adaptation
	UPROPERTY()
	bool bFixed;
};


UCLASS(abstract, config=Engine)
class UEngineBaseTypes : public UObject
{
	GENERATED_UCLASS_BODY()

};
