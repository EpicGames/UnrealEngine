// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "EngineCustomTimeStep.generated.h"

/**
 * Possible states of CustomTimeStep.
 */
UENUM()
enum class ECustomTimeStepSynchronizationState
{
	/** CustomTimeStep has not been initialized or has been shutdown. */
	Closed,

	/** CustomTimeStep error occurred during Synchronization. */
	Error,

	/** CustomTimeStep is currently synchronized with the source. */
	Synchronized,

	/** CustomTimeStep is initialized and being prepared for synchronization. */
	Synchronizing,
};

/**
 * A CustomTimeStep control the Engine Framerate/Timestep.
 * This will update the FApp::CurrentTime/FApp::DeltaTime.
 * This is useful when you want the engine to be synchronized with an external clock (genlock).
 */
UCLASS(abstract, MinimalAPI)
class UEngineCustomTimeStep : public UObject
{
	GENERATED_BODY()

public:
	/** This CustomTimeStep became the Engine's CustomTimeStep. */
	virtual bool Initialize(class UEngine* InEngine) PURE_VIRTUAL(UEngineCustomTimeStep::Initialize, return false;);

	/** This CustomTimeStep stop being the Engine's CustomTimeStep. */
	virtual void Shutdown(class UEngine* InEngine) PURE_VIRTUAL(UEngineCustomTimeStep::Shutdown, );

	/**
	 * Update FApp::CurrentTime/FApp::DeltaTime and optionally wait until the end of the frame.
	 * @return	true if the Engine's TimeStep should also be performed; false otherwise.
	 */
	virtual bool UpdateTimeStep(class UEngine* InEngine) PURE_VIRTUAL(UEngineCustomTimeStep::UpdateTimeStep, return true;);

	/** The state of the CustomTimeStep. */
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const PURE_VIRTUAL(UEngineCustomTimeStep::GetSynchronizationState, return ECustomTimeStepSynchronizationState::Closed;);

	/** Get the display name of the custom time step. Allows child classes to provide more context to users. */
	virtual FString GetDisplayName() const { return GetName(); };

public:
	/** Default behaviour of the engine. Update FApp::LastTime */
	static ENGINE_API void UpdateApplicationLastTime();
};

