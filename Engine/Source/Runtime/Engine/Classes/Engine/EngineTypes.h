// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *	This will hold all of our enums and types and such that we need to
 *	use in multiple files where the enum can't be mapped to a specific file.
 */

#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "Engine/TimerHandle.h"
#include "Engine/NaniteAssemblyData.h"
#include "UObject/StrProperty.h"
#include "EngineTypes.generated.h"

class AActor;
class UDecalComponent;
class UPhysicalMaterial;
class UPhysicalMaterialMask;
class UPrimitiveComponent;
class USceneComponent;
class USubsurfaceProfile;
class USpecularProfile;
class UToonProfile;

/**
 * Default number of components to expect in TInlineAllocators used with AActor component arrays.
 * Used by engine code to try to avoid allocations in AActor::GetComponents(), among others.
 */
enum { NumInlinedActorComponents = 24 };

/** Enum describing how to constrain perspective view port FOV */
UENUM()
enum EAspectRatioAxisConstraint : int
{
	AspectRatio_MaintainYFOV UMETA(DisplayName="Maintain Y-Axis FOV"),
	AspectRatio_MaintainXFOV UMETA(DisplayName="Maintain X-Axis FOV"),
	AspectRatio_MajorAxisFOV UMETA(DisplayName="Maintain Major Axis FOV"),
	AspectRatio_MAX,
};

/** Return values for UEngine::Browse. */
namespace EBrowseReturnVal
{
	enum Type
	{
		/** Successfully browsed to a new map. */
		Success,
		/** Immediately failed to browse. */
		Failure,
		/** A connection is pending. */
		Pending,
	};
}

/** Rules for attaching components - needs to be kept synced to EDetachmentRule */
UENUM()
enum class EAttachmentRule : uint8
{
	/** Keeps current relative transform as the relative transform to the new parent. */
	KeepRelative,

	/** Automatically calculates the relative transform such that the attached component maintains the same world transform. */
	KeepWorld,

	/** Snaps transform to the attach point */
	SnapToTarget,
};

/** Rules for attaching components */
struct FAttachmentTransformRules
{
	/** Various preset attachment rules. Note that these default rules do NOT by default weld simulated bodies */
	static ENGINE_API FAttachmentTransformRules KeepRelativeTransform;
	static ENGINE_API FAttachmentTransformRules KeepWorldTransform;
	static ENGINE_API FAttachmentTransformRules SnapToTargetNotIncludingScale;
	static ENGINE_API FAttachmentTransformRules SnapToTargetIncludingScale;

	FAttachmentTransformRules(EAttachmentRule InRule, bool bInWeldSimulatedBodies)
		: LocationRule(InRule)
		, RotationRule(InRule)
		, ScaleRule(InRule)
		, bWeldSimulatedBodies(bInWeldSimulatedBodies)
	{}

	FAttachmentTransformRules(EAttachmentRule InLocationRule, EAttachmentRule InRotationRule, EAttachmentRule InScaleRule, bool bInWeldSimulatedBodies)
		: LocationRule(InLocationRule)
		, RotationRule(InRotationRule)
		, ScaleRule(InScaleRule)
		, bWeldSimulatedBodies(bInWeldSimulatedBodies)
	{}

	/** The rule to apply to location when attaching */
	EAttachmentRule LocationRule;

	/** The rule to apply to rotation when attaching */
	EAttachmentRule RotationRule;

	/** The rule to apply to scale when attaching */
	EAttachmentRule ScaleRule;

	/** Whether to weld simulated bodies together when attaching */
	bool bWeldSimulatedBodies;
};

/** Rules for detaching components - needs to be kept synced to EAttachmentRule */
UENUM()
enum class EDetachmentRule : uint8
{
	/** Keeps current relative transform. */
	KeepRelative,

	/** Automatically calculates the relative transform such that the detached component maintains the same world transform. */
	KeepWorld,
};

/** Rules for detaching components */
struct FDetachmentTransformRules
{
	/** Various preset detachment rules */
	static ENGINE_API FDetachmentTransformRules KeepRelativeTransform;
	static ENGINE_API FDetachmentTransformRules KeepWorldTransform;

	FDetachmentTransformRules(EDetachmentRule InRule, bool bInCallModify)
		: LocationRule(InRule)
		, RotationRule(InRule)
		, ScaleRule(InRule)
		, bCallModify(bInCallModify)
	{}

	FDetachmentTransformRules(EDetachmentRule InLocationRule, EDetachmentRule InRotationRule, EDetachmentRule InScaleRule, bool bInCallModify)
		: LocationRule(InLocationRule)
		, RotationRule(InRotationRule)
		, ScaleRule(InScaleRule)
		, bCallModify(bInCallModify)
	{}

	FDetachmentTransformRules(const FAttachmentTransformRules& AttachmentRules, bool bInCallModify)
		: LocationRule(AttachmentRules.LocationRule == EAttachmentRule::KeepRelative ? EDetachmentRule::KeepRelative : EDetachmentRule::KeepWorld)
		, RotationRule(AttachmentRules.RotationRule == EAttachmentRule::KeepRelative ? EDetachmentRule::KeepRelative : EDetachmentRule::KeepWorld)
		, ScaleRule(AttachmentRules.ScaleRule == EAttachmentRule::KeepRelative ? EDetachmentRule::KeepRelative : EDetachmentRule::KeepWorld)
		, bCallModify(bInCallModify)
	{}

	/** The rule to apply to location when detaching */
	EDetachmentRule LocationRule;

	/** The rule to apply to rotation when detaching */
	EDetachmentRule RotationRule;

	/** The rule to apply to scale when detaching */
	EDetachmentRule ScaleRule;

	/** Whether to call Modify() on the components concerned when detaching */
	bool bCallModify;
};

/** Deprecated rules for setting transform on attachment, new functions should use FAttachmentTransformRules isntead */
UENUM()
namespace EAttachLocation
{
	enum Type : int
	{
		/** Keeps current relative transform as the relative transform to the new parent. */
		KeepRelativeOffset,
		
		/** Automatically calculates the relative transform such that the attached component maintains the same world transform. */
		KeepWorldPosition,

		/** Snaps location and rotation to the attach point. Calculates the relative scale so that the final world scale of the component remains the same. */
		SnapToTarget					UMETA(DisplayName = "Snap to Target, Keep World Scale"),
		
		/** Snaps entire transform to target, including scale. */
		SnapToTargetIncludingScale		UMETA(DisplayName = "Snap to Target, Including Scale"),
	};
}

/**
 * A priority for sorting scene elements by depth.
 * Elements with higher priority occlude elements with lower priority, disregarding distance.
 */
UENUM()
enum ESceneDepthPriorityGroup : int
{
	/** World scene DPG. */
	SDPG_World,
	/** Foreground scene DPG. */
	SDPG_Foreground,
	SDPG_MAX,
};

/** Quality of indirect lighting for Movable primitives. This has a large effect on Indirect Lighting Cache update time. */
UENUM()
enum EIndirectLightingCacheQuality : int
{
	/** The indirect lighting cache will be disabled for this object, so no GI from stationary lights on movable objects. */
	ILCQ_Off,
	/** A single indirect lighting sample computed at the bounds origin will be interpolated which fades over time to newer results. */
	ILCQ_Point,
	/** The object will get a 5x5x5 stable volume of interpolated indirect lighting, which allows gradients of lighting intensity across the receiving object. */
	ILCQ_Volume
};

/** Type of lightmap that is used for primitive components */
UENUM()
enum class ELightmapType : uint8
{
	/** Use the default based on Mobility: Surface Lightmap for Static components, Volumetric Lightmap for Movable components. */
	Default,
	/** Force Surface Lightmap, even if the component moves, which should otherwise change the lighting.  This is only supported on components which support surface lightmaps, like static meshes. */
	ForceSurface,
	/** 
	 * Force Volumetric Lightmaps, even if the component is static and could have supported surface lightmaps. 
	 * Volumetric Lightmaps have better directionality and no Lightmap UV seams, but are much lower resolution than Surface Lightmaps and frequently have self-occlusion and leaking problems.
	 * Note: Lightmass currently requires valid lightmap UVs and sufficient lightmap resolution to compute bounce lighting, even though the Volumetric Lightmap will be used at runtime.
	 */
	ForceVolumetric
};

/** Controls how occlusion from Distance Field Ambient Occlusion is combined with Screen Space Ambient Occlusion. */
UENUM()
enum EOcclusionCombineMode : int
{
	/** Take the minimum occlusion value.  This is effective for avoiding over-occlusion from multiple methods, but can result in indoors looking too flat. */
	OCM_Minimum,
	/** 
	 * Multiply together occlusion values from Distance Field Ambient Occlusion and Screen Space Ambient Occlusion.  
	 * This gives a good sense of depth everywhere, but can cause over-occlusion. 
	 * SSAO should be tweaked to be less strong compared to Minimum.
	 */
	OCM_Multiply,
	OCM_MAX,
};

/**
 * The blending mode for materials
 * @warning This is mirrored in Lightmass, be sure to update the blend mode structure and logic there if this changes. 
 * @warning Check UMaterialInstance::Serialize if changed!!
 */
UENUM(BlueprintType)
enum EBlendMode : int
{
	BLEND_Opaque UMETA(DisplayName="Opaque"),
	BLEND_Masked UMETA(DisplayName="Masked"),
	BLEND_Translucent UMETA(DisplayName="Translucent"),
	BLEND_Additive UMETA(DisplayName="Additive"),
	BLEND_Modulate UMETA(DisplayName="Modulate"),
	BLEND_AlphaComposite UMETA(DisplayName = "AlphaComposite (Premultiplied Alpha)"),
	BLEND_AlphaHoldout UMETA(DisplayName = "AlphaHoldout"),
	BLEND_TranslucentColoredTransmittance UMETA(DisplayName = "SUBSTRATE_ONLY - Translucent - Colored Transmittance"), /*Substrate only */
	BLEND_MAX UMETA(Hidden),
	// Renamed blend modes. These blend modes are remapped onto legacy ones and kept hidden for not confusing users in legacy mode, while allowing to use the new blend mode names into code.
	BLEND_TranslucentGreyTransmittance = BLEND_Translucent UMETA(Hidden, DisplayName = "Translucent - Grey Transmittance"), /*Substrate only */
	BLEND_ColoredTransmittanceOnly = BLEND_Modulate UMETA(Hidden, DisplayName = "Colored Transmittance Only"), /*Substrate only */
};

class FMaterial;
class UMaterialInterface;

/** The default float precision for material's pixel shaders on mobile devices*/
UENUM()
enum EMaterialFloatPrecisionMode : int
{
	/** Uses project based precision mode setting */
	MFPM_Default UMETA(DisplayName = "Default"),
	/** Force full-precision for MaterialFloat only, no effect on shader codes in .ush/.usf*/
	MFPM_Full_MaterialExpressionOnly UMETA(DisplayName = "Use Full-precision for MaterialExpressions only"),
	/** All the floats are full-precision */
	MFPM_Full UMETA(DisplayName = "Use Full-precision for every float"),
	/** Half precision, except explicit 'float' in .ush/.usf*/
	MFPM_Half UMETA(DisplayName = "Use Half-precision"),
	/* Same as 'Half precision' but includes custom shader parameters uniforms */
	MFPM_Half_WithShaderParameters UMETA(DisplayName = "Use Half-precision including shader parameters"),
	MFPM_MAX,
};

/** Controls where the sampler for different texture lookups comes from */
UENUM()
enum ESamplerSourceMode : int
{
	/** Get the sampler from the texture.  Every unique texture will consume a sampler slot, which are limited in number. */
	SSM_FromTextureAsset UMETA(DisplayName="From texture asset"),
	/** Shared sampler source that does not consume a sampler slot.  Uses wrap addressing and gets filter mode from the world texture group. */
	SSM_Wrap_WorldGroupSettings UMETA(DisplayName="Shared: Wrap"),
	/** Shared sampler source that does not consume a sampler slot.  Uses clamp addressing and gets filter mode from the world texture group. */
	SSM_Clamp_WorldGroupSettings UMETA(DisplayName="Shared: Clamp"),
	/** Shared sampler source that does not consume a sampler slot, used to sample the terrain weightmap.  Gets filter mode from the terrain weightmap texture group. */
	SSM_TerrainWeightmapGroupSettings UMETA(Hidden)
};

/** defines how MipValue is used */
UENUM()
enum ETextureMipValueMode : int
{
	/* Use hardware computed sample's mip level with automatic anisotropic filtering support. */
	TMVM_None UMETA(DisplayName="None (use computed mip level)"),

	/* Explicitly compute the sample's mip level. Disables anisotropic filtering. */
	TMVM_MipLevel UMETA(DisplayName="MipLevel (absolute, 0 is full resolution)"),
	
	/* Bias the hardware computed sample's mip level. Disables anisotropic filtering. */
	TMVM_MipBias UMETA(DisplayName="MipBias (relative to the computed mip level)"),
	
	/* Explicitly compute the sample's DDX and DDY for anisotropic filtering. */
	TMVM_Derivative UMETA(DisplayName="Derivative (explicit derivative to compute mip level)"),

	TMVM_MAX,
};

/** Describes how to handle lighting of translucent objets */
UENUM()
enum ETranslucencyLightingMode : int
{
	/** 
	 * Lighting will be calculated for a volume, without directionality.  Use this on particle effects like smoke and dust.
	 * This is the cheapest per-pixel lighting method, however the material normal is not taken into account.
	 */
	TLM_VolumetricNonDirectional UMETA(DisplayName="Volumetric NonDirectional"),

	 /** 
	 * Lighting will be calculated for a volume, with directionality so that the normal of the material is taken into account. 
	 * Note that the default particle tangent space is facing the camera, so enable bGenerateSphericalParticleNormals to get a more useful tangent space.
	 */
	TLM_VolumetricDirectional UMETA(DisplayName="Volumetric Directional"),

	/** 
	 * Same as Volumetric Non Directional, but lighting is only evaluated at vertices so the pixel shader cost is significantly less.
	 * Note that lighting still comes from a volume texture, so it is limited in range.  Directional lights become unshadowed in the distance.
	 */
	TLM_VolumetricPerVertexNonDirectional UMETA(DisplayName="Volumetric PerVertex NonDirectional"),

	 /** 
	 * Same as Volumetric Directional, but lighting is only evaluated at vertices so the pixel shader cost is significantly less.
	 * Note that lighting still comes from a volume texture, so it is limited in range.  Directional lights become unshadowed in the distance.
	 */
	TLM_VolumetricPerVertexDirectional UMETA(DisplayName="Volumetric PerVertex Directional"),

	/** 
	 * Lighting will be calculated for a surface. The light is accumulated in a volume so the result is blurry, 
	 * limited distance but the per pixel cost is very low. Use this on translucent surfaces like glass and water.
	 * Only diffuse lighting is supported.
	 */
	TLM_Surface UMETA(DisplayName="Surface TranslucencyVolume"),

	/** 
	 * Lighting will be calculated for a surface. Use this on translucent surfaces like glass and water.
	 * This is implemented with forward shading so specular highlights from local lights are supported, however many deferred-only features are not.
	 * This is the most expensive translucency lighting method as each light's contribution is computed per-pixel.
	 */
	TLM_SurfacePerPixelLighting UMETA(DisplayName="Surface ForwardShading"),

	TLM_MAX,
};

/** Determines how the refraction offset should be computed for the material. */
UENUM()
enum ERefractionMode : int
{
	/** 
	 * Refraction is computed based on the camera vector entering a medium whose index of refraction is defined by the Refraction material input.  
	 * The new medium's surface is defined by the material's normal.  With this mode, a flat plane seen from the side will have a constant refraction offset.
	 * This is a physical model of refraction but causes reading outside the scene color texture so is a poor fit for large refractive surfaces like water.
	 */
	RM_IndexOfRefraction UMETA(DisplayName="Index Of Refraction"),

	/**
	 * By default, when the root node refraction pin is unplugged, no refraction will appear.
	 * The refraction offset into Scene Color is computed based on the difference between the per-pixel normal and the per-vertex normal.  
	 * With this mode, a material whose normal is the default (0, 0, 1) will never cause any refraction.  This mode is only valid with tangent space normals.
	 * The refraction material input scales the offset, although a value of 1.0 maps to no refraction, and a value of 2 maps to a scale of 1.0 on the offset.
	 * This is a non-physical model of refraction but is useful on large refractive surfaces like water, since offsets have to stay small to avoid reading outside scene color.
	 */
	RM_PixelNormalOffset UMETA(DisplayName="Pixel Normal Offset"),

	/**
	 * By default, when the root node refraction pin is unplugged, no refraction will appear.
	 * Explicit 2D screen offset. This offset is independent of screen resolution and aspect ratio. The user is in charge of any strength and fading.
	 */
	RM_2DOffset UMETA(DisplayName = "2D Offset"),

	/**
	 * Refraction is disabled.
	 */
	RM_None UMETA(DisplayName = "None"),

	/**
	 * Refraction is computed based on the camera vector entering a medium whose index of refraction is defined by the material IOR evaluated from F0.
	 * The new medium's surface is defined by the material's normal.  With this mode, a flat plane seen from the side will have a constant refraction offset.
	 * This is a physical model of refraction but causes reading outside the scene color texture so is a poor fit for large refractive surfaces like water.
	 */
	RM_IndexOfRefractionFromF0 UMETA(DisplayName = "Index Of Refraction From F0"),
};

/** Determines how the refraction account for the coverage with Substrate. It can only be used when Substrate is enabled. */
UENUM()
enum ERefractionCoverageMode : int
{
	/**
	 * This is the pre-Substrate behavior: coverage is ignored and always 1.
	 * When rough refraction is disabled, this is behavior is forced ON.
	 */
	RCM_CoverageIgnored UMETA(DisplayName = "Coverage Ignored"),

	/**
	 * This is a new behavior available with Substrate when rough refraction are enabled: account for roughness, coverage and depth.
	 * This is a more physically based behavior: the background scene will be visible untouched according to (1-coverage),
	 * while the blurred version will be visible according to coverage.
	 */
	RCM_CoverageAccountedFor UMETA(DisplayName = "Coverage Accounted For"),
};

/** Determines how the pixel depth offset is evaluated and applied. Must match PODM_LEGACY in MaterialTemplace.ush.*/
UENUM()
enum EPixelDepthOffsetMode : int
{
	/**
	 * This is the legacy mode where PDO is applied differently for Depth (along View Forward) and world position (along Camera Vector).
	 */
	PDOM_Legacy UMETA(DisplayName = "Legacy"),

	/**
	 * PDO is applied along the Camera Vector for Depth and World Position altogether.
	 */
	PDOM_AlongCameraVector UMETA(DisplayName = "Along Camera Vector"),
};

/**
 * Enumerates available options for the translucency sort policy.
 */
UENUM()
namespace ETranslucentSortPolicy
{
	enum Type : int
	{
		/** Sort based on distance from camera centerpoint to bounding sphere centerpoint. (Default, best for 3D games.) */
		SortByDistance = 0,

		/** Sort based on the post-projection Z distance to the camera. */
		SortByProjectedZ = 1,

		/** Sort based on the projection onto a fixed axis. (Best for 2D games.) */
		SortAlongAxis = 2,
	};
}

// Note: Must match r.DynamicGlobalIlluminationMethod, this is used in URendererSettings
UENUM()
namespace EDynamicGlobalIlluminationMethod
{
	enum Type : int
	{
		/** No dynamic Global Illumination method will be used. Global Illumination can still be baked into lightmaps. */
		None, 

		/** Use Lumen Global Illumination for all lights, emissive materials casting light and SkyLight Occlusion.  Requires 'Generate Mesh Distance Fields' enabled for Software Ray Tracing and 'Support Hardware Ray Tracing' enabled for Hardware Ray Tracing. */
		Lumen,

		/** Standalone Screen Space Global Illumination.  Low cost, but limited by screen space information. */
		ScreenSpace UMETA(DisplayName="Screen Space (Deprecated)"),

		/** Use a plugin for Global Illumination */
		Plugin UMETA(DisplayName="Plugin"),
	};
}

// Note: Must match r.ReflectionMethod, this is used in URendererSettings
UENUM()
namespace EReflectionMethod
{
	enum Type : int
	{
		/** No global reflection method will be used. Reflections can still come from Reflection Captures, Planar Reflections or a Skylight placed in the level. */
		None, 

		/** Use Lumen Reflections, which supports Screen / Software / Hardware Ray Tracing together and integrates with Lumen Global Illumination for rough reflections and Global Illumination seen in reflections. */
		Lumen,

		/** Standalone Screen Space Reflections.  Low cost, but limited by screen space information. */
		ScreenSpace UMETA(DisplayName="Screen Space"),
	};
}

// Note: Must match r.Shadow.Virtual.Enable, this is used in URendererSettings
UENUM()
namespace EShadowMapMethod
{
	enum Type : int
	{
		/** Render geometry into shadow depth maps for shadowing.  Requires manual setup of shadowing distances and only culls per-component, causing poor performance with high poly scenes.  Required to enable stationary baked shadows (but which is incompatible with Nanite geometry). */
		ShadowMaps UMETA(DisplayName = "Shadow Maps"),

		/** Render geometry into virtualized shadow depth maps for shadowing.  Provides high-quality shadows for next-gen projects with simplified setup.  High efficiency culling when used with Nanite. */
		VirtualShadowMaps UMETA(DisplayName = "Virtual Shadow Maps")
	};
}

/** Ray Tracing Shadows type. */
UENUM()
namespace ECastRayTracedShadow 
{
	enum Type : int
	{
		/** Ray traced shadows disabled for this light */
		Disabled,
		/** Ray traced shadows follow Cast Ray Traced Shadows project setting */
		UseProjectSetting,
		/** Ray traced shadows enabled for this light */
		Enabled,
	};
}

/** MegaLights Shadow type for a light component.
*/
UENUM()
namespace EMegaLightsShadowMethod
{
	enum Type : int
	{
		/** Recommended. Uses the default MegaLights shadow method. */
		Default,
		/** Preferred method, which guarantees fixed MegaLights cost and correct area shadows, but is dependent on the BVH representation quality. */
		RayTracing,
		/** Has a significant per light cost, but can cast shadows directly from the Nanite geometry using rasterization. */
		VirtualShadowMap,
	};
}

/** Specifies which component of the scene rendering should be output to the final render target. */
UENUM()
enum ESceneCaptureSource : int
{ 
	SCS_SceneColorHDR UMETA(DisplayName="SceneColor (HDR) in RGB, Inv Opacity in A"),
	SCS_SceneColorHDRNoAlpha UMETA(DisplayName="SceneColor (HDR) in RGB, 0 in A"),
	SCS_FinalColorLDR UMETA(DisplayName="Final Color (LDR) in RGB"),
	SCS_SceneColorSceneDepth UMETA(DisplayName="SceneColor (HDR) in RGB, SceneDepth in A"),
	SCS_SceneDepth UMETA(DisplayName="SceneDepth in R"),
	SCS_DeviceDepth UMETA(DisplayName = "DeviceDepth in RGB"),
	SCS_Normal UMETA(DisplayName="Normal in RGB (Deferred Renderer only)"),
	SCS_BaseColor UMETA(DisplayName = "BaseColor in RGB (Deferred Renderer only)"),
	SCS_FinalColorHDR UMETA(DisplayName = "Final Color (HDR) in Linear Working Color Space"),
	SCS_FinalToneCurveHDR UMETA(DisplayName = "Final Color (with tone curve) in Linear sRGB gamut"),

	SCS_MAX
};

/** Specifies how scene captures are composited into render buffers */
UENUM()
enum ESceneCaptureCompositeMode : int
{ 
	SCCM_Overwrite UMETA(DisplayName="Overwrite"),
	SCCM_Additive UMETA(DisplayName="Additive"),
	SCCM_Composite UMETA(DisplayName="Composite")
};

/** Maximum number of custom lighting channels */
#define NUM_LIGHTING_CHANNELS 3

/** Specifies which lighting channels are relevant */
USTRUCT(BlueprintType)
struct FLightingChannels
{
	GENERATED_BODY()

	FLightingChannels() :
		bChannel0(true),
		bChannel1(false),
		bChannel2(false)
	{}

	/** Default channel for all primitives and lights. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadOnly, Category=Channels)
	uint8 bChannel0:1;

	/** First custom channel */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadOnly, Category=Channels)
	uint8 bChannel1:1;

	/** Second custom channel */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadOnly, Category=Channels)
	uint8 bChannel2:1;
};

/** Converts lighting channels into a bitfield */
inline uint8 GetLightingChannelMaskForStruct(FLightingChannels Value)
{
	// Note: this is packed into 3 bits of a stencil channel
	return (uint8)((Value.bChannel0 ? 1 : 0) | (Value.bChannel1 << 1) | (Value.bChannel2 << 2));
}

/** Returns mask for only channel 0  */
inline uint8 GetDefaultLightingChannelMask()
{
	return 1;
}

/** Returns the index of the first lighting channel set, or -1 if no channels are set. */
inline int32 GetFirstLightingChannelFromMask(uint8 Mask)
{
	return Mask ? FPlatformMath::CountTrailingZeros(Mask) : -1;
}

/** View / light masking support */
USTRUCT(BlueprintType)
struct FViewLightingChannels
{
	GENERATED_BODY()

	FViewLightingChannels() :
		bViewChannel0(true),
		bViewChannel1(false),
		bViewChannel2(false),
		bViewChannel3(false),
		bViewChannel4(false)
	{}

	/** View specific lighting channel 0 (enabled by default). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Channels)
	uint8 bViewChannel0:1;

	/** View specific lighting channel 1. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Channels)
	uint8 bViewChannel1:1;

	/** View specific lighting channel 2. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Channels)
	uint8 bViewChannel2:1;

	/** View specific lighting channel 3. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Channels)
	uint8 bViewChannel3:1;

	/** View specific lighting channel 4. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Channels)
	uint8 bViewChannel4:1;

	/** Converts view lighting channels into a bitfield for use in FLightSceneProxy */
	inline uint8 GetMaskForStruct() const
	{
		return (uint8)((bViewChannel0 << 0) | (bViewChannel1 << 1) | (bViewChannel2 << 2) | (bViewChannel3 << 3) | (bViewChannel4 << 4));
	}

	static inline constexpr uint8 GetMaskDefault() { return 0x1; }
};

/** 
 * Enumerates available GBufferFormats. 
 * @warning When this enum is updated please update CVarGBufferFormat comments 
 */
UENUM()
namespace EGBufferFormat
{
	enum Type : int
	{
		/** Forces all GBuffers to 8 bits per channel. Intended as profiling for best performance. (Substrate: Octahedral encoding as 2x11bits for simple and single materials, 2x16bits for complex materials) */
		Force8BitsPerChannel = 0 UMETA(DisplayName = "Force 8 Bits Per Channel"),
		/** See GBuffer allocation function for layout details. (Substrate: Octahedral encoding as 2x11bits for simple and single material, 2x16bits for complex materials) */
		Default = 1,
		/** Same as Default except normals are encoded at 16 bits per channel. (Substrate: Octahedral encoding as 2x16bits for all materials.) */
		HighPrecisionNormals = 3,
		/** Forces all GBuffers to 16 bits per channel. Intended as profiling for best quality. (Substrate: Octahedral encoding as 2x16bits for all materials.) */
		Force16BitsPerChannel = 5 UMETA(DisplayName = "Force 16 Bits Per Channel"),
	};
}


/** 
 * Enumerates available MobileLocalLightSetting. 
 * @warning When this enum is updated please update CVarMobileForwardEnableLocalLights comments 
 */
UENUM()
enum EMobileLocalLightSetting : int
{
	LOCAL_LIGHTS_DISABLED UMETA(DisplayName = "Local Lights Disabled"),
	LOCAL_LIGHTS_ENABLED UMETA(DisplayName = "Local Lights Enabled"),
	LOCAL_LIGHTS_BUFFER UE_DEPRECATED(5.8, "Local Lights Buffer has been deprecated. For local lights heavy games, please use mobile deferred r.Mobile.ShadingPath=1") UMETA(hidden)
};

/** Controls the way that the width scale property affects animation trails. */
UENUM()
enum ETrailWidthMode : int
{
	ETrailWidthMode_FromCentre UMETA(DisplayName = "From Centre"),
	ETrailWidthMode_FromFirst UMETA(DisplayName = "From First Socket"),
	ETrailWidthMode_FromSecond UMETA(DisplayName = "From Second Socket"),
};

/** Specifies how particle collision is computed for GPU particles */
UENUM()
namespace EParticleCollisionMode
{
	enum Type : int
	{
		SceneDepth UMETA(DisplayName="Scene Depth"),
		DistanceField UMETA(DisplayName="Distance Field")
	};
}

/** 
 * Specifies the overal rendering/shading model for a material
 * @warning Check UMaterialInstance::Serialize if changed!
 */
UENUM()
enum EMaterialShadingModel : int
{
	MSM_Unlit					UMETA(DisplayName="Unlit"),
	MSM_DefaultLit				UMETA(DisplayName="Default Lit"),
	MSM_Subsurface				UMETA(DisplayName="Subsurface"),
	MSM_PreintegratedSkin		UMETA(DisplayName="Preintegrated Skin"),
	MSM_ClearCoat				UMETA(DisplayName="Clear Coat"),
	MSM_SubsurfaceProfile		UMETA(DisplayName="Subsurface Profile"),
	MSM_TwoSidedFoliage			UMETA(DisplayName="Two Sided Foliage"),
	MSM_Hair					UMETA(DisplayName="Hair"),
	MSM_Cloth					UMETA(DisplayName="Cloth"),
	MSM_Eye						UMETA(DisplayName="Eye"),
	MSM_SingleLayerWater		UMETA(DisplayName="SingleLayerWater"),
	MSM_ThinTranslucent			UMETA(DisplayName="Thin Translucent"),
	MSM_Strata					UMETA(DisplayName="Substrate", Hidden),
	/** Number of unique shading models. */
	MSM_NUM						UMETA(Hidden),
	/** Shading model will be determined by the Material Expression Graph,
		by utilizing the 'Shading Model' MaterialAttribute output pin. */
	MSM_FromMaterialExpression	UMETA(DisplayName="From Material Expression"),
	MSM_MAX
};

static_assert(MSM_NUM <= 16, "Do not exceed 16 shading models without expanding FMaterialShadingModelField to support uint32 instead of uint16!");

/** Wrapper for a bitfield of shading models. A material contains one of these to describe what possible shading models can be used by that material. */
USTRUCT()
struct FMaterialShadingModelField
{
	GENERATED_USTRUCT_BODY()

public:
	FMaterialShadingModelField() {}
	FMaterialShadingModelField(EMaterialShadingModel InShadingModel)		{ AddShadingModel(InShadingModel); }

	void AddShadingModel(EMaterialShadingModel InShadingModel)				{ check(InShadingModel < MSM_NUM); ShadingModelField |= (1 << (uint16)InShadingModel); }
	void RemoveShadingModel(EMaterialShadingModel InShadingModel)			{ ShadingModelField &= ~(1 << (uint16)InShadingModel); }
	void ClearShadingModels()												{ ShadingModelField = 0; }

	// Check if any of the given shading models are present
	bool HasAnyShadingModel(const TArray<EMaterialShadingModel>& InShadingModels) const	
	{ 
		for (EMaterialShadingModel ShadingModel : InShadingModels)
		{
			if (HasShadingModel(ShadingModel))
			{
				return true;
			}
		}
		return false; 
	}

	bool HasShadingModel(EMaterialShadingModel InShadingModel) const		{ return (ShadingModelField & (1 << (uint16)InShadingModel)) != 0; }
	bool HasOnlyShadingModel(EMaterialShadingModel InShadingModel) const	{ return ShadingModelField == (1 << (uint16)InShadingModel); }
	bool IsUnlit() const													{ return HasShadingModel(MSM_Unlit); }
	bool IsLit() const														{ return !IsUnlit(); }
	bool IsValid() const													{ return (ShadingModelField > 0) && (ShadingModelField < (1 << MSM_NUM)); }
	uint16 GetShadingModelField() const										{ return ShadingModelField; }
	void SetShadingModelField(uint16 InShadingModelField)					{ ShadingModelField = InShadingModelField; }
	int32 CountShadingModels() const										{ return FMath::CountBits(ShadingModelField); }
	EMaterialShadingModel GetFirstShadingModel() const						{ check(IsValid()); return (EMaterialShadingModel)FMath::CountTrailingZeros(ShadingModelField); }

	bool operator==(const FMaterialShadingModelField& Other) const			{ return ShadingModelField == Other.GetShadingModelField(); }
	bool operator!=(const FMaterialShadingModelField& Other) const			{ return ShadingModelField != Other.GetShadingModelField(); }

private:
	UPROPERTY()
	uint16 ShadingModelField = 0;
};

/** 
* Enumerates available Substrate storage format
*/
UENUM()
namespace ESubstrateStorageFormat
{
	enum Type : int
	{
		/** Blendable GBuffer - Fast but with limited lighting complexity. If selected, all platforms will be forced to use this format. */
		BlendableGBuffer = 0 UMETA(DisplayName = "Blendable GBuffer"),

		/** Adaptive GBuffer - More heavy but enable complex lighting behavior per pixel. If selected, only compatible platforms will use this format. Uncompatible platforms will use Blendable GBuffer format. */
		AdaptiveBuffer = 1 UMETA(DisplayName = "Adaptive GBuffer"),
	};
}

/** 
 * Enumerates available Substrate closure configuration
 */
UENUM()
namespace ESubstrateClosureConfig
{
	enum Type : int
	{
		/** Use the mininum between the platform's closure count and the project's closure count, on platforms supporting Adaptive GBuffer */
		UseMinProjectAndPlatforClosuresPerPixel = 0 UMETA(DisplayName = "Min of project's and platfom's closure count"),

		/** Force the project's closure count, on all platforms supporting Adaptive GBuffer */
		ForceProjectClosuresPerPixel = 1 UMETA(DisplayName = "Force Closure Count"),
	};
}

/**
 * Specifies the Substrate runtime shading model summarized from the material graph
 * Not exposed in UI, only used in code.
 * Those states are deducted from the material graph and map to a specific domain/shading state.
 */
enum ESubstrateShadingModel : int
{
	SSM_Unlit,
	SSM_DefaultLit,
	SSM_ThinTranslucent,
	SSM_SubsurfaceMFP,
	SSM_SubsurfaceProfile,
	SSM_SubsurfaceWrap,
	SSM_SubsurfaceThinTwoSided,
	SSM_VolumetricFogCloud,
	SSM_Hair,
	SSM_Eye,
	SSM_Cloth,
	SSM_ClearCoat,
	SSM_SingleLayerWater,
	SSM_LightFunction,
	SSM_PostProcess,
	SSM_Decal,
	SSM_UI,
	SSM_Toon,
	/** Number of unique shading models. */
	SSM_NUM,
};
static_assert(SSM_NUM <= 32, "Do not exceed 32 shading models without expanding FSubstrateMaterialInfo::ShadingModelField to support more bits!");

// This used to track cyclic graph which we do not support. We only support acyclic graph and a depth of 128 is already too high for a realistic use case.
#define SUBSTRATE_TREE_MAX_DEPTH 48

/** Gather information from the Substrate material graph to setup material for runtime. */
struct FSubstrateMaterialInfo
{
public:
	FSubstrateMaterialInfo(bool bGatherGuids = false) { bGatherMaterialExpressionGuids = bGatherGuids; }
	FSubstrateMaterialInfo(ESubstrateShadingModel InShadingModel) { AddShadingModel(InShadingModel); }

	// Shading model
	void AddShadingModel(ESubstrateShadingModel InShadingModel) { check(InShadingModel < SSM_NUM); ShadingModelField |= (1u << (uint32)InShadingModel); }
	void SetSingleShadingModel(ESubstrateShadingModel InShadingModel) { check(InShadingModel < SSM_NUM); ShadingModelField = (1u << (uint32)InShadingModel); }
	bool HasShadingModel(ESubstrateShadingModel InShadingModel) const { return (ShadingModelField & (1u << (uint32)InShadingModel)) != 0; }
	bool HasOnlyShadingModel(ESubstrateShadingModel InShadingModel) const { return ShadingModelField == (1u << (uint32)InShadingModel); }
	uint32 GetShadingModelField() const { return ShadingModelField; }
	int32 CountShadingModels() const { return FMath::CountBits(ShadingModelField); }

	// Subsurface profiles
	void AddSubsurfaceProfile(USubsurfaceProfile* InProfile) { if (InProfile) SubsurfaceProfiles.Add(InProfile); }
	int32 CountSubsurfaceProfiles() const { return SubsurfaceProfiles.Num(); }
	USubsurfaceProfile* GetSubsurfaceProfile(int32 Index) const { return SubsurfaceProfiles.IsValidIndex(Index) ? SubsurfaceProfiles[Index] : nullptr; }

	// Specular profiles
	void AddSpecularProfile(USpecularProfile* InProfile) { if (InProfile) SpecularProfiles.Add(InProfile); }
	int32 CountSpecularProfiles() const { return SpecularProfiles.Num(); }
	USpecularProfile* GetSpecularProfile(int32 Index) const { return SpecularProfiles.IsValidIndex(Index) ? SpecularProfiles[Index] : nullptr; }

	// Toon profiles
	void AddToonProfile(UToonProfile* InProfile) { if (InProfile) ToonProfiles.Add(InProfile); }
	int32 CountToonProfiles() const { return ToonProfiles.Num(); }
	UToonProfile* GetToonProfile(int32 Index) const { return ToonProfiles.IsValidIndex(Index) ? ToonProfiles[Index] : nullptr; }

	// Shading model from expression
	void SetShadingModelFromExpression(bool bIn) { bHasShadingModelFromExpression = bIn ? 1u : 0u; }
	bool HasShadingModelFromExpression() const { return bHasShadingModelFromExpression > 0u; }

	// Substrate material expression GUIDs
	void AddGuid(const FGuid& In) { if (bGatherMaterialExpressionGuids) { MaterialExpressionGuids.Add(In); } }
	const TArray<FGuid>& GetGuids() const { return MaterialExpressionGuids; }

	uint64 GetPropertyConnected() const { return ConnectedPropertyMask; }
	void AddPropertyConnected(uint64 In) { ConnectedPropertyMask |= (1ull << In); }
	bool HasPropertyConnected(uint64 In) const { return !!(ConnectedPropertyMask & (1ull << In)); }
	static bool HasPropertyConnected(uint64 InConnectedPropertyMask, uint64 In) { return !!(InConnectedPropertyMask & (1ull << In)); }

	bool IsValid() const { return (ShadingModelField > 0) && (ShadingModelField < (1 << SSM_NUM)); }

	bool operator==(const FSubstrateMaterialInfo& Other) const { return ShadingModelField == Other.GetShadingModelField(); }
	bool operator!=(const FSubstrateMaterialInfo& Other) const { return ShadingModelField != Other.GetShadingModelField(); }

#if WITH_EDITOR
	// Returns true if everything went fine (not out of Substrate tree stack)
	bool PushSubstrateTreeStack()
	{
		bOutOfStackDepthWhenParsing = bOutOfStackDepthWhenParsing || (++ParsingStackDepth > SUBSTRATE_TREE_MAX_DEPTH);
		return !bOutOfStackDepthWhenParsing;
	}
	void PopSubstrateTreeStack()
	{
		ParsingStackDepth--;
		check(ParsingStackDepth >= 0);
	}
	bool GetSubstrateTreeOutOfStackDepthOccurred() 
	{
		return bOutOfStackDepthWhenParsing;
	}
#endif

private:
	uint32 ShadingModelField = 0;

	/* Indicates if the shading model is constant or data-driven from the shader graph */
	uint8 bHasShadingModelFromExpression = 0;

	/* Indicates which (legacy) inputs are connected */
	uint64 ConnectedPropertyMask = 0;

	TArray<TObjectPtr<USubsurfaceProfile>> SubsurfaceProfiles;

	TArray<TObjectPtr<USpecularProfile>> SpecularProfiles;

	TArray<TObjectPtr<UToonProfile>> ToonProfiles;
	
	/* Material expression GUIDs for all the traversed Substrate nodes */
	bool bGatherMaterialExpressionGuids = false;
	TArray<FGuid> MaterialExpressionGuids;

#if WITH_EDITOR
	// A simple way to detect and prevent node re-entry due to cycling graph; stop the compilation and avoid crashing.
	bool bOutOfStackDepthWhenParsing = false;
	int32 ParsingStackDepth = 0;
#endif
};

template<>
struct TStructOpsTypeTraits<FSubstrateMaterialInfo> : public TStructOpsTypeTraitsBase2<FSubstrateMaterialInfo>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

/** Describes how textures are sampled for materials */
UENUM(BlueprintType)
enum EMaterialSamplerType : int
{
	SAMPLERTYPE_Color UMETA(DisplayName="Color"),
	SAMPLERTYPE_Grayscale UMETA(DisplayName="Grayscale"),
	SAMPLERTYPE_Alpha UMETA(DisplayName="Alpha"),
	SAMPLERTYPE_Normal UMETA(DisplayName="Normal"),
	SAMPLERTYPE_Masks UMETA(DisplayName="Masks"),
	SAMPLERTYPE_DistanceFieldFont UMETA(DisplayName="Distance Field Font"),
	SAMPLERTYPE_LinearColor UMETA(DisplayName = "Linear Color"),
	SAMPLERTYPE_LinearGrayscale UMETA(DisplayName = "Linear Grayscale"),
	SAMPLERTYPE_Data UMETA(DisplayName = "Data"),
	SAMPLERTYPE_External UMETA(DisplayName = "External"),

	SAMPLERTYPE_VirtualColor UMETA(DisplayName = "Virtual Color"),
	SAMPLERTYPE_VirtualGrayscale UMETA(DisplayName = "Virtual Grayscale"),
	SAMPLERTYPE_VirtualAlpha UMETA(DisplayName = "Virtual Alpha"),
	SAMPLERTYPE_VirtualNormal UMETA(DisplayName = "Virtual Normal"),
	SAMPLERTYPE_VirtualMasks UMETA(DisplayName = "Virtual Mask"),
	/*No DistanceFiledFont Virtual*/
	SAMPLERTYPE_VirtualLinearColor UMETA(DisplayName = "Virtual Linear Color"),
	SAMPLERTYPE_VirtualLinearGrayscale UMETA(DisplayName = "Virtual Linear Grayscale"),
	/*No External Virtual*/

	SAMPLERTYPE_MAX,
};

inline bool IsVirtualSamplerType(EMaterialSamplerType Value)
{
	return ((int32)Value >= (int32)SAMPLERTYPE_VirtualColor && (int32)Value <= (int32)SAMPLERTYPE_VirtualLinearGrayscale);
}
UENUM()
enum EMaterialStencilCompare : int
{
	MSC_Less			UMETA(DisplayName = "Less Than"),
	MSC_LessEqual		UMETA(DisplayName = "Less Than or Equal"),
	MSC_Greater			UMETA(DisplayName = "Greater Than"),
	MSC_GreaterEqual	UMETA(DisplayName = "Greater Than or Equal"),
	MSC_Equal			UMETA(DisplayName = "Equal"),
	MSC_NotEqual		UMETA(DisplayName = "Not Equal"),
	MSC_Never			UMETA(DisplayName = "Never"),
	MSC_Always			UMETA(DisplayName = "Always"),
	MSC_Count			UMETA(Hidden),
};

UENUM()
enum EMaterialShadingRate : int
{
	MSR_1x1				UMETA(DisplayName = "1x1"),
	MSR_2x1				UMETA(DisplayName = "2x1"),
	MSR_1x2				UMETA(DisplayName = "1x2"),
	MSR_2x2				UMETA(DisplayName = "2x2"),
	MSR_4x2				UMETA(DisplayName = "4x2"),
	MSR_2x4				UMETA(DisplayName = "2x4"),
	MSR_4x4				UMETA(DisplayName = "4x4"),
	MSR_Count			UMETA(Hidden),
};


/**	Lighting build quality enumeration */
UENUM(BlueprintType)
enum ELightingBuildQuality : int
{
	Quality_Preview		UMETA(DisplayName = "Preview"),
	Quality_Medium		UMETA(DisplayName = "Medium"),
	Quality_High		UMETA(DisplayName = "High"),
	Quality_Production	UMETA(DisplayName = "Production"),
	Quality_MAX			UMETA(Hidden),
};

/** Movement modes for Characters. */
UENUM(BlueprintType)
enum EMovementMode : int
{
	/** None (movement is disabled). */
	MOVE_None		UMETA(DisplayName="None"),

	/** Walking on a surface. */
	MOVE_Walking	UMETA(DisplayName="Walking"),

	/** 
	 * Simplified walking on navigation data (e.g. navmesh). 
	 * If GetGenerateOverlapEvents() is true, then we will perform sweeps with each navmesh move.
	 * If GetGenerateOverlapEvents() is false then movement is cheaper but characters can overlap other objects without some extra process to repel/resolve their collisions.
	 */
	MOVE_NavWalking	UMETA(DisplayName="Navmesh Walking"),

	/** Falling under the effects of gravity, such as after jumping or walking off the edge of a surface. */
	MOVE_Falling	UMETA(DisplayName="Falling"),

	/** Swimming through a fluid volume, under the effects of gravity and buoyancy. */
	MOVE_Swimming	UMETA(DisplayName="Swimming"),

	/** Flying, ignoring the effects of gravity. Affected by the current physics volume's fluid friction. */
	MOVE_Flying		UMETA(DisplayName="Flying"),

	/** User-defined custom movement mode, including many possible sub-modes. */
	MOVE_Custom		UMETA(DisplayName="Custom"),

	MOVE_MAX		UMETA(Hidden),
};

/** Smoothing approach used by network interpolation for Characters. */
UENUM(BlueprintType)
enum class ENetworkSmoothingMode : uint8
{
	/** No smoothing, only change position as network position updates are received. */
	Disabled		UMETA(DisplayName="Disabled"),

	/** Linear interpolation from source to target. */
	Linear			UMETA(DisplayName="Linear"),

	/** Exponential. Faster as you are further from target. */
	Exponential		UMETA(DisplayName="Exponential"),

};

// Number of bits used currently from FMaskFilter.
enum { NumExtraFilterBits = 6 };

// NOTE!!Some of these values are used to index into FCollisionResponseContainers and must be kept in sync.
// @see FCollisionResponseContainer::SetResponse().

// @NOTE!!!! This DisplayName [DISPLAYNAME] SHOULD MATCH suffix of ECC_DISPLAYNAME
// Otherwise it will mess up collision profile loading
// If you change this, please also change FCollisionResponseContainers
//
// If you add any more TraceQuery="1", you also should change UCollisionProfile::LoadProfileConfig
// Metadata doesn't work outside of editor, so you'll need to add manually

// @NOTE : when you add more here for predefined engine channel
// please change the max in the CollisionProfile
// search ECC_Destructible

// in order to use this custom channels
// we recommend to define in your local file
// - i.e. #define COLLISION_WEAPON		ECC_GameTraceChannel1
// and make sure you customize these it in INI file by
// 
// in DefaultEngine.ini
//
// [/Script/Engine.CollisionProfile]
// GameTraceChannel1="Weapon"
// 
// also in the INI file, you can override collision profiles that are defined by simply redefining
// note that Weapon isn't defined in the BaseEngine.ini file, but "Trigger" is defined in Engine
// +Profiles=(Name="Trigger",CollisionEnabled=QueryOnly,ObjectTypeName=WorldDynamic, DefaultResponse=ECR_Overlap, CustomResponses=((Channel=Visibility, Response=ECR_Ignore), (Channel=Weapon, Response=ECR_Ignore)))


/** 
 * Enum indicating different type of objects for rigid-body collision purposes. 
 */
UENUM(BlueprintType)
enum ECollisionChannel : int
{

	ECC_WorldStatic UMETA(DisplayName="WorldStatic"),
	ECC_WorldDynamic UMETA(DisplayName="WorldDynamic"),
	ECC_Pawn UMETA(DisplayName="Pawn"),
	ECC_Visibility UMETA(DisplayName="Visibility" , TraceQuery="1"),
	ECC_Camera UMETA(DisplayName="Camera" , TraceQuery="1"),
	ECC_PhysicsBody UMETA(DisplayName="PhysicsBody"),
	ECC_Vehicle UMETA(DisplayName="Vehicle"),
	ECC_Destructible UMETA(DisplayName="Destructible"),

	/** Reserved for gizmo collision */
	ECC_EngineTraceChannel1 UMETA(Hidden),

	ECC_EngineTraceChannel2 UMETA(Hidden),
	ECC_EngineTraceChannel3 UMETA(Hidden),
	ECC_EngineTraceChannel4 UMETA(Hidden), 
	ECC_EngineTraceChannel5 UMETA(Hidden),
	ECC_EngineTraceChannel6 UMETA(Hidden),

	ECC_GameTraceChannel1 UMETA(Hidden),
	ECC_GameTraceChannel2 UMETA(Hidden),
	ECC_GameTraceChannel3 UMETA(Hidden),
	ECC_GameTraceChannel4 UMETA(Hidden),
	ECC_GameTraceChannel5 UMETA(Hidden),
	ECC_GameTraceChannel6 UMETA(Hidden),
	ECC_GameTraceChannel7 UMETA(Hidden),
	ECC_GameTraceChannel8 UMETA(Hidden),
	ECC_GameTraceChannel9 UMETA(Hidden),
	ECC_GameTraceChannel10 UMETA(Hidden),
	ECC_GameTraceChannel11 UMETA(Hidden),
	ECC_GameTraceChannel12 UMETA(Hidden),
	ECC_GameTraceChannel13 UMETA(Hidden),
	ECC_GameTraceChannel14 UMETA(Hidden),
	ECC_GameTraceChannel15 UMETA(Hidden),
	ECC_GameTraceChannel16 UMETA(Hidden),
	ECC_GameTraceChannel17 UMETA(Hidden),
	ECC_GameTraceChannel18 UMETA(Hidden),
	ECC_GameTraceChannel19 UMETA(Hidden),
	ECC_GameTraceChannel20 UMETA(Hidden),
	ECC_GameTraceChannel21 UMETA(Hidden),
	ECC_GameTraceChannel22 UMETA(Hidden),
	ECC_GameTraceChannel23 UMETA(Hidden),
	ECC_GameTraceChannel24 UMETA(Hidden),
	ECC_GameTraceChannel25 UMETA(Hidden),
	ECC_GameTraceChannel26 UMETA(Hidden),
	ECC_GameTraceChannel27 UMETA(Hidden),
	ECC_GameTraceChannel28 UMETA(Hidden),
	ECC_GameTraceChannel29 UMETA(Hidden),
	ECC_GameTraceChannel30 UMETA(Hidden),
	ECC_GameTraceChannel31 UMETA(Hidden),
	ECC_GameTraceChannel32 UMETA(Hidden),
	ECC_GameTraceChannel33 UMETA(Hidden),
	ECC_GameTraceChannel34 UMETA(Hidden),
	ECC_GameTraceChannel35 UMETA(Hidden),
	ECC_GameTraceChannel36 UMETA(Hidden),
	ECC_GameTraceChannel37 UMETA(Hidden),
	ECC_GameTraceChannel38 UMETA(Hidden),
	ECC_GameTraceChannel39 UMETA(Hidden),
	ECC_GameTraceChannel40 UMETA(Hidden),
	ECC_GameTraceChannel41 UMETA(Hidden),
	ECC_GameTraceChannel42 UMETA(Hidden),
	ECC_GameTraceChannel43 UMETA(Hidden),
	ECC_GameTraceChannel44 UMETA(Hidden),
	ECC_GameTraceChannel45 UMETA(Hidden),
	ECC_GameTraceChannel46 UMETA(Hidden),
	ECC_GameTraceChannel47 UMETA(Hidden),
	ECC_GameTraceChannel48 UMETA(Hidden),
	ECC_GameTraceChannel49 UMETA(Hidden),
	ECC_GameTraceChannel50 UMETA(Hidden),
	
	/** Add new serializeable channels above here (i.e. entries that exist in FCollisionResponseContainer) */
	/** Add only nonserialized/transient flags below */

	// NOTE!!!! THESE ARE BEING DEPRECATED BUT STILL THERE FOR BLUEPRINT. PLEASE DO NOT USE THEM IN CODE

	ECC_OverlapAll_Deprecated UMETA(Hidden),
	ECC_MAX,
};

DECLARE_DELEGATE_OneParam(FOnConstraintBroken, int32 /*ConstraintIndex*/);
DECLARE_DELEGATE_ThreeParams(FOnConstraintViolated, int32 /*ConstraintIndex*/, float /*LinearViolation*/, float /*AngularViolation*/);
DECLARE_DELEGATE_OneParam(FOnPlasticDeformation, int32 /*ConstraintIndex*/);

#define COLLISION_GIZMO ECC_EngineTraceChannel1

/** 
 * Specifies what types of objects to return from an overlap physics query
 * @warning If you change this, change GetCollisionChannelFromOverlapFilter() to match 
 */
UENUM(BlueprintType)
enum EOverlapFilterOption : int
{
	/** Returns both overlaps with both dynamic and static components */
	OverlapFilter_All UMETA(DisplayName="AllObjects"),
	/** returns only overlaps with dynamic actors (far fewer results in practice, much more efficient) */
	OverlapFilter_DynamicOnly UMETA(DisplayName="AllDynamicObjects"),
	/** returns only overlaps with static actors (fewer results, more efficient) */
	OverlapFilter_StaticOnly UMETA(DisplayName="AllStaticObjects"),
};

/** Specifies custom collision object types, overridable per game */
UENUM(BlueprintType)
enum EObjectTypeQuery : int
{
	ObjectTypeQuery1 UMETA(Hidden), 
	ObjectTypeQuery2 UMETA(Hidden), 
	ObjectTypeQuery3 UMETA(Hidden), 
	ObjectTypeQuery4 UMETA(Hidden), 
	ObjectTypeQuery5 UMETA(Hidden), 
	ObjectTypeQuery6 UMETA(Hidden), 
	ObjectTypeQuery7 UMETA(Hidden), 
	ObjectTypeQuery8 UMETA(Hidden), 
	ObjectTypeQuery9 UMETA(Hidden), 
	ObjectTypeQuery10 UMETA(Hidden), 
	ObjectTypeQuery11 UMETA(Hidden), 
	ObjectTypeQuery12 UMETA(Hidden), 
	ObjectTypeQuery13 UMETA(Hidden), 
	ObjectTypeQuery14 UMETA(Hidden), 
	ObjectTypeQuery15 UMETA(Hidden), 
	ObjectTypeQuery16 UMETA(Hidden), 
	ObjectTypeQuery17 UMETA(Hidden), 
	ObjectTypeQuery18 UMETA(Hidden), 
	ObjectTypeQuery19 UMETA(Hidden), 
	ObjectTypeQuery20 UMETA(Hidden), 
	ObjectTypeQuery21 UMETA(Hidden), 
	ObjectTypeQuery22 UMETA(Hidden), 
	ObjectTypeQuery23 UMETA(Hidden), 
	ObjectTypeQuery24 UMETA(Hidden), 
	ObjectTypeQuery25 UMETA(Hidden), 
	ObjectTypeQuery26 UMETA(Hidden), 
	ObjectTypeQuery27 UMETA(Hidden), 
	ObjectTypeQuery28 UMETA(Hidden), 
	ObjectTypeQuery29 UMETA(Hidden), 
	ObjectTypeQuery30 UMETA(Hidden), 
	ObjectTypeQuery31 UMETA(Hidden), 
	ObjectTypeQuery32 UMETA(Hidden),
	ObjectTypeQuery33 UMETA(Hidden), 
	ObjectTypeQuery34 UMETA(Hidden), 
	ObjectTypeQuery35 UMETA(Hidden), 
	ObjectTypeQuery36 UMETA(Hidden), 
	ObjectTypeQuery37 UMETA(Hidden), 
	ObjectTypeQuery38 UMETA(Hidden), 
	ObjectTypeQuery39 UMETA(Hidden), 
	ObjectTypeQuery40 UMETA(Hidden), 
	ObjectTypeQuery41 UMETA(Hidden), 
	ObjectTypeQuery42 UMETA(Hidden),
	ObjectTypeQuery43 UMETA(Hidden), 
	ObjectTypeQuery44 UMETA(Hidden), 
	ObjectTypeQuery45 UMETA(Hidden), 
	ObjectTypeQuery46 UMETA(Hidden), 
	ObjectTypeQuery47 UMETA(Hidden), 
	ObjectTypeQuery48 UMETA(Hidden), 
	ObjectTypeQuery49 UMETA(Hidden), 
	ObjectTypeQuery50 UMETA(Hidden),
	ObjectTypeQuery51 UMETA(Hidden),
	ObjectTypeQuery52 UMETA(Hidden),
	ObjectTypeQuery53 UMETA(Hidden),
	ObjectTypeQuery54 UMETA(Hidden),
	ObjectTypeQuery55 UMETA(Hidden),
	ObjectTypeQuery56 UMETA(Hidden),
	ObjectTypeQuery57 UMETA(Hidden),
	ObjectTypeQuery58 UMETA(Hidden),
	ObjectTypeQuery59 UMETA(Hidden),
	ObjectTypeQuery60 UMETA(Hidden),
	ObjectTypeQuery61 UMETA(Hidden),
	ObjectTypeQuery62 UMETA(Hidden),
	ObjectTypeQuery63 UMETA(Hidden),
	ObjectTypeQuery64 UMETA(Hidden),

	ObjectTypeQuery_MAX	UMETA(Hidden)
};

/** Specifies custom collision trace types, overridable per game */
UENUM(BlueprintType)
enum ETraceTypeQuery : int
{
	TraceTypeQuery1 UMETA(Hidden), 
	TraceTypeQuery2 UMETA(Hidden), 
	TraceTypeQuery3 UMETA(Hidden), 
	TraceTypeQuery4 UMETA(Hidden), 
	TraceTypeQuery5 UMETA(Hidden), 
	TraceTypeQuery6 UMETA(Hidden), 
	TraceTypeQuery7 UMETA(Hidden), 
	TraceTypeQuery8 UMETA(Hidden), 
	TraceTypeQuery9 UMETA(Hidden), 
	TraceTypeQuery10 UMETA(Hidden), 
	TraceTypeQuery11 UMETA(Hidden), 
	TraceTypeQuery12 UMETA(Hidden), 
	TraceTypeQuery13 UMETA(Hidden), 
	TraceTypeQuery14 UMETA(Hidden), 
	TraceTypeQuery15 UMETA(Hidden), 
	TraceTypeQuery16 UMETA(Hidden), 
	TraceTypeQuery17 UMETA(Hidden), 
	TraceTypeQuery18 UMETA(Hidden), 
	TraceTypeQuery19 UMETA(Hidden), 
	TraceTypeQuery20 UMETA(Hidden), 
	TraceTypeQuery21 UMETA(Hidden), 
	TraceTypeQuery22 UMETA(Hidden), 
	TraceTypeQuery23 UMETA(Hidden), 
	TraceTypeQuery24 UMETA(Hidden), 
	TraceTypeQuery25 UMETA(Hidden), 
	TraceTypeQuery26 UMETA(Hidden), 
	TraceTypeQuery27 UMETA(Hidden), 
	TraceTypeQuery28 UMETA(Hidden), 
	TraceTypeQuery29 UMETA(Hidden), 
	TraceTypeQuery30 UMETA(Hidden), 
	TraceTypeQuery31 UMETA(Hidden), 
	TraceTypeQuery32 UMETA(Hidden),
	TraceTypeQuery33 UMETA(Hidden), 
	TraceTypeQuery34 UMETA(Hidden), 
	TraceTypeQuery35 UMETA(Hidden), 
	TraceTypeQuery36 UMETA(Hidden), 
	TraceTypeQuery37 UMETA(Hidden), 
	TraceTypeQuery38 UMETA(Hidden), 
	TraceTypeQuery39 UMETA(Hidden), 
	TraceTypeQuery40 UMETA(Hidden), 
	TraceTypeQuery41 UMETA(Hidden), 
	TraceTypeQuery42 UMETA(Hidden),
	TraceTypeQuery43 UMETA(Hidden), 
	TraceTypeQuery44 UMETA(Hidden), 
	TraceTypeQuery45 UMETA(Hidden), 
	TraceTypeQuery46 UMETA(Hidden), 
	TraceTypeQuery47 UMETA(Hidden), 
	TraceTypeQuery48 UMETA(Hidden), 
	TraceTypeQuery49 UMETA(Hidden), 
	TraceTypeQuery50 UMETA(Hidden),
	TraceTypeQuery51 UMETA(Hidden),
	TraceTypeQuery52 UMETA(Hidden),
	TraceTypeQuery53 UMETA(Hidden),
	TraceTypeQuery54 UMETA(Hidden),
	TraceTypeQuery55 UMETA(Hidden),
	TraceTypeQuery56 UMETA(Hidden),
	TraceTypeQuery57 UMETA(Hidden),
	TraceTypeQuery58 UMETA(Hidden),
	TraceTypeQuery59 UMETA(Hidden),
	TraceTypeQuery60 UMETA(Hidden),
	TraceTypeQuery61 UMETA(Hidden),
	TraceTypeQuery62 UMETA(Hidden),
	TraceTypeQuery63 UMETA(Hidden),
	TraceTypeQuery64 UMETA(Hidden),

	TraceTypeQuery_MAX	UMETA(Hidden)
};

/** Enum indicating how each type should respond */
UENUM(BlueprintType, meta=(ScriptName="CollisionResponseType"))
enum ECollisionResponse : int
{
	ECR_Ignore UMETA(DisplayName="Ignore"),
	ECR_Overlap UMETA(DisplayName="Overlap"),
	ECR_Block UMETA(DisplayName="Block"),
	ECR_MAX,
};

/** Interpolation method used by animation blending */
UENUM()
enum EFilterInterpolationType : int
{
	BSIT_Average UMETA(DisplayName = "Averaged"),
	BSIT_Linear UMETA(DisplayName = "Linear"),
	BSIT_Cubic UMETA(DisplayName = "Cubic"),
	BSIT_EaseInOut UMETA(DisplayName = "Ease In/Out"),
	BSIT_ExponentialDecay UMETA(DisplayName = "Exponential"),
	BSIT_SpringDamper UMETA(DisplayName = "Spring Damper"),
	BSIT_MAX
};

/** Specifies the goal/source of a UWorld object */
namespace EWorldType
{
	enum Type
	{
		/** An untyped world, in most cases this will be the vestigial worlds of streamed in sub-levels */
		None,

		/** The game world */
		Game,

		/** A world being edited in the editor */
		Editor,

		/** A Play In Editor world */
		PIE,

		/** A preview world for an editor tool */
		EditorPreview,

		/** A preview world for a game */
		GamePreview,

		/** A minimal RPC world for a game */
		GameRPC,

		/** An editor world that was loaded but not currently being edited in the level editor */
		Inactive
	};
}

ENGINE_API const TCHAR* LexToString(const EWorldType::Type Value);

/** Describes what parts of level streaming should be forcibly handled immediately */
enum class EFlushLevelStreamingType : uint8
{
	/** Do not flush state on change */
	None,			
	/** Allow multiple load requests */
	Full,			
	/** Flush visibility only, do not allow load requests, flushes async loading as well */
	Visibility,		
};

/** Describes response for a single collision response channel */
USTRUCT()
struct FResponseChannel
{
	GENERATED_BODY()

	/** This should match DisplayName of ECollisionChannel 
	 *	Meta data of custom channels can be used as well
	 */
	 UPROPERTY(EditAnywhere, Category = FResponseChannel)
	FName Channel;

	/** Describes how the channel behaves */
	UPROPERTY(EditAnywhere, Category = FResponseChannel)
	TEnumAsByte<enum ECollisionResponse> Response;

	FResponseChannel()
		: Response(ECR_Block) {}

	FResponseChannel( FName InChannel, ECollisionResponse InResponse )
		: Channel(InChannel)
		, Response(InResponse) {}

	bool operator==(const FResponseChannel& Other) const
	{
		return Channel == Other.Channel && Response == Other.Response;
	}
};


/**
 *	Container for indicating a set of collision channels that this object will collide with.
 */
USTRUCT(BlueprintType)
struct FCollisionResponseContainer
{
	GENERATED_BODY()

#if !CPP      //noexport property

	///////////////////////////////////////
	// Reserved Engine Trace Channels
	// 
	// Note - 	If you change this (add/remove/modify) 
	// 			you should make sure it matches with ECollisionChannel (including DisplayName)
	// 			They has to be mirrored if serialized
	///////////////////////////////////////
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="WorldStatic"))
	TEnumAsByte<enum ECollisionResponse> WorldStatic;    // 0

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="WorldDynamic"))
	TEnumAsByte<enum ECollisionResponse> WorldDynamic;    // 1.

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="Pawn"))
	TEnumAsByte<enum ECollisionResponse> Pawn;    		// 2

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="Visibility"))
	TEnumAsByte<enum ECollisionResponse> Visibility;    // 3

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="Camera"))
	TEnumAsByte<enum ECollisionResponse> Camera;    // 4

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="PhysicsBody"))
	TEnumAsByte<enum ECollisionResponse> PhysicsBody;    // 5

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="Vehicle"))
	TEnumAsByte<enum ECollisionResponse> Vehicle;    // 6

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer, meta=(DisplayName="Destructible"))
	TEnumAsByte<enum ECollisionResponse> Destructible;    // 7


	///////////////////////////////////////
	// Unspecified Engine Trace Channels
	///////////////////////////////////////
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel1;    // 8

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel2;    // 9

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel3;    // 10

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel4;    // 11

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel5;    // 12

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> EngineTraceChannel6;    // 13

	///////////////////////////////////////
	// in order to use this custom channels
	// we recommend to define in your local file
	// - i.e. #define COLLISION_WEAPON		ECC_GameTraceChannel1
	// and make sure you customize these it in INI file by
	// 
	// in DefaultEngine.ini
	//	
	// [/Script/Engine.CollisionProfile]
	// GameTraceChannel1="Weapon"
	// 
	// also in the INI file, you can override collision profiles that are defined by simply redefining
	// note that Weapon isn't defined in the BaseEngine.ini file, but "Trigger" is defined in Engine
	// +Profiles=(Name="Trigger",CollisionEnabled=QueryOnly,ObjectTypeName=WorldDynamic, DefaultResponse=ECR_Overlap, CustomResponses=((Channel=Visibility, Response=ECR_Ignore), (Channel=Weapon, Response=ECR_Ignore)))
	///////////////////////////////////////
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel1;    // 14

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel2;    // 15

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel3;    // 16

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel4;    // 17

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel5;    // 18

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel6;    // 19

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel7;    // 20

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel8;    // 21

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel9;    // 22

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel10;    // 23

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel11;    // 24

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel12;    // 25

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel13;    // 26

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel14;    // 27

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel15;    // 28

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel16;    // 29

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel17;    // 30

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel18;    // 31

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel19;    // 32

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel20;    // 33

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel21;    // 34

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel22;    // 35

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel23;    // 36

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel24;    // 37

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel25;    // 38

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel26;    // 39

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel27;    // 40

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel28;    // 41

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel29;    // 42

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel30;    // 43

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel31;    // 44

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel32;    // 45

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel33;    // 46

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel34;    // 47

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel35;    // 48

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel36;    // 49

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel37;    // 50

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel38;    // 51

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel39;    // 52

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel40;    // 53

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel41;    // 54

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel42;    // 55

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel43;    // 56

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel44;    // 57

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel45;    // 58

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel46;    // 59

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel47;    // 60

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel48;    // 61

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel49;    // 62

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=CollisionResponseContainer)
	TEnumAsByte<enum ECollisionResponse> GameTraceChannel50;    // 63

#endif

	union
	{
		struct
		{
			//Reserved Engine Trace Channels
			uint8 WorldStatic;			// 0
			uint8 WorldDynamic;			// 1
			uint8 Pawn;					// 2
			uint8 Visibility;			// 3
			uint8 Camera;				// 4
			uint8 PhysicsBody;			// 5
			uint8 Vehicle;				// 6
			uint8 Destructible;			// 7

			// Unspecified Engine Trace Channels
			uint8 EngineTraceChannel1;   // 8
			uint8 EngineTraceChannel2;   // 9
			uint8 EngineTraceChannel3;   // 10
			uint8 EngineTraceChannel4;   // 11
			uint8 EngineTraceChannel5;   // 12
			uint8 EngineTraceChannel6;   // 13

			// Unspecified Game Trace Channels
			uint8 GameTraceChannel1;     // 14
			uint8 GameTraceChannel2;     // 15
			uint8 GameTraceChannel3;     // 16
			uint8 GameTraceChannel4;     // 17
			uint8 GameTraceChannel5;     // 18
			uint8 GameTraceChannel6;     // 19
			uint8 GameTraceChannel7;     // 20
			uint8 GameTraceChannel8;     // 21
			uint8 GameTraceChannel9;     // 22
			uint8 GameTraceChannel10;    // 23
			uint8 GameTraceChannel11;    // 24 
			uint8 GameTraceChannel12;    // 25
			uint8 GameTraceChannel13;    // 26
			uint8 GameTraceChannel14;    // 27
			uint8 GameTraceChannel15;    // 28
			uint8 GameTraceChannel16;    // 29 
			uint8 GameTraceChannel17;    // 30
			uint8 GameTraceChannel18;    // 31
			uint8 GameTraceChannel19;    // 32
			uint8 GameTraceChannel20;    // 33
			uint8 GameTraceChannel21;    // 34
			uint8 GameTraceChannel22;    // 35
			uint8 GameTraceChannel23;    // 36
			uint8 GameTraceChannel24;    // 37
			uint8 GameTraceChannel25;    // 38
			uint8 GameTraceChannel26;    // 39
			uint8 GameTraceChannel27;    // 40
			uint8 GameTraceChannel28;    // 41
			uint8 GameTraceChannel29;    // 42
			uint8 GameTraceChannel30;    // 43
			uint8 GameTraceChannel31;    // 44
			uint8 GameTraceChannel32;    // 45
			uint8 GameTraceChannel33;    // 46
			uint8 GameTraceChannel34;    // 47
			uint8 GameTraceChannel35;    // 48
			uint8 GameTraceChannel36;    // 49
			uint8 GameTraceChannel37;    // 50
			uint8 GameTraceChannel38;    // 51
			uint8 GameTraceChannel39;    // 52
			uint8 GameTraceChannel40;    // 53
			uint8 GameTraceChannel41;    // 54
			uint8 GameTraceChannel42;    // 55
			uint8 GameTraceChannel43;    // 56
			uint8 GameTraceChannel44;    // 57
			uint8 GameTraceChannel45;    // 58
			uint8 GameTraceChannel46;    // 59
			uint8 GameTraceChannel47;    // 60
			uint8 GameTraceChannel48;    // 61
			uint8 GameTraceChannel49;    // 62
			uint8 GameTraceChannel50;    // 63
		};
		uint8 EnumArray[64];
	};

	/** This constructor will set all channels to ECR_Block */
	ENGINE_API FCollisionResponseContainer();
	ENGINE_API FCollisionResponseContainer(ECollisionResponse DefaultResponse);

	/** Set the response of a particular channel in the structure. */
	ENGINE_API bool SetResponse(ECollisionChannel Channel, ECollisionResponse NewResponse);

	/** Set all channels to the specified response */
	ENGINE_API bool SetAllChannels(ECollisionResponse NewResponse);

	/** Replace the channels matching the old response with the new response */
	ENGINE_API bool ReplaceChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse);

	/** Returns the response set on the specified channel */
	inline ECollisionResponse GetResponse(ECollisionChannel Channel) const { return (ECollisionResponse)EnumArray[Channel]; }

	/** Set all channels from ChannelResponse Array **/
	ENGINE_API void UpdateResponsesFromArray(TArray<FResponseChannel> & ChannelResponses);
	ENGINE_API int32 FillArrayFromResponses(TArray<FResponseChannel> & ChannelResponses);

	/** Take two response containers and create a new container where each element is the 'min' of the two inputs (ie Ignore and Block results in Ignore) */
	static ENGINE_API FCollisionResponseContainer CreateMinContainer(const FCollisionResponseContainer& A, const FCollisionResponseContainer& B);

	/** Returns the game-wide default collision response */
	static const struct FCollisionResponseContainer& GetDefaultResponseContainer() { return DefaultResponseContainer; }

	bool operator==(const FCollisionResponseContainer& Other) const
	{
		return FMemory::Memcmp(EnumArray, Other.EnumArray, sizeof(Other.EnumArray)) == 0;
	}
	bool operator!=(const FCollisionResponseContainer& Other) const
	{
		return FMemory::Memcmp(EnumArray, Other.EnumArray, sizeof(Other.EnumArray)) != 0;
	}

private:

	/** static variable for default data to be used without reconstructing everytime **/
	static ENGINE_API FCollisionResponseContainer DefaultResponseContainer;

	friend class UCollisionProfile;
};

/** Enum used to indicate what type of timeline signature a function matches. */
UENUM()
enum ETimelineSigType : int
{
	ETS_EventSignature,
	ETS_FloatSignature,
	ETS_VectorSignature,
	ETS_LinearColorSignature,
	ETS_InvalidSignature,
	ETS_MAX,
};

/** Enum used to describe what type of collision is enabled on a body. */
UENUM(BlueprintType)
namespace ECollisionEnabled 
{ 
	enum Type : int
	{ 
		/** Will not create any representation in the physics engine. Cannot be used for spatial queries (raycasts, sweeps, overlaps) or simulation (rigid body, constraints). Best performance possible (especially for moving objects) */
		NoCollision UMETA(DisplayName="No Collision"), 
		/** Only used for spatial queries (raycasts, sweeps, and overlaps). Cannot be used for simulation (rigid body, constraints). Useful for character movement and things that do not need physical simulation. Performance gains by keeping data out of simulation tree. */
		QueryOnly UMETA(DisplayName="Query Only (No Physics Collision)"),
		/** Only used only for physics simulation (rigid body, constraints). Cannot be used for spatial queries (raycasts, sweeps, overlaps). Useful for jiggly bits on characters that do not need per bone detection. Performance gains by keeping data out of query tree */
		PhysicsOnly UMETA(DisplayName="Physics Only (No Query Collision)"),
		/** Can be used for both spatial queries (raycasts, sweeps, overlaps) and simulation (rigid body, constraints). */
		QueryAndPhysics UMETA(DisplayName="Collision Enabled (Query and Physics)"),
		/** Only used for probing the physics simulation (rigid body, constraints). Cannot be used for spatial queries (raycasts,
		sweeps, overlaps). Useful for when you want to detect potential physics interactions and pass contact data to hit callbacks
		or contact modification, but don't want to physically react to these contacts. */
		ProbeOnly UMETA(DisplayName="Probe Only (Contact Data, No Query or Physics Collision)"),
		/** Can be used for both spatial queries (raycasts, sweeps, overlaps) and probing the physics simulation (rigid body,
		constraints). Will not allow for actual physics interaction, but will generate contact data, trigger hit callbacks, and
		contacts will appear in contact modification. */
		QueryAndProbe UMETA(DisplayName="Query and Probe (Query Collision and Contact Data, No Physics Collision)")
	}; 
} 

struct FCollisionEnabledMask
{
	int8 Bits;

	ENGINE_API FCollisionEnabledMask(int8 InBits = 0);
	ENGINE_API FCollisionEnabledMask(ECollisionEnabled::Type CollisionEnabled);

	ENGINE_API operator int8() const;
	ENGINE_API operator bool() const;
	ENGINE_API FCollisionEnabledMask operator&(const FCollisionEnabledMask Other) const;
	ENGINE_API FCollisionEnabledMask operator&(const ECollisionEnabled::Type Other) const;
	ENGINE_API FCollisionEnabledMask operator|(const FCollisionEnabledMask Other) const;
	ENGINE_API FCollisionEnabledMask operator|(const ECollisionEnabled::Type Other) const;
};

extern FCollisionEnabledMask ENGINE_API operator&(const ECollisionEnabled::Type A, const ECollisionEnabled::Type B);
extern FCollisionEnabledMask ENGINE_API operator&(const ECollisionEnabled::Type A, const FCollisionEnabledMask B);
extern FCollisionEnabledMask ENGINE_API operator|(const ECollisionEnabled::Type A, const ECollisionEnabled::Type B);
extern FCollisionEnabledMask ENGINE_API operator|(const ECollisionEnabled::Type A, const FCollisionEnabledMask B);

inline bool CollisionEnabledHasPhysics(ECollisionEnabled::Type CollisionEnabled)
{
	return	(CollisionEnabled == ECollisionEnabled::PhysicsOnly) ||
			(CollisionEnabled == ECollisionEnabled::QueryAndPhysics);
}

inline bool CollisionEnabledHasQuery(ECollisionEnabled::Type CollisionEnabled)
{
	return	(CollisionEnabled == ECollisionEnabled::QueryOnly) ||
			(CollisionEnabled == ECollisionEnabled::QueryAndPhysics) ||
			(CollisionEnabled == ECollisionEnabled::QueryAndProbe);
}

inline bool CollisionEnabledHasProbe(ECollisionEnabled::Type CollisionEnabled)
{
	return (CollisionEnabled == ECollisionEnabled::ProbeOnly) ||
			(CollisionEnabled == ECollisionEnabled::QueryAndProbe);
}

inline ECollisionEnabled::Type CollisionEnabledIntersection(ECollisionEnabled::Type CollisionEnabledA, ECollisionEnabled::Type CollisionEnabledB)
{
	// Combine collision two enabled data.
	//
	// The intersection follows the following rules:
	//
	// * For the result to have Query, both data must have Query
	// * For the result to have Probe, either data must have Probe
	// * For the result to have Physics, both data must have Physics and the result must not have Probe
	//
	// This way if an object is query-only, for example, but one of its shapes is query-and-physics,
	// the object's settings win and the shape ends up being query-only. And if the object is marked
	// as physics, but the child is marked as probe (or vice versa), use probe.
	//
	// The following matrix represents the intersection relationship.
	// NOTE: The order of types must match the order declared in ECollisionEnabled!
	using namespace ECollisionEnabled;
	static constexpr ECollisionEnabled::Type IntersectionMatrix[5][5] = {
		/*                 |        QueryOnly       PhysicsOnly     QueryAndPhysics     ProbeOnly     QueryAndProbe   */
		/*-----------------+------------------------------------------------------------------------------------------*/
		/* QueryOnly       | */ {   QueryOnly,      NoCollision,    QueryOnly,          NoCollision,  QueryOnly       },
		/* PhysicsOnly     | */ {   NoCollision,    PhysicsOnly,    PhysicsOnly,        ProbeOnly,    ProbeOnly       },
		/* QueryAndPhysics | */ {   QueryOnly,      PhysicsOnly,    QueryAndPhysics,    ProbeOnly,    QueryAndProbe   },
		/* ProbeOnly       | */ {   NoCollision,    ProbeOnly,      ProbeOnly,          ProbeOnly,    ProbeOnly       },
		/* QueryAndProbe   | */ {   QueryOnly,      ProbeOnly,      QueryAndProbe,      ProbeOnly,    QueryAndProbe   }
	};

	// Subtract 1 because the first index is NoCollision.
	// If both indices indicate _some_ collision setting (ie, greater than -1),
	// lookup their intersection setting in the matrix.
	const int32 IndexA = (int32)CollisionEnabledA - 1;
	const int32 IndexB = (int32)CollisionEnabledB - 1;
	if (IndexA >= 0 && IndexB >= 0)
	{
		return IntersectionMatrix[IndexA][IndexB];
	}

	// Either of the two settings were set to NoCollision which trumps all
	return ECollisionEnabled::NoCollision;
}

/** Convert a set of three bools into an ECollisionEnabled */
inline ECollisionEnabled::Type CollisionEnabledFromFlags(const bool bQuery, const bool bPhysics, const bool bProbe)
{
	// Convert to ints for bit manipulation
	const int32 QueryBit = static_cast<uint32>(bQuery);
	const int32 PhysicsBit = static_cast<uint32>(bPhysics);
	const int32 ProbeBit = static_cast<uint32>(bProbe);

	// NOTE:
	// We use &~ between physics and probe because we cannot have a case with
	// both enabled, and in every case probe "beats" physics. See the collision
	// type rules outlined in the comments in CollisionEnabledIntersection.
	//
	// index    type              bits: probe/physics/query
	// ----------------------------------------------------
	// 0        NoCollision       000
	// 1        QueryOnly         001
	// 2        PhysicsOnly       010
	// 3        QueryAndPhysics   011
	// 4        ProbeOnly         100 (& 110)
	// 5        QueryAndProbe     101 (& 111)
	return static_cast<ECollisionEnabled::Type>(
		(QueryBit << 0) |
		(PhysicsBit & (~ProbeBit)) << 1 |
		(ProbeBit << 2));
}

/** Convert an ECollisionEnabled enum into a set of three bools */
inline void CollisionEnabledToFlags(const ECollisionEnabled::Type CollisionEnabled, bool& bQuery, bool& bPhysics, bool& bProbe)
{
	const int32 Bits = static_cast<int32>(CollisionEnabled);
	bQuery   = ((Bits & 0x1) != 0);
	bPhysics = ((Bits & 0x2) != 0);
	bProbe   = ((Bits & 0x4) != 0);
}

/** Describes type of wake/sleep event sent to the physics system */
enum class ESleepEvent : uint8
{
	SET_Wakeup,
	SET_Sleep
};

/** Rigid body error correction data */
USTRUCT()
struct FRigidBodyErrorCorrection
{
	GENERATED_BODY()

	/** Value between 0 and 1 which indicates how much velocity
		and ping based correction to use */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float PingExtrapolation;

	/** For the purpose of extrapolation, ping will be clamped to this value */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float PingLimit;

	/** Error per centimeter */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float ErrorPerLinearDifference;

	/** Error per degree */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float ErrorPerAngularDifference;

	/** Maximum allowable error for a state to be considered "resolved" */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float MaxRestoredStateError;

	UPROPERTY(EditAnywhere, Category = "Replication")
	float MaxLinearHardSnapDistance;

	/** How much to directly lerp to the correct position. Generally
		this should be very low, if not zero. A higher value will
		increase precision along with jerkiness. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float PositionLerp;

	/** How much to directly lerp to the correct angle. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float AngleLerp;

	/** This is the coefficient `k` in the differential equation:
		dx/dt = k ( x_target(t) - x(t) ), which is used to update
		the velocity in a replication step. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float LinearVelocityCoefficient;

	/** This is the angular analog to LinearVelocityCoefficient. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float AngularVelocityCoefficient;

	/** Number of seconds to remain in a heuristically
		unresolveable state before hard snapping. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float ErrorAccumulationSeconds;

	/** If the body has moved less than the square root of
		this amount towards a resolved state in the previous
		frame, then error may accumulate towards a hard snap. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float ErrorAccumulationDistanceSq;

	/** If the previous error projected onto the current error
		is greater than this value (indicating "similarity"
		between states), then error may accumulate towards a
		hard snap. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float ErrorAccumulationSimilarity;

	FRigidBodyErrorCorrection()
		: PingExtrapolation(0.1f)
		, PingLimit(100.f)
		, ErrorPerLinearDifference(1.0f)
		, ErrorPerAngularDifference(1.0f)
		, MaxRestoredStateError(1.0f)
		, MaxLinearHardSnapDistance(400.f)
		, PositionLerp(0.0f)
		, AngleLerp(0.4f)
		, LinearVelocityCoefficient(100.0f)
		, AngularVelocityCoefficient(10.0f)
		, ErrorAccumulationSeconds(0.5f)
		, ErrorAccumulationDistanceSq(15.0f)
		, ErrorAccumulationSimilarity(100.0f)
	{ }
};

/**
 * Information about one contact between a pair of rigid bodies.
 */
USTRUCT()
struct FRigidBodyContactInfo
{
	GENERATED_BODY()

	/** Position of contact, where two shapes intersect */
	UPROPERTY()
	FVector ContactPosition;

	/** Normal of contact, points from second shape towards first shape */
	UPROPERTY()
	FVector ContactNormal;

	/** How far the two shapes penetrated into each other */
	UPROPERTY()
	float ContactPenetration;

	/** Was this contact generated by a probe constraint */
	UPROPERTY()
	bool bContactProbe;

	/** The physical material of the two shapes involved in a contact */
	UPROPERTY()
	TObjectPtr<class UPhysicalMaterial> PhysMaterial[2];


	FRigidBodyContactInfo()
		: ContactPosition(ForceInit)
		, ContactNormal(ForceInit)
		, ContactPenetration(0)
		, bContactProbe(false)
	{
		for (int32 ElementIndex = 0; ElementIndex < 2; ElementIndex++)
		{
			PhysMaterial[ElementIndex] = nullptr;
		}
	}

	FRigidBodyContactInfo(	const FVector& InContactPosition, 
							const FVector& InContactNormal, 
							float InPenetration, 
							bool bInProbe,
							UPhysicalMaterial* InPhysMat0, 
							UPhysicalMaterial* InPhysMat1 )
		: ContactPosition(InContactPosition)
		, ContactNormal(InContactNormal)
		, ContactPenetration(InPenetration)
		, bContactProbe(bInProbe)
	{
		PhysMaterial[0] = InPhysMat0;
		PhysMaterial[1] = InPhysMat1;
	}

	/** Swap the order of info in this info  */
	ENGINE_API void SwapOrder();
};


/**
 * Information about an overall collision, including contacts.
 */
USTRUCT()
struct FCollisionImpactData
{
	GENERATED_BODY()

	/** All the contact points in the collision*/
	UPROPERTY()
	TArray<struct FRigidBodyContactInfo> ContactInfos;

	/** The total impulse applied as the two objects push against each other*/
	UPROPERTY()
	FVector TotalNormalImpulse;

	/** The total counterimpulse applied of the two objects sliding against each other*/
	UPROPERTY()
	FVector TotalFrictionImpulse;

	UPROPERTY()
	bool bIsVelocityDeltaUnderThreshold;

	FCollisionImpactData()
	: TotalNormalImpulse(ForceInit)
	, TotalFrictionImpulse(ForceInit)
    , bIsVelocityDeltaUnderThreshold(true)
	{}

	/** Iterate over ContactInfos array and swap order of information */
	ENGINE_API void SwapContactOrders();
};

/** Struct used to hold effects for destructible damage events */
USTRUCT(BlueprintType)
struct FFractureEffect
{
	GENERATED_BODY()

	/** Particle system effect to play at fracture location. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FractureEffect)
	TObjectPtr<class UParticleSystem> ParticleSystem;

	/** Sound cue to play at fracture location. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FractureEffect)
	TObjectPtr<class USoundBase> Sound;

	FFractureEffect()
		: ParticleSystem(nullptr)
		, Sound(nullptr)
	{ }
};

/**	Struct for handling positions relative to a base actor, which is potentially moving */
USTRUCT(BlueprintType)
struct FBasedPosition
{
	GENERATED_BODY()

	/** Actor that is the base */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BasedPosition)
	TObjectPtr<class AActor> Base;

	/** Position relative to the base actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BasedPosition)
	FVector Position;

	UPROPERTY()
	mutable FVector CachedBaseLocation;

	UPROPERTY()
	mutable FRotator CachedBaseRotation;

	UPROPERTY()
	mutable FVector CachedTransPosition;

	ENGINE_API FBasedPosition();
	ENGINE_API explicit FBasedPosition( class AActor *InBase, const FVector& InPosition );

	/** Retrieve world location of this position */
	ENGINE_API FVector operator*() const;

	/** Updates base/position */
	ENGINE_API void Set( class AActor* InBase, const FVector& InPosition );

	/** Clear base/position */
	ENGINE_API void Clear();

	friend FArchive& operator<<( FArchive& Ar, FBasedPosition& T );
};

/** Struct for caching Quat<->Rotator conversions. */
struct FRotationConversionCache
{
	FRotationConversionCache()
	{
	}

	/** Convert a FRotator to FQuat. Uses the cached conversion if possible, and updates it if there was no match. */
	inline FQuat RotatorToQuat(const FRotator& InRotator) const
	{
		FPayload& Payload = GetOrCreatePayload();
		if (LIKELY(Payload.CachedRotator != InRotator))
		{
			Payload.CachedRotator = InRotator.GetNormalized();
			Payload.CachedQuat = Payload.CachedRotator.Quaternion();
		}
		return Payload.CachedQuat;
	}

	/** Convert a FRotator to FQuat. Uses the cached conversion if possible, but does *NOT* update the cache if there was no match. */
	inline FQuat RotatorToQuat_ReadOnly(const FRotator& InRotator) const
	{
		if (LIKELY(PayloadPtr.IsValid()))
		{
			FPayload& Payload = *PayloadPtr;
			if (LIKELY(Payload.CachedRotator == InRotator))
			{
				return Payload.CachedQuat;
			}
		}
		return InRotator.Quaternion();
	}

	/** Convert a FQuat to FRotator. Uses the cached conversion if possible, and updates it if there was no match. */
	inline FRotator QuatToRotator(const FQuat& InQuat) const
	{
		FPayload& Payload = GetOrCreatePayload();
		if (LIKELY(Payload.CachedQuat != InQuat))
		{
			Payload.CachedQuat = InQuat.GetNormalized();
			Payload.CachedRotator = Payload.CachedQuat.Rotator();
		}
		return Payload.CachedRotator;
	}

	/** Convert a FQuat to FRotator. Uses the cached conversion if possible, but does *NOT* update the cache if there was no match. */
	inline FRotator QuatToRotator_ReadOnly(const FQuat& InQuat) const
	{
		if (LIKELY(PayloadPtr.IsValid()))
		{
			FPayload& Payload = *PayloadPtr;
			if (LIKELY(Payload.CachedQuat == InQuat))
			{
				return Payload.CachedRotator;
			}
		}
		return InQuat.GetNormalized().Rotator();
	}

	/** Version of QuatToRotator when the Quat is known to already be normalized. */
	inline FRotator NormalizedQuatToRotator(const FQuat& InNormalizedQuat) const
	{
		FPayload& Payload = GetOrCreatePayload();
		if (LIKELY(Payload.CachedQuat != InNormalizedQuat))
		{
			Payload.CachedQuat = InNormalizedQuat;
			Payload.CachedRotator = InNormalizedQuat.Rotator();
		}
		return Payload.CachedRotator;
	}

	/** Version of QuatToRotator when the Quat is known to already be normalized. Does *NOT* update the cache if there was no match. */
	inline FRotator NormalizedQuatToRotator_ReadOnly(const FQuat& InNormalizedQuat) const
	{
		if (LIKELY(PayloadPtr.IsValid()))
		{
			FPayload& Payload = *PayloadPtr;
			if (LIKELY(Payload.CachedQuat == InNormalizedQuat))
			{
				return Payload.CachedRotator;
			}
		}
		return InNormalizedQuat.Rotator();
	}

	/** Return the cached Quat. */
	inline FQuat GetCachedQuat() const
	{
		if (UNLIKELY(!PayloadPtr.IsValid()))
		{
			return FQuat::Identity;
		}
		return PayloadPtr->CachedQuat;
	}

	/** Return the cached Rotator. */
	inline FRotator GetCachedRotator() const
	{
		if (UNLIKELY(!PayloadPtr.IsValid()))
		{
			return FRotator::ZeroRotator;
		}
		return PayloadPtr->CachedRotator;
	}

	FRotationConversionCache& operator=(const FRotationConversionCache& Other)
	{
		if (LIKELY(PayloadPtr.IsValid() || Other.PayloadPtr.IsValid()))
		{
			FPayload& Payload = GetOrCreatePayload();
			Payload.CachedQuat = Other.GetCachedQuat();
			Payload.CachedRotator = Other.GetCachedRotator();
		}
		return *this;
	}

private:

	struct FPayload
	{
		mutable FQuat				CachedQuat;		// FQuat matching CachedRotator such that CachedQuat.Rotator() == CachedRotator.
		mutable FRotator			CachedRotator;	// FRotator matching CachedQuat such that CachedRotator.Quaternion() == CachedQuat.

		FPayload()
		: CachedQuat(FQuat::Identity)
		, CachedRotator(FRotator::ZeroRotator)
		{}
	};

	inline FPayload&				GetOrCreatePayload() const
	{
		if (UNLIKELY(!PayloadPtr.IsValid()))
		{
			PayloadPtr = MakeUnique<FPayload>();
		}
		return *PayloadPtr;
	}

	mutable TUniquePtr<FPayload>	PayloadPtr;
};

/** A line of subtitle text and the time at which it should be displayed. */
USTRUCT(BlueprintType)
struct FSubtitleCue
{
	GENERATED_BODY()

	/** The text to appear in the subtitle. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SubtitleCue)
	FText Text;

	/** The time at which the subtitle is to be displayed, in seconds relative to the beginning of the line. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SubtitleCue)
	float Time;

	FSubtitleCue()
		: Time(0)
	{ }
};

/**	Per-light settings for Lightmass */
USTRUCT()
struct FLightmassLightSettings
{
	GENERATED_BODY()

	/** 0 will be completely desaturated, 1 will be unchanged */
	UPROPERTY(EditAnywhere, Category=Lightmass, meta=(UIMin = "0.0", UIMax = "4.0"))
	float IndirectLightingSaturation;

	/** Controls the falloff of shadow penumbras */
	UPROPERTY(EditAnywhere, Category=Lightmass, meta=(UIMin = "0.1", UIMax = "4.0"))
	float ShadowExponent;

	/** 
	 * Whether to use area shadows for stationary light precomputed shadowmaps.  
	 * Area shadows get softer the further they are from shadow casters, but require higher lightmap resolution to get the same quality where the shadow is sharp.
	 */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	bool bUseAreaShadowsForStationaryLight;

	FLightmassLightSettings()
		: IndirectLightingSaturation(1.0f)
		, ShadowExponent(2.0f)
		, bUseAreaShadowsForStationaryLight(false)
	{ }
};

/**	Point/spot settings for Lightmass */
USTRUCT()
struct FLightmassPointLightSettings : public FLightmassLightSettings
{
	GENERATED_BODY()
};

/**	Directional light settings for Lightmass */
USTRUCT()
struct FLightmassDirectionalLightSettings : public FLightmassLightSettings
{
	GENERATED_BODY()

	/** Angle that the directional light's emissive surface extends relative to a receiver, affects penumbra sizes. */
	UPROPERTY(EditAnywhere, Category=Lightmass, meta=(UIMin = ".0001", UIMax = "5"))
	float LightSourceAngle;

	FLightmassDirectionalLightSettings()
		: LightSourceAngle(1.0f)
	{
	}
};

/**	Per-object settings for Lightmass */
USTRUCT()
struct FLightmassPrimitiveSettings
{
	GENERATED_BODY()

	/** If true, this object will be lit as if it receives light from both sides of its polygons. */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	uint32 bUseTwoSidedLighting:1;

	/** If true, this object will only shadow indirect lighting.  					*/
	UPROPERTY(EditAnywhere, Category=Lightmass)
	uint32 bShadowIndirectOnly:1;

	/** If true, allow using the emissive for static lighting.						*/
	UPROPERTY(EditAnywhere, Category=Lightmass)
	uint32 bUseEmissiveForStaticLighting:1;

	/** 
	 * Typically the triangle normal is used for hemisphere gathering which prevents incorrect self-shadowing from artist-tweaked vertex normals. 
	 * However in the case of foliage whose vertex normal has been setup to match the underlying terrain, gathering in the direction of the vertex normal is desired.
	 */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	uint32 bUseVertexNormalForHemisphereGather:1;

	/** Direct lighting falloff exponent for mesh area lights created from emissive areas on this primitive. */
	UPROPERTY()
	float EmissiveLightFalloffExponent;

	/**
	 * Direct lighting influence radius.
	 * The default is 0, which means the influence radius should be automatically generated based on the emissive light brightness.
	 * Values greater than 0 override the automatic method.
	 */
	UPROPERTY()
	float EmissiveLightExplicitInfluenceRadius;

	/** Scales the emissive contribution of all materials applied to this object.	*/
	UPROPERTY(EditAnywhere, Category=Lightmass)
	float EmissiveBoost;

	/** Scales the diffuse contribution of all materials applied to this object.	*/
	UPROPERTY(EditAnywhere, Category=Lightmass)
	float DiffuseBoost;

	/** Fraction of samples taken that must be occluded in order to reach full occlusion. */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	float FullyOccludedSamplesFraction;

	FLightmassPrimitiveSettings()
	{
		bUseTwoSidedLighting = false;
		bShadowIndirectOnly = false;
		bUseEmissiveForStaticLighting = false;
		bUseVertexNormalForHemisphereGather = false;
		EmissiveLightFalloffExponent = 8.0f;
		EmissiveLightExplicitInfluenceRadius = 0.0f;
		EmissiveBoost = 1.0f;
		DiffuseBoost = 1.0f;
		FullyOccludedSamplesFraction = 1.0f;
	}

	friend bool operator==(const FLightmassPrimitiveSettings& A, const FLightmassPrimitiveSettings& B)
	{
		//@todo Do we want a little 'leeway' in joining 
		if ((A.bUseTwoSidedLighting != B.bUseTwoSidedLighting) ||
			(A.bShadowIndirectOnly != B.bShadowIndirectOnly) || 
			(A.bUseEmissiveForStaticLighting != B.bUseEmissiveForStaticLighting) || 
			(A.bUseVertexNormalForHemisphereGather != B.bUseVertexNormalForHemisphereGather) || 
			(fabsf(A.EmissiveLightFalloffExponent - B.EmissiveLightFalloffExponent) > UE_SMALL_NUMBER) ||
			(fabsf(A.EmissiveLightExplicitInfluenceRadius - B.EmissiveLightExplicitInfluenceRadius) > UE_SMALL_NUMBER) ||
			(fabsf(A.EmissiveBoost - B.EmissiveBoost) > UE_SMALL_NUMBER) ||
			(fabsf(A.DiffuseBoost - B.DiffuseBoost) > UE_SMALL_NUMBER) ||
			(fabsf(A.FullyOccludedSamplesFraction - B.FullyOccludedSamplesFraction) > UE_SMALL_NUMBER))
		{
			return false;
		}
		return true;
	}

	// Functions.
	friend FArchive& operator<<(FArchive& Ar, FLightmassPrimitiveSettings& Settings);
};

/**	Debug options for Lightmass */
USTRUCT()
struct FLightmassDebugOptions
{
	GENERATED_BODY()

	/**
	 *	If false, UnrealLightmass.exe is launched automatically (default)
	 *	If true, it must be launched manually (e.g. through a debugger) with the -debug command line parameter.
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bDebugMode:1;

	/**	If true, all participating Lightmass agents will report back detailed stats to the log.	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bStatsEnabled:1;

	/**	If true, BSP surfaces split across model components are joined into 1 mapping	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bGatherBSPSurfacesAcrossComponents:1;

	/**	The tolerance level used when gathering BSP surfaces.	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	float CoplanarTolerance;

	/**
	 *	If true, Lightmass will import mappings immediately as they complete.
	 *	It will not process them, however.
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bUseImmediateImport:1;

	/**
	 *	If true, Lightmass will process appropriate mappings as they are imported.
	 *	NOTE: Requires ImmediateMode be enabled to actually work.
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bImmediateProcessMappings:1;

	/**	If true, Lightmass will sort mappings by texel cost. */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bSortMappings:1;

	/**	If true, the generate coefficients will be dumped to binary files. */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bDumpBinaryFiles:1;

	/**
	 *	If true, Lightmass will write out BMPs for each generated material property
	 *	sample to <GAME>\ScreenShots\Materials.
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bDebugMaterials:1;

	/**	If true, Lightmass will pad the calculated mappings to reduce/eliminate seams. */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bPadMappings:1;

	/**
	 *	If true, will fill padding of mappings with a color rather than the sampled edges.
	 *	Means nothing if bPadMappings is not enabled...
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bDebugPaddings:1;

	/**
	 * If true, only the mapping containing a debug texel will be calculated, all others
	 * will be set to white
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bOnlyCalcDebugTexelMappings:1;

	/** If true, color lightmaps a random color */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bUseRandomColors:1;

	/** If true, a green border will be placed around the edges of mappings */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bColorBordersGreen:1;

	/**
	 * If true, Lightmass will overwrite lightmap data with a shade of red relating to
	 * how long it took to calculate the mapping (Red = Time / ExecutionTimeDivisor)
	 */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	uint32 bColorByExecutionTime:1;

	/** The amount of time that will be count as full red when bColorByExecutionTime is enabled */
	UPROPERTY(EditAnywhere, Category=LightmassDebugOptions)
	float ExecutionTimeDivisor;

	ENGINE_API FLightmassDebugOptions();
};

/**
 *	Debug options for Swarm
 */
USTRUCT()
struct FSwarmDebugOptions
{
	GENERATED_BODY()

	/**
	 *	If true, Swarm will distribute jobs.
	 *	If false, only the local machine will execute the jobs.
	 */
	UPROPERTY(EditAnywhere, Category=SwarmDebugOptions)
	uint32 bDistributionEnabled:1;

	/**
	 *	If true, Swarm will force content to re-export rather than using the cached version.
	 *	If false, Swarm will attempt to use the cached version.
	 */
	UPROPERTY(EditAnywhere, Category=SwarmDebugOptions)
	uint32 bForceContentExport:1;

	UPROPERTY()
	uint32 bInitialized:1;

	FSwarmDebugOptions()
		: bDistributionEnabled(true)
		, bForceContentExport(false)
		, bInitialized(false)
	{
	}

	//@todo For some reason, the global instance is not initializing to the default settings...
	// Be sure to update this function to properly set the desired initial values!!!!
	void Touch();
};

/** Method for padding a light map in memory */
UENUM()
enum ELightMapPaddingType : int
{
	LMPT_NormalPadding,
	LMPT_PrePadding,
	LMPT_NoPadding
};

/** Bit-field flags that affects storage (e.g. packing, streaming) and other info about a shadowmap. */
UENUM()
enum EShadowMapFlags : int
{
	/** No flags. */
	SMF_None			= 0,
	/** Shadowmap should be placed in a streaming texture. */
	SMF_Streamed		= 0x00000001
};

/** Whether to teleport physics body or not */
UENUM()
enum class ETeleportType : uint8
{
	/** Do not teleport physics body. This means velocity will reflect the movement between initial and final position, and collisions along the way will occur */
	None,

	/** Teleport physics body so that velocity remains the same and no collision occurs */
	TeleportPhysics,

	/** Teleport physics body and reset physics state completely */
	ResetPhysics,
};

inline ETeleportType TeleportFlagToEnum(bool bTeleport) { return bTeleport ? ETeleportType::TeleportPhysics : ETeleportType::None; }
inline bool TeleportEnumToFlag(ETeleportType Teleport) { return ETeleportType::TeleportPhysics == Teleport; }

/** Structure containing information about minimum translation direction (MTD) */
USTRUCT()
struct FMTDResult
{
	GENERATED_BODY()

	/** Normalized direction of the minimum translation required to fix penetration. */
	UPROPERTY()
	FVector Direction;

	/** Distance required to move along the MTD vector (Direction). */
	UPROPERTY()
	float Distance;

	FMTDResult()
	{
		FMemory::Memzero(this, sizeof(FMTDResult));
	}
};

/** Used to indicate each slot name and how many channels they have. */
USTRUCT()
struct FAnimSlotDesc
{
	GENERATED_BODY()

	/** Name of the slot. */
	UPROPERTY()
	FName SlotName;

	/** Number of channels that are available in this slot. */
	UPROPERTY()
	int32 NumChannels;

	FAnimSlotDesc()
		: NumChannels(0)
	{ }
};

/** Enum for controlling buckets for update rate optimizations if we need to stagger
 *  Multiple actor populations separately.
 */
UENUM()
enum class EUpdateRateShiftBucket : uint8
{
	ShiftBucket0 = 0,
	ShiftBucket1,
	ShiftBucket2,
	ShiftBucket3,
	ShiftBucket4,
	ShiftBucket5,
	ShiftBucketMax
};

/** Container for Animation Update Rate parameters.
 * They are shared for all components of an Actor, so they can be updated in sync. */
USTRUCT()
struct FAnimUpdateRateParameters
{
	GENERATED_BODY()

public:
	enum EOptimizeMode : uint8
	{
		TrailMode,
		LookAheadMode,
	};

	/** Cache which Update Rate Optimization mode we are using */
	EOptimizeMode OptimizeMode;

	/** The bucket to use when deciding which counter to use to calculate shift values */
	UPROPERTY()
	EUpdateRateShiftBucket ShiftBucket;

	/** When skipping a frame, should it be interpolated or frozen? */
	UPROPERTY()
	uint8 bInterpolateSkippedFrames : 1;

	/** Whether or not to use the defined LOD/Frameskip map instead of separate distance factor thresholds */
	UPROPERTY()
	uint8 bShouldUseLodMap : 1;

	/** If set, LOD/Frameskip map will be queried with mesh's MinLodModel instead of current LOD (PredictedLODLevel) */
	UPROPERTY()
	uint8 bShouldUseMinLod : 1;

	/** (This frame) animation update should be skipped. */
	UPROPERTY()
	uint8 bSkipUpdate : 1;

	/** (This frame) animation evaluation should be skipped. */
	UPROPERTY()
	uint8 bSkipEvaluation : 1;

	/** How often animation will be updated/ticked. 1 = every frame, 2 = every 2 frames, etc. */
	UPROPERTY()
	int32 UpdateRate;

	/** How often animation will be evaluated. 1 = every frame, 2 = every 2 frames, etc.
	 *  has to be a multiple of UpdateRate. */
	UPROPERTY()
	int32 EvaluationRate;

	/** Track time we have lost via skipping */
	UPROPERTY(Transient)
	float TickedPoseOffestTime;

	/** Total time of the last series of skipped updates */
	UPROPERTY(Transient)
	float AdditionalTime;

	/** The delta time of the last tick */
	float ThisTickDelta;

	/** Rate of animation evaluation when non rendered (off screen and dedicated servers).
	 * a value of 4 means evaluated 1 frame, then 3 frames skipped */
	UPROPERTY()
	int32 BaseNonRenderedUpdateRate;

	/** Max Evaluation Rate allowed for interpolation to be enabled. Beyond, interpolation will be turned off. */
	UPROPERTY()
	int32 MaxEvalRateForInterpolation;

	/** Array of MaxDistanceFactor to use for AnimUpdateRate when mesh is visible (rendered).
	 * MaxDistanceFactor is size on screen, as used by LODs
	 * Example:
	 *		BaseVisibleDistanceFactorThesholds.Add(0.4f)
	 *		BaseVisibleDistanceFactorThesholds.Add(0.2f)
	 * means:
	 *		0 frame skip, MaxDistanceFactor > 0.4f
	 *		1 frame skip, MaxDistanceFactor > 0.2f
	 *		2 frame skip, MaxDistanceFactor > 0.0f
	 */
	UPROPERTY()
	TArray<float> BaseVisibleDistanceFactorThesholds;

	/** Map of LOD levels to frame skip amounts. if bShouldUseLodMap is set these values will be used for
	 * the frameskip amounts and the distance factor thresholds will be ignored. The flag and these values
	 * should be configured using the customization callback when parameters are created for a component.
	 *
	 * Note that this is # of frames to skip, so if you have 20, that means every 21th frame, it will update, and evaluate. 
	 */
	UPROPERTY()
	TMap<int32, int32> LODToFrameSkipMap;

	/** Number of update frames that have been skipped in a row */
	UPROPERTY()
	int32 SkippedUpdateFrames;

	/** Number of evaluate frames that have been skipped in a row */
	UPROPERTY()
	int32 SkippedEvalFrames;

public:

	/** Default constructor. */
	FAnimUpdateRateParameters()
		: OptimizeMode(TrailMode)
		, ShiftBucket(EUpdateRateShiftBucket::ShiftBucket0)
		, bInterpolateSkippedFrames(false)
		, bShouldUseLodMap(false)
		, bShouldUseMinLod(false)
		, bSkipUpdate(false)
		, bSkipEvaluation(false)
		, UpdateRate(1)
		, EvaluationRate(1)
		, TickedPoseOffestTime(0.f)
		, AdditionalTime(0.f)
		, ThisTickDelta(0.f)
		, BaseNonRenderedUpdateRate(4)
		, MaxEvalRateForInterpolation(4)
		, SkippedUpdateFrames(0)
		, SkippedEvalFrames(0)
	{ 
		BaseVisibleDistanceFactorThesholds.Add(0.24f);
		BaseVisibleDistanceFactorThesholds.Add(0.12f);
	}

	/** 
	 * Set parameters and verify inputs for Trail Mode (original behaviour - skip frames, track skipped time and then catch up afterwards).
	 * @param : UpdateRateShift. Shift our update frames so that updates across all skinned components are staggered
	 * @param : NewUpdateRate. How often animation will be updated/ticked. 1 = every frame, 2 = every 2 frames, etc.
	 * @param : NewEvaluationRate. How often animation will be evaluated. 1 = every frame, 2 = every 2 frames, etc.
	 * @param : bNewInterpSkippedFrames. When skipping a frame, should it be interpolated or frozen?
	 */
	void SetTrailMode(float DeltaTime, uint8 UpdateRateShift, int32 NewUpdateRate, int32 NewEvaluationRate, bool bNewInterpSkippedFrames);

	/**
	 * Set parameters and verify inputs for Lookahead mode, which handles Root Motion
	 * @param : UpdateRateShift. Shift our update frames so that updates across all skinned components are staggered
	 * @param : LookAheadAmount. Amount of time to look ahead and predict movement 
	 */
	void SetLookAheadMode(float DeltaTime, uint8 UpdateRateShift, float LookAheadAmount);

	/** Amount to interpolate bone transforms */
	float GetInterpolationAlpha() const;

	/** Amount to interpoilate root motion */
	float GetRootMotionInterp() const;

	/** Return true if evaluation rate should be optimized at all */
	bool DoEvaluationRateOptimizations() const
	{
		return OptimizeMode == LookAheadMode || EvaluationRate > 1;
	}

	/** Getter for bSkipUpdate */
	bool ShouldSkipUpdate() const
	{
		return bSkipUpdate;
	}

	/** Getter for bSkipEvaluation */
	bool ShouldSkipEvaluation() const
	{
		return bSkipEvaluation;
	}

	/** Getter for bInterpolateSkippedFrames */
	bool ShouldInterpolateSkippedFrames() const
	{
		return bInterpolateSkippedFrames;
	}

	/** Called when we are ticking a pose to make sure we accumulate all needed time */
	float GetTimeAdjustment()
	{
		return AdditionalTime;
	}

	/** Returns color to use for debug UI */
	FColor GetUpdateRateDebugColor() const
	{
		if (OptimizeMode == TrailMode)
		{
			switch (UpdateRate)
			{
			case 1: return FColor::Red;
			case 2: return FColor::Green;
			case 3: return FColor::Blue;
			}
			return FColor::Black;
		}
		else
		{
			if (bSkipUpdate)
			{
				return FColor::Yellow;
			}
			return FColor::Green;
		}
	}
};

/** Point Of View structure used in Camera calculations */
USTRUCT(BlueprintType)
struct FPOV
{
	GENERATED_BODY()

	/** Location */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=POV)
	FVector Location;

	/** Rotation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=POV)
	FRotator Rotation;

	/** FOV angle */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=POV)
	float FOV;

	FPOV() 
	: Location(ForceInit),Rotation(ForceInit), FOV(90.0f)
	{}

	FPOV(FVector InLocation, FRotator InRotation, float InFOV)
	: Location(InLocation), Rotation(InRotation), FOV(InFOV) 
	{}

	// Serializer.
	friend FArchive& operator<<(FArchive& Ar, FPOV& POV)
	{
		return Ar << POV.Location << POV.Rotation << POV.FOV;
	}
};

/**
 * Settings applied when building a mesh.
 */
USTRUCT(BlueprintType)
struct FMeshBuildSettings
{
	GENERATED_BODY()

	/** If true, degenerate triangles will be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bUseMikkTSpace:1;

	/** If true, normals in the raw mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRecomputeNormals:1;

	/** If true, tangents in the raw mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRecomputeTangents:1;

	/** If true, we will use the surface area and the corner angle of the triangle as a ratio when computing the normals. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	uint8 bComputeWeightedNormals : 1;

	/** If true, degenerate triangles will be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRemoveDegenerates:1;
	
	/** Required to optimize mesh in mirrored transform. Double index buffer size. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bBuildReversedIndexBuffer:1;

	/** If true, Tangents will be stored at 16 bit vs 8 bit precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	uint8 bUseHighPrecisionTangentBasis:1;

	/** If true, UVs will be stored at full floating point precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bUseFullPrecisionUVs:1;
	
	/** If true, UVs will use backwards-compatible F16 conversion with truncation for legacy meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, AdvancedDisplay)
	uint8 bUseBackwardsCompatibleF16TruncUVs:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bGenerateLightmapUVs:1;

	/** 
	 * Whether to generate the distance field treating every triangle hit as a front face.  
	 * When enabled prevents the distance field from being discarded due to the mesh being open, but also lowers Distance Field AO quality.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, meta=(DisplayName="Two-Sided Distance Field Generation"))
	uint8 bGenerateDistanceFieldAsIfTwoSided:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, meta=(DisplayName="Enable Physical Material Mask"))
	uint8 bSupportFaceRemap : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	int32 MinLightmapResolution;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, meta=(DisplayName="Source Lightmap Index"))
	int32 SrcLightmapIndex;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, meta=(DisplayName="Destination Lightmap Index"))
	int32 DstLightmapIndex;

	UPROPERTY()
	float BuildScale_DEPRECATED;

	/** The local scale applied when building the mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, meta=(DisplayName="Build Scale"))
	FVector BuildScale3D;

	/** 
	 * Scale to apply to the mesh when allocating the distance field volume texture.
	 * The default scale is 1, which is assuming that the mesh will be placed unscaled in the world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	float DistanceFieldResolutionScale;

#if WITH_EDITORONLY_DATA
	UPROPERTY(NotReplicated)
	float DistanceFieldBias_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	TObjectPtr<class UStaticMesh> DistanceFieldReplacementMesh;

	/** 
	 * Max Lumen mesh cards to generate for this mesh.
	 * More cards means that surface will have better coverage, but will result in increased runtime overhead.
	 * Set to 0 in order to disable mesh card generation for this mesh.
	 * Default is 12.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	int32 MaxLumenMeshCards;

	/** Default settings. */
	FMeshBuildSettings()
		: bUseMikkTSpace(true)
		, bRecomputeNormals(true)
		, bRecomputeTangents(true)
		, bComputeWeightedNormals(false)
		, bRemoveDegenerates(true)
		, bBuildReversedIndexBuffer(true)
		, bUseHighPrecisionTangentBasis(false)
		, bUseFullPrecisionUVs(false)
		, bUseBackwardsCompatibleF16TruncUVs(false)
		, bGenerateLightmapUVs(true)
		, bGenerateDistanceFieldAsIfTwoSided(false)
		, bSupportFaceRemap(false)
		, MinLightmapResolution(64)
		, SrcLightmapIndex(0)
		, DstLightmapIndex(1)
		, BuildScale_DEPRECATED(1.0f)
		, BuildScale3D(1.0f, 1.0f, 1.0f)
		, DistanceFieldResolutionScale(1.0f)
#if WITH_EDITORONLY_DATA
		, DistanceFieldBias_DEPRECATED(0.0f)
#endif
		, DistanceFieldReplacementMesh(nullptr)
		, MaxLumenMeshCards(12)
	{ }

	/** Equality operator. */
	bool operator==(const FMeshBuildSettings& Other) const
	{
		return bRecomputeNormals == Other.bRecomputeNormals
			&& bRecomputeTangents == Other.bRecomputeTangents
			&& bComputeWeightedNormals == Other.bComputeWeightedNormals
			&& bUseMikkTSpace == Other.bUseMikkTSpace
			&& bRemoveDegenerates == Other.bRemoveDegenerates
			&& bBuildReversedIndexBuffer == Other.bBuildReversedIndexBuffer
			&& bUseHighPrecisionTangentBasis == Other.bUseHighPrecisionTangentBasis
			&& bUseFullPrecisionUVs == Other.bUseFullPrecisionUVs
			&& bUseBackwardsCompatibleF16TruncUVs == Other.bUseBackwardsCompatibleF16TruncUVs
			&& bGenerateLightmapUVs == Other.bGenerateLightmapUVs
			&& MinLightmapResolution == Other.MinLightmapResolution
			&& SrcLightmapIndex == Other.SrcLightmapIndex
			&& DstLightmapIndex == Other.DstLightmapIndex
			&& BuildScale3D == Other.BuildScale3D
			&& DistanceFieldResolutionScale == Other.DistanceFieldResolutionScale
			&& bGenerateDistanceFieldAsIfTwoSided == Other.bGenerateDistanceFieldAsIfTwoSided
			&& DistanceFieldReplacementMesh == Other.DistanceFieldReplacementMesh
			&& MaxLumenMeshCards == Other.MaxLumenMeshCards;
	}

	/** Inequality. */
	bool operator!=(const FMeshBuildSettings& Other) const
	{
		return !(*this == Other);
	}
};

/**
 * Settings applied when building a mesh.
 */
USTRUCT(BlueprintType)
struct FSkeletalMeshBuildSettings
{
	GENERATED_BODY()

	/** If true, normals in the raw mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRecomputeNormals:1;

	/** If true, tangents in the raw mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRecomputeTangents:1;
	
	/** If true, degenerate triangles will be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bUseMikkTSpace:1;
	
	/** If true, we will use the surface area and the corner angle of the triangle as a ratio when computing the normals. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	uint8 bComputeWeightedNormals : 1;

	/** If true, degenerate triangles will be removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bRemoveDegenerates:1;
	
	/** If true, Tangents will be stored at 16 bit vs 8 bit precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	uint8 bUseHighPrecisionTangentBasis:1;

	/** Use 16-bit precision for rendering skin weights, instead of 8-bit precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	uint8 bUseHighPrecisionSkinWeights:1;

	/** If true, UVs will be stored at full floating point precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings)
	uint8 bUseFullPrecisionUVs:1;

	/** If true, UVs will use backwards-compatible F16 conversion with truncation for legacy meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, AdvancedDisplay)
	uint8 bUseBackwardsCompatibleF16TruncUVs:1;

	/** If true, optimizes the bone map for lower transform storage memory costs for instancing and enables GPU instanced LOD. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BuildSettings, AdvancedDisplay)
	uint8 bOptimizeForInstancing:1;

	/** Threshold use to decide if two vertex position are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	float ThresholdPosition;

	/** Threshold use to decide if two normal, tangents or bi-normals are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	float ThresholdTangentNormal;

	/** Threshold use to decide if two UVs are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	float ThresholdUV;

	/** Threshold to compare vertex position equality when computing morph target deltas. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	float MorphThresholdPosition;

	/**
	 * The maximum number of bone influences to allow each vertex in this mesh to use.
	 * 
	 * If set higher than the limit determined by the project settings, it has no effect.
	 * 
	 * If set to 0, the value is taken from the DefaultBoneInfluenceLimit project setting.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	int32 BoneInfluenceLimit;

	/** Default settings. */
	FSkeletalMeshBuildSettings()
		: bRecomputeNormals(true)
		, bRecomputeTangents(true)
		, bUseMikkTSpace(true)
		, bComputeWeightedNormals(false)
		, bRemoveDegenerates(true)
		, bUseHighPrecisionTangentBasis(false)
		, bUseHighPrecisionSkinWeights(false)
		, bUseFullPrecisionUVs(false)
		, bUseBackwardsCompatibleF16TruncUVs(false)
		, bOptimizeForInstancing(false)
		, ThresholdPosition(0.00002f)
		, ThresholdTangentNormal(0.00002f)
		, ThresholdUV(0.0009765625f)
		, MorphThresholdPosition(0.015f)
		, BoneInfluenceLimit(0)
	{}

	/** Equality operator. */
	bool operator==(const FSkeletalMeshBuildSettings& Other) const
	{
		return bRecomputeNormals == Other.bRecomputeNormals
			&& bRecomputeTangents == Other.bRecomputeTangents
			&& bUseMikkTSpace == Other.bUseMikkTSpace
			&& bComputeWeightedNormals == Other.bComputeWeightedNormals
			&& bRemoveDegenerates == Other.bRemoveDegenerates
			&& bUseHighPrecisionTangentBasis == Other.bUseHighPrecisionTangentBasis
			&& bUseHighPrecisionSkinWeights == Other.bUseHighPrecisionSkinWeights
			&& bUseFullPrecisionUVs == Other.bUseFullPrecisionUVs
			&& bUseBackwardsCompatibleF16TruncUVs == Other.bUseBackwardsCompatibleF16TruncUVs
			&& bOptimizeForInstancing == Other.bOptimizeForInstancing
			&& ThresholdPosition == Other.ThresholdPosition
			&& ThresholdTangentNormal == Other.ThresholdTangentNormal
			&& ThresholdUV == Other.ThresholdUV
			&& MorphThresholdPosition == Other.MorphThresholdPosition
			&& BoneInfluenceLimit == Other.BoneInfluenceLimit;
	}

	/** Inequality. */
	bool operator!=(const FSkeletalMeshBuildSettings& Other) const
	{
		return !(*this == Other);
	}
};

UENUM()
enum class ENaniteGenerateFallback : uint8
{
	/** Use platform setting. */
	PlatformDefault,
	/** Generate Nanite fallback regardless of platform setting. */
	Enabled,
};

UENUM()
enum class ENaniteFallbackTarget : uint8
{
	/** Automatic heuristic based on project settings. */
	Auto,
	/** Percentage of triangles to keep from source mesh for fallback. */
	PercentTriangles,
	/** Reduce until the specified error is reached relative to size of the mesh */
	RelativeError
};

USTRUCT(BlueprintType)
struct FMeshDisplacementMap
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Displacement)
	TObjectPtr<class UTexture2D> Texture = nullptr;

	UPROPERTY(EditAnywhere, Category = Displacement)
	float Magnitude = 0.0f;

	UPROPERTY(EditAnywhere, Category = Displacement)
	float Center = 0.0f;

	FMeshDisplacementMap()
	{}

	bool operator==(const FMeshDisplacementMap& Other) const
	{
		return Texture		== Other.Texture
			&& Magnitude	== Other.Magnitude
			&& Center		== Other.Center;
	}

	bool operator!=(const FMeshDisplacementMap& Other) const
	{
		return !(*this == Other);
	}
};

/**
 * Technique for a Nanite mesh to use to ensure its shape is preserved at a distance
 */
UENUM()
enum class ENaniteShapePreservation : uint8
{
	/** Do not attempt to preserve the object's shape in the distance. */
	None,
	/** Try to maintain the same surface area at all distances (Legacy foliage technique). */
	PreserveArea,
	/** Simplify triangles to voxels in the distance to preserve the perceived volume of the object. Useful for foliage that thins out otherwise. */
	Voxelize
};

/**
 * Settings applied when building Nanite data.
 */
USTRUCT(BlueprintType)
struct FMeshNaniteSettings
{
	GENERATED_USTRUCT_BODY()

	/** If true, Nanite data will be generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bEnabled : 1;

	/** Whether to store explicit tangents instead of using the implicitly derived ones. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bExplicitTangents : 1;

	/** Whether to interpolate UVs when simplifying.
	 * Should be enabled whenever possible.
	 * For real UV coordinates this allows calculating the lowest error optimal UVs for new vertices when simplifying,
	 * assuming the UVs are used as normal texture coordinates and will interpolate across the face of the triangles.
	 *
	 * Disable if data stored in UVs isn't valid to interpolate, for example if indexes are stored in UVs.
	 * Lerping an index doesn't make sense and would break the shader trying to use it.
	 * Note: If disabled, error from UVs is no longer accounted for when Nanite selects the LOD to render because
	 * error due to arbitrary vertex attributes that aren't interpolatable can't be generally reasoned about.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bLerpUVs : 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bSeparable : 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bVoxelNDF : 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	uint8 bVoxelOpacity : 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	ENaniteShapePreservation ShapePreservation = ENaniteShapePreservation::None;

	/** Position Precision. Step size is 2^(-PositionPrecision) cm. MIN_int32 is auto. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 PositionPrecision = MIN_int32;

	/** Normal Precision in bits. -1 is auto. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 NormalPrecision = -1;

	/** Tangent Precision in bits. -1 is auto. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 TangentPrecision = -1;

	/** Blend Weight Precision in bits. -1 is auto. 0 is rigid. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 BoneWeightPrecision = -1;

	/** How much of the resource should always be resident (In KB). Approximate due to paging. 0: Minimum size (single page). MAX_uint32: Entire mesh.*/
	UPROPERTY(EditAnywhere, Category = NaniteSettings)
	uint32 TargetMinimumResidencyInKB = 0;
	
	/** Percentage of triangles to keep from source mesh. 1.0 = no reduction, 0.0 = no triangles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float KeepPercentTriangles = 1.0f;

	/** Reduce until at least this amount of error is reached relative to size of the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float TrimRelativeError = 0.0f;

	/** Whether fallback mesh should be generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings, meta = (DisplayName = "Generate Fallback Mesh"))
	ENaniteGenerateFallback GenerateFallback = ENaniteGenerateFallback::PlatformDefault;

	/** Which heuristic to use when generating the fallback mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	ENaniteFallbackTarget FallbackTarget = ENaniteFallbackTarget::Auto;
	
	/** Percentage of triangles to keep from source mesh for fallback. 1.0 = no reduction, 0.0 = no triangles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float FallbackPercentTriangles = 1.0f;

	/** Reduce until at least this amount of error is reached relative to size of the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float FallbackRelativeError = 1.0f;

	/** Controls the maximum distance allowed between each vertex of the mesh on screen. Can be used to prevent oversimplification
	 * of meshes that are intended to be deformed (e.g. animation using World Position Offset, Spline Mesh Component, etc.).
	 * Should be left at default of 0 unless explicitly needed to fix oversimplification issues. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float MaxEdgeLengthFactor = 0.0f;

	/** Number of rays traced per voxel to sample the surface */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 NumRays = 64;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 VoxelLevel = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	float RayBackUp = 0.0f;

	/** UV channel used to sample displacement maps  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NaniteSettings)
	int32 DisplacementUVChannel = 0;

	UPROPERTY(EditAnywhere, Category = NaniteSettings)
	TArray<FMeshDisplacementMap> DisplacementMaps;

	/** Nanite Assembly data set up by the import */
	UPROPERTY(VisibleAnywhere, Category = NaniteSettings)
	FNaniteAssemblyData NaniteAssemblyData;

	FMeshNaniteSettings()
	: bEnabled(false)
	, bExplicitTangents(false) // TODO: Should this be the default?
	, bLerpUVs(true)
	, bSeparable(false)
	, bVoxelNDF(true)
	, bVoxelOpacity(false)
	{}


	/** Equality operator. */
	bool operator==(const FMeshNaniteSettings& Other) const
	{
		if( DisplacementMaps.Num() != Other.DisplacementMaps.Num() )
			return false;

		for( int32 i = 0; i < DisplacementMaps.Num(); i++ )
		{
			if( DisplacementMaps[i] != Other.DisplacementMaps[i] )
				return false;
		}

		return bEnabled == Other.bEnabled
			&& bExplicitTangents == Other.bExplicitTangents
			&& bLerpUVs == Other.bLerpUVs
			&& ShapePreservation == Other.ShapePreservation
			&& PositionPrecision == Other.PositionPrecision
			&& NormalPrecision == Other.NormalPrecision
			&& TangentPrecision == Other.TangentPrecision
			&& BoneWeightPrecision == Other.BoneWeightPrecision
			&& TargetMinimumResidencyInKB == Other.TargetMinimumResidencyInKB
			&& KeepPercentTriangles == Other.KeepPercentTriangles
			&& TrimRelativeError == Other.TrimRelativeError
			&& GenerateFallback == Other.GenerateFallback
			&& FallbackTarget == Other.FallbackTarget
			&& FallbackPercentTriangles == Other.FallbackPercentTriangles
			&& FallbackRelativeError == Other.FallbackRelativeError
			&& MaxEdgeLengthFactor == Other.MaxEdgeLengthFactor
			&& DisplacementUVChannel == Other.DisplacementUVChannel
			&& NumRays == Other.NumRays
			&& VoxelLevel == Other.VoxelLevel
			&& RayBackUp == Other.RayBackUp
			&& bSeparable == Other.bSeparable
			&& bVoxelNDF == Other.bVoxelNDF
			&& bVoxelOpacity == Other.bVoxelOpacity
			&& NaniteAssemblyData == Other.NaniteAssemblyData;
	}

	/** Inequality operator. */
	bool operator!=(const FMeshNaniteSettings& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT(BlueprintType)
struct FDisplacementScaling
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Displacement, meta = (NoSpinbox = true, ClampMin = "0.0", UIMin = "0.0"))
	float Magnitude;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Displacement, meta = (NoSpinbox = true, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Center;

public:
	FDisplacementScaling()
	: Magnitude(4.0f)
	, Center(0.5f)
	{
	}

	/** Equality operator. */
	bool operator==(const FDisplacementScaling& Other) const
	{
		return
			FMath::Abs(Magnitude - Other.Magnitude) <= UE_SMALL_NUMBER &&
			FMath::Abs(Center - Other.Center) <= UE_SMALL_NUMBER;
	}

	/** Inequality operator. */
	bool operator!=(const FDisplacementScaling& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT(BlueprintType)
struct FDisplacementFadeRange
{
	GENERATED_USTRUCT_BODY()

	/**
	 * How large the max displacement should be, in on-screen pixels, when beginning to fade out displacement.
	 * NOTE: This should be a LARGER number than End Fade Size.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Displacement, meta = (DisplayName="Start Fade Size (Pixels)", NoSpinbox=true, ClampMin="0.0001", UIMin="0.0001"))
	float StartSizePixels;

	/**
	 * How large the max displacement should be, in on-screen pixels, when fading out should complete, and displacement
	 * should be disabled.
	 * NOTE: This should be a SMALLER number than Start Fade Size.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Displacement, meta = (DisplayName="End Fade Size (Pixels)", NoSpinbox=true, ClampMin="0.0001", UIMin="0.0001"))
	float EndSizePixels;

public:
	static FDisplacementFadeRange Invalid()
	{
		return FDisplacementFadeRange(0.0f, 0.0f);
	}
	
	FDisplacementFadeRange()
	: StartSizePixels(4.0f)
	, EndSizePixels(1.0f)
	{
	}
	
	FDisplacementFadeRange(float InStartSizePixels, float InEndSizePixels)
	: StartSizePixels(InStartSizePixels)
	, EndSizePixels(InEndSizePixels)
	{
	}

	/** Equality operator. */
	bool operator==(const FDisplacementFadeRange& Other) const
	{
		return FMath::IsNearlyEqual(StartSizePixels, Other.StartSizePixels) &&
			FMath::IsNearlyEqual(EndSizePixels, Other.EndSizePixels);
	}

	/** Inequality operator. */
	bool operator!=(const FDisplacementFadeRange& Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return StartSizePixels > 0.0f && EndSizePixels > 0.0f;
	}
};

USTRUCT(BlueprintType)
struct FMeshRayTracingProxySettings
{
	GENERATED_USTRUCT_BODY()

	/** If true, Ray Tracing Proxy data will be generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RayTracing)
	uint8 bEnabled : 1;

	/** Which heuristic to use when generating the fallback mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RayTracing)
	ENaniteFallbackTarget FallbackTarget = ENaniteFallbackTarget::Auto;
	
	/** Percentage of triangles to keep from source mesh for fallback. 1.0 = no reduction, 0.0 = no triangles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RayTracing)
	float FallbackPercentTriangles = 1.0f;

	/** Reduce until at least this amount of error is reached relative to size of the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RayTracing)
	float FallbackRelativeError = 2.0f;
	
	/** A bias to reduce foliage over occlusion in Lumen GI. 0: no adjustment, 1: full strength. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RayTracing, meta = (NoSpinbox = true, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float FoliageOverOcclusionBias = 0.0f;

public:
	FMeshRayTracingProxySettings()
	: bEnabled(false)
	{}

	/** Equality operator. */
	bool operator==(const FMeshRayTracingProxySettings& Other) const
	{
		return bEnabled == Other.bEnabled &&
			Other.FallbackTarget == FallbackTarget &&
			Other.FallbackPercentTriangles == FallbackPercentTriangles &&
			Other.FallbackRelativeError == FallbackRelativeError &&
			Other.FoliageOverOcclusionBias == FoliageOverOcclusionBias;
	}

	/** Inequality operator. */
	bool operator!=(const FMeshRayTracingProxySettings& Other) const
	{
		return !(*this == Other);
	}
};

/** The network role of an actor on a local/remote network context */
UENUM(BlueprintType)
enum ENetRole : int
{
	/** No role at all. */
	ROLE_None UMETA(DisplayName = "None"),
	/** Locally simulated proxy of this actor. */
	ROLE_SimulatedProxy UMETA(DisplayName = "Simulated Proxy"),
	/** Locally autonomous proxy of this actor. */
	ROLE_AutonomousProxy UMETA(DisplayName = "Autonomous Proxy"),
	/** Authoritative control over the actor. */
	ROLE_Authority UMETA(DisplayName = "Authority"),
	ROLE_MAX UMETA(Hidden),
};

/** Describes if an actor can enter a low network bandwidth dormant mode */
UENUM(BlueprintType)
enum ENetDormancy : int
{
	/** This actor can never go network dormant. */
	DORM_Never UMETA(DisplayName = "Never"),
	/** This actor can go dormant, but is not currently dormant. Game code will tell it when it go dormant. */
	DORM_Awake UMETA(DisplayName = "Awake"),
	/** This actor wants to go fully dormant for all connections. */
	DORM_DormantAll UMETA(DisplayName = "Dormant All"),
	/** This actor may want to go dormant for some connections, GetNetDormancy() will be called to find out which. */
	DORM_DormantPartial UMETA(DisplayName = "Dormant Partial"),
	/** This actor is initially dormant for all connection if it was placed in map. */
	DORM_Initial UMETA(DisplayName = "Initial"),

	DORM_MAX UMETA(Hidden),
};

UENUM(BlueprintType)
enum class EPhysicsReplicationMode : uint8
{
	/** Default physics replication.*/
	Default = 0 UMETA(DisplayName = "Default"),

	/** Physics replication performing velocity interpolation.
	* Note: Not forward predicted which makes this not viable for ENetRole::ROLE_AutonomousProxy if input latency is of concern.
	* Recommendation: All physics objects that interact should use the same replication mode. Use alongside a kinematic pawn (not physics simulated).
	* Designed to handle local predictive interactions with other actors, especially actors of the local role ENetRole::ROLE_AutonomousProxy.
	* Enable Project Settings > Physics Prediction for best result */
	PredictiveInterpolation = 1 UMETA(DisplayName = "Predictive Interpolation"),

	/** Work In Progress. Forward predicted replication by simulating physics and correcting errors through resimulating physics from a correct state in the past.
	* Recommendation: All physics objects that interact should use the same replication mode. Use on physics simulated pawns along with any physics objects that will interact with such pawns.
	* Available when Project Settings > Physics Prediction is enabled. */
	Resimulation = 2 UMETA(DisplayName = "Resimulation (WIP)"),

	/** Don't apply physics replication
	* Note: Can be used to temporarily disable physics replication for specific actors. */
	None = 255 UMETA(DisplayName = "None"),
};

/** Specifies which player index will pass input to this actor/component */
UENUM()
namespace EAutoReceiveInput
{
	enum Type : int
	{
		Disabled,
		Player0,
		Player1,
		Player2,
		Player3,
		Player4,
		Player5,
		Player6,
		Player7,
	};
}

/** Specifies if an AI pawn will automatically be possessed by an AI controller */
UENUM()
enum class EAutoPossessAI : uint8
{
	/** Feature is disabled (do not automatically possess AI). */
	Disabled,
	/** Only possess by an AI Controller if Pawn is placed in the world. */
	PlacedInWorld,
	/** Only possess by an AI Controller if Pawn is spawned after the world has loaded. */
	Spawned,
	/** Pawn is automatically possessed by an AI Controller whenever it is created. */
	PlacedInWorldOrSpawned,
};

/** Specifies why an actor is being deleted/removed from a level */
UENUM(BlueprintType)
namespace EEndPlayReason
{
	enum Type : int
	{
		/** When the Actor or Component is explicitly destroyed. */
		Destroyed,
		/** When the world is being unloaded for a level transition. */
		LevelTransition,
		/** When the world is being unloaded because PIE is ending. */
		EndPlayInEditor,
		/** When the level it is a member of is streamed out. */
		RemovedFromWorld,
		/** When the application is being exited. */
		Quit,
	};
}


/**
 * Controls behavior of WalkableSlopeOverride, determining how to affect walkability of surfaces for Characters.
 * @see FWalkableSlopeOverride
 * @see UCharacterMovementComponent::GetWalkableFloorAngle(), UCharacterMovementComponent::SetWalkableFloorAngle()
 */
UENUM(BlueprintType)
enum EWalkableSlopeBehavior : int
{
	/** Don't affect the walkable slope. Walkable slope angle will be ignored. */
	WalkableSlope_Default		UMETA(DisplayName="Unchanged"),

	/**
	 * Increase walkable slope.
	 * Makes it easier to walk up a surface, by allowing traversal over higher-than-usual angles.
	 * @see FWalkableSlopeOverride::WalkableSlopeAngle
	 */
	WalkableSlope_Increase		UMETA(DisplayName="Increase Walkable Slope"),

	/**
	 * Decrease walkable slope.
	 * Makes it harder to walk up a surface, by restricting traversal to lower-than-usual angles.
	 * @see FWalkableSlopeOverride::WalkableSlopeAngle
	 */
	WalkableSlope_Decrease		UMETA(DisplayName="Decrease Walkable Slope"),

	/**
	 * Make surface unwalkable.
	 * Note: WalkableSlopeAngle will be ignored.
	 */
	WalkableSlope_Unwalkable	UMETA(DisplayName="Unwalkable"),
	
	WalkableSlope_Max		UMETA(Hidden),
};

/** Struct allowing control over "walkable" normals, by allowing a restriction or relaxation of what steepness is normally walkable. */
USTRUCT(BlueprintType)
struct FWalkableSlopeOverride
{
	GENERATED_BODY()

	/**
	 * Behavior of this surface (whether we affect the walkable slope).
	 * @see GetWalkableSlopeBehavior(), SetWalkableSlopeBehavior()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=WalkableSlopeOverride)
	TEnumAsByte<EWalkableSlopeBehavior> WalkableSlopeBehavior;

	/**
	 * Override walkable slope angle (in degrees), applying the rules of the Walkable Slope Behavior.
	 * @see GetWalkableSlopeAngle(), SetWalkableSlopeAngle()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=WalkableSlopeOverride, meta=(ClampMin="0", ClampMax="90", UIMin="0", UIMax="90"))
	float WalkableSlopeAngle;

private:

	// Cached angle for which we computed a cosine.
	mutable float CachedSlopeAngle;
	// Cached cosine of angle.
	mutable float CachedSlopeCos;

public:

	FWalkableSlopeOverride()
		: WalkableSlopeBehavior(WalkableSlope_Default)
		, WalkableSlopeAngle(0.f)
		, CachedSlopeAngle(0.f)
		, CachedSlopeCos(1.f)
	{ }

	FWalkableSlopeOverride(EWalkableSlopeBehavior NewSlopeBehavior, float NewSlopeAngle)
		: WalkableSlopeBehavior(NewSlopeBehavior)
		, WalkableSlopeAngle(NewSlopeAngle)
		, CachedSlopeAngle(0.f)
		, CachedSlopeCos(1.f)
	{
	}

	/** Gets the slope override behavior. */
	inline EWalkableSlopeBehavior GetWalkableSlopeBehavior() const
	{
		return WalkableSlopeBehavior;
	}

	/** Gets the slope angle used for the override behavior. */
	inline float GetWalkableSlopeAngle() const
	{
		return WalkableSlopeAngle;
	}

	/** Set the slope override behavior. */
	inline void SetWalkableSlopeBehavior(EWalkableSlopeBehavior NewSlopeBehavior)
	{
		WalkableSlopeBehavior = NewSlopeBehavior;
	}

	/** Set the slope angle used for the override behavior. */
	inline void SetWalkableSlopeAngle(float NewSlopeAngle)
	{
		WalkableSlopeAngle = FMath::Clamp(NewSlopeAngle, 0.f, 90.f);
	}

	/** Given a walkable floor normal Z value, either relax or restrict the value if we override such behavior. */
	float ModifyWalkableFloorZ(float InWalkableFloorZ) const
	{
		switch(WalkableSlopeBehavior)
		{
			case WalkableSlope_Default:
			{
				return InWalkableFloorZ;
			}

			case WalkableSlope_Increase:
			{
				CheckCachedData();
				return FMath::Min(InWalkableFloorZ, CachedSlopeCos);
			}

			case WalkableSlope_Decrease:
			{
				CheckCachedData();
				return FMath::Max(InWalkableFloorZ, CachedSlopeCos);
			}

			case WalkableSlope_Unwalkable:
			{
				// Z component of a normal will always be less than this, so this will be unwalkable.
				return 2.0f;
			}

			case WalkableSlope_Max:
			default:
			{
				return InWalkableFloorZ;
			}
		}
	}

private:

	void CheckCachedData() const
	{
		if (CachedSlopeAngle != WalkableSlopeAngle)
		{
			const float AngleRads = FMath::DegreesToRadians(WalkableSlopeAngle);
			CachedSlopeCos = FMath::Clamp(FMath::Cos(AngleRads), 0.f, 1.f);
			CachedSlopeAngle = WalkableSlopeAngle;
		}
	}
};

template<> struct TIsPODType<FWalkableSlopeOverride> { enum { Value = true }; };

/** Structure to hold and pass around transient flags used during replication. */
struct FReplicationFlags
{
	union
	{
		struct
		{
			/** True if replicating actor is owned by the player controller on the target machine. */
			uint32 bNetOwner:1;
			/** True if this is the initial network update for the replicating actor. */
			uint32 bNetInitial:1;
			/** True if this is actor is RemoteRole simulated. */
			uint32 bNetSimulated:1;
			/** True if this is actor's ReplicatedMovement.bRepPhysics flag is true. */
			uint32 bRepPhysics:1;
			/** True if this actor is replicating on a replay connection. */
			uint32 bReplay:1;
			/** True if this actor's RPCs should be ignored. */
			uint32 bIgnoreRPCs:1;
			/** True if we should not swap the role and remote role of this actor when receiving properties. */
			uint32 bSkipRoleSwap:1;
			/** True if we should only compare role properties in CompareProperties */
			uint32 bRolesOnly:1;
			/** True if we should force all properties dirty on initial replication. */
			uint32 bForceInitialDirty:1;
			/** True if we should serialize property names instead of handles. */
			uint32 bSerializePropertyNames : 1;
			/** True if a subclass of UActorChannel needs custom subobject replication */
			uint32 bUseCustomSubobjectReplication : 1;
			/** True if this actor is replicating on a replay connection on a game client. */
			uint32 bClientReplay : 1;
			/** Padding bits to align to a 16-bit boundary. NOTE: Need to be adjusted if adding/removing bitfields. */
			uint32 Padding0 : 4;

			/** COND_Dynamic change counter in order to force rebuilding conditionals when appropriate. */
			uint32 CondDynamicChangeCounter : 16;
		};

		uint32 Value;
	};
	FReplicationFlags()
	{
		Value = 0;
	}
};

static_assert(sizeof(FReplicationFlags) == 4, "FReplicationFlags has invalid size.");

/** Struct used to specify the property name of the component to constrain */
USTRUCT()
struct FConstrainComponentPropName
{
	GENERATED_BODY()

	/** Name of property */
	UPROPERTY(EditAnywhere, Category=Constraint)
	FName	ComponentName;
};

/** 
 *	Base class for the hard/soft component reference structs 
 */
USTRUCT(BlueprintType)
struct FBaseComponentReference
{
	GENERATED_BODY()

	FBaseComponentReference()  {}

	/** Name of component to use. If this is not specified the reference refers to the root component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Component, meta = (DisplayName = "Component Name"))
	FName ComponentProperty;

	/** Path to the component from its owner actor */
	UPROPERTY(EditDefaultsOnly, Category=Component, meta =(EditCondition="false", EditConditionHides))
	FString PathToComponent;

	/** Allows direct setting of first component to constraint. */
	TWeakObjectPtr<class UActorComponent> OverrideComponent;

	/** Extract the actual component pointer from this reference given a search actor */
	ENGINE_API class UActorComponent* ExtractComponent(AActor* SearchActor) const;

	/** FBaseComponentReference == operator */
	bool operator== (const FBaseComponentReference& Other) const
	{
		return ComponentProperty == Other.ComponentProperty && PathToComponent == Other.PathToComponent && OverrideComponent == Other.OverrideComponent;
	}
};

inline uint32 GetTypeHash(const FBaseComponentReference& Reference)
{
	return HashCombineFast(
		HashCombineFast(
			GetTypeHash(Reference.ComponentProperty),
			GetTypeHash(Reference.PathToComponent)),
			GetTypeHash(Reference.OverrideComponent));
}

/** 
 *	Struct that allows for different ways to reference a component using TObjectPtr. 
 *	If just an Actor is specified, will return RootComponent of that Actor.
 */
USTRUCT(BlueprintType)
struct FComponentReference : public FBaseComponentReference
{
	GENERATED_BODY()

	FComponentReference() : OtherActor(nullptr) {}

	/** 
	 * Weak Pointer to a different Actor that owns the Component.  
	 * If this is not provided the reference refers to a component on this / the same actor.
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=Component, meta = (DisplayName = "Referenced Actor"))
	TWeakObjectPtr<AActor> OtherActor;

	/** Get the actual component pointer from this reference */
	ENGINE_API class UActorComponent* GetComponent(AActor* OwningActor) const;

	/** FComponentReference == operator */
	bool operator== (const FComponentReference& Other) const
	{
		return (OtherActor == Other.OtherActor) && (FBaseComponentReference::operator==(Other));
	}
};

/** 
 *	Struct that allows for different ways to reference a component using TSoftObjectPtr. 
 *	If just an Actor is specified, will return RootComponent of that Actor.
 */
USTRUCT(BlueprintType)
struct FSoftComponentReference : public FBaseComponentReference
{
	GENERATED_BODY()

	FSoftComponentReference() : OtherActor(nullptr) {}

	/** 
	 * Soft Pointer to a different Actor that owns the Component.  
	 * If this is not provided the reference refers to a component on this / the same actor.
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=Component, meta = (DisplayName = "Referenced Actor"))
	TSoftObjectPtr<AActor> OtherActor;

	/** Get the actual component pointer from this reference */
	ENGINE_API class UActorComponent* GetComponent(AActor* OwningActor) const;

	/** FSoftComponentReference == operator */
	bool operator== (const FSoftComponentReference& Other) const
	{
		return (OtherActor == Other.OtherActor) && (FBaseComponentReference::operator==(Other));
	}

	ENGINE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FSoftComponentReference> : public TStructOpsTypeTraitsBase2<FSoftComponentReference>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


/** Types of valid physical material mask colors which may be associated with a physical material */
UENUM(BlueprintType)
namespace EPhysicalMaterialMaskColor
{
	enum Type : int
	{
		Red,
		Green,
		Blue,
		Cyan,
		Magenta,
		Yellow,
		White,
		Black,
		MAX
	};
}

/** Describes how often this component is allowed to move. */
UENUM(BlueprintType)
namespace EComponentMobility
{
	enum Type : int
	{
		/**
		 * Static objects cannot be moved or changed in game.
		 * - Allows baked lighting
		 * - Fastest rendering
		 */
		Static,

		/**
		 * A stationary light will only have its shadowing and bounced lighting from static geometry baked by Lightmass, all other lighting will be dynamic.
		 * - It can change color and intensity in game.
		 * - Can't move
		 * - Allows partial baked lighting
		 * - Dynamic shadows
		 */
		Stationary,

		/**
		 * Movable objects can be moved and changed in game.
		 * - Totally dynamic
		 * - Can cast dynamic shadows
		 * - Slowest rendering
		 */
		Movable
	};
}

/** Utility class for engine types */
UCLASS(abstract, config=Engine, MinimalAPI)
class UEngineTypes : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Convert a trace type to a collision channel. */
	static ENGINE_API ECollisionChannel ConvertToCollisionChannel(ETraceTypeQuery TraceType);

	/** Convert an object type to a collision channel. */
	static ENGINE_API ECollisionChannel ConvertToCollisionChannel(EObjectTypeQuery ObjectType);

	/** Convert a collision channel to an object type. Note: performs a search of object types. */
	static ENGINE_API EObjectTypeQuery ConvertToObjectType(ECollisionChannel CollisionChannel);

	/** Convert a collision channel to a trace type. Note: performs a search of trace types. */
	static ENGINE_API ETraceTypeQuery ConvertToTraceType(ECollisionChannel CollisionChannel);
};

/** Type of a socket on a scene component. */
UENUM()
namespace EComponentSocketType
{
	enum Type : int
	{
		/** Not a valid socket or bone name. */
		Invalid,

		/** Skeletal bone. */
		Bone,

		/** Socket. */
		Socket,
	};
}

/** Info about a socket on a scene component */
struct FComponentSocketDescription
{
	/** Name of the socket */
	FName Name;

	/** Type of the socket */
	TEnumAsByte<EComponentSocketType::Type> Type;

	FComponentSocketDescription()
		: Name(NAME_None)
		, Type(EComponentSocketType::Invalid)
	{
	}

	FComponentSocketDescription(FName SocketName, EComponentSocketType::Type SocketType)
		: Name(SocketName)
		, Type(SocketType)
	{
	}
};

/** Dynamic delegate to use by components that want to route the broken-event into blueprints */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FConstraintBrokenSignature, int32, ConstraintIndex);

/** Dynamic delegate to use by components that want to route the pasticity deformation event into blueprints */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlasticDeformationEventSignature, int32, ConstraintIndex);


/**
 * Reference to an editor collection of assets. This allows an editor-only picker UI
 */
USTRUCT(BlueprintType)
struct FCollectionReference
{
	GENERATED_BODY()

	/**
	 * Name of the collection
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CollectionReference)
	FName CollectionName;
};

/**
 * This is used for redirecting an old name to new name, such as for collision profiles
 * This is used for better UI in the editor
 */
USTRUCT()
struct FRedirector
{
	GENERATED_BODY()

	UPROPERTY()
	FName OldName;

	/** Types of objects that this physics objects will collide with. */
	UPROPERTY()
	FName NewName;

	FRedirector()
		: OldName(NAME_None)
		, NewName(NAME_None)
	{ }

	FRedirector(FName InOldName, FName InNewName)
		: OldName(InOldName)
		, NewName(InNewName)
	{ }
};

/** 
 * Structure for recording float values and displaying them as an Histogram through DrawDebugFloatHistory.
 */
USTRUCT(BlueprintType)
struct FDebugFloatHistory
{
	GENERATED_BODY()

private:
	/** Samples */
	UPROPERTY(Transient)
	TArray<float> Samples;

public:
	/** Max Samples to record. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="DebugFloatHistory")
	int32 MaxSamples;

	/** Min value to record. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DebugFloatHistory")
	float MinValue;

	/** Max value to record. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DebugFloatHistory")
	float MaxValue;

	/** Auto adjust Min/Max as new values are recorded? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DebugFloatHistory")
	bool bAutoAdjustMinMax;

	FDebugFloatHistory()
		: MaxSamples(100)
		, MinValue(0.f)
		, MaxValue(0.f)
		, bAutoAdjustMinMax(true)
	{ }

	FDebugFloatHistory(int32 const & InMaxSamples, float const & InMinValue, float const & InMaxValue, bool const & InbAutoAdjustMinMax)
		: MaxSamples(InMaxSamples)
		, MinValue(InMinValue)
		, MaxValue(InMaxValue)
		, bAutoAdjustMinMax(InbAutoAdjustMinMax)
	{ }

	/**
	 * Record a new Sample.
	 * if bAutoAdjustMinMax is true, this new value will potentially adjust those bounds.
	 * Otherwise value will be clamped before being recorded.
	 * If MaxSamples is exceeded, old values will be deleted.
	 * @param FloatValue new sample to record.
	 */
	void AddSample(float const & FloatValue)
	{
		if (bAutoAdjustMinMax)
		{
			// Adjust bounds and record value.
			MinValue = FMath::Min(MinValue, FloatValue);
			MaxValue = FMath::Max(MaxValue, FloatValue);
			Samples.Insert(FloatValue, 0);
		}
		else
		{
			// Record clamped value.
			Samples.Insert(FMath::Clamp(FloatValue, MinValue, MaxValue), 0);
		}

		// Do not exceed MaxSamples recorded.
		if( Samples.Num() > MaxSamples )
		{
			Samples.RemoveAt(MaxSamples, Samples.Num() - MaxSamples);
		}
	}

	/** Range between Min and Max values */
	float GetMinMaxRange() const
	{
		return (MaxValue - MinValue);
	}

	/** Min value. This could either be the min value recorded or min value allowed depending on 'bAutoAdjustMinMax'. */
	float GetMinValue() const
	{
		return MinValue;
	}

	/** Max value. This could be either the max value recorded or max value allowed depending on 'bAutoAdjustMinMax'. */
	float GetMaxValue() const
	{
		return MaxValue;
	}

	/** Number of Samples currently recorded */
	int GetNumSamples() const
	{
		return Samples.Num();
	}

	/** Read access to Samples array */
	TArray<float> const & GetSamples() const
	{
		return Samples;
	}
};

/** Info for glow when using depth field rendering */
USTRUCT(BlueprintType)
struct FDepthFieldGlowInfo
{
	GENERATED_BODY()

	/** Whether to turn on the outline glow (depth field fonts only) */
	UPROPERTY(BlueprintReadWrite, Category="Glow")
	uint32 bEnableGlow:1;

	/** Base color to use for the glow */
	UPROPERTY(BlueprintReadWrite, Category="Glow")
	FLinearColor GlowColor;

	/** 
	 * If bEnableGlow, outline glow outer radius (0 to 1, 0.5 is edge of character silhouette)
	 * glow influence will be 0 at GlowOuterRadius.X and 1 at GlowOuterRadius.Y
	 */
	UPROPERTY(BlueprintReadWrite, Category="Glow")
	FVector2D GlowOuterRadius;

	/** 
	 * If bEnableGlow, outline glow inner radius (0 to 1, 0.5 is edge of character silhouette)
	 * glow influence will be 1 at GlowInnerRadius.X and 0 at GlowInnerRadius.Y
	 */
	UPROPERTY(BlueprintReadWrite, Category="Glow")
	FVector2D GlowInnerRadius;


	FDepthFieldGlowInfo()
		: bEnableGlow(false)
		, GlowColor(ForceInit)
		, GlowOuterRadius(ForceInit)
		, GlowInnerRadius(ForceInit)
	{ }

	bool operator==(const FDepthFieldGlowInfo& Other) const
	{
		if (Other.bEnableGlow != bEnableGlow)
		{
			return false;
		}
		else if (!bEnableGlow)
		{
			// if the glow is disabled on both, the other values don't matter
			return true;
		}
		else
		{
			return (Other.GlowColor == GlowColor && Other.GlowOuterRadius == GlowOuterRadius && Other.GlowInnerRadius == GlowInnerRadius);
		}
	}

	bool operator!=(const FDepthFieldGlowInfo& Other) const
	{
		return !(*this == Other);
	}	
};

/** Information used in font rendering */
USTRUCT(BlueprintType)
struct FFontRenderInfo
{
	GENERATED_BODY()

	/** Whether to clip text */
	UPROPERTY(BlueprintReadWrite, Category="FontInfo")
	uint32 bClipText:1;

	/** Whether to turn on shadowing */
	UPROPERTY(BlueprintReadWrite, Category="FontInfo")
	uint32 bEnableShadow:1;

	/** Depth field glow parameters (only usable if font was imported with a depth field) */
	UPROPERTY(BlueprintReadWrite, Category="FontInfo")
	struct FDepthFieldGlowInfo GlowInfo;

	FFontRenderInfo()
		: bClipText(false), bEnableShadow(false)
	{}
};

/** Simple 2d triangle with UVs */
USTRUCT(BlueprintType)
struct FCanvasUVTri
{
	GENERATED_BODY()

	/** Position of first vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V0_Pos;

	/** UV of first vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V0_UV;

	/** Color of first vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FLinearColor V0_Color;

	/** Position of second vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V1_Pos;

	/** UV of second vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V1_UV;

	/** Color of second vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FLinearColor V1_Color;

	/** Position of third vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V2_Pos;

	/** UV of third vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FVector2D V2_UV;

	/** Color of third vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CanvasUVTri)
	FLinearColor V2_Color;

	FCanvasUVTri()
		: V0_Pos(ForceInit)
		, V0_UV(ForceInit)
		, V0_Color(ForceInit)
		, V1_Pos(ForceInit)
		, V1_UV(ForceInit)
		, V1_Color(ForceInit)
		, V2_Pos(ForceInit)
		, V2_UV(ForceInit)
		, V2_Color(ForceInit)
	{ }
};

template <> struct TIsZeroConstructType<FCanvasUVTri> { enum { Value = true }; };

/** Defines available strategies for handling the case where an actor is spawned in such a way that it penetrates blocking collision. */
UENUM(BlueprintType)
enum class ESpawnActorCollisionHandlingMethod : uint8
{
	/** Fall back to default settings. */
	Undefined								UMETA(DisplayName = "Default"),
	/** Actor will spawn in desired location, regardless of collisions. */
	AlwaysSpawn								UMETA(DisplayName = "Always Spawn, Ignore Collisions"),
	/** Actor will try to find a nearby non-colliding location (based on shape components), but will always spawn even if one cannot be found. */
	AdjustIfPossibleButAlwaysSpawn			UMETA(DisplayName = "Try To Adjust Location, But Always Spawn"),
	/** Actor will try to find a nearby non-colliding location (based on shape components), but will NOT spawn unless one is found. */
	AdjustIfPossibleButDontSpawnIfColliding	UMETA(DisplayName = "Try To Adjust Location, Don't Spawn If Still Colliding"),
	/** Actor will fail to spawn. */
	DontSpawnIfColliding					UMETA(DisplayName = "Do Not Spawn"),
};

/** Defines the context of a user activity. Activities triggered in Blueprints will by type Game. Those created in code might choose to set another type. */
enum class EUserActivityContext : uint8
{
	/** Event triggered from gameplay, such as from blueprints */
	Game,
	/** Event triggered from the editor UI */
	Editor,
	/** Event triggered from some other source */
	Other
};

/**
 * The description of a user activity
 */
USTRUCT(BlueprintType)
struct FUserActivity
{
	GENERATED_BODY()

	/** Describes the user's activity */
	UPROPERTY(BlueprintReadWrite, Category = "Activity")
	FString ActionName;

	/** A game or editor activity? */
	EUserActivityContext Context;

	/** Default constructor. */
	FUserActivity()
		: Context(EUserActivityContext::Game)
	{ }

	/** Creates and initializes a new instance. */
	FUserActivity(const FString& InActionName)
		: ActionName(InActionName)
		, Context(EUserActivityContext::Game)
	{ }

	/** Creates and initializes a new instance with a context other than the default "Game". */
	FUserActivity(const FString& InActionName, EUserActivityContext InContext)
		: ActionName(InActionName)
		, Context(InContext)
	{ }
};

/** Which processors will have access to Mesh Vertex Buffers. */
UENUM()
enum class EMeshBufferAccess: uint8
{
    /** Access will be determined based on the assets used in the mesh and hardware / software capability. */
    Default,

    /** Force access on both CPU and GPU. */
    ForceCPUAndGPU
};

/** Indicates the type of a level collection, used in FLevelCollection. */
enum class ELevelCollectionType : uint8
{
	/**
	 * The dynamic levels that are used for normal gameplay and the source for any duplicated collections.
	 * Will contain a world's persistent level and any streaming levels that contain dynamic or replicated gameplay actors.
	 * This collection will always exist for gameplay and editor worlds.
	 */
	DynamicSourceLevels,

	/** 
	 * Gameplay relevant levels that have been duplicated from DynamicSourceLevels if requested by the game.
	 * This collection only exists if levels have actually been duplicated.
	 */
	DynamicDuplicatedLevels,

	/**
	 * These levels are shared between the source levels and the duplicated levels, and should contain
	 * only static geometry and other visuals that are not replicated or affected by gameplay.
	 * These will not be duplicated in order to save memory.
	 * If s.World.CreateStaticLevelCollection is 0, this will not be created and static levels will be treated as dynamic.
	 */
	StaticLevels,

	MAX
};

enum class EPackageAutoSaveType : uint8
{
	/** No auto-save is happening */
	None = 0,
	/** The auto-save is happening to a transient directory (eg, the Saved/Autosaves directory) and doesn't directly replace the persistent project content */
	Transient = 1 << 0,
	/** The auto-save is happening directly to the persistent project content, though a backup may still be made in a transient directory */
	Persistent = 1 << 1,
	/** The auto-save is happening for any reason */
	Any = Transient | Persistent,
};
ENUM_CLASS_FLAGS(EPackageAutoSaveType);

#if WITH_EDITOR

ENGINE_API void SerializeNaniteSettingsForDDC(FArchive& Ar, FMeshNaniteSettings& NaniteSettings, bool bIsNaniteForceEnabled);
ENGINE_API void SerializeRayTracingProxySettingsForDDC(FArchive& Ar, FMeshRayTracingProxySettings& Settings);

#endif
