// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Experimental/UnifiedError/UnifiedError.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/SoftObjectPath.h"

#define UE_API ENGINE_API

UE_DECLARE_ERROR_MODULE(UE_API, StreamableManager);

UE_DECLARE_ERROR(UE_API, StreamableManager, 1, PackageLoadFailed, "Failed to load package {PackageName}", (FString, PackageName, TEXT("Unknown")));
UE_DECLARE_ERROR(UE_API, StreamableManager, 2, PackageLoadCanceled, "Async load canceled {PackageName}", (FString, PackageName, TEXT("Unknown")));
UE_DECLARE_ERROR(UE_API, StreamableManager, 3, DownloadError, "Failed to download");
UE_DECLARE_ERROR(UE_API, StreamableManager, 4, PackageNameInvalid, "Found invalid package name {InvalidPackageName}", (FString, InvalidPackageName, TEXT("Unknown")));

UE_DECLARE_ERROR(UE_API, StreamableManager, 6, IoStoreNotFound, "IoStore did not load correctly.");
UE_DECLARE_ERROR(UE_API, StreamableManager, 7, SyncLoadIncomplete, "Sync load did not complete correctly for {DebugName}.", (FString, DebugName, TEXT("Unknown")));

UE_DECLARE_ERROR(UE_API, StreamableManager, 8, AsyncLoadFailed, "Async load failed");
UE_DECLARE_ERROR(UE_API, StreamableManager, 9, AsyncLoadCancelled, "Async load cancelled");
UE_DECLARE_ERROR(UE_API, StreamableManager, 10, AsyncLoadUnknownError, "Unknown async loading error {AsyncLoadingErrorId}.", (int32, AsyncLoadingErrorId, -1));
UE_DECLARE_ERROR(UE_API, StreamableManager, 11, UnknownError, "Unknown error occured while streaming asset");
UE_DECLARE_ERROR(UE_API, StreamableManager, 12, AsyncLoadNotInstalled, "Async load failed because the package is not installed.");

UE_DECLARE_ERROR_CONTEXT(UE_API, StreamableManager, AdditionalContext, "(RequestedPackage: {RequestedPackageName}, MissingObject: {MissingObject})", (FString, RequestedPackageName), (FString, MissingObject));

UE_DECLARE_ERROR_CONTEXT(UE_API, StreamableManager, RequestContext, "(RequestDebugName: {DebugName}, AttemptedDownload:{AttemptedDownload}, RequestedAssets: {RequestedAssets})", (FString, DebugName), (TArray<FSoftObjectPath>, RequestedAssets), (bool, AttemptedDownload, false));

namespace StreamableManager
{
	UE::UnifiedError::FError GetStreamableError(EAsyncLoadingResult::Type Result);
	
}

#undef UE_API