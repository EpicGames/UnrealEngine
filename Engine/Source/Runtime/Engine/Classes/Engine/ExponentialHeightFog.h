// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "ExponentialHeightFog.generated.h"

/**
 * Implements an Actor for exponential height fog.
 */
UCLASS(showcategories=(Movement, Rendering, Transformation, DataLayers), ClassGroup=Fog, MinimalAPI)
class AExponentialHeightFog
	: public AInfo
{
	GENERATED_UCLASS_BODY()

	virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	/** @todo document */
	UPROPERTY(Category = ExponentialHeightFog, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UExponentialHeightFogComponent> Component;

public:

	/** replicated copy of ExponentialHeightFogComponent's bEnabled property */
	UPROPERTY(replicatedUsing=OnRep_bEnabled)
	uint32 bEnabled:1;

	/** Replication Notification Callbacks */
	UFUNCTION()
	virtual void OnRep_bEnabled();


	//Begin AActor Interface
	virtual void PostInitializeComponents() override;
	//End AActor Interface

	/** Returns Component subobject **/
	class UExponentialHeightFogComponent* GetComponent() const { return Component; }
};
