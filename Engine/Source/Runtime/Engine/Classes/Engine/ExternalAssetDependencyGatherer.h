// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/AssetDependencyGatherer.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"

class FExternalAssetDependencyGatherer : public IAssetDependencyGatherer
{
public:	
	FExternalAssetDependencyGatherer() = default;
	virtual ~FExternalAssetDependencyGatherer() = default;

	ENGINE_API virtual void GatherDependencies(FGatherDependenciesContext& Context) const override;
};

#endif
