// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/AssetDependencyGatherer.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Engine/World.h"

struct FARFilter;
struct FARCompiledFilter;

//
// This interface is used by systems who inject ContentBundle/ExternalDataLayers in worlds to inform
// FExternalObjectAndActorDependencyGatherer of where to locate those external actors and external objects
// and which CB/EDLs are associated with which world. 
//
// This is necessary since worlds do not store information linking back to which EDL/CB are injected 
// in them.
//
class IExternalAssetPathsProvider
{
	public:

		// These functions will be invoked while the AssetRegistry lock is held. All the restrictions 
		// and warnings pertaining to IAssetDependencyGathers apply here. 
		// 
		// Replicated here for clarity....
		// 
		// WARNING: For high performance these callbacks are called inside the critical section of the AssetRegistry. 
		// Attempting to call public functions on the AssetRegistry will deadlock. 
		// To send queries about what assets exist, used the passed-in interface functions instead.
		// 
		// Aside from that these should be made as simple as possible to execute quickly and without side-effects outside
		// of the cache (if using a cache).
		//

		struct FUpdateCacheContext
		{
			const FAssetRegistryState& AssetRegistryState;
			const class FPathTree& CachedPathTree;
			const TFunctionRef<FARCompiledFilter(const FARFilter&)> CompileFilterFunc;
		};

		virtual void UpdateCache(const FUpdateCacheContext& Context) = 0;
		virtual TArray<FName> GetPathsForPackage(FName PackagePath) = 0;
};

class FExternalObjectAndActorDependencyGatherer : public IAssetDependencyGatherer
{
	static IExternalAssetPathsProvider* ExternalPathsProvider;

public:	
	FExternalObjectAndActorDependencyGatherer() = default;
	virtual ~FExternalObjectAndActorDependencyGatherer() = default;
	
	static ENGINE_API void SetExternalAssetPathsProvider(IExternalAssetPathsProvider* InProvider);

	static ENGINE_API FARFilter GetQueryFilter(FName PackageName, TArray<FString>* OutQueryDirectories = nullptr);

	ENGINE_API virtual void GatherDependencies(FGatherDependenciesContext& Context) const override;
};

#endif
