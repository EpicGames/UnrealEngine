// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/SlateFontInfo.h"
#include "Engine/FontImportOptions.h"
#include "Fonts/FontProviderInterface.h"
#include "Font.generated.h"

/** Enumerates supported font caching types. */
UENUM()
enum class EFontCacheType : uint8
{
	/** The font is using offline caching (this is how UFont traditionally worked). */
	Offline,

	/** The font is using runtime caching (this is how Slate fonts work). */
	Runtime,
};

UENUM()
enum class ERuntimeFontSource : uint8
{
	/** The runtime composite font data is embedded into this Font asset */
	Asset,

	/** The runtime composite font is taken from FCoreStyle::GetDefaultFont() */
	CoreStyleDefault,
};

/** This struct is serialized using native serialization so any changes to it require a package version bump. */
USTRUCT()
struct FFontCharacter
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	int32 StartU;

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	int32 StartV;

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	int32 USize;

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	int32 VSize;

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	uint8 TextureIndex;

	UPROPERTY(EditAnywhere, Category=FontCharacter)
	int32 VerticalOffset;


	FFontCharacter()
		: StartU(0)
		, StartV(0)
		, USize(0)
		, VSize(0)
		, TextureIndex(0)
		, VerticalOffset(0)
	{ }

	/**
	 * Serialization.
	 * @param Ar - The archive with which to serialize.
	 * @returns true if serialization was successful.
	 */
	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}

	// Serializer.
	friend FArchive& operator<<( FArchive& Ar, FFontCharacter& Ch )
	{
		Ar << Ch.StartU << Ch.StartV << Ch.USize << Ch.VSize << Ch.TextureIndex;
		Ar << Ch.VerticalOffset;
		return Ar;
	}
};


template<>
struct TStructOpsTypeTraits<FFontCharacter> : public TStructOpsTypeTraitsBase2<FFontCharacter>
{
	enum 
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


/**
 * A font object, for use by Slate, UMG, and Canvas.
 *
 * A font can either be:
 *   * Runtime cached - The font contains a series of TTF files that combine to form a composite font. The glyphs are cached on demand when required at runtime.
 *   * Offline cached - The font contains a series of textures containing pre-baked cached glyphs and their associated texture coordinates.
 */
UCLASS(hidecategories=Object, autoexpandcategories=Font, MinimalAPI, BlueprintType)
class UFont : public UObject, public IFontProviderInterface
{
	GENERATED_UCLASS_BODY()

	/** What kind of font caching should we use? This controls which options we see */
	UPROPERTY(EditAnywhere, Category=Font)
	EFontCacheType FontCacheType;

	/** Where is the runtime font data for this Font stored? */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=RuntimeFont)
	ERuntimeFontSource RuntimeFontSource = ERuntimeFontSource::Asset;

	/** List of characters in the font.  For a MultiFont, this will include all characters in all sub-fonts!  Thus,
		the number of characters in this array isn't necessary the number of characters available in the font */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	TArray<struct FFontCharacter> Characters;

	/** Textures that store this font's glyph image data */
	//NOTE: Do not expose this to the editor as it has nasty crash potential
	UPROPERTY()
	TArray<TObjectPtr<class UTexture2D>> Textures;

	/** True if font is 'remapped'.  That is, the character array is not a direct mapping to unicode values.  Instead,
		all characters are indexed indirectly through the CharRemap array */
	UPROPERTY()
	int32 IsRemapped;

	/** Font metrics. */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	float EmScale;

	/** @todo document */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	float Ascent;

	/** @todo document */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	float Descent;

	/** @todo document */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	float Leading;

	/** Default horizontal spacing between characters when rendering text with this font */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	int32 Kerning;

	/** Options used when importing this font */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	struct FFontImportOptionsData ImportOptions;

	/** Number of characters in the font, not including multiple instances of the same character (for multi-fonts).
		This is cached at load-time or creation time, and is never serialized. */
	UPROPERTY(transient)
	int32 NumCharacters;

	/** The maximum height of a character in this font.  For multi-fonts, this array will contain a maximum
		character height for each multi-font, otherwise the array will contain only a single element.  This is
		cached at load-time or creation time, and is never serialized. */
	UPROPERTY(transient)
	TArray<int32> MaxCharHeight;

	/** Scale to apply to the font. */
	UPROPERTY(EditAnywhere, Category=OfflineFont)
	float ScalingFactor;

	/** The default size of the font used for legacy Canvas APIs that don't specify a font size */
	UPROPERTY(EditAnywhere, Category=RuntimeFont, meta = (ClampMin = 1, ClampMax = 1000))
	int32 LegacyFontSize;

	/** The default font name to use for legacy Canvas APIs that don't specify a font name */
	UPROPERTY(EditAnywhere, Category=RuntimeFont)
	FName LegacyFontName;

	/** Embedded composite font data */
	UE_DEPRECATED(5.7, "Public access to CompositeFont is deprecated. Use GetCompositeFont(), GetInternalCompositeFont(), or GetMutableInternalCompositeFont() instead.")
	UPROPERTY(EditAnywhere, Category = RuntimeFont)
	FCompositeFont CompositeFont;

public:

	/** This is the character that RemapChar will return if the specified character doesn't exist in the font */
	static const TCHAR NULLCHARACTER = 127;

	/** When IsRemapped is true, this array maps unicode values to entries in the Characters array */
	TMap<uint16,uint16> CharRemap;

	/** IFontProviderInterface */
	ENGINE_API virtual const FCompositeFont* GetCompositeFont() const override;

	inline const FCompositeFont& GetInternalCompositeFont() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CompositeFont;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline FCompositeFont& GetMutableInternalCompositeFont()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CompositeFont;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetInternalCompositeFontPropertyName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UFont, CompositeFont);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Get the info needed to use this UFont with Slate, using the fallback data for legacy Canvas APIs */
	ENGINE_API FSlateFontInfo GetLegacySlateFontInfo() const;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @return		Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	//~ Begin UFont Interface
	ENGINE_API TCHAR RemapChar(TCHAR CharCode) const;

	/**
	 * Calculate the width and height of a single character using this font's default size and scale.
	 *
	 * @param	InCh					the character to size
	 * @param	Width					the width of the character (in pixels)
	 * @param	Height					the height of the character (in pixels)
	 */
	ENGINE_API void GetCharSize(TCHAR InCh, float& Width, float& Height) const;

	/**
	 * Gets the kerning value for a pair of characters
	 *
	 * @param FirstChar		The first character in the pair
	 * @param SecondChar	The second character in the pair
	 *
	 * @return The kerning value
	 */
	ENGINE_API int8 GetCharKerning(TCHAR First, TCHAR Second) const;

	/**
	 * Gets the horizontal distance from the origin to the left most border of the given character
	 *
	 * @param	InCh					the character to get the offset for
	 *
	 * @return The offset value
	 */
	ENGINE_API int16 GetCharHorizontalOffset(TCHAR InCh) const;

	/**
	 * Calculate the width of the string using this font's default size and scale.
	 *
	 * @param	Text					the string to size
	 *
	 * @return	the width (in pixels) of the specified text, or 0 if Text was nullptr.
	 */
	ENGINE_API int32 GetStringSize( const TCHAR *Text ) const;

	/**
	 * Calculate the height of the string using this font's default size and scale.
	 *
	 * @param	Text					the string to size
	 *
	 * @return	the height (in pixels) of the specified text, or 0 if Text was nullptr.
	 */
	ENGINE_API int32 GetStringHeightSize( const TCHAR *Text ) const;

	//~ Begin UObject Interface
	virtual void Serialize( FArchive& Ar ) override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	//~ End UObject interface

	/**
	 * Caches the character count and maximum character height for this font (as well as sub-fonts, in the multi-font case)
	 */
	ENGINE_API void CacheCharacterCountAndMaxCharHeight();

	/**
	 *	Set the scaling factor
	 *
	 *	@param	InScalingFactor		The scaling factor to set
	 */
	inline void SetFontScalingFactor(float InScalingFactor)
	{
		ScalingFactor = InScalingFactor;
	}

	/**
	 *	Get the scaling factor
	 *
	 *	@return	float		The scaling factor currently set
	 */
	inline float GetFontScalingFactor() const
	{
		return ScalingFactor;
	}

	/** Returns the maximum height for any character in this font using this font's default size and scale. */
	ENGINE_API float GetMaxCharHeight() const;
	
	/** Determines the height and width for the passed in string. */
	ENGINE_API void GetStringHeightAndWidth( const FString& InString, int32& Height, int32& Width ) const;
	ENGINE_API void GetStringHeightAndWidth( const TCHAR *Text, int32& Height, int32& Width ) const;
};
