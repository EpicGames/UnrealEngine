// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Fonts/FontFaceInterface.h"
#include "Fonts/FontRasterizationMode.h"
#include "FontFace.generated.h"

class ITargetPlatform;
struct FPropertyChangedEvent;

/** Remapping of rasterization modes */
USTRUCT(BlueprintType)
struct FFontFacePlatformRasterizationOverrides
{
	GENERATED_BODY()

	/** Rasterization mode to be used instead of Sharp (Multi-Channel SDF) */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(DisplayName="Override for Sharp"))
	EFontRasterizationMode MsdfOverride = EFontRasterizationMode::Msdf;

	/** Rasterization mode to be used instead of Smooth (Plain SDF) */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(DisplayName="Override for Smooth"))
	EFontRasterizationMode SdfOverride = EFontRasterizationMode::Sdf;

	/** Rasterization mode to be used instead of Fast (Approximate SDF) */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(DisplayName="Override for Fast"))
	EFontRasterizationMode SdfApproximationOverride = EFontRasterizationMode::SdfApproximation;
};

/**
 * A font face asset contains the raw payload data for a source TTF/OTF file as used by FreeType.
 * During cook this asset type generates a ".ufont" file containing the raw payload data (unless loaded "Inline").
 */
UCLASS(hidecategories=Object, autoexpandcategories=FontFace, MinimalAPI, BlueprintType)
class UFontFace : public UObject, public IFontFaceInterface
{
	GENERATED_BODY()

public:
	/** Default constructor */
	UFontFace();

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
#endif // WITH_EDITOR
	//~ End UObject Interface

#if WITH_EDITORONLY_DATA
	ENGINE_API void CacheSubFaces();
#endif // WITH_EDITORONLY_DATA

	//~ Begin IFontFaceInterface Interface
#if WITH_EDITORONLY_DATA
	virtual void InitializeFromBulkData(const FString& InFilename, const EFontHinting InHinting, const void* InBulkDataPtr, const int32 InBulkDataSizeBytes) override;
#endif // WITH_EDITORONLY_DATA
	virtual const FString& GetFontFilename() const override;
	virtual EFontHinting GetHinting() const override;
	virtual EFontLoadingPolicy GetLoadingPolicy() const override;
	virtual EFontLayoutMethod GetLayoutMethod() const override;
	virtual bool IsAscendOverridden() const override;
	virtual int32 GetAscendOverriddenValue() const override;
	virtual bool IsDescendOverridden() const override;
	virtual int32 GetDescendOverriddenValue() const override;
	virtual int32 GetStrikeBrushHeightPercentage() const override;
	virtual FFontFaceDataConstRef GetFontFaceData() const override;
	virtual FFontRasterizationSettings GetRasterizationSettings() const override;
	//~ End IFontFaceInterface interface

private:
	FString GetCookedFilename() const;
	void UpdateDeviceRasterizationSettings();
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void CookAdditionalFilesOverride(const TCHAR* PackageFilename, const ITargetPlatform* TargetPlatform,
		TFunctionRef<void(const TCHAR* Filename, void* Data, int64 Size)> WriteAdditionalFile);
#endif // WITH_EDITOR
	//~ End UObject Interface

#if WITH_EDITOR
	FString GetStandaloneFilename() const;
#endif
public:
	/** The filename of the font face we were created from. This may not always exist on disk, as we may have previously loaded and cached the font data inside this asset. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=FontFace)
	FString SourceFilename;

	/** The hinting algorithm to use with the font face. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontFace)
	EFontHinting Hinting;

	/** Enum controlling how this font face should be loaded at runtime. See the enum for more explanations of the options. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontFace)
	EFontLoadingPolicy LoadingPolicy;

	/** Which method should we use when laying out the font? Try changing this if you notice clipping or height issues with your font. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontFace, AdvancedDisplay)
	EFontLayoutMethod LayoutMethod;

	/** The typographic ascender of the face, expressed in font units. */
	UPROPERTY(EditAnywhere, Category=FontFace, AdvancedDisplay, meta = (EditCondition = bIsAscendOverridden, EditConditionHides, ClampMin = "-100", ClampMax = "100"))
	int32 AscendOverriddenValue;

	/** Activate this option to use the specified ascend value instead of the value from the font. */
	UPROPERTY(EditAnywhere, Category=FontFace, AdvancedDisplay)
	bool bIsAscendOverridden;

	/** The typographic ascender of the face, expressed in font units. */
	UPROPERTY(EditAnywhere, Category=FontFace, AdvancedDisplay, meta = (EditCondition = bIsDescendOverridden, EditConditionHides, ClampMin = "-100", ClampMax = "100"))
	int32 DescendOverriddenValue;

	/** Activate this option to use the specified descend value instead of the value from the font. */
	UPROPERTY(EditAnywhere, Category=FontFace, AdvancedDisplay)
	bool bIsDescendOverridden;

	/** The percentage of the font height to draw the strike brush at.
	 * 0% is the bottom, 100% is the top.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FontFace, AdvancedDisplay, meta = (ClampMin = "0", ClampMax = "100", ForceUnits = "%"))
	int32 StrikeBrushHeightPercentage;

	/** The data associated with the font face. This should always be filled in providing the source filename is valid. CacheSubFaces should be called after manually changing this property. */
	FFontFaceDataRef FontFaceData;

#if WITH_EDITORONLY_DATA
	/** The data associated with the font face. This should always be filled in providing the source filename is valid. */
	UPROPERTY()
	TArray<uint8> FontFaceData_DEPRECATED;

	/** Transient cache of the sub-faces available within this face */
	UPROPERTY(VisibleAnywhere, Transient, Category=FontFace, AdvancedDisplay)
	TArray<FString> SubFaces;
#endif // WITH_EDITORONLY_DATA

	/** Enables distance field rendering for this face (otherwise only Bitmap rendering is used) */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode)
	bool bEnableDistanceFieldRendering = false;
	
	/** Single-channel distance field px/em resolution "low" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="Low Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MinDistanceFieldPpem = 32;

	/** Single-channel distance field px/em resolution "medium" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="Medium Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MidDistanceFieldPpem = 48;

	/** Single-channel distance field px/em resolution "high" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="High Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MaxDistanceFieldPpem = 64;
	
	/** Multi-channel distance field px/em resolution "low" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="Low Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MinMultiDistanceFieldPpem = 32;

	/** Multi-channel distance field px/em resolution "medium" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="Medium Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MidMultiDistanceFieldPpem = 40;

	/** Multi-channel distance field px/em resolution "high" quality value */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="High Quality", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	int32 MaxMultiDistanceFieldPpem = 56;

	/** If set, allows to override distance field modes set in device profiles */
	UPROPERTY(EditAnywhere, Category=DistanceFieldMode, meta=(ClampMin=8, ClampMax=256, DisplayName="Override Platform Rasterization Mode", EditCondition="bEnableDistanceFieldRendering", EditConditionHides))
	TOptional<FFontFacePlatformRasterizationOverrides> PlatformRasterizationModeOverrides;

private:
	/** Cached rasterization settings for the active device profile */
	FFontRasterizationSettings DeviceRasterizationSettings;

};
