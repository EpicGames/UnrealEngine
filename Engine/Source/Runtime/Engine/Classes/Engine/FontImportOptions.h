// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "FontImportOptions.generated.h"


/** Font character set type for importing TrueType fonts. */
UENUM()
enum EFontImportCharacterSet : int
{
	FontICS_Default,
	FontICS_Ansi,
	FontICS_Symbol,
	FontICS_MAX,
};


/** Font import options */
USTRUCT()
struct FFontImportOptionsData
{
	GENERATED_USTRUCT_BODY()

	/** Name of the typeface for the font to import */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FString FontName;

	/** Height of font (point size) */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	float Height;    

	/** Whether the font should be antialiased or not.  Usually you should leave this enabled. */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableAntialiasing:1;

	/** Whether the font should be generated in bold or not */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableBold:1;

	/** Whether the font should be generated in italics or not */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableItalic:1;

	/** Whether the font should be generated with an underline or not */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableUnderline:1;

	/** if true then forces PF_G8 and only maintains Alpha value and discards color */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bAlphaOnly:1;

	/** Character set for this font */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	TEnumAsByte<enum EFontImportCharacterSet> CharacterSet;

	/** Explicit list of characters to include in the font */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FString Chars;

	/** Range of Unicode character values to include in the font.  You can specify ranges using hyphens and/or commas (e.g. '400-900') */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FString UnicodeRange;

	/** Path on disk to a folder where files that contain a list of characters to include in the font */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FString CharsFilePath;

	/** File mask wildcard that specifies which files within the CharsFilePath to scan for characters in include in the font */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FString CharsFileWildcard;

	/** Skips generation of glyphs for any characters that are not considered 'printable' */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bCreatePrintableOnly:1;

	/** When specifying a range of characters and this is enabled, forces ASCII characters (0 thru 255) to be included as well */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bIncludeASCIIRange:1;

	/** Color of the foreground font pixels.  Usually you should leave this white and instead use the UI Styles editor to change the color of the font on the fly */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	FLinearColor ForegroundColor;

	/** Enables a very simple, 1-pixel, black colored drop shadow for the generated font */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableDropShadow:1;

	/** Horizontal size of each texture page for this font in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 TexturePageWidth;

	/** The maximum vertical size of a texture page for this font in pixels.  The actual height of a texture page may be less than this if the font can fit within a smaller sized texture page. */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 TexturePageMaxHeight;

	/** Horizontal padding between each font character on the texture page in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 XPadding;

	/** Vertical padding between each font character on the texture page in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 YPadding;

	/** How much to extend the top of the UV coordinate rectangle for each character in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 ExtendBoxTop;

	/** How much to extend the bottom of the UV coordinate rectangle for each character in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 ExtendBoxBottom;

	/** How much to extend the right of the UV coordinate rectangle for each character in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 ExtendBoxRight;

	/** How much to extend the left of the UV coordinate rectangle for each character in pixels */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 ExtendBoxLeft;

	/** Enables legacy font import mode.  This results in lower quality antialiasing and larger glyph bounds, but may be useful when debugging problems */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bEnableLegacyMode:1;

	/** The initial horizontal spacing adjustment between rendered characters.  This setting will be copied directly into the generated Font object's properties. */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	int32 Kerning;

	/** If true then the alpha channel of the font textures will store a distance field instead of a color mask */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData)
	uint32 bUseDistanceFieldAlpha:1;

	/** 
	* Scale factor determines how big to scale the font bitmap during import when generating distance field values 
	* Note that higher values give better quality but importing will take much longer.
	*/
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData, meta=(editcondition = "bUseDistanceFieldAlpha"))
	int32 DistanceFieldScaleFactor;

	/** Shrinks or expands the scan radius used to determine the silhouette of the font edges. */
	UPROPERTY(EditAnywhere, Category=FontImportOptionsData, meta=(ClampMin = "0.0", ClampMax = "4.0"))
	float DistanceFieldScanRadiusScale;

	/** Default constructor. */
	FFontImportOptionsData()
		: FontName("Arial")
		, Height(16.0f)
		, bEnableAntialiasing(true)
		, bEnableBold(false)
		, bEnableItalic(false)
		, bEnableUnderline(false)
		, bAlphaOnly(false)
		, CharacterSet(FontICS_Default)
		, bCreatePrintableOnly(false)
		, bIncludeASCIIRange(true)
		, ForegroundColor(1.0f, 1.0f, 1.0f, 1.0f )
		, bEnableDropShadow(false)
		, TexturePageWidth(256)
		, TexturePageMaxHeight(256)
		, XPadding(1)
		, YPadding(1)
		, ExtendBoxTop(0)
		, ExtendBoxBottom(0)
		, ExtendBoxRight(0)
		, ExtendBoxLeft(0)
		, bEnableLegacyMode(false)
		, Kerning(0)
		, bUseDistanceFieldAlpha(false)
		, DistanceFieldScaleFactor(16)
		, DistanceFieldScanRadiusScale(1.0f)
	{ }	
};


/**
 * Holds options for importing fonts.
 */
UCLASS(hidecategories=Object, transient, MinimalAPI)
class UFontImportOptions
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** The actual data for this object.  We wrap it in a struct so that we can copy it around between objects. */
	UPROPERTY(EditAnywhere, Category=FontImportOptions, meta=(FullyExpand = "true"))
	struct FFontImportOptionsData Data;

};
