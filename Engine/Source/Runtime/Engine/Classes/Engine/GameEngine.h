// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWindow.h"
#include "Widgets/SViewport.h"
#include "Engine/Engine.h"
#include "MovieSceneCaptureHandle.h"
#include "Templates/PimplPtr.h"
#include "GameEngine.generated.h"

class Error;
class FEngineConsoleCommandExecutor;
class FSceneViewport;
class UGameViewportClient;
class UNetDriver;

/**
 * Engine that manages core systems that enable a game.
 */
UCLASS(config=Engine, transient, MinimalAPI)
class UGameEngine
	: public UEngine
{
	GENERATED_UCLASS_BODY()

	/** Maximium delta time the engine uses to populate FApp::DeltaTime. If 0, unbound. */
	UPROPERTY(config)
	float MaxDeltaTime;

	/** Maximium time (in seconds) between the flushes of the logs on the server (best effort). If 0, this will happen every tick. */
	UPROPERTY(config)
	float ServerFlushLogInterval;

	UPROPERTY(transient)
	TObjectPtr<UGameInstance> GameInstance;

public:

	/** The game viewport window */
	TWeakPtr<class SWindow> GameViewportWindow;
	/** The primary scene viewport */
	TSharedPtr<class FSceneViewport> SceneViewport;
	/** The game viewport widget */
	TSharedPtr<class SViewport> GameViewportWidget;
	
	/**
	 * Creates the viewport widget where the games Slate UI is added to.
	 */
	ENGINE_API virtual TSharedRef<SViewport> CreateGameViewportWidget();

	UE_DEPRECATED(5.8, "Use parameterless overload instead")
	ENGINE_API void CreateGameViewportWidget( UGameViewportClient* GameViewportClient );

	/**
	 * Creates the game viewport
	 *
	 * @param GameViewportClient	The viewport client to use in the game
	 */
	ENGINE_API void CreateGameViewport( UGameViewportClient* GameViewportClient );
	
	/**
	 * Creates the game window
	 */
	static ENGINE_API TSharedRef<SWindow> CreateGameWindow();

	static ENGINE_API void SafeFrameChanged();

	/** 
	 * Enables/disables game window resolution setting overrides specified on the command line (true by default) 
	 * @note Does not trigger a refresh of the game window based on the newly effective settings
	 * @note Not thread-safe; must be called from the main thread
	 */
	static ENGINE_API void EnableGameWindowSettingsOverride(bool bEnabled);

	/**
	 * Modifies the game window resolution settings if any overrides have been specified on the command line
	 * and overrides are enabled (see EnableGameWindowSettingsOverride)
	 *
	 * @param ResolutionX	[in/out] Width of the game window, in pixels
	 * @param ResolutionY	[in/out] Height of the game window, in pixels
	 * @param WindowMode	[in/out] What window mode the game should be in
	 * @param MonitorIndex	[in] Index of the entry in FDisplayMetrics::MonitorInfo
	 */
	static ENGINE_API void ConditionallyOverrideSettings( int32& ResolutionX, int32& ResolutionY, EWindowMode::Type& WindowMode, int32 MonitorIndex = -1 );
	
	/**
	 * Determines the resolution of the game window, ensuring that the requested size is never bigger than the available desktop size
	 *
	 * @param ResolutionX	[in/out] Width of the game window, in pixels
	 * @param ResolutionY	[in/out] Height of the game window, in pixels
	 * @param WindowMode	[in/out] What window mode the game should be in
	 * @param bUseWorkArea	[in] Should we find a resolution that fits within the desktop work area for the windowed mode instead of monitor's full resolution
	 * @param MonitorIndex	[in] Index of the entry in FDisplayMetrics::MonitorInfo
	 */
	static ENGINE_API void DetermineGameWindowResolution( int32& ResolutionX, int32& ResolutionY, EWindowMode::Type& WindowMode, bool bUseWorkAreaForWindowed = false, int32 MonitorIndex = -1 );

	/**
	 * Changes the game window to use the game viewport instead of any loading screen
	 * or movie that might be using it instead
	 */
	ENGINE_API void SwitchGameWindowToUseGameViewport();

	/**
	 * Called when the game window closes (ends the game)
	 */
	ENGINE_API void OnGameWindowClosed( const TSharedRef<SWindow>& WindowBeingClosed );
	
	/**
	 * Called when the game window is moved
	 */
	UE_DEPRECATED(5.6, "Replaced by private OnGameWindowMoved_SavePosition function.")
	ENGINE_API void OnGameWindowMoved( const TSharedRef<SWindow>& WindowBeingMoved );

	/**
	 * Redraws all viewports.
	 * @param	bShouldPresent	Whether we want this frame to be presented
	 */
	ENGINE_API virtual void RedrawViewports( bool bShouldPresent = true ) override;

	ENGINE_API void OnViewportResized(FViewport* Viewport, uint32 Unused);

	/**
	 * Called when the RHI detects a GPU crash. This can be called from any thread.
	 */
	ENGINE_API void OnGPUCrash();

public:

	// UObject interface

	ENGINE_API virtual void FinishDestroy() override;

public:

	// UEngine interface

	ENGINE_API virtual void Init(class IEngineLoop* InEngineLoop) override;
	ENGINE_API virtual void Start() override;
	ENGINE_API virtual void PreExit() override;
	ENGINE_API virtual void Tick( float DeltaSeconds, bool bIdleMode ) override;
	ENGINE_API virtual float GetMaxTickRate( float DeltaTime, bool bAllowFrameRateSmoothing = true ) const override;
	ENGINE_API virtual void ProcessToggleFreezeCommand( UWorld* InWorld ) override;
	ENGINE_API virtual void ProcessToggleFreezeStreamingCommand( UWorld* InWorld ) override;
	ENGINE_API virtual bool NetworkRemapPath(UNetConnection* Connection, FString& Str, bool bReading = true) override;
	ENGINE_API virtual bool ShouldDoAsyncEndOfFrameTasks() const override;

public:

	// FExec interface
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog ) override;
#endif

public:

	// Exec command handlers
	ENGINE_API bool HandleCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleExitCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleMinimizeCommand( const TCHAR *Cmd, FOutputDevice &Ar );
	ENGINE_API bool HandleGetMaxTickRateCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleCancelCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );

#if !UE_BUILD_SHIPPING
	ENGINE_API bool HandleApplyUserSettingsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
#endif // !UE_BUILD_SHIPPING

	/** Returns the GameViewport widget */
	virtual TSharedPtr<SViewport> GetGameViewportWidget() const override
	{
		return GameViewportWidget;
	}

	/**
	 * This is a global, parameterless function used by the online subsystem modules.
	 * It should never be used in gamecode - instead use the appropriate world context function 
	 * in order to properly support multiple concurrent UWorlds.
	 */
	ENGINE_API UWorld* GetGameWorld();

protected:

	ENGINE_API FSceneViewport* GetGameSceneViewport(UGameViewportClient* ViewportClient) const;

	/** Handle to a movie capture implementation to create on startup */
	FMovieSceneCaptureHandle StartupMovieCaptureHandle;

	ENGINE_API virtual void HandleBrowseToDefaultMapFailure(FWorldContext& Context, const FString& TextURL, const FString& Error) override;

private:

	ENGINE_API virtual void HandleNetworkFailure_NotifyGameInstance(UWorld* World, UNetDriver* NetDriver, ENetworkFailure::Type FailureType) override;
	ENGINE_API virtual void HandleTravelFailure_NotifyGameInstance(UWorld* World, ETravelFailure::Type FailureType) override;

	/**
	 * Called when the game window is moved
	 */
	void OnGameWindowMoved_SavePosition(const TSharedRef<SWindow>& WindowBeingMoved, int32 SaveWinPosIndex);

	/** Last time the logs have been flushed. */
	double LastTimeLogsFlushed;

	TPimplPtr<FEngineConsoleCommandExecutor> CmdExec;
};
