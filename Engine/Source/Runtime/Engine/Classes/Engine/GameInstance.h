// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnlineFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/NetworkDelegates.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Subsystems/SubsystemCollection.h"
#include "GameFramework/OnlineReplStructs.h"
#include "ReplayTypes.h"

#include "GameInstance.generated.h"

class AGameModeBase;
class APlayerController;
class FOnlineSessionSearchResult;
class FTimerManager;
class UGameViewportClient;
class ULocalPlayer;
class UOnlineSession;
enum EPlayNetMode : int;
struct FLatentActionManager;
struct FNetDriverReplicationSystemConfig;
class ULevelEditorPlaySettings;
class IAnalyticsProvider;
namespace ERHIFeatureLevel { enum Type : int; }

// 
// 	EWelcomeScreen, 	//initial screen.  Used for platforms where we may not have a signed in user yet.
// 	EMessageScreen, 	//message screen.  Used to display a message - EG unable to connect to game.
// 	EMainMenu,		//Main frontend state of the game.  No gameplay, just user/session management and UI.	
// 	EPlaying,		//Game should be playable, or loading into a playable state.
// 	EShutdown,		//Game is shutting down.
// 	EUnknown,		//unknown state. mostly for some initializing game-logic objects.

/** Possible state of the current match, where a match is all the gameplay that happens on a single map */
namespace GameInstanceState
{
	extern ENGINE_API const FName Playing;			// We are playing the game
}

class FOnlineSessionSearchResult;

/**
 * Notification that the client is about to travel to a new URL
 *
 * @param PendingURL the travel URL
 * @param TravelType type of travel that will occur (absolute, relative, etc)
 * @param bIsSeamlessTravel is traveling seamlessly
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPreClientTravel, const FString& /*PendingURL*/, ETravelType /*TravelType*/, bool /*bIsSeamlessTravel*/);
typedef FOnPreClientTravel::FDelegate FOnPreClientTravelDelegate;
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPawnControllerChanged, APawn*, Pawn, AController*, Controller);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnUserInputDeviceConnectionChange, EInputDeviceConnectionState, NewConnectionState, FPlatformUserId, PlatformUserId, FInputDeviceId, InputDeviceId);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnUserInputDevicePairingChange, FInputDeviceId, InputDeviceId, FPlatformUserId, NewUserPlatformId, FPlatformUserId, OldUserPlatformId);


#if WITH_EDITOR

// The result of a UGameInstance PIE operation
struct FGameInstancePIEResult
{
public:
	// If not, what was the failure reason
	FText FailureReason;

	// Did the PIE operation succeed?
	bool bSuccess;

public:
	static FGameInstancePIEResult Success()
	{
		return FGameInstancePIEResult(true, FText::GetEmpty());
	}

	static FGameInstancePIEResult Failure(const FText& InReason)
	{
		return FGameInstancePIEResult(false, InReason);
	}

	bool IsSuccess() const
	{
		return bSuccess;
	}
private:
	FGameInstancePIEResult(bool bWasSuccess, const FText& InReason)
		: FailureReason(InReason)
		, bSuccess(bWasSuccess)
	{
	}
};

// Parameters used to initialize / start a PIE game instance
//@TODO: Some of these are really mutually exclusive and should be refactored (put into a struct to make this easier in the future)
struct FGameInstancePIEParameters
{
	ENGINE_API FGameInstancePIEParameters();

	// Are we doing SIE instead of PIE?
	bool bSimulateInEditor;

	// Were there any BP compile errors?
	bool bAnyBlueprintErrors;

	// Should we start in spectator mode?
	bool bStartInSpectatorMode;

	// Is this a dedicated server instance for PIE?
	bool bRunAsDedicated;

	// Is this the primary PIE client?
	bool bIsPrimaryPIEClient;

	// What time did we start PIE in the editor?
	double PIEStartTime = 0;

	// The feature level that PIE world should use
	ERHIFeatureLevel::Type WorldFeatureLevel;

	// Kept alive externally.
	ULevelEditorPlaySettings* EditorPlaySettings;

	// Which net mode should this PIE instance start in? Affects which maps are loaded.
	EPlayNetMode NetMode;

	// The map we should force the game to load instead of the one currently running in the editor. Blank for no override
	FString OverrideMapURL;
};

#endif

enum class EInputDeviceConnectionState : uint8;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnLocalPlayerEvent, ULocalPlayer*);

/**
 * GameInstance: high-level manager object for an instance of the running game.
 * Spawned at game creation and not destroyed until game instance is shut down.
 * Running as a standalone game, there will be one of these.
 * Running in PIE (play-in-editor) will generate one of these per PIE instance.
 */

UCLASS(config=Game, transient, BlueprintType, Blueprintable, MinimalAPI)
class UGameInstance : public UObject, public FExec
{
	GENERATED_UCLASS_BODY()

protected:
	struct FWorldContext* WorldContext;

	// @todo jcf list of logged-in players?

	ENGINE_API virtual bool HandleOpenCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);
	ENGINE_API virtual bool HandleDisconnectCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);
	ENGINE_API virtual bool HandleReconnectCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);
	ENGINE_API virtual bool HandleTravelCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

	/** Delegate for handling PS4 play together system events */
	ENGINE_API void OnPlayTogetherEventReceived(int32 UserIndex, const TArray<const FUniqueNetId&>& UserList);

	/** Delegate for handling external console commands */
	ENGINE_API void OnConsoleInput(const FString& Command);

	/** List of locally participating players in this game instance */
	UPROPERTY()
	TArray<TObjectPtr<ULocalPlayer>> LocalPlayers;
	
	/** Class to manage online services */
	UPROPERTY()
	TObjectPtr<class UOnlineSession> OnlineSession;

	/** List of objects that are being kept alive by this game instance. Stored as array for fast iteration, should not be modified every frame */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ReferencedObjects;

	/** Listeners to PreClientTravel call */
	FOnPreClientTravel NotifyPreClientTravelDelegates;

	/** gets triggered shortly after a pawn's controller is set. Most of the time 
	 *	it signals that the Controller has changed but in edge cases (like during 
	 *	replication) it might end up broadcasting the same pawn-controller pair 
	 *	more than once */
	UPROPERTY(BlueprintAssignable, DisplayName=OnPawnControllerChanged)
	FOnPawnControllerChanged OnPawnControllerChangedDelegates;

	/** Handle for delegate for handling PS4 play together system events */
	FDelegateHandle OnPlayTogetherEventReceivedDelegateHandle;

public:

	FString PIEMapName;
#if WITH_EDITOR
	double PIEStartTime = 0;
	bool bReportedPIEStartupTime = false;
#endif

	//~ Begin FExec Interface
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Out = *GLog) override;
#endif
	//~ End FExec Interface

	//~ Begin UObject Interface
	ENGINE_API virtual class UWorld* GetWorld() const final;
	ENGINE_API virtual void FinishDestroy() override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface

	/** virtual function to allow custom GameInstances an opportunity to set up what it needs */
	ENGINE_API virtual void Init();

	/** Opportunity for blueprints to handle the game instance being initialized. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Init"))
	ENGINE_API void ReceiveInit();

	/** virtual function to allow custom GameInstances an opportunity to do cleanup when shutting down */
	ENGINE_API virtual void Shutdown();

	/** Opportunity for blueprints to handle the game instance being shutdown. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Shutdown"))
	ENGINE_API void ReceiveShutdown();

	/**
	 * Callback for handling an Input Device's connection state change.
	 *
	 * @param NewConnectionState	The new connection state of this device
	 * @param FPlatformUserId		The User ID whose input device has changed
	 * @param FInputDeviceId		The Input Device ID that has changed connection
	 * @see IPlatformInputDeviceMapper
	 */
	ENGINE_API virtual void HandleInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	/** 
	 * Callback for when an input device connection state has changed (a new gamepad was connected or disconnected)
	 */
	UPROPERTY(BlueprintAssignable, DisplayName=OnInputDeviceConnectionChange)
	FOnUserInputDeviceConnectionChange OnInputDeviceConnectionChange;
	
	/**
	 * Callback for handling an Input Device pairing change.
	 * 
	 * @param FInputDeviceId	Input device ID
	 * @param FPlatformUserId	The NewUserPlatformId
	 * @param FPlatformUserId	The OldUserPlatformId
	 * @see IPlatformInputDeviceMapper
	 */
	ENGINE_API virtual void HandleInputDevicePairingChange(FInputDeviceId InputDeviceId, FPlatformUserId NewUserPlatformId, FPlatformUserId OldUserPlatformId);

	/**
	 * Callback when an input device has changed pairings (the owning platform user has changed for that device)
	 */
	UPROPERTY(BlueprintAssignable, DisplayName=OnUserInputDevicePairingChange)
	FOnUserInputDevicePairingChange OnUserInputDevicePairingChange;
	
	/** Opportunity for blueprints to handle network errors. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "NetworkError"))
	ENGINE_API void HandleNetworkError(ENetworkFailure::Type FailureType, bool bIsServer);

	/** Opportunity for blueprints to handle travel errors. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "TravelError"))
	ENGINE_API void HandleTravelError(ETravelFailure::Type FailureType);

	/* Called to initialize the game instance for standalone instances of the game */
	ENGINE_API void InitializeStandalone(const FName InWorldName = NAME_None, UPackage* InWorldPackage = nullptr);

	/* Called to initialize the game instance with a minimal world suitable for basic network RPC */
	ENGINE_API void InitializeForMinimalNetRPC(const FName InPackageName);

	/** Static util function used by InitializeForMinimalNetRPC and LoadMap to create the minimal world suitable for basic network RPC */
	static ENGINE_API void CreateMinimalNetRPCWorld(const FName InPackageName, UPackage*& OutWorldPackage, UWorld*& OutWorld);

#if WITH_EDITOR
	/* Called to initialize the game instance for PIE instances of the game */
	ENGINE_API virtual FGameInstancePIEResult InitializeForPlayInEditor(int32 PIEInstanceIndex, const FGameInstancePIEParameters& Params);

	/* Called to actually start the game when doing Play/Simulate In Editor */
	ENGINE_API virtual FGameInstancePIEResult StartPlayInEditorGameInstance(ULocalPlayer* LocalPlayer, const FGameInstancePIEParameters& Params);

	/** Called as soon as the game mode is spawned, to allow additional PIE setting validation prior to creating the local players / etc... (called on pure clients too, in which case the game mode is nullptr) */
	ENGINE_API virtual FGameInstancePIEResult PostCreateGameModeForPIE(const FGameInstancePIEParameters& Params, AGameModeBase* GameMode);

	ENGINE_API virtual void ReportPIEStartupTime();
#endif

	ENGINE_API class UEngine* GetEngine() const;

	struct FWorldContext* GetWorldContext() const { return WorldContext; };
	ENGINE_API class UGameViewportClient* GetGameViewportClient() const;

	/** Callback from the world context when the world changes, this ensure the world context is set properly */
	ENGINE_API virtual void OnWorldChanged(UWorld* OldWorld, UWorld* NewWorld);

	/** Starts the GameInstance state machine running */
	ENGINE_API virtual void StartGameInstance();
	virtual bool JoinSession(ULocalPlayer* LocalPlayer, int32 SessionIndexInSearchResults) { return false; }
	virtual bool JoinSession(ULocalPlayer* LocalPlayer, const FOnlineSessionSearchResult& SearchResult) { return false; }

	virtual void LoadComplete(const float LoadTime, const FString& MapName) {}

	/** Local player access */

	FOnLocalPlayerEvent OnLocalPlayerAddedEvent;
	FOnLocalPlayerEvent OnLocalPlayerRemovedEvent;

	/**
	 * Debug console command to create a player.
	 * @param ControllerId - The controller ID the player should accept input from.
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void			DebugCreatePlayer(int32 ControllerId);

	/**
	 * Debug console command to remove the player with a given controller ID.
	 * @param ControllerId - The controller ID to search for.
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void			DebugRemovePlayer(int32 ControllerId);

	ENGINE_API virtual ULocalPlayer*	CreateInitialPlayer(FString& OutError);

	/**
	 * Adds a new player.
	 * @param ControllerId - The controller ID the player should accept input from.
	 * @param OutError - If no player is returned, OutError will contain a string describing the reason.
	 * @param bSpawnPlayerController - True if a player controller should be spawned immediately for the new player.
	 * @return The player which was created.
	 */
	ENGINE_API ULocalPlayer*			CreateLocalPlayer(int32 ControllerId, FString& OutError, bool bSpawnPlayerController);

	/**
	 * Adds a new player.
	 * @param UserId - The platform user id the player should accept input from
	 * @param OutError - If no player is returned, OutError will contain a string describing the reason.
	 * @param bSpawnPlayerController - True if a player controller should be spawned immediately for the new player.
	 * @return The player which was created.
	 */
	ENGINE_API ULocalPlayer* CreateLocalPlayer(FPlatformUserId UserId, FString& OutError, bool bSpawnPlayerController);

	/**
	 * Adds a LocalPlayer to the local and global list of Players.
	 *
	 * @param	NewPlayer	The player to add
	 * @param	UserId		Id of the platform user associated with the player
	 */
	ENGINE_API virtual int32 AddLocalPlayer(ULocalPlayer* NewPlayer, FPlatformUserId UserId);

	/**
	 * Removes a player.
	 * @param Player - The player to remove.
	 * @return whether the player was successfully removed. Removal is not allowed while connected to a server.
	 */
	ENGINE_API virtual bool			RemoveLocalPlayer(ULocalPlayer * ExistingPlayer);
	
	/** Returns number of fully registered local players */
	ENGINE_API int32					GetNumLocalPlayers() const;

	/** Returns the local player at a certain index, or null if not found */
	ENGINE_API ULocalPlayer*			GetLocalPlayerByIndex(const int32 Index) const;

	/** Returns the first local player, will not be null during normal gameplay */
	ENGINE_API ULocalPlayer*			GetFirstGamePlayer() const;

	/** Returns the player controller assigned to the first local player. If World is specified it will search within that specific world */
	ENGINE_API APlayerController*		GetFirstLocalPlayerController(const UWorld* World = nullptr) const;

	/** Returns the local player assigned to a physical Controller Id, or null if not found */
	ENGINE_API ULocalPlayer*			FindLocalPlayerFromControllerId(const int32 ControllerId) const;
	
	/** Returns the local player assigned to this platform user id, or null if not found */
	ENGINE_API ULocalPlayer* FindLocalPlayerFromPlatformUserId(const FPlatformUserId UserId) const;

	/** Returns the local player who the given input device is assigned to. Null if not found */
	ENGINE_API ULocalPlayer* FindLocalPlayerFromDeviceId(const FInputDeviceId DeviceID) const;

	/** Returns the local player that has been assigned the specific unique net id */
	ENGINE_API ULocalPlayer*			FindLocalPlayerFromUniqueNetId(FUniqueNetIdPtr UniqueNetId) const;
	ENGINE_API ULocalPlayer*			FindLocalPlayerFromUniqueNetId(const FUniqueNetId& UniqueNetId) const;
	ENGINE_API ULocalPlayer*			FindLocalPlayerFromUniqueNetId(const FUniqueNetIdRepl& UniqueNetId) const;

	/** Returns const iterator for searching list of local players */
	ENGINE_API TArray<ULocalPlayer*>::TConstIterator	GetLocalPlayerIterator() const;

	/** Returns reference to entire local player list */
	ENGINE_API const TArray<ULocalPlayer*> &			GetLocalPlayers() const;

	/**
	 * Get the primary player controller on this machine (others are splitscreen children)
	 * (must have valid player state)
	 * @param bRequiresValidUniqueId - Whether the controller must also have a valid unique id (default true in order to maintain historical behaviour)
	 * @return the primary controller on this machine
	 */
	ENGINE_API APlayerController* GetPrimaryPlayerController(bool bRequiresValidUniqueId = true) const;

	/**
	 * Get the unique id for the primary player on this machine (others are splitscreen children)
	 *
	 * @return the unique id of the primary player on this machine
	 */
	ENGINE_API FUniqueNetIdRepl GetPrimaryPlayerUniqueIdRepl() const;

	ENGINE_API void CleanupGameViewport();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Called when demo playback fails for any reason */
	UE_DEPRECATED(5.1, "Now takes a EReplayResult instead.")
	virtual void HandleDemoPlaybackFailure(EDemoPlayFailure::Type FailureType, const FString& ErrorString = TEXT("")) { }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void HandleDemoPlaybackFailure(const UE::Net::TNetResult<EReplayResult>& Result) { }

	/** Called when demo recording fails for any reason */
	virtual void HandleDemoRecordFailure(const UE::Net::TNetResult<EReplayResult>& Result) { }

	/** This gets called when the player scrubs in a replay to a different level */
	virtual void OnSeamlessTravelDuringReplay() { }

	inline FTimerManager& GetTimerManager() const { return *TimerManager; }

	inline FLatentActionManager& GetLatentActionManager() const { return *LatentActionManager;  }

	/**
	 * Get a Subsystem of specified type
	 */
	UGameInstanceSubsystem* GetSubsystemBase(TSubclassOf<UGameInstanceSubsystem> SubsystemClass) const
	{
		return SubsystemCollection.GetSubsystem<UGameInstanceSubsystem>(SubsystemClass);
	}

	/**
	 * Get a Subsystem of specified type
	 */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem() const
	{
		return SubsystemCollection.GetSubsystem<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Get a Subsystem of specified type from the provided GameInstance
	 * returns nullptr if the Subsystem cannot be found or the GameInstance is null
	 */
	template <typename TSubsystemClass>
	static inline TSubsystemClass* GetSubsystem(const UGameInstance* GameInstance)
	{
		if (GameInstance)
		{
			return GameInstance->GetSubsystem<TSubsystemClass>();
		}
		return nullptr;
	}

	/**
	 * Get all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy() const
	{
		return SubsystemCollection.GetSubsystemArrayCopy<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Performs an operation on all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	void ForEachSubsystem(TFunctionRef<void(TSubsystemClass*)> Operation) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, UGameInstanceSubsystem>::IsDerived, "TSubsystemClass must be derived from UGameInstanceSubsystem");
		return SubsystemCollection.ForEachSubsystem([Operation=MoveTemp(Operation)](UGameInstanceSubsystem* Subsystem){
			Operation(CastChecked<TSubsystemClass>(Subsystem));
		}, TSubsystemClass::StaticClass());
	}

	/**
	 * Start recording a replay with the given custom name and friendly name.
	 *
	 * @param InName If not empty, the unique name to use as an identifier for the replay. If empty, a name will be automatically generated by the replay streamer implementation.
	 * @param FriendlyName An optional (may be empty) descriptive name for the replay. Does not have to be unique.
	 * @param AdditionalOptions Additional URL options to append to the URL that will be processed by the replay net driver. Will usually remain empty.
	 * @param AnalyticsProvider Optional pointer to an analytics provider which will also be passed to the replay streamer if set
	 */
	ENGINE_API virtual void StartRecordingReplay(const FString& InName, const FString& FriendlyName, const TArray<FString>& AdditionalOptions = TArray<FString>(), TSharedPtr<IAnalyticsProvider> AnalyticsProvider = nullptr);

	/** Stop recording a replay if one is currently in progress */
	ENGINE_API virtual void StopRecordingReplay();

	/**
	 * Start playing back a previously recorded replay.
	 *
	 * @param InName				Name of the replay file.
	 * @param WorldOverride			World in which the replay will be played. Passing null will cause the current world to be used.
	 * @param AdditionalOptions		Additional options that can be read by derived game instances, or the Demo Net Driver.
	 *
	 * @return True if the replay began successfully.
	 */
	ENGINE_API virtual bool PlayReplay(const FString& InName, UWorld* WorldOverride = nullptr, const TArray<FString>& AdditionalOptions = TArray<FString>());

	/**
	 * If given replay playback was deferred and has not started playback yet, this will return true.
	 * 
	 * @param ReplayName Name of the replay file.
	 * @return True if replay playback was deferred
	 */
	ENGINE_API virtual bool IsReplayDeferred(const FString& ReplayName) const;

	/**
	 * Start playing back a playlist of previously recorded replays.
	 *
	 * Using "ExitAfterReplay" on the command line will cause the system to exit *after* the last
	 * replay has been played.
	 *
	 * Using the "Demo.Loop" CVar will cause the entire replay playlist to loop.
	 *
	 * @return True if the first replay began successfully.
	 */
	ENGINE_API bool PlayReplayPlaylist(const struct FReplayPlaylistParams& PlaylistParams);

	/**
	 * Adds a join-in-progress user to the set of users associated with the currently recording replay (if any)
	 *
	 * @param UserString a string that uniquely identifies the user, usually their FUniqueNetId
	 */
	ENGINE_API virtual void AddUserToReplay(const FString& UserString);

	/** 
	 * Turns on/off listen server capability for a game instance
	 * By default this will set up the persistent URL state so it persists across server travels and spawn the appropriate network listener
	 *
	 * @param bEnable turn on or off the listen server
	 * @param PortOverride will use this specific port, or if 0 will use the URL default port
	 * @return true if desired settings were applied, games can override to deny changes in certain states
	 */
	ENGINE_API virtual bool EnableListenServer(bool bEnable, int32 PortOverride = 0);

	/** handle a game specific net control message (NMT_GameSpecific)
	 * this allows games to insert their own logic into the control channel
	 * the meaning of both data parameters is game-specific
	 */
	virtual void HandleGameNetControlMessage(class UNetConnection* Connection, uint8 MessageByte, const FString& MessageStr)
	{}
	
	/** Handle setting up encryption keys. Games that override this MUST call the delegate when their own (possibly async) processing is complete. */
	ENGINE_API virtual void ReceivedNetworkEncryptionToken(const FString& EncryptionToken, const FOnEncryptionKeyResponse& Delegate);

	/** Called when a client receives the EncryptionAck control message from the server, will generally enable encryption. */
	ENGINE_API virtual void ReceivedNetworkEncryptionAck(const FOnEncryptionKeyResponse& Delegate);

	/** Called when a connecting client fails to setup encryption */
	ENGINE_API virtual EEncryptionFailureAction ReceivedNetworkEncryptionFailure(UNetConnection* Connection);

	/** Call to preload any content before loading a map URL, used during seamless travel as well as map loading */
	ENGINE_API virtual void PreloadContentForURL(FURL InURL);

	/** Call to create the game mode for a given map URL */
	ENGINE_API virtual class AGameModeBase* CreateGameModeForURL(FURL InURL, UWorld* InWorld);

	/** Call to modify the saved url that will be used as a base for future map travels */
	ENGINE_API virtual void SetPersistentTravelURL(FURL InURL);

	/** Return the game mode subclass to use for a given map, options, and portal. By default return passed in one */
	ENGINE_API virtual TSubclassOf<AGameModeBase> OverrideGameModeClass(TSubclassOf<AGameModeBase> GameModeClass, const FString& MapName, const FString& Options, const FString& Portal) const;


	/**
	 * Game instance has an opportunity to modify the level name before the client starts travel
	 */
	virtual void ModifyClientTravelLevelURL(FString& LevelName)
	{
	}

	/** return true to delay an otherwise ready-to-join PendingNetGame performing LoadMap() and finishing up
	 * useful to wait for content downloads, etc
	 */
	virtual bool DelayPendingNetGameTravel()
	{
		return false;
	}

	/**
	 * return true to delay player controller spawn (sending NMT_Join)
	 */
	virtual bool DelayCompletionOfPendingNetGameTravel()
	{
		return false;
	}

	FTimerManager* TimerManager;
	FLatentActionManager* LatentActionManager;

	/** @return online session management object associated with this game instance */
	class UOnlineSession* GetOnlineSession() const { return OnlineSession; }

	/** @return OnlineSession class to use for this game instance  */
	ENGINE_API virtual TSubclassOf<UOnlineSession> GetOnlineSessionClass();

	/** Returns true if this instance is for a dedicated server world */
	ENGINE_API bool IsDedicatedServerInstance() const;

	/**
	 * Retrieves the name of the online subsystem for the platform used by this instance.
	 * This will be used as the value of the PlayerOnlinePlatformName parameter in
	 * the NMT_Login message when this client connects to a server.
	 * Normally this will be the same as the DefaultPlatformService config value,
	 * but games may override it if they need non-default behavior (for example,
	 * if they are using multiple online subsystems at the same time).
	 */
	ENGINE_API virtual FName GetOnlinePlatformName() const;

	/**
	 * Helper function for traveling to a session that has already been joined via the online platform
	 * Grabs the URL from the session info and travels
	 *
	 * @param ControllerId controller initiating the request
	 * @param InSessionName name of session to travel to
	 *
	 * @return true if able or attempting to travel, false otherwise
	 */
	ENGINE_API virtual bool ClientTravelToSession(int32 ControllerId, FName InSessionName);

	/** Broadcast a notification that travel is occurring */
	ENGINE_API void NotifyPreClientTravel(const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel);
	/** @return delegate fired when client travel occurs */
	FOnPreClientTravel& OnNotifyPreClientTravel() { return NotifyPreClientTravelDelegates; }

	/** @return delegate broadcasted shortly after pawn's controller is set */
	FOnPawnControllerChanged& GetOnPawnControllerChanged() { return OnPawnControllerChangedDelegates; }

	/**
	 * Calls HandleDisconnect on either the OnlineSession if it exists or the engine, to cause a travel back to the default map. The instance must have a world.
	 */
	ENGINE_API virtual void ReturnToMainMenu();

	/** Registers an object to keep alive as long as this game instance is alive */
	ENGINE_API virtual void RegisterReferencedObject(UObject* ObjectToReference);

	/** Remove a referenced object, this will allow it to GC out */
	ENGINE_API virtual void UnregisterReferencedObject(UObject* ObjectToReference);

	/** Allows a GameInstance to override the default configured replication system for it's specific conditions. Return Default to use the configured replication system. */
	ENGINE_API virtual EReplicationSystem GetDesiredReplicationSystem(FName InNetDriverDefinition) const;

	/** Allows a GameInstance to override the default configuration of the Iris replication system. For instance if specific game modes require different settings. */
	virtual void OverrideIrisReplicationSystemConfig(FNetDriverReplicationSystemConfig& OutConfig, bool bIsServer) const {}

protected:
	/** Non-virtual dispatch for OnStart, also calls the associated global OnStartGameInstance. */
	ENGINE_API void BroadcastOnStart();

	/** Called when the game instance is started either normally or through PIE. */
	ENGINE_API virtual void OnStart();

	/** Find a map override argument on the command-line string (the first argument without a leading '-' or -map=..., whichever comes first). */
	static ENGINE_API bool GetMapOverrideName(const TCHAR* CmdLine, FString& OverrideMapName);

private:

	FObjectSubsystemCollection<UGameInstanceSubsystem> SubsystemCollection;
};
