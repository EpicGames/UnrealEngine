// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/ScriptMacros.h"
#include "Input/PopupMethodReply.h"
//#include "Widgets/SWidget.h"
//#include "Widgets/SOverlay.h"
#include "ShowFlags.h"
#include "Engine/ScriptViewportClient.h"
#include "Engine/ViewportSplitScreen.h"
#include "Engine/TitleSafeZone.h"
#include "Engine/GameViewportDelegates.h"
#include "Engine/DebugDisplayProperty.h"
#include "UObject/SoftObjectPath.h"
#include "StereoRendering.h"
#include "AudioDeviceHandle.h"
#include "Slate/SceneViewport.h"

#include "GameViewportClient.generated.h"

class FCanvas;
class FSceneView;
class FSceneViewport;
class IGameLayerManager;
class SGameLayerManager;
class SOverlay;
class SViewport;
class SWidget;
class SWindow;
class UCanvas;
class UGameInstance;
class ULocalPlayer;
class UNetDriver;
struct FMargin;

enum class EViewStatusForScreenPercentage;

/** Delegate for overriding the behavior when a navigation action is taken, Not to be confused with FNavigationDelegate which allows a specific widget to override behavior for itself */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FCustomNavigationHandler, const uint32, TSharedPtr<SWidget>);

/** Delegate for overriding key input before it is routed to player controllers, returning true means it was handled by delegate */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOverrideInputKeyHandler, FInputKeyEventArgs& /*EventArgs*/);

/** Delegate for overriding axis input before it is routed to player controllers, returning true means it was handled by delegate */
DECLARE_DELEGATE_RetVal_FourParams(bool, FOverrideInputAxisHandler, FInputKeyEventArgs& /*EventArgs*/, float& /*Delta*/, float& /*DeltaTime*/, int32& /*NumSamples*/);

DECLARE_MULTICAST_DELEGATE_SevenParams(FOnInputAxisSignature, FViewport* /*InViewport*/, int32 /*ControllerId*/, FKey /*Key*/, float /*Delta*/, float /*DeltaTime*/, int32 /*NumSamples*/, bool /*bGamepad*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnInputKeySignature, const FInputKeyEventArgs& /*EventArgs*/);

/**
 * A game viewport (FViewport) is a high-level abstract interface for the
 * platform specific rendering, audio, and input subsystems.
 * GameViewportClient is the engine's interface to a game viewport.
 * Exactly one GameViewportClient is created for each instance of the game.  The
 * only case (so far) where you might have a single instance of Engine, but
 * multiple instances of the game (and thus multiple GameViewportClients) is when
 * you have more than one PIE window running.
 *
 * Responsibilities:
 * propagating input events to the global interactions list
 *
 * @see UGameViewportClient
 */
UCLASS(Within=Engine, transient, config=Engine, MinimalAPI)
class UGameViewportClient : public UScriptViewportClient, public FExec
{
	GENERATED_UCLASS_BODY()

public:
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	ENGINE_API UGameViewportClient(FVTableHelper& Helper);

	ENGINE_API virtual ~UGameViewportClient();

	/** The viewport's console.   Might be null on consoles */
	UPROPERTY()
	TObjectPtr<class UConsole> ViewportConsole;

	/** Debug properties that have been added via one of the "displayall" commands */
	UPROPERTY()
	TArray<struct FDebugDisplayProperty> DebugProperties;

	/** Array of the screen data needed for all the different splitscreen configurations */
	TArray<struct FSplitscreenData> SplitscreenInfo;

	UPROPERTY(Config)
	int32 MaxSplitscreenPlayers = 4;

	/** If true, this viewport is a play in editor viewport */
	uint32 bIsPlayInEditorViewport:1;

	/** set to disable world rendering */
	uint32 bDisableWorldRendering:1;

	/** set to true if we want to split the render of each player view into separate Viewports/RT */
	uint32 bEnablePlayersSplitRT:1;
protected:
	/**
	 * The splitscreen type that is actually being used; takes into account the number of players and other factors (such as cinematic mode)
	 * that could affect the splitscreen mode that is actually used.
	 */
	TEnumAsByte<ESplitScreenType::Type> ActiveSplitscreenType;

	/* The relative world context for this viewport */
	UPROPERTY()
	TObjectPtr<UWorld> World;

	UPROPERTY()
	TObjectPtr<UGameInstance> GameInstance;

	/** If true will suppress the blue transition text messages. */
	bool bSuppressTransitionMessage;

	/** Strong handle to the audio device used by this viewport. */
	FAudioDeviceHandle AudioDevice;

	/** Handle to delegate in case audio device is destroyed. */
	FDelegateHandle AudioDeviceDestroyedHandle;

public:

	/** see enum EViewModeIndex */
	int32 ViewModeIndex;

	/** Rotates controller ids among gameplayers, useful for testing splitscreen with only one controller. */
	UFUNCTION(exec)
	ENGINE_API virtual void SSSwapControllers();

	/** Exec for toggling the display of the title safe area
	  * @deprecated Use the cvar "r.DebugSafeZone.Mode=1".
	  */
	UFUNCTION(exec, meta = (DeprecatedFunction, DeprecationMessage = "Use the cvar \"r.DebugSafeZone.Mode=1.\""))
	ENGINE_API virtual void ShowTitleSafeArea();

	/** Sets the player which console commands will be executed in the context of. */
	UFUNCTION(exec)
	ENGINE_API virtual void SetConsoleTarget(int32 PlayerIndex);

	/** Add a Widget to use for specified cursor type*/
	UE_DEPRECATED(5.4, "AddCursorWidget is deprecated, use SetSoftwareCursorWidget instead.")
	ENGINE_API void AddCursorWidget(EMouseCursor::Type Cursor, class UUserWidget* CursorWidget);

	/** Returns a relative world context for this viewport.	 */
	ENGINE_API virtual UWorld* GetWorld() const override;

	/* Create the game viewport */
	ENGINE_API virtual TSharedRef<FSceneViewport> CreateViewport(TSharedPtr<SViewport> InViewportWidget);
	UE_DEPRECATED(5.8, "Use CreateViewport instead")
	ENGINE_API virtual FSceneViewport* CreateGameViewport(TSharedPtr<SViewport> InViewportWidget) final;

	/* Returns the game viewport */
	ENGINE_API FSceneViewport* GetGameViewport();
	ENGINE_API const FSceneViewport* GetGameViewport() const;

	/* Returns the widget for this viewport */
	ENGINE_API TSharedPtr<SViewport> GetGameViewportWidget() const;

	/* Returns the relevant game instance for this viewport */
	ENGINE_API UGameInstance* GetGameInstance() const;

	ENGINE_API virtual void Init(struct FWorldContext& WorldContext, UGameInstance* OwningGameInstance, bool bCreateNewAudioDevice = true);

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin FViewportClient Interface.
	virtual void RedrawRequested(FViewport* InViewport) override {}
	ENGINE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	ENGINE_API virtual bool InputAxis(const FInputKeyEventArgs& EventArgs) override;
	ENGINE_API virtual bool InputChar(FViewport* Viewport,int32 ControllerId, TCHAR Character) override;
	ENGINE_API virtual bool InputTouch(FViewport* const InViewport, const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force, const uint64 Timestamp) override;
	ENGINE_API virtual bool InputMotion(FViewport* Viewport, const FInputDeviceId DeviceId, const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, const uint64 Timestamp) override;
	ENGINE_API virtual bool InputGesture(FViewport* Viewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp) override;
	ENGINE_API virtual EMouseCursor::Type GetCursor(FViewport* Viewport, int32 X, int32 Y ) override;
	ENGINE_API virtual TOptional<TSharedRef<SWidget>> MapCursor(FViewport* Viewport, const FCursorReply& CursorReply) override;
	ENGINE_API virtual void Precache() override;
	ENGINE_API virtual void Draw(FViewport* Viewport,FCanvas* SceneCanvas) override;
	ENGINE_API virtual bool ProcessScreenShots(FViewport* Viewport) override;
	ENGINE_API virtual TOptional<bool> QueryShowFocus(const EFocusCause InFocusCause) const override;
	ENGINE_API virtual void LostFocus(FViewport* Viewport) override;
	ENGINE_API virtual void ReceivedFocus(FViewport* Viewport) override;
	ENGINE_API virtual void Activated(FViewport* InViewport, const FWindowActivateEvent& InActivateEvent) override;
	ENGINE_API virtual void Deactivated(FViewport* InViewport, const FWindowActivateEvent& InActivateEvent) override;
	ENGINE_API virtual bool IsInPermanentCapture() override;
	ENGINE_API virtual bool WindowCloseRequested() override;
	ENGINE_API virtual void CloseRequested(FViewport* Viewport) override;
	ENGINE_API virtual bool IsOrtho() const override;
	ENGINE_API virtual void MouseEnter(FViewport* Viewport, int32 x, int32 y) override;
	ENGINE_API virtual void MouseLeave(FViewport* Viewport) override;
	ENGINE_API virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport) override;

	//~ End FViewportClient Interface.

	/** make any adjustments to the views after they've been completely set up */
	virtual void FinalizeViews(class FSceneViewFamily* ViewFamily, const TMap<ULocalPlayer*, FSceneView*>& PlayerViewMap)
	{}

	//~ Begin FExec Interface.
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar) override;
#endif
protected:
	ENGINE_API virtual bool Exec_Runtime( UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar) override;
	//~ End FExec Interface.

public:
	/**
	 * Gives the game's custom viewport client a way to handle F11 or Alt+Enter before processing the input
	 */
	ENGINE_API bool TryToggleFullscreenOnInputKey(FKey Key, EInputEvent EventType);

	/** Function that handles remapping controller information for cases like PIE */
	ENGINE_API virtual void RemapControllerInput(FInputKeyEventArgs& InOutKeyEvent);

	/**
	 * Exec command handlers
	 */
	ENGINE_API bool HandleForceFullscreenCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleShowCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandleShowLayerCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandleViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandleNextViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandlePrevViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandlePreCacheCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleToggleFullscreenCommand();
	ENGINE_API bool HandleSetResCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleHighresScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleHighresScreenshotUICommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleBugScreenshotwithHUDInfoCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleBugScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleKillParticlesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleForceSkelLODCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	ENGINE_API bool HandleDisplayCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleDisplayAllCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleDisplayAllLocationCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleDisplayAllRotationCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleDisplayClearCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleGetAllLocationCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleGetAllRotationCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleTextureDefragCommand( const TCHAR* Cmd, FOutputDevice& Ar );

	/**
	 * Adds a widget to the Slate viewport's overlay (i.e for in game UI or tools) at the specified Z-order
	 * 
	 * @param  ViewportContent	The widget to add.  Must be valid.
	 * @param  ZOrder  The Z-order index for this widget.  Larger values will cause the widget to appear on top of widgets with lower values.
	 */
	ENGINE_API virtual void AddViewportWidgetContent( TSharedRef<class SWidget> ViewportContent, const int32 ZOrder = 0 );

	/**
	 * Removes a previously-added widget from the Slate viewport
	 *
	 * @param	ViewportContent  The widget to remove.  Must be valid.
	 */
	ENGINE_API virtual void RemoveViewportWidgetContent( TSharedRef<class SWidget> ViewportContent );

	/**
	 * Adds a widget to the Slate viewport's overlay (i.e for in game UI or tools) at the specified Z-order.
	 * associates it with a specific player and keeps it in their sub-rect.
	 * 
	 * @param  Player The player to add the widget to.
	 * @param  ViewportContent	The widget to add.  Must be valid.
	 * @param  ZOrder  The Z-order index for this widget.  Larger values will cause the widget to appear on top of widgets with lower values.
	 */
	ENGINE_API virtual void AddViewportWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent, const int32 ZOrder);

	/**
	 * Removes a previously-added widget from the Slate viewport, in the player's section.
	 *
	 * @param	Player The player to remove the widget's viewport from. Null will remove the widget regardless of which player it was added for.
	 * @param	ViewportContent  The widget to remove.  Must be valid.
	 */
	ENGINE_API virtual void RemoveViewportWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent);

	/**
	 * This function removes all widgets from the viewport overlay
	 */
	ENGINE_API void RemoveAllViewportWidgets();

	/**
	 * Adds a widget to the Slate viewport's game overlay (i.e for top-level debug tools) at the specified Z-order.
	 *
	 * @param  ViewportContent	The widget to add.  Must be valid.
	 * @param  ZOrder  The Z-order index for this widget.  Larger values will cause the widget to appear on top of widgets with lower values.
	 */
	ENGINE_API virtual void AddGameLayerWidget(TSharedRef<SWidget> ViewportContent, const int32 ZOrder);

	/**
	 * Removes a previously-added widget from the Slate viewport's game overlay.
	 *
	 * @param	ViewportContent  The widget to remove.  Must be valid.
	 */
	ENGINE_API virtual void RemoveGameLayerWidget(TSharedRef<SWidget> ViewportContent);

	/**
	 * Recreates cursor widgets from UISettings class.
	 */
	ENGINE_API void RebuildCursors();

	/**
	 * Cleans up all rooted or referenced objects created or managed by the GameViewportClient.  This method is called
	 * when this GameViewportClient has been disassociated with the game engine (i.e. is no longer the engine's GameViewport).
	 */
	ENGINE_API virtual void DetachViewportClient();

	/**
	 * Called every frame to allow the game viewport to update time based state.
	 * @param	DeltaTime	The time since the last call to Tick.
	 */
	ENGINE_API virtual void Tick( float DeltaTime );

	/**
	 * Determines whether this viewport client should receive calls to InputAxis() if the game's window is not currently capturing the mouse.
	 * Used by the UI system to easily receive calls to InputAxis while the viewport's mouse capture is disabled.
	 */
	ENGINE_API virtual bool RequiresUncapturedAxisInput() const override;

	/**
	 * Set this GameViewportClient's viewport and viewport frame to the viewport specified
	 * @param	InViewportFrame	The viewportframe to set
	 */
	UE_DEPRECATED(5.8, "UGameViewportClient::SetViewportFrame and FViewportFrame are deprecated. Use FSceneViewport/SetViewport.")
	inline void SetViewportFrame(FSceneViewport* InViewportFrame)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SetViewport(InViewportFrame);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Set this GameViewportClient's viewport to the viewport specified
	 * @param	InViewportFrame	The viewport to set
	 */
	UE_DEPRECATED(5.8, "If you must perform actions in response to the creation + assignment of this client's ISlateViewport, do so by overriding CreateGameViewport or AddAssociation")
	ENGINE_API virtual void SetViewport(FSceneViewport* InViewportFrame) final;

	UE_DEPRECATED(5.8, "Call Bind and use overlay created by UGameViewportClient, instead of creating one externally.")
	ENGINE_API void SetViewportOverlayWidget(TSharedPtr<SWindow> InWindow, TSharedRef<SOverlay> InViewportOverlayWidget);

	UE_DEPRECATED(5.8, "Call Bind and use layer manager created by UGameViewportClient, instead of creating one externally.")
	ENGINE_API void SetGameLayerManager(TSharedPtr<IGameLayerManager> LayerManager);

	/**
	 * Gets the layer manager for the UI.
	 */
	TSharedPtr<IGameLayerManager> GetGameLayerManager() const
	{
		return GameLayerManager;
	}

	/** Returns access to this viewport's Slate window */
	ENGINE_API TSharedPtr<SWindow> GetWindow() const;

	/**
	 * Process Console Command
	 *
	 * @param	Command		command to process
	 */
	ENGINE_API virtual FString ConsoleCommand( const FString& Command );

	/**
	 * Retrieve the size of the main viewport.
	 *
	 * @param	out_ViewportSize	[out] will be filled in with the size of the main viewport
	 */
	ENGINE_API void GetViewportSize( FVector2D& ViewportSize ) const;

	/** @return Whether or not the main viewport is fullscreen or windowed. */
	ENGINE_API bool IsFullScreenViewport() const;

	/** @return If we're specifically in fullscreen mode, not windowed fullscreen. */
	ENGINE_API bool IsExclusiveFullscreenViewport() const;

	/** @return mouse position in game viewport coordinates (does not account for splitscreen) */
	ENGINE_API virtual bool GetMousePosition(FVector2D& MousePosition) const;

	/**
	 * Determine whether a fullscreen viewport should be used in cases where there are multiple players.
	 *
	 * @return	true to use a fullscreen viewport; false to allow each player to have their own area of the viewport.
	 */
	ENGINE_API bool ShouldForceFullscreenViewport() const;

	/**
	 * Initialize the game viewport.
	 * @param OutError - If an error occurs, returns the error description.
	 * @return False if an error occurred, true if the viewport was initialized successfully.
	 */
	ENGINE_API virtual ULocalPlayer* SetupInitialLocalPlayer(FString& OutError);

	/** @return Returns the splitscreen type that is currently being used */
	inline ESplitScreenType::Type GetCurrentSplitscreenConfiguration() const
	{
		return ActiveSplitscreenType;
	}

	/**
	 * Sets the value of ActiveSplitscreenConfiguration based on the desired split-screen layout type, current number of players, and any other
	 * factors that might affect the way the screen should be laid out.
	 */
	ENGINE_API virtual void UpdateActiveSplitscreenType();

	/** Called before rendering to allow the game viewport to allocate subregions to players. */
	ENGINE_API virtual void LayoutPlayers();

	/** Allows game code to disable splitscreen (useful when in menus) */
	ENGINE_API void SetForceDisableSplitscreen(const bool bDisabled);

	/** Determines whether splitscreen is forced to be turned off */
	bool IsSplitscreenForceDisabled() const
	{
		return bDisableSplitScreenOverride;
	}

	/** called before rending subtitles to allow the game viewport to determine the size of the subtitle area
	 * @param Min top left bounds of subtitle region (0 to 1)
	 * @param Max bottom right bounds of subtitle region (0 to 1)
	 */
	ENGINE_API virtual void GetSubtitleRegion(FVector2D& MinPos, FVector2D& MaxPos);

	/**
	* Convert a LocalPlayer to it's index in the GamePlayer array
	* @param LPlayer Player to get the index of
	* @returns -1 if the index could not be found.
	*/
	ENGINE_API int32 ConvertLocalPlayerToGamePlayerIndex( ULocalPlayer* LPlayer );

	/** Whether the player at LocalPlayerIndex's viewport has a "top of viewport" safezone or not. */
	ENGINE_API bool HasTopSafeZone( int32 LocalPlayerIndex );

	/** Whether the player at LocalPlayerIndex's viewport has a "bottom of viewport" safezone or not.*/
	ENGINE_API bool HasBottomSafeZone( int32 LocalPlayerIndex );

	/** Whether the player at LocalPlayerIndex's viewport has a "left of viewport" safezone or not. */
	ENGINE_API bool HasLeftSafeZone( int32 LocalPlayerIndex );

	/** Whether the player at LocalPlayerIndex's viewport has a "right of viewport" safezone or not. */
	ENGINE_API bool HasRightSafeZone( int32 LocalPlayerIndex );

	/**
	* Get the total pixel size of the screen.
	* This is different from the pixel size of the viewport since we could be in splitscreen
	*/
	ENGINE_API void GetPixelSizeOfScreen( float& Width, float& Height, UCanvas* Canvas, int32 LocalPlayerIndex );

	/** Calculate the amount of safezone needed for a single side for both vertical and horizontal dimensions*/
	ENGINE_API void CalculateSafeZoneValues( FMargin& SafeZone, UCanvas* Canvas, int32 LocalPlayerIndex, bool bUseMaxPercent );

	/**
	* pixel size of the deadzone for all sides (right/left/top/bottom) based on which local player it is
	* @return true if the safe zone exists
	*/
	ENGINE_API bool CalculateDeadZoneForAllSides( ULocalPlayer* LPlayer, UCanvas* Canvas, float& fTopSafeZone, float& fBottomSafeZone, float& fLeftSafeZone, float& fRightSafeZone, bool bUseMaxPercent = false );

	/**  
	 * Draws the safe area using the current r.DebugSafeZone.Mode=1 when there is not a valid PlayerController HUD.
	 * 
	 * @param Canvas	Canvas on which to draw
	 */
	ENGINE_API virtual void DrawTitleSafeArea( UCanvas* Canvas );

	/**
	 * Called after rendering the player views and HUDs to render menus, the console, etc.
	 * This is the last rendering call in the render loop
	 *
	 * @param Canvas	The canvas to use for rendering.
	 */
	ENGINE_API virtual void PostRender( UCanvas* Canvas );

	/**
	 * Displays the transition screen.
	 *
	 * @param Canvas	The canvas to use for rendering.
	 */
	ENGINE_API virtual void DrawTransition( UCanvas* Canvas );

	/** 
	 * Print a centered transition message with a drop shadow. 
	 * 
	 * @param Canvas	The canvas to use for rendering.
	 * @param Message	Transition message
	 */
	ENGINE_API virtual void DrawTransitionMessage( UCanvas* Canvas, const FString& Message );

	/**
	 * Notifies all interactions that a new player has been added to the list of active players.
	 *
	 * @param	PlayerIndex		the index [into the GamePlayers array] where the player was inserted
	 * @param	AddedPlayer		the player that was added
	 */
	ENGINE_API virtual void NotifyPlayerAdded( int32 PlayerIndex, class ULocalPlayer* AddedPlayer );

	/**
	 * Notifies all interactions that a new player has been added to the list of active players.
	 *
	 * @param	PlayerIndex		the index [into the GamePlayers array] where the player was located
	 * @param	RemovedPlayer	the player that was removed
	 */
	ENGINE_API virtual void NotifyPlayerRemoved( int32 PlayerIndex, class ULocalPlayer* RemovedPlayer );

	/**
	 * Notification of server travel error messages, generally network connection related (package verification, client server handshaking, etc) 
	 * generally not expected to handle the failure here, but provide notification to the user
	 *
	 * @param	FailureType	the type of error
	 * @param	ErrorString	additional string detailing the error
	 */
	ENGINE_API virtual void PeekTravelFailureMessages(UWorld* InWorld, enum ETravelFailure::Type FailureType, const FString& ErrorString);

	/**
	 * Notification of network error messages
	 * generally not expected to handle the failure here, but provide notification to the user
	 *
	 * @param	World associated with failure
	 * @param	NetDriver associated with failure
	 * @param	FailureType	the type of error
	 * @param	ErrorString	additional string detailing the error
	 */
	ENGINE_API virtual void PeekNetworkFailureMessages(UWorld *InWorld, UNetDriver *NetDriver, enum ENetworkFailure::Type FailureType, const FString& ErrorString);

	/** Make sure all navigation objects have appropriate path rendering components set.  Called when EngineShowFlags.Navigation is set. */
	ENGINE_API virtual void VerifyPathRenderingComponents();

	/** Accessor for delegate called when a non-HDR screenshot is captured */
	static FOnScreenshotCaptured& OnScreenshotCaptured()
	{
		return ScreenshotCapturedDelegate;
	}

	/** Accessor for delegate called when an HDR screenshot is captured */
	static FOnHDRScreenshotCaptured& OnHDRScreenshotCaptured()
	{
		return HDRScreenshotCapturedDelegate;
	}
	
	/** Accessor for delegate called when a viewport is rendered */
	static FOnViewportRendered& OnViewportRendered()
	{
		return ViewportRenderedDelegate;
	}
	
	/* Accessor for the delegate called when a viewport is asked to close. */
	FOnCloseRequested& OnCloseRequested()
	{
		return CloseRequestedDelegate;
	}

	/** Accessor for the delegate called when the window owning the viewport is asked to close. */
	FOnWindowCloseRequested& OnWindowCloseRequested()
	{
		return WindowCloseRequestedDelegate;
	}

	/** Accessor for the delegate called when the game viewport is created. */
	static FSimpleMulticastDelegate& OnViewportCreated()
	{
		return CreatedDelegate;
	}

	// Accessor for the delegate called when a player is added to the game viewport
	FOnGameViewportClientPlayerAction& OnPlayerAdded()
	{
		return PlayerAddedDelegate;
	}

	// Accessor for the delegate called when a player is removed from the game viewport
	FOnGameViewportClientPlayerAction& OnPlayerRemoved()
	{
		return PlayerRemovedDelegate;
	}

	// Accessor for the delegate called when the engine starts drawing a game viewport
	FSimpleMulticastDelegate& OnBeginDraw()
	{
		return BeginDrawDelegate;
	}

	// Accessor for the delegate called when the game viewport is drawn, before drawing the console
	FSimpleMulticastDelegate& OnDrawn()
	{
		return DrawnDelegate;
	}

	// Accessor for the delegate called when the engine finishes drawing a game viewport
	FSimpleMulticastDelegate& OnEndDraw()
	{
		return EndDrawDelegate;
	}

	// Accessor for the delegate called when ticking the game viewport
	FOnGameViewportTick& OnTick()
	{
		return TickDelegate;
	}

	/** Set an override handler for navigation. */
	FCustomNavigationHandler& OnNavigationOverride()
	{
		return CustomNavigationEvent;
	}

	/** Set an override handler for input key handling, can be used for things like press start screen */
	FOverrideInputKeyHandler& OnOverrideInputKey()
	{
		return OnOverrideInputKeyEvent;
	}

	/** Set an override handler for input axis handling */
	FOverrideInputAxisHandler& OnOverrideInputAxis()
	{
		return OnOverrideInputAxisEvent;
	}

	/** Gets broadcast delegate for input key, happens in addition to player controller input */
	FOnInputKeySignature& OnInputKey()
	{
		return OnInputKeyEvent;
	}

	/** Gets broadcast delegate for input axis, happens in addition to player controller input */
	FOnInputAxisSignature& OnInputAxis()
	{
		return OnInputAxisEvent;
	}

	/** Return the engine show flags for this viewport */
	virtual FEngineShowFlags* GetEngineShowFlags() override
	{ 
		return &EngineShowFlags; 
	}
	
	/** Sets the view mode */
	ENGINE_API void SetViewMode(EViewModeIndex ViewMode, FOutputDevice* Output = nullptr);

	ENGINE_API bool SetHardwareCursor(EMouseCursor::Type CursorShape, FName GameContentPath, FVector2D HotSpot);

	/** 
	 * @return @true if this viewport is currently being used for simulate in editors
	 */
	ENGINE_API bool IsSimulateInEditorViewport() const;

	ENGINE_API bool GetUseMouseForTouch() const;

	void SetCurrentBufferVisualizationMode(FName NewBufferVisualizationMode) { CurrentBufferVisualizationMode = NewBufferVisualizationMode; }
	FName GetCurrentBufferVisualizationMode() const { return CurrentBufferVisualizationMode; }

	void SetCurrentNaniteVisualizationMode(FName NewNaniteVisualizationMode) { CurrentNaniteVisualizationMode = NewNaniteVisualizationMode; }
	FName GetCurrentNaniteVisualizationMode() const { return CurrentNaniteVisualizationMode; }

	void SetCurrentLumenVisualizationMode(FName NewLumenVisualizationMode) { CurrentLumenVisualizationMode = NewLumenVisualizationMode; }
	FName GetCurrentLumenVisualizationMode() const { return CurrentLumenVisualizationMode; }

	void SetCurrentMegaLightsVisualizationMode(FName NewMegaLightsVisualizationMode) { CurrentMegaLightsVisualizationMode = NewMegaLightsVisualizationMode; }
	FName GetCurrentMegaLightsVisualizationMode() const { return CurrentMegaLightsVisualizationMode; }

	void SetCurrentSubstrateVisualizationMode(FName NewSubstrateVisualizationMode) { CurrentSubstrateVisualizationMode = NewSubstrateVisualizationMode; }
	FName GetCurrentSubstrateVisualizationMode() const { return CurrentSubstrateVisualizationMode; }

	void SetCurrentGroomVisualizationMode(FName NewGroomVisualizationMode) { CurrentGroomVisualizationMode = NewGroomVisualizationMode; }
	FName GetCurrentGroomVisualizationMode() const { return CurrentGroomVisualizationMode; }

	void SetCurrentVirtualShadowMapVisualizationMode(FName NewVirtualShadowMapVisualizationMode) { CurrentVirtualShadowMapVisualizationMode = NewVirtualShadowMapVisualizationMode; }
	FName GetCurrentVirtualShadowMapVisualizationMode() const { return CurrentVirtualShadowMapVisualizationMode; }

	ENGINE_API virtual void AddAssociation(FViewport& InViewport) override;

	ENGINE_API virtual void RemoveAssociation(FViewport& InViewport) override;

protected:
	bool HasAudioFocus() const { return bHasAudioFocus; }

	/** Updates CSVProfiler camera stats */
	ENGINE_API void UpdateCsvCameraStats(const TMap<ULocalPlayer*, FSceneView*>& PlayerViewMap);

protected:
	/** FCommonViewportClient interface */
	ENGINE_API virtual float UpdateViewportClientWindowDPIScale() const override;

public:
	/** The show flags used by the viewport's players. */
	FEngineShowFlags EngineShowFlags;

	union
	{
		/** The platform-specific viewport which this viewport client is attached to. */
		FSceneViewport* Viewport;

		/** The platform-specific viewport frame which this viewport is contained by. */
		UE_DEPRECATED(5.8, "ViewportFrame is deprecated. Use the Viewport member instead.")
		FSceneViewport* ViewportFrame;
	};

	/**
	 * Controls suppression of the blue transition text messages 
	 * 
	 * @param bSuppress	Pass true to suppress messages
	 */
	void SetSuppressTransitionMessage( bool bSuppress )
	{
		bSuppressTransitionMessage = bSuppress;
	}

	/**
	 * Get a ptr to the stat unit data for this viewport
	 */
	virtual FStatUnitData* GetStatUnitData() const override
	{
		return StatUnitData;
	}

	/**
	 * Get a ptr to the stat thermals data for this viewport
	 */
	virtual FStatThermalsData* GetStatThermalsData() const override
	{
		return StatThermalsData;
	}

	/**
	 * Get a ptr to the stat unit data for this viewport
	 */
	virtual FStatHitchesData* GetStatHitchesData() const override
	{
		return StatHitchesData;
	}

	/**
	 * Get a ptr to the enabled stats list
	 */
	virtual const TArray<FString>* GetEnabledStats() const override
	{
		return &EnabledStats;
	}

	/**
	 * Sets all the stats that should be enabled for the viewport
	 *
	 * @param InEnabledStats	Stats to enable
	 */
	ENGINE_API virtual void SetEnabledStats(const TArray<FString>& InEnabledStats) override;

	/**
	 * Check whether a specific stat is enabled for this viewport
	 *
	 * @param	InName	Name of the stat to check
	 */
	virtual bool IsStatEnabled(const FString& InName) const override
	{
		return EnabledStats.Contains(InName);
	}

	/**
	 * Set whether to ignore input.
	 */
	void SetIgnoreInput(bool Ignore)
	{
		bIgnoreInput = Ignore;
	}

	/**
	 * Check whether we should ignore input.
	 */
	virtual bool IgnoreInput() override
	{
		return bIgnoreInput;
	}

	/**
	 * Set the mouse capture behavior for the viewport.
	 */
	ENGINE_API void SetMouseCaptureMode(EMouseCaptureMode Mode);

	/**
	 * Gets the mouse capture behavior when the viewport is clicked
	 */
	ENGINE_API virtual EMouseCaptureMode GetMouseCaptureMode() const override;

	/**
	 * Gets whether or not the viewport captures the Mouse on launch of the application
	 * Technically this controls capture on the first window activate, so in situations 
	 * where the application is launched but isn't activated the effect is delayed until
	 * activation.
	 */
	ENGINE_API virtual bool CaptureMouseOnLaunch() override;

	/**
	 * Gets whether or not the cursor is locked to the viewport when the viewport captures the mouse
	 */
	virtual bool LockDuringCapture() override
	{
		return MouseLockMode != EMouseLockMode::DoNotLock;
	}

	/**
	 * Gets whether or not the cursor should always be locked to the viewport
	 */
	virtual bool ShouldAlwaysLockMouse() override
	{
		return MouseLockMode == EMouseLockMode::LockAlways
			 || (MouseLockMode == EMouseLockMode::LockInFullscreen && IsExclusiveFullscreenViewport());
	}

	/**
	* Sets the current mouse cursor lock mode when the viewport is clicked
	*/
	ENGINE_API void SetMouseLockMode(EMouseLockMode InMouseLockMode);

	/**
	 * Returns the current mouse cursor lock mode when the viewport is clicked.
	 */
	ENGINE_API EMouseLockMode GetMouseLockMode() const;

	/**
	 * Sets whether or not the cursor is hidden when the viewport captures the mouse
	 */
	ENGINE_API void SetHideCursorDuringCapture(bool InHideCursorDuringCapture);

	/**
	 * Gets whether or not the cursor is hidden when the viewport captures the mouse
	 */
	virtual bool HideCursorDuringCapture() const override
	{
		return bHideCursorDuringCapture;
	}

	/** 
	 * Should we make new windows for popups or create an overlay in the current window.
	 */
	ENGINE_API virtual FPopupMethodReply OnQueryPopupMethod() const override;

	/**
	* Optionally do custom handling of a navigation.
	*/
	ENGINE_API virtual bool HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination) override;

	/**
	 * Sets whether or not the software cursor widgets are used.
	 * If no software cursor widgets are set this setting has no meaningful effect.
	 */
	void SetUseSoftwareCursorWidgets(bool bInUseSoftwareCursorWidgets)
	{
		bUseSoftwareCursorWidgets = bInUseSoftwareCursorWidgets;
	}

	/**
	* Get whether or not the viewport is currently using software cursor
	*/
	bool GetIsUsingSoftwareCursorWidgets() { return bUseSoftwareCursorWidgets; }

#if WITH_EDITOR
	/** Accessor for delegate called when a game viewport received input key */
	FOnGameViewportInputKey& OnGameViewportInputKey()
	{
		return GameViewportInputKeyDelegate;
	}
#endif

	/** Accessor for delegate called when the window moves to another display */
	FOnWindowDisplayChanged& OnWindowDisplayChanged()
	{
		return WindowDisplayChangedDelegate;
	}

	/** Accessor for delegate called when the engine toggles fullscreen */
	FOnToggleFullscreen& OnToggleFullscreen()
	{
		return ToggleFullscreenDelegate;
	}

	/** 
	 * Applies requested changes to display configuration 
	 * @param Dimensions Pointer to new dimensions of the display. nullptr for no change.
	 * @param WindowMode What window mode do we want to st the display to.
	 */
	ENGINE_API bool SetDisplayConfiguration( const FIntPoint* Dimensions, EWindowMode::Type WindowMode);

	UE_DEPRECATED(5.4, "SetVirtualCursorWidget is deprecated, use SetSoftwareCursorWidget instead.")
	ENGINE_API void SetVirtualCursorWidget(EMouseCursor::Type Cursor, class UUserWidget* Widget);

	/** Add a Widget to use for specified cursor type*/
	UE_DEPRECATED(5.4, "AddSoftwareCursorFromSlateWidget is deprecated, use SetSoftwareCursorWidget instead.")
	ENGINE_API void AddSoftwareCursorFromSlateWidget(EMouseCursor::Type InCursorType, TSharedPtr<SWidget> CursorWidgetPtr);

	UE_DEPRECATED(5.4, "AddSoftwareCursor is deprecated, use SetSoftwareCursorWidget instead and pass the Created Widget instead of the UserWidget Class.")
	ENGINE_API void AddSoftwareCursor(EMouseCursor::Type Cursor, const FSoftClassPath& CursorClass);

	/** Add a Widget to use for specified cursor type*/
	ENGINE_API void SetSoftwareCursorWidget(EMouseCursor::Type InCursorType, class UUserWidget* Widget);

	/** Add a Widget to use for specified cursor type*/
	ENGINE_API void SetSoftwareCursorWidget(EMouseCursor::Type InCursorType, TSharedPtr<SWidget> CursorWidgetPtr);

	/** Get the slate widget of the current software cursor */
	ENGINE_API TSharedPtr<SWidget> GetSoftwareCursorWidget(EMouseCursor::Type Cursor) const;

	/** Does the viewport client have a software cursor set up for the given enum? */
	ENGINE_API bool HasSoftwareCursor(EMouseCursor::Type Cursor) const;

	ENGINE_API void EnableCsvPlayerStats(int32 LocalPlayerCount);

	ENGINE_API EViewStatusForScreenPercentage GetViewStatusForScreenPercentage() const;

private:
	/** Resets the platform type shape to nullptr, to restore it to the OS default. */
	ENGINE_API void ResetHardwareCursorStates();

	/** Set the Software Cursor from the class path. This should stay private to make sure error reporting on loading is done by the caller. */
	void SetSoftwareCursorFromClassPath(EMouseCursor::Type Cursor, const FSoftClassPath& CursorClass);

	/**
	 * Set a specific stat to either enabled or disabled (returns the number of remaining enabled stats)
	 */
	int32 SetStatEnabled(const TCHAR* InName, const bool bEnable, const bool bAll = false)
	{
		if (bEnable)
		{
			check(!bAll);	// Not possible to enable all
			EnabledStats.AddUnique(InName);
		}
		else
		{
			if (bAll)
			{
				EnabledStats.Empty();
			}
			else
			{
				EnabledStats.Remove(InName);
			}
		}
		return EnabledStats.Num();
	}

	/** Process the 'show volumes' console command */
	ENGINE_API void ToggleShowVolumes();

	/** Process the 'show collision' console command */
	ENGINE_API void ToggleShowCollision();

	/** Delegate handler to see if a stat is enabled on this viewport */
	ENGINE_API void HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled);

	/** Delegate handler for when stats are enabled in a viewport */
	ENGINE_API void HandleViewportStatEnabled(const TCHAR* InName);

	/** Delegate handler for when stats are disabled in a viewport */
	ENGINE_API void HandleViewportStatDisabled(const TCHAR* InName);

	/** Delegate handler for when all stats are disabled in a viewport */
	ENGINE_API void HandleViewportStatDisableAll(const bool bInAnyViewport);

	/** Delegate handler for when a window is moved or resized */
	ENGINE_API void HandleRegularWindowMovedOrResized(TSharedRef<SWindow> InWindow);

	/** Delegate handler for when a window DPI changes and we might need to adjust the scenes resolution */
	ENGINE_API void HandleWindowDPIScaleChanged(TSharedRef<SWindow> InWindow);

	struct FPngFileData
	{
		FString FileName;
		double ScaleFactor;
		TArray<uint8> FileData;

		FPngFileData()
			: ScaleFactor(1.0)
		{
		}
	};

	/** Tries to create a hardware cursor from supplied PNGs images */
	ENGINE_API void* LoadCursorFromPngs(ICursor& PlatformCursor, const FString& InPathToCursorWithoutExtension, FVector2D InHotSpot);

	/** Finds available PNG cursor images */
	ENGINE_API bool LoadAvailableCursorPngs(TArray<TSharedPtr<FPngFileData>>& Results, const FString& InPathToCursorWithoutExtension);

	/**
	* Adds a DebugDisplayProperty to the DebugProperties array if it does not already exist. 
	* @see FDebugDisplayProperty for more info on debug properties
	* 
	* @param Obj				Object that the debug property is on
	* @param WithinClass		further limit the display to objects that have an Outer of WithinClass
	* @param PropertyName		name of the property to display
	* @param bSpecialProperty	whether PropertyName is a "special" value not directly mapping to a real property (e.g. state name)
	*/
	ENGINE_API void AddDebugDisplayProperty(class UObject* Obj, TSubclassOf<class UObject> WithinClass, const FName& PropertyName, bool bSpecialProperty = false);

protected:
	/** Handle to the audio device created for this viewport. Each viewport (for multiple PIE) will have its own audio device. */
	uint32 AudioDeviceHandle = uint32(INDEX_NONE);

	/** Whether or not this audio device is in audio-focus */
	bool bHasAudioFocus = false;

private:
	/** Cached Slate window that contains this viewport's SViewport widget. */
	mutable TWeakPtr<SWindow> Window;

	/** Overlay widget that contains widgets to draw on top of the game viewport */
	TSharedPtr<SOverlay> ViewportOverlayWidget;

	/** The game layer manager allows management of widgets for different player areas of the screen. */
	TSharedPtr<IGameLayerManager> GameLayerManager;

	/** Current buffer visualization mode for this game viewport */
	FName CurrentBufferVisualizationMode;

	/** Current Nanite visualization mode for this game viewport */
	FName CurrentNaniteVisualizationMode;

	/** Current Lumen visualization mode for this game viewport */
	FName CurrentLumenVisualizationMode;

	/** Current MegaLights visualization mode for this game viewport */
	FName CurrentMegaLightsVisualizationMode;

	/** Current Substrate visualization mode for this game viewport */
	FName CurrentSubstrateVisualizationMode;

	/** Current Groom visualization mode for this game viewport */
	FName CurrentGroomVisualizationMode;

	/** Current virtual shadow map visualization mode for this game viewport */
	FName CurrentVirtualShadowMapVisualizationMode;

	/** Weak pointer to the highres screenshot dialog if it's open */
	TWeakPtr<SWindow> HighResScreenshotDialog;

	/** Hardware Cursor Cache */
	TMap<FName, void*> HardwareCursorCache;

	/** Hardware cursor mapping for default cursor shapes. */
	TMap<EMouseCursor::Type, void*> HardwareCursors;

	/** Map of Software Cursor Widgets*/
	TMap<EMouseCursor::Type, TSharedPtr<SWidget>> CursorWidgets;

	/** Controls if the Map of Software Cursor Widgets is used */
	bool bUseSoftwareCursorWidgets;

	/* Function that handles bug screen-shot requests w/ or w/o extra HUD info (project-specific) */
	ENGINE_API bool RequestBugScreenShot(const TCHAR* Cmd, bool bDisplayHUDInfo);

#if WITH_EDITOR
	/** Delegate called when game viewport client received input key */
	FOnGameViewportInputKey GameViewportInputKeyDelegate;
#endif

	/** Delegate called at the end of the frame when a non-HDR screenshot is captured */
	static ENGINE_API FOnScreenshotCaptured ScreenshotCapturedDelegate;

	/** Delegate called at the end of the frame when an HDR screenshot is captured */
	static ENGINE_API FOnHDRScreenshotCaptured HDRScreenshotCapturedDelegate;
	
	/** Delegate called right after the viewport is rendered */
	static ENGINE_API FOnViewportRendered ViewportRenderedDelegate;

	/** Delegate called when a request to close the viewport is received */
	FOnCloseRequested CloseRequestedDelegate;

	/** Delegate called when the window owning the viewport is requested to close */
	FOnWindowCloseRequested WindowCloseRequestedDelegate;

	/** Delegate called when the game viewport is created. */
	static ENGINE_API FSimpleMulticastDelegate CreatedDelegate;

	/** Delegate called when a player is added to the game viewport */
	FOnGameViewportClientPlayerAction PlayerAddedDelegate;

	/** Delegate called when a player is removed from the game viewport */
	FOnGameViewportClientPlayerAction PlayerRemovedDelegate;

	/** Delegate called when the engine starts drawing a game viewport */
	FSimpleMulticastDelegate BeginDrawDelegate;

	/** Delegate called when the game viewport is drawn, before drawing the console */
	FSimpleMulticastDelegate DrawnDelegate;

	/** Delegate called when the engine finishes drawing a game viewport */
	FSimpleMulticastDelegate EndDrawDelegate;

	/** Delegate called when ticking the game viewport */
	FOnGameViewportTick TickDelegate;

	/** Delegate called when the window moves to another display */
	FOnWindowDisplayChanged WindowDisplayChangedDelegate;

	/** Delegate called when the engine toggles fullscreen */
	FOnToggleFullscreen ToggleFullscreenDelegate;

	/** Delegate for custom navigation behavior */
	FCustomNavigationHandler CustomNavigationEvent;

	/** Delegate for overriding input key behavior */
	FOverrideInputKeyHandler OnOverrideInputKeyEvent;

	/** Delegate for overriding input axis behavior */
	FOverrideInputAxisHandler OnOverrideInputAxisEvent;

	/** A broadcast delegate broadcasting from UGameViewportClient::InputKey */
	FOnInputKeySignature OnInputKeyEvent;

	/** A broadcast delegate broadcasting from UGameViewportClient::InputAxis */
	FOnInputAxisSignature OnInputAxisEvent;

	/** Data needed to display perframe stat tracking when STAT UNIT is enabled */
	FStatUnitData* StatUnitData;

	/** Data needed to display thermal state when STAT THERMALS is enabled */
	FStatThermalsData* StatThermalsData;

	/** Data needed to display perframe stat tracking when STAT HITCHES is enabled */
	FStatHitchesData* StatHitchesData;

	/** A list of all the stat names which are enabled for this viewport (static so they persist between runs) */
	static ENGINE_API TArray<FString> EnabledStats;

	/** Disables splitscreen, useful when game code is in menus, and doesn't want splitscreen on */
	bool bDisableSplitScreenOverride;

	/** Whether or not to ignore input */
	bool bIgnoreInput;

	/** Mouse capture behavior when the viewport is clicked */
	EMouseCaptureMode MouseCaptureMode;

	/** Whether or not the cursor is hidden when the viewport captures the mouse */
	bool bHideCursorDuringCapture;

	/** Mouse cursor locking behavior when the viewport is clicked */
	EMouseLockMode MouseLockMode;

	/** Is the mouse currently over the viewport client */
	bool bIsMouseOverClient;

#if WITH_EDITOR
	/** Should the mouse send touch events. */
	bool bUseMouseForTouchInEditor;
#endif
};
