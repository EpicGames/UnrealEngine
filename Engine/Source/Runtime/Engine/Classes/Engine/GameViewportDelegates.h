// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "GenericPlatform/GenericApplication.h"

class FViewport;

/**
 * Delegate type used by UGameViewportClient when a non-HDR screenshot has been captured
 *
 * The first parameter is the width.
 * The second parameter is the height.
 * The third parameter is the array of bitmap data (FColor).
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnScreenshotCaptured, int32 /*Width*/, int32 /*Height*/, const TArray<FColor>& /*Colors*/);

/**
 * Delegate type used by UGameViewportClient when an HDR screenshot has been captured
 *
 * The first parameter is the width.
 * The second parameter is the height.
 * The third parameter is the array of bitmap data (FLinearColor).
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnHDRScreenshotCaptured, int32 /*Width*/, int32 /*Height*/, const TArray<FLinearColor>& /*Colors*/);

/**
 * Delegate type used by UGameViewportClient when a viewport is rendered
 *
 * The first parameter is the viewport.
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewportRendered, FViewport*);

/**
 * Delegate type used by UGameViewportClient when the top level window associated
 * with the viewport has been requested to close.
 * At this point, the viewport has not been closed and the operation may be canceled.
 * This may not called from PIE, Editor Windows, on consoles, or before the game ends
 * from other methods.
 * This is only when the platform specific window is closed.
 *
 * Return indicates whether or not the window may be closed.
 *
 * @see UGameViewportClient.
 */
DECLARE_DELEGATE_RetVal(bool, FOnWindowCloseRequested);

/**
 * Delegate type used by UGameViewportClient when call is made to close a viewport
 *
 * The first parameter is the viewport being closed.
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCloseRequested, FViewport*);

/**
 * Delegate type used by UGameViewportClient for when a player is added or removed
 *
 * The first parameter is the player index
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGameViewportClientPlayerAction, int32);

/**
 * Delegate type used by UGameViewportClient for tick callbacks
 *
 * The first parameter is the time delta
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGameViewportTick, float);

DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnGameViewportInputKey, FKey, FModifierKeysState, EInputEvent);

/**
 * Delegate type used by UGameViewportClient for when the window moves to another display
 *
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE(FOnWindowDisplayChanged);

/**
 * Delegate type used by UGameViewportClient for when engine in toggling fullscreen
 *
 * @see UGameViewportClient
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnToggleFullscreen, bool);
