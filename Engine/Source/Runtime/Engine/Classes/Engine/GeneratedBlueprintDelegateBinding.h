// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "GeneratedBlueprintDelegateBinding.generated.h"

/** 
 * Entry for a delegate to assign after a blueprint has been instanced 
 * 
 * For this class we assume the delegate has been generated and exists on BPGC instance itself
 */
USTRUCT()
struct FGeneratedBlueprintDelegateBinding
{
	GENERATED_BODY()

public:

	/** Name of property on the component that we want to assign to. */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Name of function that we want to bind to the delegate. */
	UPROPERTY()
	FName FunctionNameToBind;

	FGeneratedBlueprintDelegateBinding()
		: DelegatePropertyName(NAME_None)
		, FunctionNameToBind(NAME_None)
	{ }
};


/**
 * Binding used for event nodes generated at runtime.
 */
UCLASS(MinimalAPI)
class UGeneratedBlueprintBinding
	: public UDynamicBlueprintBinding
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FGeneratedBlueprintDelegateBinding> GeneratedBlueprintBindings;

	//~ Begin DynamicBlueprintBinding Interface
	ENGINE_API virtual void BindDynamicDelegates(UObject* InInstance) const override;
	ENGINE_API virtual void UnbindDynamicDelegates(UObject* InInstance) const override;
	ENGINE_API virtual void UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty) const override;
	//~ End DynamicBlueprintBinding Interface
};
