// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SpotLight.h"
#include "GeneratedMeshAreaLight.generated.h"

/**
 * Implements a light that is created after a lighting build with Lightmass and handles mesh area light influence on dynamic objects.
 */
UCLASS(notplaceable, MinimalAPI)
class AGeneratedMeshAreaLight
	: public ASpotLight
{
	GENERATED_UCLASS_BODY()
};
