// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HLOD/HLODProxyDesc.h"
#include "HLOD/HLODProxyMesh.h"
#include "HLODProxy.generated.h"

class ALODActor;
class UMaterialInterface;
class UPrimitiveComponent;
class UStaticMesh;
class UStaticMeshComponent;
class UTexture;

/** This asset acts as a proxy to a static mesh for ALODActors to display */
UCLASS(MinimalAPI)
class UHLODProxy : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Setup the map - only called at initial construction */
	ENGINE_API void SetMap(const UWorld* InMap);

    /** Get the owner map for this HLOD proxy */
	ENGINE_API TSoftObjectPtr<UWorld> GetMap() const;

	/** Register an HLODProxyDesc, building it from an existing LODActor */
	ENGINE_API UHLODProxyDesc* AddLODActor(ALODActor* InLODActor);

	/** Adds a static mesh and the key used to generate it */
	ENGINE_API void AddMesh(ALODActor* InLODActor, UStaticMesh* InStaticMesh, const FName& InKey);

	/** Clean out invalid proxy mesh entries */
	ENGINE_API void Clean();

	/** Spawn LODActors from the HLODProxyDescs found in this proxy. */
 	ENGINE_API void SpawnLODActors(ULevel* InLevel);

	/** Update the HLODDesc stored in this proxy using existing LODActors from the level */
	ENGINE_API void UpdateHLODDescs(const ULevel* InLevel);

	/** Helper for recursive traversing LODActors to retrieve a semi deterministic first AActor for resulting asset naming */
	static ENGINE_API const AActor* FindFirstActor(const ALODActor* LODActor);

	/** Extract components that we would use for LOD generation. Used to generate keys for LOD actors. */
	static ENGINE_API void ExtractComponents(const ALODActor* LODActor, TArray<UPrimitiveComponent*>& InOutComponents);

	/** Build a unique key for the LOD actor, used to determine if the actor needs rebuilding */
	static ENGINE_API FName GenerateKeyForActor(const ALODActor* LODActor, bool bMustUndoLevelTransform = true);

	static ENGINE_API uint32 GetCRC(const FTransform& InTransform, uint32 InCRC = 0);
	static ENGINE_API uint32 GetCRC(UMaterialInterface* InMaterialInterface, uint32 InCRC = 0);
	static ENGINE_API uint32 GetCRC(UTexture* InTexture, uint32 InCRC = 0);
	static ENGINE_API uint32 GetCRC(UStaticMesh* InStaticMesh, uint32 InCRC = 0, bool bInConsiderPhysicData = false);
	static ENGINE_API uint32 GetCRC(UStaticMeshComponent* InComponent, uint32 InCRC = 0, const FTransform& TransformComponents = FTransform::Identity);

	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	/** Returns true if proxy doesn't contain any mesh entry. */
	ENGINE_API bool IsEmpty() const;

	/** Destroy all assets & delete this HLOD proxy package. */
	ENGINE_API void DeletePackage();

	/** Specify the transform to apply to the source meshes when building HLODs. */
	ENGINE_API bool SetHLODBakingTransform(const FTransform& InTransform);
#endif

	/**
	 * Recursively retrieves StaticMeshComponents from a LODActor and its child LODActors
	 *
	 * @param Actor - LODActor instance
	 * @param InOutComponents - Will hold the StaticMeshComponents
	 */
	static ENGINE_API void ExtractStaticMeshComponentsFromLODActor(const ALODActor* LODActor, TArray<UStaticMeshComponent*>& InOutComponents);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR
	/** Check if we contain data for the specified actor */
	ENGINE_API bool ContainsDataForActor(const ALODActor* InLODActor) const;
#endif

private:
#if WITH_EDITOR
	// Remove all assets associated with the given proxy mesh
	void RemoveAssets(const FHLODProxyMesh& ProxyMesh);

	// Clear object flags to ensure it can be properly GC'd and removed from its package.
	void DestroyObject(UObject* Obj);
#endif

#if WITH_EDITORONLY_DATA
	/** Keep hold of the level in the editor to allow for package cleaning etc. */
	UPROPERTY(VisibleAnywhere, Category = "Proxy Mesh")
	TSoftObjectPtr<UWorld> OwningMap;
#endif

	/** All the mesh proxies we contain */
	UPROPERTY(VisibleAnywhere, Category = "Proxy Mesh")
	TArray<FHLODProxyMesh> ProxyMeshes;

	UPROPERTY(VisibleAnywhere, Category = "Proxy Mesh")
	TMap<TObjectPtr<UHLODProxyDesc>, FHLODProxyMesh> HLODActors;
};
