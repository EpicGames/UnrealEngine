// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/PhysicsObject.h"
#include "Engine/NetSerialization.h"
#include "Engine/ActorInstanceHandle.h"
#include "HitResult.generated.h"

class AActor;
class UPackageMap;
class UPhysicalMaterial;
class UPrimitiveComponent;

/**
 * Structure containing information about one hit of a trace, such as point of impact and surface normal at that point.
 */
USTRUCT(BlueprintType, meta = (HasNativeBreak = "/Script/Engine.GameplayStatics.BreakHitResult", HasNativeMake = "/Script/Engine.GameplayStatics.MakeHitResult"))
struct FHitResult
{
	GENERATED_BODY()

	/** Face index we hit (for complex hits with triangle meshes). */
	UPROPERTY()
	int32 FaceIndex;

	/**
	 * 'Time' of impact along trace direction (ranging from 0.0 to 1.0) if there is a hit, indicating time between TraceStart and TraceEnd.
	 * For swept movement (but not queries) this may be pulled back slightly from the actual time of impact, to prevent precision problems with adjacent geometry.
	 */
	UPROPERTY()
	float Time;
	 
	/** The distance from the TraceStart to the Location in world space. This value is 0 if there was an initial overlap (trace started inside another colliding object). */
	UPROPERTY()
	float Distance;
	
	/**
	 * The location in world space where the moving shape would end up against the impacted object, if there is a hit. Equal to the point of impact for line tests.
	 * Example: for a sphere trace test, this is the point where the center of the sphere would be located when it touched the other object.
	 * For swept movement (but not queries) this may not equal the final location of the shape since hits are pulled back slightly to prevent precision issues from overlapping another surface.
	 */
	UPROPERTY()
	FVector_NetQuantize Location;

	/**
	 * Location in world space of the actual contact of the trace shape (box, sphere, ray, etc) with the impacted object.
	 * Example: for a sphere trace test, this is the point where the surface of the sphere touches the other object.
	 * @note: In the case of initial overlap (bStartPenetrating=true), ImpactPoint will be the same as Location because there is no meaningful single impact point to report.
	 */
	UPROPERTY()
	FVector_NetQuantize ImpactPoint;

	/**
	 * Normal of the hit in world space, for the object that was swept. Equal to ImpactNormal for line tests.
	 * This is computed for capsules and spheres, otherwise it will be the same as ImpactNormal.
	 * Example: for a sphere trace test, this is a normalized vector pointing in towards the center of the sphere at the point of impact.
	 */
	UPROPERTY()
	FVector_NetQuantizeNormal Normal;

	/**
	 * Normal of the hit in world space, for the object that was hit by the sweep, if any.
	 * For example if a sphere hits a flat plane, this is a normalized vector pointing out from the plane.
	 * In the case of impact with a corner or edge of a surface, usually the "most opposing" normal (opposed to the query direction) is chosen.
	 */
	UPROPERTY()
	FVector_NetQuantizeNormal ImpactNormal;

	/**
	 * Start location of the trace.
	 * For example if a sphere is swept against the world, this is the starting location of the center of the sphere.
	 */
	UPROPERTY()
	FVector_NetQuantize TraceStart;

	/**
	 * End location of the trace; this is NOT where the impact occurred (if any), but the furthest point in the attempted sweep.
	 * For example if a sphere is swept against the world, this would be the center of the sphere if there was no blocking hit.
	 */
	UPROPERTY()
	FVector_NetQuantize TraceEnd;

	/**
	  * If this test started in penetration (bStartPenetrating is true) and a depenetration vector can be computed,
	  * this value is the distance along Normal that will result in moving out of penetration.
	  * If the distance cannot be computed, this distance will be zero.
	  */
	UPROPERTY()
	float PenetrationDepth;

	/** If the hit result is from a collision this will have extra info about the item that hit the second item. */
	UPROPERTY(NotReplicated)
	int32 MyItem;

	/** Extra data about item that was hit (hit primitive specific). */
	UPROPERTY()
	int32 Item;

	/** Index to item that was hit, also hit primitive specific. */
	UPROPERTY()
	uint8 ElementIndex;

	/** Indicates if this hit was a result of blocking collision. If false, there was no hit or it was an overlap/touch instead. */
	UPROPERTY()
	uint8 bBlockingHit : 1;

	/**
	 * Whether the trace started in penetration, i.e. with an initial blocking overlap.
	 * In the case of penetration, if PenetrationDepth > 0.f, then it will represent the distance along the Normal vector that will result in
	 * minimal contact between the swept shape and the object that was hit. In this case, ImpactNormal will be the normal opposed to movement at that location
	 * (ie, Normal may not equal ImpactNormal). ImpactPoint will be the same as Location, since there is no single impact point to report.
	 */
	UPROPERTY()
	uint8 bStartPenetrating : 1;

	/**
	 * Physical material that was hit.
	 * @note Must set bReturnPhysicalMaterial on the swept PrimitiveComponent or in the query params for this to be returned.
	 */
	UPROPERTY()
	TWeakObjectPtr<UPhysicalMaterial> PhysMaterial;

	/** Handle to the object hit by the trace. */
	UPROPERTY()
	FActorInstanceHandle HitObjectHandle;

	/** PrimitiveComponent hit by the trace. */
	UPROPERTY()
	TWeakObjectPtr<UPrimitiveComponent> Component;

	/** The object that owns the PhysicsObject. This is used to determine if the PhysicsObject is still valid when not owned by an Actor */
	TWeakObjectPtr<UObject> PhysicsObjectOwner;

	/** PhysicsObjects hit by the query. Not exposed to blueprints for the time being */
	Chaos::FPhysicsObjectHandle PhysicsObject;

	/** Name of bone we hit (for skeletal meshes). */
	UPROPERTY()
	FName BoneName;

	/** Name of the _my_ bone which took part in hit event (in case of two skeletal meshes colliding). */
	UPROPERTY(NotReplicated)
	FName MyBoneName;


	FHitResult()
	{
		Init();
	}
	
	explicit FHitResult(float InTime)
	{
		Init();
		Time = InTime;
	}

	explicit FHitResult(EForceInit InInit)
	{
		Init();
	}

	explicit FHitResult(ENoInit NoInit)
	{
	}

	explicit FHitResult(FVector Start, FVector End)
	{
		Init(Start, End);
	}

	/** Initialize empty hit result with given time. */
	inline void Init()
	{
		FMemory::Memzero(this, sizeof(FHitResult));
		HitObjectHandle = FActorInstanceHandle();
		Time = 1.f;
		MyItem = INDEX_NONE;
	}

	/** Initialize empty hit result with given time, TraceStart, and TraceEnd */
	inline void Init(FVector Start, FVector End)
	{
		FMemory::Memzero(this, sizeof(FHitResult));
		HitObjectHandle = FActorInstanceHandle();
		Time = 1.f;
		TraceStart = Start;
		TraceEnd = End;
		MyItem = INDEX_NONE;
	}

	/** Ctor for easily creating "fake" hits from limited data. */
	ENGINE_API FHitResult(AActor* InActor, UPrimitiveComponent* InComponent, FVector const& HitLoc, FVector const& HitNorm);
 
	/** Reset hit result while optionally saving TraceStart and TraceEnd. */
	inline void Reset(float InTime = 1.f, bool bPreserveTraceData = true)
	{
		const FVector SavedTraceStart = TraceStart;
		const FVector SavedTraceEnd = TraceEnd;
		Init();
		Time = InTime;
		if (bPreserveTraceData)
		{
			TraceStart = SavedTraceStart;
			TraceEnd = SavedTraceEnd;
		}
	}

	/** Utility to return the Actor that owns the Component that was hit. */
	inline AActor* GetActor() const
	{
		return HitObjectHandle.FetchActor();
	}

	inline FActorInstanceHandle GetHitObjectHandle() const
	{
		return HitObjectHandle;
	}

	inline bool HasValidHitObjectHandle() const
	{
		return HitObjectHandle.IsValid();
	}

	/** Utility to return the Component that was hit. */
	inline UPrimitiveComponent* GetComponent() const
	{
		return Component.Get();
	}

	/** Optimized serialize function */
	ENGINE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);

	/** Return true if there was a blocking hit that was not caused by starting in penetration. */
	inline bool IsValidBlockingHit() const
	{
		return bBlockingHit && !bStartPenetrating;
	}

	/** Static utility function that returns the first 'blocking' hit in an array of results. */
	static FHitResult* GetFirstBlockingHit(TArray<FHitResult>& InHits)
	{
		for(int32 HitIdx=0; HitIdx<InHits.Num(); HitIdx++)
		{
			if(InHits[HitIdx].bBlockingHit)
			{
				return &InHits[HitIdx];
			}
		}
		return nullptr;
	}

	/** Static utility function that returns the number of blocking hits in array. */
	static int32 GetNumBlockingHits(const TArray<FHitResult>& InHits)
	{
		int32 NumBlocks = 0;
		for(int32 HitIdx=0; HitIdx<InHits.Num(); HitIdx++)
		{
			if(InHits[HitIdx].bBlockingHit)
			{
				NumBlocks++;
			}
		}
		return NumBlocks;
	}

	/** Static utility function that returns the number of overlapping hits in array. */
	static int32 GetNumOverlapHits(const TArray<FHitResult>& InHits)
	{
		return (InHits.Num() - GetNumBlockingHits(InHits));
	}

	/**
	 * Get a copy of the HitResult with relevant information reversed.
	 * For example when receiving a hit from another object, we reverse the normals.
	 */
	static FHitResult GetReversedHit(const FHitResult& Hit)
	{
		FHitResult Result(Hit);
		Result.Normal = -Result.Normal;
		Result.ImpactNormal = -Result.ImpactNormal;

		int32 TempItem = Result.Item;
		Result.Item = Result.MyItem;
		Result.MyItem = TempItem;

		FName TempBoneName = Result.BoneName;
		Result.BoneName = Result.MyBoneName;
		Result.MyBoneName = TempBoneName;
		return Result;
	}

	ENGINE_API FString ToString() const;
};

// All members of FHitResult are PODs.
template<> struct TIsPODType<FHitResult> { enum { Value = true }; };

template<>
struct TStructOpsTypeTraits<FHitResult> : public TStructOpsTypeTraitsBase2<FHitResult>
{
	enum
	{
		WithNetSerializer = true,
	};
};
