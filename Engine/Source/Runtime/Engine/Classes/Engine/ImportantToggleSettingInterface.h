// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// ImportantToogleSettingInterface.h
//
// Interface for settings classes containing a single, emphasized boolean value.
// Implement this interface to make a settings category containing one boolean
// property lots of extra info for the user.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "ImportantToggleSettingInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UImportantToggleSettingInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IImportantToggleSettingInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual void GetToggleCategoryAndPropertyNames(FName& OutCategory, FName& OutProperty) const { OutCategory = NAME_None; OutProperty = NAME_None; }

	virtual FText GetFalseStateLabel() const { return FText::GetEmpty(); }
	virtual FText GetFalseStateTooltip() const { return FText::GetEmpty(); }
	virtual FText GetFalseStateDescription() const { return FText::GetEmpty(); }
	virtual FText GetTrueStateLabel() const { return FText::GetEmpty(); }
	virtual FText GetTrueStateTooltip() const { return FText::GetEmpty(); }
	virtual FText GetTrueStateDescription() const { return FText::GetEmpty(); }
	virtual FString GetAdditionalInfoUrl() const { return FString(); }
	virtual FText GetAdditionalInfoUrlLabel() const { return FText::GetEmpty(); }
};
