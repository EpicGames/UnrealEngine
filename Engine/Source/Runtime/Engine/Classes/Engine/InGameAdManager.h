// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This object is responsible for the display and callbacks associated
 * with handling ingame advertisements
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/PlatformInterfaceBase.h"
#include "InGameAdManager.generated.h"

UENUM()
enum EAdManagerDelegate : int
{
	AMD_ClickedBanner,
	AMD_UserClosedAd,
	AMD_MAX,
};

/**
 * Delegate called when user clicks on an banner ad. Base class already handles pausing,
 * so a delegate is only needed if you need extra handling. If you set it to a PC or other actor
 * function, make sure to set it back when switching levels.
 */
DECLARE_DYNAMIC_DELEGATE( FOnUserClickedBanner );
/**
 * Delegate called when user closes an ad (after clicking on banner). Base class already handles 
 * pausing, so a delegate is only needed if you need extra handling.  If you set it to a PC or other actor
 * function, make sure to set it back when switching levels.
 */
DECLARE_DYNAMIC_DELEGATE( FOnUserClosedAdvertisement );


UCLASS()
class UInGameAdManager : public UPlatformInterfaceBase
{
	GENERATED_UCLASS_BODY()

	/** If true, the game will pause when the user clicks on the ad, which could take over the screen */
	UPROPERTY()
	uint32 bShouldPauseWhileAdOpen:1;

	/** @todo document */
	UPROPERTY()
	TArray<FOnUserClickedBanner> ClickedBannerDelegates;

	/** @todo document */
	UPROPERTY()
	TArray<FOnUserClosedAdvertisement> ClosedAdDelegates;


	/**
	 * Called by platform when the user clicks on the ad banner. Will pause the game before
	 * calling the delegates
	 */
	void OnUserClickedBanner();

	/**
	 * Called by platform when an opened ad is closed. Will unpause the game before
	 * calling the delegates
	 */
	void OnUserClosedAd();

	/** Perform any needed initialization */
	virtual void Init();
	
	/**
	 * Allows the platform to put up an advertisement on top of the viewport. Note that 
	 * this will not resize the viewport, simply cover up a portion of it.
	 *
	 * @param bShowOnBottomOfScreen If true, advertisement will be shown on the bottom, otherwise, the top
	 */
	virtual void ShowBanner(bool bShowBottomOfScreen);
	
	/**
	 * Hides the advertisement banner shown with ShowInGameAdvertisementBanner. If the ad is currently open
	 * (ie, the user is interacting with the ad), the ad will be forcibly closed (see ForceCloseInGameAdvertisement)
	 */
	virtual void HideBanner();

	/**
	 * If the game absolutely must close an opened (clicked on) advertisement, call this function.
	 * This may lead to loss of revenue, so don't do it unnecessarily.
	 */
	virtual void ForceCloseAd();
	
	/**
	 * Sets the value of bShouldPauseWhileAdOpen
	 */
	virtual void SetPauseWhileAdOpen(bool bShouldPause);
	
	/**
	 * Adds a delegate to the list of listeners
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	virtual void AddClickedBannerDelegate(FOnUserClickedBanner InDelegate);
	
	/**
	 * Removes a delegate from the list of listeners
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	virtual void ClearClickedBannerDelegate(FOnUserClickedBanner InDelegate);
	
	/**
	 * Adds a delegate to the list of listeners
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	virtual void AddClosedAdDelegate(FOnUserClosedAdvertisement InDelegate);
	
	/**
	 * Removes a delegate from the list of listeners
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	virtual void ClearClosedAdDelegate(FOnUserClosedAdvertisement InDelegate);
};



