// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "InheritableComponentHandler.generated.h"

class UActorComponent;
class UBlueprint;
class USCS_Node;
struct FUCSComponentId;

USTRUCT()
struct FComponentKey
{
	GENERATED_USTRUCT_BODY()

	FComponentKey()
		: OwnerClass(nullptr)
	{}

	ENGINE_API FComponentKey(const USCS_Node* SCSNode);
#if WITH_EDITOR
	ENGINE_API FComponentKey(UBlueprint* Blueprint, const FUCSComponentId& UCSComponentID);
#endif 

	ENGINE_API bool Match(const FComponentKey& OtherKey) const;

	bool IsSCSKey() const
	{
		return (SCSVariableName != NAME_None) && AssociatedGuid.IsValid();
	}

	bool IsUCSKey() const
	{
		return AssociatedGuid.IsValid() && (SCSVariableName == NAME_None);
	}

	bool IsValid() const
	{
		return OwnerClass && AssociatedGuid.IsValid() && (!IsSCSKey() || (SCSVariableName != NAME_None));
	}

	ENGINE_API USCS_Node* FindSCSNode() const;
	ENGINE_API UActorComponent* GetOriginalTemplate(const FName& TemplateName = NAME_None) const;
	ENGINE_API bool RefreshVariableName();

	UClass* GetComponentOwner()  const { return OwnerClass; }
	FName   GetSCSVariableName() const { return SCSVariableName; }
	FGuid   GetAssociatedGuid()  const { return AssociatedGuid; }

private: 
	UPROPERTY()
	TObjectPtr<UClass> OwnerClass;

	UPROPERTY()
	FName SCSVariableName;

	UPROPERTY()
	FGuid AssociatedGuid;
};

USTRUCT()
struct FComponentOverrideRecord
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UClass> ComponentClass;

	UPROPERTY()
	TObjectPtr<UActorComponent> ComponentTemplate;

	UPROPERTY()
	FComponentKey ComponentKey;

	UPROPERTY()
	FBlueprintCookedComponentInstancingData CookedComponentInstancingData;

	FComponentOverrideRecord()
		: ComponentClass(nullptr)
		, ComponentTemplate(nullptr)
	{}
};

UCLASS(MinimalAPI)
class UInheritableComponentHandler : public UObject
{
	GENERATED_BODY()

private:

	/* Template name prefix for SCS DefaultSceneRootNode overrides */
	static ENGINE_API const FString SCSDefaultSceneRootOverrideNamePrefix;

#if WITH_EDITOR

	ENGINE_API bool IsRecordValid(const FComponentOverrideRecord& Record) const;
	ENGINE_API bool IsRecordNecessary(const FComponentOverrideRecord& Record) const;

public:

	ENGINE_API UActorComponent* CreateOverridenComponentTemplate(const FComponentKey& Key);
	ENGINE_API void RemoveOverridenComponentTemplate(const FComponentKey& Key);
	ENGINE_API void UpdateOwnerClass(UBlueprintGeneratedClass* OwnerClass);
	ENGINE_API void ValidateTemplates();
	ENGINE_API bool IsValid() const;
	ENGINE_API UActorComponent* FindBestArchetype(const FComponentKey& Key, const FName& TemplateName = NAME_None) const;

	bool IsEmpty() const
	{
		return 0 == Records.Num();
	}

	ENGINE_API bool RefreshTemplateName(const FComponentKey& OldKey);

	ENGINE_API FComponentKey FindKey(UActorComponent* ComponentTemplate) const;
#endif

public:

	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif	// WITH_EDITOR
	//~ End UObject Interface

	ENGINE_API void PreloadAllTemplates();
	ENGINE_API void PreloadAll();

	ENGINE_API FComponentKey FindKey(const FName VariableName) const;

	ENGINE_API UActorComponent* GetOverridenComponentTemplate(const FComponentKey& Key) const;
	ENGINE_API const FBlueprintCookedComponentInstancingData* GetOverridenComponentTemplateData(const FComponentKey& Key) const;

	TArray<FComponentOverrideRecord>::TIterator CreateRecordIterator()
	{
		return Records.CreateIterator();
	}

	void GetAllTemplates(TArray<UActorComponent*>& OutArray, bool bIncludeTransientTemplates = false) const
	{
		OutArray.Reserve(OutArray.Num() + Records.Num() + (bIncludeTransientTemplates ? UnnecessaryComponents.Num() : 0));
		for (const FComponentOverrideRecord& Record : Records)
		{
			OutArray.Add(Record.ComponentTemplate);
		}

		if (bIncludeTransientTemplates)
		{
			OutArray.Append(UnnecessaryComponents);
		}
	}

private:
	ENGINE_API const FComponentOverrideRecord* FindRecord(const FComponentKey& Key) const;

	/** Helper method used to assist with fixing up component template names at load time. */
	ENGINE_API void FixComponentTemplateName(UActorComponent* ComponentTemplate, FName NewName);
	
	void FixComponentTemplateNames();

	/** All component records */
	UPROPERTY()
	TArray<FComponentOverrideRecord> Records;

	/** List of components that were marked unnecessary, need to keep these around so it doesn't regenerate them when a child asks for one */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UActorComponent>> UnnecessaryComponents;
};
