// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/InputDelegateBinding.h"
#include "InputActionDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputActionDelegateBinding : public FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName InputActionName;

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintInputActionDelegateBinding()
		: FBlueprintInputDelegateBinding()
		, InputActionName(NAME_None)
		, InputKeyEvent(IE_Pressed)
		, FunctionNameToBind(NAME_None)
	{
	}
};

UCLASS(MinimalAPI)
class UInputActionDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintInputActionDelegateBinding> InputActionDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
