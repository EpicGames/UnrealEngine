// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/InputDelegateBinding.h"
#include "InputAxisDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputAxisDelegateBinding : public FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName InputAxisName;

	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintInputAxisDelegateBinding()
		: FBlueprintInputDelegateBinding()
		, InputAxisName(NAME_None)
		, FunctionNameToBind(NAME_None)
	{
	}
};

UCLASS(MinimalAPI)
class UInputAxisDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintInputAxisDelegateBinding> InputAxisDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
