// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Engine/InputDelegateBinding.h"
#include "InputAxisKeyDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputAxisKeyDelegateBinding : public FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FKey AxisKey;

	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintInputAxisKeyDelegateBinding()
		: FBlueprintInputDelegateBinding()
		, FunctionNameToBind(NAME_None)
	{
	}
};

UCLASS(MinimalAPI)
class UInputAxisKeyDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintInputAxisKeyDelegateBinding> InputAxisKeyDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
