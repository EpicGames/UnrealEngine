// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "InputDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint32 bConsumeInput:1;

	UPROPERTY()
	uint32 bExecuteWhenPaused:1;

	UPROPERTY()
	uint32 bOverrideParentBinding:1;

	FBlueprintInputDelegateBinding()
		: bConsumeInput(true)
		, bExecuteWhenPaused(false)
		, bOverrideParentBinding(true)
	{
	}
};

UCLASS(abstract, MinimalAPI)
class UInputDelegateBinding : public UDynamicBlueprintBinding
{
	GENERATED_UCLASS_BODY()
	
	/**
	 * Override this function to bind a delegate to the given input component.
	 *
	 * @param InputComponent		The InputComponent to Bind a delegate to
	 * @param ObjectToBindTo		The UObject that the binding should use.
	 */
	virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const { };

	/** Returns true if the given class supports input binding delegates (i.e. it is a BP generated class) */
	UE_DEPRECATED(5.6, "SupportsInputDelegate is deprecated. All UClasses will be considered valid for input delegate support.")
	static ENGINE_API bool SupportsInputDelegate(const UClass* InClass);

	/**
	 * Calls BindToInputComponent for each dynamic binding object on the given Class if it supports input delegates.
	 *
	 * @param InClass				The class that will should be used to determine if Input Delegates are supported
	 * @param InputComponent		The InputComponent to Bind a delegate to
	 * @param ObjectToBindTo		The UObject that the binding should use. If this is null, the Owner of the input componet will be used.
	 */
	static ENGINE_API void BindInputDelegates(const UClass* InClass, UInputComponent* InputComponent, UObject* ObjectToBindTo = nullptr);
	
	/**
	 * Will bind input delegates for the given Actor and traverse it's subobjects attempting to bind
	 * each of them
	 */
	static ENGINE_API void BindInputDelegatesWithSubojects(AActor* InActor, UInputComponent* InputComponent);

protected:
	static ENGINE_API TSet<UClass*> InputBindingClasses;
};
