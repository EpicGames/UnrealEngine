// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/InputDelegateBinding.h"
#include "Framework/Commands/InputChord.h"
#include "InputKeyDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputKeyDelegateBinding : public FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FInputChord InputChord;

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintInputKeyDelegateBinding()
		: FBlueprintInputDelegateBinding()
		, InputKeyEvent(IE_Pressed)
		, FunctionNameToBind(NAME_None)
	{
	}
};

UCLASS(MinimalAPI)
class UInputKeyDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintInputKeyDelegateBinding> InputKeyDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
