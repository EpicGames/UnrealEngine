// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/InputDelegateBinding.h"
#include "InputTouchDelegateBinding.generated.h"

class UInputComponent;

USTRUCT()
struct FBlueprintInputTouchDelegateBinding : public FBlueprintInputDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	FName FunctionNameToBind;

	FBlueprintInputTouchDelegateBinding()
		: FBlueprintInputDelegateBinding()
		, InputKeyEvent(IE_Pressed)
		, FunctionNameToBind(NAME_None)
	{
	}
};


UCLASS(MinimalAPI)
class UInputTouchDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintInputTouchDelegateBinding> InputTouchDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
