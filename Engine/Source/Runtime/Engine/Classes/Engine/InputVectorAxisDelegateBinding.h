// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "InputVectorAxisDelegateBinding.generated.h"

class UInputComponent;

UCLASS(MinimalAPI)
class UInputVectorAxisDelegateBinding : public UInputAxisKeyDelegateBinding
{
	GENERATED_UCLASS_BODY()

	//~ Begin UInputDelegateBinding Interface
	ENGINE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};
