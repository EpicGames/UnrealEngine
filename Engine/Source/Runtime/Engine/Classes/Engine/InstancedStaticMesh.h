// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InstancedStaticMesh.h: Instanced static mesh header
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RayTracingGeometry.h"
#include "PrimitiveViewRelevance.h"
#include "ShaderParameters.h"
#include "SceneView.h"
#include "VertexFactory.h"
#include "LocalVertexFactory.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include "StaticMeshLight.h"

#if WITH_EDITOR
#include "LightMap.h"
#include "ShadowMap.h"
#endif

class ULightComponent;
struct FInstancedStaticMeshSceneProxyDesc;

extern TAutoConsoleVariable<float> CVarFoliageMinimumScreenSize;
extern TAutoConsoleVariable<float> CVarRandomLODRange;
extern TAutoConsoleVariable<int32> CVarMinLOD;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FInstancedStaticMeshVertexFactoryUniformShaderParameters, ENGINE_API)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_InstanceOriginBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_InstanceTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_InstanceLightmapBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, InstanceCustomDataBuffer)
	SHADER_PARAMETER(int32, NumCustomDataFloats)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FInstancedStaticMeshVFLooseUniformShaderParameters, ENGINE_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FLocalVertexFactoryInstanceDitherParameters, InstanceDitherParameters)
	SHADER_PARAMETER(FVector4f, InstancingFadeOutParams)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FInstancedStaticMeshVFLooseUniformShaderParameters> FInstancedStaticMeshVFLooseUniformShaderParametersRef;

// This must match the maximum a user could specify in the material (see 
// FHLSLMaterialTranslator::TextureCoordinate), otherwise the material will attempt 
// to look up a texture coordinate we didn't provide an element for.
extern const int32 InstancedStaticMeshMaxTexCoord;

/*-----------------------------------------------------------------------------
	FStaticMeshInstanceBuffer
-----------------------------------------------------------------------------*/

class FStaticMeshInstanceBuffer : public FRenderResource
{
public:

	/** Default constructor. */
	ENGINE_API FStaticMeshInstanceBuffer(ERHIFeatureLevel::Type InFeatureLevel, bool InRequireCPUAccess);

	/** Destructor. */
	ENGINE_API ~FStaticMeshInstanceBuffer();

	/**
	 * Initializes the buffer with the component's data.
	 * @param Other - instance data, this call assumes the memory, so this will be empty after the call
	 */
	ENGINE_API void InitFromPreallocatedData(FStaticMeshInstanceData& Other);

	/**
	 * Specialized assignment operator, only used when importing LOD's. 
	 */
	ENGINE_API void operator=(const FStaticMeshInstanceBuffer &Other);

	// Other accessors.
	inline uint32 GetNumInstances() const
	{
		return InstanceData->GetNumInstances();
	}

	inline void GetInstanceTransform(int32 InstanceIndex, FRenderTransform& Transform) const
	{
		InstanceData->GetInstanceTransform(InstanceIndex, Transform);
	}

	inline void GetInstanceRandomID(int32 InstanceIndex, float& RandomInstanceID) const
	{
		InstanceData->GetInstanceRandomID(InstanceIndex, RandomInstanceID);
	}

#if WITH_EDITOR
	inline void GetInstanceEditorData(int32 InstanceIndex, FColor& HitProxyColorOut, bool& bSelectedOut) const
	{
		InstanceData->GetInstanceEditorData(InstanceIndex, HitProxyColorOut, bSelectedOut);
	}
#endif 


	inline void GetInstanceLightMapData(int32 InstanceIndex, FVector4f& InstanceLightmapAndShadowMapUVBias) const
	{
		InstanceData->GetInstanceLightMapData(InstanceIndex, InstanceLightmapAndShadowMapUVBias);
	}
	
	inline void GetInstanceCustomDataValues(int32 InstanceIndex, TArray<float>& InstanceCustomData) const
	{
		InstanceData->GetInstanceCustomDataValues(InstanceIndex, InstanceCustomData);
	}
	
	inline FStaticMeshInstanceData* GetInstanceData() const
	{
		return InstanceData.Get();
	}

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	ENGINE_API virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseResource() override;
	virtual FString GetFriendlyName() const override { return TEXT("Static-mesh instances"); }
	SIZE_T GetResourceSize() const;

	ENGINE_API void BindInstanceVertexBuffer(const class FVertexFactory* VertexFactory, struct FInstancedStaticMeshDataType& InstancedStaticMeshData) const;

	/**
	 * Call to flush any pending GPU data copies, if bFlushToGPUPending is false it does nothing. Should be called by the Proxy on the render thread
	 * for example in CreateRenderThreadResources().
	 */
	void FlushGPUUpload(FRHICommandListBase& RHICmdList);

public:
	/** The vertex data storage type */
	TSharedPtr<FStaticMeshInstanceData, ESPMode::ThreadSafe> InstanceData;

	/** Keep CPU copy of instance data */
	bool RequireCPUAccess;

	FBufferRHIRef GetInstanceOriginBuffer()
	{
		check(!bFlushToGPUPending);
		return InstanceOriginBuffer.VertexBufferRHI;
	}

	FBufferRHIRef GetInstanceTransformBuffer()
	{
		check(!bFlushToGPUPending);
		return InstanceTransformBuffer.VertexBufferRHI;
	}

	FBufferRHIRef GetInstanceLightmapBuffer()
	{
		check(!bFlushToGPUPending);
		return InstanceLightmapBuffer.VertexBufferRHI;
	}

	/**
	 * Set flush to GPU as pending.
	 */
	void SetFlushToGPUPending()
	{
			bFlushToGPUPending = true;
		}
private:

	/** If true, then we have updates to the host data not yet committed to the GPU. This in turn means
	 * that bDeferGPUUpload is true, and the Proxy is expected to either call FlushGPUUpload() OR never 
	 * use the instance data buffers (either is fine).
	 */
	bool bFlushToGPUPending;

	class FInstanceOriginBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FInstanceOriginBuffer"); }
	} InstanceOriginBuffer;
	FShaderResourceViewRHIRef InstanceOriginSRV;

	class FInstanceTransformBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FInstanceTransformBuffer"); }
	} InstanceTransformBuffer;
	FShaderResourceViewRHIRef InstanceTransformSRV;

	class FInstanceLightmapBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FInstanceLightmapBuffer"); }
	} InstanceLightmapBuffer;
	FShaderResourceViewRHIRef InstanceLightmapSRV;

	class FInstanceCustomDataBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FInstanceCustomDataBuffer"); }
	} InstanceCustomDataBuffer;
	FShaderResourceViewRHIRef InstanceCustomDataSRV;	

	/** Delete existing resources */
	void CleanUp();

	void CreateVertexBuffer(FRHICommandListBase& RHICmdList, FResourceArrayInterface* InResourceArray, EBufferUsageFlags InUsage, uint32 InStride, uint8 InFormat, FBufferRHIRef& OutVertexBufferRHI, FShaderResourceViewRHIRef& OutInstanceSRV);
};

/*-----------------------------------------------------------------------------
	FInstancedStaticMeshVertexFactory
-----------------------------------------------------------------------------*/

struct FInstancingUserData
{
	class FInstancedStaticMeshRenderData* RenderData;
	class FStaticMeshRenderData* MeshRenderData;

	int32 MinDrawDistance;
	int32 StartCullDistance;
	int32 EndCullDistance;

	float LODDistanceScale;

	int32 MinLOD;

	FVector AverageInstancesScale;
	FVector InstancingOffset;
};

struct FInstancedStaticMeshDataType
{
	/** The stream to read the mesh transform from. */
	FVertexStreamComponent InstanceOriginComponent;

	/** The stream to read the mesh transform from. */
	FVertexStreamComponent InstanceTransformComponent[3];

	/** The stream to read the Lightmap Bias and Random instance ID from. */
	FVertexStreamComponent InstanceLightmapAndShadowMapUVBiasComponent;

	FRHIShaderResourceView* InstanceOriginSRV = nullptr;
	FRHIShaderResourceView* InstanceTransformSRV = nullptr;
	FRHIShaderResourceView* InstanceLightmapSRV = nullptr;
	FRHIShaderResourceView* InstanceCustomDataSRV = nullptr;

	int32 NumCustomDataFloats = 0;
};

/**
 * A vertex factory for instanced static meshes
 */
struct FInstancedStaticMeshVertexFactory : public FLocalVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FInstancedStaticMeshVertexFactory, ENGINE_API);
public:
	FInstancedStaticMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLocalVertexFactory(InFeatureLevel, "FInstancedStaticMeshVertexFactory")
	{
	}

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Modify compile environment to enable instancing
	 * @param OutEnvironment - shader compile environment to modify
	 */
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Get vertex elements used when during PSO precaching materials using this vertex factory type
	 */
	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FDataType& Data, FInstancedStaticMeshDataType& InstanceData, FVertexDeclarationElementList& Elements);

	static ENGINE_API void InitInstancedStaticMeshVertexFactoryComponents(
		const FStaticMeshVertexBuffers& VertexBuffers,
		const FColorVertexBuffer* ColorVertexBuffer,
		const FStaticMeshInstanceBuffer* InstanceBuffer,
		const FInstancedStaticMeshVertexFactory* VertexFactory,
		int32 LightMapCoordinateIndex,
		bool bRHISupportsManualVertexFetch,
		FInstancedStaticMeshVertexFactory::FDataType& OutData,
		FInstancedStaticMeshDataType& OutInstanceData);

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	void SetData(FRHICommandListBase& RHICmdList, const FDataType& InData, const FInstancedStaticMeshDataType* InInstanceData)
	{
		Data = InData;
		if (InInstanceData)
		{
			InstanceData = *InInstanceData;
		}
		UpdateRHI(RHICmdList);
	}

	/**
	 * Copy the data from another vertex factory
	 * @param Other - factory to copy from
	 */
	ENGINE_API void Copy(const FInstancedStaticMeshVertexFactory& Other);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Make sure we account for changes in the signature of GetStaticBatchElementVisibility() */
	static constexpr uint32 NumBitsForVisibilityMask()
	{		
		return 8 * sizeof(uint64);
	}

	inline FRHIShaderResourceView* GetInstanceOriginSRV() const
	{
		return InstanceData.InstanceOriginSRV;
	}

	inline FRHIShaderResourceView* GetInstanceTransformSRV() const
	{
		return InstanceData.InstanceTransformSRV;
	}

	inline FRHIShaderResourceView* GetInstanceLightmapSRV() const
	{
		return InstanceData.InstanceLightmapSRV;
	}

	inline FRHIShaderResourceView* GetInstanceCustomDataSRV() const
	{
		return InstanceData.InstanceCustomDataSRV;
	}

	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer.GetReference();
	}
protected:
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FDataType& Data, FInstancedStaticMeshDataType& InstanceData, FVertexDeclarationElementList& Elements, FVertexStreamList& Streams);

private:
	FInstancedStaticMeshDataType InstanceData;

	TUniformBufferRef<FInstancedStaticMeshVertexFactoryUniformShaderParameters> UniformBuffer;
};

class FInstancedStaticMeshVertexFactoryShaderParameters : public FLocalVertexFactoryShaderParametersBase
{
	DECLARE_TYPE_LAYOUT(FInstancedStaticMeshVertexFactoryShaderParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		FLocalVertexFactoryShaderParametersBase::Bind(ParameterMap);

		InstancingOffsetParameter.Bind(ParameterMap, TEXT("InstancingOffset"));
		VertexFetch_InstanceOriginBufferParameter.Bind(ParameterMap, TEXT("VertexFetch_InstanceOriginBuffer"));
		VertexFetch_InstanceTransformBufferParameter.Bind(ParameterMap, TEXT("VertexFetch_InstanceTransformBuffer"));
		VertexFetch_InstanceLightmapBufferParameter.Bind(ParameterMap, TEXT("VertexFetch_InstanceLightmapBuffer"));
		InstanceOffset.Bind(ParameterMap, TEXT("InstanceOffset"));
	}

	ENGINE_API void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
		) const;

private:
	LAYOUT_FIELD(FShaderParameter, InstancingOffsetParameter);
	LAYOUT_FIELD(FShaderResourceParameter, VertexFetch_InstanceOriginBufferParameter)
	LAYOUT_FIELD(FShaderResourceParameter, VertexFetch_InstanceTransformBufferParameter)
	LAYOUT_FIELD(FShaderResourceParameter, VertexFetch_InstanceLightmapBufferParameter)
	LAYOUT_FIELD(FShaderParameter, InstanceOffset)
};

/*-----------------------------------------------------------------------------
	FInstancedStaticMeshRenderData
-----------------------------------------------------------------------------*/

	/**
 * Container for vertex factories used in the proxy to link MDC to the attribute buffers and similar data.
	 */
class FInstancedStaticMeshRenderData
{
public:

	ENGINE_API FInstancedStaticMeshRenderData(const FInstancedStaticMeshSceneProxyDesc* InDesc, ERHIFeatureLevel::Type InFeatureLevel);

	ENGINE_API void ReleaseResources(FSceneInterface* Scene, const UStaticMesh* StaticMesh);

	/** Source component */
	// @todo: remove and use IPrimitiveComponentInterface* when we add support for static lighting through that path
	UInstancedStaticMeshComponent* Component;

	/** Cache off some component data. */
	int32 LightMapCoordinateIndex;

	/** Vertex factory */
	TIndirectArray<FInstancedStaticMeshVertexFactory> VertexFactories;

	/** LOD render data from the static mesh. */
	FStaticMeshLODResourcesArray& LODModels;

	TIndirectArray<FInstancedStaticMeshVertexFactory> RayTracingProxyVertexFactories;
	const FStaticMeshRayTracingProxyLODArray* RayTracingProxyLODs = nullptr;

	/** Feature level used when creating instance data */
	ERHIFeatureLevel::Type FeatureLevel;

	ENGINE_API void BindBuffersToVertexFactories(FRHICommandListBase& RHICmdList, FStaticMeshInstanceBuffer* InstanceBuffer);

private:
	void InitVertexFactories();
	void RegisterSpeedTreeWind(const FInstancedStaticMeshSceneProxyDesc* InProxyDesc);
};


/*-----------------------------------------------------------------------------
	FInstancedStaticMeshSceneProxy
-----------------------------------------------------------------------------*/

struct FInstancedStaticMeshSceneProxyDesc;

class FInstancedStaticMeshSceneProxy : public FStaticMeshSceneProxy
{
public:
	ENGINE_API SIZE_T GetTypeHash() const override;

	/** Initialization constructor. */
	ENGINE_API FInstancedStaticMeshSceneProxy(UInstancedStaticMeshComponent* InComponent, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API FInstancedStaticMeshSceneProxy(const FInstancedStaticMeshSceneProxyDesc& InDesc, ERHIFeatureLevel::Type InFeatureLevel);

	~FInstancedStaticMeshSceneProxy() = default;

	// FPrimitiveSceneProxy interface.
	ENGINE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;

	ENGINE_API virtual void DestroyRenderThreadResources() override;

	ENGINE_API virtual void UpdateInstances_RenderThread(FRHICommandListBase& RHICmdList, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds) override;

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		if(View->Family->EngineShowFlags.InstancedStaticMeshes)
		{
			Result = FStaticMeshSceneProxy::GetViewRelevance(View);
#if WITH_EDITOR
			// use dynamic path to render selected indices
			if( bHasSelectedInstances )
			{
				Result.bDynamicRelevance = true;
				Result.bStaticRelevance = false;
			}
#endif
		}
		return Result;
	}

	bool bAnySegmentUsesWorldPositionOffset = false;
	bool bUseGpuLodSelection = false;

	ENGINE_API virtual void SetEvaluateWorldPositionOffsetInRayTracing(FRHICommandListBase& RHICmdList, bool bEvaluate) override;

#if RHI_RAYTRACING
	ENGINE_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) final override;
#endif

	ENGINE_API virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override;
	ENGINE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	virtual int32 GetNumMeshBatches() const override
	{
		return 1;
	}

	/** Sets up a shadow FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const override;

	/** Sets up a FMeshBatch for a specific LOD and element. */
	ENGINE_API virtual bool GetMeshElement(int32 LODIndex, int32 BatchIndex, int32 ElementIndex, uint8 InDepthPriorityGroup, bool bUseSelectionOutline, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const override;

	/** Sets up a wireframe FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const override;

	/** Sets up a ray tracing FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const;

	ENGINE_API virtual void GetDistanceFieldAtlasData(const FDistanceFieldVolumeData*& OutDistanceFieldData, float& SelfShadowBias) const override;

	/**
	 * Creates the hit proxies are used when DrawDynamicElements is called.
	 * Called in the game thread.
	 * @param OutHitProxies - Hit proxies which are created should be added to this array.
	 * @return The hit proxy to use by default for elements drawn by DrawDynamicElements.
	 */
	ENGINE_API virtual HHitProxy* CreateHitProxies(IPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
	ENGINE_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;

	ENGINE_API virtual bool GetInstanceDrawDistanceMinMax(FVector2f& OutDistanceMinMax) const override;
	ENGINE_API virtual bool GetInstancingFadeOutParams(FVector2f& OutInstancingFadeOutParams) const override;

	ENGINE_API virtual float GetLodScreenSizeScale() const override;
	ENGINE_API virtual float GetGpuLodInstanceRadius() const override;
	ENGINE_API virtual FInstanceDataUpdateTaskInfo *GetInstanceDataUpdateTaskInfo() const override;

	virtual bool IsDetailMesh() const override { return true; }

	ENGINE_API virtual void SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance) override;

	ENGINE_API virtual void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const override;

protected:
	ENGINE_API FLocalVertexFactoryInstanceDitherParameters GetInstanceDitherParameters(const FSceneView* View, uint32 InstancedLODRange, uint32 InstancedLODIndex) const;
	ENGINE_API FVector2f CalcInstancingFadeOutParams() const;
	ENGINE_API FInstancedStaticMeshVFLooseUniformShaderParametersRef CreateLooseUniformBuffer(const FSceneView* View, uint32 InstancedLODRange, uint32 InstancedLODIndex, EUniformBufferUsage UniformBufferUsage) const;

	/** Cache of the StaticMesh asset, needed to release SpeedTree resources*/
	UStaticMesh* StaticMesh;

	/** Per component render data */
	TUniquePtr<FInstancedStaticMeshRenderData> InstancedRenderData;

	/** Helper function to get the number of VFs in the render data */
	int32 GetNumRenderDataVertexFactories() const;

#if WITH_EDITOR
	/* If we we have any selected instances */
	bool bHasSelectedInstances;
#else
	static const bool bHasSelectedInstances = false;
#endif

	/** LOD transition info. */
	FInstancingUserData UserData_AllInstances;

	/** LOD distance scale from component. */
	float InstanceLODDistanceScale;

#if RHI_RAYTRACING
	ENGINE_API virtual void GetRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry) const override;

	ENGINE_API void SetupRayTracingDynamicInstances(FRHICommandListBase& RHICmdList, int32 NumDynamicInstances, int32 LODIndex);
	ENGINE_API void ReleaseRayTracingDynamicInstances();

	struct FRayTracingDynamicInstanceData
	{
		FRayTracingGeometry Geometry;
		FRWBuffer VertexBuffer;
	};

	TArray<FRayTracingDynamicInstanceData> RayTracingDynamicInstanceData;

	int32 CachedRayTracingLODIndex;
#endif

	/** Common path for the Get*MeshElement functions */
	ENGINE_API void SetupInstancedMeshBatch(int32 LODIndex, int32 BatchIndex, FMeshBatch& OutMeshBatch) const;
	ENGINE_API void SetupRayTracingInstancedMeshBatch(int32 LODIndex, int32 BatchIndex, FMeshBatch& OutMeshBatch) const;

	/** Untransformed bounds of the static mesh */
	FBoxSphereBounds StaticMeshBounds;
private:

	void SetupProxy(const FInstancedStaticMeshSceneProxyDesc& InProxyDesc);
	bool CanSetupInstancedMeshBatch(int32 LODIndex) const;
	bool CanSetupRayTracingInstancedMeshBatch(int32 LODIndex) const;

	/** Stores a loose uniform buffer per LOD, used for static view relevance. */
	TMap<uint32, FInstancedStaticMeshVFLooseUniformShaderParametersRef> LODLooseUniformBuffers;

	/** Stores a loose uniform buffer used for ray tracing. */
	FInstancedStaticMeshVFLooseUniformShaderParametersRef RayTracingLooseUniformBuffer;

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataSceneProxy; 
};

#if WITH_EDITOR
/*-----------------------------------------------------------------------------
	FInstancedStaticMeshStaticLightingMesh
-----------------------------------------------------------------------------*/

/**
 * A static lighting mesh class that transforms the points by the per-instance transform of an 
 * InstancedStaticMeshComponent
 */
class FStaticLightingMesh_InstancedStaticMesh : public FStaticMeshStaticLightingMesh
{
public:

	/** Initialization constructor. */
	FStaticLightingMesh_InstancedStaticMesh(const UInstancedStaticMeshComponent* InPrimitive, int32 LODIndex, int32 InstanceIndex, const TArray<ULightComponent*>& InRelevantLights)
		: FStaticMeshStaticLightingMesh(InPrimitive, LODIndex, InRelevantLights)
	{
		// override the local to world to combine the per instance transform with the component's standard transform
		SetLocalToWorld(InPrimitive->PerInstanceSMData[InstanceIndex].Transform * InPrimitive->GetComponentTransform().ToMatrixWithScale());
	}

	FStaticLightingMesh_InstancedStaticMesh() { }

	virtual bool IsInstancedMesh() const override { return true; }
};

/*-----------------------------------------------------------------------------
	FInstancedStaticMeshStaticLightingTextureMapping
-----------------------------------------------------------------------------*/


/** Represents a static mesh primitive with texture mapped static lighting. */
class FStaticLightingTextureMapping_InstancedStaticMesh : public FStaticMeshStaticLightingTextureMapping
{
public:
	/** Initialization constructor. */
	FStaticLightingTextureMapping_InstancedStaticMesh(UInstancedStaticMeshComponent* InPrimitive, int32 LODIndex, int32 InInstanceIndex, FStaticLightingMesh* InMesh, int32 InSizeX, int32 InSizeY, int32 InTextureCoordinateIndex, bool bPerformFullQualityRebuild)
		: FStaticMeshStaticLightingTextureMapping(InPrimitive, LODIndex, InMesh, InSizeX, InSizeY, InTextureCoordinateIndex, bPerformFullQualityRebuild)
		, InstanceIndex(InInstanceIndex)
		, QuantizedData(nullptr)
		, ShadowMapData()
		, bComplete(false)
	{
	}

	FStaticLightingTextureMapping_InstancedStaticMesh (const FArchive& Ar)	
		: FStaticMeshStaticLightingTextureMapping(Ar)
		, InstanceIndex(-1)
		, QuantizedData(nullptr)
		, ShadowMapData()
		, bComplete(false)
	{
		
	}

	// FStaticLightingTextureMapping interface
	virtual void Apply(FQuantizedLightmapData* InQuantizedData, const TMap<ULightComponent*, FShadowMapData2D*>& InShadowMapData, const FStaticLightingBuildContext* LightingContext) override
	{
		check(bComplete == false);

		UInstancedStaticMeshComponent* InstancedComponent = Cast<UInstancedStaticMeshComponent>(Primitive.Get());

		if (InstancedComponent)
		{
			// Save the static lighting until all of the component's static lighting has been built.
			QuantizedData = TUniquePtr<FQuantizedLightmapData>(InQuantizedData);
			ShadowMapData.Empty(InShadowMapData.Num());
			for (auto& ShadowDataPair : InShadowMapData)
			{
				ShadowMapData.Add(ShadowDataPair.Key, TUniquePtr<FShadowMapData2D>(ShadowDataPair.Value));
			}

			InstancedComponent->ApplyLightMapping(this, LightingContext);
		}

		bComplete = true;
	}

	virtual bool DebugThisMapping() const override
	{
		return false;
	}

	virtual FString GetDescription() const override
	{
		return FString("InstancedSMLightingMapping");
	}

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

private:
	friend class UInstancedStaticMeshComponent;

	/** The instance of the primitive this mapping represents. */
	int32 InstanceIndex;

	// Light/shadow map data stored until all instances for this component are processed
	// so we can apply them all into one light/shadowmap
	TUniquePtr<FQuantizedLightmapData> QuantizedData;
	TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>> ShadowMapData;

	/** Has this mapping already been completed? */
	bool bComplete;
};

#endif

/**
 * Structure that maps a component to it's lighting/instancing specific data which must be the same
 * between all instances that are bound to that component.
 */
struct FComponentInstanceSharingData
{
	/** The component that is associated (owns) this data */
	UInstancedStaticMeshComponent* Component;

	/** Light map texture */
	UTexture* LightMapTexture;

	/** Shadow map texture (or NULL if no shadow map) */
	UTexture* ShadowMapTexture;


	FComponentInstanceSharingData()
		: Component( NULL ),
		  LightMapTexture( NULL ),
		  ShadowMapTexture( NULL )
	{
	}
};


/**
 * Helper struct to hold information about what components use what lightmap textures
 */
struct FComponentInstancedLightmapData
{
	/** List of all original components and their original instances containing */
	TMap<UInstancedStaticMeshComponent*, TArray<FInstancedStaticMeshInstanceData> > ComponentInstances;

	/** List of new components */
	TArray< FComponentInstanceSharingData > SharingData;
};
