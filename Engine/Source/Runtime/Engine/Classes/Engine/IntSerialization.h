// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "IntSerialization.generated.h"

UCLASS(MinimalAPI)
class UIntSerialization	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
//	ENGINE_API UIntSerialization(const FObjectInitializer& ObjectInitializer, uint8 InUnsignedInt8Variable, uint16 InUnsignedInt16Variable, uint32 InUnsignedInt32Variable, uint64 InUnsignedInt64Variable, int8 InSignedInt8Variable, int16 InSignedInt16Variable, int InSignedInt32Variable, int64 InSignedInt64Variable);

//	virtual void Serialize (FArchive& Ar) override;
	/**
	 * Serializes the given int Int Serilization Test test from or into the specified archive.
	 *
	 * @param Ar - The archive to serialize from or into.
	 * @param FIntSerilizationTestObject - The Int Serilization Test value to serialize.
	 *
	 * @return The archive.
	 *
	 * @todo gmp: Figure out better include order in Core.h so this can be inlined.
	 */
//	friend CORE_API class FArchive& operator<< (class FArchive& Ar, FIntSerilizationTest& IntSerilizationTest);

	//New types
	UPROPERTY()
	uint16	UnsignedInt16Variable;

	UPROPERTY()
	uint32	UnsignedInt32Variable;

	UPROPERTY()
	uint64	UnsignedInt64Variable;

	UPROPERTY()
	int8	SignedInt8Variable;

	UPROPERTY()
	int16	SignedInt16Variable;

	UPROPERTY()
	int64	SignedInt64Variable;

	//Existing types
	UPROPERTY()
	uint8	UnsignedInt8Variable;

	UPROPERTY()
	int32		SignedInt32Variable;
};

/*FArchive& operator<< (FArchive& Ar, FIntSerilizationTest& IntSerilizationTest)
{
	return Ar << IntSerilizationTest.UnsignedInt16Variable;
}*/
