// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InterpCurveEdSetup.generated.h"

// Information about a particule curve being viewed.
// Property could be an FInterpCurve, a DistributionFloat or a DistributionVector
USTRUCT()
struct FCurveEdEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<class UObject> CurveObject;

	UPROPERTY()
	FColor CurveColor;

	UPROPERTY()
	FString CurveName;

	UPROPERTY()
	int32 bHideCurve;

	UPROPERTY()
	int32 bColorCurve;

	UPROPERTY()
	int32 bFloatingPointColorCurve;

	UPROPERTY()
	int32 bClamp;

	UPROPERTY()
	float ClampLow;

	UPROPERTY()
	float ClampHigh;


	FCurveEdEntry()
		: CurveObject(NULL)
		, CurveColor(ForceInit)
		, bHideCurve(0)
		, bColorCurve(0)
		, bFloatingPointColorCurve(0)
		, bClamp(0)
		, ClampLow(0)
		, ClampHigh(0)
	{
	}

};

USTRUCT()
struct FCurveEdTab
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString TabName;

	UPROPERTY()
	TArray<struct FCurveEdEntry> Curves;

	// Remember the view setting for each tab.
	UPROPERTY()
	float ViewStartInput;

	UPROPERTY()
	float ViewEndInput;

	UPROPERTY()
	float ViewStartOutput;

	UPROPERTY()
	float ViewEndOutput;


	FCurveEdTab()
		: ViewStartInput(0)
		, ViewEndInput(0)
		, ViewStartOutput(0)
		, ViewEndOutput(0)
	{
	}

	FCurveEdTab(FString InTabName, float InViewStartInput, float InViewEndInput, float InViewStartOutput, float InViewEndOutput)
		: TabName(InTabName)
		, ViewStartInput(InViewStartInput)
		, ViewEndInput(InViewEndInput)
		, ViewStartOutput(InViewStartOutput)
		, ViewEndOutput(InViewEndOutput)
	{
	}
};

UCLASS(MinimalAPI)
class UInterpCurveEdSetup : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<struct FCurveEdTab> Tabs;

	UPROPERTY()
	int32 ActiveTab;


	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin InterpCurveEdSetup Interface
	ENGINE_API static FCurveEdInterface* GetCurveEdInterfacePointer(const FCurveEdEntry& Entry);

	/** 
	 * Add a new curve property to the current tab.
	 *
	 * @param	OutCurveEntry	The curves which are for this graph node
	 *
	 * @return	true, if new curves were added to the graph, otherwise they were already present
	 */
	ENGINE_API bool AddCurveToCurrentTab(UObject* InCurve, const FString& CurveName, const FColor& CurveColor, 
			FCurveEdEntry** OutCurveEntry=NULL, bool bInColorCurve=false, bool bInFloatingPointColor=false,
			bool bInClamp=false, float InClampLow=0.f, float InClampHigh=0.f);

	/** Remove a particuclar curve from all tabs. */
	ENGINE_API void RemoveCurve(UObject* InCurve);

	/** Replace a particuclar curve */
	ENGINE_API void ReplaceCurve(UObject* RemoveCurve, UObject* AddCurve);

	/** Create a new tab in the CurveEdSetup. */
	ENGINE_API void CreateNewTab(const FString& InTabName);

	/** Remove the tab of the given name from the CurveEdSetup. */
	ENGINE_API void RemoveTab(const FString& InTabName);
	
	/** Look through CurveEdSetup and see if any properties of selected object is being shown. */
	ENGINE_API bool ShowingCurve(UObject* InCurve);

	/** Change the color of the given curve */
	ENGINE_API void ChangeCurveColor(UObject* InCurve, const FColor& CurveColor);
	
	/** Change the displayed name for a curve in the curve editor. */
	ENGINE_API void ChangeCurveName(UObject* InCurve, const FString& NewCurveName);

	/** Remove all tabs and re-add the 'default' one */
	void ResetTabs();

};



