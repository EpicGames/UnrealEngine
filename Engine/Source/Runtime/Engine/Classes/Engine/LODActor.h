// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "HAL/IConsoleManager.h"
#include "Engine/MaterialMerging.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "LODActor.generated.h"

class UStaticMesh;
class UHLODProxy;
class UHLODProxyDesc;

extern ENGINE_API TAutoConsoleVariable<FString> CVarHLODDistanceOverride;

/**
 * Mesh/Material pair used as a key to insert/retrieve instances in the LODActor.
 */
USTRUCT()
struct FHLODInstancingKey
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<const UStaticMesh> StaticMesh;

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> Material;

	FHLODInstancingKey()
		: StaticMesh()
		, Material()
	{}

	FHLODInstancingKey(const UStaticMesh* InStaticMesh, const UMaterialInterface* InMaterial)
		: StaticMesh(InStaticMesh)
		, Material(InMaterial)
	{}

	bool operator==(const FHLODInstancingKey& Other) const
	{
		return StaticMesh == Other.StaticMesh && Material == Other.Material;
	}

	bool IsValid() const
	{
		return StaticMesh != nullptr && Material != nullptr;
	}

	friend uint32 GetTypeHash(const FHLODInstancingKey& Key)
	{
		return HashCombine(GetTypeHash(Key.StaticMesh), GetTypeHash(Key.Material));
	}
};

/**
 * LODActor is an instance of an autogenerated StaticMesh Actors by Hierarchical LOD System
 * This is essentially just StaticMeshActor that you can't move or edit, but it contains multiple actors reference
 *
 * @see https://docs.unrealengine.com/BuildingWorlds/HLOD
 * @see UStaticMesh
 */

UCLASS(notplaceable, hidecategories = (Object, Collision, Display, Input, Blueprint, Transform, Physics), MinimalAPI)
class ALODActor : public AActor
{
	GENERATED_UCLASS_BODY()

	friend class UHLODProxy;
	friend class UHLODProxyDesc;

private:
	// disable display of this component
	UPROPERTY(Category=LODActor, VisibleAnywhere)
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;

	/** Imposters to be rendered as instanced static meshes */
	UPROPERTY(Transient)
	TMap<FHLODInstancingKey, TObjectPtr<UInstancedStaticMeshComponent>> InstancedStaticMeshComponents;

	/** The mesh proxy used to display this LOD */
	UPROPERTY(Category=LODActor, VisibleAnywhere)
	TObjectPtr<UHLODProxy> Proxy;

	/** The key used to validate this actor against the proxy */
	UPROPERTY(Category=LODActor, VisibleAnywhere)
	FName Key;

	/** what distance do you want this to show up instead of SubActors */
	UPROPERTY(Category = LODActor, VisibleAnywhere)
	float LODDrawDistance;

public:
	/** The hierarchy level of this actor; the first tier of HLOD is level 1, the second tier is level 2 and so on. */
	UPROPERTY(Category=LODActor, VisibleAnywhere)
	int32 LODLevel;

	UPROPERTY(Category=LODActor, VisibleAnywhere)
	TArray<TObjectPtr<AActor>> SubActors;
	
	UPROPERTY()
	uint8 CachedNumHLODLevels;

	//~ Begin AActor Interface
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual void EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	ENGINE_API virtual void EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	ENGINE_API virtual void EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation) override;
#endif // WITH_EDITOR	
	ENGINE_API virtual FBox GetComponentsBoundingBox(bool bNonColliding = false, bool bIncludeFromChildActors = false) const override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual void Tick(float DeltaSeconds) override;	
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	//~ End AActor Interface

	/** Forces the mesh into view by setting the MinDrawDistance to zero (this pops the mesh into view, no fading)*/
	ENGINE_API void PauseDitherTransition();
	/** Makes the actor tickable and according to r.HLOD.DitherPauseTime sets the MinDrawDistance back to non-zero */
	ENGINE_API void StartDitherTransition();

	/** Sets StaticMesh and IsPreviewActor to true if InStaticMesh equals nullptr */
	ENGINE_API void SetStaticMesh(UStaticMesh* InStaticMesh);

	/** Add instances to this LODActor. */
	ENGINE_API void AddInstances(const UStaticMesh* InStaticMesh, const UMaterialInterface* InMaterial, const TArray<FTransform>& InTransforms);
	ENGINE_API void AddInstances(const UStaticMesh* InStaticMesh, const UMaterialInterface* InMaterial, const TArray<FTransform>& InTransforms, const TArray<FCustomPrimitiveData>& InCustomPrimitiveData);

	/** Clear all instances stored in this LODActor. */
	ENGINE_API void ClearInstances();

	/** Sets the LOD draw distance and updates the Static Mesh Component's min drawing distance */
	ENGINE_API void SetDrawDistance(float InDistance);

	/** Gets the LOD draw distance */
	float GetDrawDistance() const { return LODDrawDistance; }
	float GetLODDrawDistance() const { return LODDrawDistance; }
	ENGINE_API float GetLODDrawDistanceWithOverride() const;

	/** 
	 * Set LOD Parent component for all of our components, normally associated with an ALODActor. 
	 * @param InLODParent			This component used to compute visibility when hierarchical LOD is enabled. 
	 * @param InParentDrawDistance	Updates the MinDrawDistances of the LODParent
	 * @param bInApplyToImposters	Whether this should be applied to imposters instanced static meshes components.
	 */
	ENGINE_API void SetLODParent(UPrimitiveComponent* InLODParent, float InParentDrawDistance, bool bInApplyToImposters);

#if WITH_EDITORONLY_DATA
	/** 
	 * Set the named tag which describes the source of the LOD actor (HLOD Volume, single cluster / level, etc)
	 * Helps uniquely identifying auto generated LOD actors & naming of the proxy static mesh assets.
	 * @param	InLODActorTag		Named tag to assign to this LODActor.
	 */
	void SetLODActorTag(const FString& InLODActorTag) { LODActorTag = InLODActorTag; }

	/**
	 * Get the named tag which describes the source of the LOD actor (HLOD Volume, single cluster / level, etc)
	 * Helps uniquely identifying auto generated LOD actors & naming of the proxy static mesh assets.
	 * @returns the named tag assigned to this LODActor.
	 */
	const FString& GetLODActorTag() const { return LODActorTag; }
#endif // WITH_EDITORONLY_DATA

	/** Get the key that we use to check if we need to (re)build */
	const FName& GetKey() const { return Key; }

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR
	/** 
	 * Check to see if this mesh is built 
	 * @param	bInForce	Whether to force the recalculation of this actor's build flag. If this is false then the cached flag is used an only recalculated every so often.
	 */
	ENGINE_API const bool IsBuilt(bool bInForce = false) const;
#endif

	/** Returns whether or not this LODActor has valid SubActors and whether or not their contained Primitive Components are linked (LODParentPrimitive) to StaticMeshComponent*/
	ENGINE_API const bool HasValidLODChildren() const;

#if WITH_EDITOR
	/** Force this actor to appear unbuilt (zeros-out key) */
	ENGINE_API void ForceUnbuilt();

	/**
	* Adds InActor to the SubActors array and set its LODParent to this
	* @param InActor - Actor to add
	*/
	ENGINE_API void AddSubActor(AActor* InActor);

	/**
	* Append the provided actors to the SubActors array and properly setup their LODParent
	* @param InActors - Array of actors to add
	*/
	ENGINE_API void AddSubActors(const TArray<AActor*>& InActors);

	/**
	* Removes InActor from the SubActors array and sets its LODParent to nullptr
	* @param InActor - Actor to remove
	*/
	ENGINE_API const bool RemoveSubActor(AActor* InActor);

	/**
	 * Determines whether or not this LODActor has valid SubActors and can be built
	 * @return true if the subactor(s) contain at least two static mesh components
	 */
	ENGINE_API const bool HasValidSubActors() const;

	/**
	 * Determines whether or not this LODActor has any SubActors
	 * @return true if it contains any subactors
	 */
	ENGINE_API const bool HasAnySubActors() const;

	/** Toggles forcing the StaticMeshComponent drawing distance to 0 or LODDrawDistance */
	ENGINE_API void ToggleForceView();

	/** Sets forcing the StaticMeshComponent drawing distance to 0 or LODDrawDistance according to InState*/
	ENGINE_API void SetForcedView(const bool InState);

	/** Sets the state of whether or not this LODActor is hidden from the Editor view, used for forcing a HLOD to show*/
	ENGINE_API void SetHiddenFromEditorView(const bool InState, const int32 ForceLODLevel);

	/** Returns the number of triangles this LODActor's SubActors contain */
	ENGINE_API const uint32 GetNumTrianglesInSubActors();

	/** Returns the number of triangles this LODActor's SubActors contain */
	ENGINE_API const uint32 GetNumTrianglesInMergedMesh();
	
	/** Updates the LODParents for the SubActors (and the drawing distance)*/
	ENGINE_API void UpdateSubActorLODParents();

	// This will determine the shadowing flags for the static mesh component according to all sub actors
	ENGINE_API void DetermineShadowingFlags();

	/** Cleans the SubActor array (removes all NULL entries) */
	ENGINE_API void CleanSubActorArray();

	/** Recalculates the drawing distance according to a fixed FOV of 90 and the transition screen size*/
	ENGINE_API void RecalculateDrawingDistance(const float TransitionScreenSize);

	/** Get the proxy mesh we use to render */
	UHLODProxy* GetProxy() const { return Proxy; }

	/** Get the proxy description used to generated this LODActor */
	const UHLODProxyDesc* GetProxyDesc() const { return ProxyDesc; }

	/**
	 * Update the proxy description that represent this LODActor
	 * @return true if the description changed.
	 */
	ENGINE_API bool UpdateProxyDesc();

	/** Returns true if this LODActor was constructed from an HLODProxyDesc */
	bool WasBuiltFromHLODDesc() const { return bBuiltFromHLODDesc; }
#endif // WITH_EDITOR

	//~ Begin UObject Interface.
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITOR	

	

	//~ End UObject Interface.	
public:
#if WITH_EDITORONLY_DATA
	/** Cached number of triangles contained in the SubActors*/
	UPROPERTY()
	uint32 NumTrianglesInSubActors;

	/** Cached number of triangles contained in the SubActors*/
	UPROPERTY()
	uint32 NumTrianglesInMergedMesh;

	/** Flag whether or not to use the override MaterialSettings when creating the proxy mesh */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings)
	bool bOverrideMaterialMergeSettings;

	/** Override Material Settings, used when creating the proxy mesh */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings, meta = (editcondition = "bOverrideMaterialMergeSettings"))
	FMaterialProxySettings MaterialSettings;

	/** Flag whether or not to use the override TransitionScreenSize for this proxy mesh */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings)
	bool bOverrideTransitionScreenSize;

	/** 
	 * Override transition screen size value, determines the screen size at which the proxy is visible 
	 * The screen size is based around the projected diameter of the bounding 
	 * sphere of the model. i.e. 0.5 means half the screen's maximum dimension.
	 */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings, meta = (editcondition = "bOverrideTransitionScreenSize"))
	float TransitionScreenSize;

	/** Flag whether or not to use the override ScreenSize when creating the proxy mesh */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings)
	bool bOverrideScreenSize;

	/** Override screen size value used in mesh reduction, when creating the proxy mesh */
	UPROPERTY(EditAnywhere, Category = HierarchicalLODSettings, meta = (editcondition = "bOverrideScreenSize"))
	int32 ScreenSize;
#endif // WITH_EDITORONLY_DATA

	/** Returns StaticMeshComponent subobject **/
	UStaticMeshComponent* GetStaticMeshComponent() const { return StaticMeshComponent; }

	/** Returns instanced static mesh components that represents the imposters found in this LODActor. */
	TArray<UInstancedStaticMeshComponent*> GetInstancedStaticMeshComponents() const { TArray<typename TDecay<decltype(InstancedStaticMeshComponents[FHLODInstancingKey()])>::Type> ISM; InstancedStaticMeshComponents.GenerateValueArray(ISM); return ISM; }

	/** Returns an array of distances that are used to override individual LOD actors min draw distances. */
	static ENGINE_API const TArray<float>& GetHLODDistanceOverride();

private:
	// Called when CVars are changed to check to see if the maximum HLOD level value has changed
	static ENGINE_API void OnCVarsChanged();
	static ENGINE_API void ParseOverrideDistancesCVar();	
	static ENGINE_API TArray<float> HLODDistances;

	// Updates the transition distance according to values (if) set in r.HLOD.DistanceOverride
	ENGINE_API void UpdateOverrideTransitionDistance();

	// Called to make sure autoregistration/manual registration state matches based on the LOD override cvar and this actor's lod level
	ENGINE_API void UpdateRegistrationToMatchMaximumLODLevel();

	// Setup a LOD static mesh component.
	ENGINE_API void SetupComponent(UStaticMeshComponent* InComponent);

	// Utility methods to act on all static mesh components owned by this actor.
	ENGINE_API void SetComponentsMinDrawDistance(float InMinDrawDistance, bool bInMarkRenderStateDirty);
	ENGINE_API void RegisterMeshComponents();
	ENGINE_API void UnregisterMeshComponents();

	UFUNCTION()
	ENGINE_API void OnSubActorEndPlay(AActor* Actor, EEndPlayReason::Type Reason);

#if WITH_EDITOR
	// Get/Create the LOD instanced static mesh component for a given imposter material.
	ENGINE_API UInstancedStaticMeshComponent* GetISMComponent(const FHLODInstancingKey& InstancingKey) const;
	ENGINE_API UInstancedStaticMeshComponent* GetOrCreateISMComponent(const FHLODInstancingKey& InstancingKey);

	// Get/Create the LOD static mesh component to use for a given actor.
	ENGINE_API UStaticMeshComponent* GetLODComponentForActor(const AActor* InActor, bool bFallbackToDefault = true) const;
	ENGINE_API UStaticMeshComponent* GetOrCreateLODComponentForActor(const AActor* InActor);

public:
	static ENGINE_API bool ShouldUseInstancing(const UStaticMeshComponent* InComponent);
#endif

private:
 	// Have we already tried to register components? (a cache to avoid having to query the owning world when the global HLOD max level setting is changed)
 	uint8 bHasActorTriedToRegisterComponents : 1;

	// Flag whether or not the SubActors have been moved to a Parent LODActor
	uint8 bHasPatchedUpParent : 1;

	/**
	 * If true on post load we need to calculate resolution independent Display Factors from the
	 * loaded LOD screen sizes.
	 */
	uint8 bRequiresLODScreenSizeConversion : 1;

	/** Flags for forcing a dithering transition */
	uint8 bNeedsDrawDistanceReset : 1;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR
	/** Cached flag we use to avoid checking the built status of this actor every frame */
	mutable uint8 bCachedIsBuilt : 1;

	/** Timer we use to avoid checking the built status of this actor every frame */
	mutable double LastIsBuiltTime;
#endif

	float ResetDrawDistanceTime;	

	// Sink for when CVars are changed to check to see if the maximum HLOD level value has changed
	static ENGINE_API FAutoConsoleVariableSink CVarSink;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString LODActorTag;

	UPROPERTY()
	uint8 bBuiltFromHLODDesc : 1;

	/** The hlod proxy desc used to build this LODActor */
	UPROPERTY(Transient)
	TObjectPtr<UHLODProxyDesc> ProxyDesc;
#endif // WITH_EDITORONLY_DATA
}; 
