// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "LatentActionManager.generated.h"

class FPendingLatentAction;

namespace LatentActionCVars
{
	extern int32 GuaranteeEngineTickDelay;	// 0: next-tick delays may run at end of current frame (default behavior pre-5.5),  1: next-tick delays will always wait until the engine frame has advanced
}

// Latent action info
USTRUCT(BlueprintInternalUseOnly)
struct FLatentActionInfo
{
	GENERATED_USTRUCT_BODY()

	/** The resume point within the function to execute */
	UPROPERTY(meta=(NeedsLatentFixup = true))
	int32 Linkage;

	/** the UUID for this action */ 
	UPROPERTY()
	int32 UUID;

	/** The function to execute. */ 
	UPROPERTY()
	FName ExecutionFunction;

	/** Object to execute the function on. */ 
	UPROPERTY(meta=(LatentCallbackTarget = true))
	TObjectPtr<UObject> CallbackTarget;

	FLatentActionInfo()
		: Linkage(INDEX_NONE)
		, UUID(INDEX_NONE)
		, ExecutionFunction(NAME_None)
		, CallbackTarget(NULL)
	{
	}

	FORCENOINLINE FLatentActionInfo(int32 InLinkage, int32 InUUID, const TCHAR* InFunctionName, UObject* InCallbackTarget)
		: Linkage(InLinkage)
		, UUID(InUUID)
		, ExecutionFunction(InFunctionName)
		, CallbackTarget(InCallbackTarget)
	{
	}
};

enum class ELatentActionChangeType : uint8
{
	/** Latent actions were removed */
	ActionsRemoved,

	/** Latent actions were added */
	ActionsAdded,
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLatentActionsChanged, UObject*, ELatentActionChangeType)

// The latent action manager handles all pending latent actions for a single world
USTRUCT()
struct FLatentActionManager
{
	GENERATED_USTRUCT_BODY()

public: 
	/** @return A delegate that will be broadcast when a latent action is added or removed from the manager */
	static FOnLatentActionsChanged& OnLatentActionsChanged() { return LatentActionsChangedDelegate; }

	/** 
	 * Advance pending latent actions by DeltaTime.
 	 * If no object is specified it will process any outstanding actions for objects that have not been processed for this frame.
	 *
	 * @param		InOject		Specific object pending action list to advance.
	 * @param		DeltaTime	Delta time.
	 *
	 */
	ENGINE_API void ProcessLatentActions(UObject* InObject, float DeltaTime);

	/** 
	 * Finds the action instance for the supplied UUID, or will return NULL if one does not already exist.
	 *
	 * @param	InOject			ActionListType to check for pending actions.
 	 * @param	UUID			UUID of the action we are looking for.
	 * @param	FilterPredicate	Filter predicate indicating which instance to return
	 *
	 */	
	template<typename ActionType, typename PredicateType>
	ActionType* FindExistingActionWithPredicate(UObject* InActionObject, int32 UUID, const PredicateType& FilterPredicate)
	{
		FObjectActions* ObjectActionList = GetActionsForObject(InActionObject);
		if ((ObjectActionList != nullptr) && (ObjectActionList->ActionList.Num(UUID) > 0))
		{
			for (auto It = ObjectActionList->ActionList.CreateKeyIterator(UUID); It; ++It)
			{
				ActionType* Item = (ActionType*)(It.Value());
				if (FilterPredicate(Item))
				{
					return Item;
				}
			}
		}

		return nullptr;
	}

	/** 
	 * Finds the action instance for the supplied UUID, or will return NULL if one does not already exist.
	 *
	 * @param		InOject		ActionListType to check for pending actions.
 	 * @param		UUID		UUID of the action we are looking for.
	 *
	 */	
	template<typename ActionType>
	ActionType* FindExistingAction(UObject* InActionObject, int32 UUID)
	{
		struct FFirstItemPredicate
		{
			FORCEINLINE bool operator()(void*) const { return true; }
		};

		return FindExistingActionWithPredicate<ActionType, FFirstItemPredicate>(InActionObject, UUID, FFirstItemPredicate());
	}

	/** 
	 * Removes all actions for given object. 
	 * It the latent actions are being currently handled (so the function is called inside a ProcessLatentActions functions scope) 
	 * there is no guarantee, that the action will be removed before its execution.
	 *
	 * @param		InOject		Specific object
	 */	
	ENGINE_API void RemoveActionsForObject(TWeakObjectPtr<UObject> InObject);

	/** 
	 * Adds a new action to the action list under a given UUID 
	 */
	ENGINE_API void AddNewAction(UObject* InActionObject, int32 UUID, FPendingLatentAction* NewAction);

	/** Resets the list of objects we have processed the latent action list for.	 */	
	ENGINE_API void BeginFrame();

	/** Returns the number of actions for a given object */
	ENGINE_API int32 GetNumActionsForObject(TWeakObjectPtr<UObject> InObject);

#if WITH_EDITOR
	/** 
	 * Builds a set of the UUIDs of pending latent actions on a specific object.
	 *
	 * @param	InObject	Object to query for latent actions
	 * @param	UUIDList	Array to add the UUID of each pending latent action to.
	 *
	 */
	ENGINE_API void GetActiveUUIDs(UObject* InObject, TSet<int32>& UUIDList) const;

	/** 
	 * Gets the description string of a pending latent action with the specified UUID for a given object, or the empty string if it's an invalid UUID
	 *
	  * @param		InOject		Object to get list of UUIDs for
	  * @param		UUIDList	Array to add UUID to.
	 *
	 */
	ENGINE_API FString GetDescription(UObject* InObject, int32 UUID) const;
#endif

	ENGINE_API ~FLatentActionManager();

protected:
	/** Map of UUID->Action(s). */
	typedef TMultiMap<int32, class FPendingLatentAction*> FActionList;

	struct FObjectActions
	{
		/** Map of UUID->Action(s). */
		FActionList ActionList;
		bool bProcessedThisFrame = false;
	};

	/** Map to convert from object to FActionList. */
	typedef TMap< TWeakObjectPtr<UObject>, TSharedPtr<FObjectActions>, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UObject>, TSharedPtr<FObjectActions> > > FObjectToActionListMap;
	FObjectToActionListMap ObjectToActionListMap;

	/** 
	 * Finds the action instance for the supplied object will return NULL if one does not exist.
	 *
	 * @param		InOject		ActionListType to check for pending actions.
	 *
	 */	
	const FObjectActions* GetActionsForObject(const TWeakObjectPtr<UObject>& InObject) const
	{
		auto ObjectActionListPtr = ObjectToActionListMap.Find(InObject);
		return ObjectActionListPtr ? ObjectActionListPtr->Get() : nullptr;
	}

	/** 
	 * Finds the action instance for the supplied object will return NULL if one does not exist.
	 *
	 * @param		InOject		ActionListType to check for pending actions.
	 *
	 */	
	FObjectActions* GetActionsForObject(const TWeakObjectPtr<UObject>& InObject)
	{
		auto ObjectActionListPtr = ObjectToActionListMap.Find(InObject);
		return ObjectActionListPtr ? ObjectActionListPtr->Get() : nullptr;
	}

	/** 
	  * Ticks the latent action for a single UObject.
	  *
	  * @param		DeltaTime			time delta.
	  * @param		ObjectActionList	the action list for the object
	  * @param		InObject			the object itself.
	  *
	  */
	ENGINE_API void TickLatentActionForObject(float DeltaTime, FActionList& ObjectActionList, UObject* InObject);

protected:
	/**List of actions that will be unconditionally removed at the begin of next tick */
	typedef TPair<int32, class FPendingLatentAction*> FUuidAndAction;
	typedef TPair<TWeakObjectPtr<UObject>, TSharedPtr<TArray<FUuidAndAction>>> FWeakObjectAndActions;
	typedef TArray<FWeakObjectAndActions> FActionsForObject;
	FActionsForObject ActionsToRemoveMap;

	/** Delegate called when a latent action is added or removed */
	static ENGINE_API FOnLatentActionsChanged LatentActionsChangedDelegate;
};
