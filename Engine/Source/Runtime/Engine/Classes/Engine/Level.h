// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineBaseTypes.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "RenderCommandFence.h"
#include "Templates/ScopedCallback.h"
#include "Misc/WorldCompositionUtility.h"
#include "Engine/MaterialMerging.h"
#include "Engine/TextureStreamingTypes.h"
#include "Misc/EditorPathObjectInterface.h"
#include "Components/ActorComponent.h"

#include <atomic>

#include "Level.generated.h"

class AActor;
class ABrush;
class AInstancedFoliageActor;
class ALevelBounds;
class APlayerController;
class AWorldSettings;
class AWorldDataLayers;
class FSceneInterface;
class ITargetPlatform;
class UAssetUserData;
class UMapBuildDataRegistry;
class UNavigationDataChunk;
class UTexture2D;
struct FLevelCollection;
class ULevelActorContainer;
class FRegisterComponentContext;
class FUnregisterComponentContext;
class SNotificationItem;
class UActorFolder;
class IWorldPartitionCell;
class UWorldPartitionRuntimeCell;
struct FWorldRenameFromRootContext;
struct FFolder;

#if WITH_EDITOR
struct FLevelActorFoldersHelper
{
private:
	static ENGINE_API void SetUseActorFolders(ULevel* InLevel, bool bInEnabled);
	static ENGINE_API void AddActorFolder(ULevel* InLevel, UActorFolder* InActorFolder, bool bInShouldDirtyLevel, bool bInShouldBroadcast = true);
	static ENGINE_API void RenameFolder(ULevel* InLevel, const FFolder& InOldFolder, const FFolder& InNewFolder);
	static ENGINE_API void DeleteFolder(ULevel* InLevel, const FFolder& InFolder);

	friend class UWorld;
	friend class ULevel;
	friend class UActorFolder;
	friend class FWorldPartitionConverter;
	friend class UWorldPartitionConvertCommandlet;
	friend class FWorldPartitionLevelHelper;
	friend class UWorldPartitionLevelStreamingDynamic;
};

#endif

// Actor container class used to duplicate actors during cells streaming in PIE
UCLASS(MinimalAPI)
class UActorContainer : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FName, TObjectPtr<AActor>> Actors;

#if WITH_EDITORONLY_DATA
	// Can be used to resolve the Actor Level while they are outered to the UActorContainer in a PIE duplication
	UPROPERTY()
	TObjectPtr<ULevel> RuntimeLevel = nullptr;
#endif
};

USTRUCT()
struct FActorFolderSet
{
	GENERATED_BODY()

public:

	ENGINE_API void Add(UActorFolder* InActorFolder);
	int32 Remove(UActorFolder* InActorFolder) { return ActorFolders.Remove(InActorFolder); }
	bool IsEmpty() const { return ActorFolders.IsEmpty(); }
	const TSet<TObjectPtr<UActorFolder>>& GetActorFolders() const { return ActorFolders; }

private:

	UPROPERTY(Transient)
	TSet<TObjectPtr<UActorFolder>> ActorFolders;
};

/**
 * Structure containing all information needed for determining the screen space
 * size of an object/ texture instance.
 */
USTRUCT()
struct FStreamableTextureInstance
{
	GENERATED_USTRUCT_BODY()

	/** Bounding sphere/ box of object */
	FBoxSphereBounds  Bounds;

	/** Min distance from view where this instance is usable */
	float MinDistance;
	/** Max distance from view where this instance is usable */
	float MaxDistance;

	/** Object (and bounding sphere) specific texel scale factor  */
	float	TexelFactor;

	/**
	 * FStreamableTextureInstance serialize operator.
	 *
	 * @param	Ar					Archive to to serialize object to/ from
	 * @param	TextureInstance		Object to serialize
	 * @return	Returns the archive passed in
	 */
	friend FArchive& operator<<( FArchive& Ar, FStreamableTextureInstance& TextureInstance );
};

/**
 * Serialized ULevel information about dynamic texture instances
 */
USTRUCT()
struct FDynamicTextureInstance : public FStreamableTextureInstance
{
	GENERATED_USTRUCT_BODY()

	/** Texture that is used by a dynamic UPrimitiveComponent. */
	UPROPERTY()
	TObjectPtr<UTexture2D>					Texture = nullptr;

	/** Whether the primitive that uses this texture is attached to the scene or not. */
	UPROPERTY()
	bool						bAttached = false;
	
	/** Original bounding sphere radius, at the time the TexelFactor was calculated originally. */
	UPROPERTY()
	float						OriginalRadius = 0.0f;

	/**
	 * FDynamicTextureInstance serialize operator.
	 *
	 * @param	Ar					Archive to to serialize object to/ from
	 * @param	TextureInstance		Object to serialize
	 * @return	Returns the archive passed in
	 */
	friend FArchive& operator<<( FArchive& Ar, FDynamicTextureInstance& TextureInstance );
};

/** Struct that holds on to information about Actors that wish to be auto enabled for input before the player controller has been created */
struct FPendingAutoReceiveInputActor
{
	TWeakObjectPtr<AActor> Actor;
	int32 PlayerIndex;

	FPendingAutoReceiveInputActor(AActor* InActor, const int32 InPlayerIndex);
	~FPendingAutoReceiveInputActor();
};

/** A precomputed visibility cell, whose data is stored in FCompressedVisibilityChunk. */
class FPrecomputedVisibilityCell
{
public:

	/** World space min of the cell. */
	FVector Min;

	/** Index into FPrecomputedVisibilityBucket::CellDataChunks of this cell's data. */
	uint16 ChunkIndex;

	/** Index into the decompressed chunk data of this cell's visibility data. */
	uint16 DataOffset;

	friend FArchive& operator<<( FArchive& Ar, FPrecomputedVisibilityCell& D )
	{
		Ar << D.Min << D.ChunkIndex << D.DataOffset;
		return Ar;
	}
};

/** A chunk of compressed visibility data from multiple FPrecomputedVisibilityCell's. */
class FCompressedVisibilityChunk
{
public:
	/** Whether the chunk is compressed. */
	bool bCompressed;

	/** Size of the uncompressed chunk. */
	int32 UncompressedSize;

	/** Compressed visibility data if bCompressed is true. */
	TArray<uint8> Data;

	friend FArchive& operator<<( FArchive& Ar, FCompressedVisibilityChunk& D )
	{
		Ar << D.bCompressed << D.UncompressedSize << D.Data;
		return Ar;
	}
};

/** A bucket of visibility cells that have the same spatial hash. */
class FPrecomputedVisibilityBucket
{
public:
	/** Size in bytes of the data of each cell. */
	int32 CellDataSize;

	/** Cells in this bucket. */
	TArray<FPrecomputedVisibilityCell> Cells;

	/** Data chunks corresponding to Cells. */
	TArray<FCompressedVisibilityChunk> CellDataChunks;

	friend FArchive& operator<<( FArchive& Ar, FPrecomputedVisibilityBucket& D )
	{
		Ar << D.CellDataSize << D.Cells << D.CellDataChunks;
		return Ar;
	}
};

/** Handles operations on precomputed visibility data for a level. */
class FPrecomputedVisibilityHandler
{
public:

	FPrecomputedVisibilityHandler() :
		Id(NextId)
	{
		NextId++;
	}
	
	~FPrecomputedVisibilityHandler() 
	{ 
		UpdateVisibilityStats(false);
	}

	/** Updates visibility stats. */
	ENGINE_API void UpdateVisibilityStats(bool bAllocating) const;

	/** Sets this visibility handler to be actively used by the rendering scene. */
	ENGINE_API void UpdateScene(FSceneInterface* Scene) const;

	/** Invalidates the level's precomputed visibility and frees any memory used by the handler. */
	ENGINE_API void Invalidate(FSceneInterface* Scene);

	/** Shifts origin of precomputed visibility volume by specified offset */
	ENGINE_API void ApplyWorldOffset(const FVector& InOffset);

	/** @return the Id */
	int32 GetId() const { return Id; }

	friend FArchive& operator<<( FArchive& Ar, FPrecomputedVisibilityHandler& D );
	
private:

	/** World space origin of the cell grid. */
	FVector2D PrecomputedVisibilityCellBucketOriginXY;

	/** World space size of every cell in x and y. */
	float PrecomputedVisibilityCellSizeXY;

	/** World space height of every cell. */
	float PrecomputedVisibilityCellSizeZ;

	/** Number of cells in each bucket in x and y. */
	int32	PrecomputedVisibilityCellBucketSizeXY;

	/** Number of buckets in x and y. */
	int32	PrecomputedVisibilityNumCellBuckets;

	static int32 NextId;

	/** Id used by the renderer to know when cached visibility data is valid. */
	int32 Id;

	/** Visibility bucket data. */
	TArray<FPrecomputedVisibilityBucket> PrecomputedVisibilityCellBuckets;

	friend class FLightmassProcessor;
	friend class FSceneViewState;
};

/** Volume distance field generated by Lightmass, used by image based reflections for shadowing. */
class FPrecomputedVolumeDistanceField
{
public:

	/** Sets this volume distance field to be actively used by the rendering scene. */
	ENGINE_API void UpdateScene(FSceneInterface* Scene) const;

	/** Invalidates the level's volume distance field and frees any memory used by it. */
	ENGINE_API void Invalidate(FSceneInterface* Scene);

	friend FArchive& operator<<( FArchive& Ar, FPrecomputedVolumeDistanceField& D );

private:
	/** Largest world space distance stored in the volume. */
	float VolumeMaxDistance;
	/** World space bounding box of the volume. */
	FBox VolumeBox;
	/** Volume dimension X. */
	int32 VolumeSizeX;
	/** Volume dimension Y. */
	int32 VolumeSizeY;
	/** Volume dimension Z. */
	int32 VolumeSizeZ;
	/** Distance field data. */
	TArray<FColor> Data;

	friend class FScene;
	friend class FLightmassProcessor;
};

USTRUCT()
struct FLevelSimplificationDetails
{
	GENERATED_USTRUCT_BODY()

	/** Whether to create separate packages for each generated asset. All in map package otherwise */
	UPROPERTY(Category=General, EditAnywhere)
	bool bCreatePackagePerAsset;

	/** Percentage of details for static mesh proxy */
	UPROPERTY(Category=StaticMesh, EditAnywhere, meta=(DisplayName="Static Mesh Details Percentage", ClampMin = "0", ClampMax = "100", UIMin = "0", UIMax = "100"))	
	float DetailsPercentage;

	/** Landscape material simplification */
	UPROPERTY(Category = Landscape, EditAnywhere)
	FMaterialProxySettings StaticMeshMaterialSettings;

	UPROPERTY(Category = Landscape, EditAnywhere, meta=(InlineEditConditionToggle))
	bool bOverrideLandscapeExportLOD;

	/** Landscape LOD to use for static mesh generation, when not specified 'Max LODLevel' from landscape actor will be used */
	UPROPERTY(Category=Landscape, EditAnywhere, meta=(ClampMin = "0", ClampMax = "7", UIMin = "0", UIMax = "7", editcondition = "bOverrideLandscapeExportLOD"))
	int32 LandscapeExportLOD;
	
	/** Landscape material simplification */
	UPROPERTY(Category = Landscape, EditAnywhere)
	FMaterialProxySettings LandscapeMaterialSettings;

	/** Whether to bake foliage into landscape static mesh texture */
	UPROPERTY(Category=Landscape, EditAnywhere)
	bool bBakeFoliageToLandscape;

	/** Whether to bake grass into landscape static mesh texture */
	UPROPERTY(Category=Landscape, EditAnywhere)
	bool bBakeGrassToLandscape;

	ENGINE_API FLevelSimplificationDetails();
	ENGINE_API bool operator == (const FLevelSimplificationDetails& Other) const;
};

/**
 * Stored information about replicated static/placed actors that have been destroyed in a level.
 * This information is cached in ULevel so that any net drivers that are created after these actors
 * are destroyed can access this info and correctly replicate the destruction to their clients.
 */
USTRUCT()
struct FReplicatedStaticActorDestructionInfo
{
	GENERATED_BODY()

	FName PathName;
	FString FullName;
	FVector	DestroyedPosition;
	TWeakObjectPtr<UObject> ObjOuter;
	UPROPERTY()
	TObjectPtr<UClass> ObjClass = nullptr;
};

#if WITH_EDITORONLY_DATA
/** Enum defining how external actors are saved on disk */
UENUM()
enum class EActorPackagingScheme : uint8
{
	Original,	// Original scheme: ZZ/ZZ/... (maximum 1679616 folders,  ~0.6 files per folder with 1000000 files)
	Reduced		//  Reduced scheme:  Z/ZZ/... (maximum   46656 folders, ~21.4 files per folder with 1000000 files)
};
#endif

// Whether the level is in the process of being made visible or invisible (refer to bIsVisible when None).
enum class ELevelPendingVisibilityState : uint8
{
	None,
	MakingVisible,
	MakingInvisible
};

//
// The level object.  Contains the level's actor list, BSP information, and brush list.
// Every Level has a World as its Outer and can be used as the PersistentLevel, however,
// when a Level has been streamed in the OwningWorld represents the World that it is a part of.
//


/**
 * A Level is a collection of Actors (lights, volumes, mesh instances etc.).
 * Multiple Levels can be loaded and unloaded into the World to create a streaming experience.
 * 
 * @see https://docs.unrealengine.com/latest/INT/Engine/Levels
 * @see UActor
 */
UCLASS(MinimalAPI)
class ULevel : public UObject, public IInterface_AssetUserData, public ITextureStreamingContainer, public IEditorPathObjectInterface
{
	GENERATED_BODY()

public:

	/** URL associated with this level. */
	FURL					URL;

	/** Array of all actors in this level, used by FActorIteratorBase and derived classes */
	TArray<TObjectPtr<AActor>> Actors;

	/** Array of actors to be exposed to GC in this level. All other actors will be referenced through ULevelActorContainer */
	TArray<TObjectPtr<AActor>> ActorsForGC;

	/** Number of actors marked for pending unload by GC */
	int32 NumActorsMarkedForPendingUnloadByGC = 0;

	/** Add actor to list */
	ENGINE_API bool TryAddActorToList(AActor* InActor, bool bAddUnique);

#if WITH_EDITORONLY_DATA
	AActor* PlayFromHereActor;

	/** Use external actors, new actor spawned in this level will be external and existing external actors will be loaded on load. */
	UPROPERTY(EditInstanceOnly, Category=World)
	bool bUseExternalActors;
#endif

	/** Set before calling LoadPackage for a streaming level to ensure that OwningWorld is correct on the Level */
	ENGINE_API static TMap<FName, TWeakObjectPtr<UWorld> > StreamedLevelsOwningWorld;
		
	/** 
	 * The World that has this level in its Levels array. 
	 * This is not the same as GetOuter(), because GetOuter() for a streaming level is a vestigial world that is not used. 
	 * It should not be accessed during BeginDestroy(), just like any other UObject references, since GC may occur in any order.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UWorld> OwningWorld;

	/** BSP UModel. */
	UPROPERTY()
	TObjectPtr<class UModel> Model;

	/** BSP Model components used for rendering. */
	UPROPERTY()
	TArray<TObjectPtr<class UModelComponent>> ModelComponents;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TObjectPtr<ULevelActorContainer> ActorCluster;

#if WITH_EDITORONLY_DATA
	/** Reference to the blueprint for level scripting */
	UPROPERTY(NonTransactional)
	TObjectPtr<class ULevelScriptBlueprint> LevelScriptBlueprint;

	/** The Guid list of all materials and meshes Guid used in the last texture streaming build. Used to know if the streaming data needs rebuild. Only used for the persistent level. */
	UPROPERTY(NonTransactional)
	TArray<FGuid> TextureStreamingResourceGuids;
#endif //WITH_EDITORONLY_DATA

	/** Num of components missing valid texture streaming data. Updated in map check. */
	UPROPERTY(NonTransactional)
	int32 NumTextureStreamingUnbuiltComponents;

	/** Num of resources that have changed since the last texture streaming build. Updated in map check. */
	UPROPERTY(NonTransactional)
	int32 NumTextureStreamingDirtyResources;

	/** The level scripting actor, created by instantiating the class from LevelScriptBlueprint.  This handles all level scripting */
	UPROPERTY(NonTransactional)
	TObjectPtr<class ALevelScriptActor> LevelScriptActor;

	/**
	 * Start and end of the navigation list for this level, used for quickly fixing up
	 * when streaming this level in/out. @TODO DEPRECATED - DELETE
	 */
	UPROPERTY()
	TObjectPtr<class ANavigationObjectBase> NavListStart;
	UPROPERTY()
	TObjectPtr<class ANavigationObjectBase>	NavListEnd;
	
	/** Navigation related data that can be stored per level */
	UPROPERTY()
	TArray<TObjectPtr<UNavigationDataChunk>> NavDataChunks;
	
	/** Total number of KB used for lightmap textures in the level. */
	UPROPERTY(VisibleAnywhere, Category=Level)
	float LightmapTotalSize;
	/** Total number of KB used for shadowmap textures in the level. */
	UPROPERTY(VisibleAnywhere, Category=Level)
	float ShadowmapTotalSize;

	/** threes of triangle vertices - AABB filtering friendly. Stored if there's a runtime need to rebuild navigation that accepts BSPs 
	 *	as well - it's a lot easier this way than retrieve this data at runtime */
	UPROPERTY()
	TArray<FVector> StaticNavigableGeometry;

	/** The Guid of each streamable texture refered by FStreamingTextureBuildInfo::TextureLevelIndex	*/
	UPROPERTY()
	TArray<FGuid> StreamingTextureGuids;

	/** The name of each streamable texture referred by FStreamingTextureBuildInfo::TextureLevelIndex */
	UPROPERTY()
	TArray<FName> StreamingTextures;

	/** Packed quality level and feature level used when building texture streaming data. This is used by runtime to determine if built data can be used or not. */
	UPROPERTY()
	uint32 PackedTextureStreamingQualityLevelFeatureLevel;

	/** Data structures for holding the tick functions **/
	class FTickTaskLevel*						TickTaskLevel;

	/** 
	* The precomputed light information for this level.  
	* The extra level of indirection is to allow forward declaring FPrecomputedLightVolume.
	*/
	class FPrecomputedLightVolume*				PrecomputedLightVolume;

	/** The volumetric lightmap data for this level. */
	class FPrecomputedVolumetricLightmap*			PrecomputedVolumetricLightmap;

	/** Contains precomputed visibility data for this level. */
	FPrecomputedVisibilityHandler				PrecomputedVisibilityHandler;

	/** Precomputed volume distance field for this level. */
	FPrecomputedVolumeDistanceField				PrecomputedVolumeDistanceField;

	/** Fence used to track when the rendering thread has finished referencing this ULevel's resources. */
	FRenderCommandFence							RemoveFromSceneFence;

	/** Identifies map build data specific to this level, eg lighting volume samples. */
	UPROPERTY()
	FGuid LevelBuildDataId;

	/** 
	 * Registry for data from the map build.  This is stored in a separate package from the level to speed up saving / autosaving. 
	 * ReleaseRenderingResources must be called before changing what is referenced, to update the rendering thread state.
	 */
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UMapBuildDataRegistry> MapBuildData;

	/** Level offset at time when lighting was built */
	UPROPERTY()
	FIntVector LightBuildLevelOffset;

	/** The volumetric lightmap grid manager for this map */
	class FVolumetricLightmapGridManager* VolumetricLightmapGridManager = nullptr;

	/** 
	 * Whether the level is a lighting scenario.  Lighting is built separately for each lighting scenario level with all other scenario levels hidden. 
	 * Only one lighting scenario level should be visible at a time for correct rendering, and lightmaps from that level will be used on the rest of the world.
	 * Note: When a lighting scenario level is present, lightmaps for all streaming levels are placed in the scenario's _BuildData package.  
	 *		This means that lightmaps for those streaming levels will not be streamed with them.
	 */
	UPROPERTY()
	uint8 bIsLightingScenario:1;

	/** Whether components are currently registered or not. */
	uint8										bAreComponentsCurrentlyRegistered:1;

	/** Whether the geometry needs to be rebuilt for correct lighting */
	uint8										bGeometryDirtyForLighting:1;

	/** Whether a level transform rotation was applied since the texture streaming builds. Invalidates the precomputed streaming bounds. */
	UPROPERTY()
	uint8 										bTextureStreamingRotationChanged : 1;

	/** 
	 * Whether the level has finished registering all static components in the streaming manager.
	 * Once a level static components are registered, all new components need to go through the dynamic path.
	 * This flag is used to direct the registration to the right path with a low perf impact.
	 */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	uint8										bStaticComponentsRegisteredInStreamingManager: 1;

	/** 
	 * Whether the level is currently visible/ associated with the world. 
	 * If false, may not yet be fully removed from the world.
	 */
	UPROPERTY(transient)
	uint8										bIsVisible:1;

#if WITH_EDITORONLY_DATA
	/** Whether this level is locked; that is, its actors are read-only 
	 *	Used by WorldBrowser to lock a level when corresponding ULevelStreaming does not exist
	 */
	UPROPERTY()
	uint8 										bLocked:1;

	/** Whether the level has been saved after introducing actor GUIDs */
	uint8										bContainsStableActorGUIDs:1;

	/** Whether the level should call FixupActorFolders on its actors when loading the level/actors (only used when level is using actor folder objects) */
	uint8										bFixupActorFoldersAtLoad:1;

private:
	/** Whether the level is set not to be reusable after unload (editor-only) */
	uint8										bForceCantReuseUnloadedButStillAround:1;

	/** Whether the level will force trash its packages in CleanupLevel (including object packages of objects outered to the level) (editor-only) */
	uint8										bForcePackageTrashingAtCleanup:1;
#endif

	/** Whether the level owns it's MapBuildData and associated resources and must release them (LevelInstance will use their owner level MapBuildData) */
	uint8										bIsMapBuildDataOwner:1;
	
public:
	/** The below variables are used temporarily while making a level visible.				*/

	/** Whether we already moved actors.													*/
	uint8										bAlreadyMovedActors:1;
	/** Whether we already shift actors positions according to world composition.			*/
	uint8										bAlreadyShiftedActors:1;
	/** Whether we already updated components.												*/
	uint8										bAlreadyUpdatedComponents:1;
	/** Whether we already associated streamable resources.									*/
	uint8										bAlreadyAssociatedStreamableResources:1;
	/** Whether we already initialized network actors.										*/
	uint8										bAlreadyInitializedNetworkActors:1;
	/** Whether we already cleared AActor::bActorSeamlessTraveled.							*/
	uint8										bAlreadyClearedActorsSeamlessTravelFlag:1;
	/** Whether we already sorted the actor list.											*/
	uint8										bAlreadySortedActorList:1;
	/** Whether this level is in the process of being associated with its world	(i.e. we are within AddToWorld for this level */
	uint8										bIsAssociatingLevel:1;
	/** Whether this level is in the process of being disassociated with its world (i.e. we are within RemoveFromWorld for this level */
	uint8										bIsDisassociatingLevel : 1;
	/** Whether this level should be fully added to the world before rendering its components	*/
	uint8										bRequireFullVisibilityToRender:1;
	/** Whether this level is specific to client, visibility state will not be replicated to server	*/
	uint8										bClientOnlyVisible:1;
	/** Whether this level was duplicated */
	uint8										bWasDuplicated:1;
	/** Whether this level was duplicated for PIE	*/
	uint8										bWasDuplicatedForPIE:1;
	/** Whether the level is currently being removed from the world */
	uint8										bIsBeingRemoved:1;
	/** Whether the level incremental unregistration of components is completed */
	uint8										bIncrementalUnregisterComponentsCompleted:1;
	/** Whether this level has gone through a complete rerun construction script pass. */
	uint8										bHasRerunConstructionScripts:1;
	/** Whether the level had its actor cluster created. This doesn't mean that the creation was successful. */
	uint8										bActorClusterCreated : 1;
	/** If true, allows garbage collection clustering for the level */
	uint8										bGarbageCollectionClusteringEnabled : 1;

	/** Whether the level is partitioned or not. */
    UPROPERTY()
	uint8										bIsPartitioned : 1;

	enum class EIncrementalComponentState : uint8
	{
		Init,
		PreRegisterInitialComponents,
		RegisterInitialComponents,
#if WITH_EDITOR
		RunConstructionScripts,
#endif
		Finalize
	};

	UE_DEPRECATED(5.5, "This property shouldn't be used anymore.")
	uint8										bHasCurrentActorCalledPreRegister:1;

private:
	/** Whether the level is in the process of being made visible or invisible (refer to bIsVisible when None). */
	ELevelPendingVisibilityState				PendingVisibilityState:2;

public:
	/** The current stage for incrementally updating actor components in the level*/
	EIncrementalComponentState					IncrementalComponentState;
	/** Current index into actors array for updating components.							*/
	int32										CurrentActorIndexForIncrementalUpdate;
	/** Current index into actors array for updating components.							*/
	int32										CurrentActorIndexForUnregisterComponents;


	/** Whether the level is currently pending being made invisible or visible.				*/
	ENGINE_API bool HasVisibilityChangeRequestPending() const;

	// Event on level transform changes
	DECLARE_MULTICAST_DELEGATE_OneParam(FLevelTransformEvent, const FTransform&);
	FLevelTransformEvent OnApplyLevelTransform;

	// Event on level cleanup
	DECLARE_MULTICAST_DELEGATE(FLevelCleanupEvent);
	FLevelCleanupEvent OnCleanupLevel;

#if WITH_EDITORONLY_DATA
	/** Level simplification settings for each LOD */
	UPROPERTY()
	FLevelSimplificationDetails LevelSimplification[WORLDTILE_LOD_MAX_INDEX];

	/** 
	 * The level color used for visualization. (Show -> Advanced -> Level Coloration)
	 * Used only in world composition mode
	 */
	UPROPERTY()
	FLinearColor LevelColor;

	std::atomic<uint64> FixupOverrideVertexColorsTimeMS = 0;
	std::atomic<uint32> FixupOverrideVertexColorsCount = 0;

	UPROPERTY(transient)
	bool bPromptWhenAddingToLevelBeforeCheckout;

	UPROPERTY(transient)
	bool bPromptWhenAddingToLevelOutsideBounds;

	UPROPERTY()
	EActorPackagingScheme ActorPackagingScheme;

#endif //WITH_EDITORONLY_DATA

	/** Actor which defines level logical bounding box				*/
	TWeakObjectPtr<ALevelBounds>				LevelBoundsActor;

	/** Cached pointer to Foliage actor		*/
	TWeakObjectPtr<AInstancedFoliageActor>		InstancedFoliageActor;

	/** Called when Level bounds actor has been updated */
	DECLARE_EVENT( ULevel, FLevelBoundsActorUpdatedEvent );
	FLevelBoundsActorUpdatedEvent& LevelBoundsActorUpdated() { return LevelBoundsActorUpdatedEvent; }
	/**	Broadcasts that Level bounds actor has been updated */ 
	void BroadcastLevelBoundsActorUpdated() { LevelBoundsActorUpdatedEvent.Broadcast(); }

	/** Marks level bounds as dirty so they will be recalculated  */
	ENGINE_API void MarkLevelBoundsDirty();

#if WITH_EDITOR
	ENGINE_API static bool GetLevelBoundsFromAsset(const FAssetData& Asset, FBox& OutLevelBounds);
	ENGINE_API static bool GetLevelBoundsFromPackage(FName LevelPackage, FBox& OutLevelBounds);

	ENGINE_API static bool GetWorldExternalActorsReferencesFromAsset(const FAssetData& Asset, TArray<FGuid>& OutWorldExternalActorsReferences);
	ENGINE_API static bool GetWorldExternalActorsReferencesFromPackage(FName LevelPackage, TArray<FGuid>& OutWorldExternalActorsReferences);

	ENGINE_API static bool GetIsLevelPartitionedFromAsset(const FAssetData& Asset);
	ENGINE_API static bool GetIsLevelPartitionedFromPackage(FName LevelPackage);

	ENGINE_API static bool GetIsLevelUsingExternalActorsFromAsset(const FAssetData& Asset);
	ENGINE_API static bool GetIsLevelUsingExternalActorsFromPackage(FName LevelPackage);

	ENGINE_API static bool GetIsLevelUsingActorsDescsFromAsset(const FAssetData& Asset);
	ENGINE_API static bool GetIsLevelUsingActorsDescsFromPackage(FName LevelPackage);

	ENGINE_API static bool GetIsUsingActorFoldersFromAsset(const FAssetData& Asset);
	ENGINE_API static bool GetIsUsingActorFoldersFromPackage(FName LevelPackage);

	ENGINE_API static bool GetIsStreamingDisabledFromAsset(const FAssetData& Asset);	
	ENGINE_API static bool GetIsStreamingDisabledFromPackage(FName LevelPackage);

	ENGINE_API static bool GetIsLevelCookedFromAsset(const FAssetData& Asset);
	ENGINE_API static bool GetIsLevelCookedFromPackage(FName LevelPackage);

	UE_DEPRECATED(5.3, "GetPartitionedLevelCanBeUsedByLevelInstanceFromAsset is deprecated.")
	static bool GetPartitionedLevelCanBeUsedByLevelInstanceFromAsset(const FAssetData& Asset) { return true; }
	UE_DEPRECATED(5.3, "GetPartitionedLevelCanBeUsedByLevelInstanceFromPackage is deprecated.")
	static bool GetPartitionedLevelCanBeUsedByLevelInstanceFromPackage(FName LevelPackage) { return true; }
	UE_DEPRECATED(5.4, "GetLevelScriptExternalActorsReferencesFromAsset is deprecated.")
	static bool GetLevelScriptExternalActorsReferencesFromAsset(const FAssetData& Asset, TArray<FGuid>& OutLevelScriptExternalActorsReferences) { return false;}
	UE_DEPRECATED(5.4, "GetLevelScriptExternalActorsReferencesFromPackageis deprecated.")
	static bool GetLevelScriptExternalActorsReferencesFromPackage(FName LevelPackage, TArray<FGuid>& OutLevelScriptExternalActorsReferences) { return false; }

	ENGINE_API static FVector GetLevelInstancePivotOffsetFromAsset(const FAssetData& Asset);
	ENGINE_API static FVector GetLevelInstancePivotOffsetFromPackage(FName LevelPackage);

	ENGINE_API static const FName LoadAllExternalObjectsTag;
	ENGINE_API static const FName DontLoadExternalObjectsTag;
	ENGINE_API static const FName DontLoadExternalFoldersTag;

	ENGINE_API bool GetPromptWhenAddingToLevelOutsideBounds() const;
	ENGINE_API bool GetPromptWhenAddingToLevelBeforeCheckout() const;

	void SetEditorPathOwner(UObject* InEditorPathOwner) { EditorPathOwner = InEditorPathOwner; }
	virtual UObject* GetEditorPathOwner() const override { return EditorPathOwner.Get(); }

	bool GetForceCantReuseUnloadedButStillAround() const { return bForceCantReuseUnloadedButStillAround; }
	void SetForceCantReuseUnloadedButStillAround(bool bNewValue) { bForceCantReuseUnloadedButStillAround = bNewValue; }
#endif

private:
	enum class EIncrementalUnregisterComponentState : uint8
	{
		Init,
		PreUnregisterComponents,
		UnregisterComponents,
		Finalize
	};

	EIncrementalUnregisterComponentState IncrementalUnregisterComponentState;

	FLevelBoundsActorUpdatedEvent LevelBoundsActorUpdatedEvent; 

	UPROPERTY()
	TObjectPtr<AWorldSettings> WorldSettings;

	UPROPERTY()
	TObjectPtr<AWorldDataLayers> WorldDataLayers;

	UPROPERTY()
	TSoftObjectPtr<UWorldPartitionRuntimeCell> WorldPartitionRuntimeCell;

	/** Cached level collection that this level is contained in, for faster access than looping through the collections in the world. */
	FLevelCollection* CachedLevelCollection;

	void OnPreRegisterComponent(UActorComponent* Component)
	{
		check(IsValid(Component));
		PreRegisteringActorComponents.Add(Component);
	}

	void OnRemovedPreRegisteringComponent(UActorComponent* Component)
	{
		check(IsValid(Component));
		verify(PreRegisteringActorComponents.Remove(Component));
	}

	void OnPreRegisteredComponent(UActorComponent* Component)
	{
		OnRemovedPreRegisteringComponent(Component);
	}

	bool HasPreRegisteringComponents() const
	{
		return PreRegisteringActorComponents.Num() > 0;
	}

	void OnPreUnregisterComponent(UActorComponent* Component) 
	{
		check(IsValid(Component));
		PreUnregisteringActorComponents.Add(Component);
	}

	void OnRemovedPreUnregisteringComponent(UActorComponent* Component) 
	{
		check(IsValid(Component));
		verify(PreUnregisteringActorComponents.Remove(Component));
	}

	void OnPreUnregisteredComponent(UActorComponent* Component) 
	{
		OnRemovedPreUnregisteringComponent(Component);
	}

	bool HasPreUnregisteringComponents() const
	{
		return PreUnregisteringActorComponents.Num() > 0;
	}

	TSet<UActorComponent*> PreUnregisteringActorComponents;
	TSet<UActorComponent*> PreRegisteringActorComponents;
	friend struct FLevelRegistrationAccessor;
	friend struct FLevelPendingVisibilityStateAccessor;
	friend struct FLevelRemoveFromWorldAccessor;

protected:

	/** Array of user data stored with the asset */
	UPROPERTY()
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

private:
	// Actors awaiting input to be enabled once the appropriate PlayerController has been created
	TArray<FPendingAutoReceiveInputActor> PendingAutoReceiveInputActors;

	/** List of replicated static actors that have been destroyed. Used by net drivers to replicate destruction to clients. */
	UPROPERTY(Transient)
	TArray<FReplicatedStaticActorDestructionInfo> DestroyedReplicatedStaticActors;

#if WITH_EDITORONLY_DATA
	/** Use actor folder objects, actor folders of this level will be persistent in their own object. */
	UPROPERTY(EditInstanceOnly, Category = World)
	bool bUseActorFolders;

	/** Actor folder objects. They can either be saved inside level or in their own package. */
	UPROPERTY(Transient)
	TMap<FGuid, TObjectPtr<UActorFolder>> ActorFolders;

	/** Acceleration table used to find an ActorFolder object for a given folder path. */
	UPROPERTY(Transient)
	TMap<FString, FActorFolderSet> FolderLabelToActorFolders;

	/** Temporary array containing actor folder objects manually loaded from their external packages (only used while loading the level). */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UActorFolder>> LoadedExternalActorFolders;

	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> EditorPathOwner;
#endif // #if WITH_EDITORONLY_DATA

	enum class ERouteActorInitializationState : uint8
	{
		Preinitialize,
		Initialize,
		BeginPlay,
		Finished
	};
	ERouteActorInitializationState RouteActorInitializationState;
	int32 RouteActorInitializationIndex;
	int32 RouteActorEndPlayForRemoveFromWorldIndex;

public:
	// Used internally to determine which actors should go on the world's NetworkActor list
	ENGINE_API static bool IsNetActor(const AActor* Actor);

	/** Populate an entry for Actor in the DestroyedReplicatedStaticActors list */
	void CreateReplicatedDestructionInfo(AActor* const Actor);

	const TArray<FReplicatedStaticActorDestructionInfo>& GetDestroyedReplicatedStaticActors() const;

	//** DO NOT USE - Part of internal networking implentation iteration. */
	ENGINE_API void AddReplicatedDestructionInfo(AActor* const Actor);

	/** Called when a level package has been dirtied. */
	ENGINE_API static FSimpleMulticastDelegate LevelDirtiedEvent;

	// Constructor.
	ENGINE_API void Initialize(const FURL& InURL);
	ULevel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	ULevel(FVTableHelper& Helper)
		: Super(Helper)
		, Actors()
	{}

	~ULevel();

	//~ Begin UObject Interface.
	virtual void PostInitProperties() override;	
	virtual void Serialize( FArchive& Ar ) override;
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void FinishDestroy() override;
	ENGINE_API virtual UWorld* GetWorld() const override final;

#if WITH_EDITOR
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;	
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
#endif // WITH_EDITOR
	virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists) override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	virtual bool CanBeClusterRoot() const override;
	virtual void CreateCluster() override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.

	void PreSaveFromRoot(FObjectPreSaveRootContext ObjectSaveContext);
	void PostSaveFromRoot(FObjectPostSaveRootContext ObjectSaveContext);

#if WITH_EDITOR
	void PreRenameFromRoot(const FWorldRenameFromRootContext& RenameContext);
	void PostRenameFromRoot(const FWorldRenameFromRootContext& RenameContext);

	//~Begin ITextureStreamingContainer Interface.
	virtual void InitializeTextureStreamingContainer(uint32 InPackedTextureStreamingQualityLevelFeatureLevel) override;
	virtual uint16 RegisterStreamableTexture(UTexture* InTexture) override;
	//~End ITextureStreamingContainer Interface.
	ENGINE_API uint16 RegisterStreamableTexture(const FString& InTextureName, const FGuid& InTextureGuid);
#endif

	/**
	 * Flag this level instance for destruction.
	 * This is called by UWorld::CleanupWorld to flag the level and its owned packages for destruction.
	 *
	 * @param bCleanupResources Whether to uninit anything that was initialized through OnLevelLoaded()
	 * @param bUnloadFromEditor Whether to also remove RF_Standalone flags in the editor on level-specific subassets so that they can be garbage collected
	 */
	ENGINE_API void CleanupLevel(bool bCleanupResources = true, bool bUnloadFromEditor = true);

	/**
	 * Clears all components of actors associated with this level (aka in Actors array) and 
	 * also the BSP model components.
	 */
	ENGINE_API void ClearLevelComponents();

	/**
	 * Updates all components of actors associated with this level (aka in Actors array) and 
	 * creates the BSP model components.
	 * @param bRerunConstructionScripts	If we want to rerun construction scripts on actors in level
	 */
	ENGINE_API void UpdateLevelComponents(bool bRerunConstructionScripts, FRegisterComponentContext* Context = nullptr);

	/**
	 * Incrementally updates all components of actors associated with this level.
	 *
	 * @param NumComponentsToUpdate		Number of components to update in this run, 0 for all
	 * @param bRerunConstructionScripts	If we want to rerun construction scripts on actors in level
	 */
	void IncrementalUpdateComponents( int32 NumComponentsToUpdate, bool bRerunConstructionScripts, FRegisterComponentContext* Context = nullptr);

	/**
	* Incrementally unregisters all components of actors associated with this level.
	* This is done at the granularity of actors (individual actors have all of their components unregistered)
    *
	* @param NumComponentsToUnregister		Minimum number of components to unregister in this run, 0 for all
	*/
	bool IncrementalUnregisterComponents(int32 NumComponentsToUnregister, FUnregisterComponentContext* Context = nullptr);


	/**
	 * Invalidates the cached data used to render the level's UModel.
	 */
	void InvalidateModelGeometry();
	
	/** Marks all level components render state as dirty */
	ENGINE_API void MarkLevelComponentsRenderStateDirty();

#if WITH_EDITOR
	/** Called to create ModelComponents for BSP rendering */
	void CreateModelComponents();
#endif // WITH_EDITOR

	/**
	 * Updates the model components associated with this level
	 */
	ENGINE_API void UpdateModelComponents();

	/**
	 * Commits changes made to the UModel's surfaces.
	 */
	ENGINE_API void CommitModelSurfaces();

	/**
	 * Discards the cached data used to render the level's UModel.  Assumes that the
	 * faces and vertex positions haven't changed, only the applied materials.
	 */
	void InvalidateModelSurface();

	/**
	 * Sorts the actor list by net relevancy and static behaviour. First all not net relevant static
	 * actors, then all net relevant static actors and then the rest. This is done to allow the dynamic
	 * and net relevant actor iterators to skip large amounts of actors.
	 */
	ENGINE_API void SortActorList();

#if WITH_EDITOR
	/**
	 * Add a dynamically loaded actor to this level, as if it was part of the original map load process.
	 *
	 * @param Actor				The actor to add to the level
	 * @param TransformToApply	The transform to apply to this actor if it's not already in the level
	 */
	ENGINE_API void AddLoadedActor(AActor* Actor, const FTransform* TransformToApply = nullptr);
	ENGINE_API void AddLoadedActors(const TArray<AActor*>& ActorList, const FTransform* TransformToApply = nullptr);

	/**
	 * Remove a dynamically loaded actor from this level.
	 *
	 * @param Actor				The actor to remove from the level
	 * @param TransformToRemove	The transform that was applied to this actor when it was added to the level
	 */
	ENGINE_API void RemoveLoadedActor(AActor* Actor, const FTransform* TransformToRemove = nullptr);
	ENGINE_API void RemoveLoadedActors(const TArray<AActor*>& ActorList, const FTransform* TransformToRemove = nullptr);

	/** Called when dynamically loaded actors are being added to this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorAddedToLevelPreEvent, const TArray<AActor*>&);
	FLoadedActorAddedToLevelPreEvent OnLoadedActorAddedToLevelPreEvent;

	/** Called when dynamically loaded actor is added to this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorAddedToLevelEvent, AActor&);
	FLoadedActorAddedToLevelEvent OnLoadedActorAddedToLevelEvent;

	/** Called when dynamically loaded actors were added to this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorAddedToLevelPostEvent, const TArray<AActor*>&);
	FLoadedActorAddedToLevelPostEvent OnLoadedActorAddedToLevelPostEvent;

	/** Called when dynamically loaded actors are being removed from this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorRemovedFromLevelPreEvent, const TArray<AActor*>&);
	FLoadedActorRemovedFromLevelPreEvent OnLoadedActorRemovedFromLevelPreEvent;

	/** Called when dynamically loaded actor is removed from this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorRemovedFromLevelEvent, AActor&);
	FLoadedActorRemovedFromLevelEvent OnLoadedActorRemovedFromLevelEvent;

	/** Called when dynamically loaded actors were removed from this level */
	DECLARE_EVENT_OneParam(ULevel, FLoadedActorRemovedFromLevelPostEvent, const TArray<AActor*>&);
	FLoadedActorRemovedFromLevelPostEvent OnLoadedActorRemovedFromLevelPostEvent;
#endif

	/* Called when level is loaded. */
	ENGINE_API void OnLevelLoaded();

	virtual bool IsNameStableForNetworking() const override { return true; }		// For now, assume all levels have stable net names

	/** Handles network initialization for actors in this level */
	void InitializeNetworkActors();

	void ClearActorsSeamlessTraveledFlag();

	/** Initializes rendering resources for this level. */
	ENGINE_API void InitializeRenderingResources();

	/** Releases rendering resources for this level. */
	ENGINE_API void ReleaseRenderingResources();

	/**
	 * Returns whether the level has completed routing actor initialization.
	 */
	bool IsFinishedRouteActorInitialization() const { return RouteActorInitializationState == ERouteActorInitializationState::Finished; }

	/**
	 * Method for resetting routing actor initialization for the next time this level is streamed.
	 */
	void ResetRouteActorInitializationState();

	/**
	 * Routes pre and post initialize to actors and also sets volumes.
	 *
	 * @param NumActorsToProcess	The maximum number of actors to update in this pass, 0 to process all actors.
	 * @todo seamless worlds: this doesn't correctly handle volumes in the multi- level case
	 */
	void RouteActorInitialize(int32 NumActorsToProcess);

	/**
	 * Rebuilds static streaming data for all levels in the specified UWorld.
	 *
	 * @param World				Which world to rebuild streaming data for. If NULL, all worlds will be processed.
	 * @param TargetLevel		[opt] Specifies a single level to process. If NULL, all levels will be processed.
	 * @param TargetTexture		[opt] Specifies a single texture to process. If NULL, all textures will be processed.
	 */
	ENGINE_API static void BuildStreamingData(UWorld* World, ULevel* TargetLevel=NULL, UTexture2D* TargetTexture=NULL);

	/**
	 * Returns the default brush for this level.
	 *
	 * @return		The default brush for this level.
	 */
	ENGINE_API ABrush* GetDefaultBrush() const;

	/**
	 * Returns the world info for this level.
	 *
	 * @return		The AWorldSettings for this level.
	 */
	ENGINE_API AWorldSettings* GetWorldSettings(bool bChecked = true) const;

	ENGINE_API void SetWorldSettings(AWorldSettings* NewWorldSettings);

	/**
	 * Returns the world data layers info for this level.
	 *
	 * @return		The AWorldDataLayers for this level.
	 */
	ENGINE_API AWorldDataLayers* GetWorldDataLayers() const;

	ENGINE_API void SetWorldDataLayers(AWorldDataLayers* NewWorldDataLayers);

	/**
	 * Returns the RuntimeCell associated with this Level if it is a level representing a cell of a WorldPartition World.
	 *
	 * @return		The cell associated with the level.
	 */
	ENGINE_API const IWorldPartitionCell* GetWorldPartitionRuntimeCell() const;

	/**
	 * Returns if the level is a cell from a WorldPartition World.
	 *
	 * @return		The cell associated with the level.
	 */
	bool IsWorldPartitionRuntimeCell() const { return !WorldPartitionRuntimeCell.GetUniqueID().IsNull(); }

	/**
	 * Returns the UWorldPartition for this level.
	 *
	 * @return		The UWorldPartition for this level (nullptr if not found).
	 */
	ENGINE_API class UWorldPartition* GetWorldPartition() const;

	/**
	 * Returns the level scripting actor associated with this level
	 * @return	a pointer to the level scripting actor for this level (may be NULL)
	 */
	ENGINE_API class ALevelScriptActor* GetLevelScriptActor() const;

	/** Returns the cached collection that contains this level, if any. May be null. */
	FLevelCollection* GetCachedLevelCollection() const { return CachedLevelCollection; }

	/** Sets the cached level collection that contains this level. Should only be called by FLevelCollection. */
	void SetCachedLevelCollection(FLevelCollection* const InCachedLevelCollection) { CachedLevelCollection = InCachedLevelCollection; }

	/**
	 * Utility searches this level's actor list for any actors of the specified type.
	 */
	bool HasAnyActorsOfType(UClass *SearchType);

	/**
	 * Resets the level nav list.
	 */
	ENGINE_API void ResetNavList();

	ENGINE_API UPackage* CreateMapBuildDataPackage() const;

	ENGINE_API UMapBuildDataRegistry* GetOrCreateMapBuildData();

	ENGINE_API bool IsMapBuildDataOwner() const;

	/** Returns level's pending visibility state */
	ENGINE_API ELevelPendingVisibilityState GetPendingVisibilityState() const;

	/** Sets whether this level is a lighting scenario and handles propagating the change. */
	ENGINE_API void SetLightingScenario(bool bNewIsLightingScenario);

	/** Creates UMapBuildDataRegistry entries for legacy lightmaps from components loaded for this level. */
	ENGINE_API void HandleLegacyMapBuildData();

#if WITH_EDITOR
	/**
	 * Get the package name for this actor
	 * @param InLevelPackage the package to get the external actors package name of
	 * @param InActorPackagingScheme the packaging scheme to use
	 * @param InActorPath the fully qualified actor path, in the format: 'Outermost.Outer.Name'
	 * @param InLevelMountPointContext an optional context object used to determine the mount point of the package
	 * @return the package name
	 */
	static ENGINE_API FString GetActorPackageName(UPackage* InLevelPackage, EActorPackagingScheme InActorPackagingScheme, const FString& InActorPath, const UObject* InLevelMountPointContext = nullptr);

	/**
	 * Returns a resolved level path using the level mount point context
	 * @param InLevelPackage the level package name
	 * @param InLevelMountPointContext an optional context object used to determine the mount point of the package
	 */
	static ENGINE_API FString ResolveRootPath(const FString& LevelPackageName, const UObject* InLevelMountPointContext = nullptr);

	/**
	 * Get the package name for this actor
	 * @param InBaseDir the base directory used when building the actor package name
	 * @param InActorPackagingScheme the packaging scheme to use
	 * @param InActorPath the fully qualified actor path, in the format: 'Outermost.Outer.Name'
	 * @return the package name
	 */
	static ENGINE_API FString GetActorPackageName(const FString& InBaseDir, EActorPackagingScheme InActorPackagingScheme, const FString& InActorPath);

	/**
	 * Get the folder containing the external actors for this level path
	 * @param InLevelPackageName The package name to get the external actors path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the folder
	 */
	static ENGINE_API FString GetExternalActorsPath(const FString& InLevelPackageName, const FString& InPackageShortName = FString());

	/**
	 * Get the folders containing the external actors for this level path, including actor folders of registered plugins for this level 
	 * @param InLevelPackageName The package name to get the external actors path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the folder
	 */
	static ENGINE_API TArray<FString> GetExternalActorsPaths(const FString& InLevelPackageName, const FString& InPackageShortName = FString());

	/**
	 * Get the folder containing the external actors for this level
	 * @param InLevelPackage The package to get the external actors path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the folder
	 */
	static ENGINE_API FString GetExternalActorsPath(UPackage* InLevelPackage, const FString& InPackageShortName = FString());

	/**
	 * Extract the packaging Scheme used by an external actor package based on the name of the package
	 */
	static ENGINE_API EActorPackagingScheme GetActorPackagingSchemeFromActorPackageName(const FStringView InActorPackageName);

	/**
	 * Scans/Updates all Level Assets (level package and external packages)
	 */
	static ENGINE_API void ScanLevelAssets(const FString& InLevelPackageName);

	/**
	 * Get the folder name from which all external actors paths are created
	 * @return folder name
	 */
	static ENGINE_API const TCHAR* GetExternalActorsFolderName();

	/** Returns true if the level uses external actors mode. */
	ENGINE_API bool IsUsingExternalActors() const;

	/** Sets if the level uses external actors mode or not. */
	ENGINE_API void SetUseExternalActors(bool bEnable);

	/**
	 * Get the folders containing the external objects for this level path
	 * @param InLevelPackageName The package name to get the external objects path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the folders
	 */
	static ENGINE_API TArray<FString> GetExternalObjectsPaths(const FString& InLevelPackageName, const FString& InPackageShortName = FString());

	/** Returns true if the level uses external objects. */
	ENGINE_API bool IsUsingExternalObjects() const;

	/** Returns true if the level uses actor folders mode. */
	ENGINE_API bool IsUsingActorFolders() const;

	/** Updates all actors/folders that refer to folders marked as deleted, reparent to valid folder, deletes folders marked as deleted. */
	ENGINE_API void CreateOrUpdateActorFolders();

	/** Deletes all actor folders marked as deleted and unreferenced by neither an actor nor another actor folder */
	ENGINE_API void CleanupDeletedAndUnreferencedActorFolders();

	/** Sets if the level uses actor folders mode or not. Returns true if succeeded. */
	ENGINE_API bool SetUseActorFolders(bool bEnabled, bool bInteractive = false);

	/** Finds the level actor folder by its guid. Returns null if not found. */
	ENGINE_API UActorFolder* GetActorFolder(const FGuid& InGuid, bool bSkipDeleted = true) const;

	/** Finds the level actor folder by its path. Returns null if not found. */
	ENGINE_API UActorFolder* GetActorFolder(const FName& InPath) const;

	/** Iterates on all valid level actor folders. */
	ENGINE_API void ForEachActorFolder(TFunctionRef<bool(UActorFolder*)> Operation, bool bSkipDeleted = false);

	/** Returns true if the level wants newly spawned actors to be external */
	ENGINE_API bool ShouldCreateNewExternalActors() const;

	/** Returns the level's actor packaging scheme */
	EActorPackagingScheme GetActorPackagingScheme() const { return ActorPackagingScheme; }

	/** 
	 * Convert this level actors to the specified loading strategy
	 * @param bExternal if true will convert internal actors to external, will convert external actors to internal otherwise
	 * @note does not affect the level bUseExternalActors flag
	 */
	ENGINE_API void ConvertAllActorsToPackaging(bool bExternal);

	/**
	* Get a properly formated external actor package instance name for this level package to be used in FLinkerInstancingContext
	* @return external actor package instance name
	*/
	static ENGINE_API FString GetExternalActorPackageInstanceName(const FString& LevelPackageName, const FString& ActorPackageName);

	/**
	 * Get the list of (on disk) external actor packages associated with this external actors path
	 * @param ExternalActorsPath the path to scan for external actor packages
	 * @return Array of packages associated with this level
	 */
	static ENGINE_API TArray<FString> GetOnDiskExternalActorPackages(const FString& ExternalActorsPath);

	/**
	 * Get the list of (on disk) external actor packages associated with this level
	 * @return Array of packages associated with this level
	 */
	ENGINE_API TArray<FString> GetOnDiskExternalActorPackages(bool bTryUsingPackageLoadedPath = false) const;

	/**
	 * Get the list of (loaded) external object packages (actors/folders) associated with this level
	 * @return Array of packages associated with this level
	 */
	ENGINE_API TArray<UPackage*> GetLoadedExternalObjectPackages() const;

	/**
	 * Create an package for this actor
	 * @param InLevelPackage the level package used when building the actor package name
	 * @param InActorPackagingScheme the packaging scheme to use
	 * @param InActorPath the fully qualified actor path, in the format: 'Outermost.Outer.Name'
	 * @param InMountPointContext an optional context object used to determine the mount point of the package
	 * @return the created package
	 */
	static ENGINE_API UPackage* CreateActorPackage(UPackage* InLevelPackage, EActorPackagingScheme InActorPackagingScheme, const FString& InActorPath, const UObject* InMountPointContext = nullptr);

	/** 
	*  Called after lighting was built and data gets propagated to this level
	*  @param	bLightingSuccessful	 Whether lighting build was successful
	*/
	ENGINE_API void OnApplyNewLightingData(bool bLightingSuccessful);

	/**
	 *	Grabs a reference to the level scripting blueprint for this level.  If none exists, it creates a new blueprint
	 *
	 * @param	bDontCreate		If true, if no level scripting blueprint is found, none will be created
	 */
	ENGINE_API class ULevelScriptBlueprint* GetLevelScriptBlueprint(bool bDontCreate=false);

	/**
	 * Nulls certain references related to the LevelScriptBlueprint. Called by UWorld::CleanupWorld.
	 */
	ENGINE_API void CleanupLevelScriptBlueprint();

	/**
	 *  Called when the level script blueprint has been successfully changed and compiled.  Handles creating an instance of the blueprint class in LevelScriptActor
	 */
	ENGINE_API void OnLevelScriptBlueprintChanged(class ULevelScriptBlueprint* InBlueprint);

	/** 
	 * Call on a level that was loaded from disk instead of PIE-duplicating, to fixup actor references
	 */
	ENGINE_API void FixupForPIE(int32 PIEInstanceID);
	ENGINE_API void FixupForPIE(int32 PIEInstanceID, TFunctionRef<void(int32, FSoftObjectPath&)> CustomFixupFunction);

	/**
	 * Returns true if the level contains static meshes that have not finished compiling yet.
	 */
	ENGINE_API bool HasStaticMeshCompilationPending();
#endif

	/** @todo document */
	TArray<FVector> const* GetStaticNavigableGeometry() const { return &StaticNavigableGeometry;}

	/** 
	* Is this the persistent level 
	*/
	ENGINE_API bool IsPersistentLevel() const;

	/** 
	* Is this the current level in the world it is owned by
	*/
	ENGINE_API bool IsCurrentLevel() const;
	
	/**
	 * Is this a level instance
	 */
	ENGINE_API bool IsInstancedLevel() const;

	/** 
	 * Shift level actors by specified offset
	 * The offset vector will get subtracted from all actors positions and corresponding data structures
	 *
	 * @param InWorldOffset	 Vector to shift all actors by
	 * @param bWorldShift	 Whether this call is part of whole world shifting
	 */
	ENGINE_API void ApplyWorldOffset(const FVector& InWorldOffset, bool bWorldShift);

	/** Register an actor that should be added to a player's input stack when they are created */
	void RegisterActorForAutoReceiveInput(AActor* Actor, const int32 PlayerIndex);

	/** Push any pending auto receive input actor's input components on to the player controller's input stack */
	void PushPendingAutoReceiveInput(APlayerController* PC);
	
	//~ Begin IInterface_AssetUserData Interface
	virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	//~ End IInterface_AssetUserData Interface

	/** Estimate the amount of AddToWorld work for this level. Used by the adaptive level streaming timeslice (see s.AdaptiveAddToWorld.Enabled) */
	int32 GetEstimatedAddToWorldWorkUnitsRemaining() const;
	/** Estimate the total amount of AddToWorld work for this level. Used by the adaptive level streaming timeslice (see s.AdaptiveAddToWorld.Enabled) */
	int32 GetEstimatedAddToWorldWorkUnitsTotal() const;

#if WITH_EDITOR
	DECLARE_DELEGATE_ThreeParams(FLevelExternalActorsPathsProviderDelegate, const FString&, const FString&, TArray<FString>&);
	/** Registers a level external actor paths provider */
	static ENGINE_API FDelegateHandle RegisterLevelExternalActorsPathsProvider(const FLevelExternalActorsPathsProviderDelegate& Provider);
	/** Unregisters a level external actor paths provider */
	static ENGINE_API void UnregisterLevelExternalActorsPathsProvider(const FDelegateHandle& ProviderDelegateHandle);

	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FLevelMountPointResolverDelegate, const FString&, const UObject*, FString&);
	/** Registers a level mount point resolver */
	static ENGINE_API FDelegateHandle RegisterLevelMountPointResolver(const FLevelMountPointResolverDelegate& Resolver);
	/** Unregisters a level mount point resolver */
	static ENGINE_API void UnregisterLevelMountPointResolver(const FDelegateHandle& ResolverDelegateHandle);
	/** Returns override spawning level mount point object */
	static const UObject* GetOverrideSpawningLevelMountPointObject() { return ULevel::OverrideSpawningLevelMountPointObject.Get(); }
#endif

private:
	void RouteActorEndPlayForRemoveFromWorld(int32 NumActorsToProcess);
	void ResetRouteActorEndPlayForRemoveFromWorld();
	bool IsFinishedRouteActorEndPlayForRemoveFromWorld() const;

	bool IncrementalPreRegisterComponents(FRegisterComponentContext& Context);
	bool IncrementalRegisterComponents(FRegisterComponentContext& Context);
	bool IncrementalPreUnregisterComponents(FUnregisterComponentContext& Context);
#if WITH_EDITOR
	bool IncrementalRunConstructionScripts(bool bProcessAllActors);
	TOptional<bool> bCachedHasStaticMeshCompilationPending;

	/** Array of registered delegates used by GetExternalActorsPaths. */
	static TArray<FLevelExternalActorsPathsProviderDelegate> LevelExternalActorsPathsProviders;

	/** Array of registered delegates used by GetExternalActorsPaths. */
	static TArray<FLevelMountPointResolverDelegate> LevelMountPointResolvers;

	/** Override spawning Level Mount Point object used when spawning an actor using external packaging */
	static ENGINE_API TWeakObjectPtr<const UObject> OverrideSpawningLevelMountPointObject;
private:
	/**
	 * Potentially defer the running of an actor's construction script on load
	 * The running will be deferred if the actor's user construction script is non trivial 
	 * and if there are outstanding asset compilation currently running (currently checks against static meshes)
	 * @param InActor The actor to check
	 * @return true if the running of the construction script was deferred
	 */
	bool DeferRunningConstructionScripts(AActor* InActor);

	/** Attempts to detect and fix any issues with the level script blueprint and associated objects */
	void RepairLevelScript();

	/** Prepares/fixes actor folder objects once level is fully loaded. */
	void FixupActorFolders();

	void AddActorFolder(UActorFolder* InActorFolder);
	void RemoveActorFolder(UActorFolder* InActorFolder);
	void OnFolderMarkAsDeleted(UActorFolder* InActorFolder);
	void OnFolderLabelChanged(UActorFolder* InActorFolder, const FString& InOldFolderLabel);

	/** Sets the level to use or not the actor folder objects feature. */
	void SetUseActorFoldersInternal(bool bInEnabled);

	/** Returns unreferenced actor folders that are marked as deleted. */
	TSet<FGuid> GetDeletedAndUnreferencedActorFolders() const;

	friend struct FScopedOverrideSpawningLevelMountPointObject;
	friend struct FLevelActorFoldersHelper;
	friend struct FSetWorldPartitionRuntimeCell;
	friend class FWorldPartitionLevelHelper;
	friend class UActorFolder;
	friend class AWorldDataLayers;
	friend class UWorldPartitionRuntimeHashSet;

	/** Replace the existing LSA (if set) by spawning a new one based on this level's script blueprint */
	void RegenerateLevelScriptActor();

	/** Find and destroy any extra LSAs, as they will cause duplicated level script operations */
	void RemoveExtraLevelScriptActors();

	/** Notification popup used to guide the user to repair multiple LSAs detected upon loading in the editor */
	TWeakPtr<SNotificationItem> MultipleLSAsNotification;
	void OnMultipleLSAsPopupClicked();
	void OnMultipleLSAsPopupDismissed();

	/** Returns the optional cook package name (used by FUpdateLevelVisibilityLevelInfo) */
	ENGINE_API FName GetCookPackageName() const;
	friend struct FUpdateLevelVisibilityLevelInfo;

#endif // WITH_EDITOR
};

/** Accessor used by World when removing level */
struct FLevelRemoveFromWorldAccessor
{
private:
	static void RouteActorEndPlayForRemoveFromWorld(ULevel* InLevel, int32 InNumActorsToProcess)
	{
		InLevel->RouteActorEndPlayForRemoveFromWorld(InNumActorsToProcess);
	}

	static void ResetRouteActorEndPlayForRemoveFromWorld(ULevel* InLevel)
	{
		InLevel->ResetRouteActorEndPlayForRemoveFromWorld();
	}

	static bool IsFinishedRouteActorEndPlayForRemoveFromWorld(ULevel* InLevel)
	{ 
		return InLevel->IsFinishedRouteActorEndPlayForRemoveFromWorld();
	}

	friend class UWorld;
};

/** Accessor used by World/ActorComponent for component registration/unregistration. */
struct FLevelRegistrationAccessor
{
private:
	static void OnPreRegisterComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnPreRegisterComponent(InComponent);
	}

	static void OnPreRegisteredComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnPreRegisteredComponent(InComponent);
	}

	static void OnRemovedPreRegisteringComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnRemovedPreRegisteringComponent(InComponent);
	}

	static bool HasPreRegisteringComponents(const ULevel* InLevel)
	{
		return InLevel->HasPreRegisteringComponents();
	}

	static void OnPreUnregisterComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnPreUnregisterComponent(InComponent);
	}

	static void OnPreUnregisteredComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnPreUnregisteredComponent(InComponent);
	}

	static void OnRemovedPreUnregisteringComponent(ULevel* InLevel, UActorComponent* InComponent)
	{
		InLevel->OnRemovedPreUnregisteringComponent(InComponent);
	}

	static bool HasPreUnregisteringComponents(const ULevel* InLevel)
	{
		return InLevel->HasPreUnregisteringComponents();
	}

	friend class FRegisterComponentContext;
	friend class UActorComponent;
	friend class UWorld;
};

struct FLevelPendingVisibilityStateAccessor
{
private:
	static void SetPendingVisibilityState(ULevel* InLevel, ELevelPendingVisibilityState InState)
	{
		InLevel->PendingVisibilityState = InState;
	}

	friend class UWorld;
};

#if WITH_EDITOR
struct FScopedOverrideSpawningLevelMountPointObject
{
	FScopedOverrideSpawningLevelMountPointObject(const UObject* InObject)
	{
		PreviousValue = ULevel::OverrideSpawningLevelMountPointObject;
		ULevel::OverrideSpawningLevelMountPointObject = InObject;
	}
	~FScopedOverrideSpawningLevelMountPointObject()
	{
		ULevel::OverrideSpawningLevelMountPointObject = PreviousValue;
	}
private:
	TWeakObjectPtr<const UObject> PreviousValue;
};

struct FSetWorldPartitionRuntimeCell
{
private:
	FSetWorldPartitionRuntimeCell(ULevel* InLevel, const FSoftObjectPath& InWorldPartitionRuntimeCell)
	{
		InLevel->WorldPartitionRuntimeCell = InWorldPartitionRuntimeCell;
	}
	friend class FWorldPartitionLevelHelper;
	friend class UWorldPartition;
};
#endif 

/**
 * Macro for wrapping Delegates in TScopedCallback
 */
 #define DECLARE_SCOPED_DELEGATE( CallbackName, TriggerFunc )						\
	class FScoped##CallbackName##Impl												\
	{																				\
	public:																			\
		static void FireCallback() { TriggerFunc; }									\
	};																				\
																					\
	typedef TScopedCallback<FScoped##CallbackName##Impl> FScoped##CallbackName;

DECLARE_SCOPED_DELEGATE( LevelDirtied, ULevel::LevelDirtiedEvent.Broadcast() );

#undef DECLARE_SCOPED_DELEGATE
