// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "LevelActorContainer.generated.h"

class AActor;

/**
 * Root object for all level actors
 */
UCLASS(MinimalAPI, DefaultToInstanced)
class ULevelActorContainer : public UObject
{
	friend class ULevel;

	GENERATED_BODY()

public:

	/** Array of actors in a level */
	UPROPERTY(transient)
	TArray<TObjectPtr<AActor>> Actors;

	virtual void CreateCluster() override;
	virtual void OnClusterMarkedAsPendingKill() override;
};
