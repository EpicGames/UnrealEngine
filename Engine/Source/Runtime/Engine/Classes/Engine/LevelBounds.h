// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Tickable.h"
#include "LevelBounds.generated.h"

/**
 * Defines level bounds
 * Updates bounding box automatically based on actors transformation changes or holds fixed user defined bounding box
 * Uses only actors where AActor::IsLevelBoundsRelevant() == true
 */
UCLASS(hidecategories=(Advanced, Collision, Display, Rendering, Physics, Input), showcategories=("Input|MouseInput", "Input|TouchInput"), MinimalAPI)
class ALevelBounds
	: public AActor
	, public FTickableGameObject 
{
	GENERATED_BODY()

public:
	ENGINE_API ALevelBounds(const FObjectInitializer& ObjectInitializer);
		
	/** Bounding box for the level bounds. */
	UPROPERTY(EditAnywhere, Category = LevelBounds)
	TObjectPtr<class UBoxComponent> BoxComponent;

	/** Whether to automatically update actor bounds based on all relevant actors bounds belonging to the same level */
	UPROPERTY(EditAnywhere, Category=LevelBounds)
	bool bAutoUpdateBounds;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin AActor Interface.
	ENGINE_API virtual FBox GetComponentsBoundingBox(bool bNonColliding = false, bool bIncludeFromChildActors = false) const override;
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	//~ End AActor Interface.

	/** @return Bounding box which includes all relevant actors bounding boxes belonging to specified level */
	static ENGINE_API FBox CalculateLevelBounds(const ULevel* InLevel);

#if WITH_EDITOR
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditMove(bool bFinished) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual void PostUnregisterAllComponents() override;
	
	/** Marks level bounds as dirty so they will be recalculated on next tick */
	ENGINE_API void MarkLevelBoundsDirty();

	/** @return True if there were no actors contributing to bounds and we are currently using the default bounds */
	ENGINE_API bool IsUsingDefaultBounds() const;

	/** Update level bounds immediately so the bounds are accurate when returning. Use only when needed because updating the bounds is slow */
	ENGINE_API void UpdateLevelBoundsImmediately();

	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	
protected:
	/** FTickableGameObject interface, this ticks only in the editor regardless of viewport 'Realtime' option */
	ENGINE_API virtual void Tick(float DeltaTime) override;
	virtual UWorld* GetTickableGameObjectWorld() const override { return GetWorld(); }
	ENGINE_API virtual TStatId GetStatId() const override;
	ENGINE_API virtual bool IsTickable() const override;
	ENGINE_API virtual bool IsTickableInEditor() const override;
	
	/** Updates this actor bounding box by summing all level actors bounding boxes  */
	ENGINE_API void UpdateLevelBounds();

	/** Broadcasts LevelBoundsActorUpdatedEvent in case this actor acts as a level bounds */
	ENGINE_API void BroadcastLevelBoundsUpdated();

	/** Called whenever any actor moved  */
	ENGINE_API void OnLevelActorMoved(AActor* InActor);
	
	/** Called whenever any actor added or removed  */
	ENGINE_API void OnLevelActorAddedRemoved(AActor* InActor);

	/** Subscribes for actors transformation events */
	ENGINE_API void SubscribeToUpdateEvents();
	
	/** Unsubscribes from actors transformation events */
	ENGINE_API void UnsubscribeFromUpdateEvents();
	
	/** Whether currently level bounds is dirty and needs to be updated  */
	bool bLevelBoundsDirty;

	/** True when there are no actors contributing to the bounds and we are currently using the default bounds instead */
	bool bUsingDefaultBounds;

	/** Handles to various registered delegates */
	FDelegateHandle OnLevelActorMovedDelegateHandle;
	FDelegateHandle OnLevelActorDeletedDelegateHandle;
	FDelegateHandle OnLevelActorAddedDelegateHandle;

private:
	virtual bool ActorTypeIsMainWorldOnly() const override { return true; }

#else
	// This never actually ticks as it starts disabled
	virtual void Tick(float DeltaTime) override {}
	virtual TStatId GetStatId() const override { return TStatId(); }
#endif // WITH_EDITOR
};
