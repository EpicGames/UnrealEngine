// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LevelScriptActor.generated.h"

/**
 * ALevelScriptActor is the base class for classes generated by 
 * ULevelScriptBlueprints. ALevelScriptActor instances are hidden actors that 
 * exist within a level, and can execute level-wide logic (operating on specific
 * actor instances within the level). The level-script's functionality is defined
 * inside the ULevelScriptBlueprint itself (using the blueprint's node-based 
 * interface).
 *
 * @see AActor
 * @see https://docs.unrealengine.com/latest/INT/Engine/Blueprints/UserGuide/Types/LevelBlueprint/index.html
 * @see ULevelScriptBlueprint
 * @see https://docs.unrealengine.com/latest/INT/Engine/Blueprints/index.html
 * @see UBlueprint
 */
UCLASS(notplaceable, meta=(ChildCanTick, KismetHideOverrides = "ReceiveAnyDamage,ReceivePointDamage,ReceiveRadialDamage,ReceiveActorBeginOverlap,ReceiveActorEndOverlap,ReceiveHit,ReceiveDestroyed,ReceiveActorBeginCursorOver,ReceiveActorEndCursorOver,ReceiveActorOnClicked,ReceiveActorOnReleased,ReceiveActorOnInputTouchBegin,ReceiveActorOnInputTouchEnd,ReceiveActorOnInputTouchEnter,ReceiveActorOnInputTouchLeave"), HideCategories=(Collision,Rendering,Transformation), MinimalAPI)
class ALevelScriptActor : public AActor
{
	GENERATED_UCLASS_BODY()

	// --- Utility Functions ----------------------------
	
	/** Tries to find an event named "EventName" on all other levels, and calls it */
	UFUNCTION(BlueprintCallable, meta=(BlueprintProtected = "true"), Category="Miscellaneous")
	ENGINE_API virtual bool RemoteEvent(FName EventName);

	/**
	 * Sets the cinematic mode on all PlayerControllers
	 *
	 * @param	bInCinematicMode	specify true if the player is entering cinematic mode; false if the player is leaving cinematic mode.
	 * @param	bHidePlayer			specify true to hide the player's pawn (only relevant if bInCinematicMode is true)
	 * @param	bAffectsHUD			specify true if we should show/hide the HUD to match the value of bCinematicMode
	 * @param	bAffectsMovement	specify true to disable movement in cinematic mode, enable it when leaving
	 * @param	bAffectsTurning		specify true to disable turning in cinematic mode or enable it when leaving
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintProtected = "true"), Category="Game|Cinematic")
	ENGINE_API virtual void SetCinematicMode(bool bCinematicMode, bool bHidePlayer = true, bool bAffectsHUD = true, bool bAffectsMovement = false, bool bAffectsTurning = false);

	// --- Level State Functions ------------------------
	/** @todo document */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly)
	ENGINE_API void LevelReset();

	/**
	 * Event called on world origin location changes
	 *
	 * @param	OldOriginLocation	Previous world origin location
	 * @param	NewOriginLocation	New world origin location
	 */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void WorldOriginLocationChanged(FIntVector OldOriginLocation, FIntVector NewOriginLocation);
	
	//~ Begin UObject Interface
	ENGINE_API virtual void PreInitializeComponents() override;
	//~ End UObject Interface

	//~ Begin AActor Interface
	ENGINE_API virtual void EnableInput(class APlayerController* PlayerController) override;
	ENGINE_API virtual void DisableInput(class APlayerController* PlayerController) override;

#if WITH_EDITOR
	virtual bool SupportsExternalPackaging() const override { return false; }

	/** Utility function for finding a map corruption issue that can cause duplicate level scripting events */
	ENGINE_API TArray<ALevelScriptActor*> FindSiblingLevelScriptActors() const;
#endif
	//~ End AActor Interface

	bool InputEnabled() const { return bInputEnabled; }

private:
	UPROPERTY()
	uint32 bInputEnabled:1;
};



