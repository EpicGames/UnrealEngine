// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Level.h"
#include "Engine/Blueprint.h"

#include "LevelScriptBlueprint.generated.h"

/**
 * A level blueprint is a specialized type of blueprint. It is used to house
 * global, level-wide logic. In a level blueprint, you can operate on specific 
 * level-actor instances through blueprint's node-based interface. Unreal Engine users 
 * should be familiar with this concept, as it is very similar to Kismet.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Blueprints/UserGuide/Types/LevelBlueprint/index.html
 * @see https://docs.unrealengine.com/latest/INT/Engine/Blueprints/index.html
 * @see UBlueprint
 * @see ALevelScriptActor
 */
UCLASS(MinimalAPI, NotBlueprintType)
class ULevelScriptBlueprint : public UBlueprint
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The friendly name to use for UI */
	UPROPERTY(transient)
	FString FriendlyName;
#endif

#if WITH_EDITOR
	ULevel* GetLevel() const
	{
		return Cast<ULevel>(GetOuter());
	}

	//~ Begin UBlueprint Interface
	ENGINE_API virtual FString GetFriendlyName() const override;
	//~ End UBlueprint Interface

	/** Generate a name for a level script blueprint from the current level */
	static FString CreateLevelScriptNameFromLevel (const class ULevel* Level );

#endif	//#if WITH_EDITOR
};
