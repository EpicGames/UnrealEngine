// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectAnnotation.h"
#include "Engine/LatentActionManager.h"
#include "LatentActions.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/Timeout.h"

#if WITH_EDITOR
#include "Folder.h"
#include "Misc/Optional.h"
#endif

class ALevelScriptActor;
class ALevelStreamingVolume;
class ULevel;
class ULevelStreaming;
class IWorldPartitionCell;
struct FNetLevelVisibilityTransactionId;

enum class ENetLevelVisibilityRequest
{
	MakingVisible,
	MakingInvisible
};

struct FNetLevelVisibilityState
{
	TOptional<bool> ClientAckedRequestCanMakeVisible;
	TOptional<ENetLevelVisibilityRequest> PendingRequestType;
	uint32 ServerRequestIndex = 0;
	uint32 ClientPendingRequestIndex = 0;
	uint32 ClientAckedRequestIndex = 0;
	bool bHasClientPendingRequest = false;

	void InvalidateClientPendingRequest()
	{
		bHasClientPendingRequest = false;
		PendingRequestType.Reset();
	}
};

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogLevelStreaming, Log, All);

// Stream Level Action
class FStreamLevelAction : public FPendingLatentAction
{
public:
	bool			bLoading;
	bool			bMakeVisibleAfterLoad;
	bool			bShouldBlock;
	TWeakObjectPtr<ULevelStreaming> Level;
	FName			LevelName;

	FLatentActionInfo LatentInfo;

	ENGINE_API FStreamLevelAction(bool bIsLoading, const FName& InLevelName, bool bIsMakeVisibleAfterLoad, bool bShouldBlock, const FLatentActionInfo& InLatentInfo, UWorld* World);

	/**
	 * Given a level name, returns level name that will work with Play on Editor or Play on Console
	 *
	 * @param	InLevelName		Raw level name (no UEDPIE or UED<console> prefix)
	 */
	static ENGINE_API FString MakeSafeLevelName( const FName& InLevelName, UWorld* InWorld );

	/**
	 * Helper function to potentially find a level streaming object by name and cache the result
	 *
	 * @param	LevelName							Name of level to search streaming object for in case Level is NULL
	 * @return	level streaming object or NULL if none was found
	 */
	static ENGINE_API ULevelStreaming* FindAndCacheLevelStreamingObject( const FName LevelName, UWorld* InWorld );

	/**
	 * Handles "Activated" for single ULevelStreaming object.
	 *
	 * @param	LevelStreamingObject	LevelStreaming object to handle "Activated" for.
	 */
	ENGINE_API void ActivateLevel( ULevelStreaming* LevelStreamingObject );
	/**
	 * Handles "UpdateOp" for single ULevelStreaming object.
	 *
	 * @param	LevelStreamingObject	LevelStreaming object to handle "UpdateOp" for.
	 *
	 * @return true if operation has completed, false if still in progress
	 */
	ENGINE_API bool UpdateLevel( ULevelStreaming* LevelStreamingObject );

	ENGINE_API virtual void UpdateOperation(FLatentResponse& Response) override;

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	ENGINE_API virtual FString GetDescription() const override;
#endif
};

#include "LevelStreaming.generated.h"

// Delegate signatures
DECLARE_DYNAMIC_MULTICAST_DELEGATE( FLevelStreamingLoadedStatus );
DECLARE_DYNAMIC_MULTICAST_DELEGATE( FLevelStreamingVisibilityStatus );

enum class ELevelStreamingState : uint8
{
	Removed,
	Unloaded,
	FailedToLoad,
	Loading,
	LoadedNotVisible,
	MakingVisible,
	LoadedVisible,
	MakingInvisible
};
ENGINE_API const TCHAR* EnumToString(ELevelStreamingState InState);

enum class ELevelStreamingTargetState : uint8
{
	Unloaded,
	UnloadedAndRemoved,
	LoadedNotVisible,
	LoadedVisible,
};
ENGINE_API const TCHAR* EnumToString(ELevelStreamingTargetState InTargetState);

/**
 * Abstract base class of container object encapsulating data required for streaming and providing 
 * interface for when a level should be streamed in and out of memory.
 *
 */
UCLASS(abstract, editinlinenew, BlueprintType, Within=World, MinimalAPI)
class ULevelStreaming : public UObject
{
	GENERATED_UCLASS_BODY()

	enum class UE_DEPRECATED(5.2, "ULevelStreaming::ECurrentState has been replaced by ELevelStreamingState")
	ECurrentState : uint8
	{
	 	Removed = (uint8)ELevelStreamingState::Removed,
		Unloaded = (uint8)ELevelStreamingState::Unloaded,
		FailedToLoad = (uint8)ELevelStreamingState::FailedToLoad,
		Loading = (uint8)ELevelStreamingState::Loading,
		LoadedNotVisible = (uint8)ELevelStreamingState::LoadedNotVisible,
		MakingVisible = (uint8)ELevelStreamingState::MakingVisible,
		LoadedVisible = (uint8)ELevelStreamingState::LoadedVisible,
		MakingInvisible = (uint8)ELevelStreamingState::MakingInvisible
	};
ENGINE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static const TCHAR* EnumToString(ECurrentState InCurrentState);
	friend bool operator==(ELevelStreamingState A, ECurrentState B)
	{
		return A == (ELevelStreamingState)B;
	}
	friend bool operator==(ECurrentState A, ELevelStreamingState B)
	{
		return (ELevelStreamingState)A == (ELevelStreamingState)B;
	}
	friend bool operator!=(ELevelStreamingState A, ECurrentState B)
	{
		return A != (ELevelStreamingState)B;
	}
	friend bool operator!=(ECurrentState A, ELevelStreamingState B)
	{
		return (ELevelStreamingState)A != (ELevelStreamingState)B;
	}
ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static ULevelStreaming* FindStreamingLevel(const ULevel* Level);
	static ENGINE_API void RemoveLevelAnnotation(const ULevel* Level);

	// Annotation for fast inverse lookup
	struct FLevelAnnotation
	{
		FLevelAnnotation() {}
		FLevelAnnotation(ULevelStreaming* InLevelStreaming) : LevelStreaming(InLevelStreaming){}
	
		inline bool IsDefault()
		{
			return !LevelStreaming;
		}

		ULevelStreaming* LevelStreaming = nullptr;
	};

	static ENGINE_API FUObjectAnnotationSparse<FLevelAnnotation, false> LevelAnnotations;


#if WITH_EDITORONLY_DATA
	/** Deprecated name of the package containing the level to load. Use GetWorldAsset() or GetWorldAssetPackageFName() instead.		*/
	UPROPERTY()
	FName PackageName_DEPRECATED;
#endif

protected:
	/** The reference to the world containing the level to load																	*/
	UPROPERTY(Category=LevelStreaming, VisibleAnywhere, BlueprintReadOnly, meta=(DisplayName = "Level", AllowPrivateAccess="true"))
	TSoftObjectPtr<UWorld> WorldAsset;

	/** The relative priority of considering the streaming level. Changing the priority will not interrupt the currently considered level, but will affect the next time a level is being selected for evaluation. */
	UPROPERTY(EditAnywhere, Category = LevelStreaming, BlueprintSetter = SetPriority)
	int32 StreamingPriority;

private:
	TOptional<int32> StreamingPriorityOverrideBackup;

public:

	/** If this isn't Name_None, then we load from this package on disk to the new package named PackageName					*/
	UPROPERTY()
	FName PackageNameToLoad;

	/** LOD versions of this level																								*/
	UPROPERTY()
	TArray<FName> LODPackageNames;

	/** LOD package names on disk																								*/
	TArray<FName> LODPackageNamesToLoad;

	/** Transform applied to actors after loading.                                                                              */
	UPROPERTY(EditAnywhere, Category=LevelStreaming, BlueprintReadWrite)
	FTransform LevelTransform;

	/** Applied to LoadedLevel						                                                                            */
	UPROPERTY()
	bool bClientOnlyVisible;

#if WITH_EDITORONLY_DATA
	/** Applied to LoadedLevel */
	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> EditorPathOwner;
#endif

private:

	/** Requested LOD. Non LOD sub-levels have Index = -1  */
	UPROPERTY(transient, Category = LevelStreaming, BlueprintSetter = SetLevelLODIndex)
	int32 LevelLODIndex;

	/** Whether this level streaming object's level should be unloaded and the object be removed from the level list.			*/
	uint8 bIsRequestingUnloadAndRemoval:1;

	/* Whether CachedWorldAssetPackageFName is valid */
	mutable uint8 bHasCachedWorldAssetPackageFName:1;

	/* Whether CachedLoadedLevelPackageName is valid */
	mutable uint8 bHasCachedLoadedLevelPackageName:1;

#if WITH_EDITORONLY_DATA
	/** Whether this level should be visible in the Editor																		*/
	UPROPERTY()
	uint8 bShouldBeVisibleInEditor:1;
#endif

	/** Whether the level should be visible if it is loaded																		*/
	UPROPERTY(Category=LevelStreaming, BlueprintSetter=SetShouldBeVisible)
	uint8 bShouldBeVisible:1;

protected:
	/** Whether the level should be loaded																						*/
	UPROPERTY(Category=LevelStreaming, BlueprintSetter=SetShouldBeLoaded, BlueprintGetter=ShouldBeLoaded)
	uint8 bShouldBeLoaded:1;

	/** Whether the streaming level can safely skip making invisible transaction request from the client to the server */
	uint8 bSkipClientUseMakingInvisibleTransactionRequest:1;

	/** Whether the streaming level can safely skip making visible transaction request from the client to the server */
	uint8 bSkipClientUseMakingVisibleTransactionRequest:1;

	/** Last async load request package path */
	FPackagePath LastAsyncLoadPackagePath;

	/** IDs used to track the underlying async load requests for the level */
	TArray<int32> AsyncRequestIDs;

private:
	/** Used to get the CurrentState from an async task while it can be modified by the gamethread */
	mutable FTransactionallySafeRWLock CurrentStateLock;

	/** What the current streamed state of the streaming level is */
	ELevelStreamingState CurrentState;

	/** What streamed state the streaming level is transitioning towards */
	ELevelStreamingTargetState TargetState;

public:

	/** Whether this level is locked; that is, its actors are read-only. */
	UPROPERTY()
	uint8 bLocked:1;

	/**
	 * Whether this level only contains static actors that aren't affected by gameplay or replication.
	 * If true, the engine can make certain optimizations and will add this level to the StaticLevels collection.
	 */
	UPROPERTY(EditDefaultsOnly, Category=LevelStreaming)
	uint8 bIsStatic:1;

	/** Whether we want to force a blocking load																				*/
	UPROPERTY(Category=LevelStreaming, BlueprintReadWrite)
	uint8 bShouldBlockOnLoad:1;

	/** Whether we want to force a blocking unload																				*/
	UPROPERTY(Category=LevelStreaming, BlueprintReadWrite)
	uint8 bShouldBlockOnUnload:1;

	/** 
	 *  Whether this level streaming object should be ignored by world composition distance streaming, 
	 *  so streaming state can be controlled by other systems (ex: in blueprints)
	 */
	UPROPERTY(transient, Category=LevelStreaming, BlueprintReadWrite)
	uint8 bDisableDistanceStreaming:1;

	/** If true, will be drawn on the 'level streaming status' map (STAT LEVELMAP console command) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = LevelStreaming)
	uint8 bDrawOnLevelStatusMap : 1;

	/** If true, allows garbage collection clustering for the level. */
	uint8 bGarbageCollectionClusteringEnabled : 1;

#if WITH_EDITORONLY_DATA
	/** Deprecated level color used for visualization. */
	UPROPERTY()
	FColor DrawColor_DEPRECATED;
#endif

	/** The level color used for visualization. (Show -> Advanced -> Level Coloration) */
	UPROPERTY(EditAnywhere, Category = LevelStreaming)
	FLinearColor LevelColor;

	/** The level streaming volumes bound to this level. */
	UPROPERTY(EditAnywhere, Category=LevelStreaming, meta=(DisplayName = "Streaming Volumes", NoElementDuplicate))
	TArray<TObjectPtr<ALevelStreamingVolume>> EditorStreamingVolumes;

	/** Cooldown time in seconds between volume-based unload requests.  Used in preventing spurious unload requests. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=LevelStreaming, meta=(ClampMin = "0", UIMin = "0", UIMax = "10"))
	float MinTimeBetweenVolumeUnloadRequests;

	/** Time of last volume unload request.  Used in preventing spurious unload requests. */
	float LastVolumeUnloadRequestTime;

#if WITH_EDITORONLY_DATA
	/** List of keywords to filter on in the level browser */
	UPROPERTY()
	TArray<FString> Keywords;
#endif

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	virtual const IWorldPartitionCell* GetWorldPartitionCell() const { return nullptr; }
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	/** Remove duplicates in EditorStreamingVolumes list*/
	ENGINE_API void RemoveStreamingVolumeDuplicates();
#endif
	//~ End UObject Interface

	/** Returns the current loaded/visible state of the streaming level. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.2, "ULevelStreaming::ECurrentState has been replaced by ELevelStreamingState. Use GetLevelStreamingState instead.")
	ECurrentState GetCurrentState() const { return (ECurrentState)GetLevelStreamingState(); }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ENGINE_API ELevelStreamingState GetLevelStreamingState() const;

protected:	
	/** Updates the current state of the streaming level and notifies any observers. */
	ENGINE_API void SetCurrentState(ELevelStreamingState NewState);

	/** 
	 * Prepare the transition to the new state.
	 * @return true if the state change can be performed, false otherwise.
	 **/
	ENGINE_API virtual bool RequestVisibilityChange(bool bVisible);

	/** Returns whether the streaming level can make visible (can call AddToWorld). */
	ENGINE_API bool CanMakeVisible();

	/** Returns whether the streaming level can make invisible (can call RemoveFromWorld). */
	ENGINE_API bool CanMakeInvisible();

private:

	/** If true client will wait for acknowledgment from server before making streaming levels invisible */
	ENGINE_API bool ShouldClientUseMakingInvisibleTransactionRequest() const;
	/** If true client will wait for acknowledgment from server before making streaming levels visible */
	ENGINE_API bool ShouldClientUseMakingVisibleTransactionRequest() const;
	/** Returns whether the streaming level should wait for the server ack before changing its visibility. */
	ENGINE_API bool ShouldWaitForServerAckBeforeChangingVisibilityState(ENetLevelVisibilityRequest InRequestType);
	/** Ack a client instigated visibility/streaming transaction */
	ENGINE_API void AckNetVisibilityTransaction(FNetLevelVisibilityTransactionId AckedClientTransactionId, bool bClientAckCanMakeVisible);

	/** Determine what the streaming level's target state should be. */
	ENGINE_API ELevelStreamingTargetState DetermineTargetState() const;

	/** Determines a new target state, fires delegates, returns true if the level should continue to be considered for streaming. */
	ENGINE_API bool UpdateTargetState();

	/** Update the load process of the streaming level. Out parameters instruct calling code how to proceed. */
	ENGINE_API void UpdateStreamingState(bool& bOutUpdateAgain, bool& bOutRedetermineTarget, const TOptional<const UE::FTimeout>& InExternalTimeout);

	/** Update internal variables when the level is added from the streaming levels array */
	ENGINE_API void OnLevelAdded();

	/** Update internal variables when the level is removed from the streaming levels array */
	ENGINE_API void OnLevelRemoved();

	/** Internal function for checking if the desired level is the currently loaded level */
	ENGINE_API bool IsDesiredLevelLoaded() const;

public:

	/** Begin a client instigated NetVisibility request */
	ENGINE_API void BeginClientNetVisibilityRequest(bool bInShouldBeVisible);

	/** Check if we are waiting for a making visible or invisible streaming transaction */
	ENGINE_API bool IsWaitingForNetVisibilityTransactionAck(ENetLevelVisibilityRequest InRequestType = ENetLevelVisibilityRequest::MakingInvisible) const;

	/** Set the current state of the current visibility/streaming transaction */
	ENGINE_API void UpdateNetVisibilityTransactionState(bool bInShouldBeVisible, FNetLevelVisibilityTransactionId TransactionId);

	/** Returns the value of bShouldBeVisible. Use ShouldBeVisible to query whether a streaming level should be visible based on its own criteria. */
	bool GetShouldBeVisibleFlag() const { return bShouldBeVisible; }

	/** Sets the should be visible flag and marks the streaming level as requiring consideration. */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetShouldBeVisible(bool bInShouldBeVisible);

	/** Returns whether level should start to render only when it will be fully added to the world or not. */
	virtual bool ShouldRequireFullVisibilityToRender() const { return LODPackageNames.Num() > 0; }

	/** Returns whether level status can be replicated from the server to the client */
	virtual bool CanReplicateStreamingStatus() const { return true; }

	/** 
	 * Virtual that can be overridden to change whether a streaming level should be loaded.
	 * Doesn't do anything at the base level as should be loaded defaults to true 
	 */
	UFUNCTION(BlueprintSetter)
	ENGINE_API virtual void SetShouldBeLoaded(bool bInShouldBeLoaded);

	/** Returns the world composition level LOD index. */
	int32 GetLevelLODIndex() const { return LevelLODIndex; }

	/** Sets the world composition level LOD index and marks the streaming level as requiring consideration. */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetLevelLODIndex(int32 LODIndex);

	/** Sets the relative priority of considering the streaming level. Changing the priority will not interrupt the currently considered level, but will affect the next time a level is being selected for evaluation. */
 	int32 GetPriority() const { return StreamingPriority; }

	/** Sets the relative priority of considering the streaming level. Changing the priority will not interrupt the currently considered level, but will affect the next time a level is being selected for evaluation. */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetPriority(int32 NewPriority);

	/** Provide a temporary priority override for the streaming level. Only one override can be specified at a time. Original priority must be restored with ResetPriorityOverride(). */
	ENGINE_API void SetPriorityOverride(int32 PriorityOverride);

	/** Restore the original streaming level priority after a call to SetPriorityOverride(). */
	ENGINE_API void ResetPriorityOverride();

	/** Returns whether the streaming level is in the loading state. */
	bool HasLoadRequestPending() const { return CurrentState == ELevelStreamingState::Loading; }

	/** Returns whether the streaming level has loaded a level. */
	bool HasLoadedLevel() const
	{
		return (LoadedLevel || PendingUnloadLevel);
	}

	/** Returns the RequestIDs for the async load requests required to load the level. These IDs are cleared during OnLoadingFinished */
	const TArray<int32>& GetAsyncRequestIDs() const
	{
		return AsyncRequestIDs; 
	}

	/** Returns if the streaming level has requested to be unloaded and removed. */
	UFUNCTION(BlueprintPure, Category = LevelStreaming)
	bool GetIsRequestingUnloadAndRemoval() const { return bIsRequestingUnloadAndRemoval; }

	/** Sets if the streaming level should be unloaded and removed. */
	UFUNCTION(BlueprintCallable, Category = LevelStreaming)
	ENGINE_API void SetIsRequestingUnloadAndRemoval(bool bInIsRequestingUnloadAndRemoval);

#if WITH_EDITORONLY_DATA
	/** Returns if the streaming level should be visible in the editor. */
	bool GetShouldBeVisibleInEditor() const { return bShouldBeVisibleInEditor; }
#endif
#if WITH_EDITOR
	/** Sets if the streaming level should be visible in the editor. */
	ENGINE_API void SetShouldBeVisibleInEditor(bool bInShouldBeVisibleInEditor);

	/** Returns if the streaming level is visible in LevelCollectionModel */
	virtual bool ShowInLevelCollection() const { return true; }

	/** Returns if the streaming level can be modified by user */
	virtual bool IsUserManaged() const { return true; }
#endif

	/** Returns a constant reference to the world asset this streaming level object references  */
	const TSoftObjectPtr<UWorld>& GetWorldAsset() const { return WorldAsset; }

	/** Setter for WorldAsset. Use this instead of setting WorldAsset directly to update the cached package name. */
	ENGINE_API virtual void SetWorldAsset(const TSoftObjectPtr<UWorld>& NewWorldAsset);

	/** Gets the package name for the world asset referred to by this level streaming */
	ENGINE_API FString GetWorldAssetPackageName() const;

	/** Gets the package name for the world asset referred to by this level streaming as an FName */
	UFUNCTION(BlueprintCallable, Category = "Game")
	ENGINE_API virtual FName GetWorldAssetPackageFName() const;

	/** Sets the world asset based on the package name assuming it contains a world of the same name. */
	ENGINE_API void SetWorldAssetByPackageName(FName InPackageName);

	/** Rename package name to PIE appropriate name */
	ENGINE_API void RenameForPIE(int PIEInstanceID, bool bKeepWorldAssetName = false);

	/**
	 * Return whether this level should be present in memory which in turn tells the 
	 * streaming code to stream it in. Please note that a change in value from false 
	 * to true only tells the streaming code that it needs to START streaming it in 
	 * so the code needs to return true an appropriate amount of time before it is 
	 * needed.
	 *
	 * @return true if level should be loaded/ streamed in, false otherwise
	 */
	UFUNCTION(BlueprintGetter)
	virtual bool ShouldBeLoaded() const { return true; }

	/**
	 * Return whether this level should be visible/ associated with the world if it is
	 * loaded.
	 * 
	 * @return true if the level should be visible, false otherwise
	 */
	ENGINE_API virtual bool ShouldBeVisible() const;

	virtual bool ShouldBlockOnUnload() const { return bShouldBlockOnUnload; }

	virtual bool ShouldBeAlwaysLoaded() const { return false; }

	/** Get a bounding box around the streaming volumes associated with this LevelStreaming object */
	ENGINE_API FBox GetStreamingVolumeBounds();

	/** Gets a pointer to the LoadedLevel value */
	UFUNCTION(BlueprintCallable, Category="Game")
	ULevel* GetLoadedLevel() const { return LoadedLevel; }
	
	/** Sets the LoadedLevel value to NULL */
	void ClearLoadedLevel() { SetLoadedLevel(nullptr); }
	
#if WITH_EDITOR
	/** Override Pre/PostEditUndo functions to handle editor transform */
	ENGINE_API virtual void PostEditUndo() override;
#endif
	
	/** Matcher for searching streaming levels by PackageName */
	struct FPackageNameMatcher
	{
		FPackageNameMatcher( const FName& InPackageName )
			: PackageName( InPackageName )
		{
		}

		bool operator()(const ULevelStreaming* Candidate) const
		{
			return Candidate->GetWorldAssetPackageFName() == PackageName;
		}

		FName PackageName;
	};

	ENGINE_API virtual UWorld* GetWorld() const override final;

	/** Returns the UWorld that triggered the streaming of this streaming level. */
	ENGINE_API virtual UWorld* GetStreamingWorld() const;

	/** Returns whether streaming level is visible */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API bool IsLevelVisible() const;

	/** Returns whether streaming level is loaded */
	UFUNCTION(BlueprintCallable, Category="Game")
	bool IsLevelLoaded() const { return (LoadedLevel != nullptr); }

	/** Returns whether level has streaming state change pending */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API bool IsStreamingStatePending() const;

	/** Creates a new instance of this streaming level with a provided unique instance name */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API ULevelStreaming* CreateInstance(const FString& UniqueInstanceName);

	/** Returns the Level Script Actor of the level if the level is loaded and valid */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true"))
	ENGINE_API ALevelScriptActor* GetLevelScriptActor();

	/** Returns false if the level package associated to that streaming level is invalid. */
	ENGINE_API bool IsValidStreamingLevel() const;

	/** Used for debugging Level's status */
	ENGINE_API EStreamingStatus GetLevelStreamingStatus() const;

	/** Utility that gets a color for a particular level status */
	static ENGINE_API FColor GetLevelStreamingStatusColor(EStreamingStatus Status);

	/** Utility that returns a string for a streaming level status */
	static ENGINE_API const TCHAR* GetLevelStreamingStatusDisplayName(EStreamingStatus Status);

	/** Utility that draws a legend of level streaming status */
	static ENGINE_API void DebugDrawLegend(const UWorld* World, class UCanvas* Canvas, const FVector2D& Offset);

#if WITH_EDITOR
	/** Get the folder path for this level for use in the world browser. Only available in editor builds */
	ENGINE_API const FName& GetFolderPath() const;

	/** Sets the folder path for this level in the world browser. Only available in editor builds */
	ENGINE_API void SetFolderPath(const FName& InFolderPath);

	ENGINE_API virtual TOptional<FFolder::FRootObject> GetFolderRootObject() const;
#endif	// WITH_EDITOR

	//~==============================================================================================
	// Delegates
	
	/** Called when level is streamed in  */
	UPROPERTY(BlueprintAssignable)
	FLevelStreamingLoadedStatus			OnLevelLoaded;
	
	/** Called when level is streamed out  */
	UPROPERTY(BlueprintAssignable)
	FLevelStreamingLoadedStatus			OnLevelUnloaded;
	
	/** Called when level is added to the world and is visible  */
	UPROPERTY(BlueprintAssignable)
	FLevelStreamingVisibilityStatus		OnLevelShown;
	
	/** Called when level is no longer visible, may not be removed from world yet  */
	UPROPERTY(BlueprintAssignable)
	FLevelStreamingVisibilityStatus		OnLevelHidden;

	/** Whether client should be using making invisible transaction requests to the server (default value). */
	static ENGINE_API bool DefaultAllowClientUseMakingInvisibleTransactionRequests();

	/** Whether client should be using making visible transaction requests to the server (default value). */
	static ENGINE_API bool DefaultAllowClientUseMakingVisibleTransactionRequests();

	/** If true server will wait for client acknowledgment before making treating streaming levels as visible for the client */
	static ENGINE_API bool ShouldServerUseMakingVisibleTransactionRequest();

	/** If true level streaming can reuse an unloaded level that wasn't GC'd yet. */
	static ENGINE_API bool ShouldReuseUnloadedButStillAroundLevels(const ULevel* InLevel);

	/** Whether incremental removal of a streaming level can be done while there's a pending visible streaming level being processed. */
	static ENGINE_API bool AllowIncrementalRemovalWhilePendingVisibility();

	/** 
	 * Traverses all streaming level objects in the persistent world and in all inner worlds and calls appropriate delegate for streaming objects that refer specified level 
	 *
	 * @param PersistentWorld	World to traverse
	 * @param LevelPackageName	Level which loaded status was changed
	 * @param bLoaded			Whether level was loaded or unloaded
	 */
	static ENGINE_API void BroadcastLevelLoadedStatus(UWorld* PersistentWorld, FName LevelPackageName, bool bLoaded);
	
	/** 
	 * Traverses all streaming level objects in the persistent world and in all inner worlds and calls appropriate delegate for streaming objects that refer specified level 
	 *
	 * @param PersistentWorld	World to traverse
	 * @param LevelPackageName	Level which visibility status was changed
	 * @param bVisible			Whether level become visible or not
	 */
	static ENGINE_API void BroadcastLevelVisibleStatus(UWorld* PersistentWorld, FName LevelPackageName, bool bVisible);

	enum EReqLevelBlock
	{
		/** Block load AlwaysLoaded levels. Otherwise Async load. */
		BlockAlwaysLoadedLevelsOnly,
		/** Block all loads */
		AlwaysBlock,
		/** Never block loads */
		NeverBlock,
	};

#if WITH_EDITOR
	// After a sub level is reloaded in the editor the cache state needs to be refreshed
	ENGINE_API void RemoveLevelFromCollectionForReload();
	ENGINE_API void AddLevelToCollectionAfterReload();
#endif

protected:
	/**
	 * Try to find loaded level in memory, issue a loading request otherwise
	 *
	 * @param	PersistentWorld			Persistent world
	 * @param	bAllowLevelLoadRequests	Whether to allow level load requests
	 * @param	BlockPolicy				Whether loading operation should block
	 * @return							true if the load request was issued or a package was already loaded
	 */
	ENGINE_API virtual bool RequestLevel(UWorld* PersistentWorld, bool bAllowLevelLoadRequests, EReqLevelBlock BlockPolicy);

	/** Sets loaded level, fixups for PIE, notifies level is loaded, apply necessary modifications on level once loaded. */
	ENGINE_API void PrepareLoadedLevel(ULevel* InLevel, UPackage* InLevelPackage, int32 InPIEInstanceID);

	/** Sets the value of LoadedLevel */
	ENGINE_API virtual void SetLoadedLevel(ULevel* Level);

	/** Called by SetLoadedLevel */
	virtual void OnLevelLoadedChanged(ULevel* Level) {}

	/** Called when the current state changes */
	virtual void OnCurrentStateChanged(ELevelStreamingState OldState, ELevelStreamingState NewState) {}

	/** Called by RequestLevel to detect existing streaming level with same world asset */
	bool ValidateUniqueWorldAsset(UWorld* PersistentWorld);

	ENGINE_API void OnLoadingStarted();
	ENGINE_API void OnLoadingFinished();

	/** @return Name of the level package that is currently loaded.																	*/
	ENGINE_API FName GetLoadedLevelPackageName() const;

	/** Pointer to Level object if currently loaded/ streamed in.																*/
	UPROPERTY(transient)
	TObjectPtr<class ULevel> LoadedLevel;

	/** Pointer to a Level object that was previously active and was replaced with a new LoadedLevel (for LOD switching) */
	UPROPERTY(transient)
	TObjectPtr<class ULevel> PendingUnloadLevel;

private:
	/** @return Name of the LOD level package used for loading.																		*/
	ENGINE_API FName GetLODPackageName() const;

	/** @return Name of the LOD package on disk to load to the new package named PackageName, Name_None otherwise					*/
	ENGINE_API FName GetLODPackageNameToLoad() const;

	/** Hide and queue for unloading previously used level */
	ENGINE_API void DiscardPendingUnloadLevel(UWorld* PersistentWorld);

	/** 
	 * Handler for level async loading completion 
	 *
	 * @param LevelPackage	Loaded level package
	 */
	ENGINE_API void AsyncLevelLoadComplete(const FName& PackageName, UPackage* LevelPackage, EAsyncLoadingResult::Type Result);

#if WITH_EDITORONLY_DATA
	/** The folder path for this level within the world browser. This is only available in editor builds. 
		A NONE path indicates that it exists at the root. It is '/' separated. */
	UPROPERTY()
	FName FolderPath;
#endif	// WITH_EDITORONLY_DATA

	/** The cached package name of the world asset that is loaded by the levelstreaming */
	mutable FName CachedWorldAssetPackageFName;

	/** The cached package name of the currently loaded level. */
	mutable FName CachedLoadedLevelPackageName;

	/** State for server handshake of NetLevelVisibilty requests */
	FNetLevelVisibilityState NetVisibilityState;

	/** Whether streaming level is concerned by net visibility transactions */
	ENGINE_API bool IsConcernedByNetVisibilityTransactionAck() const;

	/** Helper method that updates server level visibility for each player controller of streaming level world */
	ENGINE_API void ServerUpdateLevelVisibility(bool bIsVisible, bool bTryMakeVisible = false, FNetLevelVisibilityTransactionId TransactionId = FNetLevelVisibilityTransactionId());

	friend struct FAckNetVisibilityTransaction;
	friend struct FStreamingLevelPrivateAccessor;
};

/** Internal struct used by APlayerController to call AckNetVisibilityTransaction on streaming level object */
struct FAckNetVisibilityTransaction
{
private:
	static void Call(ULevelStreaming* StreamingLevel, FNetLevelVisibilityTransactionId TransactionId, bool bClientAckCanMakeVisibleResponse)
	{
		StreamingLevel->AckNetVisibilityTransaction(TransactionId, bClientAckCanMakeVisibleResponse);
	}
	friend class APlayerController;
};

struct FStreamingLevelPrivateAccessor
{
private:

	/** Specifies which level should be the loaded level for the streaming level. */
	static void SetLoadedLevel(ULevelStreaming* StreamingLevel, ULevel* Level) { StreamingLevel->SetLoadedLevel(Level); }
	/** Issue a loading request for the streaming level. */
	static bool RequestLevel(ULevelStreaming* StreamingLevel, UWorld* PersistentWorld, bool bAllowLevelLoadRequests, ULevelStreaming::EReqLevelBlock BlockPolicy) { return StreamingLevel->RequestLevel(PersistentWorld, bAllowLevelLoadRequests, BlockPolicy); }
	/** Update internal variables when the level is added from the streaming levels array */
	static void OnLevelAdded(ULevelStreaming* StreamingLevel) { StreamingLevel->OnLevelAdded(); }
	/** Update internal variables when the level is removed from the streaming levels array */
	static void OnLevelRemoved(ULevelStreaming* StreamingLevel) { StreamingLevel->OnLevelRemoved(); }
	/** Determine what the streaming levels target state should be. Returns whether the streaming level should be in the consider list. */
	static bool UpdateTargetState(ULevelStreaming* StreamingLevel) { return StreamingLevel->UpdateTargetState(); }
	/** Update the load process of the streaming level. Out parameters instruct calling code how to proceed. */
	static void UpdateStreamingState(ULevelStreaming* StreamingLevel, bool& bOutUpdateAgain, bool& bOutRedetermineTarget, const TOptional<const UE::FTimeout>& InExternalTimeout) { StreamingLevel->UpdateStreamingState(bOutUpdateAgain, bOutRedetermineTarget, InExternalTimeout); }

	/** Friend classes to manipulate the streaming level more extensively */
	friend class UEngine;
	friend class UWorld;
};
