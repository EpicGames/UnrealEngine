// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * LevelStreamingAlwaysLoaded
 *
 * @documentation
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/LevelStreaming.h"
#include "LevelStreamingAlwaysLoaded.generated.h"

UCLASS(MinimalAPI)
class ULevelStreamingAlwaysLoaded : public ULevelStreaming
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void GetPrestreamPackages(TArray<UObject*>& OutPrestream) override;
	//~ End UObject Interface

	//~ Begin ULevelStreaming Interface
	virtual bool ShouldBeLoaded() const override { return true; }
	virtual bool ShouldBeAlwaysLoaded() const override { return true; } 
	//~ End ULevelStreaming Interface


};

