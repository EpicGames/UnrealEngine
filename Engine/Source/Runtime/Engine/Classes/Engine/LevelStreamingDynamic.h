// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * LevelStreamingDynamic
 *
 * Dynamically controlled streaming implementation.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/LevelStreaming.h"
#include "LevelStreamingDynamic.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class ULevelStreamingDynamic : public ULevelStreaming
{
	GENERATED_UCLASS_BODY()

	/** Whether the level should be loaded at startup																			*/
	UPROPERTY(Category=LevelStreaming, EditAnywhere)
	uint32 bInitiallyLoaded:1;

	/** Whether the level should be visible at startup if it is loaded 															*/
	UPROPERTY(Category=LevelStreaming, EditAnywhere)
	uint32 bInitiallyVisible:1;
	
	struct FLoadLevelInstanceParams
	{
		FLoadLevelInstanceParams(UWorld* InWorld, const FString& InLongPackageName, FTransform InLevelTransform)
			: World(InWorld)
			, LongPackageName(FPackageName::ObjectPathToPackageName(InLongPackageName))
			, LevelTransform(InLevelTransform) 
		{}

		/** World to instance the level into. */
		UWorld* World = nullptr;

		/** Level long package name to load. */
		FString LongPackageName;

		/** Transform of the instanced level. */
		FTransform LevelTransform;

		/** If set, the loaded level package have this name, which is used by other functions like UnloadStreamLevel. Note this is necessary for server and client networking because the level must have the same name on both. */
		const FString* OptionalLevelNameOverride = nullptr;

		/** If set, the level streaming class will be used instead of ULevelStreamingDynamic. */
		TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass;

		/** If set, package path is prefixed by /Temp. */
		bool bLoadAsTempPackage = false;

		/** Set whether the level will be made visible initially. */
		bool bInitiallyVisible = true;

		/** Set whether we allow to reuse an existing level streaming. */
		bool bAllowReuseExitingLevelStreaming = false;

		/** Set EditorPath Owner */
		UObject* EditorPathOwner = nullptr;

		/** Callback when ULevelStreaming object is created */
		TUniqueFunction<void(ULevelStreaming*)> LevelStreamingCreatedCallback;
	};

	/**  
 	* Stream in a level with a specific location and rotation. You can create multiple instances of the same level!
 	*
 	* The level to be loaded does not have to be in the persistent map's Levels list, however to ensure that the .umap does get
 	* packaged, please be sure to include the .umap in your Packaging Settings:
 	*
 	*   Project Settings -> Packaging -> List of Maps to Include in a Packaged Build (you may have to show advanced or type in filter)
 	* 
 	* @param LevelName - Level package name to load, ex: /Game/Maps/MyMapName, specifying short name like MyMapName will force very slow search on disk
 	* @param Location - World space location where the level should be spawned
 	* @param Rotation - World space rotation for rotating the entire level
	* @param bOutSuccess - Whether operation was successful (map was found and added to the sub-levels list)
	* @param OptionalLevelNameOverride - If set, the loaded level package have this name, which is used by other functions like UnloadStreamLevel. Note this is necessary for server and client networking because the level must have the same name on both.
	* @param OptionalLevelStreamingClass - If set, the level streaming class will be used instead of ULevelStreamingDynamic
	* @param bLoadAsTempPackage - If set, package path is prefixed by /Temp
 	* @return Streaming level object for a level instance
 	*/ 
 	UFUNCTION(BlueprintCallable, Category = LevelStreaming, meta=(DisplayName = "Load Level Instance (by Name)", WorldContext="WorldContextObject"))
 	static ENGINE_API ULevelStreamingDynamic* LoadLevelInstance(UObject* WorldContextObject, FString LevelName, FVector Location, FRotator Rotation, bool& bOutSuccess, const FString& OptionalLevelNameOverride = TEXT(""), TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass = {}, bool bLoadAsTempPackage = false);

 	UFUNCTION(BlueprintCallable, Category = LevelStreaming, meta=(DisplayName = "Load Level Instance (by Object Reference)", WorldContext="WorldContextObject"))
 	static ENGINE_API ULevelStreamingDynamic* LoadLevelInstanceBySoftObjectPtr(UObject* WorldContextObject, TSoftObjectPtr<UWorld> Level, FVector Location, FRotator Rotation, bool& bOutSuccess, const FString& OptionalLevelNameOverride = TEXT(""), TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass = {}, bool bLoadAsTempPackage = false);
 	
	static ENGINE_API ULevelStreamingDynamic* LoadLevelInstanceBySoftObjectPtr(UObject* WorldContextObject, TSoftObjectPtr<UWorld> Level, const FTransform LevelTransform, bool& bOutSuccess, const FString& OptionalLevelNameOverride = TEXT(""), TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass = {}, bool bLoadAsTempPackage = false);

	static ENGINE_API ULevelStreamingDynamic* LoadLevelInstance(const FLoadLevelInstanceParams& Params, bool& bOutSuccess);

	static ENGINE_API FString GetLevelInstancePackageName(const FLoadLevelInstanceParams& Params);

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin ULevelStreaming Interface
	virtual bool ShouldBeLoaded() const override { return bShouldBeLoaded; }
	//~ End ULevelStreaming Interface

	ENGINE_API virtual void SetShouldBeLoaded(bool bShouldBeLoaded) override;

private:

	// Counter used by LoadLevelInstance to create unique level names
	static ENGINE_API int32 UniqueLevelInstanceId;

 	static ENGINE_API ULevelStreamingDynamic* LoadLevelInstance_Internal(const FLoadLevelInstanceParams& Params, bool& bOutSuccess);

};
