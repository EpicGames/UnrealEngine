// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"

class ULevel;
class UWorld;

/**
 * Helper structure encapsulating functionality used to defer marking actors and their components as pending
 * kill till right before garbage collection by registering a callback.
 */
struct FLevelStreamingGCHelper
{
	/** Called when streamed out levels are going to be garbage collected  */
	DECLARE_MULTICAST_DELEGATE(FOnGCStreamedOutLevelsEvent);
	static ENGINE_API FOnGCStreamedOutLevelsEvent OnGCStreamedOutLevels;

	/**
	 * Register with the garbage collector to receive callbacks pre and post garbage collection
	 */
	static ENGINE_API void AddGarbageCollectorCallback();

	/**
	 * Request to be unloaded.
	 *
	 * @param InLevel	Level that should be unloaded
	 */
	static ENGINE_API void RequestUnload( ULevel* InLevel );

	/**
	 * Cancel any pending unload requests for passed in Level.
	 */
	static ENGINE_API void CancelUnloadRequest( ULevel* InLevel );

	/**
	 * @return	The number of levels pending a purge by the garbage collector
	 */
	static ENGINE_API int32 GetNumLevelsPendingPurge();

	/**
	 * @return	The number of actors (from levels) pending a purge by the garbage collector
	 */
	static ENGINE_API int32 GetNumActorsPendingPurge();

	/**
	 * Trash a package, making sure it will be reloaded if needed again by level streaming
	 */
	UE_DEPRECATED(5.8, "TrashPackage is deprecated.")
	static ENGINE_API void TrashPackage(UPackage* InPackage);

	/**
	 * Allows FLevelStreamingGCHelper to be used in a commandlet.
	 */
	static ENGINE_API void EnableForCommandlet();
	
private:
	/** Prepares levels that are marked for unload for the next GC call by marking their actors and components as garbage. */
	static void PrepareStreamedOutLevelsForGC();

	/** Verify that the level packages are no longer around. */
	static void VerifyLevelsGotRemovedByGC();

	/** Called before garbage collect. */
	static void OnPreGarbageCollect();

	/** Called at the end of a world tick. */
	static void OnWorldTickEnd(UWorld* InWorld, ELevelTick InTickType, float InDeltaSeconds);

	/** Prepares a level that is marked for unload for the next GC call by marking its actors and components as garbage.  */
	static void PrepareStreamedOutLevelForGC(ULevel* Level);

	/** Static array of levels that should be unloaded */
	static TArray<TWeakObjectPtr<ULevel> > LevelsPendingUnload;

	/** Static set of level packages that have been marked by PrepareStreamedOutLevelsForGC */
	static TSet<FName> LevelPackageNames;

	/** Static bool allows FLevelStreamingGCHelper to be used in a commandlet */
	static bool bEnabledForCommandlet;

	/** Whether RequestUnload delayed its call to PrepareStreamedOutLevelForGC after world tick. */
	static bool bIsPrepareStreamedOutLevelForGCDelayedToWorldTickEnd;

	/** The number of unloaded levels prepared for the next GC (used by GetNumLevelsPendingPurge). */
	static int32 NumberOfPreparedStreamedOutLevelsForGC;

	/** The number of unloaded actors prepared for the next GC (used by GetNumActorsPendingPurge). */
	static int32 NumberOfPreparedStreamedOutActorsForGC;

	/** The number of actors that should be unloaded */
	static int32 ActorsPendingUnload;
};	
