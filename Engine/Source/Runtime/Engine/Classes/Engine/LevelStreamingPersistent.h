// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * LevelStreamingPersistent
 *
 * @documentation
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/LevelStreaming.h"
#include "LevelStreamingPersistent.generated.h"

UCLASS(transient)
class ULevelStreamingPersistent : public ULevelStreaming
{
	GENERATED_UCLASS_BODY()


	//~ Begin ULevelStreaming Interface
	virtual bool ShouldBeLoaded() const override
	{
		return true;
	}
	virtual bool ShouldBeVisible() const override
	{
		return true;
	}
	//~ End ULevelStreaming Interface
};

