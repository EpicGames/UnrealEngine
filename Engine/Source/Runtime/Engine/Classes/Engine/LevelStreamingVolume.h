// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Used to affect level streaming in the game and level visibility in the editor.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "LevelStreamingVolume.generated.h"

/** Enum for different usage cases of level streaming volumes. */
UENUM()
enum EStreamingVolumeUsage : int
{
	SVB_Loading,
	SVB_LoadingAndVisibility,
	SVB_VisibilityBlockingOnLoad,
	SVB_BlockingOnLoad,
	SVB_LoadingNotVisible,
	SVB_MAX,
};


UCLASS(hidecategories=(Advanced, Attachment, Collision, Volume), MinimalAPI)
class ALevelStreamingVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	/** Levels names affected by this level streaming volume. */
	UPROPERTY(Category=LevelStreamingVolume, VisibleAnywhere, BlueprintReadOnly, meta=(DisplayName = "Streaming Levels"))
	TArray<FName> StreamingLevelNames;

	/** If true, this streaming volume should only be used for editor streaming level previs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LevelStreamingVolume)
	uint32 bEditorPreVisOnly:1;

	/**
	 * If true, this streaming volume is ignored by the streaming volume code.  Used to either
	 * disable a level streaming volume without disassociating it from the level, or to toggle
	 * the control of a level's streaming between Kismet and volume streaming.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LevelStreamingVolume)
	uint32 bDisabled:1;

	/** Determines what this volume is used for, e.g. whether to control loading, loading and visibility or just visibilty (blocking on load) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LevelStreamingVolume)
	TEnumAsByte<enum EStreamingVolumeUsage> StreamingUsage;

	//~ Begin UObject Interface.
	virtual ENGINE_API void PostLoad() override;
	//~ End UObject Interface.
	
#if WITH_EDITOR
	//~ Begin AActor Interface.
	virtual ENGINE_API void CheckForErrors() override;
	//~ End AActor Interface.
	
	/** Updates list of streaming levels that are referenced by this streaming volume */
	void UpdateStreamingLevelsRefs();
#endif
};



