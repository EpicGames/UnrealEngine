// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Light.generated.h"

class UMaterialInterface;

UCLASS(Abstract, ClassGroup=Lights, hideCategories=(Input,Collision,Replication), showCategories=("Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass, ConversionRoot, meta=(ChildCanTick), MinimalAPI)
class ALight : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** @todo document */
	UPROPERTY(Category = Light, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Light,Rendering,Rendering|Components|Light", AllowPrivateAccess = "true"))
	TObjectPtr<class ULightComponent> LightComponent;
public:

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

	/** replicated copy of LightComponent's bEnabled property */
	UPROPERTY(replicatedUsing=OnRep_bEnabled)
	uint32 bEnabled:1;

	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_bEnabled();

	/** Function to change mobility type of light */
	ENGINE_API void SetMobility(EComponentMobility::Type InMobility);

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetEnabled(bool bSetEnabled);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API bool IsEnabled() const;
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void ToggleEnabled();
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetBrightness(float NewBrightness);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API float GetBrightness() const;
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetLightColor(FLinearColor NewLightColor);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API FLinearColor GetLightColor() const;
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetLightFunctionMaterial(UMaterialInterface* NewLightFunctionMaterial);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetLightFunctionScale(FVector NewLightFunctionScale);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetLightFunctionFadeDistance(float NewLightFunctionFadeDistance);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetCastShadows(bool bNewValue);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetAffectTranslucentLighting(bool bNewValue);
	// END DEPRECATED

public:
	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface.

#if WITH_EDITOR
	//~ Begin AActor Interface.
	ENGINE_API virtual void CheckForErrors() override;
	//~ End AActor Interface.
#endif

	/**
	 * Return whether the light supports being toggled off and on on-the-fly.
	 */
	ENGINE_API bool IsToggleable() const;

	//~ Begin AActor Interface.
	ENGINE_API virtual void Destroyed() override;
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	//~ End AActor Interface.

	/** Returns LightComponent subobject **/
	class ULightComponent* GetLightComponent() const { return LightComponent; }
};