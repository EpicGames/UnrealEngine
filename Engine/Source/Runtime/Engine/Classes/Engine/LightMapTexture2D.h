// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * A 2D texture containing lightmap coefficients.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture2D.h"
#include "LightMapTexture2D.generated.h"

/**
 * Bit-field flags that affects storage (e.g. packing, streaming) and other info about a lightmap.
 */
enum ELightMapFlags
{
	LMF_None			= 0,			// No flags
	LMF_Streamed		= 0x00000001,	// Lightmap should be placed in a streaming texture
};

UCLASS(MinimalAPI)
class ULightMapTexture2D : public UTexture2D
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface.
	virtual void Serialize( FArchive& Ar ) override;
	virtual FString GetDesc() override;
	//~ End UObject Interface. 
	/** Bit-field with lightmap flags. */
	ELightMapFlags LightmapFlags;
};
