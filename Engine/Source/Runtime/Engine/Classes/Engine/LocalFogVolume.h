// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Info.h"
#include "LocalFogVolume.generated.h"

class ULocalFogVolumeComponent;

/**
 *	Actor used to position a local fog volume in the scene.
 *	@see https://dev.epicgames.com/documentation/en-us/unreal-engine/local-fog-volumes-in-unreal-engine
 */
UCLASS(showcategories = (Movement, Rendering, Transformation, DataLayers, WorldPartition, "Input|MouseInput", "Input|TouchInput"), ClassGroup = Fog, hidecategories = (Info, Object, Input), MinimalAPI)
class ALocalFogVolume : public AInfo
{
	GENERATED_UCLASS_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	/** Object used to visualize the local fog volume */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Fog, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<ULocalFogVolumeComponent> LocalFogVolumeVolume;

public:

	/** Returns LocalFogVolumeVolume subobject **/
	ULocalFogVolumeComponent* GetComponent() const { return LocalFogVolumeVolume; }
	
#if WITH_EDITOR
	//Allow LocalFogVolume instances to be spatially loaded (this flag enabled the streaming UI properties), and ULocalFogVolumeComponent is a USceneComponent which should provide streaming bounds.
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return true; }
#endif

};

