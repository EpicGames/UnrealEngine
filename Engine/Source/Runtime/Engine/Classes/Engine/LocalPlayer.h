// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// LocalPlayer
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Input/Reply.h"
#include "Online/CoreOnline.h"
#include "SceneTypes.h"
#include "Engine/Player.h"
#include "GameFramework/OnlineReplStructs.h"
#include "GameFramework/PlayerController.h"

#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/SubsystemCollection.h"

#include "LocalPlayer.generated.h"


#define INVALID_CONTROLLERID (-1)

class AActor;
class FSceneView;
class FSlateUser;
class FViewport;
class UGameInstance;
class UGameViewportClient;
class ULocalPlayer;
class UScriptViewportClient;
struct FMinimalViewInfo;
struct FSceneViewProjectionData;

/** A context object that binds to a LocalPlayer. Useful for UI or other things that need to pass around player references */
struct FLocalPlayerContext
{
	ENGINE_API FLocalPlayerContext();
	ENGINE_API FLocalPlayerContext(const class ULocalPlayer* InLocalPlayer, UWorld* InWorld = nullptr);
	ENGINE_API FLocalPlayerContext(const class APlayerController* InPlayerController);

	/** Is this context initialized and still valid? */
	ENGINE_API bool IsValid() const;

	/** Is this context initialized */
	ENGINE_API bool IsInitialized() const;

	/** This function tests if the given Actor is connected to the Local Player in any way. 
		It tests against the APlayerController, APlayerState, and APawn. */
	ENGINE_API bool IsFromLocalPlayer(const AActor* ActorToTest) const;

	/** Returns the world context. */
	ENGINE_API UWorld* GetWorld() const;

	/** Returns the game instance */
	ENGINE_API UGameInstance* GetGameInstance() const;

	/** Returns the local player. */
	ENGINE_API class ULocalPlayer* GetLocalPlayer() const;

	/** Returns the player controller. */
	ENGINE_API class APlayerController* GetPlayerController() const;

	/** Templated version of GetPlayerController() */
	template<class T>
	inline T* GetPlayerController(bool bCastChecked = false) const
	{
		if (bCastChecked)
		{ 
			return CastChecked<T>(GetPlayerController(), ECastCheckedType::NullAllowed);
		}
		else
		{
			return Cast<T>(GetPlayerController());
		}
	}

	/** Getter for the Game State Base */
	ENGINE_API class AGameStateBase* GetGameState() const;

	/** Templated Getter for the Game State */
	template<class T>
	inline T* GetGameState(bool bCastChecked = false) const
	{
		if (bCastChecked)
		{
			return CastChecked<T>(GetGameState(), ECastCheckedType::NullAllowed);
		}
		else
		{
			return Cast<T>(GetGameState());
		}
	}

	/** Getter for the Player State */
	ENGINE_API class APlayerState* GetPlayerState() const;

	/** Templated Getter for the Player State */
	template<class T>
	inline T* GetPlayerState(bool bCastChecked = false) const
	{
		if (bCastChecked)
		{
			return CastChecked<T>(GetPlayerState(), ECastCheckedType::NullAllowed);
		}
		else
		{
			return Cast<T>(GetPlayerState());
		}
	}

	/** Getter for this player's HUD */
	ENGINE_API class AHUD* GetHUD() const;

	/** Templated Getter for the HUD */
	template<class T>
	inline T* GetHUD(bool bCastChecked = false) const
	{
		if (bCastChecked)
		{
			return CastChecked<T>(GetHUD(), ECastCheckedType::NullAllowed);
		}
		else
		{
			return Cast<T>(GetHUD());
		}
	}

	/** Getter for the base pawn of this player */
	ENGINE_API class APawn* GetPawn() const;

	/** Templated getter for the player's pawn */
	template<class T>
	inline T* GetPawn(bool bCastChecked = false) const
	{
		if (bCastChecked)
		{
			return CastChecked<T>(GetPawn(), ECastCheckedType::NullAllowed);
		}
		else
		{
			return Cast<T>(GetPawn());
		}
	}

private:	

	/* Set the local player. */
	ENGINE_API void SetLocalPlayer( const class ULocalPlayer* InLocalPlayer );

	/* Set the local player via a player controller. */
	ENGINE_API void SetPlayerController( const class APlayerController* InPlayerController );

	TWeakObjectPtr<class ULocalPlayer>		LocalPlayer;

	TWeakObjectPtr<UWorld>					World;
};

/**
 *	Each player that is active on the current client/listen server has a LocalPlayer.
 *	It stays active across maps, and there may be several spawned in the case of splitscreen/coop.
 *	There will be 0 spawned on dedicated servers.
 */
UCLASS(Within=Engine, config=Engine, transient, MinimalAPI)
class ULocalPlayer : public UPlayer
{
	GENERATED_UCLASS_BODY()

public:

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	ENGINE_API ULocalPlayer(FVTableHelper& Helper);

	/** The FUniqueNetId which this player is associated with. */
	FUniqueNetIdRepl CachedUniqueNetId;

	/** The UGameViewportClient for the UGameInstance that this player belongs to. */
	UPROPERTY()
	TObjectPtr<class UGameViewportClient> ViewportClient;

	/** The primary viewport containing this player's view (different from ViewportClient if override is set). */
	ENGINE_API UScriptViewportClient* GetViewportClient() const;

	/** Declares that a different viewport client will render this player.  Pass nullptr to render in the primary viewport. */
	ENGINE_API void OverrideViewportClient(class UScriptViewportClient* InViewportClient);

	/** The coordinates for the upper left corner of the primary viewport subregion allocated to this player. 0-1 */
	FVector2D Origin;

	/** The size of the primary viewport subregion allocated to this player. 0-1 */
	FVector2D Size;

	/** The location of the player's view the previous frame. */
	FVector LastViewLocation;

	/** How to constrain perspective viewport FOV */
	UPROPERTY(config)
	TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint;

	/** The class of PlayerController to spawn for players logging in. */
	UPROPERTY()
	TSubclassOf<class APlayerController> PendingLevelPlayerControllerClass;

	/** set when we've sent a split join request */
	UPROPERTY(VisibleAnywhere, transient, Category=LocalPlayer)
	uint32 bSentSplitJoin:1;

	DECLARE_EVENT_TwoParams(ULocalPlayer, FOnControllerIdChanged, int32 /*NewId*/, int32 /*OldId*/);
	FOnControllerIdChanged& OnControllerIdChanged() const { return OnControllerIdChangedEvent; }

	/** Event called when this local player has been assigned to a new platform-level user */
	DECLARE_EVENT_TwoParams(ULocalPlayer, FOnPlatformUserIdChanged, FPlatformUserId /*NewId*/, FPlatformUserId /*OldId*/);
	FOnPlatformUserIdChanged& OnPlatformUserIdChanged() { return OnPlatformUserIdChangedEvent; }

	/** Event called when this local player has had a new outer PlayerController set */
	DECLARE_EVENT_OneParam(ULocalPlayer, FOnPlayerControllerChanged, APlayerController* /*NewPC*/);
	FOnPlayerControllerChanged& OnPlayerControllerChanged() { return OnPlayerControllerChangedEvent; }

	/** Whether or not we're emulating splitscreen with this player. Should only be set from screenshot functional tests. */
	bool bEmulateSplitscreen = false;
	
	/** 
	 * Set the connection identifier for this local user. 
	 *
	 * The connection identifier is a unique identifier for this local user on a client and is used to 
	 * associate the local player that began a connection handshake with a player controller replicated
	 * to the client when the connection handshake is complete.
	 *
	 * Only set this if you are connecting to a game server outside of the normal connection flow in the 
	 * engine (e.g. you are sending NMT_Join manually to initiate a connection).
	 * 
	 * This is the expected flow for how the connection identifier should be used:
	 *	* (Client) Set the connection identifier before sending NMT_Join, NMT_JoinSplit, NMT_JoinNoPawnSplit or NMT_JoinNoPawn.
	 * 	* (Client) The connection identifier is sent with the join message as a URL argument.
	 *	* (Server) Set the connection identifier on the player controller spawned on the game server.
	 *	* (Client) Match the player controller connection identifier to the correct local user connection identifier.
	 */
	ENGINE_API void SetConnectionIdentifier(const int32 Id);

	/** Get the client connection identifier for this local user. */
	ENGINE_API int32 GetConnectionIdentifier() const;

private:
	TArray<FSceneViewStateReference> ViewStates;

	UPROPERTY()
	TObjectPtr<UScriptViewportClient> ViewportClientOverride;

	/** The controller ID which this player accepts input from. */
	UPROPERTY()
	int32 ControllerId = INVALID_CONTROLLERID;

	mutable FOnControllerIdChanged OnControllerIdChangedEvent;

	int32 ConnectionIdentifier = 0;

	/** The platform user this player is assigned to, could correspond to multiple input devices */
	FPlatformUserId PlatformUserId;

	/** Event called when platform user id changes */
	FOnPlatformUserIdChanged OnPlatformUserIdChangedEvent;

	/** Event called when the outer player controller changes */
	FOnPlayerControllerChanged OnPlayerControllerChangedEvent;

	FObjectSubsystemCollection<ULocalPlayerSubsystem> SubsystemCollection;

public:
	// UObject interface
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// End of UObject interface

	// Begin UPlayer interface
	ENGINE_API virtual void ReceivedPlayerController(APlayerController* NewController) override;
	// End UPlayer interface
	
	// FExec interface
public:
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar) override;
#endif
protected:
	ENGINE_API virtual bool Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	// End of FExec interface

public:
	/** 
	 * Exec command handlers
	 */

	ENGINE_API bool HandleDNCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleExitCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleListMoveBodyCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleListAwakeBodiesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleListSimBodiesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleMoveComponentTimesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleListSkelMeshesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleListPawnComponentsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleExecCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleToggleDrawEventsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	ENGINE_API bool HandleToggleStreamingVolumesCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	
protected:
	/**
	 * Retrieve the viewpoint of this player.
	 * @param OutViewInfo - Upon return contains the view information for the player.
	 */
	ENGINE_API virtual void GetViewPoint(FMinimalViewInfo& OutViewInfo) const;

	/** @todo document */
	ENGINE_API void ExecMacro( const TCHAR* Filename, FOutputDevice& Ar );

	/** FReply used to defer some slate operations. */
	FReply SlateOperations;

public:

	/**
	 *  Getter for slate operations.
	 */
	FReply& GetSlateOperations() { return SlateOperations; }
	const FReply& GetSlateOperations() const { return SlateOperations; }

	/** Get the SlateUser that this LocalPlayer corresponds to */
	ENGINE_API virtual TSharedPtr<FSlateUser> GetSlateUser();
	ENGINE_API virtual TSharedPtr<const FSlateUser> GetSlateUser() const;

	/**
	 * Get the world the players actor belongs to
	 *
	 * @return  Returns the world of the LocalPlayer's PlayerController. NULL if the LocalPlayer does not have a PlayerController
	 */
	ENGINE_API virtual UWorld* GetWorld() const override;

	/**
	 * Get the game instance associated with this local player
	 * 
	 * @return GameInstance related to local player
	 */
	ENGINE_API UGameInstance* GetGameInstance() const;

	/**
	 * Returns the index of this player in the Game instances local players array
	 *
	 * @return Index in array, will be >= 0 if this is a fully registered player
	 */
	ENGINE_API int32 GetIndexInGameInstance() const;

	/**
	 * Get a Subsystem of specified type
	 */
	ULocalPlayerSubsystem* GetSubsystemBase(TSubclassOf<ULocalPlayerSubsystem> SubsystemClass) const
	{
		return SubsystemCollection.GetSubsystem<ULocalPlayerSubsystem>(SubsystemClass);
	}

	/**
	 * Get a Subsystem of specified type
	 */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem() const
	{
		return SubsystemCollection.GetSubsystem<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Get a Subsystem of specified type from the provided LocalPlayer
	 * returns nullptr if the Subsystem cannot be found or the LocalPlayer is null
	 */
	template <typename TSubsystemClass>
	static inline TSubsystemClass* GetSubsystem(const ULocalPlayer* LocalPlayer)
	{
		if (LocalPlayer)
		{
			return LocalPlayer->GetSubsystem<TSubsystemClass>();
		}
		return nullptr;
	}

	/**
	 * Get a subsystem of the specificied type from the provided Player Controller's owning ULocalPlayer.
	 * Returns nullptr if the Subsystem cannot be found or the given controller or its local player are null.
	 */
	template <typename TSubsystemClass>
	static FORCEINLINE TSubsystemClass* GetSubsystemFromController(const APlayerController* PC)
	{
		if (PC)
		{
			const ULocalPlayer* LocalPlayer = PC->GetLocalPlayer();
			return LocalPlayer ? LocalPlayer->GetSubsystem<TSubsystemClass>() : nullptr;
		}
		return nullptr;
	}
	
	/**
	 * Get all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy() const
	{
		return SubsystemCollection.GetSubsystemArrayCopy<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Performs an operation on all Subsystem of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	void ForEachSubsystem(TFunctionRef<void(TSubsystemClass*)> Operation) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, ULocalPlayerSubsystem>::IsDerived, "TSubsystemClass must be derived from ULocalPlayerSubsystem");
		return SubsystemCollection.ForEachSubsystem([Operation=MoveTemp(Operation)](ULocalPlayerSubsystem* Subsystem){
			Operation(CastChecked<TSubsystemClass>(Subsystem));
		}, TSubsystemClass::StaticClass());
	}


	/**
	* Calculate the view init settings for drawing from this view actor
	*
	* @param	OutInitOptions - output view struct. Not every field is initialized, some of them are only filled in by CalcSceneView
	* @param	Viewport - current client viewport
	* @param	ViewDrawer - optional drawing in the view
	* @param	StereoViewIndex - index of the view when using stereoscopy
	* @return	true if the view options were filled in. false in various fail conditions.
	*/
	ENGINE_API virtual bool CalcSceneViewInitOptions(
		struct FSceneViewInitOptions& OutInitOptions,
		FViewport* Viewport,
		class FViewElementDrawer* ViewDrawer = NULL,
		int32 StereoViewIndex = INDEX_NONE);

	/**
	 * Calculate the view settings for drawing from this view actor
	 *
	 * @param	View - output view struct
	 * @param	OutViewLocation - output actor location
	 * @param	OutViewRotation - output actor rotation
	 * @param	Viewport - current client viewport
	 * @param	ViewDrawer - optional drawing in the view
	 * @param	StereoViewIndex - index of the view when using stereoscopy
	 */
	ENGINE_API virtual FSceneView* CalcSceneView(class FSceneViewFamily* ViewFamily,
		FVector& OutViewLocation,
		FRotator& OutViewRotation,
		FViewport* Viewport,
		class FViewElementDrawer* ViewDrawer = NULL,
		int32 StereoViewIndex = INDEX_NONE);

	/**
	 * Called at creation time for internal setup
	 */
	ENGINE_API virtual void PlayerAdded(class UGameViewportClient* InViewportClient, int32 InControllerID);

	/**
	 * Called at creation time for internal setup
	 */
	ENGINE_API virtual void PlayerAdded(class UGameViewportClient* InViewportClient, FPlatformUserId InUserId);

	/**
	 * Called to initialize the online delegates
	 */
	ENGINE_API virtual void InitOnlineSession();

	/**
	 * Called when the player is removed from the viewport client
	 */
	ENGINE_API virtual void PlayerRemoved();

	/**
	 * Create an actor for this player.
	 * @param URL - The URL the player joined with.
	 * @param OutError - If an error occurred, returns the error description.
	 * @param InWorld - World in which to spawn the play actor
	 * @return False if an error occurred, true if the play actor was successfully spawned.	 
	 */
	ENGINE_API virtual bool SpawnPlayActor(const FString& URL,FString& OutError, UWorld* InWorld);

	DECLARE_DELEGATE(FOnPreBeginHandshakeCompleteDelegate);

	/**
	 * Allow local player to run any async tasks needed before starting travel to a server.
	 */
	virtual void PreBeginHandshake(const FOnPreBeginHandshakeCompleteDelegate& OnComplete) { OnComplete.ExecuteIfBound(); }

	/** Send a splitscreen join command to the server to allow a splitscreen player to connect to the game
	 * the client must already be connected to a server for this function to work
	 * @note this happens automatically for all viewports that exist during the initial server connect
	 * 	so it's only necessary to manually call this for viewports created after that
	 * if the join fails (because the server was full, for example) all viewports on this client will be disconnected
	 *
	 * This call will send the command using the current net driver associated with world.
	 *
	 * @param	Options		array of URL options to append.
	 */
	ENGINE_API virtual void SendSplitJoin(TArray<FString>& Options);

	/** 
	 * Same as SendSplitJoin() above but with additional arguments.
	 *
	 * @param	NetDriver		Use this NetDriver to send the join request instead of the NetDriver associated with the world.
	 * @param	Flags			Configuration flags used by the join request sent to the server.
	 */
	ENGINE_API virtual void SendSplitJoin(TArray<FString>& Options, UNetDriver* NetDriver, UE::Net::EJoinFlags Flags);
	
	/**
	 * Change the physical ControllerId for this player; if the specified ControllerId is already taken by another player, changes the ControllerId
	 * for the other player to the ControllerId currently in use by this player.
	 *
	 * @param	NewControllerId		the ControllerId to assign to this player.
	 */
	ENGINE_API virtual void SetControllerId(int32 NewControllerId);

	/**
	 * Returns the controller ID for the player.
	 * This is a legacy identifier corresponding to the primary physical controller used by the player.
	 * You may want to use GetPlatformUserIndex or GetLocalPlayerIndex instead.
	 */
	int32 GetControllerId() const { return ControllerId; }

	/**
	 * Changes the platform user that is assigned to this player
	 */
	ENGINE_API virtual void SetPlatformUserId(FPlatformUserId InPlatformUserId);

	/**
	 * Returns the platform user that is assigned to this player
	 */
	FPlatformUserId GetPlatformUserId() const { return PlatformUserId; }

	/**
	 * Converts the platform user id to an index where 0 is the first logged in user.
	 * This index is used for platform functions like save games, user selection, and slate input.
	 */
	ENGINE_API virtual int32 GetPlatformUserIndex() const;

	/**
	 * Returns the logical local player index where 0 is the first LocalPlayer that was created.
	 * By default, this uses index in the game instance array.
	 * This index is used for gameplay purposes but will not be correct for platform functions.
	 */
	ENGINE_API virtual int32 GetLocalPlayerIndex() const;

	/** 
	 * Retrieves this player's name/tag from the online subsystem
	 * if this function returns a non-empty string, the returned name will replace the "Name" URL parameter
	 * passed around in the level loading and connection code, which normally comes from DefaultEngine.ini
	 * 
	 * @return Name of player if specified (by onlinesubsystem or otherwise), Empty string otherwise
	 */
	ENGINE_API virtual FString GetNickname() const;

	/** 
	 * Retrieves any game-specific login options for this player
	 * if this function returns a non-empty string, the returned option or options be added
	 * passed in to the level loading and connection code.  Options are in URL format,
	 * key=value, with multiple options concatenated together with an & between each key/value pair
	 * 
	 * @return URL Option or options for this game, Empty string otherwise
	 */
	virtual FString GetGameLoginOptions() const { return TEXT(""); }

	// This should be deprecated when engine code has been changed to expect FPlatformUserId
	// UE_DEPRECATED(5.x, "Use GetUniqueNetIdForPlatformUser instead")
	ENGINE_API FUniqueNetIdRepl GetUniqueNetIdFromCachedControllerId() const;

	/**
	 * Retrieves this player's unique net ID from the online subsystem using the platform user Id
	 *
	 * @return unique Id associated with this player
	 */
	ENGINE_API virtual FUniqueNetIdRepl GetUniqueNetIdForPlatformUser() const;

	/** 
	 * Retrieves this player's unique net ID that was previously cached
	 *
	 * @return unique Id associated with this player
	 */
	ENGINE_API FUniqueNetIdRepl GetCachedUniqueNetId() const;

	/** Sets the players current cached unique net id */
	ENGINE_API void SetCachedUniqueNetId(const FUniqueNetIdRepl& NewUniqueNetId);

	/** 
	 * Retrieves the preferred unique net id. This is for backwards compatibility for games that don't use the cached unique net id logic
	 *
	 * @return unique Id associated with this player
	 */
	ENGINE_API virtual FUniqueNetIdRepl GetPreferredUniqueNetId() const;

	/**
	 * This function will give you two points in Pixel Space that surround the World Space box.
	 *
	 * @param	ActorBox		The World Space Box
	 * @param	OutLowerLeft	The Lower Left corner of the Pixel Space box
	 * @param	OutUpperRight	The Upper Right corner of the pixel space box
	 * @return  False if there is no viewport, or if the box is behind the camera completely
	 */
	ENGINE_API bool GetPixelBoundingBox(const FBox& ActorBox, FVector2D& OutLowerLeft, FVector2D& OutUpperRight, const FVector2f* OptionalAllotedSize = nullptr);
	static ENGINE_API bool GetPixelBoundingBox(const FSceneViewProjectionData& ProjectionData, const FBox& ActorBox, FVector2D& OutLowerLeft, FVector2D& OutUpperRight, const FVector2f* OptionalAllotedSize = nullptr);

	/**
	 * This function will give you a point in Pixel Space from a World Space position
	 *
	 * @param	InPoint		The point in world space
	 * @param	OutPoint	The point in pixel space
	 * @return  False if there is no viewport, or if the box is behind the camera completely
	 */
	ENGINE_API bool GetPixelPoint(const FVector& InPoint, FVector2D& OutPoint, const FVector2f* OptionalAllotedSize = nullptr);
	static ENGINE_API bool GetPixelPoint(const FSceneViewProjectionData& ProjectionData, const FVector& InPoint, FVector2D& OutPoint, const FVector2f* OptionalAllotedSize = nullptr);

	/**
	 * Helper function for deriving various bits of data needed for projection
	 *
	 * @param	Viewport				The ViewClient's viewport
	 * @param	ProjectionData			The structure to be filled with projection data
     * @param	StereoViewIndex		    The index of the view when using stereoscopy
	 * @return  False if there is no viewport, or if the Actor is null
	 */
	ENGINE_API virtual bool GetProjectionData(FViewport* Viewport, FSceneViewProjectionData& ProjectionData, int32 StereoViewIndex = INDEX_NONE) const;

	/**
	 * Determines whether this player is the first and primary player on their machine.
	 * @return	true if this player is not using splitscreen, or is the first player in the split-screen layout.
	 */
	ENGINE_API bool IsPrimaryPlayer() const;
	 
	/**
	 * Clear cached view state.  Suitable for calling when cleaning up the world but the view state has some references objects (usually mids) owned by the world (thus preventing GC) 
	 */
	ENGINE_API virtual void CleanupViewState(FStringView MidParentRootPath = {});

	/** Locked view state needs access to GetViewPoint. */
	friend class FLockedViewState;
};

