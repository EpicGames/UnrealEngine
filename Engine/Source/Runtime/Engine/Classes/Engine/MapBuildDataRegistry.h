// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Registry for built data from a map build
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "SceneTypes.h"
#include "UObject/UObjectAnnotation.h"
#include "RenderCommandFence.h"
#include "LightMap.h"
#include "Engine/TextureCube.h"
#include "MapBuildDataRegistry.generated.h"

class FPrecomputedLightVolumeData;
class FPrecomputedVolumetricLightmapData;
struct FAssetCompileData;
struct FVolumetricLightMapGridDesc;

DECLARE_LOG_CATEGORY_EXTERN(LogMapBuildDataRegistry, Log, All);

#define UE_LOG_MAPBUILDATA_ENABLED 0

#if UE_LOG_MAPBUILDATA_ENABLED
#define UE_LOG_MAPBUILDDATA(...) UE_LOG(LogMapBuildDataRegistry, __VA_ARGS__)
#else
#define UE_LOG_MAPBUILDDATA(...) 
#endif

struct FPerInstanceLightmapData
{
	FVector2f LightmapUVBias;
	FVector2f ShadowmapUVBias;

	FPerInstanceLightmapData()
		: LightmapUVBias(ForceInit)
		, ShadowmapUVBias(ForceInit)
	{}

	friend FArchive& operator<<(FArchive& Ar, FPerInstanceLightmapData& InstanceData)
	{
		// @warning BulkSerialize: FPerInstanceLightmapData is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << InstanceData.LightmapUVBias << InstanceData.ShadowmapUVBias;
		return Ar;
	}
};

class FMeshMapBuildData
{
public:
	FLightMapRef LightMap;
	FShadowMapRef ShadowMap;
	TArray<FGuid> IrrelevantLights;
	TArray<FPerInstanceLightmapData> PerInstanceLightmapData;
	const FLightmapResourceCluster* ResourceCluster;

	ENGINE_API FMeshMapBuildData();

	/** Destructor. */
	ENGINE_API ~FMeshMapBuildData();

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return LightMap == DefaultAnnotation.LightMap && ShadowMap == DefaultAnnotation.ShadowMap;
	}

	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/** Serializer. */
	friend ENGINE_API FArchive& operator<<(FArchive& Ar,FMeshMapBuildData& MeshMapBuildData);

	/** Default state for annotations (no flags changed)*/
	static const FMeshMapBuildData DefaultAnnotation;
};

class FMeshMapBuildLegacyData
{
public:

	TArray<TPair<FGuid, FMeshMapBuildData*>> Data;

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return Data.Num() == 0;
	}
};

class FStaticShadowDepthMapData
{
public:
	/** Transform from world space to the coordinate space that DepthSamples are stored in. */
	FMatrix WorldToLight;
	/** Dimensions of the generated shadow map. */
	int32 ShadowMapSizeX;
	int32 ShadowMapSizeY;
	/** Shadowmap depth values */
	TArray<FFloat16> DepthSamples;

	FStaticShadowDepthMapData() :
		WorldToLight(FMatrix::Identity),
		ShadowMapSizeX(0),
		ShadowMapSizeY(0)
	{}

	ENGINE_API void Empty();

	size_t GetAllocatedSize() const
	{
		return DepthSamples.GetAllocatedSize();
	}
	friend FArchive& operator<<(FArchive& Ar, FStaticShadowDepthMapData& ShadowMap);
};

class FLevelLegacyMapBuildData
{
public:

	FGuid Id;
	FPrecomputedLightVolumeData* Data;

	FLevelLegacyMapBuildData() :
		Data(NULL)
	{}

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return Id == FGuid();
	}
};

class FLightComponentMapBuildData
{
public:

	FLightComponentMapBuildData() :
		ShadowMapChannel(INDEX_NONE)
	{}

	ENGINE_API ~FLightComponentMapBuildData();

	ENGINE_API void FinalizeLoad();

	/** 
	 * Shadow map channel which is used to match up with the appropriate static shadowing during a deferred shading pass.
	 * This is generated during a lighting build.
	 */
	int32 ShadowMapChannel;

	FStaticShadowDepthMapData DepthMap;

	friend FArchive& operator<<(FArchive& Ar, FLightComponentMapBuildData& ShadowMap);
};

class FLightComponentLegacyMapBuildData
{
public:

	FGuid Id;
	FLightComponentMapBuildData* Data;

	FLightComponentLegacyMapBuildData() :
		Data(NULL)
	{}

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return Id == FGuid();
	}
};

class FReflectionCaptureData
{
public:
	int32 CubemapSize;
	float AverageBrightness;

	TArray<uint8> FullHDRCapturedData;
	TArray<uint8> EncodedHDRCapturedData;

	FReflectionCaptureData() :
		CubemapSize(0),
		AverageBrightness(0.0f),
		bUploadedFinal(false)
	{}

	bool HasBeenUploadedFinal() const
	{
		return bUploadedFinal;
	}

	void OnDataUploadedToGPUFinal()
	{
		check(!bUploadedFinal);

		// In the editor we need this data for serialization
		if (!GIsEditor)
		{
			FullHDRCapturedData.Empty();
			EncodedHDRCapturedData.Empty();
			CubemapSize = 0;
			bUploadedFinal = true;
		}
	}

private:
	bool bUploadedFinal;
};

class FReflectionCaptureMapBuildData : public FReflectionCaptureData
{
public:

	// Stored redundantly to support stats after discarding data
	size_t AllocatedSize;

	FReflectionCaptureMapBuildData() :
		AllocatedSize(0)
	{}

	ENGINE_API ~FReflectionCaptureMapBuildData();

	size_t GetAllocatedSize() const
	{
		return AllocatedSize;
	}

	ENGINE_API void FinalizeLoad();

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return CubemapSize == DefaultAnnotation.CubemapSize && FullHDRCapturedData.Num() == DefaultAnnotation.FullHDRCapturedData.Num();
	}

	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/** Serializer. */
	friend ENGINE_API FArchive& operator<<(FArchive& Ar,FReflectionCaptureMapBuildData& ReflectionCaptureMapBuildData);

	/** Default state for annotations (no flags changed)*/
	static const FReflectionCaptureMapBuildData DefaultAnnotation;
};

class FReflectionCaptureMapBuildLegacyData
{
public:

	FGuid Id;
	FReflectionCaptureMapBuildData* MapBuildData;

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return Id == FGuid();
	}
};

class FSkyAtmosphereMapBuildData
{
public:
	bool bDummy = false;
};

UCLASS(MinimalAPI)
class UMapBuildDataRegistry : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	/** The lighting quality the level was last built with */
	UPROPERTY(Category=Lighting, VisibleAnywhere)
	TEnumAsByte<enum ELightingBuildQuality> LevelLightingQuality;

	//~ Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
	//~ End UObject Interface

	/** 
	 * Allocates a new FMeshMapBuildData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FMeshMapBuildData& AllocateMeshBuildData(const FGuid& MeshId, bool bMarkDirty);
	ENGINE_API const FMeshMapBuildData* GetMeshBuildData(FGuid MeshId) const;
	ENGINE_API FMeshMapBuildData* GetMeshBuildData(FGuid MeshId);
	ENGINE_API FMeshMapBuildData* GetMeshBuildDataDuringBuild(FGuid MeshId);

	/** 
	 * Allocates a new FPrecomputedLightVolumeData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FPrecomputedLightVolumeData& AllocateLevelPrecomputedLightVolumeBuildData(const FGuid& LevelId);
	ENGINE_API void AddLevelPrecomputedLightVolumeBuildData(const FGuid& LevelId, FPrecomputedLightVolumeData* InData);
	ENGINE_API const FPrecomputedLightVolumeData* GetLevelPrecomputedLightVolumeBuildData(FGuid LevelId) const;
	ENGINE_API FPrecomputedLightVolumeData* GetLevelPrecomputedLightVolumeBuildData(FGuid LevelId);

	/** 
	 * Allocates a new FPrecomputedVolumetricLightmapData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FPrecomputedVolumetricLightmapData& AllocateLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId);
	ENGINE_API void AddLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId, FPrecomputedVolumetricLightmapData* InData);
	ENGINE_API const FPrecomputedVolumetricLightmapData* GetLevelPrecomputedVolumetricLightmapBuildData(FGuid LevelId) const;
	ENGINE_API FPrecomputedVolumetricLightmapData* GetLevelPrecomputedVolumetricLightmapBuildData(FGuid LevelId);

	/** 
	 * Allocates a new FLightComponentMapBuildData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FLightComponentMapBuildData& FindOrAllocateLightBuildData(FGuid LightId, bool bMarkDirty);
	ENGINE_API const FLightComponentMapBuildData* GetLightBuildData(FGuid LightId) const;
	ENGINE_API FLightComponentMapBuildData* GetLightBuildData(FGuid LightId);

	/** 
	 * Allocates a new FReflectionCaptureMapBuildData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FReflectionCaptureMapBuildData& AllocateReflectionCaptureBuildData(const FGuid& CaptureId, bool bMarkDirty);
	ENGINE_API const FReflectionCaptureMapBuildData* GetReflectionCaptureBuildData(FGuid CaptureId) const;
	ENGINE_API FReflectionCaptureMapBuildData* GetReflectionCaptureBuildData(FGuid CaptureId);

	/**
	 * Allocates a new FSkyAtmosphereMapBuildData from the registry.
	 * Warning: Further allocations will invalidate the returned reference.
	 */
	ENGINE_API FSkyAtmosphereMapBuildData& FindOrAllocateSkyAtmosphereBuildData(const FGuid& Guid);
	/**
	 * @returns pointer to the FSkyAtmosphereMapBuildData, nullptr if built data has not been built yet.
	 */
	ENGINE_API const FSkyAtmosphereMapBuildData* GetSkyAtmosphereBuildData(const FGuid& Guid) const;
	ENGINE_API void ClearSkyAtmosphereBuildData();

	ENGINE_API void InvalidateStaticLighting(UWorld* World, bool bRecreateRenderState = true, const TSet<FGuid>* ResourcesToKeep = nullptr);
	ENGINE_API void InvalidateSurfaceLightmaps(UWorld* World, bool bRecreateRenderState = true, const TSet<FGuid>* ResourcesToKeep = nullptr);
	ENGINE_API void InvalidateReflectionCaptures(const TSet<FGuid>* ResourcesToKeep = nullptr);

	ENGINE_API bool IsLegacyBuildData() const;

	ENGINE_API bool IsLightingValid(ERHIFeatureLevel::Type InFeatureLevel) const;

	/** Must be called once MeshBuildData is done being modified, to build resource clusters. */
	ENGINE_API void SetupLightmapResourceClusters();

	ENGINE_API void GetLightmapResourceClusterStats(int32& NumMeshes, int32& NumClusters) const;

	/** Initializes rendering resources for all lightmap resource clusters. */
	ENGINE_API void InitializeClusterRenderingResources(ERHIFeatureLevel::Type InFeatureLevel);
	
	/**
		Called by HandleLegacyMapBuildData with legacy BuildData without ReflectionCapture Data
		or called by PostLoad for legacy BuildData with old EncodedData
	*/
	ENGINE_API void HandleLegacyEncodedCubemapData();

	/** 
	    Various helpers to access the UUMapBuildDataRegistry, necessary since the access patterns
		are different in WorldPartition maps than in map. 
	*/

	/** Returns the associated UUMapBuildDataRegistry for an Actor Component */
	ENGINE_API static UMapBuildDataRegistry* Get(const UActorComponent* Component);

	/** Returns the associated UUMapBuildDataRegistry for an Actor */
	ENGINE_API static UMapBuildDataRegistry* Get(const AActor* Actor);

	/** Returns the associated UUMapBuildDataRegistry for a pair of Level/World */
	ENGINE_API static UMapBuildDataRegistry* Get(ULevel* OwnerLevel, UWorld* World);
	
	FVolumetricLightMapGridDesc* GetVolumetricLightMapGridDesc() { return VolumetricLightMapGridDesc; } 
	ENGINE_API void SetVolumetricLightMapGridDesc(FVolumetricLightMapGridDesc* GridDesc); 

#if WITH_EDITOR
	void RedirectToRegistry(TArray<FGuid>& ActorInstances, UMapBuildDataRegistry* Registry);
	void RemoveRedirect(TArray<FGuid>& ActorInstances, UMapBuildDataRegistry* Registry);
#else
	void RemoveRegistry(UMapBuildDataRegistry* Registry);
#endif


private:
#if WITH_EDITOR
	void HandleAssetPostCompileEvent(const TArray<FAssetCompileData>& CompiledAssets);
#endif

	void ReleaseResources(const TSet<FGuid>* ResourcesToKeep = nullptr);
	void EmptyLevelData(const TSet<FGuid>* ResourcesToKeep = nullptr);
	void CleanupTransientOverrideMapBuildData();

	TMap<FGuid, FMeshMapBuildData> MeshBuildData;
	TMap<FGuid, FPrecomputedLightVolumeData*> LevelPrecomputedLightVolumeBuildData;
	TMap<FGuid, FPrecomputedVolumetricLightmapData*> LevelPrecomputedVolumetricLightmapBuildData;
	TMap<FGuid, FLightComponentMapBuildData> LightBuildData;
	TMap<FGuid, FReflectionCaptureMapBuildData> ReflectionCaptureBuildData;
	TMap<FGuid, FSkyAtmosphereMapBuildData> SkyAtmosphereBuildData;

	bool bSetupResourceClusters;
	TArray<FLightmapResourceCluster> LightmapResourceClusters;

	FRenderCommandFence DestroyFence;

	FVolumetricLightMapGridDesc*	VolumetricLightMapGridDesc;

	UMapBuildDataRegistry* FindRegistryWorldPartition(const AActor* Actor);

#if WITH_EDITOR
	TMap<FGuid, UMapBuildDataRegistry*> Redirects;
	TMap<FName, int32> RedirectedRegistriesRefcount;
#else
	FCriticalSection PackagesToMapBuildDataLock;
	TMap<UPackage*, UMapBuildDataRegistry*>	PackagesToMapBuildData;
#endif
};

extern ENGINE_API FUObjectAnnotationSparse<FMeshMapBuildLegacyData, true> GComponentsWithLegacyLightmaps;
extern ENGINE_API FUObjectAnnotationSparse<FLevelLegacyMapBuildData, true> GLevelsWithLegacyBuildData;
extern ENGINE_API FUObjectAnnotationSparse<FLightComponentLegacyMapBuildData, true> GLightComponentsWithLegacyBuildData;
extern ENGINE_API FUObjectAnnotationSparse<FReflectionCaptureMapBuildLegacyData, true> GReflectionCapturesWithLegacyBuildData;
