// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineTypes.h"
#include "MaterialMerging.generated.h"

struct FMeshDescription;

UENUM()
enum ETextureSizingType : int
{
	TextureSizingType_UseSingleTextureSize UMETA(DisplayName = "Use TextureSize for all material properties"),
	TextureSizingType_UseAutomaticBiasedSizes UMETA(DisplayName = "Use automatically biased texture sizes based on TextureSize"),
	TextureSizingType_UseManualOverrideTextureSize UMETA(DisplayName = "Use per property manually overriden texture sizes"),
	TextureSizingType_UseSimplygonAutomaticSizing UMETA(DisplayName = "Use Simplygon's automatic texture sizing"),
	TextureSizingType_AutomaticFromTexelDensity UMETA(DisplayName = "Automatic - From Texel Density"),
	TextureSizingType_AutomaticFromMeshScreenSize UMETA(DisplayName = "Automatic - From Mesh Screen Size"),
	TextureSizingType_AutomaticFromMeshDrawDistance UMETA(DisplayName = "Automatic - From Mesh Draw Distance", ToolTip = "When working with World Partition HLODs, the draw distance is automatically deduced from the runtime grid loading range."),
	TextureSizingType_MAX,
};

UENUM()
enum EMaterialMergeType : int
{
	MaterialMergeType_Default,
	MaterialMergeType_Simplygon
};

USTRUCT(Blueprintable)
struct FMaterialProxySettings
{
	GENERATED_USTRUCT_BODY()

	// Method that should be used to generate the sizes of the output textures
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	TEnumAsByte<ETextureSizingType> TextureSizingType;

	// Size of generated BaseColor map
	UPROPERTY(Category = Material, EditAnywhere, meta =(ClampMin = "1", UIMin = "1", EditConditionHides, EditCondition = "TextureSizingType == ETextureSizingType::TextureSizingType_UseSingleTextureSize || TextureSizingType == ETextureSizingType::TextureSizingType_UseAutomaticBiasedSizes"))
	FIntPoint TextureSize;

	// Target texel density
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (ClampMin = "0.1", ClampMax = "1024", EditConditionHides, EditCondition = "TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromTexelDensity"))
	float TargetTexelDensityPerMeter;

	// Expected maximum screen size for the mesh
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (ClampMin = "0.01", ClampMax = "1.0", EditConditionHides, EditCondition = "TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize"))
	float MeshMaxScreenSizePercent;

	// Expected minimum distance at which the mesh will be rendered
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (ClampMin = "0", EditConditionHides, EditCondition = "TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance"))
	double MeshMinDrawDistance;
	
	// Gutter space to take into account 
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere)
	float GutterSpace;

	// Constant value to use for the Metallic property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bMetallicMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bMetallicMap"))
	float MetallicConstant;

	// Constant value to use for the Roughness property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bRoughnessMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bRoughnessMap"))
	float RoughnessConstant;

	// Constant value to use for the Anisotropy property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bAnisotropyMap", ClampMin = "-1", ClampMax = "1", UIMin = "-1", UIMax = "1", editcondition = "!bAnisotropyMap"))
	float AnisotropyConstant;

	// Constant value to use for the Specular property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bSpecularMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bSpecularMap"))
	float SpecularConstant;

	// Constant value to use for the Opacity property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bOpacityMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bOpacityMap"))
	float OpacityConstant;

	// Constant value to use for the Opacity mask property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bOpacityMaskMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bOpacityMaskMap"))
	float OpacityMaskConstant;

	// Constant value to use for the Ambient Occlusion property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter="bAmbientOcclusionMap", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", editcondition = "!bAmbientOcclusionMap"))
	float AmbientOcclusionConstant;

	UPROPERTY()
	TEnumAsByte<EMaterialMergeType> MaterialMergeType;

	// Target blend mode for the generated material
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta=(DisplayAfter="AmbientOcclusionTextureSize"))
	TEnumAsByte<EBlendMode> BlendMode;

	// Whether or not to allow the generated material can be two-sided
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere, meta = (DisplayAfter = "BlendMode"))
	uint8 bAllowTwoSidedMaterial : 1;

	// Whether to generate a texture for the Normal property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bNormalMap:1;

	// Whether to generate a texture for the Tangent property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bTangentMap:1;

	// Whether to generate a texture for the Metallic property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bMetallicMap:1;

	// Whether to generate a texture for the Roughness property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bRoughnessMap:1;

	// Whether to generate a texture for the Anisotropy property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bAnisotropyMap:1;

	// Whether to generate a texture for the Specular property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bSpecularMap:1;

	// Whether to generate a texture for the Emissive property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bEmissiveMap:1;

	// Whether to generate a texture for the Opacity property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bOpacityMap:1;

	// Whether to generate a texture for the Opacity Mask property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bOpacityMaskMap:1;

	// Whether to generate a texture for the Ambient Occlusion property
	UPROPERTY(Category = Material, BlueprintReadWrite, EditAnywhere)
	uint8 bAmbientOcclusionMap:1;

	// Override Diffuse texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint DiffuseTextureSize;

	// Override Normal texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint NormalTextureSize;

	// Override Tangent texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint TangentTextureSize;

	// Override Metallic texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint MetallicTextureSize;

	// Override Roughness texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint RoughnessTextureSize;

	// Override Anisotropy texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint AnisotropyTextureSize;

	// Override Specular texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint SpecularTextureSize;

	// Override Emissive texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint EmissiveTextureSize;

	// Override Opacity texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint OpacityTextureSize;
	
	// Override Opacity Mask texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint OpacityMaskTextureSize;

	// Override Ambient Occlusion texture size
	UPROPERTY(Category = Material, BlueprintReadWrite, AdvancedDisplay, EditAnywhere, meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint AmbientOcclusionTextureSize;

	ENGINE_API FMaterialProxySettings();

	ENGINE_API bool operator == (const FMaterialProxySettings& Other) const;
	ENGINE_API bool operator != (const FMaterialProxySettings& Other) const;

	ENGINE_API FIntPoint GetMaxTextureSize() const;

#if WITH_EDITOR
	ENGINE_API bool ResolveTexelDensity(const TArray<class UPrimitiveComponent*>& InComponents);
	ENGINE_API bool ResolveTexelDensity(const TArray<class UPrimitiveComponent*>& InComponents, float& OutTexelDensity) const;

	ENGINE_API void ResolveTextureSize(const FMeshDescription& InMesh);
	ENGINE_API void ResolveTextureSize(const float InWorldSpaceRadius, const double InWorldSpaceArea, const double InUVSpaceArea = 1.0);
#endif
};
