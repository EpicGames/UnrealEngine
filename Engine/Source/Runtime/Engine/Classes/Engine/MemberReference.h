// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Blueprint.h"
#include "EngineLogs.h"
#include "MemberReference.generated.h"

/** Helper struct to allow us to redirect properties and functions through renames and additionally between classes if necessary */
struct FFieldRemapInfo
{
	/** The new name of the field after being renamed */
	FName FieldName;

	/** The new name of the field's outer class if different from its original location, or NAME_None if it hasn't moved */
	FName FieldClass;

	bool operator==(const FFieldRemapInfo& Other) const
	{
		return FieldName == Other.FieldName && FieldClass == Other.FieldClass;
	}

	friend uint32 GetTypeHash(const FFieldRemapInfo& RemapInfo)
	{
		return GetTypeHash(RemapInfo.FieldName) + GetTypeHash(RemapInfo.FieldClass) * 23;
	}

	FFieldRemapInfo()
		: FieldName(NAME_None)
		, FieldClass(NAME_None)
	{
	}
};

/** Helper struct to allow us to redirect pin name for node class */
struct FParamRemapInfo
{
	bool	bCustomValueMapping;
	FName	OldParam;
	FName	NewParam;
	FName	NodeTitle;
	TMap<FString, FString> ParamValueMap;

	// constructor
	FParamRemapInfo()
		: OldParam(NAME_None)
		, NewParam(NAME_None)
		, NodeTitle(NAME_None)
	{
	}
};

// @TODO: this can encapsulate globally defined fields as well (like with native 
//        delegate signatures); consider renaming to FFieldReference
USTRUCT()
struct FMemberReference
{
	GENERATED_USTRUCT_BODY()

protected:
	/** 
	 * Most often the Class that this member is defined in. Could be a UPackage 
	 * if it is a native delegate signature function (declared globally). Should 
	 * be NULL if bSelfContext is true.  
	 */
	UPROPERTY(SaveGame)
	mutable TObjectPtr<UObject> MemberParent;

	/**  */
	UPROPERTY(SaveGame)
	mutable FString MemberScope;

	/** Name of variable */
	UPROPERTY(SaveGame)
	mutable FName MemberName;

	/** The Guid of the variable */
	UPROPERTY(SaveGame)
	mutable FGuid MemberGuid;

	/** Whether or not this should be a "self" context */
	UPROPERTY(SaveGame)
	mutable bool bSelfContext;

	/** Whether or not this property has been deprecated */
	UPROPERTY(SaveGame)
	mutable bool bWasDeprecated;
	
public:
	FMemberReference()
		: MemberParent(nullptr)
		, MemberName(NAME_None)
		, bSelfContext(false)
		, bWasDeprecated(false)
	{
	}

	/** Set up this reference from a supplied field */
	template<class TFieldType>
	void SetFromField(const typename TFieldType::BaseFieldClass* InField, const bool bIsConsideredSelfContext, UClass* OwnerClass=nullptr)
	{
		// if we didn't get an owner passed in try to figure out what it should be based on the field
		if (!OwnerClass)
		{
			OwnerClass = InField->GetOwnerClass();
		}
		MemberParent = OwnerClass;

		if (bIsConsideredSelfContext)
		{
			MemberParent = nullptr;
		}
		else if ((MemberParent == nullptr) && InField->GetName().EndsWith(HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX))
		{
			MemberParent = InField->GetOutermost();
		}

		MemberName = InField->GetFName();
		bSelfContext = bIsConsideredSelfContext;
		bWasDeprecated = false;

#if WITH_EDITOR
		if (UClass* ParentAsClass = GetMemberParentClass())
		{
			MemberParent = ParentAsClass->GetAuthoritativeClass();
		}
#endif

		MemberGuid.Invalidate();
		if (OwnerClass != nullptr)
		{
			UBlueprint::GetGuidFromClassByFieldName<TFieldType>(OwnerClass, InField->GetFName(), MemberGuid);
		}
	}

#if WITH_EDITOR
	template<class TFieldType>
	void SetFromField(const typename TFieldType::BaseFieldClass* InField, UClass* SelfScope)
	{
		UClass* OwnerClass = InField->GetOwnerClass();

		FGuid FieldGuid;
		if (OwnerClass != nullptr)
		{
			UBlueprint::GetGuidFromClassByFieldName<TFieldType>(OwnerClass, InField->GetFName(), FieldGuid);
		}

		SetGivenSelfScope(InField->GetFName(), FieldGuid, OwnerClass, SelfScope);
	}

	/** Update given a new self */
	template<class TFieldType>
	void RefreshGivenNewSelfScope(UClass* SelfScope)
	{
		UClass* ParentAsClass = GetMemberParentClass();
		if ((ParentAsClass != nullptr) && (SelfScope != nullptr))
		{
			UBlueprint::GetGuidFromClassByFieldName<TFieldType>((ParentAsClass), MemberName, MemberGuid);
			SetGivenSelfScope(MemberName, MemberGuid, ParentAsClass, SelfScope);
		}
		else
		{
			// We no longer have enough information to known if we've done the right thing, and just have to hope...
		}
	}
#endif

	/** Set to a non-'self' member, so must include reference to class owning the member. */
	ENGINE_API void SetExternalMember(FName InMemberName, TSubclassOf<class UObject> InMemberParentClass);
	ENGINE_API void SetExternalMember(FName InMemberName, TSubclassOf<class UObject> InMemberParentClass, const FGuid& InMemberGuid);

	/** Set to reference a global field (intended for things like natively defined delegate signatures) */
	ENGINE_API void SetGlobalField(FName InFieldName, UPackage* InParentPackage);

	/** Set to a non-'self' delegate member, this is not self-context but is not given a parent class */
	ENGINE_API void SetExternalDelegateMember(FName InMemberName);

	/** Set up this reference to a 'self' member name */
	ENGINE_API void SetSelfMember(FName InMemberName);
	ENGINE_API void SetSelfMember(FName InMemberName, const FGuid& InMemberGuid);

	/** Set up this reference to a 'self' member name, scoped to a struct */
	ENGINE_API void SetLocalMember(FName InMemberName, UStruct* InScope, const FGuid InMemberGuid);

	/** Set up this reference to a 'self' member name, scoped to a struct name */
	ENGINE_API void SetLocalMember(FName InMemberName, FString InScopeName, const FGuid InMemberGuid);

	/** Only intended for backwards compat! */
	ENGINE_API void SetDirect(const FName InMemberName, const FGuid InMemberGuid, TSubclassOf<class UObject> InMemberParentClass, bool bIsConsideredSelfContext);

	/** Invalidate the current MemberParent, if this is a self scope, or the MemberScope if it is not (and set).  Intended for PostDuplication fixups */
	ENGINE_API void InvalidateScope();

	/** Get the name of this member */
	FName GetMemberName() const
	{
		return MemberName;
	}

#if WITH_EDITOR
	/** Reset the member name only. Intended for use primarily as a helper method for rename operations. */
	void SetMemberName(FName NewName)
	{
		MemberName = NewName;
	}
#endif

	FGuid GetMemberGuid() const
	{
		return MemberGuid;
	}

	UClass* GetMemberParentClass() const
	{
		return Cast<UClass>(MemberParent);
	}

	UPackage* GetMemberParentPackage() const
	{
		if (UPackage* ParentAsPackage = Cast<UPackage>(MemberParent))
		{
			return ParentAsPackage;
		}
		else if (MemberParent != nullptr)
		{
			return MemberParent->GetOutermost();
		}
		return nullptr;
	}

	/** Returns if this is a 'self' context. */
	bool IsSelfContext() const
	{
		return bSelfContext;
	}

	/** Returns if this is a local scope. */
	bool IsLocalScope() const
	{
		return !MemberScope.IsEmpty();
	}

	/** Returns true if this is in the sparse data struct for OwningClass */
	ENGINE_API bool IsSparseClassData(const UClass* OwningClass) const;

#if WITH_EDITOR
	/**
	 * Returns a search string to submit to Find-in-Blueprints to find references to this reference
	 *
	 * @param InFieldOwner		The owner of the field, cannot be resolved internally
	 * @return					Search string to find this reference in other Blueprints
	 */
	ENGINE_API FString GetReferenceSearchString(UClass* InFieldOwner) const;
#endif

private:
#if WITH_EDITOR
	/**
	 * Refreshes a local variable reference name if it has changed
	 *
	 * @param InSelfScope		Scope to lookup the variable in, to see if it has changed
	 */
	ENGINE_API FName RefreshLocalVariableName(UClass* InSelfScope) const;
#endif

protected:
#if WITH_EDITOR
	/** Only intended for backwards compat! */
	ENGINE_API void SetGivenSelfScope(const FName InMemberName, const FGuid InMemberGuid, TSubclassOf<class UObject> InMemberParentClass, TSubclassOf<class UObject> SelfScope) const;
#endif
public:

	/** Get the class that owns this member */
	UClass* GetMemberParentClass(UClass* SelfScope) const
	{
		// Local variables with a MemberScope act much the same as being SelfContext, their parent class is SelfScope.
		return (bSelfContext || !MemberScope.IsEmpty())? SelfScope : GetMemberParentClass();
	}

	/** Get the scope of this member */
	UStruct* GetMemberScope(UClass* InMemberParentClass) const
	{
		return FindUField<UStruct>(InMemberParentClass, *MemberScope);
	}

	/** Get the name of the scope of this member */
	FString GetMemberScopeName() const
	{
		return MemberScope;
	}

	/** Compares with another MemberReference to see if they are identical */
	bool IsSameReference(const FMemberReference& InReference) const
	{
		return 
			bSelfContext == InReference.bSelfContext &&
			MemberParent == InReference.MemberParent &&
			MemberName == InReference.MemberName &&
			MemberGuid == InReference.MemberGuid &&
			MemberScope == InReference.MemberScope;
	}

	/** Returns whether or not the variable has been deprecated */
	bool IsDeprecated() const
	{
		return bWasDeprecated;
	}

	/** Returns the scope for the current member. This will vary based on whether or not this member uses the self context or not. */
	UClass* GetScope(UClass* SelfScope = nullptr) const
	{
		return bSelfContext ? SelfScope : GetMemberParentClass();
	}

	/** 
	 *	Returns the member FProperty/UFunction this reference is pointing to, or NULL if it no longer exists 
	 *	Derives 'self' scope from supplied Blueprint node if required
	 *	Will check for redirects and fix itself up if one is found (when WITH_EDITOR, or when bAlwaysFollowRedirects is true).
	 */
	template<class TFieldType>
	TFieldType* ResolveMember(UClass* SelfScope = nullptr, const bool bAlwaysFollowRedirects = false) const
	{
		return static_cast<TFieldType*>(ResolveMemberProperty(SelfScope, bAlwaysFollowRedirects, TFieldType::StaticClass()));
	}
	template<>
	UFunction* ResolveMember(UClass* SelfScope, const bool bAlwaysFollowRedirects) const
	{
		return ResolveMemberFunction(SelfScope, bAlwaysFollowRedirects);
	}

	/** ResolveMember overload for UBlueprint, uses the skeleton class so it can be more up to date */
	template<class TFieldType>
	TFieldType* ResolveMember(UBlueprint* SelfScope)
	{
		return ResolveMember<TFieldType>(SelfScope->SkeletonGeneratedClass);
	}

	/**
	 * Searches the field redirect map for the specified named field in the scope, and returns the remapped field if found
	 *
	 * @param	FieldClass		UClass of field type we are looking for
	 * @param	InitialScope	The scope the field was initially defined in.  The function will search up into parent scopes to attempt to find remappings
	 * @param	InitialName		The name of the field to attempt to find a redirector for
	 * @param	bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect If true, and a redirect is found in a parent scope, and the targetscope of the redirect
	 *          is a child scope of the parent scope that is not the initial scope or a parent of the initial scope, then the redirect will be ignored.
	 * @return	The remapped field, if one exists
	 */
	ENGINE_API static UField* FindRemappedField(UClass *FieldClass, UClass* InitialScope, FName InitialName,
		bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect = false);
	ENGINE_API static FField* FindRemappedField(FFieldClass* FieldClass, UClass* InitialScope, FName InitialName,
		bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect = false);

	/** Templated version of above, extracts FieldClass and Casts result */
	template<class TFieldType>
	static TFieldType* FindRemappedField(UClass* InitialScope, FName InitialName,
		bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect = false)
	{
		return FFieldVariant(FindRemappedField(TFieldType::StaticClass(), InitialScope, InitialName,
			bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect)).Get<TFieldType>();
	}

#if WITH_EDITOR
	/** Init the field redirect map (if not already done) from .ini file entries */
	ENGINE_API static void InitFieldRedirectMap();

protected:

	/** Has the field map been initialized this run */
	static bool bFieldRedirectMapInitialized;
	
	/** @return the 'real' generated class for blueprint classes, but only if we're already passed through CompileClassLayout */
	ENGINE_API static UClass* GetClassToUse(UClass* InClass, bool bUseUpToDateClass);
#endif
	template<class TFieldType, class TFieldTypeClass>
	TFieldType* ResolveMemberImpl(UClass* SelfScope, TFieldTypeClass* FieldClas, const bool bAlwaysFollowRedirects, const bool bIsUFunctionOrMulticastDelegate) const;

	ENGINE_API FProperty* ResolveMemberProperty(UClass* SelfScope, const bool bAlwaysFollowRedirects, FFieldClass* FieldClass) const;
	ENGINE_API UFunction* ResolveMemberFunction(UClass* SelfScope, const bool bAlwaysFollowRedirects) const;

public:
	template<class TFieldType>
	static void FillSimpleMemberReference(const TFieldType* InField, FSimpleMemberReference& OutReference)
	{
		OutReference.Reset();

		if (InField)
		{
			FMemberReference TempMemberReference;
			TempMemberReference.SetFromField<TFieldType>(InField, false);

			OutReference.MemberName = TempMemberReference.MemberName;
			OutReference.MemberParent = TempMemberReference.MemberParent;
			OutReference.MemberGuid = TempMemberReference.MemberGuid;
		}
	}

	template<class TFieldType>
	static TFieldType* ResolveSimpleMemberReference(const FSimpleMemberReference& Reference, UClass* SelfScope = nullptr)
	{
		FMemberReference TempMemberReference;

		const FName Name = Reference.MemberGuid.IsValid() ? NAME_None : Reference.MemberName; // if the guid is valid don't check the name, it could be renamed 
		TempMemberReference.MemberName   = Name;
		TempMemberReference.MemberGuid   = Reference.MemberGuid;
		TempMemberReference.MemberParent = Reference.MemberParent;

		auto Result = TempMemberReference.ResolveMember<TFieldType>(SelfScope);
		if (!Result && (Name != Reference.MemberName))
		{
			TempMemberReference.MemberName = Reference.MemberName;
			Result = TempMemberReference.ResolveMember<TFieldType>(SelfScope);
		}

		return Result;
	}
};

