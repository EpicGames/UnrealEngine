// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "MeshMergeCullingVolume.generated.h"

/** A volume that can be added a level in order to remove triangles from source meshes before generating HLOD or merged meshes */
UCLASS(Experimental, MinimalAPI)
class AMeshMergeCullingVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

public:

};
