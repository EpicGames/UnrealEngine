// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "MeshSimplificationSettings.generated.h"

UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Mesh Simplification"), MinimalAPI)
class UMeshSimplificationSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual FName GetContainerName() const override;
	ENGINE_API virtual FName GetCategoryName() const override;

	ENGINE_API void SetMeshReductionModuleName(FName InMeshReductionModuleName);

public:
	/** Mesh reduction plugin to use when simplifying mesh geometry */
	UPROPERTY(config, EditAnywhere, Category=General, meta=(ConsoleVariable="r.MeshReductionModule", DisplayName="Mesh Reduction Plugin", ConfigRestartRequired=true))
	FName MeshReductionModuleName;

	UPROPERTY(config, EditAnywhere, Category=General, meta=(DisplayName="Mesh Reduction Backward Compatible", ConfigRestartRequired = true))
	bool bMeshReductionBackwardCompatible;

	ENGINE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};
