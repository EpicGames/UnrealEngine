// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This is the base class for per-platform microtransaction support
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/PlatformInterfaceBase.h"
#include "MicroTransactionBase.generated.h"

/** All the types of delegate callbacks that a MicroTransaction subclass may receive from C++. */
UENUM()
enum EMicroTransactionDelegate : int
{
	/**
	 * Data:None
	 * Desc:QueryForAvailablePurchases() is complete and AvailableProducts is ready for use.
	 */
	MTD_PurchaseQueryComplete,
	/**
	 * Data:Result code, and identifier of the product that completed.
	 * Type:Custom
	 * Desc:IntValue will have one of the enums in EMicroTransactionResult, and StringValue
	 *      will have the Identifier from the PurchaseInfo that was bought with BeginPurchase
	 *      If MTR_Failed was returned, then LastError and LastErrorSolution should be filled
	 *		out with the most recent localized and possible resolutions.
	 */
	MTD_PurchaseComplete,
	MTD_MAX,
};

/** Result of a purchase. */
UENUM()
enum EMicroTransactionResult : int
{
	MTR_Succeeded,
	MTR_Failed,
	MTR_Canceled,
	MTR_RestoredFromServer,
	MTR_MAX,
};

// enum EPurchaseType
// {
// 	MTPT_Consumable,
// 	MTPT_OneTime,
// 	MTPT_Subscription,
// };

/**
 * Purchase information structure
 */
USTRUCT()
struct FPurchaseInfo
{
	GENERATED_USTRUCT_BODY()

// 	/** What kind of microtransaction purchase is this? */
// 	var EPurchaseType Type;

	/** Unique identifier for the purchase */
	UPROPERTY()
	FString Identifier;

	/** Name displayable to the user */
	UPROPERTY()
	FString DisplayName;

	/** Description displayable to the user */
	UPROPERTY()
	FString DisplayDescription;

	/** Price displayable to the user */
	UPROPERTY()
	FString DisplayPrice;

};

UCLASS()
class UMicroTransactionBase : public UPlatformInterfaceBase
{
	GENERATED_UCLASS_BODY()

	/** The list of products available to purchase, filled out by the time a MTD_PurchaseQueryComplete is fired */
	UPROPERTY()
	TArray<struct FPurchaseInfo> AvailableProducts;

	/** In case of errors, this will describe the most recent error */
	UPROPERTY()
	FString LastError;

	/** In case of errors, this will describe possible solutions (if there are any) */
	UPROPERTY()
	FString LastErrorSolution;


	virtual void Init();

	/**
	 * Query system for what purchases are available. Will fire a MTD_PurchaseQueryComplete
	 * if this function returns true.
	 *
	 * @return True if the query started successfully (delegate will receive final result)
	 */
	virtual bool QueryForAvailablePurchases();

	/**
	 * @return True if the user is allowed to make purchases - should give a nice message if not
	 */
	virtual bool IsAllowedToMakePurchases();

	/**
	 * Triggers a product purchase. Will fire a MTF_PurchaseComplete if this function
	 * returns true.
	 *
	 * @param Index which product to purchase
	 * 
	 * @param True if the purchase was kicked off (delegate will receive final result)
	 */
	virtual bool BeginPurchase(int32 Index);
};

