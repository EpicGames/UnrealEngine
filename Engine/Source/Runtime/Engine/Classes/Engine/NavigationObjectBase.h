// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "NavigationObjectBase.generated.h"

UCLASS(hidecategories=(Lighting, LightColor, Force), ClassGroup=Navigation, NotBlueprintable, abstract, MinimalAPI)
class ANavigationObjectBase : public AActor, public INavAgentInterface
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY()
	TObjectPtr<class UCapsuleComponent> CapsuleComponent;

	/** Normal editor sprite. */
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> GoodSprite;

	/** Used to draw bad collision intersection in editor. */
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> BadSprite;
public:

	/** True if this nav point was spawned to be a PIE player start. */
	UPROPERTY()
	uint32 bIsPIEPlayerStart:1;


	//~ Begin AActor Interface
	ENGINE_API virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const override;

#if	WITH_EDITOR
	ENGINE_API virtual void PostEditMove(bool bFinished) override;
	ENGINE_API virtual void PostEditUndo() override;
#endif	// WITH_EDITOR
	//~ End AActor Interface

	/** @todo document */
	ENGINE_API virtual bool ShouldBeBased();
	/** @todo document */
	ENGINE_API virtual void FindBase();

	/** 
	 * Check to make sure the navigation is at a valid point. 
	 * Detect when path building is going to move a pathnode around
	 * This may be undesirable for LDs (ie b/c cover links define slots by offsets)
	 */
	ENGINE_API virtual void Validate();

	/** Return Physics Volume for this Actor. **/
	ENGINE_API class APhysicsVolume* GetNavPhysicsVolume();

	// INavAgentInterface start
	virtual FVector GetNavAgentLocation() const override { return GetActorLocation(); }
	ENGINE_API virtual void GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const override;
	// INavAgentInterface end

public:
	/** Returns CapsuleComponent subobject **/
	class UCapsuleComponent* GetCapsuleComponent() const { return CapsuleComponent; }
	/** Returns GoodSprite subobject **/
	class UBillboardComponent* GetGoodSprite() const { return GoodSprite; }
	/** Returns BadSprite subobject **/
	class UBillboardComponent* GetBadSprite() const { return BadSprite; }
};



