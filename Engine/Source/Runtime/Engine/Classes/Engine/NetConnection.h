// Copyright Epic Games, Inc. All Rights Reserved.

//
// A network connection.
//

#pragma once

#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Serialization/BitWriter.h"
#include "Serialization/CustomVersion.h"
#include "Misc/EngineVersion.h"
#include "Misc/NetworkGuid.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "GameFramework/OnlineReplStructs.h"
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"
#include "Engine/NetDriver.h"
#include "Net/DataBunch.h"
#include "Net/NetPacketNotify.h"
#include "Engine/Player.h"
#include "Engine/Channel.h"
#include "ProfilingDebugging/Histogram.h"
#include "Containers/ArrayView.h"
#include "Containers/CircularBuffer.h"
#include "Net/Core/Trace/NetTraceConfig.h"
#include "ReplicationDriver.h"
#include "Analytics/EngineNetAnalytics.h"
#include "Net/Common/Packets/PacketTraits.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Net/NetAnalyticsTypes.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Core/Connection/NetCloseResult.h"
#include "Net/TrafficControl.h"
#include "Net/NetDormantHolder.h"
#include "HAL/LowLevelMemTracker.h"
#include "GameFramework/Actor.h"

#include "NetConnection.generated.h"

#define NETCONNECTION_HAS_SETENCRYPTIONKEY 1

LLM_DECLARE_TAG_API(NetConnection, ENGINE_API);

class FInternetAddr;
class FObjectReplicator;
class StatelessConnectHandlerComponent;
class UActorChannel;
class UChildConnection;
class ULevelStreaming;
struct FEncryptionKeyResponse;
class PacketHandler;
class FRPCDoSDetection;
enum class EEngineNetworkRuntimeFeatures : uint16;

namespace UE::Net
{
	class FNetPing;
	class FNetConnectionFaultRecovery;
	struct FStatelessHandshakeFailureInfo;

} // end namespace UE::Net

typedef TMap<TWeakObjectPtr<AActor>, UActorChannel*, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<AActor>, UActorChannel*>> FActorChannelMap;

namespace NetConnectionHelper
{
	/** Number of bits to use in the packet header for sending the milliseconds on the clock when the packet is sent */
	inline constexpr int32 NumBitsForJitterClockTimeInHeader = 10;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ENGINE_API bool HasDisabledBandwidthThrottling();
#endif
}

extern ENGINE_API TAutoConsoleVariable<int32> CVarNetEnableCongestionControl;

/*-----------------------------------------------------------------------------
	Types.
-----------------------------------------------------------------------------*/
enum { RELIABLE_BUFFER = 512 }; // Power of 2 >= 1.
enum { MAX_PACKETID = FNetPacketNotify::SequenceNumberT::SeqNumberCount };  // Power of 2 >= 1, covering guaranteed loss/misorder time.
enum { MAX_CHSEQUENCE = 1024 }; // Power of 2 >RELIABLE_BUFFER, covering loss/misorder time.
enum { MAX_BUNCH_HEADER_BITS = 256 };
enum { MAX_PACKET_RELIABLE_SEQUENCE_HEADER_BITS = 32 /*PackedHeader*/ + 16 /*HistorySequence*/ + FNetPacketNotify::SequenceHistoryT::MaxSizeInBits };
enum { MAX_PACKET_INFO_HEADER_BITS = 1 /*bHasPacketInfo*/ + NetConnectionHelper::NumBitsForJitterClockTimeInHeader + 1 /*bHasServerFrameTime*/ + 8 /*ServerFrameTime*/};
enum { MAX_PACKET_HEADER_BITS = MAX_PACKET_RELIABLE_SEQUENCE_HEADER_BITS + MAX_PACKET_INFO_HEADER_BITS  };
enum { MAX_PACKET_TRAILER_BITS = 1 };

// 
// State of a connection.
//
enum EConnectionState
{
	USOCK_Invalid   = 0, // Connection is invalid, possibly uninitialized.
	USOCK_Closed    = 1, // Connection permanently closed.
	USOCK_Pending	= 2, // Connection is awaiting connection.
	USOCK_Open      = 3, // Connection is open.
	USOCK_Closing   = 4, // Connection is closing and waiting for all reliable data to be acked. No new data will be sent.
};
ENGINE_API const TCHAR* LexToString(const EConnectionState Value);

namespace UE::Net
{
	/** The source of a net upgrade message */
	enum class ENetUpgradeSource : uint8
	{
		ControlChannel,			// The upgrade message came from the control channel
		StatelessHandshake		// The upgrade message came from the stateless handshake
	};
}


/** If this connection is from a client, this is the current login state of this connection/login attempt */
namespace EClientLoginState
{
	enum Type
	{
		Invalid		= 0,		// This must be a client (which doesn't use this state) or uninitialized.
		LoggingIn	= 1,		// The client is currently logging in.
		Welcomed	= 2,		// Told client to load map and will respond with SendJoin
		ReceivedJoin = 3,		// NMT_Join received and a player controller has been created
		CleanedUp	= 4			// Cleanup has been called at least once, the connection is considered abandoned/terminated/gone
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString( const EClientLoginState::Type EnumVal )
	{
		switch (EnumVal)
		{
			case Invalid:
			{
				return TEXT("Invalid");
			}
			case LoggingIn:
			{
				return TEXT("LoggingIn");
			}
			case Welcomed:
			{
				return TEXT("Welcomed");
			}
			case ReceivedJoin:
			{
				return TEXT("ReceivedJoin");
			}
			case CleanedUp:
			{
				return TEXT("CleanedUp");
			}
		}
		return TEXT("");
	}
};

/** Type of property data resend used by replay checkpoints */
 enum class EResendAllDataState : uint8
 {
	 None,
	 SinceOpen,
	 SinceCheckpoint
 };

// Delegates
#if !UE_BUILD_SHIPPING
/**
 * Delegate for hooking the net connections 'ReceivedRawPacket'
 * 
 * @param Data				The data received
 * @param Count				The number of bytes received
 * @param bBlockReceive		Whether or not to block further processing of the packet (defaults to false)
*/
DECLARE_DELEGATE_ThreeParams(FOnReceivedRawPacket, void* /*Data*/, int32 /*Count*/, bool& /*bBlockReceive*/);

/**
 * Delegate for hooking the net connections 'LowLevelSend' (at the socket level, after PacketHandler parsing)
 *
 * @param Data			The data being sent
 * @param Count			The number of bytes being sent
 * @param bBlockSend	Whether or not to block the send (defaults to false)
*/
DECLARE_DELEGATE_ThreeParams(FOnLowLevelSend, void* /*Data*/, int32 /*Count*/, bool& /*bBlockSend*/);
#endif


#if DO_ENABLE_NET_TEST
/**
 * An artificially lagged packet
 */
struct FDelayedPacket
{
	/** The packet data to send */
	TArray<uint8> Data;

	/** The size of the packet in bits */
	int32 SizeBits;

	/** The traits applied to the packet */
	FOutPacketTraits Traits;

	/** The time at which to send the packet */
	double SendTime = 0.0;

	/** The number of frames we delay sending the packet */
	uint32 DelayFrameCount = 0;

	/** Flag telling the packet was sent and can be cleaned up */
	bool bSent = false;

public:

	inline FDelayedPacket(uint8* InData, int32 InSizeBits, FOutPacketTraits& InTraits)
		: Data()
		, SizeBits(InSizeBits)
		, Traits(InTraits)
	{
		int32 SizeBytes = FMath::DivideAndRoundUp(SizeBits, 8);

		Data.AddUninitialized(SizeBytes);
		FMemory::Memcpy(Data.GetData(), InData, SizeBytes);
	}

	void CountBytes(FArchive& Ar) const
	{
		Data.CountBytes(Ar);
	}
};

struct FDelayedIncomingPacket
{
	TUniquePtr<FBitReader> PacketData;

	/** Time at which the packet should be reinjected into the connection */
	double ReinjectionTime = 0.0;

	/** Number of frames until we can reinject the incoming packet */
	uint32 ReinjectionFrameCount = 0;

	void CountBytes(FArchive& Ar) const
	{
		if (PacketData.IsValid())
		{
			PacketData->CountMemory(Ar);
		}
		Ar.CountBytes(sizeof(ReinjectionTime), sizeof(ReinjectionTime));
	}
};

#endif //#if DO_ENABLE_NET_TEST

struct FChannelCloseInfo
{
	uint32 Id;
	EChannelCloseReason CloseReason;
};
typedef TArray<FChannelCloseInfo, TInlineAllocator<8>> FChannelsToClose;

/** Record of channels with data written into each outgoing packet. */
struct FWrittenChannelsRecord
{
	enum { DefaultInitialSize = 1024 };

	struct FChannelRecordEntry
	{
		uint32 Value : 31;
		uint32 IsSequence : 1;
	};

	typedef TResizableCircularQueue<FChannelRecordEntry> FChannelRecordEntryQueue;

	FChannelRecordEntryQueue ChannelRecord;
	int32 LastPacketId;

public:
	FWrittenChannelsRecord(size_t InitialSize = DefaultInitialSize)
		: ChannelRecord(InitialSize)
		, LastPacketId(-1)
	{
	}
};

UCLASS(customConstructor, Abstract, MinimalAPI, transient, config=Engine)
class UNetConnection : public UPlayer
{
	using FNetResult = UE::Net::FNetResult;
	using FNetCloseResult = UE::Net::FNetCloseResult;

	GENERATED_BODY()

public:
	/** child connections for secondary viewports */
	UPROPERTY(transient)
	TArray<TObjectPtr<class UChildConnection>> Children;

	/** Owning net driver */
	UPROPERTY()
	TObjectPtr<class UNetDriver> Driver;	

	/** The class name for the PackageMap to be loaded */
	UPROPERTY()
	TSubclassOf<UPackageMap> PackageMapClass;

	UPROPERTY()
	/** Package map between local and remote. (negotiates net serialization) */
	TObjectPtr<class UPackageMap> PackageMap;

	/** @todo document */
	UPROPERTY()
	TArray<TObjectPtr<class UChannel>> OpenChannels;
	 
	/** This actor is bNetTemporary, which means it should never be replicated after it's initial packet is complete */
	UPROPERTY()
	TArray<TObjectPtr<class AActor>> SentTemporaries;

	/** The actor that is currently being viewed/controlled by the owning controller */
	UPROPERTY()
	TObjectPtr<class AActor> ViewTarget;

	/** Reference to controlling actor (usually PlayerController) */
	UPROPERTY()
	TObjectPtr<class AActor> OwningActor;

	UPROPERTY()
	int32	MaxPacket;						// Maximum packet size.

private:
	UPROPERTY()
	uint32 InternalAck:1;					// Internally ack all packets, for 100% reliable connections.

	UPROPERTY(transient)
	TMap<int32, TObjectPtr<class UChildConnection>> ChildrenLookup;

	/** 
	 * Cache a pointer to an instance of ANoPawnPlayerController to avoid having to re-spawn it when changing this 
	 * connection from a normal connection with a pawn and player controller to a no pawn connection.
	 *
	 * This is currently only used by APlayerController::PostMigrate() when migrating players between servers.
	 */
	UPROPERTY(transient)
	TObjectPtr<class ANoPawnPlayerController> CachedNoPawnPlayerController;

	int32 NextServerNetPlayerIndex = 1;

public:
	bool IsInternalAck() const { return bInternalAck; }
	void SetInternalAck(bool bValue) 
	{ 
		bInternalAck = bValue; 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InternalAck = bValue;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool IsReplay() const { return bReplay; }
	void SetReplay(bool bValue)
	{
		bReplay = bValue;
	}

	virtual bool IsReplayReady() const { return false; }

	bool IsForceInitialDirty() const { return bForceInitialDirty; }
	void SetForceInitialDirty(bool bValue)
	{
		bForceInitialDirty = bValue;
	}

	/**	Used to allow connections to ignore the bunch size limitation applied before splitting into partial bunch packets, 
	*	or when receiving partial bunches to reassemble.
	* 
	*	!!!WARNING!!!  This is a security risk as the connection will accept much larger bunches before hitting the 
	*	reliable buffer limit, or the array/string serialization limit.
	*/
	bool IsUnlimitedBunchSizeAllowed() const { return bUnlimitedBunchSizeAllowed; }
	void SetUnlimitedBunchSizeAllowed(bool bValue)
	{
		bUnlimitedBunchSizeAllowed = bValue;
	}

	/** Destructor */
	ENGINE_API virtual ~UNetConnection();

private:
	uint32 bInternalAck : 1;				// Internally ack all packets, for 100% reliable connections.
	uint32 bReplay : 1;						// Flag to indicate a replay connection, independent of reliability
	uint32 bForceInitialDirty : 1;			// Force all properties dirty on initial replication
	uint32 bUnlimitedBunchSizeAllowed : 1;	// Ignore the value of net.MaxConstructedPartialBunchSizeBytes

public:
	struct FURL			URL;				// URL of the other side.
	
	/** The remote address of this connection, typically generated from the URL. */
	TSharedPtr<FInternetAddr>	RemoteAddr;

	// Track each type of bit used per-packet for bandwidth profiling

	/** Number of bits used for the packet id in the current packet. */
	int NumPacketIdBits;

	/** Number of bits used for bunches in the current packet. */
	int NumBunchBits;

	/** Number of bits used for acks in the current packet. */
	int NumAckBits;

	/** Number of bits used for padding in the current packet. */
	int NumPaddingBits;

	/** The maximum number of bits all packet handlers will reserve */
	int32 MaxPacketHandlerBits;

	/** Sets all of the bit-tracking variables to zero. */
	void ResetPacketBitCounts();

	/** What type of data is being written */
	enum class EWriteBitsDataType
	{
		Unknown,
		Bunch,
		Ack
	};

	/** Returns the actor starvation map */
	TMap<FString, TArray<float>>& GetActorsStarvedByClassTimeMap() { return ActorsStarvedByClassTimeMap; }
	
	/** Clears the actor starvation map */
	void ResetActorsStarvedByClassTimeMap() { ActorsStarvedByClassTimeMap.Empty(); }

	/** Get the client handshake id for this connection. */
	ENGINE_API uint32 GetClientHandshakeId() const;

	/** Set the client handshake id for this connection. */
	ENGINE_API void SetClientHandshakeId(uint32 InClientHandshakeId);

private:
	// Connection information.
	EConnectionState	State;					// State this connection is in.

	/** An optional unique identifier set by the client when establishing a connection. */
	uint32 ClientHandshakeId = 0;

public:
	ENGINE_API const EConnectionState GetConnectionState() const;
	ENGINE_API void SetConnectionState(EConnectionState ConnectionState);
	
	/** Returns true if this connection's state is USOCK_Closing or USOCK_Closed. No new data should be sent in this case. */
	ENGINE_API bool IsClosingOrClosed() const;

	uint32 bPendingDestroy:1;    // when true, playercontroller or beaconclient is being destroyed


	/** PacketHandler, for managing layered handler components, which modify packets as they are sent/received */
	TUniquePtr<PacketHandler> Handler;

	/** Reference to the PacketHandler component, for managing stateless connection handshakes */
	TWeakPtr<StatelessConnectHandlerComponent> StatelessConnectComponent;


	/** Whether this channel needs to byte swap all data or not */
	bool			bNeedsByteSwapping;
	/** Net id of remote player on this connection. Only valid on client connections (server side).*/
	UPROPERTY()
	FUniqueNetIdRepl PlayerId;

	// Negotiated parameters.
	int32			PacketOverhead;			// Bytes overhead per packet sent.
	FString			Challenge;				// Server-generated challenge.
	FString			ClientResponse;			// Client-generated response.
	FString			RequestURL;				// URL requested by client

	UE_DEPRECATED(5.5, "Variable is deprecated and unassigned")
	int32			ResponseId;				// Id assigned by the server for linking responses to connections upon authentication

	// Login state tracking
	EClientLoginState::Type	ClientLoginState;
	uint8					ExpectedClientLoginMsgType;	// Used to determine what the next expected control channel msg type should be from a connecting client

	// Internal.
	UPROPERTY()
	double			LastReceiveTime;		// Last time a packet was received, for timeout checking.
	double			LastReceiveRealtime;	// Last time a packet was received, using real time seconds (FPlatformTime::Seconds)
	double			LastGoodPacketRealtime;	// Last real time a packet was considered valid
	double			LastSendTime;			// Last time a packet was sent, for keepalives.
	double			LastTickTime;			// Last time of polling.

	/** 
	 * The amount of bits allowed to be sent in a given tick to respect the bandwidth per second net speed.
	 * Starts negative and increases with every serialized bits. When >0 it means we sent more bits then allowed.
	 * Set at the end of every Tick based on the data sent that tick.
	 */
	int32			QueuedBits;

	/** Count of ticks. */
	int32			TickCount;

	/** The last frame where we gathered and processed actors for this connection */
	uint32			LastProcessedFrame;

	double GetLastRecvAckTime() const { return LastRecvAckTimestamp; }
	double GetConnectTime() const { return ConnectTimestamp; }

private:
	/** The last time an ack was received */
	double			LastRecvAckTimestamp;

	/** Time when connection request was first initiated */
	double			ConnectTimestamp;

	FPacketTimestamp	LastOSReceiveTime;		// Last time a packet was received at the OS/NIC layer
	bool				bIsOSReceiveTimeLocal;	// Whether LastOSReceiveTime uses the same clock as the game, or needs translating

	/** Did we write the dummy PacketInfo in the current SendBuffer */
	bool bSendBufferHasDummyPacketInfo = false;

	/** Stores the bit number where we wrote the dummy packet info in the packet header */
	FBitWriterMark HeaderMarkForPacketInfo;

	/** The difference between the send and receive clock time of the last received packet */
	int32 PreviousJitterTimeDelta;

	/** Timestamp of the last packet sent*/
	double PreviousPacketSendTimeInS;

public:
	// Merge info.
	FBitWriterMark  LastStart;				// Most recently sent bunch start.
	FBitWriterMark  LastEnd;				// Most recently sent bunch end.
	bool			AllowMerge;				// Whether to allow merging.
	bool			TimeSensitive;			// Whether contents are time-sensitive.
	FOutBunch*		LastOutBunch;			// Most recent outgoing bunch.
	FOutBunch		LastOut;
	/** The singleton buffer for sending bunch header information */
	FBitWriter		SendBunchHeader;

	// Stat display.
	/** Time of last stat update */
	double			StatUpdateTime;
	/** Interval between gathering stats */
	float			StatPeriod;

	/** Average lag seen during the last StatPeriod */
	float 			AvgLag;

	/** Total accumulated lag values during the current StatPeriod */
	double			LagAcc;
	/** Nb of stats accumulated in LagAcc */
	int32			LagCount;
	
	/** Monitors frame time */
	double			LastTime, FrameTime;

	/** Total frames times accumulator */
	double			CumulativeTime;
	
	/** The average frame delta time over the last 1 second period.*/
	double			AverageFrameTime;

	/** The raw ping value of the last packet the connection processed */
	double			RawPingInSeconds = 0.0;

	/** Current jitter for this connection in milliseconds */
	float			GetAverageJitterInMS() const { return AverageJitterInMS; }

	/** Nb of stats accumulated in CumulativeTime */
	int32			CountedFrames;

	/** bytes sent/received on this connection (accumulated during a StatPeriod) */
	int32 InBytes, OutBytes;
	/** total bytes sent/received on this connection */
	int32 InTotalBytes, OutTotalBytes;
	/** packets sent/received on this connection (accumulated during a StatPeriod) */
	int32 InPackets, OutPackets;
	/** Packets received in the current tick */
	int32 InPacketsThisFrame = 0;
	/** Packets sent in the current tick */
	int32 OutPacketsThisFrame = 0;
	/** total packets sent/received on this connection */
	int32 InTotalPackets, OutTotalPackets;
	/** bytes sent/received on this connection (per second) - these are from previous StatPeriod interval */
	int32 InBytesPerSecond, OutBytesPerSecond;
	/** packets sent/received on this connection (per second) - these are from previous StatPeriod interval */
	int32 InPacketsPerSecond, OutPacketsPerSecond;
	/** packets lost on this connection (accumulated during a StatPeriod) */
	int32 InPacketsLost, OutPacketsLost;
	/** total packets lost on this connection */
	int32 InTotalPacketsLost, OutTotalPacketsLost;
	/** total acks sent on this connection */
	int32 OutTotalAcks;
	/** Delayed RPCs and the total average frame delay */
	int32 TotalDelayedRPCs = 0;
	int32 TotalDelayedRPCsFrameCount = 0;

private:
	/** total packets received on this connection, including PacketHandler */
	int32 InTotalHandlerPackets;

	/** Driver ElapsedTime at which a graceful close will time out and terminate. */
	double GracefulCloseTimeoutDeadline = 0.0;

public:
	int32 GetInTotalHandlerPackets() const
	{
		return InTotalHandlerPackets;
	}

	/** Percentage of packets lost during the last StatPeriod */
	using FNetConnectionPacketLoss = TPacketLossData<3>;
	const FNetConnectionPacketLoss& GetInLossPercentage() const  { return InPacketsLossPercentage; }
	const FNetConnectionPacketLoss& GetOutLossPercentage() const { return OutPacketsLossPercentage;  }


private:

	FNetConnectionPacketLoss InPacketsLossPercentage;
	FNetConnectionPacketLoss OutPacketsLossPercentage;

	/** Counts the number of stat samples taken */
	int32 StatPeriodCount;

	/**
	* Jitter represents the average time divergence of all sent packets.
	* Ex:
	* If the time between the sending and the reception of packets is always 100ms; the jitter will be 0.
	* If the time difference is either 150ms or 100ms, the jitter will tend towards 50ms.
	*/
	float AverageJitterInMS;

public:

	/** Net Analytics */

	/** The locally cached/updated analytics variables, for the NetConnection - aggregated upon connection Close */
	FNetConnAnalyticsVars							AnalyticsVars;

	/** The net analytics data holder for the NetConnection analytics, which is where analytics variables are aggregated upon Close */
	TNetAnalyticsDataPtr<FNetConnAnalyticsData>		NetAnalyticsData;

	// Packet.
	FBitWriter		SendBuffer;						// Queued up bits waiting to send
	double			OutLagTime[256];				// The sent time of a packetId to calculate RTT from
	int32			OutLagPacketId[256];			// Keep track of the packedId we stored the sent time for
	uint8			OutBytesPerSecondHistory[256];	// For saturation measuring.
	int32			InPacketId;						// Full incoming packet index.
	int32			OutPacketId;					// Most recently sent packet.
	int32 			OutAckPacketId;					// The id of the last ACK included in the current packet

private:
	int32			OutFirstAckPacketId = -1;		// The id of the first ACK included in the current packet

public:

	bool			bLastHasServerFrameTime;

	// Channel table.

	UPROPERTY(config)
	int32 DefaultMaxChannelSize;

	TArray<TObjectPtr<UChannel>>	Channels;
	TArray<int32>		OutReliable;
	TArray<int32>		InReliable;
	TArray<int32>		PendingOutRec;	// Outgoing reliable unacked data from previous (now destroyed) channel in this slot.  This contains the first chsequence not acked
	int32				InitOutReliable;
	int32				InitInReliable;

	uint32 GetNetworkCustomVersion(const FGuid& VersionGuid) const;
	void SetNetworkCustomVersions(const FCustomVersionContainer& CustomVersions);

private:
	FCustomVersionContainer NetworkCustomVersions;

public:
#if UE_SUPPORT_PARALLEL_IRIS
	FTransactionallySafeCriticalSection& GetSharedNetTickMutex();
#endif // UE_SUPPORT_PARALLEL_IRIS

public:

	// Log tracking
	double			LogCallLastTime;
	int32			LogCallCount;
	int32			LogSustainedCount;

	/** Returns a connection identifier which contains both parent connection and child connection information. */
	UE::Net::FConnectionHandle GetConnectionHandle() const
	{
		return ConnectionHandle;
	}

	UE_DEPRECATED(5.6, "Use GetConnectionHandle() instead")
	uint32 GetConnectionId() const
	{ 
		return ConnectionHandle.IsParentConnection() ? ConnectionHandle.GetParentConnectionId() : UE::Net::FConnectionHandle().GetParentConnectionId();
	}

	UE_DEPRECATED(5.6, "External code should not be setting connection IDs. For valid use cases SetConnectionHandle() should be used instead.")
	void SetConnectionId(uint32 InConnectionId)
	{
		// Using this deprecated method will result in previous behavior which didn't support child connections in a meaningful way. We assume this is a parent connection.
		ConnectionHandle = UE::Net::FConnectionHandle(InConnectionId);
	}
	
	/** If this is a child connection it will return the topmost parent coonnection ID, otherwise it will return its own ID. */
	UE_DEPRECATED(5.6, "Use GetConnectionHandle instead")
	ENGINE_API uint32 GetParentConnectionId() const;

	FNetTraceCollector* GetInTraceCollector() const;
	FNetTraceCollector* GetOutTraceCollector() const;

	/** Returns the view target for this connection. Controlled by the player controller when one is assigned. The view target is the owning actor when no PC's are assigned or the PC's view target is invalid */
	AActor* GetConnectionViewTarget() const;

	// ----------------------------------------------
	// Actor Channel Accessors
	// ----------------------------------------------

	void RemoveActorChannel(AActor* Actor)
	{
		ActorChannels.Remove(Actor);
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyActorChannelRemoved(Actor);
		}
	}

	void AddActorChannel(AActor* Actor, UActorChannel* Channel)
	{
		ActorChannels.Add(Actor, Channel);
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyActorChannelAdded(Actor, Channel);
		}
	}

	UActorChannel* FindActorChannelRef(const TWeakObjectPtr<AActor>& Actor)
	{
		return ActorChannels.FindRef(Actor);
	}

	UActorChannel** FindActorChannel(const TWeakObjectPtr<AActor>& Actor)
	{
		return ActorChannels.Find(Actor);
	}

	bool ContainsActorChannel(const TWeakObjectPtr<AActor>& Actor)
	{
		return ActorChannels.Contains(Actor);
	}

	int32 ActorChannelsNum() const
	{
		return ActorChannels.Num();
	}

	FActorChannelMap::TConstIterator ActorChannelConstIterator() const
	{
		return ActorChannels.CreateConstIterator();
	}

	const FActorChannelMap& ActorChannelMap() const
	{
		return ActorChannels;
	}

	UReplicationConnectionDriver* GetReplicationConnectionDriver()
	{
		return ReplicationConnectionDriver;
	}

	void SetReplicationConnectionDriver(UReplicationConnectionDriver* NewReplicationConnectionDriver)
	{
		ReplicationConnectionDriver = NewReplicationConnectionDriver;
	}

	void TearDownReplicationConnectionDriver()
	{
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->TearDown();
			ReplicationConnectionDriver = nullptr;
		}
	}

private:
	/** @todo document */
	FActorChannelMap ActorChannels;

	UReplicationConnectionDriver* ReplicationConnectionDriver;

	/** Engine package version for compatibility */
	FPackageFileVersion PackageVersionUE;

	/** Licensee package version for compatibility */
	int32 PackageVersionLicenseeUE;

	/** Engine version information for compatibility */
	FEngineVersion EngineVersion;

public:
	/** Sets the UE package version for compatibility purposes */
	void SetPackageVersionUE(FPackageFileVersion InPackageVersionUE) { PackageVersionUE = InPackageVersionUE; }
	
	/** Sets the licensee package version for compatibility purposes */
	void SetPackageVersionLicenseeUE(int32 InPackageVersionLicenseeUE) { PackageVersionLicenseeUE = InPackageVersionLicenseeUE; }

	/** Sets engine version information for compatibility purposes */
	void SetEngineVersion(const FEngineVersion& InEngineVersion) { EngineVersion = InEngineVersion; }

	/** Set version information on an archive that can be used for compatibility checks */
	void SetNetVersionsOnArchive(FArchive& Ar) const;

	void AddDestructionInfo(FActorDestructionInfo* DestructionInfo)
	{
		if (Driver && Driver->GetReplicationSystem())
		{
			return;
		}
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyAddDestructionInfo(DestructionInfo);
		}
		else
		{
			DestroyedStartupOrDormantActorGUIDs.Add(DestructionInfo->NetGUID);
		}
	}

	void RemoveDestructionInfo(FActorDestructionInfo* DestructionInfo)
	{
		if (Driver && Driver->GetReplicationSystem())
		{
			return;
		}
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyRemoveDestructionInfo(DestructionInfo);
		}
		else
		{
			DestroyedStartupOrDormantActorGUIDs.Remove(DestructionInfo->NetGUID);
		}
	}
	
	void ResetDestructionInfos()
	{
		if (Driver && Driver->GetReplicationSystem())
		{
			return;
		}
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyResetDestructionInfo();
		}
		else
		{
			DestroyedStartupOrDormantActorGUIDs.Reset(); 
		}
	}

	TSet<FNetworkGUID>& GetDestroyedStartupOrDormantActorGUIDs() { return DestroyedStartupOrDormantActorGUIDs; }

private:

	/** The server adds GUIDs to this set for each destroyed actor that does not have a channel
	 *  but that the client still knows about: startup, dormant, or recently dormant set.
	 *  This set is also populated from the UNetDriver for clients who join-in-progress, so that they can destroy any
	 *  startup actors that the server has already destroyed.
	 */
	TSet<FNetworkGUID>	DestroyedStartupOrDormantActorGUIDs;

	/** On the server, the package names of streaming levels that the client has told us it is making visible */
	TSet<FName> ClientMakingVisibleLevelNames;

public:

	/** This holds a list of actor channels that want to fully shutdown, but need to continue processing bunches before doing so */
	TMap<FNetworkGUID, TArray<TObjectPtr<class UActorChannel>>> KeepProcessingActorChannelBunchesMap;

private:

	UE::Net::Private::FDormantReplicatorHolder DormantReplicatorSet;

public:

	FName GetClientWorldPackageName() const { return ClientWorldPackageName; }

	ENGINE_API void SetClientWorldPackageName(FName NewClientWorldPackageName);

	/** 
	 * on the server, the package names of streaming levels that the client has told us it has made visible
	 * the server will only replicate references to Actors in visible levels so that it's impossible to send references to
	 * Actors the client has not initialized
	 */
	TSet<FName> ClientVisibleLevelNames;

	/** Called by PlayerController to tell connection about client level visibility change */
	ENGINE_API void UpdateLevelVisibility(const struct FUpdateLevelVisibilityLevelInfo& LevelVisibility);

	/** Called by PlayerController to tell connection about client container visibility change */
	ENGINE_API void UpdateContainerVisibility(UObject* Container, bool bVisible);

	const TSet<FName>& GetClientMakingVisibleLevelNames() const { return ClientMakingVisibleLevelNames; }
	
#if DO_ENABLE_NET_TEST

	/** Packet settings for testing lag, net errors, etc */
	FPacketSimulationSettings PacketSimulationSettings;

	/** Copies the settings from the net driver to our local copy */
	void UpdatePacketSimulationSettings();

private:

	/** delayed outgoing packet array */
	TArray<FDelayedPacket> Delayed;

	/** delayed incoming packet array */
	TArray<FDelayedIncomingPacket> DelayedIncomingPackets;

	/** Allow emulated packets to be sent out of order. This causes packet loss on the receiving end */
	bool bSendDelayedPacketsOutofOrder = false;

	/** set to true when already delayed packets are reinjected */
	bool bIsReinjectingDelayedPackets = false;

	/** Time when the next flush of outgoing delayed packets will occur. */
	double NextBufferBloatFlushOut = 0.0;
	/** Time when the next flush of incoming delayed packets will occur. */
	double NextBufferBloatFlushIn = 0.0;
	
	/** Time at which we restart the buffer bloat emulation. */
	double NextBufferBloatActiveTimeOut = 0.0;
	double NextBufferBloatActiveTimeIn = 0.0;

	/** Process incoming packets that have been delayed for long enough */
	void ReinjectDelayedPackets();

	/** Update local packets we queued to add latency and send those that are due */
	void UpdateDelayedPackets(const double CurrentRealtimeSeconds);

#endif //#if DO_ENABLE_NET_TEST

public:

	/** 
	 *	This functionality is used during replay checkpoints for example, so we can re-use the existing connection and channels to record
	 *	a version of each actor and capture all properties that have changed since the actor has been alive...
	 *	This will also act as if it needs to re-open all the channels, etc.
	 *   NOTE - This doesn't force all exports to happen again though, it will only export new stuff, so keep that in mind.
	 */
	EResendAllDataState ResendAllDataState;

#if !UE_BUILD_SHIPPING
	/** Delegate for hooking ReceivedRawPacket */
	FOnReceivedRawPacket	ReceivedRawPacketDel;

	/** Delegate for hooking LowLevelSend */
	FOnLowLevelSend			LowLevelSendDel;
#endif

	/** Set a callback that returns a local player to be used by HandleClientPlayer(). */
	ENGINE_API void SetLocalPlayerFunction(TFunction<class ULocalPlayer*(UNetConnection*, APlayerController*)> GetPlayerFunction);
	ENGINE_API TFunction<class ULocalPlayer*(UNetConnection*, APlayerController*)> GetLocalPlayerFunction();

	/**
	* Delegate that is called when a connection has been successfully completed on a client and
	* calls HandleClientPlayer(). 
	* 
	* In the case of UChildConnection the delegate on the parent connection will be called.
	*
	* @param ChildGameServerConnection 		A child UNetConnection (will be equal to ParentGameServerConnection on a root connection).
	* @param PlayerController				The controller replicated from the server.
	* @param ParentGameServerConnection		A parent UNetConnection.
	*/
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConnectionOpened, 
										   UNetConnection* 		/* ChildGameServerConnection */, 
										   APlayerController* 	/* PlayerController */, 
										   UNetConnection* 		/* ParentGameServerConnection */);
	FOnConnectionOpened& GetOnConnectionOpened() { return OnConnectionOpened; };

private:

	FOnConnectionOpened OnConnectionOpened;
	TFunction<class ULocalPlayer*(UNetConnection*, APlayerController*)> PlayerConnectionFunction;

public:

	/**
	 * Called to determine if a voice packet should be replicated to this
	 * connection or any of its child connections
	 *
	 * @param Sender - the sender of the voice packet
	 *
	 * @return true if it should be sent on this connection, false otherwise
	 */
	ENGINE_API bool ShouldReplicateVoicePacketFrom(const FUniqueNetId& Sender);
	
	/**
	 * @hack: set to net connection currently inside CleanUp(), for HasClientLoadedCurrentWorld() to be able to find it during PlayerController
	 * destruction, since we clear its Player before destroying it. (and that's not easily reversed)
	 */
	static class UNetConnection* GNetConnectionBeingCleanedUp;

	// Constructors and destructors.
	ENGINE_API UNetConnection(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API UNetConnection(FVTableHelper& Helper);

	//~ Begin UObject Interface.

	ENGINE_API virtual void Serialize( FArchive& Ar ) override;

	ENGINE_API virtual void FinishDestroy() override;

	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Get the world the connection belongs to
	 *
	 * @return  Returns the world of the net driver, or the owning actor on this connection
	 */
	ENGINE_API virtual UWorld* GetWorld() const override;

	//~ End UObject Interface.


	//~ Begin FExec Interface.
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog ) override;
#endif
	//~ End FExec Interface.

	/** read input */
	void ReadInput( float DeltaSeconds );

	/** 
	 * get the representation of a secondary splitscreen connection that reroutes calls to the parent connection
	 * @return NULL for this connection.
	 */
	virtual UChildConnection* GetUChildConnection()
	{
		return NULL;
	}

	/** @return the remote machine address */
	virtual FString LowLevelGetRemoteAddress(bool bAppendPort=false) PURE_VIRTUAL(UNetConnection::LowLevelGetRemoteAddress,return TEXT(""););

	/** @return the description of connection */
	virtual FString LowLevelDescribe() PURE_VIRTUAL(UNetConnection::LowLevelDescribe,return TEXT(""););

	/** Describe the connection. */
	ENGINE_API virtual FString Describe();

	/**
	 * Sends a byte stream to the remote endpoint using the underlying socket
	 *
	 * @param Data			The byte stream to send
	 * @param CountBits		The length of the stream to send, in bits (to support bit-level additions to packets, from PacketHandler's)
	 * @param Traits		Special traits for the packet, passed down from the NetConnection through the PacketHandler
	 */
	// @todo: Traits should be passed within bit readers/writers, eventually
	virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
		PURE_VIRTUAL(UNetConnection::LowLevelSend,);

	/** Validates the FBitWriter to make sure it's not in an error state */
	ENGINE_API virtual void ValidateSendBuffer();

	/** Resets the FBitWriter to its default state */
	ENGINE_API virtual void InitSendBuffer();

	/** Make sure this connection is in a reasonable state. */
	ENGINE_API virtual void AssertValid();

	/**
	 * flushes any pending data, bundling it into a packet and sending it via LowLevelSend()
	 * also handles network simulation settings (simulated lag, packet loss, etc) unless bIgnoreSimulation is true
	 */
	ENGINE_API virtual void FlushNet(bool bIgnoreSimulation = false);

	/** Poll the connection. If it is timed out, close it. */
	ENGINE_API virtual void Tick(float DeltaSeconds);

	/** Return whether this connection is ready for sending. */
	UE_DEPRECATED(5.6, "Replaced with a version that doesn't have a saturate option. Call Saturate() directly if passing true before." )
	ENGINE_API virtual int32 IsNetReady( bool bSaturate);

	/** Returns true if the connection is ready to send and has not exceeded the bandwidth limit yet. */
	ENGINE_API virtual bool IsNetReady() const;

	/**
	 * Return whether this connection can send packets without exhausting the packet sequence history window, as it could cause packets to be NAKed even when they've been received by the remote peer. 
	 * @param SafetyMargin A small number representing how many packets you would like to keep as a safety margin for heart beats or other important packets.
	 */
	ENGINE_API bool IsPacketSequenceWindowFull(uint32 SafetyMargin=0U);

	/** 
	 * Handle the player controller client
	 *
	 * @param PC player controller for this player
	 * @param NetConnection the connection the player is communicating on
	 */
	ENGINE_API virtual void HandleClientPlayer( class APlayerController* PC, class UNetConnection* NetConnection );

	/** @return the port of the connection as an integer */
	virtual int32 GetAddrPort(void)
	{
		if (RemoteAddr.IsValid())
		{
			return RemoteAddr->GetPort();
		}
		return 0;
	}

	/** Return the child connection that uses the provided NetPlayerIndex. */
	ENGINE_API UChildConnection* GetChildConnectionWithNetPlayerIndex(const int32 NetPlayerIndex);

	/** Return true if the provided child connection belongs to the parent and the NetPlayerIndex output to OutNetPlayerIndex. */
	ENGINE_API bool GetNetPlayerIndex(UChildConnection* ChildConnection, int32* OutNetPlayerIndex);

	/** 
	 * Create and return a child connection and assign it a new NetPlayerIndex.
	 *
	 * The new NetPlayerIndex is output in the argument OutNetPlayerIndex.
	 */
	ENGINE_API UChildConnection* CreateChild(int32* OutNetPlayerIndex);

	/**
	 * Create and return a child connection and assign it a NetPlayerIndex provided as an argument.
	 */
	ENGINE_API UChildConnection* CreateChildWithNetPlayerIndex(int32 NetPlayerIndex);

	/** 
	 * Remove a child connection from internal data structures.
	 *
	 * This function won't actually close the child connection or perform any other clean-up. To close
	 * the child connection call UChildConnection::CloseAndRemoveChild().
	 */
	ENGINE_API void RemoveChild(UChildConnection* ChildConnection);

	/**
	 * Return the platform specific FInternetAddr type, containing this connections address.
	 * If nullptr is returned, connection is not added to MappedClientConnections, and can't receive net packets which depend on this.
	 *
	 * @return	The platform specific FInternetAddr containing this connections address
	 */
	virtual TSharedPtr<const FInternetAddr> GetRemoteAddr() { return RemoteAddr; }

	/**
	 * Closes the connection (including sending a close notify across the network)
	 * NOTE: To be deprecated in the near future.
	 */
	void Close()
	{
		Close(FNetCloseResult());
	}

	/**
	 * Closes the connection (including sending a close notify across the network)
	 *
	 * @param CloseReason	Specifies the reason for the Close
	 */
	void Close(FNetCloseResult&& CloseReason)
	{
		Close(static_cast<FNetResult&&>(MoveTemp(CloseReason)));
	}

	/**
	 * Closes the connection (including sending a close notify across the network)
	 *
	 * @param CloseReason	Specifies the reason for the Close
	 */
	ENGINE_API void Close(FNetResult&& CloseReason);


	/**
	 * Starts a graceful close. If net.EnableGracefulClose is true, waits longer for reliable RPCs to be acknowledged before fully cleaning up the connection.
	 * If net.EnableGracefulClose is false, immediately sends the control channel close bunch. Reliable RPCs may be lost.
	 *
	 * @param CloseReason	Specifies the reason for the Close
	 */
	ENGINE_API void GracefulClose(FNetResult&& CloseReason);

	/**
	 * Starts a graceful close. If net.EnableGracefulClose is true, waits longer for reliable RPCs to be acknowledged before fully cleaning up the connection.
	 * If net.EnableGracefulClose is false, immediately sends the control channel close bunch. Reliable RPCs may be lost.
	 *
	 * @param CloseReason	Specifies the reason for the Close
	 */
	virtual void GracefulClose(FNetCloseResult&& CloseReason)
	{
		GracefulClose(static_cast<FNetResult&&>(MoveTemp(CloseReason)));
	}

	/** closes the control channel, cleans up structures, and prepares for deletion */
	ENGINE_API virtual void CleanUp();

	/**
	 * Initialize common settings for this connection instance
	 *
	 * @param InDriver the net driver associated with this connection
	 * @param InSocket the socket associated with this connection
	 * @param InURL the URL to init with
	 * @param InState the connection state to start with for this connection
	 * @param InMaxPacket the max packet size that will be used for sending
	 * @param InPacketOverhead the packet overhead for this connection type
	 */
	ENGINE_API virtual void InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0);

	/**
	 * Initialize this connection instance *from* a remote source
	 *
	 * @param InDriver the net driver associated with this connection
	 * @param InSocket the socket associated with this connection
	 * @param InURL the URL to init with
	 * @param InRemoteAddr the remote address for this connection
	 * @param InState the connection state to start with for this connection
	 * @param InMaxPacket the max packet size that will be used for sending
	 * @param InPacketOverhead the packet overhead for this connection type
	 */
	virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) PURE_VIRTUAL(UNetConnection::InitRemoteConnection, );
	
	/**
	 * Initialize this connection instance *to* a remote source
	 *
	 * @param InDriver the net driver associated with this connection
	 * @param InSocket the socket associated with this connection
	 * @param InURL the URL to init with
	 * @param InRemoteAddr the remote address for this connection
	 * @param InState the connection state to start with for this connection
	 * @param InMaxPacket the max packet size that will be used for sending
	 * @param InPacketOverhead the packet overhead for this connection type
	 */
	virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) PURE_VIRTUAL(UNetConnection::InitLocalConnection, );
	
	/**
	 * Initializes an "addressless" connection with the passed in settings
	 *
	 * @param InDriver the net driver associated with this connection
	 * @param InState the connection state to start with for this connection
	 * @param InURL the URL to init with
	 * @param InConnectionSpeed Optional connection speed override
	 */
	ENGINE_API virtual void InitConnection(UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed=0, int32 InMaxPacket=0);

	/**
	 * Initializes the PacketHandler
	 */
	ENGINE_API virtual void InitHandler();

	/**
	 * Initializes the sequence numbers for the connection, usually from shared randomized data
	 *
	 * @param IncomingSequence	The initial sequence number for incoming packets
	 * @param OutgoingSequence	The initial sequence number for outgoing packets
	 */
	ENGINE_API virtual void InitSequence(int32 IncomingSequence, int32 OutgoingSequence);

	/**
	 * Notification that the NetDriver analytics provider has been updated
	 * NOTE: Can also mean disabled, e.g. during hotfix
	 */
	ENGINE_API virtual void NotifyAnalyticsProvider();

	/**
	 * Sets the encryption data and enables encryption.
	 */
	ENGINE_API void EnableEncryption(const FEncryptionData& EncryptionData);

	/**
	 * Sets the encryption data, enables encryption, and sends the encryption ack to the client.
	 */
	ENGINE_API void EnableEncryptionServer(const FEncryptionData& EncryptionData);

	/**
	 * Sets the data for the underlying encryption packet handler component, but doesn't modify encryption enabled state.
	 */
	ENGINE_API void SetEncryptionData(const FEncryptionData& EncryptionData);

	/**
	 * Sends an NMT_EncryptionAck message
	 */
	ENGINE_API void SendClientEncryptionAck();

	/**
	 * Enables encryption for the underlying encryption packet handler component.
	 */
	ENGINE_API void EnableEncryption();

	/**
	 * Returns true if encryption is enabled for this connection.
	 */
	ENGINE_API virtual bool IsEncryptionEnabled() const;

	/** 
	* Gets a unique ID for the connection, this ID depends on the underlying connection
	* For IP connections this is an IP Address and port, for steam this is a SteamID
	*/
	virtual FString RemoteAddressToString()
	{
		if (RemoteAddr.IsValid())
		{
			return RemoteAddr->ToString(true);
		}
		return TEXT("Invalid");
	}
	
	
	/** Called by UActorChannel. Handles creating a new replicator for an actor */
	ENGINE_API virtual TSharedPtr<FObjectReplicator> CreateReplicatorForNewActorChannel(UObject* Object);

	// Functions.

	/** 
	 * Appends the passed in data to the SendBuffer to be sent when FlushNet is called
	 * @param Bits Data as bits to be appended to the send buffer
	 * @param SizeInBits Number of bits to append
	 * @param ExtraBits (optional) Second set of bits to be appended to the send buffer that need to send with the first set of bits
	 * @param ExtraSizeInBits (optional) Number of secondary bits to append
	 * @param TypeOfBits (optional) The type of data being written, for profiling and bandwidth tracking purposes
	 */
	int32 WriteBitsToSendBuffer( 
		const uint8 *	Bits, 
		const int32		SizeInBits, 
		const uint8 *	ExtraBits = NULL, 
		const int32		ExtraSizeInBits = 0,
		EWriteBitsDataType DataType =  EWriteBitsDataType::Unknown);

	/** Returns number of bits left in current packet that can be used without causing a flush.  */
	int64 GetFreeSendBufferBits();

	/** Pops the LastStart bits off of the send buffer, used for merging bunches */
	void PopLastStart();

	/**
	 * returns whether the client has initialized the given level
	 * @return true if the client has initialized the given level, false otherwise
	 */
	ENGINE_API virtual bool ClientHasInitializedLevel(const ULevel* TestLevel) const;

	/** 
	 * returns whether the client has initialized the level required for the given object
	 * @return true if the client has initialized the level the object is in or the object is not in a level, false otherwise
	 */
	UE_DEPRECATED(5.4, "ClientHasInitializedLevelFor is deprecated. Use ClientHasInitializedLevel and pass the actor's level (Actor->GetLevel()) instead.")
	ENGINE_API virtual bool ClientHasInitializedLevelFor(const AActor* TestActor) const;

	/**
	 * Allows the connection to process the raw data that was received
	 *
	 * @param Data the data to process
	 * @param Count the size of the data buffer to process
	 */
	ENGINE_API virtual void ReceivedRawPacket(void* Data,int32 Count);

	/** Send a raw bunch */
	ENGINE_API int32 SendRawBunch(FOutBunch& Bunch, bool InAllowMerge, const FNetTraceCollector* BunchCollector);
	inline int32 SendRawBunch( FOutBunch& Bunch, bool InAllowMerge ) { return SendRawBunch(Bunch, InAllowMerge, nullptr); }

	/** The maximum number of bits allowed within a single bunch. */
	inline int32 GetMaxSingleBunchSizeBits() const
	{
		return (MaxPacket * 8) - MAX_BUNCH_HEADER_BITS - MAX_PACKET_TRAILER_BITS - MAX_PACKET_HEADER_BITS - MaxPacketHandlerBits;
	}

	/** @return The driver object */
	UNetDriver* GetDriver() {return Driver;}
	const UNetDriver* GetDriver() const { return Driver; }

	/** Create a channel. */
	ENGINE_API UChannel* CreateChannelByName( const FName& ChName, EChannelCreateFlags CreateFlags, int32 ChannelIndex=INDEX_NONE );

	/** 
	* Handle a packet we just received. 
	* bIsReinjectedPacket is true if a packet is reprocessed after getting cached 
	* bDispatchPacket if true the packet will be processed (passed to DispatchPacket)
	*/
	ENGINE_API virtual void ReceivedPacket( FBitReader& Reader, bool bIsReinjectedPacket=false, bool bDispatchPacket=true );
	
	/**
	* Disassemble and dispatch packet.
	* PacketId is an id of the packet to dispatch
	* bOutSkipAck Flag that marks packet as dropped
	*/
	ENGINE_API virtual void DispatchPacket( FBitReader& Reader, int32 PacketId, bool& bOutSkipAck, bool& bOutHasBunchErrors );

	/** Packet was negatively acknowledged. */
	void ReceivedNak( int32 NakPacketId );

	/** Clear all Game specific state. Called during seamless travel */
	void ResetGameWorldState();

	/** Make sure this connection is in a reasonable state. */
	void SlowAssertValid()
	{
#if DO_GUARD_SLOW
		AssertValid();
#endif
	}

	/**
	 * @return Finds the voice channel for this connection or NULL if none
	 */
	ENGINE_API class UVoiceChannel* GetVoiceChannel();

	ENGINE_API virtual void FlushDormancy(class AActor* Actor);

	/** Forces properties on this actor to do a compare for one frame (rather than share shadow state) */
	ENGINE_API void ForcePropertyCompare( AActor* Actor );

	/**
	* Validate an objects dormancy state and prepare the object for replication again
	* 
	* @param DormantActor The dormant actor that owns the replicated object that needs to be flushed
	* @param ReplicatedObject The replicated object that is flushed.
	*/
	void FlushDormancyForObject(AActor* DormantActor, UObject* ReplicatedObject);

	/** 
	 * Wrapper for setting the current client login state, so we can trap for debugging, and verbosity purposes. 
	 * Only valid on the server
	 */
	ENGINE_API virtual void SetClientLoginState( const EClientLoginState::Type NewState );

	/** 
	 * Wrapper for setting the current expected client login msg type. 
	 * Only valid on the server
	 */
	ENGINE_API void SetExpectedClientLoginMsgType( const uint8 NewType );

	/**
	 * This function validates that ClientMsgType is the next expected msg type. 
	 * Only valid on the server
	 */
	ENGINE_API bool IsClientMsgTypeValid( const uint8 ClientMsgType );

	/**
	 * This function tracks the number of log calls per second for this client, 
	 * and disconnects the client if it detects too many calls are made per second
	 */
	ENGINE_API bool TrackLogsPerSecond();

	/**
	* Return current timeout value that should be used
	*/
	ENGINE_API virtual float GetTimeoutValue();

	/** Adds the channel to the ticking channels list. USed to selectively tick channels that have queued bunches or are pending dormancy. */
	void StartTickingChannel(UChannel* Channel) { ChannelsToTick.AddUnique(Channel); }

	/** Removes a channel from the ticking list directly */
	void StopTickingChannel(UChannel* Channel) { ChannelsToTick.Remove(Channel); }

	int32 GetNumTickingChannels() const { return ChannelsToTick.Num(); }

	inline FHistogram GetNetHistogram() const { return NetConnectionHistogram; }

	/** Whether or not a client packet has been received - used serverside, to delay any packet sends */
	inline bool HasReceivedClientPacket()
	{
		// The InternalAck and ServerConnection conditions, are only there to exclude demo's and clients from this check,
		// so that the check is only performed on servers.
		return IsInternalAck() || Driver->ServerConnection != nullptr || InReliable[0] != InitInReliable;
	}

	/**
	 * Sets the PlayerOnlinePlatformName member.
	 * Called by the engine during the login process with the NMT_Login message parameter.
	 */
	ENGINE_API void SetPlayerOnlinePlatformName(const FName InPlayerOnlinePlatformName);

	/** Returns the online platform name for the player on this connection. Only valid for client connections on servers. */
	FName GetPlayerOnlinePlatformName() const { return PlayerOnlinePlatformName; }
	
	/** Sets whether we handle opening channels with an index that already exists, used by replays to fast forward the packet stream */
	void SetAllowExistingChannelIndex(bool bAllow);

	/** Set to true if this connection's remote endpoint is a proxy. */
	ENGINE_API virtual void SetProxyConnection(bool bEnabled);
	ENGINE_API virtual bool IsProxyConnection() const;

private:
	void RestoreRemappedChannel(const int32 ChIndex);

	bool bIsProxyConnection = false;

public:

	/**
	 * Sets whether or not we should ignore bunches for a specific set of NetGUIDs.
	 * Should only be used with InternalAck.
	 */
	void SetIgnoreActorBunches(bool bInIgnoreActorBunches, TSet<FNetworkGUID>&& InIgnoredBunchGuids);

	/**
	 * Sets whether or not we should track released channel indices, see also SetIgnoreReservedChannels
	 * Should only be used with InternalAck.
	 */
	void SetReserveDestroyedChannels(bool bInReserveChannels);

	bool IsReservingDestroyedChannels() const { return bReserveDestroyedChannels; }

	void AddReservedChannel(int32 ChIndex) { ReservedChannels.Add(ChIndex); }

	/**
	 * Sets whether or not GetFreeChannelIndex should ignore reserved channels
	 * Should only be used with InternalAck.
	 */
	void SetIgnoreReservedChannels(bool bInIgnoreReservedChannels);

	/** Returns the OutgoingBunches array, only to be used by UChannel::SendBunch */
	TArray<FOutBunch *>& GetOutgoingBunches() { return OutgoingBunches; }

	/** Store a replicator to the dormancy map and release its strong pointer to its object */
	void StoreDormantReplicator(AActor* OwnerActor, UObject* Object, const TSharedRef<FObjectReplicator>& ObjectReplicator);
	
	/** Find a dormant replicator for the channel actor or one of its subobjects. Removes it from the map if found. */
	TSharedPtr<FObjectReplicator> FindAndRemoveDormantReplicator(AActor* OwnerActor, UObject* Object);

	/** Remove any reference to the dormant object replicator */
	void RemoveDormantReplicator(AActor* Actor, UObject* Object);

	/** Remove the reference of all dormant object replicators owned by an actor */
	void CleanupDormantReplicatorsForActor(AActor* Actor);

	/** Trigger a callback on all dormant replicators of every dormant actor we stored */
	ENGINE_API void ExecuteOnAllDormantReplicators(UE::Net::FExecuteForEachDormantReplicator ExecuteFunction);

	/** Trigger a callback on all dormant replicators owned by a dormant actor */
	void ExecuteOnAllDormantReplicatorsOfActor(AActor* OwnerActor, UE::Net::FExecuteForEachDormantReplicator ExecuteFunction);

	/** Removes dormant object replicators from objects now invalid. */
	void CleanupStaleDormantReplicators();

	/** Called before Driver.TickDispatch processes received packets */
	void PreTickDispatch();

	/** Called after Driver.TickDispatch has processed received packets */
	void PostTickDispatch();

	/**
	 * Flush the cache of sequenced packets waiting for a missing packet. Will flush only up to the next missing packet, unless bFlushWholeCache is set.
	 *
	 * @param bFlushWholeCache	Whether or not the whole cache should be flushed, or only flush up to the next missing packet
	 */
	void FlushPacketOrderCache(bool bFlushWholeCache=false);

	/**
	 * Get the total number of out of order packets on this connection.
	 *
	 * @return The total number of out of order packets.
	 */
	int32 GetTotalOutOfOrderPackets() const
	{
		return TotalOutOfOrderPacketsLost + TotalOutOfOrderPacketsRecovered + TotalOutOfOrderPacketsDuplicate;
	}

	/**
	 * Get the total number of out of order packets lost on this connection.
	 *
	 * @return The total number of out of order packets lost.
	 */
	int32 GetTotalOutOfOrderPacketsLost() const
	{
		return TotalOutOfOrderPacketsLost;
	}

	/**
	 * Get the total number of out of order packets recovered on this connection.
	 *
	 * @return The total number of out of order packets recovered.
	 */
	int32 GetTotalOutOfOrderPacketsRecovered() const
	{
		return TotalOutOfOrderPacketsRecovered;
	}

	/**
	 * Get the total number of out of order packets that were duplicates on this connection.
	 *
	 * @return The total number of out of order packets that were duplicates.
	 */
	int32 GetTotalOutOfOrderPacketsDuplicate() const
	{
		return TotalOutOfOrderPacketsDuplicate;
	}

	/**
	 * Sets the OS/NIC level timestamp, for the last packet that was received
	 *
	 * @param InOSReceiveTime			The OS/NIC level timestamp, for the last packet that was received
	 * @param bInIsOSReceiveTimeLocal	Whether the input timestamp is based on the same clock as the game thread, or needs translating
	 */
	void SetPacketOSReceiveTime(const FPacketTimestamp& InOSReceiveTime, bool bInIsOSReceiveTimeLocal)
	{
		LastOSReceiveTime = InOSReceiveTime;
		bIsOSReceiveTimeLocal = bInIsOSReceiveTimeLocal;
	}

	/** Called when an actor channel is open and knows its NetGUID. */
	virtual void NotifyActorNetGUID(UActorChannel* Channel) {}

	/**
	 * Returns the current delinquency analytics and resets them.
	 * This would be similar to calls to Get and Reset separately, except that the caller
	 * will assume ownership of data in this case.
	 */
	ENGINE_API void ConsumeQueuedActorDelinquencyAnalytics(FNetQueuedActorDelinquencyAnalytics& Out);

	/** Returns the current delinquency analytics. */
	ENGINE_API const FNetQueuedActorDelinquencyAnalytics& GetQueuedActorDelinquencyAnalytics() const;

	/** Resets the current delinquency analytics. */
	ENGINE_API void ResetQueuedActorDelinquencyAnalytics();

	/**
	 * Returns the current saturation analytics and resets them.
	 * This would be similar to calls to Get and Reset separately, except that the caller
	 * will assume ownership of data in this case.
	 */
	ENGINE_API void ConsumeSaturationAnalytics(FNetConnectionSaturationAnalytics& Out);

	/** Returns the current saturation analytics. */
	ENGINE_API const FNetConnectionSaturationAnalytics& GetSaturationAnalytics() const;

	/** Resets the current saturation analytics. */
	ENGINE_API void ResetSaturationAnalytics();

	/**
	 * Returns the current packet stability analytics and resets them.
	 * This would be similar to calls to Get and Reset separately, except that the caller
	 * will assume ownership of the data in this case.
	 */
	ENGINE_API void ConsumePacketAnalytics(FNetConnectionPacketAnalytics& Out);

	/** Returns the current packet stability analytics. */
	ENGINE_API const FNetConnectionPacketAnalytics& GetPacketAnalytics() const;

	/** Resets the current packet stability analytics. */
	ENGINE_API void ResetPacketAnalytics();

	/**
	 * Called to notify the connection that we attempted to replicate its actors this frame.
	 * This is primarily for analytics book keeping.
	 *
	 * @param bWasSaturated		True if we failed to replicate all data because we were saturated.
	 */
	ENGINE_API void TrackReplicationForAnalytics(const bool bWasSaturated);

	/**
	 * Get the current number of sent packets for which we have received a delivery notification
	 */
	uint32 GetOutTotalNotifiedPackets() const { return OutTotalNotifiedPackets; }

	/** Sends the NMT_Challenge message */
	ENGINE_API void SendChallengeControlMessage();

	/** Sends the NMT_Challenge message based on encryption response */
	void SendChallengeControlMessage(const FEncryptionKeyResponse& Response);

	bool GetPendingCloseDueToReplicationFailure() const
	{
		return bConnectionPendingCloseDueToReplicationFailure;
	}

	ENGINE_API void SetPendingCloseDueToReplicationFailure();

	/** Called when owning network driver receives NotifyActorDestroyed. */
	ENGINE_API virtual void NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel = false);

	ENGINE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason);

	/** Update FNetLevelVisibilityTransactionId for server instigated level streaming */
	FNetLevelVisibilityTransactionId UpdateLevelStreamStatusChangedTransactionId(const ULevelStreaming* LevelObject, const FName PackageName, bool bShouldBeVisible);

protected:

	bool GetPendingCloseDueToSocketSendFailure() const
	{
		return bConnectionPendingCloseDueToSocketSendFailure;
	}

	ENGINE_API void SetPendingCloseDueToSocketSendFailure();

	void CleanupDormantActorState();

	void ClearDormantReplicatorsReference();

	/** During cleanup this will destroy the actor owned by this connection (generally a PlayerController) */
	ENGINE_API virtual void DestroyOwningActor();

	/** Called internally to destroy an actor during replay fast-forward when the actor channel index will be recycled */
	ENGINE_API virtual void DestroyIgnoredActor(AActor* Actor);

	/** This is called whenever a connection has passed the relative time to be considered timed out. */
	ENGINE_API virtual void HandleConnectionTimeout(const FString& Error);

	/**
	 * Notification that information about this connection may have been updated
	 */
	ENGINE_API void NotifyConnectionUpdated();

	/**
	 * Return last notified packet id
	 */
	ENGINE_API int32 GetLastNotifiedPacketId() const;

private:
	/**
	 * The channels that need ticking. This will be a subset of OpenChannels, only including
	 * channels that need to process either dormancy or queued bunches. Should be a significant
	 * optimization over ticking and calling virtual functions on the potentially hundreds of
	 * OpenChannels every frame.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UChannel>> ChannelsToTick;

	/** Histogram of the received packet time */
	FHistogram NetConnectionHistogram;

	/** Online platform ID of remote player on this connection. Only valid on client connections (server side).*/
	FName PlayerOnlinePlatformName;

	/** This is an acceleration set that is derived from ClientWorldPackageName and ClientVisibleLevelNames. We use this to quickly test an AActor*'s visibility while replicating. */
	mutable TMap<FName, bool> ClientVisibleActorOuters;

	/** This is used to capture visibility updates while the server is in transition and deffer the update until the server has completed the transition */
	TMap<FName, FUpdateLevelVisibilityLevelInfo> PendingUpdateLevelVisibility;

	/** This is used to track any pending streaming status requests the server has sent */
	TMap<FName, FNetLevelVisibilityTransactionId> ClientPendingStreamingStatusRequest;

private:

	/** Called when the stateless handshake component is enabled and we received an handshake failure */
	void OnStatelessHandshakeFailure(UE::Net::FStatelessHandshakeFailureInfo HandshakeFailureInfo);

	/** Called by PlayerController to tell connection about client level visibility change */
	void UpdateLevelVisibilityInternal(const struct FUpdateLevelVisibilityLevelInfo& LevelVisibility);

	/** Called by PlayerController to tell connection about client container visibility change */
	void UpdateContainerVisibilityInternal(UObject* Container, bool bVisible);

	/** Called internally to update cached acceleration map */
	bool UpdateCachedLevelVisibility(const FName& PackageName) const;

	/** Updates entire cached LevelVisibility map */
	void UpdateAllCachedLevelVisibility() const;

	/** Returns true if an outgoing packet should be dropped due to packet simulation settings, including loss burst simulation. */
#if DO_ENABLE_NET_TEST
	bool ShouldDropOutgoingPacketForLossSimulation(int64 NumBits) const;
#endif

	/**
	*	Test the current emulation settings to delay, drop, etc the current packet
	*	Returns true if the packet was emulated and false when the packet must be sent via the normal path
	*/
#if DO_ENABLE_NET_TEST
	bool CheckOutgoingPacketEmulation(double CurrentTimeInS, FOutPacketTraits& Traits);
	bool CheckIncomingPacketEmulation(double CurrentTimeInS, FBitReader& Reader);
#endif

	/** Write packetHeader */
	void WritePacketHeader(FBitWriter& Writer);

	/** Write placeholder bits for data filled before the real send (ServerFrameTime, JitterClockTime) */
	void WriteDummyPacketInfo(FBitWriter& Writer);

	/** Overwrite the dummy packet info values with real values before sending */
	void WriteFinalPacketInfo(FBitWriter& Writer, double PacketSentTimeInS);
	
	/** Read extended packet header information (ServerFrameTime) */
	bool ReadPacketInfo(FBitReader& Reader, bool bHasPacketInfoPayload, FEngineNetworkCustomVersion::Type EngineNetVer);

	/** Packet was acknowledged as delivered */
	void ReceivedAck(int32 AckPacketId, FChannelsToClose& OutChannelsToClose);

	/** Calculate the average jitter while adding the new packet's jitter value */
	void ProcessJitter(uint32 PacketJitterClockTimeMS);

	/** Flush outgoing packet if needed before we write data to it */
	void PrepareWriteBitsToSendBuffer(const int32 SizeInBits, const int32 ExtraSizeInBits);
	
	/** Write data to outgoing send buffer, PrepareWriteBitsToSendBuffer should be called before to make sure that
		data will fit in packet. */
	int32 WriteBitsToSendBufferInternal( 
		const uint8 *	Bits, 
		const int32		SizeInBits, 
		const uint8 *	ExtraBits, 
		const int32		ExtraSizeInBits,
		EWriteBitsDataType DataType =  EWriteBitsDataType::Unknown);

	/**
	 * on the server, the world the client has told us it has loaded
	 * used to make sure the client has traveled correctly, prevent replicating actors before level transitions are done, etc
	 */
	FName ClientWorldPackageName;

	/** A map of class names to arrays of time differences between replication of actors of that class for each connection */
	TMap<FString, TArray<float>> ActorsStarvedByClassTimeMap;

	/** Tracks channels that we should ignore when handling special demo data. */
	TMap<int32, FNetworkGUID> IgnoringChannels;

	/** Track remapped channel index values when replay flag is set */
	TMap<int32, int32> ChannelIndexMap;

	bool bAllowExistingChannelIndex;

	/** Set of guids we may need to ignore when processing a delta checkpoint */
	TSet<FNetworkGUID> IgnoredBunchGuids;

	/** Set of channels we may need to ignore when processing a delta checkpoint */
	TSet<int32> IgnoredBunchChannels;

	bool bIgnoreActorBunches;

	/** Set of channel index values to reserve so GetFreeChannelIndex won't use them */
	TSet<int32> ReservedChannels;

	bool bReserveDestroyedChannels;
	bool bIgnoreReservedChannels;

	/** This is only used in UChannel::SendBunch. It's a member so that we can preserve the allocation between calls, as an optimization, and in a thread-safe way to be compatible with demo.ClientRecordAsyncEndOfFrame */
	TArray<FOutBunch*> OutgoingBunches;

	/** Per packet bookkeeping of written channelIds */
	FWrittenChannelsRecord ChannelRecord;

	/** Sequence data used to implement reliability */
	FNetPacketNotify PacketNotify;

	/** Full PacketId  of last sent packet that we have received notification for (i.e. we know if it was delivered or not). Related to OutAckPacketId which is tha last successfully delivered PacketId */
	int32 LastNotifiedPacketId;

	/** Count the number of notified packets, i.e. packets that we know if they are delivered or not. Used to reliably measure outgoing packet loss */
	uint32 OutTotalNotifiedPackets;

	/** Keep old behavior where we send a packet with only acks even if we have no other outgoing data if we got incoming data */
	uint32 HasDirtyAcks;
	
	/** True if we've hit the actor channel limit and logged a warning about it */
	bool bHasWarnedAboutChannelLimit;

	/** True if we are pending close due to a socket failure during send */
	bool bConnectionPendingCloseDueToSocketSendFailure;

	/** True if we are pending close due to unrecoverable replication errors. */
	bool bConnectionPendingCloseDueToReplicationFailure;

	FNetworkGUID GetActorGUIDFromOpenBunch(FInBunch& Bunch);

	/** Out of order packet tracking/correction */

	/** Stat tracking for the total number of out of order packets lost */
	int32 TotalOutOfOrderPacketsLost = 0;

	/** Stat tracking for the total number of out of order packets recovered */
	int32 TotalOutOfOrderPacketsRecovered = 0;

	/** Stat tracking for the total number of out of order packets that were duplicates */
	int32 TotalOutOfOrderPacketsDuplicate = 0;

	/** Buffer of partially read (post-PacketHandler) sequenced packets, which are waiting for a missing packet/sequence */
	TOptional<TCircularBuffer<TUniquePtr<FBitReader>>> PacketOrderCache;

	/** The current start index for PacketOrderCache */
	int32 PacketOrderCacheStartIdx;

	/** The current number of valid packets in PacketOrderCache */
	int32 PacketOrderCacheCount;

	FNetConnectionSaturationAnalytics SaturationAnalytics;
	FNetConnectionPacketAnalytics PacketAnalytics;

	/** Whether or not PacketOrderCache is presently being flushed */
	bool bFlushingPacketOrderCache;

	/** Per NetDriver unique ID that can be used instead of passing around a pointer to the connection. Be careful if caching as the handle can be re-purposed as connections are destroyed and created. */
	UE::Net::FConnectionHandle ConnectionHandle;

#if UE_NET_TRACE_ENABLED
	FNetTraceCollector* InTraceCollector = nullptr;
	FNetTraceCollector* OutTraceCollector = nullptr;
	/** Cached NetTrace id from the NetDriver */
	uint32 NetTraceId = 0;
#endif

	/** 
	* Set to true after a packet is flushed (sent) and reset at the end of the connection's Tick. 
	*/
	bool bFlushedNetThisFrame = false;

	/** Only set temporarily during the frame, should not need to be tracked for gc */
	AActor* RepContextActor;
	ULevel* RepContextLevel;

	friend struct FScopedRepContext;
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Required by APlayerController::PostMigrate() to read and update CachedNoPawnPlayerController.
	friend class APlayerController;
#endif

	bool bAutoFlush;

	/** Used to limit logging when we detect QueuedBits overflow */
	bool bLoggedFlushNetQueuedBitsOverflow = false;

	/** RPC/Replication code DoS detection */
	TUniquePtr<FRPCDoSDetection> RPCDoS;

	/** NetConnection specific Fault Recovery for attempting to recover from connection faults, before triggering Close */
	TUniquePtr<UE::Net::FNetConnectionFaultRecovery> FaultRecovery;

	/** Whether or not this NetConnection has already received an NMT_CloseReason message */
	bool bReceivedCloseReason = false;

	/** Stored close reason from a GracefulClose call, to be used in the actual Close */
	TPimplPtr<UE::Net::FNetResult, EPimplPtrMode::DeepCopy> PendingGracefulCloseResult;

	/** Ping collection and calculation */
	TPimplPtr<UE::Net::FNetPing> NetPing;


	int32 GetFreeChannelIndex(const FName& ChName) const;

public:
	AActor* GetRepContextActor() const { return RepContextActor; }
	ULevel* GetRepContextLevel() const { return RepContextLevel; }

	bool GetAutoFlush() const { return bAutoFlush; }
	void SetAutoFlush(bool bValue) { bAutoFlush = bValue; }

	FRPCDoSDetection* GetRPCDoS()
	{
		return RPCDoS.Get();
	}

	UE::Net::FNetConnectionFaultRecovery* GetFaultRecovery()
	{
		return FaultRecovery.Get();
	}

	UE::Net::FNetPing* GetNetPing()
	{
		return NetPing.Get();
	}

	/**
	 * Sends an NMT_CloseReason message, with the specified close reason or close reason chain.
	 * Called automatically by UNetConnection::Close, but should be called beforehand if sending a packet that will trigger a remote close.
	 *
	 * @param CloseReason	The close reason or close reason chain to send
	 */
	void SendCloseReason(FNetCloseResult&& CloseReason)
	{
		SendCloseReason(static_cast<FNetResult&&>(MoveTemp(CloseReason)));
	}

	/**
	 * Sends an NMT_CloseReason message, with the specified close reason or close reason chain.
	 * Called automatically by UNetConnection::Close, but should be called beforehand if sending a packet that will trigger a remote close.
	 *
	 * @param CloseReason	The close reason or close reason chain to send
	 */
	ENGINE_API void SendCloseReason(FNetResult&& CloseReason);

	/**
	 * Handles parsing/validation and logging of NMT_CloseReason messages
	 *
	 * @param CloseReasonList	Delimited list of close reasons
	 */
	ENGINE_API void HandleReceiveCloseReason(const FString& CloseReasonList);

	/**
	 * Handles receiving NMT_Upgrade messages (including at stateless handshake level)
	 *
	 * @param RemoteNetworkVersion		The net version of the remote side
	 * @param RemoteNetworkFeatures		The net runtime features of the remote side
	 * @param NetUpgradeSource			The source of the net upgrade message
	 * @return Return true if the connection could be upgraded and we can restart the handshake process. Return false if the upgrade was impossible and the connection must disconnect.
	 */
	ENGINE_API bool HandleReceiveNetUpgrade(uint32 RemoteNetworkVersion, EEngineNetworkRuntimeFeatures RemoteNetworkFeatures,
											UE::Net::ENetUpgradeSource NetUpgradeSource=UE::Net::ENetUpgradeSource::ControlChannel);

private:
	/**
	 * Attempts to recover from a NetConnection error, and closes the connection if that fails
	 *
	 * @param InResult		The type of error result being handled
	 */
	void HandleNetResultOrClose(ENetCloseResult InResult);

	/**
	 * If this connection is in the USOCK_Closing state, check whether all reliables have been acknowledged, if so, actually Close (send control channel close bunch).
	 */
	void TryClosePendingGracefulClose();

protected:
	TOptional<FNetworkCongestionControl> NetworkCongestionControl;

	void InitChannelData();

	void SetConnectionHandle(UE::Net::FConnectionHandle Handle)
	{
		ConnectionHandle = Handle;
	}

public:
	/**
	 * Retrieve stored set of replicated sub-objects of the given actor at the time of the last dormancy flush
	 * This data is cleared when the actor is processed by ReplicateActor
	 *
	 * @param Actor		The actor to retrieve the object map for
	 * @return A map of network guids to weak object pointers 
	 */
	UE::Net::FDormantObjectMap* GetDormantFlushedObjectsForActor(AActor* Actor);

	/**
	 * Clear stored flushed replicated sub-objects for a given actor, generally after replication or when the actor is destroyed
	 *
	 * @param Actor		The actor to clear the flushed object data for
	 */
	void ClearDormantFlushedObjectsForActor(AActor* Actor);
};

struct FScopedRepContext
{
public:
	FScopedRepContext(UNetConnection* InConnection, AActor* InActor);

	FScopedRepContext(UNetConnection* InConnection, ULevel* InLevel)
		: Connection(InConnection)
	{
		if (Connection)
		{
			check(!Connection->RepContextActor);
			check(!Connection->RepContextLevel);

			Connection->RepContextLevel = InLevel;
		}
	}

	~FScopedRepContext()
	{
		if (Connection)
		{
			Connection->RepContextActor = nullptr;
			Connection->RepContextLevel = nullptr;
		}
	}

	FScopedRepContext(FScopedRepContext&&) = delete;
	FScopedRepContext(const FScopedRepContext&) = delete;
	FScopedRepContext& operator=(const FScopedRepContext&) = delete;
	FScopedRepContext& operator=(FScopedRepContext&&) = delete;

private:
	UNetConnection* Connection;
};

/** Help structs for temporarily setting network settings */
struct FNetConnectionSettings
{
	FNetConnectionSettings( UNetConnection* InConnection )
	{
#if DO_ENABLE_NET_TEST
		PacketLag = InConnection->PacketSimulationSettings.PktLag;
		PacketLoss = InConnection->PacketSimulationSettings.PktLoss;
#else
		PacketLag = 0;
		PacketLoss = 0;
#endif
	}

	FNetConnectionSettings( int32 InPacketLag )
	{
		PacketLag = InPacketLag;
		PacketLoss = 0;
	}

	void ApplyTo(UNetConnection* Connection)
	{
#if DO_ENABLE_NET_TEST
		Connection->PacketSimulationSettings.PktLag = PacketLag;
		Connection->PacketSimulationSettings.PktLoss = PacketLoss;
#endif
	}

	int32 PacketLag;
	int32 PacketLoss;
};

/** Allows you to temporarily set connection settings within a scape. This will also force flush the connection before/after.
 *	Lets you do things like force a single channel to delay or drop packets
 */
struct FScopedNetConnectionSettings
{
	FScopedNetConnectionSettings(UNetConnection* InConnection, FNetConnectionSettings NewSettings, bool Apply=true) : 
		Connection(InConnection), OldSettings(InConnection), ShouldApply(Apply)
	{
		if (ShouldApply)
		{
			Connection->FlushNet();
			NewSettings.ApplyTo(Connection.Get());
		}
	}
	~FScopedNetConnectionSettings()
	{
		UNetConnection* LocalConnection = Connection.Get();
		if (LocalConnection && ShouldApply)
		{
			LocalConnection->FlushNet();
			OldSettings.ApplyTo(LocalConnection);
		}
	}

	TWeakObjectPtr<UNetConnection> Connection;
	FNetConnectionSettings OldSettings;
	bool ShouldApply;
};

/** A fake connection that will absorb traffic and auto ack every packet. Useful for testing scaling. Use net.SimulateConnections command to add at runtime. */
UCLASS(transient, config=Engine, MinimalAPI)
class USimulatedClientNetConnection
	: public UNetConnection
{
	GENERATED_UCLASS_BODY()
public:

	virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override { }
	ENGINE_API void HandleClientPlayer( APlayerController* PC, UNetConnection* NetConnection ) override;
	virtual FString LowLevelGetRemoteAddress(bool bAppendPort=false) override { return FString(); }
	virtual bool ClientHasInitializedLevel(const ULevel* TestLevel) const override { return true; }

	virtual void DestroyOwningActor() override { /* Don't destroy the OwningActor since we follow a real PlayerController*/ }

	virtual TSharedPtr<const FInternetAddr> GetRemoteAddr() override { return nullptr; }

	virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}
	virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}

	virtual FString LowLevelDescribe() override { return TEXT("Simulated Client"); }

};

#if UE_NET_TRACE_ENABLED
	inline FNetTraceCollector* UNetConnection::GetInTraceCollector() const { return InTraceCollector; }
	inline FNetTraceCollector* UNetConnection::GetOutTraceCollector() const { return OutTraceCollector; }
#else
	inline FNetTraceCollector* UNetConnection::GetInTraceCollector() const { return nullptr; }
	inline FNetTraceCollector* UNetConnection::GetOutTraceCollector() const { return nullptr; }
#endif
