// Copyright Epic Games, Inc. All Rights Reserved.

//
// Base class of a network driver attached to an active or pending level.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Net/NetworkMetricsDatabase.h"
#endif
#include "HAL/IConsoleManager.h"
#include "Iris/IrisConfig.h"
#include "Math/RandomStream.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "Misc/NetworkGuid.h"
#include "UObject/CoreNet.h"
#include "Channel.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Core/Misc/DDoSDetection.h"
#include "IPAddress.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Net/NetAnalyticsTypes.h"
#include "Net/NetConnectionIdHandler.h"
#include "HAL/LowLevelMemTracker.h"
#include "Templates/PimplPtr.h"
#include "NetDriver.generated.h"

/**
 *****************************************************************************************
 * NetDrivers, NetConnections, and Channels
 *****************************************************************************************
 *
 *
 * UNetDrivers are responsible for managing sets of UNetConnections, and data that can be shared between them.
 * There is typically a relatively small number of UNetDrivers for a given game. These may include:
 *	- The Game NetDriver, responsible for standard game network traffic.
 *	- The Demo NetDriver, responsible for recording or playing back previously recorded game data. This is how Replays work.
 *	- The Beacon NetDriver, responsible for network traffic that falls outside of "normal" gameplay traffic.
 *
 * Custom NetDrivers can also be implemented by games or applications and used.
 * NetConnections represent individual clients that are connected to a game (or more generally, to a NetDriver).
 *
 * End point data isn't directly handled by NetConnections. Instead NetConnections will route data to Channels.
 * Every NetConnection will have its own set of channels.
 *
 * Common types of channels:
 *
 *	- A Control Channel is used to send information regarding state of a connection (whether or not the connection should close, etc.)
 *	- A Voice Channel may be used to send voice data between client and server.
 *	- A Unique Actor Channel will exist for every Actor replicated from the server to the client.
 *
 * Custom Channels can also be created and used for specialized purposes (although, this isn't very common).
 *
 *
 *****************************************************************************************
 * Game Net Drivers, Net Connections, and Channels
 *****************************************************************************************
 *
 *
 * Under normal circumstances, there will exist only a single NetDriver (created on Client and Server) for "standard" game traffic and connections.
 *
 * The Server NetDriver will maintain a list of NetConnections, each representing a player that is in the game. It is responsible for
 * replicating Actor Data.
 *
 * Client NetDrivers will have a single NetConnection representing the connection to the Server.
 *
 * On both Server and Client, the NetDriver is responsible for receiving Packets from the network and passing those to the appropriate
 * NetConnection (and establishing new NetConnections when necessary).
 *
 *
 *****************************************************************************************
 *****************************************************************************************
 *****************************************************************************************
 * Initiating Connections / Handshaking Flow.
 *****************************************************************************************
 *****************************************************************************************
 *****************************************************************************************
 *
 *
 * UIpNetDriver and UIpConnection (or derived classes) are the engine defaults for almost every platform, and everything below
 * describes how they establish and manage connections. These processes can differ between implementations of NetDriver, however.
 *
 * Both Server and Clients will have have their own NetDrivers, and all UE Replicated Game traffic will be sent by or Received
 * from the IpNetDriver. This traffic also includes logic for establishing connections, and re-establishing connections when
 * something goes wrong.
 *
 * Handshaking is split across a couple of different places: NetDriver, PendingNetGame, World, PacketHandlers, and maybe others.
 * The split is due to having separate needs, things such as: determining whether or not an incoming connection is sending data in "UE-Protocol",
 * determining whether or not an address appears to be malicious, whether or not a given client has the correct version of a game, etc.
 *
 *
 *****************************************************************************************
 * Startup and Handshaking
 *****************************************************************************************
 *
 *
 * Whenenever a server Loads a map (via UEngine::LoadMap), we will make a call into UWorld::Listen.
 * That code is responsible for creating the main Game Net Driver, parsing out settings, and calling UNetDriver::InitListen.
 * Ultimately, that code will be responsible for figuring out what how exactly we listen for client connections.
 * For example, in IpNetDriver, that is where we determine the IP / Port we will bind to by calls to our configured Socket Subsystem
 * (see ISocketSubsystem::GetLocalBindAddresses and ISocketSubsystem::BindNextPort).
 *
 * Once the server is listening, it's ready to start accepting client connections.
 *
 * Whenever a client wants to Join a server, they will first establish a new UPendingNetGame in UEngine::Browse with the server's IP.
 * UPendingNetGame::Initialize and UPendingNetGame::InitNetDriver are responsible for initializing settings and setting up the NetDriver respectively.
 * Clients will immediately setup a UNetConnection for the server as a part of this initialization, and will start sending data to the server on that connection,
 * initiating the handshaking process.
 *
 * On both Clients and Server, UNetDriver::TickDispatch is typically responsible for receiving network data.
 * Typically, when we receive a packet, we inspect its address and see whether or not it's from a connection we already know about.
 * We determine whether or not we've established a connection for a given source address by simply keeping a map from FInternetAddr to UNetConnection.
 *
 * If a packet is from a connection that's already established, we pass the packet along to the connection via UNetConnection::ReceivedRawPacket.
 * If a packet is not from a connection that's already established, we treat is as "connectionless" and begin the handshaking process.
 *
 * See StatelessConnectionHandlerComponent.cpp for details on how this handshaking works.
 *
 *
 *****************************************************************************************
 * UWorld / UPendingNetGame / AGameModeBase Startup and Handshaking
 *****************************************************************************************
 *
 *
 * After the UNetDriver and UNetConnection have completed their handshaking process on Client and Server,
 * UPendingNetGame::SendInitialJoin will be called on the Client to kick off game level handshaking.
 *
 * Game Level Handshaking is done through a more structured and involved set of FNetControlMessages.
 * The full set of control messages can be found in DataChannel.h.
 *
 * Most of the work for handling these control messages are done either in UWorld::NotifyControlMessage,
 * and UPendingNetGame::NotifyControlMessage. Briefly, the flow looks like this:
 *
 * Client's UPendingNetGame::SendInitialJoin sends NMT_Hello.
 *
 * Server's UWorld::NotifyControlMessage receives NMT_Hello, sends NMT_Challenge.
 *
 * Client's UPendingNetGame::NotifyControlMessage receives NMT_Challenge, and sends back data in NMT_Login.
 *
 * Server's UWorld::NotifyControlMessage receives NMT_Login, verifies challenge data, and then calls AGameModeBase::PreLogin.
 * If PreLogin doesn't report any errors, Server calls UWorld::WelcomePlayer, which call AGameModeBase::GameWelcomePlayer,
 * and send NMT_Welcome with map information.
 *
 * Client's UPendingNetGame::NotifyControlMessage receives NMT_Welcome, reads the map info (so it can start loading later),
 * and sends an NMT_NetSpeed message with the configured Net Speed of the client.
 *
 * Server's UWorld::NotifyControlMessage receives NMT_NetSpeed, and adjusts the connections Net Speed appropriately.
 *
 * At this point, the handshaking is considered to be complete, and the player is fully connected to the game.
 * Depending on how long it takes to load the map, the client could still receive some non-handshake control messages
 * on UPendingNetGame before control transitions to UWorld.
 *
 * There are also additional steps for handling Encryption when desired.
 *
 *
 *****************************************************************************************
 * Reestablishing Lost Connections
 *****************************************************************************************
 *
 *
 * Throughout the course of a game, it's possible for connections to be lost for a number of reasons.
 * Internet could drop out, users could switch from LTE to WIFI, they could leave a game, etc.
 *
 * If the server initiated one of these disconnects, or is otherwise aware of it (due to a timeout or error),
 * then the disconnect will be handled by closing the UNetConnection and notifying the game.
 * At that point, it's up to a game to decide whether or not they support Join In Progress or Rejoins.
 * If the game does support it, we will completely restart the handshaking flow as above.
 *
 * If something just briefly interrupts the client's connection, but the server is never made aware,
 * then the engine / game will typically recover automatically (albeit with some packet loss / lag spike).
 *
 * However, if the Client's IP Address or Port change for any reason, but the server isn't aware of this,
 * then we will begin a recovery process by redoing the low level handshake. In this case, game code
 * will not be alerted.
 *
 * This process is covered in StatlessConnectionHandlerComponent.cpp.
 *
 *
 *****************************************************************************************
 *****************************************************************************************
 *****************************************************************************************
 * Data Transmission
 *****************************************************************************************
 *****************************************************************************************
 *****************************************************************************************
 *
 *
 * Game NetConnections and NetDrivers are generally agnostic to the underlying communication method / technology used.
 * That is is left up to subclasses to decide (classes such as UIpConnection / UIpNetDriver or UWebSocketConnection / UWebSocketNetDriver).
 *
 * Instead, UNetDriver and UNetConnection work with Packets and Bunches.
 *
 * Packets are blobs of data that are sent between pairs of NetConnections on Host and Client.
 * Packets consist of meta data about the packet (such as header information and acknowledgments), and Bunches.
 *
 * Bunches are blobs of data that are sent between pairs of Channels on Host and Client.
 * When a Connection receives a Packet, that packet will be disassembled into individual bunches.
 * Those bunches are then passed along to individual Channels to be processed further.
 *
 * A Packet may contain no bunches, a single bunch, or multiple bunches.
 * Because size limits for bunches may be larger than the size limits of a single packet, UE supports
 * the notion of partial bunches.
 *
 * When a bunch is too large, before transmission we will slice it into a number of smaller bunches.
 * these bunches will be flagged as PartialInitial, Partial, or PartialFinal. Using this information,
 * we can reassemble the bunches on the receiving end.
 *
 *	Example: Client RPC to Server.
 *		- Client makes a call to Server_RPC.
 *		- That request is forwarded (via NetDriver and NetConnection) to the Actor Channel that owns the Actor on which the RPC was called.
 *		- The Actor Channel will serialize the RPC Identifier and parameters into a Bunch. The Bunch will also contain the ID of its Actor Channel.
 *		- The Actor Channel will then request the NetConnection send the Bunch.
 *		- Later, the NetConnection will assemble this (and other) data into a Packet which it will send to the server.
 *		- On the Server, the Packet will be received by the NetDriver.
 *		- The NetDriver will inspect the Address that sent the Packet, and hand the Packet over to the appropriate NetConnection.
 *		- The NetConnection will disassemble the Packet into its Bunches (one by one).
 *		- The NetConnection will use the Channel ID on the bunch to Route the bunch to the corresponding Actor Channel.
 *		- The ActorChannel will them disassemble the bunch, see it contains RPC data, and use the RPC ID and serialized parameters
 *			to call the appropriate function on the Actor.
 *
 *
 *****************************************************************************************
 * Reliability and Retransmission
 *****************************************************************************************
 *
 *
 * UE Networking typically assumes reliability isn't guaranteed by the underlying network protocol.
 * Instead, it implements its own reliability and retransmission of both packets and bunches.
 *
 * When a NetConnection is established, it will establish a Sequence Number for its packets and bunches.
 * These can either be fixed, or randomized (when randomized, the sequence will be sent by the server).
 *
 * The packet number is per NetConnection, incremented for every packet sent, every packet will include its packet number,
 * and we will never retransmit a packet with the same packet number.
 *
 * The bunch number is per Channel, incremented for every **reliable** bunch sent, and every **reliable** bunch will
 * include its bunch number. Unlike packets, though, exact (reliable) bunches may be retransmitted. This means we
 * will resend bunches with the same bunch number.
 *
 * Note, throughout the code what are described above as both bunch numbers and packet numbers are commonly referred to
 * just as sequence numbers. We make the distinction here for clearer understanding.
 *
 *	--- Detecting Incoming Dropped Packets ---
 *
 *
 *	By assigning packet numbers, we can easily detect when incoming packets are lost.
 *	This is done simply by taking the difference between the last successfully received packet number, and the
 *	packet number of the current packet being processed.
 *
 *	Under good conditions, all packets will be received in the order they are sent. This means that the difference will
 *	be +1.
 *
 *	If the difference is greater than 1, that indicates that we missed some packets. We will just
 *	assume that the missing packets were dropped, but consider the current packet to have been successfully received,
 *	and use its number going forward.
 *
 *	If the difference is negative (or 0), that indicates that we either received some packets out of order, or an external
 *	service is trying to resend data to us (remember, the engine will not reuse sequence numbers).
 *
 *	In either case, the engine will typically ignore the missing or invalid packets, and will not send ACKs for them.
 *
 *	We do have methods for "fixing" out of order packets that are received on the same frame.
 *	When enabled, if we detect missing packets (difference > 1), we won't process the current packet immediately.
 *	Instead, it will add it to a queue. The next time we receive a packet successfully (difference == 1), we will
 *	see if the head of our queue is properly ordered. If so, we will process it, otherwise we will continue
 *	receiving packets.
 *
 *	Once we've read all packets that are currently available, we will flush this queue processing any remaining packets.
 *	Anything that's missing at this point will be assumed to have been dropped.
 *
 *	Every packet successfully received will have its packet number sent back to the sender as an acknowledgment (ACK).
 *
 *
 *	--- Detecting Outgoing Dropped Packets ---
 *
 *
 *	As mentioned above, whenever a packet is received successfully the recipient will send back an ACK.
 *	These ACKs will contain the packet numbers of successfully received packets, in sequence order.
 *
 *	Similar to how the recipient tracks the packet number, the sender will track the highest ACKed packet number.
 *
 *	When ACKs are being processed, any ACK below our last received ACK is ignored and any gaps in packet
 *	numbers are considered Not Acknowledged (NAKed).
 *
 *	It is the sender's responsibility to handle these ACKs and NAKs and resend any missing data.
 *	The new data will be added to new outgoing packets (again, we will not resend packets we've already sent,
 *	or reuse packet sequence numbers).
 *
 *
 *	--- Resending Missing Data ---
 *
 *
 *	As mentioned above, packets alone don't contain useful game data. Instead, it's the bunches that comprise them
 *	that have meaningful data.
 *
 *	Bunches can either be marked as Reliable or Unreliable.
 *
 *	The engine will make no attempt at resending unreliable bunches if they are dropped. Therefore, if bunches
 *	are marked unreliable, the game / engine should be able to continue without them, or external retry
 *	mechanisms must be put in place, or the data must be sent redundantly. Therefore, everything below only
 *	applies to reliable bunches.
 *
 *	However, the engine will attempt to resend reliable bunches. Whenever a reliable bunch is sent, it will
 *	be added to a list of un-ACKed reliable bunches. If we receive a NAK for a packet that contained the bunch,
 *	the engine will retransmit an exact copy of that bunch. Note, because bunches may be partial, dropping even
 *	a single partial bunch will result in retransmission of the entire bunch. When all packets containing a bunch
 *	have been ACKed, we will remove it from the list.
 *
 *	Similar to packets, we will compare the bunch number for received reliable bunches to the last successfully
 *	received bunch. If we detect that the difference is negative, we simply ignore the bunch. If the difference
 *	is greater than one, we will assume we missed a bunch. Unlike packet handling, we will not discard this data.
 *	Instead, we will queue the bunch and pause processing of **any** bunches, reliable or unreliable.
 *	Processing will not be resumed until we detect have received the missing bunches, at which point we will process
 *	them, and then start processing our queued bunches.
 *	Any new bunches that are received while waiting for the missing bunches, or while we still have any bunches in our
 *	queue, will be added to the queue instead of being processed immediately.
 *
 */

LLM_DECLARE_TAG_API(NetDriver, ENGINE_API);

class Error;
class FNetGUIDCache;
struct FNetSyncLoadReport;
class FNetworkNotify;
class FNetworkObjectList;
class FObjectReplicator;
class FRepChangedPropertyTracker;
class FRepLayout;
class FReplicationChangelistMgr;
class FVoicePacket;
class StatelessConnectHandlerComponent;
class UNetConnection;
class UNetworkMetricsDatabase;
class UNetworkMetricsBaseListener;
class UReplicationDriver;
struct FNetworkObjectInfo;
class UChannel;
class IAnalyticsProvider;
class FNetAnalyticsAggregator;
class UNetDriver;
class UActorChannel;
class PacketHandler;
struct FReplicatedStaticActorDestructionInfo;

enum class ECreateReplicationChangelistMgrFlags;
enum class EEngineNetworkRuntimeFeatures : uint16;
class UReplicationSystem;
class UReplicationBridge;
namespace UE::Net
{
	class FNetObjectGroupHandle;
}

namespace UE::Net
{
	class FNetTokenStore;
	class FNetIDVariant;
	struct FServerReplicateActors_ForConnectionParams;

	class FScopedIgnoreStaticActorDestruction
	{
	public:
		ENGINE_API FScopedIgnoreStaticActorDestruction();
		ENGINE_API ~FScopedIgnoreStaticActorDestruction();

		UE_NONCOPYABLE(FScopedIgnoreStaticActorDestruction);

	private:
		bool bCachedValue = false;
	};

	bool ShouldIgnoreStaticActorDestruction();

	/** Utility function that counts the number of replicated subobjects owned by a given actor. Be aware that this function is not cheap because it needs to iterate and count all replicated components and their subobjects too. */
	ENGINE_API uint32 CountReplicatedSubObjectsOfActor(AActor* ReplicatedActor);
}

using FConnectionMap = TMap<TSharedRef<const FInternetAddr>, TObjectPtr<UNetConnection>, FDefaultSetAllocator, FInternetAddrConstKeyMapFuncs<TObjectPtr<UNetConnection>>>;

extern ENGINE_API TAutoConsoleVariable<int32> CVarNetAllowEncryption;
extern ENGINE_API int32 GNumSaturatedConnections;
extern ENGINE_API int32 GNumSharedSerializationHit;
extern ENGINE_API int32 GNumSharedSerializationMiss;
extern ENGINE_API int32 GNumReplicateActorCalls;
extern ENGINE_API bool GReplicateActorTimingEnabled;
extern ENGINE_API bool GReceiveRPCTimingEnabled;
extern ENGINE_API double GReplicateActorTimeSeconds;
extern ENGINE_API uint32 GNetOutBytes;
extern ENGINE_API double GReplicationGatherPrioritizeTimeSeconds;
extern ENGINE_API double GServerReplicateActorTimeSeconds;
extern ENGINE_API int32 GNumClientConnections;
extern ENGINE_API int32 GNumClientUpdateLevelVisibility;

namespace UE::Net::Private
{
	/** Allow other internal systems to check this cvar */
	extern int32 SerializeNewActorOverrideLevel;
}

// Delegates

#if !UE_BUILD_SHIPPING
/**
 * Delegate for hooking ProcessRemoteFunction (used by NetcodeUnitTest)
 *
 * @param Actor				The actor the RPC will be called in
 * @param Function			The RPC to call
 * @param Parameters		The parameters data blob
 * @param OutParms			Out parameter information (irrelevant for RPC's)
 * @param Stack				The script stack
 * @param SubObject			The sub-object the RPC is being called in (if applicable)
 * @param bBlockSendRPC		Whether or not to block sending of the RPC (defaults to false)
 */
DECLARE_DELEGATE_SevenParams(FOnSendRPC, AActor* /*Actor*/, UFunction* /*Function*/, void* /*Parameters*/,
									FOutParmRec* /*OutParms*/, FFrame* /*Stack*/, UObject* /*SubObject*/, bool& /*bBlockSendRPC*/);

/**
 * Delegate for hooking ShouldSkipRepNotifies
 */
DECLARE_DELEGATE_RetVal(bool, FShouldSkipRepNotifies);

#endif

/**
 * The structure to pass to the OnConsiderListUpdate delegate 
 * 
 * @param DeltaSeconds     Time between the frames
 * @param Connection       NetConnection to process
 * @param bCPUSaturated    Not used by the engine at the moment but kept for compatibility
 */
struct ConsiderListUpdateParams
{
	float DeltaSeconds = 0;
	UNetConnection* Connection = nullptr;
	bool bCPUSaturated = false;
};

DECLARE_DELEGATE_ThreeParams(FOnConsiderListUpdate, const ConsiderListUpdateParams& UpdateParams, int32& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList);

//
// Whether to support net lag and packet loss testing.
//
#define DO_ENABLE_NET_TEST !(UE_BUILD_SHIPPING)

#ifndef NET_DEBUG_RELEVANT_ACTORS
#define NET_DEBUG_RELEVANT_ACTORS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif 

/** Holds the packet simulation settings in one place */
USTRUCT()
struct FPacketSimulationSettings
{
	GENERATED_BODY()

	/**
	 * When set, will cause calls to FlushNet to drop packets.
	 * Value is treated as % of packets dropped (i.e. 0 = None, 100 = All).
	 * No general pattern / ordering is guaranteed.
	 * Clamped between 0 and 100.
	 *
	 * Works with all other settings.
	 */
	UPROPERTY(EditAnywhere, Category="Simulation Settings")
	int32	PktLoss = 0;

	/**
	* Sets the maximum size of packets in bytes that will be dropped
	* according to the PktLoss setting. Default is INT_MAX.
	*
	* Works with all other settings.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktLossMaxSize = 0;

	/**
	* Sets the minimum size of packets in bytes that will be dropped
	* according to the PktLoss setting. Default is 0.
	*
	* Works with all other settings.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktLossMinSize = 0;

	/**
	 * When set, will cause calls to FlushNet to change ordering of packets at random.
	 * Value is treated as a bool (i.e. 0 = False, anything else = True).
	 * This works by randomly selecting packets to be delayed until a subsequent call to FlushNet.
	 *
	 * Takes precedence over PktDup and PktLag.
	 */
	UPROPERTY(EditAnywhere, Category="Simulation Settings")
	int32	PktOrder = 0;

	/**
	 * When set, will cause calls to FlushNet to duplicate packets.
	 * Value is treated as % of packets duplicated (i.e. 0 = None, 100 = All).
	 * No general pattern / ordering is guaranteed.
	 * Clamped between 0 and 100.
	 *
	 * Cannot be used with PktOrder or PktLag.
	 */
	UPROPERTY(EditAnywhere, Category="Simulation Settings")
	int32	PktDup = 0;
	
	/**
	 * When set, will cause calls to FlushNet to delay packets.
	 * Value is treated as millisecond lag.
	 *
	 * Cannot be used with PktOrder.
	 */
	UPROPERTY(EditAnywhere, Category="Simulation Settings")
	int32	PktLag = 0;
	
	/**
	 * When set, will cause PktLag to use variable lag instead of constant.
	 * Value is treated as millisecond lag range (e.g. -GivenVariance <= 0 <= GivenVariance).
	 *
	 * Can only be used when PktLag is enabled.
	 */
	UPROPERTY(EditAnywhere, Category="Simulation Settings")
	int32	PktLagVariance = 0;

	/**
	 * If set lag values will randomly fluctuate between Min and Max.
	 * Ignored if PktLag value is set
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktLagMin = 0;
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktLagMax = 0;

	/**
	 * Set a value to add a minimum delay in milliseconds to incoming
	 * packets before they are processed.
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktIncomingLagMin = 0;
	
	/**
	 * The maximum delay in milliseconds to add to incoming
	 * packets before they are processed.
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktIncomingLagMax = 0;

	/**
	 * The ratio of incoming packets that will be dropped
	 * to simulate packet loss
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktIncomingLoss = 0;

	/**
	 * Causes sent packets to have a variable latency that fluctuates from [PktLagMin] to [PktLagMin+PktJitter]
	 * Note that this will cause packet loss on the receiving end.
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktJitter = 0;

	/**
	 * Delays sending packets for a specific number of ticks
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktFrameDelay = 0;

	/**
	 * Delays processing received packets for a specific number of ticks
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32	PktIncomingFrameDelay = 0;

	/** Time in milliseconds during which outgoing packets will get delayed before being sent all at once. */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32 PktBufferBloatInMS = 0;

	/** Time in milliseconds during which incoming packets will get delayed before being processed all at once. */
	UPROPERTY(EditAnywhere, Category = "Simulation Settings")
	int32 PktIncomingBufferBloatInMS = 0;

	/** reads in settings from the .ini file 
	 * @note: overwrites all previous settings
	 */
	ENGINE_API void LoadConfig(const TCHAR* OptionalQualifier = nullptr);
	
	/** 
	 * Load a preconfigured emulation profile from the .ini
	 * Returns true if the given profile existed
	 */
	ENGINE_API bool LoadEmulationProfile(const TCHAR* ProfileName);

	/**
	 * Force new emulation settings and ignore config or cmdline values
	 */
	ENGINE_API void ApplySettings(const FPacketSimulationSettings& NewSettings);

	/**
	 * Ensure that settings have proper values
	 */
	ENGINE_API void ValidateSettings();
	ENGINE_API void ResetSettings();

	/**
	* Tells if a packet fits the size settings to potentially be dropped
	*/
	bool ShouldDropPacketOfSize(int32 NumBits) const
	{
		const bool bIsBigEnough = NumBits > PktLossMinSize * 8;
		const bool bIsSmallEnough = PktLossMaxSize == 0 || NumBits < PktLossMaxSize * 8;
		return bIsBigEnough && bIsSmallEnough;
	}

	/**
	 * Reads the settings from a string: command line or an exec
	 *
	 * @param Stream the string to read the settings from
	 * @Param OptionalQualifier: optional string to prepend to Pkt* settings. E.g, "GameNetDriverPktLoss=50"
	 */
	ENGINE_API bool ParseSettings(const TCHAR* Stream, const TCHAR* OptionalQualifier=nullptr);

	ENGINE_API bool ParseHelper(const TCHAR* Cmd, const TCHAR* Name, int32& Value, const TCHAR* OptionalQualifier);

	ENGINE_API bool ConfigHelperInt(const TCHAR* Name, int32& Value, const TCHAR* OptionalQualifier);
	ENGINE_API bool ConfigHelperBool(const TCHAR* Name, bool& Value, const TCHAR* OptionalQualifier);
};

struct FActorDestructionInfo
{
public:
	FActorDestructionInfo()
		: Reason(EChannelCloseReason::Destroyed)
		, bIgnoreDistanceCulling(false) 
	{}

	TWeakObjectPtr<ULevel> Level; 
	TWeakObjectPtr<UObject> ObjOuter;
	FVector DestroyedPosition;
	FNetworkGUID NetGUID;
	FString PathName;
	FName StreamingLevelName;
	EChannelCloseReason Reason;

	/** When true the destruction info data will be sent even if the viewers are not close to the actor */
	bool bIgnoreDistanceCulling;

	void CountBytes(FArchive& Ar)
	{
		PathName.CountBytes(Ar);
	}
};

/** Used to configure the replication system default values */
USTRUCT()
struct FNetDriverReplicationSystemConfig
{
	GENERATED_USTRUCT_BODY()

	/** Override the max object count. If 0 use the default system value. */
	UPROPERTY()
	uint32 MaxReplicatedObjectCount = 0;

	/** Override the preallocated size of net object lists. If 0 use the default value. */
	UPROPERTY()
	uint32 InitialNetObjectListCount = 0;

	/** Override the amount to grow every net object list by when they hit the preallocated count. If 0 use the default system value. */
	UPROPERTY()
	uint32 NetObjectListGrowCount = 0;

	/** Override the number of pre-allocated memory buffers that can hold up to the specified number of objects before they have to grow. */
	UPROPERTY()
	uint32 PreAllocatedMemoryBuffersObjectCount = 0;

	/** Override the number of pre-allocated objects in FReplicationWriter. */
	UPROPERTY()
	uint32 MaxReplicationWriterObjectCount = 0;
	
	/** Override the max compressed object count. If 0 use the default system value. */
	UPROPERTY()
	uint32 MaxDeltaCompressedObjectCount = 0;
	
	/** Override the max group count. If 0 use the default system value. */
	UPROPERTY()
	uint32 MaxNetObjectGroupCount = 0;

	/** 
	 * Sets the minimum amount of connected clients before we start executing NetConnection::Tick in parallel.
	 * This limits the overhead of kicking off parallel tasks when concurrency gains would be limited.
	 * Default is to wait for 2 connections.
	 */
	UPROPERTY()
	int32 MinConnectionsForParallelTick = 2;

	/** 
	*   When true, allow subsystems to run parallel workloads, such as the PollAndCopy step running several asynchronous tasks to speed up game thread execution time.
	*   Only supported when bIsServer = true and bAllowObjectReplication = true
	*/
	UPROPERTY()
	bool bAllowParallelTasks = false;
};

//
// Priority sortable list.
//
struct FActorPriority
{
	int32						Priority;	// Update priority, higher = more important.
	
	FNetworkObjectInfo*			ActorInfo;	// Actor info.
	UActorChannel*		        Channel;	// Actor channel.

	FActorDestructionInfo *	DestructionInfo;	// Destroy an actor

	FActorPriority() : 
		Priority(0), ActorInfo(NULL), Channel(NULL), DestructionInfo(NULL)
	{}

	ENGINE_API FActorPriority(class UNetConnection* InConnection, UActorChannel* InChannel, FNetworkObjectInfo* InActorInfo, const TArray<struct FNetViewer>& Viewers, bool bLowBandwidth);
	ENGINE_API FActorPriority(class UNetConnection* InConnection, FActorDestructionInfo * DestructInfo, const TArray<struct FNetViewer>& Viewers );
};

struct FCompareFActorPriority
{
	inline bool operator()( const FActorPriority& A, const FActorPriority& B ) const
	{
		return B.Priority < A.Priority;
	}
};

/** Used to specify properties of a channel type */
USTRUCT()
struct FChannelDefinition
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName ChannelName;			// Channel type identifier

	UPROPERTY()
	FName ClassName;			// UClass name used to create the UChannel

	UPROPERTY()
	TObjectPtr<UClass> ChannelClass;		// UClass used to create the UChannel

	UPROPERTY()
	int32 StaticChannelIndex;	// Channel always uses this index, INDEX_NONE if dynamically chosen

	UPROPERTY()
	uint8 bTickOnCreate : 1;			// Whether to immediately begin ticking the channel after creation

	UPROPERTY()
	uint8 bServerOpen : 1;			// Channel opened by the server

	UPROPERTY()
	uint8 bClientOpen : 1;			// Channel opened by the client

	UPROPERTY()
	uint8 bInitialServer : 1;		// Channel created on server when connection is established

	UPROPERTY()
	uint8 bInitialClient : 1;		// Channel created on client before connecting

	FChannelDefinition() : 
		ChannelName(NAME_None),
		ClassName(NAME_None),
		ChannelClass(nullptr),
		StaticChannelIndex(INDEX_NONE),
		bTickOnCreate(false),
		bServerOpen(false),
		bClientOpen(false),
		bInitialServer(false),
		bInitialClient(false)
	{
	}
};

/**
 * Information about disconnected client NetConnection's
 */
struct FDisconnectedClient
{
	/** The address of the client */
	TSharedRef<const FInternetAddr>	Address;

	/** The time at which the client disconnected  */
	double						DisconnectTime;

	FDisconnectedClient(TSharedRef<const FInternetAddr>& InAddress, double InDisconnectTime)
		: Address(InAddress)
		, DisconnectTime(InDisconnectTime)
	{
	}
};

enum class EProcessRemoteFunctionFlags : uint32
{
	None = 0,
	ReplicatedActor = 1 << 0,	//! The owning actor has been replicated at least once
								//! while processing the remote function.
};
ENUM_CLASS_FLAGS(EProcessRemoteFunctionFlags);

UCLASS(Abstract, customConstructor, transient, MinimalAPI, config=Engine)
class UNetDriver : public UObject, public FExec
{
	GENERATED_UCLASS_BODY()

protected:

	virtual ENGINE_API void InternalProcessRemoteFunction(
		class AActor* Actor,
		class UObject* SubObject,
		class UNetConnection* Connection,
		class UFunction* Function,
		void* Parms,
		FOutParmRec* OutParms,
		FFrame* Stack,
		bool bIsServer);

private:

	void InternalProcessRemoteFunctionPrivate(
		class AActor* Actor,
		class UObject* SubObject,
		class UNetConnection* Connection,
		class UFunction* Function,
		void* Parms,
		FOutParmRec* OutParms,
		FFrame* Stack,
		const bool bIsServer,
		EProcessRemoteFunctionFlags& RemoteFunctionFlags);

public:

	/** Destructor */
	ENGINE_API virtual ~UNetDriver();

	/** Used to specify the class to use for connections */
	UPROPERTY(Config)
	FString NetConnectionClassName;

	UPROPERTY(Config)
	FString ReplicationDriverClassName;

	/** Used to specify the class to use for ReplicationBridge */
	UPROPERTY(Config)
	FString ReplicationBridgeClassName;
	
	/** Can be used to configure Server settings for the ReplicationSystem */
	UPROPERTY(Config)
	FNetDriverReplicationSystemConfig ReplicationSystemConfigServer;

	/** Can be used to configure Client settings for the ReplicationSystem */
	UPROPERTY(Config)
	FNetDriverReplicationSystemConfig ReplicationSystemConfigClient;

	/** @todo document */
	UPROPERTY(Config)
	int32 MaxDownloadSize;

	/** @todo document */
	UPROPERTY(Config)
	uint32 bClampListenServerTickRate:1;

	/** 
	* Limit the tick rate of the engine when running in dedicated server mode. 
	* @see UGameEngine::GetMaxTickRate 
	*/
	UE_DEPRECATED(5.3, "Variable will be made private. Use GetNetServerMaxTickRate and SetNetServerMaxTickRate instead.")
	UPROPERTY(Config)
	int32 NetServerMaxTickRate;

	/** The current max tick rate of the engine when running in dedicated server mode. */
	int32 GetNetServerMaxTickRate() const 
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NetServerMaxTickRate;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Override the configured server tick rate. Value is in ticks per second. */
	ENGINE_API void SetNetServerMaxTickRate(int32 InServerMaxTickRate);

	/** Change the netdriver to have runtime features compatible with the remote connection. */
	void TryUpgradeNetworkFeatures(EEngineNetworkRuntimeFeatures RemoteFeatures);

	/** 
	* Delegate triggered when SetNetServerMaxTickRate is called and causes a change to the current max tick rate.
	* @param UNetDriver The netdriver that changed max tick rate.
	* @param int32 The new value of NetServerMaxTickRate
	* @param int32 The old value of NetServerMaxTickRate 
	*/
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnNetServerMaxTickRateChanged, UNetDriver*, int32, int32);
	FOnNetServerMaxTickRateChanged OnNetServerMaxTickRateChanged;

	/** Limit tick rate of replication to allow very high frame rates to still replicate data. A value less or equal to zero means use the engine tick rate. A value greater than zero will clamp the net tick rate to this value.  */
	UPROPERTY(Config)
	int32 MaxNetTickRate;

	/** @todo document */
	UPROPERTY(Config)
	int32 MaxInternetClientRate;

	/** @todo document */
	UPROPERTY(Config)
	int32 MaxClientRate;

	/** Amount of time a server will wait before traveling to next map, gives clients time to receive final RPCs on existing level @see NextSwitchCountdown */
	UPROPERTY(Config)
	float ServerTravelPause;

	/** @todo document */
	UPROPERTY(Config)
	float SpawnPrioritySeconds;

	/** 
	 * Amount of time (in seconds) before we close the channel of a replicated actor when it goes out of relevancy. Ex: If the client moves farther then the actor's netculldistance, the actor will still be replicated to that client for the duration of the RelevantTimeout.
	 * Default value is 5 seconds
	 * Note: ReplicationGraph and Iris do not use this value directly and instead have their unique config.
	 */
	UPROPERTY(Config)
	float RelevantTimeout;

	/** @todo document */
	UPROPERTY(Config)
	float KeepAliveTime;

	/** Amount of time to wait for a new net connection to be established before destroying the connection */
	UPROPERTY(Config)
	float InitialConnectTimeout;

	/** 
	 * Amount of time to wait before considering an established connection timed out.  
	 * Typically shorter than the time to wait on a new connection because this connection
	 * should already have been setup and any interruption should be trapped quicker.
	 */
	UPROPERTY(Config)
	float ConnectionTimeout;

	/** 
	 * Amount of time to wait for a graceful close/bPendingDestroy to complete before considering the connection timed out.  
	 * This is the time used to allow any existing, pending reliable data to be acknowledged.
	 */
	UPROPERTY(Config)
	float GracefulCloseConnectionTimeout = 2.0f;

	/**
	* A multiplier that is applied to the above values when we are running with unoptimized builds (debug)
	* or data (uncooked). This allows us to retain normal timeout behavior while debugging without resorting
	* to the nuclear 'notimeouts' option or bumping the values above. If ==0 multiplier = 1
	*/
	UPROPERTY(Config)
	float TimeoutMultiplierForUnoptimizedBuilds;

	/** Cached at the beginning of net driver TickFlush step so that parallel threads can access it safely */
	int32 CachedEngineMaxTickRate = 0;

	/** Connection to the server (this net driver is a client) */
	UPROPERTY()
	TObjectPtr<class UNetConnection> ServerConnection;

	/** Array of connections to clients (this net driver is a host) - unsorted, and ordering changes depending on actor replication */
	UPROPERTY()
	TArray<TObjectPtr<UNetConnection>> ClientConnections;

	/**
	 * Map of IP's to NetConnection's - for fast lookup, particularly under DDoS.
	 * Only valid IP's mapped (e.g. excludes DemoNetConnection). Recently disconnected clients remain mapped as nullptr connections.
	 */
	FConnectionMap MappedClientConnections;

	/** Tracks recently disconnected client IP's, and the disconnect time - so they can be cleaned from MappedClientConnections */
	TArray<FDisconnectedClient> RecentlyDisconnectedClients;

	/** The amount of time, in seconds, that recently disconnected clients should be tracked */
	UPROPERTY(Config)
	int32 RecentlyDisconnectedTrackingTime;


	/** Serverside PacketHandler for managing connectionless packets */
	TUniquePtr<PacketHandler> ConnectionlessHandler;

	/** Reference to the PacketHandler component, for managing stateless connection handshakes */
	TWeakPtr<StatelessConnectHandlerComponent> StatelessConnectComponent;

	/** The analytics provider used by the packet handler */
	TSharedPtr<IAnalyticsProvider> AnalyticsProvider;

	/** Special analytics aggregator tied to AnalyticsProvider - combines analytics from all NetConnections/PacketHandlers, in one event */
	TSharedPtr<FNetAnalyticsAggregator> AnalyticsAggregator;

	/** World this net driver is associated with */
	UPROPERTY()
	TObjectPtr<class UWorld> World;

	UPROPERTY()
	TObjectPtr<class UPackage> WorldPackage;

	/** @todo document */
	//UE_DEPRECATED(5.6, "Variable will be made private. Use GetNetGuidCache() methods instead.")
	TSharedPtr<class FNetGUIDCache> GuidCache;

	TSharedPtr< class FClassNetCacheMgr >	NetCache;

	/** The loaded UClass of the net connection type to use */
	UPROPERTY()
	TObjectPtr<UClass> NetConnectionClass;

	UPROPERTY()
	TObjectPtr<UClass> ChildNetConnectionClass;

	UPROPERTY()
	TObjectPtr<UClass> ReplicationDriverClass;

	UPROPERTY(transient)
	TObjectPtr<UClass> ReplicationBridgeClass;

	/** @todo document */
	FProperty* RoleProperty;
	
	/** @todo document */
	FProperty* RemoteRoleProperty;

	/** Used to specify the net driver to filter actors with (NAME_None || NAME_GameNetDriver is the default net driver) */
	UPROPERTY(Config)
	FName NetDriverName;

	/** Used to specify available channel types and their associated UClass */
	UPROPERTY(Config)
	TArray<FChannelDefinition> ChannelDefinitions;

	/** Used for faster lookup of channel definitions by name. */
	UPROPERTY()
	TMap<FName, FChannelDefinition> ChannelDefinitionMap;

	/** @return true if the specified channel definition exists. */
	inline bool IsKnownChannelName(const FName& ChName) const
	{
		return ChannelDefinitionMap.Contains(ChName);
	}

	/** Creates a channel of each type that is set as bInitialClient. */
	ENGINE_API void CreateInitialClientChannels();

	/** Creates a channel of each type that is set as bIniitalServer for the given connection. */
	ENGINE_API void CreateInitialServerChannels(UNetConnection* ClientConnection);
	
	/** 
	 * Utility function that adds an object and, if an actor or component, their sub-objects to the driver's NetGUID cache.
	 * 
	 * Only currently supports generic replication.
	 */
	ENGINE_API void AddObjectToNetworkIdCache(UObject* Object);

	/** 
	 * Utility function that removes an object and, if an actor or component, their sub-objects from the driver's NetGUID cache.
	 *
	 * Only currently supports generic replication.
	 */
	ENGINE_API void RemoveObjectFromNetworkIdCache(UObject* Object);

protected:
	/** 
	 * Register each metric used by the networking system. 
	 *
	 * If a sub-class needs to create new metrics, those metrics should be added to the metrics database in the sub-class
	 * implementation of this function. Metrics should only be added to the database after calling Super::SetupNetworkMetrics().
	 */
	ENGINE_API virtual void SetupNetworkMetrics();

private:

	/** List of channels that were previously used and can be used again */
	UPROPERTY()
	TArray<TObjectPtr<UChannel>> ActorChannelPool;

	/** Name of net driver definition used to create this driver */
	FName NetDriverDefinition;

	/** Cached copy of MaxChannelsOverride from the net driver definition to avoid extra lookups */
	int32 MaxChannelsOverride;

	/** A metrics database that holds statistics calcluated by the networking system. */
	UPROPERTY()
	TObjectPtr<UNetworkMetricsDatabase> NetworkMetricsDatabase;

	/** A cache of UNetworkMetricsBaseListener sub-class instances provided by the *.ini file (one instance per sub-class). */
	UPROPERTY()
	TMap<FName, TObjectPtr<UNetworkMetricsBaseListener>> NetworkMetricsListeners;

	/** Register metric listeners provided by the *.ini file. */
	void SetupNetworkMetricsListeners(bool bIsClient);

	/** Create an instance of UNetworkMetricsStats that is associated with a given Stat and cached with other listeners in NetworkMetricsListeners. */
	void RegisterStatsListener(const FName MetricName, const FName StatName);

	/** Reset any network metrics database values at the beginning of a frame. */
	void ResetNetworkMetrics();

public:
	/** Delegate that will notify when an actors NetUpdateFrequency UPROPERTY has changed. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNetUpdateFrequencyChanged, const AActor* Actor);
	FOnNetUpdateFrequencyChanged& GetOnNetUpdateFrequencyChanged() { return OnNetUpdateFrequencyChanged; };

private:
	FOnNetUpdateFrequencyChanged OnNetUpdateFrequencyChanged;

public:
	/** Get the value of MaxChannelsOverride cached from the net driver definition */
	int32 GetMaxChannelsOverride() const { return MaxChannelsOverride; }

	/** Creates a new channel of the specified type name. If the type is pooled, it will return a pre-created channel */
	UChannel* GetOrCreateChannelByName(const FName& ChName);

	/** If the channel's type is pooled, this will add the channel to the pool. Otherwise, nothing will happen. */
	void ReleaseToChannelPool(UChannel* Channel);

	/** Change the NetDriver's NetDriverName. This will also reinit packet simulation settings so that settings can be qualified to a specific driver. */
	void SetNetDriverName(FName NewNetDriverNamed);

	/** Set the NetDriver's NetDriverDefinition. */
	void SetNetDriverDefinition(FName NewNetDriverDefinition);

	/** Get the NetDriver's NetDriverDefinition. */
	FName GetNetDriverDefinition() const { return NetDriverDefinition; }

	/** Callback after the engine created the NetDriver and set our name for the first time */
	ENGINE_API void PostCreation(bool bInitializeWithIris, bool bEnableParallelNetTick);

	/** Reset some NetDriver settings after destroying and recreating a replication system or replication driver */
	ENGINE_API void ReinitBase();

	void InitPacketSimulationSettings();

	/** Returns true during the duration of a packet loss burst triggered by the net.pktlossburst command. */
#if DO_ENABLE_NET_TEST
	bool IsSimulatingPacketLossBurst() const;
#endif

	/** Interface for communication network state to others (ie World usually, but anything that implements FNetworkNotify) */
	class FNetworkNotify*		Notify;
	
	double GetElapsedTime() const { return ElapsedTime; }
	void ResetElapsedTime() { ElapsedTime = 0.0; }

	/** Trigger the destruction of this NetDriver. If called while the NetDriver is ticking it will queue the destruction request and execute it after all the NetDriver work is finished. */
	ENGINE_API void RequestNetDriverDestruction();

	/** True only inside one of the NetDriver's tick function. */
	bool IsInTick() const { return bInTick; }

	/** Is a destruction request queued to be executed later. */
	bool GetPendingDestruction() const { return bPendingDestruction; }

	UE_DEPRECATED(5.6, "SetPendingDestruction will be made private. Use RequestNetDriverDestruction instead")
	ENGINE_API void SetPendingDestruction(bool bDestroy);

	/**
	 * Get const GuidCache
	 * Note: GuidCache/FNetworkGUIDs are not used when using Iris replication.
	*/
	const TSharedPtr<class FNetGUIDCache>& GetNetGuidCache() const
	{
		// Disabled until we have addressed the issues found.
		//ensureMsgf(!IsUsingIrisReplication(), TEXT("Using NetGuidCache/FNetworkGUIDs is not supported when using Iris replication."));
	
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GuidCache;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	* Get const GuidCache
	* Note: GuidCache/FNetworkGUIDs are not used when using Iris replication.
	*/
	TSharedPtr<class FNetGUIDCache>& GetNetGuidCache()
	{
		// Disabled until we have addressed the issues found.
		//ensureMsgf(!IsUsingIrisReplication(), TEXT("Using NetGuidCache/FNetworkGUIDs is not supported using Iris replication."));
	
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GuidCache;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if UE_SUPPORT_PARALLEL_IRIS
	void LockNetDriverStats(const bool bWriteLock)
	{
		if (bWriteLock)
		{
			NetDriverStatsMutex.WriteLock();
		}
		else
		{
			NetDriverStatsMutex.ReadLock();
		}
	}

	void UnlockNetDriverStats(const bool bWriteLock)
	{
		if (bWriteLock)
		{
			NetDriverStatsMutex.WriteUnlock();
		}
		else
		{
			NetDriverStatsMutex.ReadUnlock();
		}
	}
#endif // UE_SUPPORT_PARALLEL_IRIS

private:
#if UE_SUPPORT_PARALLEL_IRIS
	/** Used by NetConnection to protect the public stats variables */
	FTransactionallySafeRWLock NetDriverStatsMutex;
#endif // UE_SUPPORT_PARALLEL_IRIS

	/** Accumulated delta time since the creation of the NetDriver */
	double ElapsedTime;

	/** Whether or not the NetDriver is ticking */
	bool bInTick;

	uint8 bPendingDestruction : 1;

#if DO_ENABLE_NET_TEST
	/** Dont load packet settings from config or cmdline when true*/
	uint8 bForcedPacketSettings : 1;
#endif 

	uint8 bDidHitchLastFrame : 1;

	/** cache whether or not we have a replay connection, updated when a connection is added or removed */
	uint8 bHasReplayConnection : 1;

protected:
	uint8 bMaySendProperties : 1;

	uint8 bSkipServerReplicateActors : 1;

	uint8 bSkipClearVoicePackets : 1;

public:
	/**
	 * If true, ignore timeouts completely.  Should be used only in development
	 */
	UPROPERTY(Config)
	uint8 bNoTimeouts : 1;

	/**
	 * If true this NetDriver will not apply the network emulation settings that simulate
	 * latency and packet loss in non-shippable builds
	 */
	UPROPERTY(Config)
	uint8 bNeverApplyNetworkEmulationSettings : 1;

	/** If true then client connections are to other client peers */
	uint8						bIsPeer : 1;
	/** @todo document */
	uint8						ProfileStats : 1;
	/** If true, it assumes the stats are being set by server data */
	uint8						bSkipLocalStats : 1;
	/** Collect net stats even if not FThreadStats::IsCollectingData(). */
	uint8 bCollectNetStats : 1;
	/** Used to determine if checking for standby cheats should occur */
	uint8						bIsStandbyCheckingEnabled : 1;
	/** Used to determine whether we've already caught a cheat or not */
	uint8						bHasStandbyCheatTriggered : 1;

	/** Last realtime a tick dispatch occurred. Used currently to try and diagnose timeout issues */
	double						LastTickDispatchRealtime;
	/** Timings for Socket::SendTo() */
	int32						SendCycles;
	/** Stats for network perf */
	uint32						InBytesPerSecond;
	/** todo document */
	uint32						OutBytesPerSecond;
	/** todo document */
	uint32						InBytes;
	/** Total bytes in packets received since the net driver's creation */
	uint32						InTotalBytes;
	/** todo document */
	uint32						OutBytes;
	/** Total bytes in packets sent since the net driver's creation */
	uint32						OutTotalBytes;
	/** Outgoing rate of NetGUID Bunches */
	uint32						NetGUIDOutBytes;
	/** Incoming rate of NetGUID Bunches */
	uint32						NetGUIDInBytes;
	/** The number of packets per second that have been received. */
	uint32						InPackets;
	/** The number of packets that were received in the previous frame. */
	uint32 						PrevInPackets;
	/** Total packets received since the net driver's creation  */
	uint32						InTotalPackets;
	/** The number of packets per second that have been sent. */
	uint32						OutPackets;
	/** The number of packets that were sent in the previous frame. */
	uint32 						PrevOutPackets;
	/** Total packets sent since the net driver's creation  */
	uint32						OutTotalPackets;
	/** todo document */
	uint32						InBunches;
	/** todo document */
	uint32						OutBunches;
	/** Total bunches received since the net driver's creation  */
	uint32						InTotalBunches;
	/** Total bunches sent since the net driver's creation  */
	uint32						OutTotalBunches;
	/** Total number of outgoing reliable bunches */
	uint32						OutTotalReliableBunches;
	/** Total number of incoming reliable bunches */
	uint32						InTotalReliableBunches;
	/** The percentage of incoming packets that have been found to be lost. */
	uint32						InPacketsLost;
	/** The number of incoming packets that were found to be lost in the previous frame. */
	uint32 						PrevInPacketsLost;
	/** Total packets lost that have been sent by clients since the net driver's creation  */
	uint32						InTotalPacketsLost;
	/** The percentage of outgoing packets that have been found to be lost. */
	uint32						OutPacketsLost;
	/** The number of outgoing packets that were found to be lost in the previous frame.  */
	uint32 						PrevOutPacketsLost;
	/** Total packets lost that have been sent by the server since the net driver's creation  */
	uint32						OutTotalPacketsLost;
	/** Tracks the total number of voice packets sent */
	uint32						VoicePacketsSent;
	/** Tracks the total number of voice bytes sent */
	uint32						VoiceBytesSent;
	/** Tracks the total number of voice packets received */
	uint32						VoicePacketsRecv;
	/** Tracks the total number of voice bytes received */
	uint32						VoiceBytesRecv;
	/** Tracks the voice data percentage of in bound bytes */
	uint32						VoiceInPercent;
	/** Tracks the voice data percentage of out bound bytes */
	uint32						VoiceOutPercent;
	/** Time of last stat update */
	double						StatUpdateTime;
	/** Interval between gathering stats */
	float						StatPeriod;
	/** Total RPCs called since the net driver's creation  */
	uint32						TotalRPCsCalled;
	/** Total acks sent since the net driver's creation  */
	uint32						OutTotalAcks;

	/** Time of last netdriver cleanup pass */
	double						LastCleanupTime;
	/** The amount of time without packets before triggering the cheat code */
	float						StandbyRxCheatTime;
	/** todo document */
	float						StandbyTxCheatTime;
	/** The point we think the host is cheating or shouldn't be hosting due to crappy network */
	int32						BadPingThreshold;
	/** The number of clients missing data before triggering the standby code */
	float						PercentMissingForRxStandby;
	float						PercentMissingForTxStandby;
	/** The number of clients with bad ping before triggering the standby code */
	float						PercentForBadPing;
	/** The amount of time to wait before checking a connection for standby issues */
	float						JoinInProgressStandbyWaitTime;
	/** Used to track whether a given actor was replicated by the net driver recently */
	int32						NetTag;

#if NET_DEBUG_RELEVANT_ACTORS
	/** Dumps next net update's relevant actors when true*/
	bool						DebugRelevantActors;

	/** These are debug list of actors. They are using TWeakObjectPtr so that they do not affect GC performance since they are rarely in use (DebugRelevantActors) */
	TArray< TWeakObjectPtr<AActor> >	LastPrioritizedActors;
	TArray< TWeakObjectPtr<AActor> >	LastRelevantActors;
	TArray< TWeakObjectPtr<AActor> >	LastSentActors;
	TArray< TWeakObjectPtr<AActor> >	LastNonRelevantActors;

	void						PrintDebugRelevantActors();
#endif // NET_DEBUG_RELEVANT_ACTORS
	
	/** The server adds an entry into this map for every actor that is destroyed that join-in-progress
	 *  clients need to know about, that is, startup actors. Also, individual UNetConnections
	 *  need to keep track of FActorDestructionInfo for dormant and recently-dormant actors in addition
	 *  to startup actors (because they won't have an associated channel), and this map stores those
	 *  FActorDestructionInfos also.
	 */
	TMap<FNetworkGUID, TUniquePtr<FActorDestructionInfo>>	DestroyedStartupOrDormantActors;

private:

	/** Tracks the network guids in DestroyedStartupOrDormantActors above, but keyed on the streaming level name. */
	TMap<FName, TSet<FNetworkGUID>> DestroyedStartupOrDormantActorsByLevel;

	/** Cached list of analytic attributes that can be appended via FNetAnalyticsAggregator::AppendGameInstanceAttributes */
	TMap<FString, FString> CachedNetAnalyticsAttributes;

	/** Maps an object to the respective FReplicationChangelistMgr */
	TMap<FObjectKey, TSharedPtr<FReplicationChangelistMgr>>	ReplicationChangeListMap;

public:

	const TSet<FNetworkGUID>& GetDestroyedStartupOrDormantActors(const FName& LevelName)
	{
		return DestroyedStartupOrDormantActorsByLevel.FindOrAdd(LevelName);
	}

	/** 
	 * Add or overwrite an analytics attribute that will be appended via FNetAnalyticsAggregator::AppendGameInstanceAttributes
	 * Useful to add game specific attributes to your analytics.
	 * @param AttributeKey The key of the attribute. Stored in a case-insensitive map
	 * @param AttributeValue Value of the attribute to store. Only supports strings.
	 */
	void SetNetAnalyticsAttributes(const FString& AttributeKey, const FString& AttributeValue) { CachedNetAnalyticsAttributes.Add(AttributeKey, AttributeValue); }

	/** The server adds an entry into this map for every startup actor that has been renamed, and will
	 *  always map from current name to original name
	 */
	TMap<FName, FName>	RenamedStartupActors;

	/** Used to invalidate properties marked "unchanged" in FRepChangedPropertyTracker's */
	uint32 ReplicationFrame;

	/** Maps FRepLayout to the respective UClass */
	TMap<TWeakObjectPtr<UObject>, TSharedPtr<FRepLayout>, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UObject>, TSharedPtr<FRepLayout> > >	RepLayoutMap;

	/** Creates if necessary, and returns a FRepLayout that maps to the passed in UClass */
	TSharedPtr< FRepLayout >	GetObjectClassRepLayout( UClass * InClass );

	/** Creates if necessary, and returns a FRepLayout that maps to the passed in UFunction */
	ENGINE_API TSharedPtr<FRepLayout> GetFunctionRepLayout( UFunction * Function );

	/** Creates if necessary, and returns a FRepLayout that maps to the passed in UStruct */
	ENGINE_API TSharedPtr<FRepLayout> GetStructRepLayout( UStruct * Struct );

	/**
	 * Returns the FReplicationChangelistMgr that is associated with the passed in object,
	 * creating one if none exist.
	 *
	 * This should **never** be called on client NetDrivers!
	 */
	TSharedPtr< FReplicationChangelistMgr > GetReplicationChangeListMgr( UObject* Object );

	TMap< FNetworkGUID, TSet< FObjectReplicator* > >	GuidToReplicatorMap;
	int32												TotalTrackedGuidMemoryBytes;
	TSet< FObjectReplicator* >							UnmappedReplicators;
	TSet< FObjectReplicator* >							AllOwnedReplicators;

	/** Handles to various registered delegates */
	FDelegateHandle TickDispatchDelegateHandle;
	FDelegateHandle PostTickDispatchDelegateHandle;
	FDelegateHandle TickFlushDelegateHandle;
	FDelegateHandle PostTickFlushDelegateHandle;

#if !UE_BUILD_SHIPPING
	/** Delegate for hooking ProcessRemoteFunction */
	FOnSendRPC	SendRPCDel;

	/** Delegate for hooking ShouldSkipRepNotifies */
	FShouldSkipRepNotifies SkipRepNotifiesDel;
#endif

	/** Tracks the amount of time spent during the current frame processing queued bunches. */
	float ProcessQueuedBunchesCurrentFrameMilliseconds;

	/** DDoS detection management */
	FDDoSDetection DDoS;

	/** Local address this net driver is associated with */
	TSharedPtr<FInternetAddr> LocalAddr;

	/**
	* Updates the standby cheat information and
	 * causes the dialog to be shown/hidden as needed
	 */
	void UpdateStandbyCheatStatus(void);

	/** Sets the analytics provider */
	ENGINE_API virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider);

#if DO_ENABLE_NET_TEST
	FPacketSimulationSettings	PacketSimulationSettings;

	/**
	 * Modify the current emulation settings
	 */
	ENGINE_API void SetPacketSimulationSettings(const FPacketSimulationSettings& NewSettings);

	ENGINE_API void OnPacketSimulationSettingsChanged();
#endif

	// Constructors.
	ENGINE_API UNetDriver(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API UNetDriver(FVTableHelper& Helper);


	//~ Begin UObject Interface.
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostReloadConfig(FProperty* PropertyToLoad) override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.

	//~ Begin FExec Interface
protected:
	/**
	 * Handle exec commands
	 *
	 * @param InWorld	the world context
	 * @param Cmd		the exec command being executed
	 * @param Ar		the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	ENGINE_API virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog) override;
//~ End FExec Interface.

public:

	/** Enable/disable the execution of received RPC functions. */
	ENGINE_API void EnableExecuteRPCFunctions(bool bEnable);
	ENGINE_API bool IsExecuteRPCFunctionsEnabled() const;

	/** Enable/disable calls to AActor::PreReplication(). */
	ENGINE_API void EnablePreReplication(bool bEnable);
	ENGINE_API bool IsPreReplicationEnabled() const;

	/** Enable/disable swapping local and remote roles of all actors when replicating actors (overrides calls to SetRoleSwapOnReplicate()). */
	ENGINE_API void EnableRoleSwappingOfAllActors(bool bEnable);
	ENGINE_API bool IsRoleSwappingOnAllActorsEnabled() const;

	/** (Iris replication only) Enable/disable replication of actors that have a local role other than ROLE_Authority. */
	ENGINE_API void EnableReplicationOfActorsWithAnyRole(bool bEnable);
	ENGINE_API bool IsReplicationOfActorsWithAnyRoleEnabled() const;

	/** (Iris replication only) Set the proxy type of the replication system. */
	ENGINE_API void SetProxyType(UE::Net::EProxyType InProxyType);
	ENGINE_API UE::Net::EProxyType GetProxyType() const;

	/** (Generic Replicaton Only) Return true if an actor's role can be downgraded during replication to a given connection.  */
	virtual bool CanDowngradeActorRole(UNetConnection* Connection, AActor* Actor) const { return true; };

private:

	bool bExecuteRPCFunctions = true;
	bool bCallPreReplication = true;
	bool bSwapRolesOfAllActors = false;
	bool bReplicationOfActorsWithAnyRole = false;

	UE::Net::EProxyType ProxyType;

	/** A set of replicators that have queued RPCs for transactions (server only) */
	TSet< FObjectReplicator* > ReplicatorsWithPendingTransactions;

public:

	ENGINE_API ENetMode	GetNetMode() const;

	/** 
	 * Returns true if this net driver is valid for the current configuration.
	 * Safe to call on a CDO if necessary
	 *
	 * @return true if available, false otherwise
	 */
	virtual bool IsAvailable() const PURE_VIRTUAL( UNetDriver::IsAvailable, return false;)

	/**
	 * Common initialization between server and client connection setup
	 * 
	 * @param bInitAsClient are we a client or server
	 * @param InNotify notification object to associate with the net driver
	 * @param URL destination
	 * @param bReuseAddressAndPort whether to allow multiple sockets to be bound to the same address/port
	 * @param Error output containing an error string on failure
	 *
	 * @return true if successful, false otherwise (check Error parameter)
	 */
	ENGINE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error);

	/**
	 * Initialize the net driver in client mode
	 *
	 * @param InNotify notification object to associate with the net driver
	 * @param ConnectURL remote ip:port of host to connect to
	 * @param Error resulting error string from connection attempt
	 * 
	 * @return true if successful, false otherwise (check Error parameter)
	 */
	virtual bool InitConnect(class FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error ) PURE_VIRTUAL( UNetDriver::InitConnect, return true;);

	/**
	 * Initialize the network driver in server mode (listener)
	 *
	 * @param InNotify notification object to associate with the net driver
	 * @param ListenURL the connection URL for this listener
	 * @param bReuseAddressAndPort whether to allow multiple sockets to be bound to the same address/port
	 * @param Error out param with any error messages generated 
	 *
	 * @return true if successful, false otherwise (check Error parameter)
	 */
	virtual bool InitListen(class FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error) PURE_VIRTUAL( UNetDriver::InitListen, return true;);

	/** Initialize the list of destroyed net startup actors from the current World */
	ENGINE_API virtual void InitDestroyedStartupActors();

	/**
	 * Initialize a PacketHandler for serverside net drivers, for handling connectionless packets
	 * NOTE: Only triggered by net driver subclasses that support it - from within InitListen.
	 */
	ENGINE_API virtual void InitConnectionlessHandler();

	/**
	 * Flushes all packets queued by the connectionless PacketHandler
	 * NOTE: This should be called shortly after all calls to PacketHandler::IncomingConnectionless, to minimize packet buffer buildup.
	 */
	ENGINE_API virtual void FlushHandler();

	/** Initializes the net connection class to use for new connections */
	ENGINE_API virtual bool InitConnectionClass(void);

	/** Initialized the replication driver class to use for this driver */
	ENGINE_API virtual bool InitReplicationDriverClass();

	/** Initialized the replication bridge class to use for this driver if using iris replication*/
	ENGINE_API virtual bool InitReplicationBridgeClass();
	/** Shutdown all connections managed by this net driver */
	ENGINE_API virtual void Shutdown();

	/* Close socket and Free the memory the OS allocated for this socket */
	ENGINE_API virtual void LowLevelDestroy();

	/* @return network number */
	ENGINE_API virtual FString LowLevelGetNetworkNumber();

	/* @return local addr of this machine if set */
	virtual TSharedPtr<const FInternetAddr> GetLocalAddr() { return LocalAddr; }

	/** Make sure this connection is in a reasonable state. */
	ENGINE_API virtual void AssertValid();

	/**
	 * Called to replicate any relevant actors to the connections contained within this net driver
	 *
	 * Process as many clients as allowed given Engine.NetClientTicksPerSecond, first building a list of actors to consider for relevancy checking,
	 * and then attempting to replicate each actor for each connection that it is relevant to until the connection becomes saturated.
	 *
	 * NetClientTicksPerSecond is used to throttle how many clients are updated each frame, hoping to avoid saturating the server's upstream bandwidth, although
	 * the current solution is far from optimal.  Ideally the throttling could be based upon the server connection becoming saturated, at which point each
	 * connection is reduced to priority only updates, and spread out amongst several ticks.  Also might want to investigate eliminating the redundant consider/relevancy
	 * checks for Actors that were successfully replicated for some channels but not all, since that would make a decent CPU optimization.
	 *
	 * @param DeltaSeconds elapsed time since last call
	 *
	 * @return the number of actors that were replicated
	 */
	ENGINE_API virtual int32 ServerReplicateActors(float DeltaSeconds);

	/**
	 * Process a remote function call on some actor destined for a remote location
	 *
	 * @param Actor actor making the function call
	 * @param Function function definition called
	 * @param Parms parameters in a UObject memory layout
	 * @param OutParms out parameter information (irrelevant for RPC's)
	 * @param Stack stack frame the UFunction is called in
	 * @param SubObject optional: sub object to actually call function on
	 */
	ENGINE_API virtual void ProcessRemoteFunction(class AActor* Actor, class UFunction* Function, void* Parms, struct FOutParmRec* OutParms, struct FFrame* Stack, class UObject* SubObject = nullptr );

	/** Return a reference to the database that holds metrics calcluated by the networking system. */
	TObjectPtr<UNetworkMetricsDatabase> GetMetrics() { return NetworkMetricsDatabase; };

	enum class ERemoteFunctionSendPolicy
	{		
		/** Unreliable multicast are queued. Everything else is send immediately */
		Default, 

		/** Bunch is send immediately no matter what */
		ForceSend,

		/** Bunch is queued until next actor replication, no matter what */
		ForceQueue,
	};
	UE_DEPRECATED(5.4, "Use fully scoped enum class value UNetDriver::ERemoteFunctionSendPolicy::Default")
	static constexpr ERemoteFunctionSendPolicy Default = ERemoteFunctionSendPolicy::Default;
	UE_DEPRECATED(5.4, "Use fully scoped enum class value UNetDriver::ERemoteFunctionSendPolicy::ForceSend")
	static constexpr ERemoteFunctionSendPolicy ForceSend = ERemoteFunctionSendPolicy::ForceSend;
	UE_DEPRECATED(5.4, "Use fully scoped enum class value UNetDriver::ERemoteFunctionSendPolicy::ForceQueue")
	static constexpr ERemoteFunctionSendPolicy ForceQueue = ERemoteFunctionSendPolicy::ForceQueue;

	/** Process a remote function on given actor channel. This is called by ::ProcessRemoteFunction.*/
	ENGINE_API void ProcessRemoteFunctionForChannel(
		UActorChannel* Ch,
		const class FClassNetCache* ClassCache,
		const FFieldNetCache* FieldCache,
		UObject* TargetObj,
		UNetConnection* Connection,
		UFunction* Function,
		void* Parms,
		FOutParmRec* OutParms,
		FFrame* Stack,
		const bool IsServer,
		const ERemoteFunctionSendPolicy SendPolicy = ERemoteFunctionSendPolicy::Default);

	ENGINE_API void ProcessRemoteFunctionForChannel(
		UActorChannel* Ch,
		const class FClassNetCache* ClassCache,
		const FFieldNetCache* FieldCache,
		UObject* TargetObj,
		UNetConnection* Connection,
		UFunction* Function,
		void* Parms,
		FOutParmRec* OutParms,
		FFrame* Stack,
		const bool IsServer,
		const ERemoteFunctionSendPolicy SendPolicy,
		EProcessRemoteFunctionFlags& RemoteFunctionFlags);

private:

	void ProcessRemoteFunctionForChannelPrivate(
		UActorChannel* Ch,
		const class FClassNetCache* ClassCache,
		const FFieldNetCache* FieldCache,
		UObject* TargetObj,
		UNetConnection* Connection,
		UFunction* Function,
		void* Parms,
		FOutParmRec* OutParms,
		FFrame* Stack,
		const bool bIsServer,
		const ERemoteFunctionSendPolicy SendPolicy,
		EProcessRemoteFunctionFlags& RemoteFunctionFlags);

public:

	/** handle time update: read and process packets */
	ENGINE_API virtual void TickDispatch( float DeltaTime );

	/** PostTickDispatch actions */
	ENGINE_API virtual void PostTickDispatch();

	/** ReplicateActors and Flush */
	ENGINE_API virtual void TickFlush(float DeltaSeconds);

	/** PostTick actions */
	ENGINE_API virtual void PostTickFlush();

	/**
	 * Sends a 'connectionless' (not associated with a UNetConection) packet, to the specified address.
	 * NOTE: Address is an abstract format defined by subclasses. Anything calling this, must use an address supplied by the net driver.
	 *
	 * @param Address		The address the packet should be sent to (format is abstract, determined by net driver subclasses)
	 * @param Data			The packet data
	 * @param CountBits		The size of the packet data, in bits
	 * @param Traits		Traits for the packet, if applicable
	 */
	virtual void LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits)
		PURE_VIRTUAL(UNetDriver::LowLevelSend,);

	/**
	 * Process any local talker packets that need to be sent to clients
	 */
	ENGINE_API virtual void ProcessLocalServerPackets();

	/**
	 * Process any local talker packets that need to be sent to the server
	 */
	ENGINE_API virtual void ProcessLocalClientPackets();

	/**
	 * Update the LagState based on a heuristic to determine if we are network lagging
	 */
	ENGINE_API virtual void UpdateNetworkLagState();

	/**
	 * Determines which other connections should receive the voice packet and
	 * queues the packet for those connections. Used for sending both local/remote voice packets.
	 *
	 * @param VoicePacket the packet to be queued
	 * @param CameFromConn the connection this packet came from (NULL if local)
	 */
	ENGINE_API virtual void ReplicateVoicePacket(TSharedPtr<class FVoicePacket> VoicePacket, class UNetConnection* CameFromConn);

#if !UE_BUILD_SHIPPING
	/**
	 * Exec command handlers
	 */
	bool HandleSocketsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandlePackageMapCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleNetFloodCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleNetDebugTextCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleNetDisconnectCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleNetDumpServerRPCCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleNetDumpDormancy(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleDumpSubObjectsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleDumpRepLayoutFlagsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandlePushModelMemCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandlePropertyConditionsMemCommand(const TCHAR* Cmd, FOutputDevice& Ar);
#endif

	ENGINE_API void HandlePacketLossBurstCommand( int32 DurationInMilliseconds );

	// ---------------------------------------------------------------
	//	Game code API for updating server Actor Replication State
	// ---------------------------------------------------------------

	ENGINE_API virtual void ForceNetUpdate(AActor* Actor);

	ENGINE_API void ForceAllActorsNetUpdateTime(float NetUpdateTimeOffset, TFunctionRef<bool(const AActor* const)> ValidActorTestFunc);

	/** Flushes actor from NetDriver's dormancy list, but does not change any state on the Actor itself */
	ENGINE_API void FlushActorDormancy(AActor *Actor, bool bWasDormInitial=false);

	//~ This probably doesn't need to be exported, since it's only called by AActor::SetNetDormancy.

	/** Notifies the NetDriver that the desired Dormancy state for this Actor has changed. */
	ENGINE_API void NotifyActorDormancyChange(AActor* Actor, ENetDormancy OldDormancyState);

	/** Called after an actor channel is opened on a client when the actor was previously dormant. */
	ENGINE_API virtual void NotifyActorClientDormancyChanged(AActor* Actor, ENetDormancy OldDormancyState);

	/** Forces properties on this actor to do a compare for one frame (rather than share shadow state) */
	ENGINE_API void ForcePropertyCompare( AActor* Actor );

	/** Force this actor to be relevant for at least one update */
	ENGINE_API void ForceActorRelevantNextUpdate(AActor* Actor);

	/** Tells the net driver about a networked actor that was spawned */
	ENGINE_API virtual void AddNetworkActor(AActor* Actor);

	/** Called when a spawned actor is destroyed. */
	ENGINE_API virtual void NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel = false, EChannelCloseReason CloseReason = EChannelCloseReason::Destroyed);

	void NotifySubObjectDestroyed(UObject* SubObject);

	/** Called when an actor is renamed. */
	UE_DEPRECATED(5.4, "Replaced by overload that takes the PreviousOuter")
	ENGINE_API virtual void NotifyActorRenamed(AActor* Actor, FName PreviousName);
	
	/** Called when an actor is renamed. */
	ENGINE_API virtual void NotifyActorRenamed(AActor* Actor, UObject* PreviousOuter, FName PreviousName);

	ENGINE_API virtual void RemoveNetworkActor(AActor* Actor);

	/** Called when an authoritative actor wants to delete a replicated subobject on the clients it was already replicated to */
	void DeleteSubObjectOnClients(AActor* Actor, UObject* SubObject);

	/** Called when an authoritative actor wants to tear off a subobject on the clients it was already replicated to */
	void TearOffSubObjectOnClients(AActor* Actor, UObject* SubObject);

	ENGINE_API virtual void NotifyActorLevelUnloaded( AActor* Actor );

	ENGINE_API virtual void NotifyActorTearOff(AActor* Actor);

	/** Called when an actor is about to be carried during a seamless travel */
	ENGINE_API void NotifyActorIsTraveling(AActor* TravelingActor);

	/** Set whether this actor should swap roles before replicating properties. */
	ENGINE_API void SetRoleSwapOnReplicate(AActor* Actor, bool bSwapRoles);

	// ---------------------------------------------------------------
	//
	// ---------------------------------------------------------------	

	ENGINE_API virtual void NotifyStreamingLevelUnload( ULevel* );

	/** creates a child connection and adds it to the given parent connection */
	ENGINE_API virtual class UChildConnection* CreateChild(UNetConnection* Parent);

	/** @return String that uniquely describes the net driver instance */
	FString GetDescription() const
	{ 
		return FString::Printf(TEXT("Name:%s Def:%s %s%s"), *NetDriverName.ToString(), *NetDriverDefinition.ToString(), *GetName(), bIsPeer ? TEXT("(PEER)") : TEXT(""));
	}

	/** @return true if this netdriver is handling accepting connections */
	ENGINE_API virtual bool IsServer() const;

	ENGINE_API virtual void CleanPackageMaps();

	void RemoveClassRepLayoutReferences(UClass* Class);

	ENGINE_API void CleanupWorldForSeamlessTravel();

	ENGINE_API void PreSeamlessTravelGarbageCollect();

	ENGINE_API void PostSeamlessTravelGarbageCollect();

	/**
	 * Get the socket subsytem appropriate for this net driver
	 */
	virtual class ISocketSubsystem* GetSocketSubsystem() PURE_VIRTUAL(UNetDriver::GetSocketSubsystem, return NULL;);

	/**
	 * Associate a world with this net driver. 
	 * Disassociates any previous world first.
	 * 
	 * @param InWorld the world to associate with this netdriver
	 */
	ENGINE_API virtual void SetWorld(class UWorld* InWorld);

	/**
	 * Get the world associated with this net driver
	 */
	virtual class UWorld* GetWorld() const override final { return World; }

	class UPackage* GetWorldPackage() const { return WorldPackage; }

	/** Called during seamless travel to clear all state that was tied to the previous game world (actor lists, etc) */
	ENGINE_API virtual void ResetGameWorldState();

	/** @return true if the net resource is valid or false if it should not be used */
	virtual bool IsNetResourceValid(void) PURE_VIRTUAL(UNetDriver::IsNetResourceValid, return false;);

	bool NetObjectIsDynamic(const UObject *Object) const;

	/** Draws debug markers in the world based on network state */
	void DrawNetDriverDebug();

	/** 
	 * Finds a FRepChangedPropertyTracker associated with an object.
	 * If not found, creates one.
	*/
	TSharedPtr<FRepChangedPropertyTracker> FindOrCreateRepChangedPropertyTracker(UObject *Obj);

	/** Finds a FRepChangedPropertyTracker associated with an object. */
	TSharedPtr<FRepChangedPropertyTracker> FindRepChangedPropertyTracker(UObject* Obj);

	/** Returns true if the client should destroy immediately any actor that becomes torn-off */
	virtual bool ShouldClientDestroyTearOffActors() const { return false; }

	/** Returns whether or not properties that are replicating using this driver should not call RepNotify functions. */
	ENGINE_API virtual bool ShouldSkipRepNotifies() const;

	/** Returns true if actor channels with InGUID should queue up bunches, even if they wouldn't otherwise be queued. */
	virtual bool ShouldQueueBunchesForActorGUID(FNetworkGUID InGUID) const { return false; }

	/** Returns whether or not RPCs processed by this driver should be ignored. */
	virtual bool ShouldIgnoreRPCs() const { return false; }

	/** Returns the existing FNetworkGUID of InActor, if it has one. */
	virtual FNetworkGUID GetGUIDForActor(const AActor* InActor) const { return FNetworkGUID(); }

	/** Returns the actor that corresponds to InGUID, if one can be found. */
	virtual AActor* GetActorForGUID(FNetworkGUID InGUID) const { return nullptr; }

	/** Returns true if RepNotifies should be checked and generated when receiving properties for the given object. */
	virtual bool ShouldReceiveRepNotifiesForObject(UObject* Object) const { return true; }

	/** Returns the object that manages the list of replicated UObjects. */
	FNetworkObjectList& GetNetworkObjectList() { return *NetworkObjects; }

	/** Returns the object that manages the list of replicated UObjects. */
	const FNetworkObjectList& GetNetworkObjectList() const { return *NetworkObjects; }

	/**
     *	Get the network object matching the given Actor.
	 *	If the Actor is not present in the NetworkObjectInfo list, it will be added.
	 */
	UE_DEPRECATED(5.3, "Will be made private in a future release")
	ENGINE_API FNetworkObjectInfo* FindOrAddNetworkObjectInfo(const AActor* InActor);

	/** Get the network object matching the given Actor. */
	UE_DEPRECATED(5.3, "Will be made private in a future release")
	ENGINE_API FNetworkObjectInfo* FindNetworkObjectInfo(const AActor* InActor);

	UE_DEPRECATED(5.3, "Will be made private in a future release")
	const FNetworkObjectInfo* FindNetworkObjectInfo(const AActor* InActor) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return const_cast<UNetDriver*>(this)->FindNetworkObjectInfo(InActor);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Returns whether adaptive net frequency is enabled. If enabled, update frequency is allowed to ramp down to MinNetUpdateFrequency for an actor when no replicated properties have changed.
	 * This is currently controlled by the CVar "net.UseAdaptiveNetUpdateFrequency".
	 */
	ENGINE_API static bool IsAdaptiveNetUpdateFrequencyEnabled();

	/** Returns true if adaptive net update frequency is enabled and the given actor is having its update rate lowered from its standard rate. */
	ENGINE_API bool IsNetworkActorUpdateFrequencyThrottled(const AActor* InActor) const;

	/** Returns true if adaptive net update frequency is enabled and the given actor is having its update rate lowered from its standard rate. */
	UE_DEPRECATED(5.3, "Will be made private in a future release, please use version that takes an actor")
	ENGINE_API bool IsNetworkActorUpdateFrequencyThrottled(const FNetworkObjectInfo& InNetworkActor) const;

	/** Stop adaptive replication for the given actor if it's currently throttled. It maybe be allowed to throttle again later. */
	UE_DEPRECATED(5.3, "Will be made private in a future release, please use version that takes an actor")
	ENGINE_API void CancelAdaptiveReplication(FNetworkObjectInfo& InNetworkActor);

	ENGINE_API void CancelAdaptiveReplication(const AActor* InActor);

	/** Returns true if the driver's world time has exceeded the next replication update time for this actor, or if it is pending replication from a previous frame. */
	ENGINE_API bool IsPendingNetUpdate(const AActor* InActor) const;

	/** Returns the level ID/PIE instance ID for this netdriver to use. */
	int32 GetDuplicateLevelID() const { return DuplicateLevelID; }

	/** Sets the level ID/PIE instance ID for this netdriver to use. */
	void SetDuplicateLevelID(const int32 InDuplicateLevelID) { DuplicateLevelID = InDuplicateLevelID; }

	/** Explicitly sets the ReplicationDriver instance (you instantiate it and initialize it). Shouldn't be done during gameplay: ok to do in GameMode startup or via console commands for testing. Existing ReplicationDriver (if set) is destroyed when this is called.  */
	ENGINE_API void SetReplicationDriver(UReplicationDriver* NewReplicationManager);

	UReplicationDriver* GetReplicationDriver() const { return ReplicationDriver; }

	/** Returns if this netdriver is initialized to replicate using the Iris replication system or the Legacy replication system. */
	inline bool IsUsingIrisReplication() const
	{
		return bIsUsingIris;
	}

	/** Returns the bitflag telling which network features are activated for this NetDriver. */
	ENGINE_API EEngineNetworkRuntimeFeatures GetNetworkRuntimeFeatures() const;

	/**
	 * Destroy and recreate the iris replication system for an active netdriver.
	 * This will re-add all existing replicated actors back in the system.
	 * Useful if you need to reapply hotfix configs downloaded post-initialization.
	 */
	ENGINE_API void RestartIrisSystem();

	/** Destroy the Iris replication system before it gets recreated. */
	ENGINE_API void DestroyIrisSystem();

	/** Create the Iris replication system after it got destroyed. */
	ENGINE_API void RecreateIrisSystem();

	template<class T>
	T* GetReplicationDriver() const { return Cast<T>(ReplicationDriver); }

	inline UReplicationSystem* GetReplicationSystem() { return ReplicationSystem; }
	inline UReplicationSystem* GetReplicationSystem() const { return ReplicationSystem; }

	void UpdateGroupFilterStatusForLevel(const ULevel* Level, UE::Net::FNetObjectGroupHandle LevelGroupHandle);
	void DisallowReplicationOfAllLevelGroupsForConnection(UE::Net::FConnectionHandle Handle);
	void UpdateGroupFilterStatusForAllLevelsForConnection(UE::Net::FConnectionHandle Handle);
	void UpdateGroupFilterStatusForAllLevelsForAllConnections();

	/** Returns NetTokenStore that is required to create and serialize NetTokens */
	const UE::Net::FNetTokenStore* GetNetTokenStore() const { return NetTokenStore.Get(); }
	UE::Net::FNetTokenStore* GetNetTokenStore() { return NetTokenStore.Get(); }
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNetTokenStoreReady, UNetDriver*);

protected:
	/** Return true if this driver should update network stats in UpdateNetworkStats(). */
	ENGINE_API virtual bool ShouldUpdateStats() const;

	/** Return true if this driver should register metrics database listeners based on UNetworkMetricsConfig ini configuration. */
	ENGINE_API virtual bool ShouldRegisterMetricsDatabaseListeners() const;

private:
	FOnNetTokenStoreReady OnNetTokenStoreReadyDelegate;
public:
	/** Fired when the NetTokenStore is ready for use and has been initialized, but before it will be used for any NetSerializations */
	FOnNetTokenStoreReady::RegistrationType& OnNetTokenStoreReady()
	{
		return OnNetTokenStoreReadyDelegate;
	}



	ENGINE_API void RemoveClientConnection(UNetConnection* ClientConnectionToRemove);

	/** Adds (fully initialized, ready to go) client connection to the ClientConnections list + any other game related setup */
	ENGINE_API void	AddClientConnection(UNetConnection * NewConnection);

	/** 
	 * Enumerate through each client connection (including any child connections).
	 * 
	 * Func is a function that is called with the current instance of UNetConnection in the enumeration, and
	 * should return true if the enumeration must stop (i.e. no more calls to Func) or false for the enumeration
	 * to continue.
	 */
	ENGINE_API void ForEachClientConnection(TFunction<bool(TObjectPtr<UNetConnection>)> Func);

	//~ This method should only be called by internal networking systems.
	ENGINE_API void NotifyActorFullyDormantForConnection(AActor* Actor, UNetConnection* Connection);

	/** Returns true if this actor is considered to be in a loaded level */
	ENGINE_API virtual bool IsLevelInitializedForActor( const AActor* InActor, const UNetConnection* InConnection ) const;

	/** Called after processing RPC to track time spent */
	void NotifyRPCProcessed(UFunction* Function, UNetConnection* Connection, double ElapsedTimeSeconds);

	/** Returns true if this network driver will handle the remote function call for the given actor. */
	ENGINE_API virtual bool ShouldReplicateFunction(AActor* Actor, UFunction* Function) const;

	/** Returns true if this network driver will forward a received remote function call to other active net drivers. */
	ENGINE_API virtual bool ShouldForwardFunction(AActor* Actor, UFunction* Function, void* Parms) const;

	/** Returns true if this network driver will replicate the given actor. */
	ENGINE_API virtual bool ShouldReplicateActor(AActor* Actor) const;

	/** Returns true if this network driver should execute this remote call locally. */
	ENGINE_API virtual bool ShouldCallRemoteFunction(UObject* Object, UFunction* Function, const FReplicationFlags& RepFlags) const;

	/** Returns true if clients should destroy the actor when the channel is closed. */
	ENGINE_API virtual bool ShouldClientDestroyActor(AActor* Actor, EChannelCloseReason CloseReason = EChannelCloseReason::Destroyed) const;

	/** Called when an actor channel is remotely opened for an actor. */
	ENGINE_API virtual void NotifyActorChannelOpen(UActorChannel* Channel, AActor* Actor);
	
	/** Called when an actor channel is cleaned up for an actor. */
	ENGINE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason);

	ENGINE_API virtual void NotifyActorTornOff(AActor* Actor);

	/** Called on clients when an actor channel is closed because it went dormant. */
	ENGINE_API virtual void ClientSetActorDormant(AActor* Actor);

	/** Called on clients when an actor is torn off. */
	ENGINE_API virtual void ClientSetActorTornOff(AActor* Actor);

	/**
	 * Returns the current delinquency analytics and resets them.
	 * This would be similar to calls to Get and Reset separately, except that the caller
	 * will assume ownership of data in this case.
	 */
	ENGINE_API void ConsumeAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics& Out);

	/** Returns the current delinquency analytics. */
	ENGINE_API const FNetAsyncLoadDelinquencyAnalytics& GetAsyncLoadDelinquencyAnalytics() const;

	/** Resets the current delinquency analytics. */
	ENGINE_API void ResetAsyncLoadDelinquencyAnalytics();

	inline uint32 AllocateConnectionId() { return ConnectionIdHandler.Allocate(); }
	inline void FreeConnectionId(uint32 Id) { return ConnectionIdHandler.Free(Id); };

	/** Returns the NetConnection associated with the ParentConnectionId. Slow. */
	UE_DEPRECATED(5.6, "Use GetConnectionByHandle()")
	ENGINE_API UNetConnection* GetConnectionById(uint32 ParentConnectionId) const;

	/** Returns the NetConnection associated with the ConnectionHandle. Slow as it iterates over all connections to find one with matching handle. */
	ENGINE_API UNetConnection* GetConnectionByHandle(UE::Net::FConnectionHandle ConnectionHandle) const;

	/** Returns identifier used for NetTrace */
	inline uint32 GetNetTraceId() const { return NetTraceId; }

	/** Is the netdriver allowed to create destruction infos */
	ENGINE_API bool CanCreateDestructionInfo() const;

	/** Sends a message to a client to destroy an actor to the client.  The actor may already be destroyed locally. */
	ENGINE_API int64 SendDestructionInfo(UNetConnection* Connection, FActorDestructionInfo* DestructionInfo);

	/**
	 * Creates and sends a destruction info with the LevelUnloaded reason, only if ThisActor is dormant or recently dormant on Connection.
	 * Returns true if the destruction info was sent, false if the actor isn't replicated or dormant/recently dormant.
	 */
	bool SendDestructionInfoForLevelUnloadIfDormant(AActor* ThisActor, UNetConnection* Connection);

	/**
	 * Explicitly generates and assigns a new network ID (FNetRefHandle if using Iris, FNetworkGUID if using generic replication)
	 * to Object and returns it. This can be used in combination with AssignNetIDToObject to associate actor instances
	 * that are spawned manually for replication, as opposed to the default behavior of letting the engine spawn
	 * the client's instance of a replicated actor automatically.
	 * This function would typically be called on a server, and the caller is responsible for communicating
	 * the returned ID to the remote/client side and calling AssignNetIDToObject to associate the ID with
	 * the client instance of the object.
	 * Only actors or Iris root objects are supported.
	 * @see UNetDriver::AssignNetIDToObject
	 */
	ENGINE_API UE::Net::FNetIDVariant CreateNetIDForObject(UObject* Object);

	/**
	 * Explicitly assigns NetID to Object. Can be called on clients that have received a NetID generated on
	 * the server by a call to CreateNetIDForObject. These functions can be used to associate actor instances
	 * that are spawned manually for replication, as opposed to the default behavior of letting the engine spawn
	 * the client's instance of a replicated actor automatically.
	 * Only actors or Iris root objects are supported.
	 * @see UNetDriver::CreateNetIDForObject
	 */
	ENGINE_API void AssignNetIDToObject(UE::Net::FNetIDVariant NetID, const UObject* Object);

protected:

	void SetIsInTick(bool bIsInTick) { bInTick = bIsInTick; }

	/** Register all TickDispatch, TickFlush, PostTickFlush to tick in World */
	ENGINE_API void RegisterTickEvents(class UWorld* InWorld);
	/** Unregister all TickDispatch, TickFlush, PostTickFlush to tick in World */
	ENGINE_API void UnregisterTickEvents(class UWorld* InWorld);

private:
	void InternalTickDispatch(float DeltaSeconds);
	void InternalTickFlush(float DeltaSeconds);

protected:
	/** Subclasses may override this to customize channel creation. Called by GetOrCreateChannel if the pool is exhausted and a new channel must be allocated. */
	ENGINE_API virtual UChannel* InternalCreateChannelByName(const FName& ChName);

	/** Update stats related to networking. */
	void UpdateNetworkStats();

private:
	/** Reset all the per-frame net connection stats */
	void ResetNetConnectionStats();

protected:

#if WITH_SERVER_CODE
	/**
	* Helper functions for ServerReplicateActors
	*/
	int32 ServerReplicateActors_PrepConnections( const float DeltaSeconds );
	void ServerReplicateActors_BuildConsiderList( TArray<FNetworkObjectInfo*>& OutConsiderList, const float ServerTickTime );

	// Actor prioritization
	ENGINE_API int32 ServerReplicateActors_PrioritizeActors( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, const TArray<FNetworkObjectInfo*>& ConsiderList, const bool bCPUSaturated, FActorPriority*& OutPriorityList, FActorPriority**& OutPriorityActors );
	
	UE_DEPRECATED(5.3, "This function has been deprecated. Please use ServerReplicateActors_ProcessPrioritizedActorsRange instead")
	int32 ServerReplicateActors_ProcessPrioritizedActors( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, FActorPriority** PriorityActors, const int32 FinalSortedCount, int32& OutUpdated );
	
	// Actor relevancy processing within specified range
	ENGINE_API int32 ServerReplicateActors_ProcessPrioritizedActorsRange( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, FActorPriority** PriorityActors, const TInterval<int32>& ActorsIndexRange, int32& OutUpdated, bool bIgnoreSaturation = false );
	
	// Relevant actors that could not be processed this frame are marked to be considered for next frame
	ENGINE_API void ServerReplicateActors_MarkRelevantActors( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, int32 StartActorIndex, int32 EndActorIndex, FActorPriority** PriorityActors );
	
	/**
	* Delegate for overriding the method ServerReplicateActors
	* in the part that prepares prioritized actors list of
	* client connections
	*/
	FOnConsiderListUpdate OnPreConsiderListUpdateOverride;

	/**
	* Delegate that complements the method ServerReplicateActors
	* with the additional replication logic
	*/
	FOnConsiderListUpdate OnPostConsiderListUpdateOverride;

	/**
	* Delegate that allows to implement additional procedures after
	* main replication logic in the corresponding part of the method
	* ServerReplicateActors
	*/
	FOnConsiderListUpdate OnProcessConsiderListOverride;
#endif

	/** Version of RemoveNetworkActor where the actor pointer must be assumed to be inaccessible. */
	ENGINE_API virtual void RemoveInvalidNetworkActor(const AActor* BadActor);

	/** Used to handle any NetDriver specific cleanup once a level has been removed from the world. */
	ENGINE_API virtual void OnLevelRemovedFromWorld(class ULevel* Level, class UWorld* World);

	/** Used to handle any NetDriver specific setup when a level has been added to the world. */
	ENGINE_API virtual void OnLevelAddedToWorld(class ULevel* Level, class UWorld* World);

	/** Handles that track our LevelAdded / Removed delegates. */
	FDelegateHandle OnLevelRemovedFromWorldHandle;
	FDelegateHandle OnLevelAddedToWorldHandle;


public:

	/**
	 * Typically, properties will only ever be replicated / sent from Server net drivers.
	 * Therefore, on clients sending replication state won't be created.
	 * Setting this to true will force creation of sending replication state.
	 *
	 * Note, this doesn't imply the NetDriver *will* ever send properties and will not
	 * standard NetDrivers to send properties from clients.
	 */
	const bool MaySendProperties() const
	{
		return bMaySendProperties;
	}

	/**
	 * Get the current number of sent packets for which we have received a delivery notification
	 */
	uint32 GetOutTotalNotifiedPackets() const { return OutTotalNotifiedPackets; }

	/**
	 * Increase the current number of sent packets for which we have received a delivery notification
	 */
	inline void IncreaseOutTotalNotifiedPackets() { ++OutTotalNotifiedPackets; }

	/**
	 * Get the total number of out of order packets for all connections.
	 *
	 * @return The total number of out of order packets.
	 */
	int32 GetTotalOutOfOrderPackets() const
	{
		return TotalOutOfOrderPacketsLost + TotalOutOfOrderPacketsRecovered + TotalOutOfOrderPacketsDuplicate;
	}

	/**
	 * Get the total number of out of order packets lost for all connections.
	 *
	 * @return The total number of out of order packets lost.
	 */
	int32 GetTotalOutOfOrderPacketsLost() const
	{
		return TotalOutOfOrderPacketsLost;
	}

	/**
	 * Increase the value of TotalOutOfOrderPacketsLost.
	 *
	 * @param Count		The amount to add to TotalOutOfOrderPacketsLost
	 */
	void IncreaseTotalOutOfOrderPacketsLost(int32 Count=1)
	{
		TotalOutOfOrderPacketsLost += Count;
	}

	/**
	 * Get the total number of out of order packets recovered for all connections.
	 *
	 * @return The total number of out of order packets recovered.
	 */
	int32 GetTotalOutOfOrderPacketsRecovered() const
	{
		return TotalOutOfOrderPacketsRecovered;
	}

	/**
	 * Increase the value of TotalOutOfOrderPacketsRecovered.
	 *
	 * @param Count		The amount to add to TotalOutOfOrderPacketsRecovered
	 */
	void IncreaseTotalOutOfOrderPacketsRecovered(int32 Count=1)
	{
		TotalOutOfOrderPacketsRecovered += Count;
	}

	/**
	 * Get the total number of out of order packets that were duplicates for all connections.
	 *
	 * @return The total number of out of order packets that were duplicates.
	 */
	int32 GetTotalOutOfOrderPacketsDuplicate() const
	{
		return TotalOutOfOrderPacketsDuplicate;
	}

	/**
	 * Increase the value of TotalOutOfOrderPacketsDuplicate.
	 *
	 * @param Count		The amount to add to TotalOutOfOrderPacketsDuplicate
	 */
	void IncreaseTotalOutOfOrderPacketsDuplicate(int32 Count=1)
	{
		TotalOutOfOrderPacketsDuplicate += Count;
	}

	uint32 GetCachedGlobalNetTravelCount() const
	{
		return CachedGlobalNetTravelCount;
	}

	bool DidHitchLastFrame() const;

	static bool IsDormInitialStartupActor(AActor* Actor);

	/** Unmap all references to this object, so that if later we receive this object again, we can remap the original references */
	void MoveMappedObjectToUnmapped(const UObject* Object);

	/** Whether or not this driver has an IsReplay() connection, updated in Add/RemoveClientConnection */
	bool HasReplayConnection() const { return bHasReplayConnection; }

	/**
	 * Whether or not this NetDriver supports encryption. Does not signify that encryption is actually enabled, nor setup by the PacketHandler.
	 *
	 * @return		Whether or not this NetDriver supports encryption.
	 */
	virtual bool DoesSupportEncryption() const
	{
		return true;
	}

	/**
	 * Whether or not this NetDriver requires encryption. Does signify that encryption is enabled, but does not signify that it's setup properly.
	 *
	 * @return		Whether or not encryption is presently required for connections.
	 */
	ENGINE_API virtual bool IsEncryptionRequired() const;

	/** Returns the value of cvar net.ClientIncomingBunchFrameTimeLimitMS on clients, or 0 otherwise. 0 = no limit. */
	ENGINE_API float GetIncomingBunchFrameProcessingTimeLimit() const;

	/** Returns true if the cvar net.ClientIncomingBunchFrameTimeLimitMS is set and the limit was exceeded */
	ENGINE_API bool HasExceededIncomingBunchFrameProcessingTime() const;

	/** Called internally by channels to track processing time for net.ClientIncomingBunchFrameTimeLimitMS and HasExceededIncomingBunchFrameProcessingTime() */
	void AddBunchProcessingFrameTimeMS(float Milliseconds) { IncomingBunchProcessingElapsedFrameTimeMS += Milliseconds; }

	/** Called internally by channels to track how many hit net.QueuedBunchTimeFailsafeSeconds */
	void AddQueuedBunchFailsafeChannel() { ++QueuedBunchFailsafeNumChannels; }

	/** Sets whether this driver should run certain operations via UE::RemoteExecutor::ExecuteTransactional */
	void SetReplicateTransactionally(bool bInReplicateTransactionally)
	{
		bReplicateTransactionally = bInReplicateTransactionally;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	/** Returns true if this driver should run certain operations via UE::RemoteExecutor::ExecuteTransactional */
	bool ShouldReplicateTransactionally() const
	{
		return bReplicateTransactionally;
	}
#else
	/** This function is not valid unless UE_WITH_REMOTE_OBJECT_HANDLE is set */ 
	constexpr bool ShouldReplicateTransactionally() const
	{
		return false;
	}
#endif

	void SetDeferTransactionalSend(bool bInDeferTransactionalSend)
	{
		bDeferTransactionalSend = bInDeferTransactionalSend;
	}

	bool ShouldDeferTransactionalSend() const
	{
		return bDeferTransactionalSend;
	}

	/** Returns true if references to UObjects should be serialized as FRemoteObjectReference */
	bool IsUsingRemoteObjectReferences() const { return bUseRemoteObjectRefs; }

	/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, determines whether or not references to UObjects (either from TObjectPtr or TWeakObjectPtr) are serialized as FRemoteObjectReference */
	void SetUsingRemoteObjectReferences(bool bValue)
	{
		bUseRemoteObjectRefs = bValue;
	}

	/** Enable/disable support for NMT_JoinNoPawn and NMT_JoinNoPawnSplit. */
	void SetSupportForNoPawnConnection(bool bInSupportNoPawnConnection)
	{
		bSupportNoPawnConnection = bInSupportNoPawnConnection;
	}

	/** Return true if the driver supports NMT_JoinNoPawn and NMT_JoinNoPawnSplit. */
	bool HasSupportForNoPawnConnection();

	/** Enumerate all connections to find one that matches a particular player id. */
	TObjectPtr<UNetConnection> FindConnection(struct FUniqueNetIdRepl& PlayerId);

	/**
	 * Used by UNetConnection after receiving a packet to determine whether to send an ack packet on the next
	 * connection flush regardless of whether there's any other data to send. Set via the config property bSendImmediateAcks.
	 */
	bool IsSendingImmediateAcks() const
	{
		return bSendImmediateAcks;
	}

protected:
	
	/** Stream of random numbers to be used by this instance of UNetDriver */
	FRandomStream UpdateDelayRandomStream;

	/** Creates a trace event that updates the name and properties of the associated Game Instance */
	void NotifyGameInstanceUpdated();

	/** Indicates whether ticking throttle is enabled for this instance of NetDriver */
	bool bTickingThrottleEnabled = true;
private:
	// Only for ForwardRemoteFunction
	friend FObjectReplicator;

	ENGINE_API virtual ECreateReplicationChangelistMgrFlags GetCreateReplicationChangelistMgrFlags() const;

	FDelegateHandle PostGarbageCollectHandle;
	void PostGarbageCollect();

	FActorDestructionInfo* CreateDestructionInfo(AActor* ThisActor, FActorDestructionInfo *DestructionInfo);

	void CreateReplicatedStaticActorDestructionInfo(ULevel* Level, const FReplicatedStaticActorDestructionInfo& Info);

	void FlushActorDormancyInternal(AActor *Actor);

	void LoadChannelDefinitions();

	/** Used with FNetDelegates::OnSyncLoadDetected to log sync loads */
	void ReportSyncLoad(const FNetSyncLoadReport& Report);

	enum class ECrashContextUpdate
	{
		Default,
		UpdateRepModel,
		ClearRepModel,
	};
	void UpdateCrashContext(ECrashContextUpdate UpdateType=ECrashContextUpdate::Default);

	void RemoveDestroyedGuidsByLevel(const ULevel* Level, const TArray<FNetworkGUID>& RemovedGUIDs);

	/** Private helper for ServerReplicateActors */
	void ServerReplicateActors_ForConnection(UE::Net::FServerReplicateActors_ForConnectionParams& Params);

	/** Handle to FNetDelegates::OnSyncLoadDetected delegate */
	FDelegateHandle ReportSyncLoadDelegateHandle;

	void InitIrisSettings(FName NewDriverName);
	void SetReplicationSystem(UReplicationSystem* ReplicationSystem);
	void CreateReplicationSystem(bool bInitAsClient);
	void UpdateIrisReplicationViews() const;
	void SendClientMoveAdjustments();
	void PostDispatchSendUpdate();
	void InternalIrisUpdateTransactional(float DeltaSeconds);
	void InternalIrisUpdate(float DeltaSeconds);

	/** Tell the registered NetAnalytics to send their analytics via the provider */
	void SendNetAnalytics();

	/** Description of the replication model used by this Driver (RepGraph, Iris or Generic) */
	FString GetReplicationModelName() const;

	void InitNetTraceId();

protected:
	/** Called from RPC processing code to forward RPC to other NetDrivers if ShouldForwardFunction returns true. */
	ENGINE_API virtual void ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms);

	/** Determine the correct value for bNoTimeouts based on config/command line/PIE */
	void EvaluateNoTimeouts();

private:
	/** Go over imported network guids and map them to the newly created object. */
	void UpdateUnmappedObjects();

	/** Periodically look for invalid dormant replicators tied to destroyed objects. */
	void CleanupStaleDormantReplicators();

private:

	UPROPERTY(transient)
	TObjectPtr<UReplicationDriver> ReplicationDriver;

	TUniquePtr<UE::Net::FNetTokenStore> NetTokenStore;

	UReplicationSystem* ReplicationSystem = nullptr;

	/** When set this will skip registering all the network relevant actors when setting the World */
	bool bSkipStartReplicatingWorldForIris = false;

	/** True when the NetDriver has been configured to run with the Iris replication system.*/
	bool bIsUsingIris = false;

	/** True when the NetDriver has been configured to run NetConnection::Tick in parallel*/
	bool bIsUsingParallelNetTick = false;

	// For FindOrAddNetworkObjectInfo
	TPimplPtr<FNetworkObjectInfo> DummyNetworkObjectInfo;

	/** Stores the list of objects to replicate into the replay stream. This should be a TUniquePtr, but it appears the generated.cpp file needs the full definition of the pointed-to type. */
	TSharedPtr<FNetworkObjectList> NetworkObjects;

	/** Transient ConsiderList only used inside ServerReplicateActors. It's a member so that it can be easily updated between transactions when bReplicateTransactionally is true to remove stale entries. */
	TArray<FNetworkObjectInfo*> CurrentConsiderList;

	/** Set to "Lagging" on the server when all client connections are near timing out. We are lagging on the client when the server connection is near timed out. */
	ENetworkLagState::Type LagState;

	/** Duplicate level instance to use for playback (PIE instance ID) */
	int32 DuplicateLevelID;

	/** NetDriver time to end packet loss burst simulation. */
	double PacketLossBurstEndTime;

	/** Count the number of notified packets, i.e. packets that we know if they are delivered or not. Used to reliably measure outgoing packet loss */
	uint32 OutTotalNotifiedPackets;

	/** Assigns driver unique IDs to client connections */
	FNetConnectionIdHandler ConnectionIdHandler;

	/** Unique id used by NetTrace to identify driver */
	uint32 NetTraceId;

	/** Stat tracking for the total number of out of order packets lost */
	int32 TotalOutOfOrderPacketsLost = 0;

	/** Stat tracking for the total number of out of order packets recovered */
	int32 TotalOutOfOrderPacketsRecovered = 0;

	/** Stat tracking for the total number of out of order packets that were duplicates */
	int32 TotalOutOfOrderPacketsDuplicate = 0;

	/** Cached value for UEngine.GlobalNetTravelCount, at the time of NetDriver initialization */
	uint32 CachedGlobalNetTravelCount = 0;
	
	/** Accumulated number of frames in the current stat gathering period */
	uint32 StatUpdateFrames = 0;

	/** Milliseconds spent processing incoming bunches in the current frame, directly from the network and queued on channels. */
	float IncomingBunchProcessingElapsedFrameTimeMS = 0.0f;
	
	/** Accumulated number of frames in the current stat period for which HasExceededIncomingBunchFrameProcessingTime would return true */
	uint32 NumFramesOverIncomingBunchTimeLimit = 0;

	/** Accumulated number of channels that hit the net.QueuedBunchTimeFailsafeSeconds this frame */
	uint32 QueuedBunchFailsafeNumChannels = 0;

	/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, determines whether or not property replication happens within a transaction */
	bool bReplicateTransactionally = true;

	/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, determines whether or not references to UObjects (either from TObjectPtr or TWeakObjectPtr) are serialized as FRemoteObjectReference */
	bool bUseRemoteObjectRefs = false;
	
	/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, determines whether or not we defer the actual network send until the transaction commits */
	bool bDeferTransactionalSend = true;

	/** Server support for NMT_JoinNoPawn and NMT_JoinNoPawnSplit. */
	bool bSupportNoPawnConnection = false;

	/**
	 * If true, after receiving a packet, send an ack packet on the next connection flush regardless of whether
	 * there's any other data to send. This is the default.
	 * If false, the ack won't be sent until other data is ready or the KeepAliveTime is reached. This can help
	 * reduce the number of individual packets sent.
	 */
	UPROPERTY(Config)
	bool bSendImmediateAcks = true;
};
