// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkSerialization.h: 
	Contains custom network serialization functionality.
=============================================================================*/

#pragma once

#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Misc/NetworkVersion.h"
#include "UObject/CoreNet.h"
#include "EngineLogs.h"
#include "Net/Core/Serialization/QuantizedVectorSerialization.h"

#include <type_traits>

#include "NetSerialization.generated.h"

/**
 * Helper to optionally serialize a value (using operator<< on Archive).
 * A single signal bit is indicates whether to serialize, or whether to just use the default value.
 * Returns true if the value was not the default and needed to be serialized.
 */
template<typename ValueType>
bool SerializeOptionalValue(const bool bIsSaving, FArchive& Ar, ValueType& Value, const ValueType& DefaultValue)
{
	bool bNotDefault = (bIsSaving && (Value != DefaultValue));
	Ar.SerializeBits(&bNotDefault, 1);
	if (bNotDefault)
	{
		// Non-default value, need to save or load it.
		Ar << Value;
	}
	else if (!bIsSaving)
	{
		// Loading, and should use default
		Value = DefaultValue;
	}

	return bNotDefault;
}

/**
 * Helper to optionally serialize a value (using the NetSerialize function).
 * A single signal bit indicates whether to serialize, or whether to just use the default value.
 * Returns true if the value was not the default and needed to be serialized.
 */
template<typename ValueType>
bool NetSerializeOptionalValue(const bool bIsSaving, FArchive& Ar, ValueType& Value, const ValueType& DefaultValue, class UPackageMap* PackageMap)
{
	bool bNotDefault = (bIsSaving && (Value != DefaultValue));
	Ar.SerializeBits(&bNotDefault, 1);
	if (bNotDefault)
	{
		// Non-default value, need to save or load it.
		bool bLocalSuccess = true;
		Value.NetSerialize(Ar, PackageMap, bLocalSuccess);
	}
	else if (!bIsSaving)
	{
		// Loading, and should use default
		Value = DefaultValue;
	}

	return bNotDefault;
}

/**
 *	===================== NetSerialize and NetDeltaSerialize customization. =====================
 *
 *	The main purpose of this file it to hold custom methods for NetSerialization and NetDeltaSerialization. A longer explanation on how this all works is
 *	covered below. For quick reference however, this is how to customize net serialization for structs.
 *
 *		
 *	To define your own NetSerialize and NetDeltaSerialize on a structure:
 *		(of course you don't need to define both! Usually you only want to define one, but for brevity Im showing both at once)
 */
#if 0

USTRUCT()
struct FExampleStruct
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * @param Ar			FArchive to read or write from.
	 * @param Map			PackageMap used to resolve references to UObject*
	 * @param bOutSuccess	return value to signify if the serialization was succesfull (if false, an error will be logged by the calling function)
	 *
	 * @return return true if the serialization was fully mapped. If false, the property will be considered 'dirty' and will replicate again on the next update.
	 *	This is needed for UActor* properties. If an actor's Actorchannel is not fully mapped, properties referencing it must stay dirty.
	 *	Note that UPackageMap::SerializeObject returns false if an object is unmapped. Generally, you will want to return false from your ::NetSerialize
	 *  if you make any calls to ::SerializeObject that return false.
	 *
	*/
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		// Your code here!
		return true;
	}

	/**
	 * @param DeltaParms	Generic struct of input parameters for delta serialization
	 *
	 * @return return true if the serialization was fully mapped. If false, the property will be considered 'dirty' and will replicate again on the next update.
	 *	This is needed for UActor* properties. If an actor's Actorchannel is not fully mapped, properties referencing it must stay dirty.
	 *	Note that UPackageMap::SerializeObject returns false if an object is unmapped. Generally, you will want to return false from your ::NetSerialize
	 *  if you make any calls to ::SerializeObject that return false.
	 *
	*/
	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
	{
		// Your code here!
		return true;
	}
}

template<>
struct TStructOpsTypeTraits< FExampleStruct > : public TStructOpsTypeTraitsBase2< FExampleStruct >
{
	enum 
	{
		WithNetSerializer = true,
		WithNetDeltaSerializer = true,
	};
};

#endif

/**
 * Everything related to Fast TArray Replication has been moved to "Net/Serialization/FastArraySerializer.h"
 */

/**
 *	===================== Vector NetSerialization customization. =====================
 *	Provides custom NetSerilization for FVectors.
 *
 *	There are two types of net quantization available:
 *
 *	Fixed Quantization (SerializeFixedVector)
 *		-Fixed number of bits
 *		-Max Value specified as template parameter
 *
 *		Serialized value is scaled based on num bits and max value. Precision is determined by MaxValue and NumBits
 *		(if 2^NumBits is > MaxValue, you will have room for extra precision).
 *
 *		This format is good for things like normals, where the magnitudes are often similar. For example normal values may often
 *		be in the 0.1f - 1.f range. In a packed format, the overhead in serializing num of bits per component would outweigh savings from
 *		serializing very small ( < 0.1f ) values.
 *
 *		It is also good for performance critical sections since you can guarantee byte alignment if that is important.
 *	
 *
 *
 *	Packed Quantization (SerializePackedVector)
 *		-Scaling factor (usually 10, 100, etc)
 *		-Max number of bits per component (this is maximum, not a constant)
 *
 *		The format is <num of bits per component> <N bits for X> <N bits for Y> <N bits for Z>
 *
 *		The advantages to this format are that packed nature. You may support large magnitudes and have as much precision as you want. All while
 *		having small magnitudes take less space.
 *
 *		The trade off is that there is overhead in serializing how many bits are used for each component, and byte alignment is almost always thrown
 *		off.
 *
*/

namespace UE::Net::Private
{

// ScaleFactor is multiplied before send and divided by post receive. A higher ScaleFactor means more precision.
// MaxBitsPerComponent is the maximum number of bits to use per component. This is only a maximum. A header is
// written (size = Log2 (MaxBitsPerComponent)) to indicate how many bits are actually used. 
template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool LegacyReadPackedVector(FVector3f& Value, FArchive& Ar)
{
	uint32 Bits	= 0;

	// Serialize how many bits each component will have
	Ar.SerializeInt( Bits, MaxBitsPerComponent );

	int32  Bias = 1<<(Bits+1);
	uint32 Max	= 1<<(Bits+2);
	uint32 DX	= 0;
	uint32 DY	= 0;
	uint32 DZ	= 0;
	
	Ar.SerializeInt( DX, Max );
	Ar.SerializeInt( DY, Max );
	Ar.SerializeInt( DZ, Max );
	
	float fact = (float)ScaleFactor;

	Value.X = (float)(static_cast<int32>(DX)-Bias) / fact;
	Value.Y = (float)(static_cast<int32>(DY)-Bias) / fact;
	Value.Z = (float)(static_cast<int32>(DZ)-Bias) / fact;

	return true;
}

}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool WritePackedVector(FVector3f Value, FArchive& Ar)
{
	return UE::Net::WriteQuantizedVector(ScaleFactor, Value, Ar);
}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool WritePackedVector(FVector3d Vector, FArchive& Ar)
{
	return UE::Net::WriteQuantizedVector(ScaleFactor, Vector, Ar);
}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool ReadPackedVector(FVector3f& Value, FArchive& Ar)
{
	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::PackedVectorLWCSupport && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
	{
		return UE::Net::ReadQuantizedVector(ScaleFactor, Value, Ar);
	}
	else
	{
		return UE::Net::Private::LegacyReadPackedVector<ScaleFactor, MaxBitsPerComponent>(Value, Ar);
	}
}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool ReadPackedVector(FVector3d& Value, FArchive& Ar)
{
	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::PackedVectorLWCSupport && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
	{
		return UE::Net::ReadQuantizedVector(ScaleFactor, Value, Ar);
	}
	else
	{
		FVector3f AsFloat;
		const bool bRet =  UE::Net::Private::LegacyReadPackedVector<ScaleFactor, MaxBitsPerComponent>(AsFloat, Ar);
		Value = FVector3d(AsFloat);
		return bRet;
	}
}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool SerializePackedVector(FVector3f& Value, FArchive& Ar)
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::PackedVectorLWCSupport && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
	{
		return UE::Net::SerializeQuantizedVector<ScaleFactor>(Value, Ar);
	}
	else
	{
		check(Ar.IsLoading());
		return UE::Net::Private::LegacyReadPackedVector<ScaleFactor, MaxBitsPerComponent>(Value, Ar);
	}
}

template<int32 ScaleFactor, int32 MaxBitsPerComponent>
bool SerializePackedVector(FVector3d& Value, FArchive& Ar)
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::PackedVectorLWCSupport && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
	{
		return UE::Net::SerializeQuantizedVector<ScaleFactor>(Value, Ar);
	}
	else
	{
		check(Ar.IsLoading());
		FVector3f AsFloat(Value);
		const bool bRet = UE::Net::Private::LegacyReadPackedVector<ScaleFactor, MaxBitsPerComponent>(AsFloat, Ar);
		Value = FVector3d(AsFloat);
		return bRet;
	}
}

// --------------------------------------------------------------

template<int32 MaxValue, uint32 NumBits>
struct TFixedCompressedFloatDetails
{
	                                                                // NumBits = 8:
	static constexpr int32 MaxBitValue = (1 << (NumBits - 1)) - 1;  //   0111 1111 - Max abs value we will serialize
	static constexpr int32 Bias = (1 << (NumBits - 1));             //   1000 0000 - Bias to pivot around (in order to support signed values)
	static constexpr int32 SerIntMax = (1 << (NumBits - 0));        // 1 0000 0000 - What we pass into SerializeInt
	static constexpr int32 MaxDelta = (1 << (NumBits - 0)) - 1;     //   1111 1111 - Max delta is
};

template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T> && NumBits < 32)>
bool WriteFixedCompressedFloat(const T Value, FArchive& Ar)
{
	using Details = TFixedCompressedFloatDetails<MaxValue, NumBits>;

	bool clamp = false;
	int64 ScaledValue;
	if ( MaxValue > Details::MaxBitValue )
	{
		// We have to scale this down
		const T Scale = T(Details::MaxBitValue)/MaxValue;
		ScaledValue = FMath::TruncToInt(Scale * Value);
	}
	else
	{
		// We will scale up to get extra precision. But keep is a whole number preserve whole values
		constexpr int32 Scale = Details::MaxBitValue / MaxValue;
		ScaledValue = FMath::RoundToInt( Scale * Value );
	}

	uint32 Delta = static_cast<uint32>(ScaledValue + Details::Bias);

	if (Delta > Details::MaxDelta)
	{
		clamp = true;
		Delta = static_cast<int32>(Delta) > 0 ? Details::MaxDelta : 0;
	}

	Ar.SerializeInt( Delta, Details::SerIntMax );

	return !clamp;
}

template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T> && NumBits < 32)>
bool ReadFixedCompressedFloat(T& Value, FArchive& Ar)
{
	using Details = TFixedCompressedFloatDetails<MaxValue, NumBits>;

	uint32 Delta;
	Ar.SerializeInt(Delta, Details::SerIntMax);
	T UnscaledValue = static_cast<T>( static_cast<int32>(Delta) - Details::Bias );

	if constexpr (MaxValue > Details::MaxBitValue)
	{
		// We have to scale down, scale needs to be a float:
		constexpr T InvScale = MaxValue / (T)Details::MaxBitValue;
		Value = UnscaledValue * InvScale;
	}
	else
	{
		constexpr int32 Scale = Details::MaxBitValue / MaxValue;
		constexpr T InvScale = T(1) / (T)Scale;

		Value = UnscaledValue * InvScale;
	}

	return true;
}

// --------------------------------------------------------------
// MaxValue is the max abs value to serialize. If abs value of any vector components exceeds this, the serialized value will be clamped.
// NumBits is the total number of bits to use - this includes the sign bit!
//
// So passing in NumBits = 8, and MaxValue = 2^8, you will scale down to fit into 7 bits so you can leave 1 for the sign bit.
template<int32 MaxValue, uint32 NumBits>
bool SerializeFixedVector(FVector3f &Vector, FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		bool success = true;
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.X, Ar);
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.Y, Ar);
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.Z, Ar);
		return success;
	}

	ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.X, Ar);
	ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.Y, Ar);
	ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.Z, Ar);
	return true;
}

template<int32 MaxValue, uint32 NumBits>
bool SerializeFixedVector(FVector3d &Vector, FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		bool success = true;
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.X, Ar);
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.Y, Ar);
		success &= WriteFixedCompressedFloat<MaxValue, NumBits>(Vector.Z, Ar);
		return success;
	}
	else
	{
		ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.X, Ar);
		ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.Y, Ar);
		ReadFixedCompressedFloat<MaxValue, NumBits>(Vector.Z, Ar);
		return true;
	}
}
// --------------------------------------------------------------

/**
 *	FVector_NetQuantize
 *
 *	0 decimal place of precision.
 *	Up to 20 bits per component.
 *	Valid range: 2^20 = +/- 1,048,576
 *
 *	Note: this is the historical UE format for vector net serialization
 *
 */
USTRUCT(meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector_NetQuantize", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector_NetQuantize"))
struct FVector_NetQuantize : public FVector
{
	GENERATED_USTRUCT_BODY()

	FVector_NetQuantize() = default;

	explicit inline FVector_NetQuantize(EForceInit E)
	: FVector(E)
	{}

	inline FVector_NetQuantize(double InX, double InY, double InZ)
	: FVector(InX, InY, InZ)
	{}

	inline FVector_NetQuantize(const FVector &InVec)
	{
		FVector::operator=(InVec);
	}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		bOutSuccess = SerializePackedVector<1, 20>(*this, Ar);
		return true;
	}
};

template<>
struct TStructOpsTypeTraits< FVector_NetQuantize > : public TStructOpsTypeTraitsBase2< FVector_NetQuantize >
{
	enum 
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};

/**
 *	FVector_NetQuantize10
 *
 *	1 decimal place of precision.
 *	Up to 24 bits per component.
 *	Valid range: 2^24 / 10 = +/- 1,677,721.6
 *
 */
USTRUCT(meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector_NetQuantize10", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector_NetQuantize10"))
struct FVector_NetQuantize10 : public FVector
{
	GENERATED_USTRUCT_BODY()

	FVector_NetQuantize10() = default;

	explicit inline FVector_NetQuantize10(EForceInit E)
	: FVector(E)
	{}

	inline FVector_NetQuantize10(double InX, double InY, double InZ)
	: FVector(InX, InY, InZ)
	{}

	inline FVector_NetQuantize10(const FVector &InVec)
	{
		FVector::operator=(InVec);
	}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		bOutSuccess = SerializePackedVector<10, 24>(*this, Ar);
		return true;
	}
};

template<>
struct TStructOpsTypeTraits< FVector_NetQuantize10 > : public TStructOpsTypeTraitsBase2< FVector_NetQuantize10 >
{
	enum 
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};

/**
 *	FVector_NetQuantize100
 *
 *	2 decimal place of precision.
 *	Up to 30 bits per component.
 *	Valid range: 2^30 / 100 = +/- 10,737,418.24
 *
 */
USTRUCT(meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector_NetQuantize100", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector_NetQuantize100"))
struct FVector_NetQuantize100 : public FVector
{
	GENERATED_USTRUCT_BODY()

	FVector_NetQuantize100() = default;

	explicit inline FVector_NetQuantize100(EForceInit E)
	: FVector(E)
	{}

	inline FVector_NetQuantize100(double InX, double InY, double InZ)
	: FVector(InX, InY, InZ)
	{}

	inline FVector_NetQuantize100(const FVector3f& InVec) : FVector(InVec) {}
	inline FVector_NetQuantize100(const FVector3d& InVec) : FVector(InVec) {}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		bOutSuccess = SerializePackedVector<100, 30>(*this, Ar);
		return true;
	}
};

template<>
struct TStructOpsTypeTraits< FVector_NetQuantize100 > : public TStructOpsTypeTraitsBase2< FVector_NetQuantize100 >
{
	enum 
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};

/**
 *	FVector_NetQuantizeNormal
 *
 *	16 bits per component
 *	Valid range: -1..+1 inclusive
 */
USTRUCT(meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector_NetQuantizeNormal", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector_NetQuantizeNormal"))
struct FVector_NetQuantizeNormal : public FVector
{
	GENERATED_USTRUCT_BODY()

	FVector_NetQuantizeNormal() = default;

	explicit inline FVector_NetQuantizeNormal(EForceInit E)
	: FVector(E)
	{}

	inline FVector_NetQuantizeNormal(double InX, double InY, double InZ)
	: FVector(InX, InY, InZ)
	{}

	inline FVector_NetQuantizeNormal(const FVector &InVec)
	{
		FVector::operator=(InVec);
	}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		bOutSuccess = SerializeFixedVector<1, 16>(*this, Ar);
		return true;
	}
};

template<>
struct TStructOpsTypeTraits< FVector_NetQuantizeNormal > : public TStructOpsTypeTraitsBase2< FVector_NetQuantizeNormal >
{
	enum 
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};

// --------------------------------------------------------------


/**
 *	===================== Safe TArray Serialization ===================== 
 *	
 *	These are helper methods intended to make serializing TArrays safer in custom
 *	::NetSerialize functions. These enforce max limits on array size, so that a malformed
 *	packet is not able to allocate an arbitrary amount of memory (E.g., a hacker serilizes
 *	a packet where a TArray size is of size MAX_int32, causing gigs of memory to be allocated for
 *	the TArray).
 *	
 *	These should only need to be used when you are overriding ::NetSerialize on a UStruct via struct traits.
 *	When using default replication, TArray properties already have this built in security.
 *	
 *	SafeNetSerializeTArray_Default - calls << operator to serialize the items in the array.
 *	SafeNetSerializeTArray_WithNetSerialize - calls NetSerialize to serialize the items in the array.
 *	
 *	When saving, bOutSuccess will be set to false if the passed in array size exceeds to MaxNum template parameter.
 *	
 *	Example:
 *	
 *	FMyStruct {
 *		
 *		TArray<float>						MyFloats;		// We want to call << to serialize floats
 *		TArray<FVector_NetQuantizeNormal>	MyVectors;		// We want to call NetSeriailze on these *		
 *		
 *		bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
 *		{
 *			// Don't do this:
 *			Ar << MyFloats;
 *			Ar << MyVectors;
 *			
 *			// Do this instead:
 *			SafeNetSerializeTArray_Default<31>(Ar, MyFloats);
 *			SafeNetSerializeTArray_WithNetSerialize<31>(Ar, MyVectors, Map);
 *		}
 *	}	
 *	
 */

template<int32 MaxNum, typename T, typename A>
int32 SafeNetSerializeTArray_HeaderOnly(FArchive& Ar, TArray<T, A>& Array, bool& bOutSuccess)
{
	const uint32 NumBits = FMath::CeilLogTwo(MaxNum)+1;
	
	int32 ArrayNum = 0;

	// Clamp number of elements on saving side
	if (Ar.IsSaving())
	{
		ArrayNum = Array.Num();
		if (ArrayNum > MaxNum)
		{
			// Overflow. This is on the saving side, so the calling code is exceeding the limit and needs to be fixed.
			bOutSuccess = false;
			ArrayNum = MaxNum;
		}		
	}

	// Serialize num of elements
	Ar.SerializeBits(&ArrayNum, NumBits);

	// Preallocate new items on loading side
	if (Ar.IsLoading())
	{
		if (ArrayNum > MaxNum)
		{
			// If MaxNum doesn't fully utilize all bits that are needed to send the array size we can receive a larger value.
			bOutSuccess = false;
			ArrayNum = MaxNum;
		}
		Array.Reset();
		Array.AddDefaulted(ArrayNum);
	}

	return ArrayNum;
}

template<int32 MaxNum, typename T, typename A>
bool SafeNetSerializeTArray_Default(FArchive& Ar, TArray<T, A>& Array)
{
	bool bOutSuccess = true;
	int32 ArrayNum = SafeNetSerializeTArray_HeaderOnly<MaxNum, T, A>(Ar, Array, bOutSuccess);

	// Serialize each element in the array with the << operator
	for (int32 idx=0; idx < ArrayNum && Ar.IsError() == false; ++idx)
	{
		Ar << Array[idx];
	}

	// Return
	bOutSuccess &= !Ar.IsError();
	return bOutSuccess;
}

template<int32 MaxNum, typename T, typename A>
bool SafeNetSerializeTArray_WithNetSerialize(FArchive& Ar, TArray<T, A>& Array, class UPackageMap* PackageMap)
{
	bool bOutSuccess = true;
	int32 ArrayNum = SafeNetSerializeTArray_HeaderOnly<MaxNum, T, A>(Ar, Array, bOutSuccess);

	// Serialize each element in the array with the << operator
	for (int32 idx=0; idx < ArrayNum && Ar.IsError() == false; ++idx)
	{
		Array[idx].NetSerialize(Ar, PackageMap, bOutSuccess);
	}

	// Return
	bOutSuccess &= !Ar.IsError();
	return bOutSuccess;
}
