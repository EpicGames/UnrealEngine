// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/CoreNet.h"

class UPendingNetGame;
class UNetDriver;

/**
* Accepting connection response codes
*/
namespace EAcceptConnection
{
	enum Type
	{
		/** Reject the connection */
		Reject,
		/** Accept the connection */
		Accept,
		/** Ignore the connection, sending no reply, while server traveling */
		Ignore
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EAcceptConnection::Type EnumVal)
	{
		switch (EnumVal)
		{
			case Reject:
			{
				return TEXT("Reject");
			}
			case Accept:
			{
				return TEXT("Accept");
			}
			case Ignore:
			{
				return TEXT("Ignore");
			}
		}
		return TEXT("");
	}
};

/**
 * The net code uses this to send notifications.
 */
class FNetworkNotify
{
public:
	/**
	 * Notification that an incoming connection is pending, giving the interface a chance to reject the request
	 *
	 * @return EAcceptConnection indicating willingness to accept the connection at this time
	 */
	virtual EAcceptConnection::Type NotifyAcceptingConnection() PURE_VIRTUAL(FNetworkNotify::NotifyAcceptedConnection, return EAcceptConnection::Ignore;);

	/**
	 * Notification that a new connection has been created/established as a result of a remote request, previously approved by NotifyAcceptingConnection
	 *
	 * @param Connection newly created connection
	 */
	virtual void NotifyAcceptedConnection(class UNetConnection* Connection) PURE_VIRTUAL(FNetworkNotify::NotifyAcceptedConnection, );

	/**
	 * Notification that a new channel is being created/opened as a result of a remote request (Actor creation, etc)
	 *
	 * @param Channel newly created channel
	 *
	 * @return true if the channel should be opened, false if it should be rejected (destroying the channel)
	 */
	virtual bool NotifyAcceptingChannel(class UChannel* Channel) PURE_VIRTUAL(FNetworkNotify::NotifyAcceptingChannel, return false;);

	/**
	 * Handler for messages sent through a remote connection's control channel
	 * not required to handle the message, but if it reads any data from Bunch, it MUST read the ENTIRE data stream for that message (i.e. use FNetControlMessage<TYPE>::Receive())
	 */
	virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch) PURE_VIRTUAL(FNetworkNotify::NotifyReceivedText, );
};

/** Types of responses games are meant to return to let the net connection code about the outcome of an encryption key request */
enum class EEncryptionResponse : uint8
{
	/** General failure */
	Failure,
	/** Key success */
	Success,
	/** Token given was invalid */
	InvalidToken,
	/** No key found */
	NoKey,
	/** Token doesn't match session */
	SessionIdMismatch,
	/** Invalid parameters passed to callback */
	InvalidParams
};

inline const TCHAR* const LexToString(const EEncryptionResponse Response)
{
	switch (Response)
	{
		case EEncryptionResponse::Failure:
			return TEXT("Failure");
		case EEncryptionResponse::Success:
			return TEXT("Success");
		case EEncryptionResponse::InvalidToken:
			return TEXT("InvalidToken");
		case EEncryptionResponse::NoKey:
			return TEXT("NoKey");
		case EEncryptionResponse::SessionIdMismatch:
			return TEXT("SessionIdMismatch");
		case EEncryptionResponse::InvalidParams:
			return TEXT("InvalidParams");
		default:
			break;
	}

	checkf(false, TEXT("Missing EncryptionResponse Type: %d"), static_cast<const int32>(Response));
	return TEXT("");
}

struct FEncryptionKeyResponse
{
	/** Result of the encryption key request */
	EEncryptionResponse Response;
	/** Error message related to the response */
	FString ErrorMsg;
	/** Encryption data */
	FEncryptionData EncryptionData;

	FEncryptionKeyResponse()
		: Response(EEncryptionResponse::Failure)
	{
	}

	FEncryptionKeyResponse(EEncryptionResponse InResponse, const FString& InErrorMsg)
		: Response(InResponse)
		, ErrorMsg(InErrorMsg)
	{
	}
};

/** Specifies how to handle encryption failures */
enum class EEncryptionFailureAction : uint8
{
	/** Default handling of encryption failures - net.AllowEncryption determines whether to reject or accept, with exceptions for PIE etc. */
	Default,
	/** Reject the connection */
	RejectConnection,
	/** Allow the connection */
	AllowConnection
};

/** Types of punishment to apply to a cheating client */
enum class ECheatPunishType : uint8
{
	/* Unknown type of cheat punishment */
	Unknown,
	/** User should be booted from the game client via logout */
	KickClient,
	/** User should be booted from the current game session via disconnect */
	KickMatch,
	/** User received info about being punished (eg. ban type etc) */
	PunishInfo
};

inline const TCHAR* const LexToString(const ECheatPunishType Response)
{
	switch (Response)
	{
		case ECheatPunishType::Unknown:
			return TEXT("Unknown");
		case ECheatPunishType::KickClient:
			return TEXT("KickClient");
		case ECheatPunishType::KickMatch:
			return TEXT("KickMatch");
		case ECheatPunishType::PunishInfo:
			return TEXT("PunishInfo");
		default:
			break;
	}

	checkf(false, TEXT("Missing ECheatPunishType Type: %d"), static_cast<const int32>(Response));
	return TEXT("");
}

/** 
 * Delegate called by the game to provide a response to the encryption key request
 * Provides the encryption key if successful, or a failure reason so the network connection may proceed
 *
 * @param Response the response from the game indicating the success or failure of retrieving an encryption key from an encryption token
 */
DECLARE_DELEGATE_OneParam(FOnEncryptionKeyResponse, const FEncryptionKeyResponse& /** Response */);

/** The different ways net sync loads can be triggered */
enum class ENetSyncLoadType : int32
{
	Unknown,
	PropertyReference,
	ActorSpawn
};

/**
 * Struct used as the parameter to FOnSyncLoadDetected to report sync loads.
 */
struct FNetSyncLoadReport
{
	FNetSyncLoadReport()
		: Type(ENetSyncLoadType::Unknown)
		, NetDriver(nullptr)
		, OwningObject(nullptr)
		, Property(nullptr)
		, LoadedObject(nullptr) {}

	/** How the sync load was triggered, if known */
	ENetSyncLoadType Type;

	/** The driver that reported this load */
	const UNetDriver* NetDriver;

	/** The replicated object, usually an actor, whose replication caused the sync load, if known. May be null. */
	const UObject* OwningObject;

	/** The replicated property that refers to the object that was sync loaded, if known. May be null. */
	const FProperty* Property;

	/** The object that was sync loaded */
	const UObject* LoadedObject;
};

class FNetDelegates
{

public:

	/**
	 * Delegate fired when an encryption key is required by the network layer in order to proceed with a connection with a client requesting a connection
	 *
	 * Binding to this delegate overrides UGameInstance's handling of encryption (@see ReceivedNetworkEncryptionToken)
	 *
	 * @param EncryptionToken token sent by the client that should be used to retrieve an encryption key
	 * @param Delegate delegate that MUST be fired after retrieving the encryption key in order to complete the connection handshake
	 */
	DECLARE_DELEGATE_TwoParams(FReceivedNetworkEncryptionToken, const FString& /*EncryptionToken*/, const FOnEncryptionKeyResponse& /*Delegate*/);
	static ENGINE_API FReceivedNetworkEncryptionToken OnReceivedNetworkEncryptionToken;

	/**
	 * Delegate fired when encryption has been setup and acknowledged by the host.  The client should setup their connection to continue future communication via encryption
	 *
	 * Binding to this delegate overrides UGameInstance's handling of encryption (@see ReceivedNetworkEncryptionAck)
	 *
	 * @param Delegate delegate that MUST be fired after retrieving the encryption key in order to complete the connection handshake
	 */
	DECLARE_DELEGATE_OneParam(FReceivedNetworkEncryptionAck, const FOnEncryptionKeyResponse& /*Delegate*/);
	static ENGINE_API FReceivedNetworkEncryptionAck OnReceivedNetworkEncryptionAck;

	/**
	 * Delegate fired when encryption has failed for a specific connection (client OR server) - allowing gameplay code to override how this is handled
	 * (e.g. to accept connections without encryption in development environments, and reject them in shipping environments).
	 *
	 * Binding to this delegate overrides UGameInstance's handling of encryption (@see ReceivedNetworkEncryptionFailure)
	 *
	 * @param Connection	The server or client NetConnection which failed encryption.
	 * @return				Returns how to handle the encryption failure.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(EEncryptionFailureAction, FReceivedNetworkEncryptionFailure, UNetConnection* /* Connection */);
	static ENGINE_API FReceivedNetworkEncryptionFailure OnReceivedNetworkEncryptionFailure;


	/**
	 * Delegate fires whenever a client cheater is detected in the networking code
	 *
	 * @param PlayerId net id of the cheating client
	 * @param PunishType type of punishment to apply to a cheating client
	 * @param ReasonStr why the client is being punished due to cheating
	 * @param InfoStr extra info about the punishment to be applied (eg. ban received, etc)
	 */
	DECLARE_DELEGATE_FourParams(FNetworkCheatDetected, const class FUniqueNetId& /*PlayerId*/, ECheatPunishType /*PunishType*/, const FString& /*ReasonStr*/, const FString& /*InfoStr*/);
	static ENGINE_API FNetworkCheatDetected OnNetworkCheatDetected;

	/**
	 * Delegate fired when a pending net game has created a UNetConnection to the server but hasn't sent the initial join message yet.
	 *
	 * @param PendingNetGame pointer to the PendingNetGame that is initializing its connection to a server.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPendingNetGameConnectionCreated, UPendingNetGame* /*PendingNetGame*/);
	static ENGINE_API FOnPendingNetGameConnectionCreated OnPendingNetGameConnectionCreated;

	/**
	 * Delegate fired when net.ReportSyncLoads is enabled and the replication system has determined which property or object caused the load.
	 * This is likely reported after the load itself, since the load can occur while parsing export bunches, but we don't know what property
	 * or object refers to the loaded object until the property itself is read.
	 *
	 * @param SyncLoadReport struct containing information about a sync load that occurred.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSyncLoadDetected, const FNetSyncLoadReport& /* SyncLoadReport */);
	static ENGINE_API FOnSyncLoadDetected OnSyncLoadDetected;
};
