// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/NetConnection.h"

class AActor;
class FArchive;

#ifndef UE_REPLICATED_OBJECT_REFCOUNTING
	// Allows every network actor to keep track of the number of channels an individual subobject was replicated. 
	// Not needed on clients so its compiled out to reduce memory usage.
	#define UE_REPLICATED_OBJECT_REFCOUNTING WITH_SERVER_CODE
#endif

#ifndef DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
	// Run additional code to validate any errors in the SubObject channel refcounting at the cost of extra memory usage and cpu overhead.
	#define DO_REPLICATED_OBJECT_CHANNELREF_CHECKS (UE_REPLICATED_OBJECT_REFCOUNTING && WITH_SERVER_CODE && UE_BUILD_DEVELOPMENT) 
#endif

/** Indicates the status of a replicated subobject replicated by actor channels. */
enum class ENetSubObjectStatus : uint8
{
	Active,  // The subobject is active and replicated
	TearOff, // The subobject should be torn off on the clients
	Delete,  // The subobject should be deleted on the clients
};

/**
 * Struct to store an actor pointer and any internal metadata for that actor used
 * internally by a UNetDriver.
 */
struct FNetworkObjectInfo
{
	/** Pointer to the replicated actor. */
	AActor* Actor;

	/** WeakPtr to actor. This is cached here to prevent constantly constructing one when needed for (things like) keys in TMaps/TSets */
	TWeakObjectPtr<AActor> WeakActor;

	/** Next time to consider replicating the actor. Based on FPlatformTime::Seconds(). */
	double NextUpdateTime;

	/** Last absolute time in seconds since actor actually sent something during replication */
	double LastNetReplicateTime;

	/** Optimal delta between replication updates based on how frequently actor properties are actually changing */
	float OptimalNetUpdateDelta;

	/** Last time this actor was updated for replication via NextUpdateTime
	* @warning: internal net driver time, not related to WorldSettings.TimeSeconds */
	double LastNetUpdateTimestamp;


	/**
	* Key definitions for TSet/TMap that works with invalidated weak pointers
	*/
	struct FNetConnectionKeyFuncs : BaseKeyFuncs<TWeakObjectPtr<UNetConnection>, const TWeakObjectPtr<UNetConnection>&, false>
	{
		static KeyInitType GetSetKey(ElementInitType& Element)			{ return Element; }
		static bool Matches(KeyInitType A, KeyInitType B)				{ return A.HasSameIndexAndSerialNumber(B); }
		static uint32 GetKeyHash(KeyInitType Key)						{ return GetTypeHash(Key); }
	};

	/** List of connections that this actor is dormant on */
	TSet<TWeakObjectPtr<UNetConnection>, FNetConnectionKeyFuncs> DormantConnections;

	/** A list of connections that this actor has recently been dormant on, but the actor doesn't have a channel open yet.
	*  These need to be differentiated from actors that the client doesn't know about, but there's no explicit list for just those actors.
	*  (this list will be very transient, with connections being moved off the DormantConnections list, onto this list, and then off once the actor has a channel again)
	*/
	TSet<TWeakObjectPtr<UNetConnection>, FNetConnectionKeyFuncs> RecentlyDormantConnections;

	/** Is this object still pending a full net update due to clients that weren't able to replicate the actor at the time of LastNetUpdateTime */
	uint8 bPendingNetUpdate : 1;

	/** Should this object be considered for replay checkpoint writes */
	uint8 bDirtyForReplay : 1;

	/** Should channel swap roles while calling ReplicateActor */
	uint8 bSwapRolesOnReplicate : 1;

	/** Force this object to be considered relevant for at least one update */
	uint32 ForceRelevantFrame = 0;

	FNetworkObjectInfo()
		: Actor(nullptr)
		, NextUpdateTime(0.0)
		, LastNetReplicateTime(0.0)
		, OptimalNetUpdateDelta(0.0f)
		, LastNetUpdateTimestamp(0.0)
		, bPendingNetUpdate(false)
		, bDirtyForReplay(false)
		, bSwapRolesOnReplicate(false) {}

	FNetworkObjectInfo(AActor* InActor)
		: Actor(InActor)
		, WeakActor(InActor)
		, NextUpdateTime(0.0)
		, LastNetReplicateTime(0.0)
		, OptimalNetUpdateDelta(0.0f) 
		, LastNetUpdateTimestamp(0.0)
		, bPendingNetUpdate(false)
		, bDirtyForReplay(false)
		, bSwapRolesOnReplicate(false) {}

	void CountBytes(FArchive& Ar) const;
};

/**
 * KeyFuncs to allow using the actor pointer as the comparison key in a set.
 */
struct FNetworkObjectKeyFuncs : BaseKeyFuncs<TSharedPtr<FNetworkObjectInfo>, AActor const*, false>
{
	/**
	 * @return The key used to index the given element.
	 */
	static KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.Get()->Actor;
	}

	/**
	 * @return True if the keys match.
	 */
	static bool Matches(KeyInitType A,KeyInitType B)
	{
		return A == B;
	}

	/** Calculates a hash index for a key. */
	static uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

/**
 * Stores the list of replicated actors for a given UNetDriver.
 */
class FNetworkObjectList
{
public:
	typedef TSet<TSharedPtr<FNetworkObjectInfo>, FNetworkObjectKeyFuncs> FNetworkObjectSet;

	/**
	 * Adds replicated actors in World to the internal set of replicated actors.
	 * Used when a net driver is initialized after some actors may have already
	 * been added to the world.
	 *
	 * @param World The world from which actors are added.
	 * @param NetDriver The net driver to which this object list belongs.
	 */
	ENGINE_API void AddInitialObjects(UWorld* const World, UNetDriver* NetDriver);

	/**
	 * Attempts to find the Actor's FNetworkObjectInfo.
	 * If no info is found, then the Actor will be added to the list, and will assumed to be active.
	 *
	 * If the Actor is dormant when this is called, it is the responsibility of the caller to call
	 * MarkDormant immediately.
	 *
	 * If info cannot be found or created, nullptr will be returned.
	 */
	ENGINE_API TSharedPtr<FNetworkObjectInfo>* FindOrAdd(AActor* const Actor, UNetDriver* NetDriver, bool* OutWasAdded=nullptr);

	/**
	 * Attempts to find the Actor's FNetworkObjectInfo.
	 *
	 * If info is not found (or the Actor is in an invalid state) an invalid TSharedPtr is returned.
	 */
	ENGINE_API TSharedPtr<FNetworkObjectInfo> Find(AActor* const Actor);
	const TSharedPtr<FNetworkObjectInfo> Find(const AActor* const Actor) const
	{
		return const_cast<FNetworkObjectList*>(this)->Find(const_cast<AActor* const>(Actor));
	}

	/** Removes actor from the internal list, and any cleanup that is necessary (i.e. resetting dormancy state) */
	ENGINE_API void Remove(const AActor* Actor);

	/** Removes an actor with inaccessible memory from the internal list, and any cleanup that is necessary (i.e. resetting dormancy state) */
	ENGINE_API void RemoveInvalid(const AActor* BadActor);

	/** Marks this object as dormant for the passed in connection */
	ENGINE_API void MarkDormant(AActor* const Actor, UNetConnection* const Connection, const int32 NumConnections, UNetDriver* NetDriver);

	/** Marks this object as active for the passed in connection */
	ENGINE_API bool MarkActive(AActor* const Actor, UNetConnection* const Connection, UNetDriver* NetDriver);

	/** Marks this object dirty for replays using delta checkpoints */
	ENGINE_API void MarkDirtyForReplay(AActor* const Actor);

	ENGINE_API void ResetReplayDirtyTracking();

	/** Removes the recently dormant status from the passed in connection */
	ENGINE_API void ClearRecentlyDormantConnection(AActor* const Actor, UNetConnection* const Connection, UNetDriver* NetDriver);

	/** Called when a replicated actor is about to be carried from one world to another */
	ENGINE_API void OnActorIsTraveling(AActor* TravelingAtor);

	/** Called when seamless traveling is almost done just before we initialize the new world */
	ENGINE_API void OnPostSeamlessTravel();

	/** 
	 *	Does the necessary house keeping when a new connection is added 
	 *	When a new connection is added, we must add all objects back to the active list so the new connection will process it
	 *	Once the objects is dormant on that connection, it will then be removed from the active list again
	*/
	ENGINE_API void HandleConnectionAdded();

	/** Clears all state related to dormancy */
	ENGINE_API void ResetDormancyState();

	/** Returns a const reference to the entire set of tracked actors. */
	const FNetworkObjectSet& GetAllObjects() const { return AllNetworkObjects; }

	/** Returns a const reference to the active set of tracked actors. */
	const FNetworkObjectSet& GetActiveObjects() const { return ActiveNetworkObjects; }

	/** Returns a const reference to the entire set of dormant actors. */
	const FNetworkObjectSet& GetDormantObjectsOnAllConnections() const { return ObjectsDormantOnAllConnections; }

	ENGINE_API int32 GetNumDormantActorsForConnection( UNetConnection* const Connection ) const;

	/** Force this actor to be relevant for at least one update */
	ENGINE_API void ForceActorRelevantNextUpdate(AActor* const Actor, UNetDriver* NetDriver);
		
	ENGINE_API void Reset();

	ENGINE_API void CountBytes(FArchive& Ar) const;

	/** Marks any actors in the given package/level active if they were fully dormant or dormant for the passed in connection */
	ENGINE_API void FlushDormantActors(UNetConnection* const Connection, const FName& PackageName);

	/** Called when the netdriver gets notified that an actor is destroyed */
	ENGINE_API void OnActorDestroyed(AActor* DestroyedActor);

#if UE_REPLICATED_OBJECT_REFCOUNTING

	/** Set the subobject to be flagged for deletion */
	ENGINE_API void SetSubObjectForDeletion(AActor* Actor, UObject* SubObject);

	/** Set the subobject to be flagged for tear off */
	ENGINE_API void SetSubObjectForTearOff(AActor* Actor, UObject* SubObject);

	/**
	* Called when a channel starts replicating a subobject for the first time.
	* Used to keep track of the number of channels having an active reference to a specific subobject
	*/
	ENGINE_API void AddSubObjectChannelReference(AActor* OwnerActor, UObject* ReplicatedSubObject, UObject* ReferenceOwner);

	/** Called when a channel stops replicating a subobject. */
	ENGINE_API void RemoveSubObjectChannelReference(AActor* OwnerActor, const TWeakObjectPtr<UObject>& ReplicatedSubObject, UObject* ReferenceOwner);

	/** Called when multiple subobjects need to remove their reference from either the active or inactive list */
	ENGINE_API void RemoveMultipleSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner);

	/** Called when multiple subobjects that were flagged torn off or delete have removed their channel reference */
	ENGINE_API void RemoveMultipleInvalidSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner);

	/** Called when multiple subobjects that were still considered active have removed their channel reference */
	ENGINE_API void RemoveMultipleActiveSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner);

	struct FActorInvalidSubObjectView;

	/** Returns a struct holding the dirty count and a possible list of invalid subobjects who still have references to specific connections.*/
	ENGINE_API FNetworkObjectList::FActorInvalidSubObjectView FindActorInvalidSubObjects(AActor* OwnerActor) const;

	/** 
	* Keep track of the transfer of ownership from the channel to the connection when the actor becomes dormant.
	* This is only needed to ensure the debug reference tracking is up to date.
	* No actual logic is modified here.
	*/
#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
	ENGINE_API void SwapMultipleReferencesForDormancy(AActor* OwnerActor, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToSwap, UActorChannel* PreviousChannelRefOwner, UNetConnection* NewConnectionRefOwner);
	ENGINE_API void SwapReferenceForDormancy(AActor* OwnerActor, UObject* ReplicatedSubObject, UNetConnection* PreviousConnectionRefOwner, UActorChannel* NewChannelRefOwner);
#endif 

#endif //#if UE_REPLICATED_OBJECT_REFCOUNTING

private:
	ENGINE_API bool MarkActiveInternal(const TSharedPtr<FNetworkObjectInfo>& ObjectInfo, UNetConnection* const Connection, UNetDriver* NetDriver);

#if UE_REPLICATED_OBJECT_REFCOUNTING
	ENGINE_API void InvalidateSubObject(AActor* Actor, UObject* SubObject, ENetSubObjectStatus InvalidStatus);

	struct FActorSubObjectReferences;
	ENGINE_API void HandleRemoveAnySubObjectChannelRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& ReplicatedSubObject, UObject* ReferenceOwner);
	ENGINE_API bool HandleRemoveActiveSubObjectRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& ReplicatedSubObject, UObject* ReferenceOwner);
	ENGINE_API bool HandleRemoveInvalidSubObjectRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& ReplicatedSubObject, UObject* ReferenceOwner);
#endif

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
	ENGINE_API void HandleSwapReferenceForDormancy(FActorSubObjectReferences* ActorNetInfo, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* PreviousRefOwner, UObject* NewRefOwner);
#endif

	FNetworkObjectSet AllNetworkObjects;
	FNetworkObjectSet ActiveNetworkObjects;
	FNetworkObjectSet ObjectsDormantOnAllConnections;

	/** Store the network info of actors that travel to the new world during a seamless travel. */
	FNetworkObjectSet SeamlessTravelingObjects;

	TMap<TWeakObjectPtr<UNetConnection>, int32 > NumDormantObjectsPerConnection;

	TMap<FName, FNetworkObjectSet> FullyDormantObjectsByLevel;
	TMap<TObjectKey<UNetConnection>, TMap<FName, FNetworkObjectSet>> DormantObjectsPerConnection;

public:

	/**
	* Keeps track of the number of channels that have replicated a subobject.
	* When the status is not Active anymore it is expected of existing references to be gradually removed as the actor replicates itself to each connection.
	*/
	struct FSubObjectChannelReference
	{
		/** The replicated subobject */
		TWeakObjectPtr<UObject> SubObjectPtr;

		/** Number of channels that replicated the subobject */
		uint16 ChannelRefCount = 0;

		/** Current status of the subobject */
		ENetSubObjectStatus Status = ENetSubObjectStatus::Active;

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		/**
		* Debug array to keep track of every individual references. Helps to trigger an ensure early if something wrong is detected.
		* Consists of UActorChannel for active replicators or UNetConnection's for dormant replicators
		*/
		TArray<UObject*> RegisteredOwners;
#endif

		inline bool operator==(const FSubObjectChannelReference& rhs) const { return SubObjectPtr.HasSameIndexAndSerialNumber(rhs.SubObjectPtr); }
		inline bool operator==(const TWeakObjectPtr<UObject>& rhs) const { return SubObjectPtr.HasSameIndexAndSerialNumber(rhs); }

		friend uint32 GetTypeHash(const FSubObjectChannelReference& SubObjChannelRef)
		{
			return GetTypeHash(SubObjChannelRef.SubObjectPtr);
		}

		inline bool IsTearOff() const { return Status == ENetSubObjectStatus::TearOff; }
		inline bool IsDelete() const { return Status == ENetSubObjectStatus::Delete; }
		inline bool IsActive() const { return Status == ENetSubObjectStatus::Active; }

		FSubObjectChannelReference() = default;
		explicit FSubObjectChannelReference(const TWeakObjectPtr<UObject>& InSubObject)
			: SubObjectPtr(InSubObject)
			, ChannelRefCount(1)
			, Status(ENetSubObjectStatus::Active)
		{
		}
	};

	/** Key definitions for TSet that works with invalided weak pointers */
	struct FSubObjectChannelRefKeyFuncs : BaseKeyFuncs<FSubObjectChannelReference, const TWeakObjectPtr<UObject>&, false>
	{
		static KeyInitType	GetSetKey(ElementInitType& Element)		{ return Element.SubObjectPtr; }
		static bool			Matches(KeyInitType A, KeyInitType B)	{ return A.HasSameIndexAndSerialNumber(B); }
		static uint32		GetKeyHash(KeyInitType Key)				{ return GetTypeHash(Key); }
	};

#if UE_REPLICATED_OBJECT_REFCOUNTING

	/** Structure giving const-only access to the list of invalid subobjects of a given actor */
	struct FActorInvalidSubObjectView
	{
	public:
		explicit FActorInvalidSubObjectView(uint16 InDirtyCount, const TArray<FSubObjectChannelReference>* InArrayPtr=nullptr)
			: InvalidSubObjectDirtyCount(InDirtyCount)
			, InvalidSubObjectsPtr(InArrayPtr)
		{ }

		FActorInvalidSubObjectView() = delete;

		inline uint16 GetDirtyCount() const { return InvalidSubObjectDirtyCount; }
		inline bool HasInvalidSubObjects() const { return InvalidSubObjectsPtr != nullptr; }
		inline const TArray<FSubObjectChannelReference>& GetInvalidSubObjects() const 
		{
			check(HasInvalidSubObjects());
			return *InvalidSubObjectsPtr;
		}

	private:
		/** The current dirty count */
		uint16 InvalidSubObjectDirtyCount = 0;

		/** The list of invalid subobjects that were set to be torn off or deleted */
		const TArray<FSubObjectChannelReference>* InvalidSubObjectsPtr;
	};

private:

	struct FActorSubObjectReferences
	{
		/** The actor who is replicating the subobjects */
		FObjectKey ActorKey;

		/** The set of active replicated subobjects for an actor */
		TSet<FSubObjectChannelReference, FSubObjectChannelRefKeyFuncs> ActiveSubObjectChannelReferences;

		/** The list of replicated subobjects that need to be torn off or deleted by the channels still referencing it */
		TArray<FSubObjectChannelReference> InvalidSubObjectChannelReferences;

		/**
		* This variable gets increased when a replicated subobject gets added to the invalid list.
		* Actor channels will test against this count and when it differs will check if they have any replicated subobjects to remove.
		*/
		uint16 InvalidSubObjectDirtyCount = 0;

		explicit FActorSubObjectReferences(AActor* InActor) : ActorKey(InActor) {}

		/** Returns true when the actor's subobjects have no more references and this entry itself can be deleted */
		bool HasNoSubObjects() const 
		{
			return ActiveSubObjectChannelReferences.IsEmpty() && InvalidSubObjectChannelReferences.IsEmpty();
		}
		
		void CountBytes(FArchive& Ar) const;
	};

	/** Definition to use the struct's actor as the key */
	struct FActorSubObjectRefKeyFuncs : BaseKeyFuncs<FActorSubObjectReferences, FObjectKey, false>
	{
		static KeyInitType	GetSetKey(ElementInitType& Element)		{ return Element.ActorKey; }
		static bool			Matches(KeyInitType A, KeyInitType B)	{ return A == B; }
		static uint32		GetKeyHash(KeyInitType Key)				{ return GetTypeHash(Key); }
	};

private:

	/** Map keeping track of the number of connections that have a reference to each actor's subobjects */
	TSet<FActorSubObjectReferences, FActorSubObjectRefKeyFuncs> SubObjectChannelReferences;

#endif //UE_REPLICATED_OBJECT_REFCOUNTING
};
