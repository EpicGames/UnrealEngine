// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "NetworkSettings.generated.h"

struct FPropertyChangedEvent;

USTRUCT()
struct FNetworkEmulationProfileDescription
{
	GENERATED_BODY()

	UPROPERTY()
	FString ProfileName;

	UPROPERTY()
	FString ToolTip;
};

/**
 * Network settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Network"), MinimalAPI)
class UNetworkSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	//! Default MaxRepArraySize @see MaxRepArraySize.
	static const int32 DefaultMaxRepArraySize = 2 * 1024;

	//! Default MaxRepArrayMemory @see MaxRepArrayMemory.
	static const int32 DefaultMaxRepArrayMemory = UINT16_MAX;

	UPROPERTY(config, EditAnywhere, Category=libcurl, meta=(
		ConsoleVariable="n.VerifyPeer",DisplayName="Verify Peer",
		ToolTip="If true, libcurl authenticates the peer's certificate. Disable to allow self-signed certificates."))
	uint32 bVerifyPeer:1;

	UPROPERTY(config, EditAnywhere, Category=World, meta = (
		ConsoleVariable = "p.EnableMultiplayerWorldOriginRebasing", DisplayName = "Enable Multiplayer World Origin Rebasing",
		ToolTip="If true, origin rebasing is enabled in multiplayer games, meaning that servers and clients can have different local world origins."))
	uint32 bEnableMultiplayerWorldOriginRebasing : 1;

	/** This lists the common network emulation profiles that will be selectable in PIE settings */
	UPROPERTY(config)
	TArray<FNetworkEmulationProfileDescription> NetworkEmulationProfiles;

public:

	//~ Begin UObject Interface

	ENGINE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	//~ End UObject Interface
};
