// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "NeuralProfile.generated.h"

#define MAX_NEURAL_PROFILE_COUNT 64


UENUM(BlueprintType)
enum class ENeuralProfileFormat : uint8
{
	Type32 UMETA(DisplayName = "32bit"),
	Type16 UMETA(DisplayName = "16bit"),
};

UENUM(BlueprintType)
enum class ENeuralModelTileType : uint8
{
	/** The NNE model is loaded and used as it is. No dimension augmentation. E.g.,
if the input texture has different dimensions, it will be scaled down before application*/
	OneByOne		UMETA(DisplayName = "1x1"),

	TwoByTwo		UMETA(DisplayName = "2x2"),
	
	FourByFour		UMETA(DisplayName = "4x4"),
	
	EightByEight	UMETA(DisplayName = "8x8"),

	/** Create tiled buffers in batch dimension automatically, where each tile runs the neural model
	e.g., if the model input dimension is (1x3x200x200) and the used buffer size of the post processing
	is 1000x1000, then 5x5 tiles ((5x5)x3x200x200) will be run and recombined.
	*/
	Auto			UMETA(DisplayName = "Auto")
};

UENUM(BlueprintType)
enum class ETileOverlapResolveType : uint8
{
	/** Overlapped tile regions have no contribution to adjecent tiles*/
	Ignore			UMETA(DisplayName = "Ignore"),

	/** Overlapped regions are blended linearly to adjecent tiles*/
	Feathering		UMETA(DisplayName = "Feathering")
};


UENUM(BlueprintType)
enum class ENeuralProfileRuntimeType : uint8
{
	NNERuntimeORTDml UMETA(DisplayName = "NNERuntimeORTDml"),

	/** Does not have full operator support*/
	NNERuntimeRDGHlsl UMETA(DisplayName = "NNERuntimeRDGHlsl"),

	MAX				 UMETA(Hidden)
};

// struct with all the settings we want in UNeuralProfile, separate to make it easer to pass this data around in the engine.
USTRUCT(BlueprintType)
struct FNeuralProfileStruct
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * Define the expected input format, if any output from material is not this format, a custom conversion
	 * will be applied for this conversion.
	 */
	UPROPERTY(Category = "Common", EditAnywhere, BlueprintReadOnly, meta=(DisplayName="Input Format", editcondition = "false", EditConditionHides))
	ENeuralProfileFormat InputFormat;

	/**
	 * Define the expected output format. A conversion between the output format and the actual format will
	 * be applied automatically.
	 */
	UPROPERTY(Category = "Common", EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "Output Format", editcondition = "false", EditConditionHides))
	ENeuralProfileFormat OutputFormat;

	//runtime type (support "NNERuntimeORTDml" only at this moment)
	UPROPERTY(Category = "Model", EditAnywhere, BlueprintReadOnly)
	ENeuralProfileRuntimeType RuntimeType;

	/** Stores the NNEModelData imported from e.g., onnx model */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Model, meta = (DisplayName = "NNE Model Data"))
	TObjectPtr<UObject> NNEModelData;

	/** Input dimension of the NNEModelData model */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Model, meta = (DisplayName = "Input Dimension", editcondition = "false"))
	FIntVector4 InputDimension;

	/** Output dimension of the NNEModelData model */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Model, meta = (DisplayName = "Output Dimension", editcondition = "false"))
	FIntVector4 OutputDimension;

	/** Used to override the batch size if the batch dimension is dynamic (-1)*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Override", meta = (DisplayName = "Batch Size"))
	int32 BatchSizeOverride;

	/** Total tiles used. Each tile will be executed by 1 batch */
	UPROPERTY(EditAnywhere, Category = "Tile")
	ENeuralModelTileType TileSize;

	/** Tile border overlaps (Left|Right, Top|Bottom). The larger this value, the more tiles are required to cover the whole screen when TileSize is Auto.*/
	UPROPERTY(EditAnywhere, Category = "Tile", meta = (DisplayName = "Border Overlaps"))
	FIntPoint TileOverlap;

	UPROPERTY(EditAnywhere, Category = "Tile", meta = (DisplayName = "Overlap Resolve Type"))
	ETileOverlapResolveType TileOverlapResolveType;

	FNeuralProfileStruct()
	{
		InputFormat = ENeuralProfileFormat::Type32;
		OutputFormat = ENeuralProfileFormat::Type32;
		RuntimeType = ENeuralProfileRuntimeType::NNERuntimeORTDml;
		NNEModelData = nullptr;
		InputDimension = FIntVector4(0);
		OutputDimension = FIntVector4(0);
		TileSize = ENeuralModelTileType::OneByOne;
		BatchSizeOverride = 1;
		TileOverlap = FIntPoint(0, 0);
		TileOverlapResolveType = ETileOverlapResolveType::Ignore;
	}

	void Invalidate()
	{
		*this = FNeuralProfileStruct();
	}
};

/**
 * This class represents assets that stores the neural network model data and the conversion from/to the model.
 *
 * Neural network model profile typically consist of a neural network model data and the config of preprocessing/post-processing
 * of applying this neural network model.
 */
UCLASS(autoexpandcategories = NeuralProfile, MinimalAPI)
class UNeuralProfile : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(Category = UNeuralProfile, EditAnywhere, meta = (ShowOnlyInnerProperties))
	struct FNeuralProfileStruct Settings;

	UPROPERTY()
	FGuid Guid;

	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

namespace NeuralProfile
{
	// Profile - Initializes or updates the contents of the neural profile.
	ENGINE_API int32 AddOrUpdateProfile(const UNeuralProfile* InProfile, const FGuid& InGuid, const FNeuralProfileStruct InSettings);

	ENGINE_API FNeuralProfileStruct GetProfileSetting(int32 AllocationId);

	// Profile - Returns the profile ID for a given neural profile object
	ENGINE_API int32 GetNeuralProfileId(const UNeuralProfile* In);

	// Experimental APIs to manage neural profiles.
	class INeuralProfileManager
	{
	public:
		virtual ~INeuralProfileManager() = default;

		virtual void UpdateModel(int32 AllocationId, UObject* NNEModelData, FString RuntimeName) = 0;
		virtual void RemoveModel(int32 AllocationId) = 0;

		virtual void UpdateTileType(int32 AllocationId, ENeuralModelTileType ModelTileSize) = 0;
		virtual bool UpdateBatchSize(int32 AllocationId, int32 BatchSize) = 0;
		virtual void UpdateTileOverlap(int32 AllocationId, FIntPoint TileOverlap) = 0;
		virtual void UpdateTileOverlapResolveType(int32 AllocationId, ETileOverlapResolveType TileOverlapResolveType) = 0;

		virtual FIntVector4 GetInputDimension(UObject* NNEModelData, FString RuntimeName) = 0;
		virtual FIntVector4 GetOutputDimension(UObject* NNEModelData, FString RuntimeName) = 0;		
	};
}

extern ENGINE_API TUniquePtr<NeuralProfile::INeuralProfileManager> GNeuralProfileManager;
