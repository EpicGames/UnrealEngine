// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * A sticky note.  Level designers can place these in the level and then
 * view them as a batch in the error/warnings window.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Note.generated.h"

UCLASS(MinimalAPI, hidecategories = (Input), showcategories=("Input|MouseInput", "Input|TouchInput"))
class ANote : public AActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Note, meta = (MultiLine = "true"))
	FString Text;

	// Reference to sprite visualization component
private:
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;

	// Reference to arrow visualization component
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
public:

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	//~ Begin AActor Interface
	ENGINE_API virtual void CheckForErrors() override;
	//~ End AActor Interface
#endif

#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	class UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif
};



