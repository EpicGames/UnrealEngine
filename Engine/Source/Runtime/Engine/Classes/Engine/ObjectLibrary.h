// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetRegistry/AssetData.h"
#include "ObjectLibrary.generated.h"

struct FPropertyChangedEvent;

/** Class that holds a library of Objects */
UCLASS(MinimalAPI)
class UObjectLibrary : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Class that Objects must be of. If ContainsBlueprints is true, this is the native class that the blueprints are instances of and not UClass  */
	UPROPERTY(EditAnywhere, Category=ObjectLibrary, meta=(AllowAbstract = ""))
	TObjectPtr<UClass>				ObjectBaseClass;

	/** True if this library holds blueprint classes, false if it holds other objects */
	UPROPERTY(EditAnywhere, Category=ObjectLibrary)
	bool bHasBlueprintClasses;

protected:

	/** List of Objects in library */
	UPROPERTY(EditAnywhere, Category=ObjectLibrary)
	TArray<TObjectPtr<UObject>>	Objects;

	/** Weak pointers to objects */
	UPROPERTY()
	TArray<TWeakObjectPtr <UObject> >	WeakObjects;

	/** If this library should use weak pointers */
	UPROPERTY(Transient)
	bool bUseWeakReferences;

	/** True if we've already fully loaded this library, can't do it twice */
	UPROPERTY(Transient)
	bool bIsFullyLoaded;

	/** Asset data of objects that will belong in library, possibly not loaded yet */
	TArray<FAssetData>	AssetDataList;

public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	/** 
	 * Static function to create a new ObjectLibrary at runtime, with various options set
	 * There is now a better version of this functionality in AssetManager, if you are creating many game-specific libraries you should switch to using AssetManager instead
	 * @param	InBaseClass				Only objects of this class can exist in the library
	 * @param	bInHasBlueprintClasses	If true, this library contains blueprint classes derived from BaseClass, will convert them correctly
	 * @param	InUseWeak				If true, references to objects are weak, so they can be garbage collected. Useful in the editor to allow deletion
	 */
	ENGINE_API static class UObjectLibrary* CreateLibrary(UClass* InBaseClass, bool bInHasBlueprintClasses, bool bInUseWeak);

	/** Set rather this library is using weak or strong references */
	virtual void UseWeakReferences(bool bSetUseWeak);

	/** Attempt to add a new Object, return true if added */
	virtual bool AddObject(UObject *NewObject);

	/** Attempt to remove an Object from the object, return true if removed */
	virtual bool RemoveObject(UObject *ObjectToRemove);

	DECLARE_EVENT_OneParam(UObjectLibrary, FObjectLibraryOnObjectAdded, UObject* /*NewObject*/);
	FObjectLibraryOnObjectAdded& OnObjectAdded() { return OnObjectAddedEvent; }

	DECLARE_EVENT_OneParam(UObjectLibrary, FObjectLibraryOnObjectRemoved, UObject* /*ObjectToRemove*/);
	FObjectLibraryOnObjectRemoved& OnObjectRemoved() { return OnObjectRemovedEvent; }

	/** Fills in a passed in array of objects, casts to proper type */
	template <typename T>
	void GetObjects(TArray<T*>& OutObjects)
	{
		for (int32 i = 0; i < Objects.Num(); i++)
		{
			T* Obj = Cast<T>(Objects[i]);
			if (Obj)
			{
				OutObjects.Add(Obj);
			}
		}

		for (int32 i = 0; i < WeakObjects.Num(); i++)
		{
			T* Obj = Cast<T>(WeakObjects[i].Get());
			if (Obj)
			{
				OutObjects.Add(Obj);
			}
		}
	}

	/** Returns the number of objects */
	int32 GetObjectCount() const 
	{
		return Objects.Num() + WeakObjects.Num();
	}

	/** Returns the list of asset data */
	virtual void GetAssetDataList(TArray<FAssetData>& OutAssetData);

	/** Returns the number of objects */
	int32 GetAssetDataCount() const 
	{
		return AssetDataList.Num();
	}

	bool IsLibraryFullyLoaded() const
	{
		return bIsFullyLoaded;
	}

	/** Clears the current loaded objects and asset data */
	virtual void ClearLoaded();

	/** Load an entire subdirectory of assets into this object library. Returns number of assets loaded */
	virtual int32 LoadAssetsFromPaths(const TArray<FString>& Paths);

	virtual int32 LoadAssetsFromPath(const FString& Path)
	{
		TArray<FString> Paths;
		Paths.Add(Path);
		return LoadAssetsFromPaths(Paths);
	}

	/** Load an entire subdirectory of blueprints into this object library. Only loads blueprints of passed in class. Returns number of assets loaded */
	virtual int32 LoadBlueprintsFromPaths(const TArray<FString>& Paths);

	virtual int32 LoadBlueprintsFromPath(const FString& Path)
	{
		TArray<FString> Paths;
		Paths.Add(Path);
		return LoadBlueprintsFromPaths(Paths);
	}

	/** Gets asset data for assets in a subdirectory. Returns number of assets data loaded */
	virtual int32 LoadAssetDataFromPaths(const TArray<FString>& Paths, bool bForceSynchronousScan = true);

	virtual int32 LoadAssetDataFromPath(const FString& Path)
	{
		TArray<FString> Paths;
		Paths.Add(Path);
		return LoadAssetDataFromPaths(Paths);
	}

	/** Load an entire subdirectory of blueprints into this object library. Only loads asset data for blueprints of passed in class. Returns number of asset data loaded */
	virtual int32 LoadBlueprintAssetDataFromPaths(const TArray<FString>& Paths, bool bForceSynchronousScan = true);

	virtual int32 LoadBlueprintAssetDataFromPath(const FString& Path)
	{
		TArray<FString> Paths;
		Paths.Add(Path);
		return LoadBlueprintAssetDataFromPaths(Paths);
	}

	/** Load all of the objects in asset data list into memory */
	virtual int32 LoadAssetsFromAssetData();

	/** Whether to set bIncludeOnlyOnDiskAssets when setting up AR filters for discovering asset data */
	bool bIncludeOnlyOnDiskAssets;

	/** Whether to set bRecursivePaths when setting up AR filters for discovering asset data */
	bool bRecursivePaths;

#if WITH_EDITOR
	/** The paths that we will query again once assets are finished being discovered */
	TArray<FString> DeferredAssetDataPaths;

	/** True if we are running a build that is already scanning assets globally so we can perhaps avoid scanning paths synchronously */
	bool bIsGlobalAsyncScanEnvironment;

	/** Handler for when assets have finished scanning in the asset registry */
	virtual void OnAssetRegistryFilesLoaded();
#endif // WITH_EDITOR

protected:
	/** Delegates for when an object is added or removed from this library. This happens when objects are being loaded by the library if represented by asset data. */
	FObjectLibraryOnObjectAdded OnObjectAddedEvent;
	FObjectLibraryOnObjectRemoved OnObjectRemovedEvent;
};
