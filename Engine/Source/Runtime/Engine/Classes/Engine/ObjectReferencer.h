// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Helper class to force object references for various reasons.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ObjectReferencer.generated.h"

UCLASS()
class UObjectReferencer : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Array of objects being referenced. */
	UPROPERTY(EditAnywhere, Category=ObjectReferencer)
	TArray<TObjectPtr<class UObject>> ReferencedObjects;

};

