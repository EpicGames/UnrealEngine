// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/PhysicsObject.h"
#include "Engine/ActorInstanceHandle.h"
#include "OverlapResult.generated.h"

/** Structure containing information about one hit of an overlap test */
USTRUCT()
struct FOverlapResult
{
	GENERATED_BODY()

	UPROPERTY()
	FActorInstanceHandle OverlapObjectHandle;

	/** PrimitiveComponent that the check hit. */
	UPROPERTY()
	TWeakObjectPtr<class UPrimitiveComponent> Component;

	/** This is the index of the overlapping item.
		For DestructibleComponents, this is the ChunkInfo index.
		For SkeletalMeshComponents this is the Body index or INDEX_NONE for single body */
	int32 ItemIndex;

	/** Index of the body on the *querying* component that produced this overlap (INDEX_NONE for single-body components). */
	int32 MyItemIndex;

	UE_INTERNAL int32 GetItemIndexInternal() const;
	ENGINE_API int32 GetItemIndex() const;
	ENGINE_API void SetItemIndex(const int32 InItemIndex);

	/** Utility to return the Actor that owns the Component that was hit */
	AActor* GetActor() const;

	/** Utility to return the Component that was hit */
	UPrimitiveComponent* GetComponent() const;

	/** The object that owns the PhysicsObject. This is used to determine if the PhysicsObject is still valid when not owned by an Actor */
	TWeakObjectPtr<UObject> PhysicsObjectOwner;

	/** PhysicsObjects hit by the query. Not exposed to blueprints for the time being */
	Chaos::FPhysicsObjectHandle PhysicsObject;

	/** Indicates if this hit was requesting a block - if false, was requesting a touch instead */
	UPROPERTY()
	uint32 bBlockingHit : 1;

	FOverlapResult()
	{
		FMemory::Memzero(this, sizeof(FOverlapResult));
		MyItemIndex = INDEX_NONE;
	}
};

// All members of FOverlapResult are PODs.
template<> struct TIsPODType<FOverlapResult> { enum { Value = true }; };

inline AActor* FOverlapResult::GetActor() const
{
	return OverlapObjectHandle.FetchActor();
}

inline UPrimitiveComponent* FOverlapResult::GetComponent() const
{
	return Component.Get();
}
