// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * PackageMap implementation that is client/connection specific. This subclass implements all NetGUID Acking and interactions with a UConnection.
 *	On the server, each client will have their own instance of UPackageMapClient.
 *
 *	UObjects are first serialized as <NetGUID, Name/Path> pairs. UPackageMapClient tracks each NetGUID's usage and knows when a NetGUID has been ACKd.
 *	Once ACK'd, objects are just serialized as <NetGUID>. The result is higher bandwidth usage upfront for new clients, but minimal bandwidth once
 *	things gets going.
 *
 *	A further optimization is enabled by default. References will actually be serialized via:
 *	<NetGUID, <(Outer *), Object Name> >. Where Outer * is a reference to the UObject's Outer.
 *
 *	The main advantages from this are:
 *		-Flexibility. No precomputed net indices or large package lists need to be exchanged for UObject serialization.
 *		-Cross version communication. The name is all that is needed to exchange references.
 *		-Efficiency in that a very small % of UObjects will ever be serialized. Only Objects that serialized are assigned NetGUIDs.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/NetworkGuid.h"
#include "Misc/NetworkVersion.h"
#include "UObject/CoreNet.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Net/DataBunch.h"
#include "Net/NetAnalyticsTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/LowLevelMemTracker.h"

#include "PackageMapClient.generated.h"

LLM_DECLARE_TAG_API(GuidCache, ENGINE_API);

class UNetConnection;
class UNetDriver;
enum class EChannelGetAdditionalRequiredBunchesFlags : uint32;

class FNetFieldExport
{
public:
	FNetFieldExport() : Blob ( {} ), Handle( 0 ), CompatibleChecksum( 0 ), bExported( false ),  bDirtyForReplay( true ), bExportBlob( false ), bIncompatible( false )
	{
	}

	FNetFieldExport( const uint32 InHandle, const uint32 InCompatibleChecksum, const FName& InName ) :
		Blob ( {} ),
		Handle( InHandle ),
		CompatibleChecksum( InCompatibleChecksum ),
		ExportName( InName ),
		bExported( false ),
		bDirtyForReplay( true ),
		bExportBlob( false ),
		bIncompatible( false )
	{
	}

	FNetFieldExport( const uint32 InHandle, const uint32 InCompatibleChecksum, const FName& InName, const TArray<uint8>& InBlob ) :
		Blob ( InBlob ),
		Handle( InHandle ),
		CompatibleChecksum( InCompatibleChecksum ),
		ExportName( InName ),
		bExported( false ),
		bDirtyForReplay( true ),
		bExportBlob( true ),
		bIncompatible( false )
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FNetFieldExport& C);

	ENGINE_API void CountBytes(FArchive& Ar) const;
	
	TArray<uint8>	Blob;
	uint32			Handle;
	uint32			CompatibleChecksum;
	FName			ExportName;
	bool			bExported;
	bool			bDirtyForReplay;
	bool			bExportBlob;
	

	// Transient properties
	mutable bool	bIncompatible;		// If true, we've already determined that this property isn't compatible. We use this to curb warning spam.
};

class FNetFieldExportGroup
{
public:
	FNetFieldExportGroup() : PathNameIndex( 0 ), bDirtyForReplay( true ) { }

	FString						PathName;
	uint32						PathNameIndex;
	TArray< FNetFieldExport >	NetFieldExports;
	bool						bDirtyForReplay;

	friend FArchive& operator<<(FArchive& Ar, FNetFieldExportGroup& C);

	int32 FindNetFieldExportHandleByChecksum(const uint32 Checksum) const
	{
		for (int32 i = 0; i < NetFieldExports.Num(); i++)
		{
			if (NetFieldExports[i].CompatibleChecksum == Checksum)
			{
				return i;
			}
		}

		return INDEX_NONE;
	}

	ENGINE_API void CountBytes(FArchive& Ar) const;
};

/** Stores an object with path associated with FNetworkGUID */
class FNetGuidCacheObject
{
public:
	FNetGuidCacheObject() : NetworkChecksum( 0 ), ReadOnlyTimestamp( 0 ), bNoLoad( 0 ), bIgnoreWhenMissing( 0 ), bIsPending( 0 ), bIsBroken( 0 ), bDirtyForReplay( 1 )
	{
	}

	TWeakObjectPtr< UObject >	Object;

	// These fields are set when this guid is static
	FNetworkGUID				OuterGUID;
	FName						PathName;
	uint32						NetworkChecksum;			// Network checksum saved, used to determine backwards compatible

	double						ReadOnlyTimestamp;			// Time in second when we should start timing out after going read only

	uint8						bNoLoad				: 1;	// Don't load this, only do a find
	uint8						bIgnoreWhenMissing	: 1;	// Don't warn when this asset can't be found or loaded
	uint8						bIsPending			: 1;	// This object is waiting to be fully loaded
	uint8						bIsBroken			: 1;	// If this object failed to load, then we set this to signify that we should stop trying
	uint8						bDirtyForReplay		: 1;	// If this object has been modified, used by replay checkpoints
};

enum class EAppendNetExportFlags : uint32
{
	None = 0,
	ForceExportDirtyGroups = (1 << 0),
};

ENUM_CLASS_FLAGS(EAppendNetExportFlags);


/** Convenience type for holding onto references to objects while we have queued bunches referring to those objects. */
struct FQueuedBunchObjectReference
{
private:

	friend class FNetGUIDCache;

	FQueuedBunchObjectReference(const FNetworkGUID InNetGUID, UObject* InObject) :
		NetGUID(InNetGUID),
		Object(InObject)
	{
	}

	FNetworkGUID NetGUID;
	TObjectPtr<UObject> Object;
};

namespace UE::Net::Private
{
	/** Simple class to manage a reference-counted array of FNetworkGUIDs */
	class FRefCountedNetGUIDArray
	{
	public:
		FRefCountedNetGUIDArray() {}
		FRefCountedNetGUIDArray(const FRefCountedNetGUIDArray&) = delete;
		FRefCountedNetGUIDArray& operator=(const FRefCountedNetGUIDArray&) = delete;

		FRefCountedNetGUIDArray(FRefCountedNetGUIDArray&& Other) = default;
		FRefCountedNetGUIDArray& operator=(FRefCountedNetGUIDArray&& Other) = default;
		~FRefCountedNetGUIDArray() = default;

		void Add(FNetworkGUID NetGUID);
		void RemoveSwap(FNetworkGUID NetGUID);

		const TArray<FNetworkGUID>& GetNetGUIDs() const { return NetGUIDs; }

	private:
		TArray<FNetworkGUID> NetGUIDs;
		TArray<int32> RefCounts;
	};
}

class FNetGUIDCache
{
public:
	ENGINE_API FNetGUIDCache(UNetDriver* InDriver, uint64 NetworkGuidSeed = 0);
	virtual ~FNetGUIDCache() = default;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNetGUIDCache(FNetGUIDCache&&) = default;
	FNetGUIDCache(const FNetGUIDCache&) = delete;
	FNetGUIDCache& operator=(FNetGUIDCache&&) = default;
	FNetGUIDCache& operator=(const FNetGUIDCache&) = delete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	enum class ENetworkChecksumMode : uint8
	{
		None			= 0,		// Don't use checksums
		SaveAndUse		= 1,		// Save checksums in stream, and use to validate while loading packages
		SaveButIgnore	= 2,		// Save checksums in stream, but ignore when loading packages
	};

	enum class EAsyncLoadMode : uint8
	{
		UseCVar			= 0,		// Use CVar (net.AllowAsyncLoading) to determine if we should async load
		ForceDisable	= 1,		// Disable async loading
		ForceEnable		= 2,		// Force enable async loading
	};

	ENGINE_API void			CleanReferences();
	ENGINE_API bool			SupportsObject( const UObject* Object, const TWeakObjectPtr<UObject>* WeakObjectPtr=nullptr /** Optional: pass in existing weakptr to prevent this function from constructing one internally */ ) const;
	ENGINE_API bool			IsDynamicObject( const UObject* Object );
	ENGINE_API virtual bool	IsNetGUIDAuthority() const;
	ENGINE_API FNetworkGUID	GetOrAssignNetGUID( UObject* Object, const TWeakObjectPtr<UObject>* WeakObjectPtr=nullptr /** Optional: pass in existing weakptr to prevent this function from constructing one internally */ );
	ENGINE_API FNetworkGUID	GetNetGUID( const UObject* Object ) const;
	ENGINE_API FNetworkGUID	GetOuterNetGUID( const FNetworkGUID& NetGUID ) const;
	ENGINE_API virtual FNetworkGUID	AssignNewNetGUID_Server( UObject* Object );
	ENGINE_API virtual FNetworkGUID	AssignNewNetGUIDFromPath_Server( const FString& PathName, UObject* ObjOuter, UClass* ObjClass );
	ENGINE_API void			RegisterNetGUID_Internal( const FNetworkGUID& NetGUID, const FNetGuidCacheObject& CacheObject );
	ENGINE_API void			RegisterNetGUID_Server( const FNetworkGUID& NetGUID, UObject* Object );
	ENGINE_API void			RegisterNetGUID_Client( const FNetworkGUID& NetGUID, const UObject* Object );
	ENGINE_API void			RegisterNetGUIDFromPath_Client( const FNetworkGUID& NetGUID, const FString& PathName, const FNetworkGUID& OuterGUID, const uint32 NetworkChecksum, const bool bNoLoad, const bool bIgnoreWhenMissing );
	ENGINE_API void			RegisterNetGUIDFromPath_Server( const FNetworkGUID& NetGUID, const FString& PathName, const FNetworkGUID& OuterGUID, const uint32 NetworkChecksum, const bool bNoLoad, const bool bIgnoreWhenMissing );
	ENGINE_API UObject *		GetObjectFromNetGUID( const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped );
	ENGINE_API bool			ShouldIgnoreWhenMissing( const FNetworkGUID& NetGUID ) const;
	ENGINE_API FNetGuidCacheObject const * const GetCacheObject(const FNetworkGUID& NetGUID) const;
	ENGINE_API bool			IsGUIDRegistered( const FNetworkGUID& NetGUID ) const;
	ENGINE_API bool			IsGUIDLoaded( const FNetworkGUID& NetGUID ) const;
	ENGINE_API bool			IsGUIDBroken( const FNetworkGUID& NetGUID, const bool bMustBeRegistered ) const;
	ENGINE_API bool			IsGUIDNoLoad( const FNetworkGUID& NetGUID ) const;
	ENGINE_API bool			IsGUIDPending( const FNetworkGUID& NetGUID ) const;
	ENGINE_API FString			FullNetGUIDPath( const FNetworkGUID& NetGUID ) const;
	ENGINE_API void			GenerateFullNetGUIDPath_r( const FNetworkGUID& NetGUID, FString& FullPath ) const;
	ENGINE_API uint32			GetClassNetworkChecksum( UClass* Class );
	ENGINE_API uint32			GetNetworkChecksum( UObject* Obj );
	ENGINE_API void			SetNetworkChecksumMode( const ENetworkChecksumMode NewMode );
	ENGINE_API void			SetAsyncLoadMode( const EAsyncLoadMode NewMode );
	ENGINE_API bool			ShouldAsyncLoad() const;
	ENGINE_API bool			CanClientLoadObject( const UObject* Object, const FNetworkGUID& NetGUID ) const;
	ENGINE_API FString			Describe(const FNetworkGUID& NetGUID) const;

	ENGINE_API bool 		RemoveNetGUID(const UObject* Object);

	/** Remove an actor, it's components and all sub-object NetGUIDs from the NetGUID cache. */
	ENGINE_API void 		RemoveActorNetGUIDs(const AActor* Actor);

	ENGINE_API void			AsyncPackageCallback(const FName& PackageName, UPackage * Package, EAsyncLoadingResult::Type Result);

	ENGINE_API void			ResetCacheForDemo();

	ENGINE_API void			CountBytes(FArchive& Ar) const;

	ENGINE_API void ConsumeAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics& Out);
	ENGINE_API const FNetAsyncLoadDelinquencyAnalytics& GetAsyncLoadDelinquencyAnalytics() const;
	ENGINE_API void ResetAsyncLoadDelinquencyAnalytics();

	ENGINE_API void CollectReferences(class FReferenceCollector& ReferenceCollector);
	ENGINE_API TSharedRef<FQueuedBunchObjectReference> TrackQueuedBunchObjectReference(const FNetworkGUID InNetGUID, UObject* InObject);

	bool WasGUIDSyncLoaded(FNetworkGUID NetGUID) const { return SyncLoadedGUIDs.Contains(NetGUID); }
	void ClearSyncLoadedGUID(FNetworkGUID NetGUID) { SyncLoadedGUIDs.Remove(NetGUID); }

	/**
	 * If LogNetSyncLoads is enabled, log all objects that caused a sync load that haven't been otherwise reported
	 * by the package map yet, and clear that list.
	 */
	ENGINE_API void ReportSyncLoadedGUIDs();

	ENGINE_API void ResetReplayDirtyTracking();

	ENGINE_API const TArray<FNetworkGUID>* FindUnmappedStablyNamedGuidsWithOuter(FNetworkGUID OuterGUID) const;
	const void RemoveUnmappedStablyNamedGuidsWithOuter(FNetworkGUID OuterGUID) { UnmappedStablyNamedGuids_OuterToInner.Remove(OuterGUID); }

	TMap< FNetworkGUID, FNetGuidCacheObject >		ObjectLookup;
	TMap< TWeakObjectPtr< UObject >, FNetworkGUID >	NetGUIDLookup;

	UE_DEPRECATED(5.3, "No longer used")
	int32											UniqueNetIDs[2];

	TSet< FNetworkGUID >							ImportedNetGuids;
	TMap< FNetworkGUID, TSet< FNetworkGUID > >		PendingOuterNetGuids;

	UNetDriver *									Driver;

	ENetworkChecksumMode							NetworkChecksumMode;
	EAsyncLoadMode									AsyncLoadMode;

	bool											IsExportingNetGUIDBunch;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	static FNetworkGUID GetNetworkGUIDFromRemoteObjectId(const FRemoteObjectId RemoteObjectId, const bool bIsStatic);
	static FRemoteObjectId GetRemoteObjectIdFromNetworkGUID(const FNetworkGUID NetGuid);
#endif

private:

	friend class UPackageMapClient;

	uint64 NetworkGuidIndex[2];

	TMap<FName, FNetworkGUID> PendingAsyncPackages;

	/** Maps net field export group name to the respective FNetFieldExportGroup */
	TMap < FString, TSharedPtr< FNetFieldExportGroup > >	NetFieldExportGroupMap;

	/** Maps field export group path to assigned index */
	TMap < FString, uint32 >								NetFieldExportGroupPathToIndex;

	/** Maps assigned net field export group index to pointer to group, lifetime of the referenced FNetFieldExportGroups are managed by NetFieldExportGroupMap **/
	TMap < uint32, FNetFieldExportGroup* >					NetFieldExportGroupIndexToGroup;

	/** Current index used when filling in NetFieldExportGroupPathToIndex/NetFieldExportGroupIndexToPath */
	int32													UniqueNetFieldExportGroupPathIndex;

	/** Store all GUIDs that caused the sync loading of a package, for debugging & logging with LogNetSyncLoads */
	TArray<FNetworkGUID> SyncLoadedGUIDs;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
public:

	static ENGINE_API const bool IsHistoryEnabled();

	// History for debugging entries in the guid cache
	TMap<FNetworkGUID, FString>						History;
private:
#endif

	struct FPendingAsyncLoadRequest
	{
		FPendingAsyncLoadRequest(const FNetworkGUID InNetGUID, const double InRequestStartTime):
			RequestStartTime(InRequestStartTime)
		{
			NetGUIDs.Add(InNetGUID);
		}

		void Merge(const FPendingAsyncLoadRequest& Other)
		{
			for (FNetworkGUID OtherGUID : Other.NetGUIDs)
			{
				NetGUIDs.AddUnique(OtherGUID);
			}

#if CSV_PROFILER_STATS
			bWasRequestedByOwnerOrPawn |= Other.bWasRequestedByOwnerOrPawn;
#endif
		}

		void Merge(FNetworkGUID InNetGUID)
		{
			NetGUIDs.AddUnique(InNetGUID);
		}

		// Net GUIDs that requested loading for the same UPackage
		TArray<FNetworkGUID, TInlineAllocator<2>> NetGUIDs;

		double RequestStartTime;

#if CSV_PROFILER_STATS
		bool bWasRequestedByOwnerOrPawn = false;
#endif
	};

	/** Set of packages that are currently pending Async loads, referenced by package name. */
	TMap<FName, FPendingAsyncLoadRequest> PendingAsyncLoadRequests;

	FNetAsyncLoadDelinquencyAnalytics DelinquentAsyncLoads;

	ENGINE_API void StartAsyncLoadingPackage(FNetGuidCacheObject& Object, const FNetworkGUID ObjectGUID, const bool bWasAlreadyAsyncLoading);
	ENGINE_API void ValidateAsyncLoadingPackage(FNetGuidCacheObject& Object, const FNetworkGUID ObjectGUID);

	ENGINE_API void UpdateQueuedBunchObjectReference(const FNetworkGUID NetGUID, UObject* NewObject);

	/**
	 * Set of all current Objects that we've been requested to be referenced while channels
	 * resolve their queued bunches. This is used to prevent objects (especially async load objects,
	 * which may have no other references) from being GC'd while a channel is waiting for more
	 * pending guids. 
	 */
	TMap<FNetworkGUID, TWeakPtr<FQueuedBunchObjectReference>> QueuedBunchObjectReferences;

	/**
	 * Map of outer GUIDs to their unmapped, stably named subobjects.
	 * So that the inner objects can be remapped when their outer GUID is imported again.
	 */
	TMap<FNetworkGUID, UE::Net::Private::FRefCountedNetGUIDArray> UnmappedStablyNamedGuids_OuterToInner;

#if CSV_PROFILER_STATS
public:

	ENGINE_API bool IsTrackingOwnerOrPawn() const;

	struct FIsOwnerOrPawnHelper
	{
	private:
		friend class UActorChannel;

		FIsOwnerOrPawnHelper(
			class FNetGUIDCache* const InGuidCache,
			const class AActor* InConnectionActor,
			const class AActor* ChannelActor);

	public:

		~FIsOwnerOrPawnHelper();

		bool IsOwnerOrPawn() const;

	private:

		class FNetGUIDCache* const GuidCache;
		const class AActor* ConnectionActor;
		const class AActor* ChannelActor;
		mutable int8 CachedResult = INDEX_NONE;

		// No copying, moving, or constructing off stack.
		// References or pointers to this class should never be kept alive.
		FIsOwnerOrPawnHelper(const FIsOwnerOrPawnHelper&) = delete;
		FIsOwnerOrPawnHelper(FIsOwnerOrPawnHelper&&) = delete;

		FIsOwnerOrPawnHelper& operator=(const FIsOwnerOrPawnHelper&) = delete;
		FIsOwnerOrPawnHelper& operator=(FIsOwnerOrPawnHelper&&) = delete;

		void* operator new (size_t) = delete;
		void* operator new[](size_t) = delete;
		void operator delete (void *) = delete;
		void operator delete[](void*) = delete;
	};

private:

	FIsOwnerOrPawnHelper* TrackingOwnerOrPawnHelper = nullptr;
#endif
};

class FPackageMapAckState
{
public:
	TMap< FNetworkGUID, int32 >	NetGUIDAckStatus;				// Map that represents the ack state of each net guid for this connection
	TSet< uint32 >				NetFieldExportGroupPathAcked;	// Map that represents whether or not a net field export group has been ack'd by the client
	TSet< uint64 >				NetFieldExportAcked;			// Map that represents whether or not a net field export has been ack'd by the client

	void Reset()
	{
		NetGUIDAckStatus.Empty();
		NetFieldExportGroupPathAcked.Empty();
		NetFieldExportAcked.Empty();
	}

	ENGINE_API void CountBytes(FArchive& Ar) const;
};

UCLASS(transient, MinimalAPI)
class UPackageMapClient : public UPackageMap
{
public:
	GENERATED_BODY()

	ENGINE_API UPackageMapClient(const FObjectInitializer & ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API void Initialize(UNetConnection * InConnection, TSharedPtr<FNetGUIDCache> InNetGUIDCache);

	virtual ~UPackageMapClient()
	{
		if (CurrentExportBunch)
		{
			delete CurrentExportBunch;
			CurrentExportBunch = NULL;
		}
	}
	
	// UPackageMap Interface
	ENGINE_API virtual bool SerializeObject( FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID *OutNetGUID = NULL ) override;
	ENGINE_API virtual bool SerializeNewActor( FArchive& Ar, class UActorChannel *Channel, class AActor*& Actor) override;
	
	ENGINE_API virtual bool WriteObject( FArchive& Ar, UObject* InOuter, FNetworkGUID NetGUID, FString ObjName ) override;
	ENGINE_API virtual bool IsUsingRemoteObjectReferences() const override;

	// UPackageMapClient Connection specific methods

	ENGINE_API bool NetGUIDHasBeenAckd(FNetworkGUID NetGUID);

	ENGINE_API virtual void ReceivedNak( const int32 NakPacketId ) override;
	ENGINE_API virtual void ReceivedAck( const int32 AckPacketId ) override;
	ENGINE_API virtual void NotifyBunchCommit( const int32 OutPacketId, const FOutBunch* OutBunch ) override;
	ENGINE_API virtual void GetNetGUIDStats(int32 &AckCount, int32 &UnAckCount, int32 &PendingCount) override;
	virtual const UE::Net::FNetTokenResolveContext* GetNetTokenResolveContext() const override { return &NetTokenResolveContext; }

	ENGINE_API void ReceiveNetGUIDBunch( FInBunch &InBunch );

	UE_DEPRECATED(5.5, "Use GetAdditionalRequiredBunches")
	ENGINE_API void AppendExportBunches(TArray<FOutBunch *>& OutgoingBunches);

	/** Get bunches to prepend to the OutgoingBunch. The required bunches usually contain exports or extension data. */
	ENGINE_API TArray<FOutBunch*> GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags);

	ENGINE_API int32 GetNumExportBunches() const;

	ENGINE_API void AppendExportData(FArchive& Archive);
	ENGINE_API void ReceiveExportData(FArchive& Archive);

	TMap<FNetworkGUID, int32>	NetGUIDExportCountMap;	// How many times we've exported each NetGUID on this connection. Public for ListNetGUIDExports 

	ENGINE_API void HandleUnAssignedObject( UObject* Obj );

	static ENGINE_API void	AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	ENGINE_API virtual void NotifyStreamingLevelUnload(UObject* UnloadedLevel) override;

	ENGINE_API virtual bool PrintExportBatch() override;

	virtual void ResetTrackedSyncLoadedGuids() override { TrackedSyncLoadedGUIDs.Reset(); }
	ENGINE_API virtual void ReportSyncLoadsForProperty(const FProperty* Property, const UObject* Object) override;

	ENGINE_API virtual void			LogDebugInfo( FOutputDevice & Ar) override;
	ENGINE_API virtual UObject *		GetObjectFromNetGUID( const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped ) override;
	ENGINE_API virtual FNetworkGUID	GetNetGUIDFromObject( const UObject* InObject) const override;
	virtual bool			IsGUIDBroken( const FNetworkGUID& NetGUID, const bool bMustBeRegistered ) const override { return GuidCache->IsGUIDBroken( NetGUID, bMustBeRegistered ); }

	/** Returns true if this guid is directly pending, or depends on another guid that is pending */
	ENGINE_API virtual bool			IsGUIDPending( const FNetworkGUID& NetGUID ) const;

	/** Set rather this actor is associated with a channel with queued bunches */
	ENGINE_API virtual void			SetHasQueuedBunches( const FNetworkGUID& NetGUID, bool bHasQueuedBunches );

	TArray< FNetworkGUID > & GetMustBeMappedGuidsInLastBunch() { return MustBeMappedGuidsInLastBunch; }

	class UNetConnection* GetConnection() { return Connection; }

	ENGINE_API void SyncPackageMapExportAckStatus( const UPackageMapClient* Source );

	ENGINE_API void SavePackageMapExportAckStatus( FPackageMapAckState& OutState );
	ENGINE_API void RestorePackageMapExportAckStatus( const FPackageMapAckState& InState );
	ENGINE_API void OverridePackageMapExportAckStatus( FPackageMapAckState* NewState );

	/** Resets the AckState and empties the PendingAckGuids, not meant to reset the OverrideAckState. */
	ENGINE_API void ResetAckState();

	/** Functions to help with exporting/importing net field info */
	ENGINE_API TSharedPtr< FNetFieldExportGroup >	GetNetFieldExportGroup( const FString& PathName );
	ENGINE_API void								AddNetFieldExportGroup( const FString& PathName, TSharedPtr< FNetFieldExportGroup > NewNetFieldExportGroup );
	ENGINE_API void								TrackNetFieldExport( FNetFieldExportGroup* NetFieldExportGroup, const int32 NetFieldExportHandle );
	ENGINE_API TSharedPtr< FNetFieldExportGroup >	GetNetFieldExportGroupChecked( const FString& PathName ) const;
	ENGINE_API void								SerializeNetFieldExportGroupMap( FArchive& Ar, bool bClearPendingExports=true );
	ENGINE_API void								SerializeNetFieldExportDelta(FArchive& Ar);

	/** This is called during the sending of a network bunch.  The returned bunch is prepended to the outgoing bunches and guaranteed to be processed upon receive (before ack) */
	ENGINE_API FOutBunch* CreateCustomExportsBunch(const FOutBunch& OutgoingBunch);

	/** Receive a bunch that contains the data for Package Map Extensions */
	void ReceiveCustomExportsBunch(FInBunch& InBunch);

public:
	TUniquePtr<TGuardValue<bool>> ScopedIgnoreReceivedExportGUIDs()
	{
		return MakeUnique<TGuardValue<bool>>(bIgnoreReceivedExportGUIDs, true);
	}

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	FString GetFullNetGUIDPath(const FNetworkGUID& NetGUID) const
	{
		FString FullGuidCachePath;

		if (const FNetGUIDCache * const GuidCacheLocal = GuidCache.Get())
		{
			FullGuidCachePath = GuidCacheLocal->FullNetGUIDPath(NetGUID);
		}

		return FullGuidCachePath;
	}

	ENGINE_API virtual void AddUnmappedNetGUIDReference(FNetworkGUID UnmappedGUID) override;
	ENGINE_API virtual void RemoveUnmappedNetGUIDReference(FNetworkGUID MappedGUID) override;
	ENGINE_API bool IsNetGUIDAuthority() const;

protected:

	/** Functions to help with exporting/importing net field export info */
	ENGINE_API void AppendNetFieldExports( FArchive& Archive );
	ENGINE_API void ReceiveNetFieldExports( FArchive& Archive );

	ENGINE_API void AppendNetFieldExportsInternal( FArchive& Archive, const TSet<uint64>& InNetFieldExports, EAppendNetExportFlags Flags );

	ENGINE_API void AppendNetExportGUIDs( FArchive& Archive );
	ENGINE_API void ReceiveNetExportGUIDs( FArchive& Archive );

	ENGINE_API bool ExportNetGUIDForReplay( FNetworkGUID&, UObject* Object, FString& PathName, UObject* ObjOuter );
	ENGINE_API bool ExportNetGUID( FNetworkGUID NetGUID, UObject* Object, FString PathName, UObject* ObjOuter );
	ENGINE_API void ExportNetGUIDHeader();

	ENGINE_API void			InternalWriteObject( FArchive& Ar, FNetworkGUID NetGUID, UObject* Object, FString ObjectPathName, UObject* ObjectOuter );	
	ENGINE_API FNetworkGUID	InternalLoadObject( FArchive & Ar, UObject *& Object, const int32 InternalLoadObjectRecursionCount );

	ENGINE_API virtual UObject* ResolvePathAndAssignNetGUID( const FNetworkGUID& NetGUID, const FString& PathName ) override;

	ENGINE_API bool	ShouldSendFullPath(const UObject* Object, const FNetworkGUID &NetGUID);
	
	class UNetConnection* Connection;

	ENGINE_API bool ObjectLevelHasFinishedLoading(UObject* Obj) const;

	TArray<TArray<uint8>>				ExportGUIDArchives;
	TSet< FNetworkGUID >				CurrentExportNetGUIDs;				// Current list of NetGUIDs being written to the Export Bunch.

	/** Set of Actor NetGUIDs with currently queued bunches and the time they were first queued. */
	TMap<FNetworkGUID, double> CurrentQueuedBunchNetGUIDs;

	TArray< FNetworkGUID >				PendingAckGUIDs;					// Quick access to all GUID's that haven't been acked

	FPackageMapAckState					AckState;							// Current ack state of exported data
	FPackageMapAckState*				OverrideAckState;					// This is a pointer that allows us to override the current ack state, it's never NULL (it will point to AckState by default)

	// Bunches of NetGUID/path tables to send with the current content bunch
	TArray<FOutBunch* >					ExportBunches;
	FOutBunch *							CurrentExportBunch;

	int32								ExportNetGUIDCount;

	TSharedPtr< FNetGUIDCache >			GuidCache;

	TArray< FNetworkGUID >				MustBeMappedGuidsInLastBunch;

	/** List of net field exports that need to go out on next bunch */
	TSet< uint64 >						NetFieldExports;

private:
	ENGINE_API void ReceiveNetFieldExportsCompat(FInBunch& InBunch);

	/** Used by SerializeNewActor to report sync loads with LogNetSyncLoads */
	ENGINE_API void ReportSyncLoadsForActorSpawn(const AActor* Actor);

	bool bIgnoreReceivedExportGUIDs;

public:

	ENGINE_API int32 GetNumQueuedBunchNetGUIDs() const;
	ENGINE_API void ConsumeQueuedActorDelinquencyAnalytics(FNetQueuedActorDelinquencyAnalytics& Out);
	ENGINE_API const FNetQueuedActorDelinquencyAnalytics& GetQueuedActorDelinquencyAnalytics() const;
	ENGINE_API void ResetQueuedActorDelinquencyAnalytics();

private:

	FNetQueuedActorDelinquencyAnalytics DelinquentQueuedActors;

	/**
	 * Tracks GUIDs contained in FNetGUIDCache::SyncLoadedGUIDs that are referenced during deserialization
	 * of a particular property or actor spawn, to help correlate those sync loads with that property or actor.
	 */
	TArray<FNetworkGUID> TrackedSyncLoadedGUIDs;

	// Stored information required to resolve NetTokens
	UE::Net::FNetTokenResolveContext NetTokenResolveContext;

	/** NetTokens acknowledged by remote */
	TSet<UE::Net::FNetToken> AcknowledgedExportedNetTokens;
	
	/** NetTokens still pending acknowledgment by remote */
	TMultiMap<int32, UE::Net::FNetToken> NetTokenPendingAckMap;
};
