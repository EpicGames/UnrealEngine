// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineUtils.h"
#include "GameFramework/Pawn.h"

/** Wrapper object that tries to imitate the TWeakObjectPtr interface for the objects previously in the PawnList and iterated by FConstPawnIterator. */
struct FPawnIteratorObject
{
	APawn* operator->() const { return Pawn; }
	APawn& operator*() const { return *Pawn; }
	APawn* Get() const { return Pawn; }

	bool operator==(const UObject* Other) const { return Pawn == Other; }
	bool operator!=(const UObject* Other) const { return Pawn != Other; }

private:
	FPawnIteratorObject()
		: Pawn(nullptr)
	{
	}

	FPawnIteratorObject(APawn* InPawn)
		: Pawn(InPawn)
	{
	}

	APawn* Pawn;

	friend class FConstPawnIterator;
};

template< class T > inline T* Cast(const FPawnIteratorObject& Src) { return Cast<T>(Src.Get()); }

/**
 * Imitation iterator class that attempts to provide the basic interface that FConstPawnIterator previously did when a typedef of TArray<TWeakObjectPtr<APawn>>::Iterator.
 * In general you should prefer not to use this iterator and instead use TActorIterator<APawn> or TActorRange<APawn> (or the desired more derived type).
 * This iterator will likely be deprecated in a future release.
 */
class FConstPawnIterator
{
private:
	ENGINE_API FConstPawnIterator(UWorld* World);

public:
	ENGINE_API ~FConstPawnIterator();

	ENGINE_API FConstPawnIterator(FConstPawnIterator&&);
	ENGINE_API FConstPawnIterator& operator=(FConstPawnIterator&&);

	ENGINE_API explicit operator bool() const;
	ENGINE_API FPawnIteratorObject operator*() const;
	ENGINE_API TUniquePtr<FPawnIteratorObject> operator->() const;

	ENGINE_API FConstPawnIterator& operator++();
	ENGINE_API FConstPawnIterator& operator++(int);

private:
	TUniquePtr<TActorIterator<APawn>> Iterator;

	friend UWorld;
};
