// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "NetworkDelegates.h"
#include "PendingNetGame.generated.h"

#define UE_API ENGINE_API

class UEngine;
class UNetConnection;
class UNetDriver;
struct FWorldContext;

UCLASS(MinimalAPI, customConstructor, transient)
class UPendingNetGame :
	public UObject,
	public FNetworkNotify
{
	GENERATED_BODY()

public:

	/** 
	 * Net driver created for contacting the new server
	 * Transferred to world on successful connection
	 */
	UPROPERTY()
	TObjectPtr<class UNetDriver>		NetDriver;

private:
	/** 
	 * Demo Net driver created for loading demos, but we need to go through pending net game
	 * Transferred to world on successful connection
	 */
	UPROPERTY()
	TObjectPtr<class UDemoNetDriver>	DemoNetDriver;

public:
	/** Gets the demo net driver for this pending world. */
	UDemoNetDriver* GetDemoNetDriver() const { return DemoNetDriver; }

	/** Sets the demo net driver for this pending world. */
	void SetDemoNetDriver(UDemoNetDriver* const InDemoNetDriver) { DemoNetDriver = InDemoNetDriver; }

	/**
	 * Setup the connection for encryption with a given key
	 * All future packets are expected to be encrypted
	 *
	 * @param Response response from the game containing its encryption key or an error message
	 * @param WeakConnection the connection related to the encryption request
	 */
	UE_API void FinalizeEncryptedConnection(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection);

	/**
	 * Set the encryption key for the connection. This doesn't cause outgoing packets to be encrypted,
	 * but it allows the connection to decrypt any incoming packets if needed.
	 *
	 * @param Response response from the game containing its encryption key or an error message
	 */
	UE_API void SetEncryptionKey(const FEncryptionKeyResponse& Response);

	bool HasFailedTravel() const {return bFailedTravel; }
	void SetFailedTravel(bool bInFailedTravel) { bFailedTravel = bInFailedTravel; }

public:
	/** URL associated with this level. */
	FURL					URL;

	/** @todo document */
	bool					bSuccessfullyConnected;

	/** @todo document */
	bool					bSentJoinRequest;

	/** set when we call LoadMapCompleted */
	bool					bLoadedMapSuccessfully;
private:
	/** initialized to true, delaytravel steps can set this to false to indicate error during pendingnetgame travel */
	bool					bFailedTravel;
public:
	/** @todo document */
	FString					ConnectionError;

	// Constructor.
	UE_API void Initialize(const FURL& InURL);

	// Constructor.
	UE_API UPendingNetGame(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** 
	 * Inititalize the NetDriver to be used for the server connection handshake.
	 * 
	 * If InNetDriver is null then a new NetDriver will be created in the world using the 
	 * NAME_GameNetDriver driver definition.
	 * 
	 * If InNetDriver is not null then it's expected that the NetDriver passed in has only
	 * been created and has not begun listening for connections or connecting to a remote
	 * server.
	 */
	UE_API void	InitNetDriver(UNetDriver* InNetDriver = nullptr);

	/**
	 * Begin initial handshake if needed, or call SendInitialJoin.
	 */
	UE_API void BeginHandshake();

	/**
	 * Send the packet for triggering the initial join
	 */
	UE_API void SendInitialJoin();

	//~ Begin FNetworkNotify Interface.
	UE_API virtual EAcceptConnection::Type NotifyAcceptingConnection() override;
	UE_API virtual void NotifyAcceptedConnection( class UNetConnection* Connection ) override;
	UE_API virtual bool NotifyAcceptingChannel( class UChannel* Channel ) override;
	UE_API virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch) override;
	//~ End FNetworkNotify Interface.

	/**  Update the pending level's status. */
	UE_API virtual void Tick( float DeltaTime );

	/** @todo document */
	virtual UNetDriver* GetNetDriver() { return NetDriver; }

	/** Send JOIN to other end */
	UE_API virtual void SendJoin();

	/** Send Join to the server with optional flags */
	UE_API virtual void SendJoinWithFlags(UE::Net::EJoinFlags Flags);

	//~ Begin UObject Interface.
	UE_API virtual void Serialize( FArchive& Ar ) override;

	UE_API virtual void FinishDestroy() override;
	
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.
	

	/** Create the peer net driver and a socket to listen for new client peer connections. */
	UE_API void InitPeerListen();

	/** Called by the engine after it calls LoadMap for this PendingNetGame. */
	UE_API virtual bool LoadMapCompleted(UEngine* Engine, FWorldContext& Context, bool bLoadedMapSuccessfully, const FString& LoadMapError);

	/** Called by the engine after loadmapCompleted and the GameInstance has finished delaying */
	UE_API virtual void TravelCompleted(UEngine* Engine, FWorldContext& Context);

protected:

	UE_API virtual class ULocalPlayer* GetFirstGamePlayer();
};

#undef UE_API
