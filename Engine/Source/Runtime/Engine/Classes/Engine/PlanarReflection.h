// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * APlanarReflection
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SceneCapture.h"
#include "PlanarReflection.generated.h"

class UBillboardComponent;

UCLASS(hidecategories=(Collision, Material, Attachment, Actor), MinimalAPI)
class APlanarReflection : public ASceneCapture
{
	GENERATED_UCLASS_BODY()

private:
	/** Planar reflection component. */
	UPROPERTY(Category = SceneCapture, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UPlanarReflectionComponent> PlanarReflectionComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif

public:
	UE_DEPRECATED(4.22, "Please use ShowPreviewPlane on PlanarReflectionComponent instead")
	UPROPERTY()
	bool bShowPreviewPlane_DEPRECATED;

	ENGINE_API virtual void PostLoad() override;

	//~ Begin AActor Interface
#if WITH_EDITOR
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
#endif
	//~ End AActor Interface.

	UFUNCTION(BlueprintCallable, Category="Rendering")
	void OnInterpToggle(bool bEnable);

	/** Returns subobject **/
	class UPlanarReflectionComponent* GetPlanarReflectionComponent() const
	{
		return PlanarReflectionComponent;
	}

#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};



