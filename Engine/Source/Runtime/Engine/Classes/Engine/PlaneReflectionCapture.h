// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Not yet implemented plane capture class
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/ReflectionCapture.h"
#include "PlaneReflectionCapture.generated.h"

UCLASS(abstract, hidecategories=(Collision, Attachment, Actor), MinimalAPI)
class APlaneReflectionCapture : public AReflectionCapture
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITOR
	//~ Begin AActor Interface.
	virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif

};



