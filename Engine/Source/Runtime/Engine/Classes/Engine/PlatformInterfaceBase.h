// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This is the base class for the various platform interface classes and has support
 * for a generic delegate system, as well has having subclasses determine if they
 * should register for a tick.
 * 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "PlatformInterfaceBase.generated.h"

/** An enum for the types of data used in a PlatformInterfaceData struct. */
UENUM()
enum EPlatformInterfaceDataType : int
{
	/** No data type specified. */
	PIDT_None,
	PIDT_Int,
	PIDT_Float,
	PIDT_String,
	PIDT_Object,
	/** A custom type where more than one value may be filled out. */
	PIDT_Custom,
	PIDT_MAX,
};


/** 
 * Struct that encompasses the most common types of data. This is the data payload
 * of PlatformInterfaceDelegateResult.
 */
USTRUCT()
struct FPlatformInterfaceData
{
	GENERATED_USTRUCT_BODY()

	/** An optional tag for this data */
	UPROPERTY()
	FName DataName;

	/** Specifies which value is valid for this structure */
	UPROPERTY()
	TEnumAsByte<enum EPlatformInterfaceDataType> Type;

	/** Various typed result values */
	UPROPERTY()
	int32 IntValue;

	UPROPERTY()
	float FloatValue;

	UPROPERTY()
	FString StringValue;

	UPROPERTY()
	TObjectPtr<class UObject> ObjectValue;


	FPlatformInterfaceData()
		: Type(0)
		, IntValue(0)
		, FloatValue(0)
		, ObjectValue(NULL)
	{
	}

};

/** Generic structure for returning most any kind of data from C++ to delegate functions */
USTRUCT()
struct FPlatformInterfaceDelegateResult
{
	GENERATED_USTRUCT_BODY()

	/** This is always usable, no matter the type */
	UPROPERTY()
	bool bSuccessful;

	/** The result actual data */
	UPROPERTY()
	struct FPlatformInterfaceData Data;


	FPlatformInterfaceDelegateResult()
		: bSuccessful(false)
	{
	}

};

/** Generic platform interface delegate signature */
DECLARE_DYNAMIC_DELEGATE_OneParam( FPlatformInterfaceDelegate, const struct FPlatformInterfaceDelegateResult&, Result );

/**
 * Helper struct, since UnrealScript doesn't allow arrays of arrays, but
 * arrays of structs of arrays are okay.
 */
USTRUCT()
struct FDelegateArray
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FPlatformInterfaceDelegate> Delegates;

};

UCLASS(transient,MinimalAPI)
class UPlatformInterfaceBase : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Array of delegate arrays. Only add and remove via helper functions, and call via the helper delegate call function */
	UPROPERTY()
	TArray<struct FDelegateArray> AllDelegates;


	/**
	 * C++ interface to get the singleton
	 */
	// @todo document
	static class UCloudStorageBase* GetCloudStorageInterfaceSingleton();

	// @todo document
	static class UInGameAdManager* GetInGameAdManagerSingleton();

	// @todo document
	static class UMicroTransactionBase* GetMicroTransactionInterfaceSingleton();

	// @todo document
	static class UTwitterIntegrationBase* GetTwitterIntegrationSingleton();

	/**
	 * Check for certain exec commands that map to the various subclasses (it will only
	 * get/create the singleton if the first bit of the exec command matches a one of 
	 * the special strings, like "ad" for ad manager)
	 */
	static bool StaticExec(const TCHAR* Cmd, FOutputDevice& Ar);

	/**
	 * Determines if there are any delegates of the given type on this platform interface object.
	 * This is useful to skip a bunch of FPlatformInterfaceDelegateResult if there is no
	 * one even listening!
	 *
	 * @param DelegateType The type of delegate to look up delegates for
	 *
	 * @return true if there are any delegates set of the given type
	 */
	bool HasDelegates(int32 DelegateType);

	/**
	 * Call all the delegates currently set for the given delegate type with the given data
	 *
	 * @param DelegateType Which set of delegates to call (this is defined in the subclass of PlatformInterfaceBase)
	 * @param Result Data to pass to each delegate
	 */
	ENGINE_API void CallDelegates(int32 DelegateType, FPlatformInterfaceDelegateResult& Result);
	
	/** @return the CloudStorage singleton object */
	class UCloudStorageBase* GetCloudStorageInterface();
	

	/** @return the AdManager singleton object */
	class UInGameAdManager* GetInGameAdManager();
	

	/** @return the MicroTransaction singleton object */
	class UMicroTransactionBase* GetMicroTransactionInterface();
	

	/** @return the Twitter singleton object */
	class UTwitterIntegrationBase* GetTwitterIntegration();
	

	/**
	 * Adds a typed delegate (the value of the type is subclass dependent, make an enum per subclass)
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	ENGINE_API virtual void AddDelegate(int32 DelegateType, FPlatformInterfaceDelegate InDelegate);
	

	/**
	 * Removes a delegate from the list of listeners
	 *
	 * @param InDelegate the delegate to use for notifications
	 */
	ENGINE_API virtual void ClearDelegate(int32 DelegateType, FPlatformInterfaceDelegate InDelegate);
};

