// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This is the a generic web response object that holds the entirety of the 
 * web response made from PlatformInterfaceBase subclasses
 * 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "PlatformInterfaceWebResponse.generated.h"

UCLASS(transient,MinimalAPI)
class UPlatformInterfaceWebResponse : public UObject
{
	GENERATED_UCLASS_BODY()

	/** This holds the original requested URL */
	UPROPERTY()
	FString OriginalURL;

	/** Result code from the response (200=OK, 404=Not Found, etc) */
	UPROPERTY()
	int32 ResponseCode;

	/** A user-specified tag specified with the request */
	UPROPERTY()
	int32 Tag;

	/** For string results, this is the response */
	UPROPERTY()
	FString StringResponse;

	/** For non-string results, this is the response */
	UPROPERTY()
	TArray<uint8> BinaryResponse;

	/** @return the number of header/value pairs */
	UFUNCTION()
	virtual int32 GetNumHeaders();

	/** Retrieve the header and value for the given index of header/value pair */
	UFUNCTION()
	virtual void GetHeader(int32 HeaderIndex, FString& Header, FString& Value);

	/** @return the value for the given header (or "" if no matching header) */
	UFUNCTION()
	virtual FString GetHeaderValue(const FString& HeaderName);


public:
	/** Response headers and their values */
	TMap<FString, FString> Headers;
};

