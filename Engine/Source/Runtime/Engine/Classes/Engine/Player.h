// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Player: Corresponds to a real player (a local camera or remote net player).
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Player.generated.h"

class APlayerController;

UCLASS(MinimalAPI, transient, config=Engine)
class UPlayer : public UObject, public FExec
{
	GENERATED_UCLASS_BODY()

	/** The actor this player controls. */
	UPROPERTY(transient)
	TObjectPtr<class APlayerController> PlayerController;

	// Net variables.
	/** the current speed of the connection */
	UPROPERTY()
	int32 CurrentNetSpeed;

	/** @todo document */
	UPROPERTY(globalconfig)
	int32 ConfiguredInternetSpeed;
	
	/** @todo document */
	UPROPERTY(globalconfig)
	int32 ConfiguredLanSpeed;

public:
	//~ Begin FExec Interface.
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar) override;
#endif
	//~ End FExec Interface.

	/**
	 * Dynamically assign Controller to Player and set viewport.
	 *
	 * @param    PC - new player controller to assign to player
	 */
	ENGINE_API virtual void SwitchController( class APlayerController* PC );

	/**
	 * Executes the Exec() command
	 *
	 * @param Command command to execute (string of commands optionally separated by a | (pipe))
	 * @param bWriteToLog write out to the log
	 */
	ENGINE_API FString ConsoleCommand(const FString& Cmd, bool bWriteToLog = true);

	/**
	 * Gets the player controller in the given world for this player.
	 *
	 * @param InWorld The world in which to search for player controllers.
	 *
	 * @return The controller associated with this player in InWorld, if one exists.
	 */
	ENGINE_API APlayerController* GetPlayerController(const UWorld* const InWorld) const;

	/**
	 * Called when this player has had it's outer Player Controller set for remote net connections
	 */
	ENGINE_API virtual void ReceivedPlayerController(APlayerController* NewController);
};
