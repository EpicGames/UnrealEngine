// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// Player start for PIE - can be spawned during play.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/PlayerStart.h"
#include "PlayerStartPIE.generated.h"

UCLASS(notplaceable,MinimalAPI)
class APlayerStartPIE : public APlayerStart
{
	GENERATED_UCLASS_BODY()

};



