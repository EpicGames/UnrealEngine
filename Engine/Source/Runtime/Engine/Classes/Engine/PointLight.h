// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Light.h"
#include "PointLight.generated.h"

UCLASS(ClassGroup=(Lights, PointLights), ComponentWrapperClass, MinimalAPI, meta=(ChildCanTick))
class APointLight : public ALight
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Light", meta=(ExposeFunctionCategories="PointLight,Rendering|Lighting"))
	TObjectPtr<class UPointLightComponent> PointLightComponent;

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetRadius(float NewRadius);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	ENGINE_API void SetLightFalloffExponent(float NewLightFalloffExponent);
	// END DEPRECATED

#if WITH_EDITOR
	//~ Begin AActor Interface.
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
#endif
	//~ End UObject Interface.
};