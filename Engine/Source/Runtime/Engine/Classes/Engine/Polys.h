// Copyright Epic Games, Inc. All Rights Reserved.

//
// List of FPolys.
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Polys.generated.h"

class ABrush;
class UMaterialInterface;
class UModel;

// Results from FPoly.SplitWithPlane, describing the result of splitting
// an arbitrary FPoly with an arbitrary plane.
enum ESplitType
{
	SP_Coplanar		= 0, // Poly wasn't split, but is coplanar with plane
	SP_Front		= 1, // Poly wasn't split, but is entirely in front of plane
	SP_Back			= 2, // Poly wasn't split, but is entirely in back of plane
	SP_Split		= 3, // Poly was split into two new editor polygons
};

//
// A general-purpose polygon used by the editor.  An FPoly is a free-standing
// class which exists independently of any particular level, unlike the polys
// associated with Bsp nodes which rely on scads of other objects.  FPolys are
// used in UnrealEd for internal work, such as building the Bsp and performing
// boolean operations.
//
class FPoly
{
public:
	// Store up to 16 vertices inline.
	typedef TArray<FVector3f,TInlineAllocator<16> > VerticesArrayType;

	FVector3f				Base;					// Base point of polygon.
	FVector3f				Normal;					// Normal of polygon.
	FVector3f				TextureU;				// Texture U vector.
	FVector3f				TextureV;				// Texture V vector.
	VerticesArrayType	Vertices;
	uint32				PolyFlags;				// FPoly & Bsp poly bit flags (PF_).
	TObjectPtr<ABrush>				Actor;					// Brush where this originated, or NULL.
	TObjectPtr<UMaterialInterface>	Material;				// Material.
	FName				RulesetVariation;		// Name of variation within a ProcBuilding Ruleset for this face
	FName				ItemName;				// Item name.
	int32					iLink;					// iBspSurf, or brush fpoly index of first identical polygon, or MAX_uint16.
	int32					iLinkSurf;
	int32					iBrushPoly;				// Index of editor solid's polygon this originated from.
	uint32				SmoothingMask;			// A mask used to determine which smoothing groups this polygon is in.  SmoothingMask & (1 << GroupNumber)
	float				LightMapScale;			// The number of units/shadowmap texel on this surface.

	// This MUST be the format of FLightmassPrimitiveSettings
	// The Lightmass settings for surfaces generated from this poly
 	FLightmassPrimitiveSettings		LightmassSettings;

	/**
	 * Constructor, initializing all member variables.
	 */
	ENGINE_API FPoly();

	/**
	 * Initialize everything in an  editor polygon structure to defaults.
	 * Changes to these default values should also be mirrored to UPolysExporterT3D::ExportText(...).
	 */
	ENGINE_API void Init();

	/**
	 * Reverse an FPoly by reversing the normal and reversing the order of its vertices.
	 */
	ENGINE_API void Reverse();

	/**
	 * Transform an editor polygon with a post-transformation addition.
	 */
	ENGINE_API void Transform(const FVector3f &PostAdd);

	/**
	 * Rotate an editor polygon.
	 */
	ENGINE_API void Rotate(const FRotator3f &Rotation);

	/**
	 * Scale an editor polygon.
	 */
	ENGINE_API void Scale(const FVector3f &Scale);

	/**
	 * Fix up an editor poly by deleting vertices that are identical.  Sets
	 * vertex count to zero if it collapses.  Returns number of vertices, 0 or >=3.
	 */
	ENGINE_API int32 Fix();

	/**
	 * Compute normal of an FPoly.  Works even if FPoly has 180-degree-angled sides (which
	 * are often created during T-joint elimination).  Returns nonzero result (plus sets
	 * normal vector to zero) if a problem occurs.
	 */
	ENGINE_API int32 CalcNormal( bool bSilent = 0 );

	/**
	 * Split with plane. Meant to be numerically stable.
	 */
	ENGINE_API int32 SplitWithPlane(const FVector3f &InBase,const FVector3f &InNormal,FPoly *FrontPoly,FPoly *BackPoly,int32 VeryPrecise) const;

	/** Split with a Bsp node. */
	ENGINE_API int32 SplitWithNode(const UModel *Model,int32 iNode,FPoly *FrontPoly,FPoly *BackPoly,int32 VeryPrecise) const;

	/**
	 * Split with plane quickly for in-game geometry operations.
	 * Results are always valid. May return sliver polys.
	 */
	ENGINE_API int32 SplitWithPlaneFast(const FPlane& Plane,FPoly *FrontPoly,FPoly *BackPoly) const;

	/** Split a poly and keep only the front half. Returns number of vertices, 0 if clipped away. */
	ENGINE_API int32 Split(const FVector3f &InNormal, const FVector3f &InBase );

	/** Remove colinear vertices and check convexity.  Returns 1 if convex, 0 if nonconvex or collapsed. */
	ENGINE_API int32 RemoveColinears();

	/**
	 * Return whether this poly and Test are facing each other.
	 * The polys are facing if they are noncoplanar, one or more of Test's points is in 
	 * front of this poly, and one or more of this poly's points are behind Test.
	 */
	int32 Faces(const FPoly &Test) const;

	/** Computes the 2D area of the polygon.  Returns zero if the polygon has less than three verices. */
	ENGINE_API float Area();

	/**
	 * Checks to see if the specified line intersects this poly or not.  If "Intersect" is
	 * a valid pointer, it is filled in with the intersection point.
	 */
	ENGINE_API bool DoesLineIntersect( FVector Start, FVector End, FVector* Intersect = NULL );

	/**
	 * Checks to see if the specified vertex is on this poly.  Assumes the vertex is on the same
	 * plane as the poly and that the poly is convex.
	 *
	 * This can be combined with FMath::LinePlaneIntersection to perform a line-fpoly intersection test.
	 */
	bool OnPoly( FVector InVtx );

	/** Checks to see if the specified vertex lies on this polygons plane. */
	ENGINE_API bool OnPlane( FVector InVtx );

	/** Inserts a vertex into the poly at a specific position. */
	ENGINE_API void InsertVertex( int32 InPos, FVector InVtx );

	/** Removes a vertex from the polygons list of vertices */
	ENGINE_API void RemoveVertex( FVector InVtx );

	/** Checks to see if all the vertices on a polygon are coplanar. */
	ENGINE_API bool IsCoplanar();

	/**
	 * Checks to see if this polygon is a convex shape.
	 *
	 * @return	true if this polygon is convex.
	 */
	ENGINE_API bool IsConvex();

	/**
	 * Breaks down this polygon into triangles.  The triangles are then returned to the caller in an array.
	 *
	 * @param	InOwnerBrush	The brush that owns this polygon.
	 * @param	OutTriangles	An array to store the resulting triangles in.
	 *
	 * @return	The number of triangles created
	 */
	ENGINE_API int32 Triangulate( ABrush* InOwnerBrush, TArray<FPoly>& OutTriangles );

	/**
	 * Finds the index of the specific vertex.
	 *
	 * @param	InVtx	The vertex to find the index of
	 *
	 * @return	The index of the vertex, if found.  Otherwise INDEX_NONE.
	 */
	ENGINE_API int32 GetVertexIndex( FVector3f& InVtx );

	/** Computes the mid point of the polygon (in local space). */
	ENGINE_API FVector GetMidPoint();

	/**
	* Builds a huge poly aligned with the specified plane.  This poly is
	* carved up by the calling routine and used as a capping poly following a clip operation.
	*
	* @param	InPlane			The plane to lay the polygon on.
	* @param	InCutPlanes		The planes to split the poly with.
	* @param	InOwnerBrush	The brush that owns the polygons.
	*
	* @return	The resulting polygon
	*/
	ENGINE_API static FPoly BuildInfiniteFPoly(const FPlane& InPlane);

#if WITH_EDITOR

	ENGINE_API static FPoly BuildAndCutInfiniteFPoly(const FPlane& InPlane, const TArray<FPlane>& InCutPlanes, ABrush* InOwnerBrush);

	/**
	 * Compute all remaining polygon parameters (normal, etc) that are blank.
	 * Returns 0 if ok, nonzero if problem.
	 */
	ENGINE_API int32 Finalize( ABrush* InOwner, int32 NoError );

	/**
	* Optimizes a set of polygons into a smaller set of convex polygons.
	*
	* @param	InOwnerBrush	The brush that owns the polygons.
	* @param	InPolygons		An array of FPolys that will be replaced with a new set of polygons that are merged together as much as possible.
	*/
	template<typename ArrayType>
	static void OptimizeIntoConvexPolys(ABrush* InOwnerBrush, ArrayType& InPolygons);

	/**
	* Takes a set of polygons and returns a vertex array representing the outside winding
	* for them.
	*
	* NOTE : This will work for convex or concave sets of polygons but not for concave polygons with holes.
	*
	* @param	InOwnerBrush	The brush that owns the polygons.
	* @param	InPolygons		An array of FPolys that you want to get the winding for
	* @param	InWindings		The resulting sets of vertices that represent the windings
	*/
	template<typename ArrayType>
	static void GetOutsideWindings( ABrush* InOwnerBrush, ArrayType& InPolygons, TArray< TArray<FVector3f> >& InWindings )
	{
		InWindings.Empty();
		FVector3f SaveNormal(0);

		// Break up every polygon passed into triangles

		TArray<FPoly> Triangles;

		for( int32 p = 0 ; p < InPolygons.Num() ; ++p )
		{
			FPoly* Poly = &InPolygons[p];

			SaveNormal = Poly->Normal;

			TArray<FPoly> Polys;
			Poly->Triangulate( InOwnerBrush, Polys );

			Triangles.Append( Polys );
		}

		// Generate a list of ordered edges that represent the outside winding

		TArray<FEdge> EdgePool;

		for( int32 p = 0 ; p < Triangles.Num() ; ++p )
		{
			FPoly* Poly = &Triangles[p];

			// Create a list of edges that are in this shape and set their counts to the number of times they are used.

			for( int32 v = 0 ; v < Poly->Vertices.Num() ; ++v )
			{
				const FVector3f vtx0 = Poly->Vertices[v];
				const FVector3f vtx1 = Poly->Vertices[ (v+1) % Poly->Vertices.Num() ];

				FEdge Edge( (FVector)vtx0, (FVector)vtx1 );

				int32 idx;
				if( EdgePool.Find( Edge, idx ) )
				{
					EdgePool[idx].Count++;
				}
				else
				{
					Edge.Count = 1;
					EdgePool.AddUnique( Edge );
				}
			}
		}

		// Remove any edges from the list that are used more than once.  This will leave us with a collection of edges that represents the outside of the brush shape.

		for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
		{
			const FEdge* Edge = &EdgePool[e];

			if( Edge->Count > 1 )
			{
				EdgePool.RemoveAt( e );
				e = -1;
			}
		}

		// Organize the remaining edges in the list so that the vertices will meet up, start to end, properly to form a continuous outline around the brush shape.

		while( EdgePool.Num() )
		{
			TArray<FEdge> OrderedEdges;

			FEdge Edge0 = EdgePool[0];

			OrderedEdges.Add( Edge0 );

			for( int32 e = 1 ; e < EdgePool.Num() ; ++e )
			{
				FEdge Edge1 = EdgePool[e];

				if( Edge0.Vertex[1].Equals( Edge1.Vertex[0] ) )
				{
					// If these edges are already lined up correctly then add Edge1 into the ordered array, remove it from the pool and start over.

					OrderedEdges.Add( Edge1 );
					Edge0 = Edge1;
					EdgePool.RemoveAt( e );
					e = -1;
				}
				else if( Edge0.Vertex[1].Equals( Edge1.Vertex[1] ) )
				{
					// If these edges are lined up but the verts are backwards, swap the verts on Edge1, add it into the ordered array, remove it from the pool and start over.

					Exchange( Edge1.Vertex[0], Edge1.Vertex[1] );

					OrderedEdges.Add( Edge1 );
					Edge0 = Edge1;
					EdgePool.RemoveAt( e );
					e = -1;
				}
			}

			// Create a polygon from the first 3 edges.  Compare the normal of the original brush shape polygon and see if they match.  If
			// they don't, the list of edges will need to be flipped so it faces the other direction.

			if( OrderedEdges.Num() > 2 )
			{
				FPoly TestPoly;
				TestPoly.Init();

				TestPoly.Vertices.Add( (FVector3f)OrderedEdges[0].Vertex[0] );
				TestPoly.Vertices.Add( (FVector3f)OrderedEdges[1].Vertex[0] );
				TestPoly.Vertices.Add( (FVector3f)OrderedEdges[2].Vertex[0] );

				if( TestPoly.Finalize( InOwnerBrush, 1 ) == 0 )
				{
					if( TestPoly.Normal.Equals( SaveNormal ) == false )
					{
						TArray<FEdge> SavedEdges = OrderedEdges;
						OrderedEdges.Empty();

						for( int32 e = SavedEdges.Num() - 1 ; e > -1 ; --e )
						{
							FEdge* Edge = &SavedEdges[e];

							Exchange( Edge->Vertex[0], Edge->Vertex[1] );
							OrderedEdges.Add( *Edge );
						}
					}
				}
			}

			// Create the winding array

			TArray<FVector3f> WindingVerts;
			for( int32 e = 0 ; e < OrderedEdges.Num() ; ++e )
			{
				FEdge* Edge = &OrderedEdges[e];

				WindingVerts.Add( (FVector3f)Edge->Vertex[0] );
			}

			InWindings.Add( WindingVerts );
		}
	}
#endif // WITH_EDITOR

	// Serializer.
	ENGINE_API friend FArchive& operator<<( FArchive& Ar, FPoly& Poly );

	// Inlines.
	int32 IsBackfaced( const FVector3f &Point ) const
		{return ((Point-Base) | Normal) < 0.f;}
	int32 IsCoplanar( const FPoly &Test ) const
		{return FMath::Abs((Base - Test.Base)|Normal)<0.01f && FMath::Abs(Normal|Test.Normal)>0.9999f;}

	friend bool operator==(const FPoly& A,const FPoly& B)
	{
		if(A.Vertices.Num() != B.Vertices.Num())
		{
			return false;
		}

		for(int32 VertexIndex = 0;VertexIndex < A.Vertices.Num();VertexIndex++)
		{
			if(A.Vertices[VertexIndex] != B.Vertices[VertexIndex])
			{
				return false;
			}
		}

		return true;
	}
	friend bool operator!=(const FPoly& A,const FPoly& B)
	{
		return !(A == B);
	}
};

UCLASS(customConstructor, MinimalAPI)
class UPolys : public UObject
{
	GENERATED_UCLASS_BODY()
	// Elements.
	TArray<FPoly> Element;

	// Constructors.
	UPolys(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	:  UObject(ObjectInitializer)
	, Element( )
	{}

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	UPolys(FVTableHelper& Helper)
		: Super(Helper)
		, Element()
	{}

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;	
#endif
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	virtual bool IsAsset() const override { return false; }
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface
};
