// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BoneIndices.h"
#include "PoseWatch.generated.h"

struct FCompactHeapPose;
class UAnimBlueprint;
class UBlendProfile;
struct FSlateIcon;

namespace PoseWatchUtil
{
	/** Gets all pose watches that are parented to Folder, if Folder is nullptr then gets orphans */
	TSet<UPoseWatch*> GetChildrenPoseWatchOf(const UPoseWatchFolder* Folder, const UAnimBlueprint* AnimBlueprint);

	/** Gets all pose watches folders that are parented to Folder, if Folder is nullptr then gets orphans */
	TSet<UPoseWatchFolder*> GetChildrenPoseWatchFoldersOf(const UPoseWatchFolder* Folder, const UAnimBlueprint* AnimBlueprint);

	/** Returns a new random color */
	FColor ChoosePoseWatchColor();

	template <typename TParent, typename TItem> FText FindUniqueNameInParent(TParent* InParent, const TItem* InItem);
}

UCLASS(MinimalAPI)
class UPoseWatchFolder
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	/** Returns the slash delimited path of this folder (e.g. MyFolder/MyNestedFolder/MyPoseWatch)*/
	ENGINE_API const FText GetPath() const;

	/** The default name given to all new folders */
	ENGINE_API FText GetDefaultLabel() const;

	/** Returns the display label assigned to this pose watch folder */
	ENGINE_API FText GetLabel() const;

	/** Returns the visibility of this pose watch folder */
	ENGINE_API bool GetIsVisible() const;

	/** Returns the parent folder this folder belongs to, if any */
	ENGINE_API UPoseWatchFolder* GetParent() const;
	
	/**  Attempts to set this folder's parent, returns false if unsuccessful (if moving causes a name clash)
	 *	 Using bForce may change the folder's label to ensure no name clashes
	 */
	ENGINE_API bool SetParent(UPoseWatchFolder* Parent, bool bForce = false);

	/** Alias of SetParent */
	ENGINE_API void MoveTo(UPoseWatchFolder* InFolder);

	/** Attempts to set the label, returns false if unsuccessful (if there's a name clash with another  folder in the current directory) */
	ENGINE_API bool SetLabel(const FText& InLabel);

	/** Sets the visibility of this folder, must contain at least one post watch descendant to become visible */
	ENGINE_API void SetIsVisible(bool bInIsVisible, bool bUpdateChildren=true);

	/** Called before the pose watch folder is deleted to cleanup it's children and update it's parent */
	ENGINE_API void OnRemoved();

	/** Returns true if InFolder is the parent of this */
	ENGINE_API bool IsIn(const UPoseWatchFolder* InFolder) const;

	/** Returns true if this is a descendant of InFolder */
	ENGINE_API bool IsDescendantOf(const UPoseWatchFolder* InFolder) const;

	/** Returns true if this folder is inside another folder */
	ENGINE_API bool IsAssignedFolder() const;

	/** Returns true if there would be no name clashes when assigning the label InLabel */
	ENGINE_API bool ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage);

	/** Returns true if InLabel is a unique folder label among the children of InFolder,  excluding this */
	ENGINE_API bool IsLabelUniqueInParent(const FText& InLabel, UPoseWatchFolder* InFolder) const;

	/** Returns true if at least one UPoseWatch/UPoseWatchFolder has this as it's parent */
	ENGINE_API bool HasChildren() const;

	/** Takes GetDefaultLabel() and generates unique labels to avoid name clashes (e.g. NewFolder1, NewFolder2, ...) */
	ENGINE_API void SetUniqueDefaultLabel();

	/** Called when a child is removed/added to a folder */
	ENGINE_API void UpdateVisibility();

	/** Returns the anim blueprint this pose watch folder is stored inside */
	ENGINE_API UAnimBlueprint* GetAnimBlueprint() const;

	/** Set whether this should display its children in the pose watch manager window */
	ENGINE_API void SetIsExpanded(bool bInIsExpanded);

	/** Returns true if this should display its children in the pose watch manager window */
	ENGINE_API bool GetIsExpanded() const;

private:
	/** Returns a unique name for a new folder placed within InParent */
	FText FindUniqueNameInParent(UPoseWatchFolder* InParent) const;

	/** Returns true if there is at least one direct pose watch child */
	bool HasPoseWatchChildren() const;

	/** Returns true if there is at least one pose watch descendant (nested folders) */
	bool HasPoseWatchDescendents() const;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:
	UPROPERTY()
	FText Label;

	UPROPERTY()
	TWeakObjectPtr<UPoseWatchFolder> Parent;

	UPROPERTY()
	bool bIsVisible = false;

	UPROPERTY(Transient)
	bool bIsExpanded = true;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UPoseWatchElement
	: public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR

	virtual ~UPoseWatchElement() = default;

	/** The default name given to all new pose watch elements */
	ENGINE_API FText GetDefaultLabel() const;

	/** Returns the display label assigned to this pose watch */
	ENGINE_API FText GetLabel() const;

	/** Returns the visibility of this pose watch */
	ENGINE_API bool GetIsVisible() const;

	/** Returns true if the pose watch is able to draw anything in the viewport */
	ENGINE_API virtual bool GetIsEnabled() const;

	/** Returns the parent pose watch this element belongs to. */
	ENGINE_API UPoseWatch* GetParent() const;

	/**  Sets this element's parent. */
	ENGINE_API void SetParent(UPoseWatch* InParent);

	/** Attempts to set the label, returns false if unsuccessful (if there's a name clash with another pose watch in the current directory). */
	ENGINE_API bool SetLabel(const FText& InLabel);

	/** Sets whether or not to render this pose watch to the view port. */
	ENGINE_API virtual void SetIsVisible(bool bInIsVisible);

	/** Returns true if there would be no name clashes when assigning the label InLabel. */
	ENGINE_API bool ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage);

	/** Returns true if InLabel is a unique among the children of this elements parent, excluding this. */
	ENGINE_API bool IsLabelUniqueInParent(const FText& InLabel, UPoseWatch* InParent) const;

	/** Takes GetDefaultLabel() and generates unique labels to avoid name clashes (e.g. PoseWatch1, PoseWatch2, ...). */
	ENGINE_API void SetUniqueDefaultLabel();

	/** Takes InLabel and generates unique labels to avoid name clashes (e.g. PoseWatch1, PoseWatch2, ...). */
	ENGINE_API void SetUniqueLabel(const FText& InLabel);

	/** Returns the color to display the pose watch using. */
	ENGINE_API FColor GetColor() const;

	/** Sets the display color of this pose watch in the UI and view port. */
	ENGINE_API virtual void SetColor(const FColor& InColor);

	/** Returns true if it is possible to set the color of this element. */
	ENGINE_API bool HasColor() const;

	/** Set the can set the color flag. */
	ENGINE_API void SetHasColor(const bool bInHasColor);

	/** Get the name of the icon used to represent this element. */
	ENGINE_API FName GetIconName() const;

	/** Set the name of the icon used to represent this element. */
	ENGINE_API void SetIconName(const FName InIconName);

	/** Toggle's the pose watch's visibility */
	ENGINE_API void ToggleIsVisible();

private:
	ENGINE_API FText FindUniqueNameInParent(class UPoseWatch* InParent) const;

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:

	// If true will draw the node to the view port.
	UPROPERTY()
	bool bIsVisible = true;

	UPROPERTY()
	bool bHasColor = true;

	UPROPERTY()
	FColor Color;

	UPROPERTY()
	FText Label;

	UPROPERTY()
	FName IconName;

	UPROPERTY()
	TWeakObjectPtr<UPoseWatch> Parent;

#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UPoseWatchPoseElement : public UPoseWatchElement
{
	GENERATED_BODY()

public:

	ENGINE_API UPoseWatchPoseElement(const class FObjectInitializer& PCIP);

#if WITH_EDITOR

	ENGINE_API virtual bool GetIsEnabled() const;

	static ENGINE_API FSlateIcon StaticGetIcon();

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Optionally select a Blend Mask to control which bones on the skeleton are rendered. Any non-zero entries are rendered. */
	UPROPERTY(EditAnywhere, editfixedsize, Category = Default, meta = (UseAsBlendMask = true))
	TObjectPtr<UBlendProfile> ViewportMask;

	/** Invert which bones are rendered when using a viewport mask */
	UPROPERTY(EditAnywhere, Category = Default, meta = (EditCondition = "ViewportMask != nullptr"))
	bool bInvertViewportMask;

	/** The threshold which each bone's blend scale much surpass to be rendered using the viewport mask */
	UPROPERTY(EditAnywhere, Category = Default, meta = (ClampMin = 0.f, ClampMax = 1.f, EditCondition = "ViewportMask != nullptr"))
	float BlendScaleThreshold;

	/** Offset the rendering of the bones in the viewport. */
	UPROPERTY(EditAnywhere, Category = Default)
	FVector3d ViewportOffset;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UPoseWatch
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	/** Returns the slash delimited path of this pose watch */
	ENGINE_API const FText GetPath() const;

	/** The default name given to all new pose watches */
	ENGINE_API FText GetDefaultLabel() const;

	/** Returns the display label assigned to this pose watch */
	ENGINE_API FText GetLabel() const;

	/** Returns the visibility of this pose watch */
	ENGINE_API bool GetIsVisible() const;

	/** Returns true if any child element is enabled */
	ENGINE_API bool GetIsEnabled() const;

	/** Returns true if the this pose watch's node is active in its anim graph */
	ENGINE_API bool GetIsNodeEnabled() const;

	/** Returns the color to display the pose watch using */
	UE_DEPRECATED(5.1, "Node watches no longer have colors, use the color of its elements instead.")
	ENGINE_API FColor GetColor() const;

	/** Set whether this should display its children in the pose watch manager window */
	ENGINE_API void SetIsExpanded(bool bInIsExpanded);

	/** Returns true if this should display its children in the pose watch manager window */
	ENGINE_API bool GetIsExpanded() const;

	/** Returns true if this pose watch should be deleted after the user has deselected its assigned node (Editor preference) */
	ENGINE_API bool GetShouldDeleteOnDeselect() const;

	/** Returns the parent folder this pose watch belongs to, if any */
	ENGINE_API UPoseWatchFolder* GetParent() const;

	/**  Attempts to set this pose watch's parent, returns false if unsuccessful (if moving causes a name clash)
	 *	 Using bForce may change the pose watch's label to ensure no name clashes
	 */
	ENGINE_API bool SetParent(UPoseWatchFolder* InParent, bool bForce=false);

	/** If set, denotes the pose watch is able to be drawn to the viewport */
	ENGINE_API void SetIsNodeEnabled(const bool bInIsEnabled);

	/** Alias of SetParent */
	ENGINE_API void MoveTo(UPoseWatchFolder* InFolder);

	/** Attempts to set the label, returns false if unsuccessful (if there's a name clash with another pose watch in the current directory) */
	ENGINE_API bool SetLabel(const FText& InLabel);

	/** Sets whether or not to render this pose watch to the viewport */
	ENGINE_API void SetIsVisible(bool bInIsVisible);

	/** Sets the display color of this pose watch in the UI and viewport */
	UE_DEPRECATED(5.1, "Node watches no longer have colors, use the color of its elements instead.")
	ENGINE_API void SetColor(const FColor& InColor);

	/** Sets whether this pose watch should delete after deselecting it's assigned node (Editor preference) */
	ENGINE_API void SetShouldDeleteOnDeselect(const bool bInDeleteOnDeselection);

	/** Called when a pose watch is deleted to update it's parent */
	ENGINE_API void OnRemoved();

	/** Toggle's the pose watch's visibility */
	ENGINE_API void ToggleIsVisible();

	/** Returns true if this pose watch is inside InFolder */
	ENGINE_API bool IsIn(const UPoseWatchFolder* InFolder) const;

	/** Returns true if this pose watch is inside some pose watch folder */
	ENGINE_API bool IsAssignedFolder() const;

	/** Returns true if there would be no name clashes when assigning the label InLabel */
	ENGINE_API bool ValidateLabelRename(const FText& InLabel, FText& OutErrorMessage);

	/** Returns true if InLabel is a unique pose watch label among the children of InFolder, excluding this */
	ENGINE_API bool IsLabelUniqueInParent(const FText& InLabel, UPoseWatchFolder* InFolder) const;

	/** Takes GetDefaultLabel() and generates unique labels to avoid name clashes (e.g. PoseWatch1, PoseWatch2, ...) */
	ENGINE_API void SetUniqueDefaultLabel();

	/** Called when a child element's visibility is changed */
	ENGINE_API void UpdateVisibility();

	/** Returns the anim blueprint this pose watch is stored inside */
	ENGINE_API UAnimBlueprint* GetAnimBlueprint() const;

	/** Create a new element and add it to this pose watch */
	ENGINE_API TObjectPtr<UPoseWatchElement> AddElement(const FText InLabel, const FName IconName);

	/** Create a new element of a specified type and add it to this pose watch */
	template< class TElementType > TObjectPtr<TElementType> AddElement(const FText InLabel, const FName IconName);
	
	/** Returns a reference to the array of elements stored in this */
	TArray<TObjectPtr<UPoseWatchElement>>& GetElements() { return Elements; }

	/** Find an element with the supplied label. */
	ENGINE_API TObjectPtr<UPoseWatchElement> FindElement(const FText InLabel);

	/** Find and return and existing element with the supplied label or, if none exists, create a new element and add it to this pose watch. */
	ENGINE_API TObjectPtr<UPoseWatchElement> FindOrAddElement(const FText InLabel, const FName InIconName);

	/** Returns true if this pose watch contains the specified element */
	bool Contains(const TObjectPtr<UPoseWatchElement> InElement) { return Elements.Contains(InElement); }
	bool Contains(const UPoseWatchElement* const InElement) { return Elements.ContainsByPredicate([InElement](const TObjectPtr<UPoseWatchElement>& ContainedElement) { return InElement == ContainedElement.Get(); }); }

	/** Returns the first element with the matching type */
	template< class TElementType > TObjectPtr<TElementType> GetFirstElementOfType()
	{
		for (UPoseWatchElement* Element : Elements)
		{
			if (TElementType* FoundElement = Cast<TElementType>(Element))
			{
				return FoundElement;
			}
		}
		return nullptr;
	}

	
private:
	/** Returns a unique name for a new pose watch placed within InParent */
	ENGINE_API FText FindUniqueNameInParent(UPoseWatchFolder* InParent) const;

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
public:
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	UPROPERTY()
	TWeakObjectPtr<class UEdGraphNode> Node;

	UPROPERTY()
	TObjectPtr<UBlendProfile> ViewportMask_DEPRECATED;
	UPROPERTY()
	bool bInvertViewportMask_DEPRECATED;
	UPROPERTY()
	float BlendScaleThreshold_DEPRECATED;
	UPROPERTY()
	FVector3d ViewportOffset_DEPRECATED;

protected:
	UPROPERTY()
	TArray<TObjectPtr<UPoseWatchElement>> Elements;

	UPROPERTY()
	bool bDeleteOnDeselection = false;

	// If true will draw the pose to the viewport
	UPROPERTY()
	bool bIsVisible = true;

	// If true, the node is able to be drawn to the view port.
	UPROPERTY(Transient)
	bool bIsNodeEnabled = false;

	UPROPERTY()
	bool bIsExpanded = true;

	UPROPERTY()
	FColor Color_DEPRECATED;

	UPROPERTY()
	FText Label;

	UPROPERTY()
	FName IconName_DEPRECATED;

	UPROPERTY()
	TWeakObjectPtr<UPoseWatchFolder> Parent;
#endif // WITH_EDITORONLY_DATA
};


#if WITH_EDITOR

template< class TElementType > TObjectPtr<TElementType> UPoseWatch::AddElement(const FText InLabel, const FName InIconName)
{
	TObjectPtr<TElementType> PoseWatchElement(NewObject<TElementType>(this));
	PoseWatchElement->SetParent(this);
	PoseWatchElement->SetUniqueLabel(InLabel);
	PoseWatchElement->SetIconName(InIconName);
	PoseWatchElement->SetColor(FColor::MakeRandomColor());
	Elements.Add(PoseWatchElement);

	return PoseWatchElement;
}

#endif // WITH_EDITOR
