// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AttributesRuntime.h"

#if WITH_EDITORONLY_DATA

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BoneIndices.h"
#include "Animation/AnimCurveTypes.h"

namespace UE::Anim
{
struct FStackAttributeContainer;
struct FHeapAttributeContainer;
}

struct FReferenceSkeleton;

struct FAnimNodePoseWatch
{
public:
	ENGINE_API FAnimNodePoseWatch();

	// Object (anim instance) that this pose came from
	const UObject* Object;
	class UPoseWatch* PoseWatch;
	class UPoseWatchPoseElement* PoseWatchPoseElement;
	int32 NodeID;

	ENGINE_API bool IsValid() const;

	ENGINE_API void SetCurves(const FBlendedCurve& InCurves);

	// _Moves_ Attributes out of InAttributes 
	ENGINE_API void SetAttributes(UE::Anim::FMeshAttributeContainer& InAttributes);

	template<class TAllocator>
	bool SetPose(const TArray<FBoneIndexType>& InRequiredBones, const TArray<FTransform, TAllocator>& InBoneTransforms)
	{
		RequiredBones = InRequiredBones;
		BoneTransforms = InBoneTransforms;
		return true;
	}

	ENGINE_API void SetWorldTransform(const FTransform& InWorldTransform);

	/**
	 * Take a snapshot of the pose watch properties that this struct represents
	 * to be used when drawing the debug skeleton
	 */
	ENGINE_API void CopyPoseWatchData(const FReferenceSkeleton& RefSkeleton);

	ENGINE_API const TArray<FBoneIndexType>& GetRequiredBones() const;

	ENGINE_API const TArray<FTransform>& GetBoneTransforms() const;

	ENGINE_API const FBlendedHeapCurve& GetCurves() const;
	
	ENGINE_API const UE::Anim::FMeshAttributeContainer& GetAttributes() const;
	
	ENGINE_API const FTransform& GetWorldTransform() const;

	ENGINE_API FLinearColor GetBoneColor() const;

	ENGINE_API FVector GetViewportOffset() const;

	ENGINE_API const TArray<int32>& GetViewportAllowList() const;

	ENGINE_API const TArray<int32>& GetParentIndices() const;

private:
	FTransform WorldTransform;
	TArray<FBoneIndexType> RequiredBones;
	TArray<FTransform> BoneTransforms;
	FBlendedHeapCurve Curves;
	UE::Anim::FMeshAttributeContainer MeshAttributes;
	
	// Mirrored properties updated on CopyPoseWatchData
	FLinearColor BoneColor;
	FVector ViewportOffset;
	TArray<int32> ViewportMaskAllowedList;
	TArray<int32> ParentIndices;
};

#endif
