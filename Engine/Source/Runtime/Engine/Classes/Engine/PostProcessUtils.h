// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API ENGINE_API

struct FPostProcessSettings;

/**
 * A utility class for blending FPostProcessSettings together without creating a FFinalPostProcessSettings
 * instance. This is useful for building up post-process settings in a modular way before handing it
 * off to the player camera manager, or other engine class that takes an FPostProcessSettings.
 */
struct FPostProcessUtils
{
	/**
	 * Overwrites post-process settings with another set of post-process settings. Only settings that are 
	 * overriden in the other set (OtherTo) are overwritten. This effectively "overlays" one set of settings
	 * on top of another.
	 *
	 * @param ThisFrom  The settings to write over.
	 * @param OtherTo   The settings to overlay on top.
	 * @return          Whether any setting was overwritten.
	 */
	static UE_API bool OverridePostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo);

	/**
	 * Blends values from one set of post-process settings to another, storing the result in place of 
	 * that first set. Settings that are overriden in *either* set are blended, so that settings may blend
	 * between different values, from a default value to a custom value, or from a custom value back to
	 * a default value.
	 *
	 * Some non-interpolable properties, like enums, get "flipped" over 50% blend. A couple others, like 
	 * ambient cubemaps, don't get accumulated like with FFinalPostProcessSettings and are instead also
	 * "flipped" over 50% blend. Blendable objects are not supported at this point.
	 *
	 * @param ThisFrom  The settings to blend from. Will receive the blended values.
	 * @param OtherTo   The settings to blend towards.
	 * @param BlendFactor  The blend factor.
	 * @return          Whether any setting was blended.
	 */
	static UE_API bool BlendPostProcessSettings(FPostProcessSettings& ThisFrom, const FPostProcessSettings& OtherTo, float BlendFactor);
};

#undef UE_API
