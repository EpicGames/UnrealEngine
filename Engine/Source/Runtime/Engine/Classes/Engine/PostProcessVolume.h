// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * PostProcessVolume:  a post process settings volume
 * Used to affect post process settings in the game and editor.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "GameFramework/Volume.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Scene.h"
#include "Interfaces/Interface_PostProcessVolume.h"

#include "PostProcessVolume.generated.h"

	// for FPostprocessSettings
UCLASS(autoexpandcategories=PostProcessVolume, hidecategories=(Advanced, Collision, Volume, Brush, Attachment), MinimalAPI)
class APostProcessVolume : public AVolume, public IInterface_PostProcessVolume
{
	GENERATED_UCLASS_BODY()

	/** Post process settings to use for this volume. */
	UPROPERTY(interp, Category=PostProcessVolumeSettings, meta=(ShowPostProcessCategories))
	struct FPostProcessSettings Settings;

	/**
	 * Priority of this volume. In the case of overlapping volumes the one with the highest priority
	 * overrides the lower priority ones. The order is undefined if two or more overlapping volumes have the same priority.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PostProcessVolumeSettings)
	float Priority;

	/** World space radius around the volume that is used for blending (only if not unbound).			*/
	UPROPERTY(interp, Category=PostProcessVolumeSettings, meta=(ClampMin = "0.0", UIMin = "0.0", UIMax = "6000.0"))
	float BlendRadius;

	/** 0:no effect, 1:full effect */
	UPROPERTY(interp, Category=PostProcessVolumeSettings, BlueprintReadWrite, meta=(UIMin = "0.0", UIMax = "1.0"))
	float BlendWeight;

	/** Whether this volume is enabled or not.															*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PostProcessVolumeSettings)
	uint32 bEnabled:1;

	/** Whether this volume covers the whole world, or just the area inside its bounds.								*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PostProcessVolumeSettings, meta=(DisplayName = "Infinite Extent (Unbound)"))
	uint32 bUnbound:1;

	/** Automatically generated GUID for this post process volume, used for deterministic sorting of equal priority volumes. */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient, NonTransactional)
	FGuid VolumeGuid;

	//~ Begin IInterface_PostProcessVolume Interface
	ENGINE_API virtual bool EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint) override;
	virtual FPostProcessVolumeProperties GetProperties() const override
	{
		FPostProcessVolumeProperties Ret;
		Ret.bIsEnabled = IsPPVEnabled();
		Ret.bIsUnbound = bUnbound != 0;
		Ret.BlendRadius = BlendRadius;
		Ret.BlendWeight = BlendWeight;
		Ret.Priority = Priority;
		Ret.Settings = &Settings;
		Ret.Size = Ret.bIsUnbound ? DBL_MAX : GetBounds().GetBox().GetVolume();
		Ret.VolumeGuid = VolumeGuid;
		return Ret;
	}
#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	virtual FString GetDebugName() const override
	{
		return GetActorNameOrLabel();
	}
#endif
	//~ End IInterface_PostProcessVolume Interface


	//~ Begin AActor Interface
	ENGINE_API virtual void PostUnregisterAllComponents( void ) override;

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return bUnbound ? false : Super::CanChangeIsSpatiallyLoadedFlag(); }
#endif

protected:
	ENGINE_API virtual void PostRegisterAllComponents() override;
	//~ End AActor Interface

	ENGINE_API bool IsPPVEnabled() const;
public:
	
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar);
	ENGINE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	/** Adds an Blendable (implements IBlendableInterface) to the array of Blendables (if it doesn't exist) and update the weight */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void AddOrUpdateBlendable(TScriptInterface<IBlendableInterface> InBlendableObject, float InWeight = 1.0f) { Settings.AddBlendable(InBlendableObject, InWeight); }
};



