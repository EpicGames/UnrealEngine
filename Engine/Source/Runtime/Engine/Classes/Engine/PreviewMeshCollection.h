// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimBlueprint.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/DataAsset.h"
#include "Animation/PreviewCollectionInterface.h"
#include "PreviewMeshCollection.generated.h"

class USkeleton;

/** An entry in a preview mesh collection */
USTRUCT()
struct FPreviewMeshCollectionEntry
{
	GENERATED_BODY()

	FPreviewMeshCollectionEntry()
	{}

	FPreviewMeshCollectionEntry(USkeletalMesh* InMesh)
		: SkeletalMesh(InMesh)
	{}

	bool operator==(const FPreviewMeshCollectionEntry& InEntry) const
	{
		return SkeletalMesh == InEntry.SkeletalMesh;
	}

	/** The skeletal mesh to display */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta=(DisplayThumbnail=true, DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh;

	/** The custom animation blueprint for the mesh */
	UPROPERTY(EditAnywhere, Category = "Anim Blueprint", meta=(DisplayThumbnail=true))
	TSoftObjectPtr<UAnimBlueprint> AnimBlueprint;
};

/** A simple collection of skeletal meshes used for in-editor preview */
UCLASS(MinimalAPI, BlueprintType)
class UPreviewMeshCollection : public UDataAsset, public IPreviewCollectionInterface
{
public:
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Skeleton", AssetRegistrySearchable)
	TObjectPtr<USkeleton> Skeleton;

	/** The skeletal meshes that this collection contains */
	UPROPERTY(EditAnywhere, Category = "Skeletal Meshes")
	TArray<FPreviewMeshCollectionEntry> SkeletalMeshes;

	/** return list of preview SkeletalMesh */
	virtual void GetPreviewSkeletalMeshes(TArray<USkeletalMesh*>& OutList, TArray<TSubclassOf<UAnimInstance>>& OutAnimBP) const override;
};
