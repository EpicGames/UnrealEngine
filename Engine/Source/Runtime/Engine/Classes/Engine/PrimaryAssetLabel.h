// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataAsset.h"
#include "Engine/AssetManagerTypes.h"
#include "Engine/EngineTypes.h"
#include "PrimaryAssetLabel.generated.h"

/** A seed file that is created to mark referenced assets as part of this primary asset */
UCLASS(MinimalAPI)
class UPrimaryAssetLabel : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	/** Bundle name used for directory-tagged assets */
	static ENGINE_API const FName DirectoryBundle;

	/** Bundle used for collection assets */
	static ENGINE_API const FName CollectionBundle;

	/** Constructor */
	ENGINE_API UPrimaryAssetLabel();
		
	/** Management rules for this specific asset, if set it will override the type rules */
	UPROPERTY(EditAnywhere, Category = Rules, meta = (ShowOnlyInnerProperties))
	FPrimaryAssetRules Rules;

	/** True to Label everything in this directory and sub directories */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel)
	uint32 bLabelAssetsInMyDirectory : 1;

	/** Set to true if the label asset itself should be cooked and available at runtime. This does not affect the assets that are labeled, they are set with cook rule */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel)
	uint32 bIsRuntimeLabel : 1;

	/** If true, redirectors found by the AssetLabel's explicit assets or directory will be labeled. If false, redirectors will be skipped. */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel)
	uint32 bIncludeRedirectors : 1;

	/** List of manually specified assets to label */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel, meta = (AssetBundles = "Explicit"))
	TArray<TSoftObjectPtr<UObject>> ExplicitAssets;

	/** List of manually specified blueprint assets to label */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel, meta = (AssetBundles = "Explicit", BlueprintBaseOnly))
	TArray<TSoftClassPtr<UObject>> ExplicitBlueprints;

	/** Collection to load asset references out of */
	UPROPERTY(EditAnywhere, Category = PrimaryAssetLabel)
	FCollectionReference AssetCollection;

	/** Set to editor only if this is not available in a cooked build */
	virtual bool IsEditorOnly() const override
	{
		return !bIsRuntimeLabel && !IsTemplate();
	}

#if WITH_EDITORONLY_DATA
	/** This scans the class for AssetBundles metadata on asset properties and initializes the AssetBundleData with InitializeAssetBundlesFromMetadata */
	ENGINE_API virtual void UpdateAssetBundleData();
#endif

};
