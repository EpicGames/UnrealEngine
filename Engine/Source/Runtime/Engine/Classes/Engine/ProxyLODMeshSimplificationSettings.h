// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "ProxyLODMeshSimplificationSettings.generated.h"

/**
* Controls the system used to generate proxy LODs with merged meshes (i.e. the HLOD system).
*/
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Hierarchical LOD Mesh Simplification"), MinimalAPI)
class UProxyLODMeshSimplificationSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual FName GetContainerName() const override;
	ENGINE_API virtual FName GetCategoryName() const override;

	ENGINE_API void SetProxyLODMeshReductionModuleName(FName InProxyLODMeshReductionModuleName);

public:
	/** Mesh reduction plugin to use when simplifying mesh geometry for Hierarchical LOD */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (ConsoleVariable = "r.ProxyLODMeshReductionModule", DisplayName = "Hierarchical LOD Mesh Reduction Plugin", ConfigRestartRequired = true))
	FName ProxyLODMeshReductionModuleName;

	ENGINE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};
