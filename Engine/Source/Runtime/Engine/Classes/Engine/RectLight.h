// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Light.h"
#include "RectLight.generated.h"

UCLASS(ClassGroup=(Lights, RectLights), ComponentWrapperClass, MinimalAPI, meta=(ChildCanTick))
class ARectLight : public ALight
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Light", meta=(ExposeFunctionCategories="RectLight,Rendering|Lighting"))
	TObjectPtr<class URectLightComponent> RectLightComponent;

#if WITH_EDITOR
	//~ Begin AActor Interface.
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif
};