// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "ReflectionCapture.generated.h"

class UBillboardComponent;

UCLASS(abstract, hidecategories=(Collision, Attachment, Actor), MinimalAPI)
class AReflectionCapture : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** Reflection capture component. */
	UPROPERTY(Category = DecalActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UReflectionCaptureComponent> CaptureComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

	UPROPERTY()
	TObjectPtr<UBillboardComponent> CaptureOffsetComponent;
#endif // WITH_EDITORONLY_DATA

public:	

	virtual bool IsLevelBoundsRelevant() const override { return false; }

	/** Returns CaptureComponent subobject **/
	class UReflectionCaptureComponent* GetCaptureComponent() const { return CaptureComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
	UBillboardComponent* GetCaptureOffsetComponent() const { return CaptureOffsetComponent; }
#endif
};



