// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "Engine/DeveloperSettings.h"
#include "PixelFormat.h"
#include "UObject/PerPlatformProperties.h"
#include "LegacyScreenPercentageDriver.h"

#include "RendererSettings.generated.h"

enum class ESkinCacheDefaultBehavior : uint8;
enum class EPSOPrecacheProxyCreationStrategy : uint8;

struct FPropertyChangedEvent;

/**
 * Enumerates ways to clear a scene.
 */
UENUM()
namespace EClearSceneOptions
{
	enum Type : int
	{
		NoClear = 0 UMETA(DisplayName="Do not clear",ToolTip="This option is fastest but can cause artifacts unless you render to every pixel. Make sure to use a skybox with this option!"),
		HardwareClear = 1 UMETA(DisplayName="Hardware clear",ToolTip="Perform a full hardware clear before rendering. Most projects should use this option."),
		QuadAtMaxZ = 2 UMETA(DisplayName="Clear at far plane",ToolTip="Draws a quad to perform the clear at the far plane, this is faster than a hardware clear on some GPUs."),
	};
}


/**
 * Enumerates available compositing sample counts.
 */
UENUM()
namespace ECompositingSampleCount
{
	enum Type : int
	{
		One = 1 UMETA(DisplayName="No MSAA"),
		Two = 2 UMETA(DisplayName="2x MSAA"),
		Four = 4 UMETA(DisplayName="4x MSAA"),
		Eight = 8 UMETA(DisplayName="8x MSAA"),
	};
}


/**
 * Enumerates available options for custom depth.
 */
UENUM()
namespace ECustomDepthStencil
{
	enum Type : int
	{
		Disabled = 0,
		Enabled = 1 UMETA(ToolTip="Depth buffer created immediately. Stencil disabled."),
		EnabledOnDemand = 2 UMETA(ToolTip="Depth buffer created on first use, can save memory but cause stalls. Stencil disabled."),
		EnabledWithStencil = 3 UMETA(ToolTip="Depth buffer created immediately. Stencil available for read/write."),
	};
}


/**
 * Enumerates available options for early Z-passes.
 */
UENUM()
namespace EEarlyZPass
{
	enum Type : int
	{
		None = 0 UMETA(DisplayName="None"),
		OpaqueOnly = 1 UMETA(DisplayName="Opaque meshes only"),
		OpaqueAndMasked = 2 UMETA(DisplayName="Opaque and masked meshes"),
		Auto = 3 UMETA(DisplayName="Decide automatically",ToolTip="Let the engine decide what to render in the early Z pass based on the features being used."),
	};
}

/**
 * Enumerates available options for velocity pass.
 */
UENUM()
namespace EVelocityOutputPass
{
	enum Type : int
	{
		DepthPass = 0 UMETA(DisplayName = "Write during depth pass"),
		BasePass = 1 UMETA(DisplayName = "Write during base pass"),
		AfterBasePass = 2 UMETA(DisplayName = "Write after base pass"),
	};
}

/**
* Enumerates available options for Single Layer Water velocity pass.
*/
UENUM()
namespace ESingleLayerWaterVelocityOutputPass
{
	enum Type : int
	{
		None = 0 UMETA(DisplayName = "Do not write velocity"),
		DepthPrepass = 1 UMETA(DisplayName = "Write during SLW depth prepass"),
		BasePass = 2 UMETA(DisplayName = "Write during SLW base pass"),
	};
}

/**
 * Enumerates available options for Vertex Deformation Outputs Velocity.
 */
UENUM()
namespace EVertexDeformationOutputsVelocity
{
	enum Type : int
	{
		Off = 0 UMETA(ToolTip = "Always off"),
		On = 1 UMETA(ToolTip = "Always on"),
		Auto = 2 UMETA(ToolTip = "On when the performance cost is low (velocity in depth or base pass)."),
	};
}

UENUM()
namespace EAlphaChannelMode
{
	enum UE_DEPRECATED(5.5, "Propagate alpha was converted back to a boolean.") Type : int
	{
		/** Disabled, reducing GPU cost to the minimum. (default). */
		Disabled = 0,

		/** Propagate alpha channel through post-processing. */
		Enabled = 1,

		// Deprecated
		LinearColorSpaceOnly = Enabled UMETA(Hidden),
		AllowThroughTonemapper = Enabled UMETA(Hidden)
	};
}

/** used by FPostProcessSettings AutoExposure*/
UENUM()
namespace EAutoExposureMethodUI
{
	enum Type : int
	{
		/** requires compute shader to construct 64 bin histogram */
		AEM_Histogram  UMETA(DisplayName = "Auto Exposure Histogram"),
		/** faster method that computes single value by downsampling */
		AEM_Basic      UMETA(DisplayName = "Auto Exposure Basic"),
		/** Uses camera settings. */
		AEM_Manual   UMETA(DisplayName = "Manual"),
		AEM_MAX,
	};
}

/** used by GetDefaultBackBufferPixelFormat*/
UENUM()
namespace EDefaultBackBufferPixelFormat
{
	enum Type : int
	{
		DBBPF_B8G8R8A8 = 0				UMETA(DisplayName = "8bit RGBA"),
		DBBPF_A16B16G16R16_DEPRECATED	UMETA(DisplayName = "DEPRECATED - 16bit RGBA", Hidden),
		DBBPF_FloatRGB_DEPRECATED		UMETA(DisplayName = "DEPRECATED - Float RGB", Hidden),
		DBBPF_FloatRGBA					UMETA(DisplayName = "Float RGBA"),
		DBBPF_A2B10G10R10				UMETA(DisplayName = "10bit RGB, 2bit Alpha"),
		DBBPF_MAX,
	};
}

/**
* Enumerates VRS Fixed-foveation levels
*/
UENUM()
namespace EFixedFoveationLevels
{
	enum Type : int
	{
		Disabled = 0 UMETA(DisplayName = "Disabled"),
		Low = 1 UMETA(DisplayName = "Low"),
		Medium = 2 UMETA(DisplayName = "Medium"),
		High = 3 UMETA(DisplayName = "High"),
		HighTop = 4 UMETA(DisplayName = "High Top"),
	};
}

UENUM()
namespace EMobileAntiAliasingMethod
{
	enum Type : int
	{
		None = AAM_None UMETA(DisplayName = "None"),
		FXAA = AAM_FXAA UMETA(DisplayName = "Fast Approximate Anti-Aliasing (FXAA)"),
		TemporalAA = AAM_TemporalAA UMETA(DisplayName = "Temporal Anti-Aliasing (TAA)"),
		/** Only supported with forward shading.  MSAA sample count is controlled by r.MSAACount. */
		MSAA = AAM_MSAA UMETA(DisplayName = "Multisample Anti-Aliasing (MSAA)"),
		TSR = AAM_TSR UMETA(DisplayName = "Temporal Super-Resolution (TSR)"),
		SMAA = AAM_SMAA UMETA(DisplayName = "Subpixel Morphological Anti-Aliasing (SMAA)"),
	};
}

/** The default float precision for material's pixel shaders on mobile devices*/
UENUM()
namespace EMobileFloatPrecisionMode
{
	enum Type : int
	{
		/** Half precision, except explict 'float' in .ush/.usf*/
		Half = 0 UMETA(DisplayName = "Use Half-precision"),
		/** Half precision, except Full precision for material floats and explicit floats in .ush/.usf*/
		Full_MaterialExpressionOnly = 1 UMETA(DisplayName = "Use Full-precision for MaterialExpressions only"),
		/** All the floats are full-precision */
		Full = 2 UMETA(DisplayName = "Use Full-precision for every float"),
	};
}

UENUM()
namespace EMobileShadingPath
{
	enum Type : int
	{
		/** The default shading path for mobile, supported on all devices. */
		Forward = 0 UMETA(DisplayName = "Forward Shading"),
		/** Use deferred shading. This path supports additional features compared to the Forward option but requires more modern devices. Features include: IES light profiles, light functions, lit deferred decals. Does not support MSAA. */
		Deferred = 1 UMETA(DisplayName = "Deferred Shading"),
	};
}

namespace EDefaultBackBufferPixelFormat
{
	ENGINE_API EPixelFormat Convert2PixelFormat(EDefaultBackBufferPixelFormat::Type InDefaultBackBufferPixelFormat);
	ENGINE_API int32 NumberOfBitForAlpha(EDefaultBackBufferPixelFormat::Type InDefaultBackBufferPixelFormat);
	ENGINE_API EDefaultBackBufferPixelFormat::Type FromInt(int32 InDefaultBackBufferPixelFormat);
}

UENUM()
namespace ELightFunctionAtlasPixelFormat
{
	enum Type : int
	{
		LFAPF_R8 = 0				UMETA(DisplayName = "8 bits Gray Scale"),
		LFAPF_R8G8B8 = 1			UMETA(DisplayName = "8 bits RGB  Color")
	};
}

/**
 * Enumerates supported shader compression formats.
 */
UENUM()
namespace EShaderCompressionFormat
{
	enum Type : int
	{
		None = 0 UMETA(DisplayName = "Do not compress", ToolTip = "Fastest, but disk and memory footprint will be large"),
		LZ4 = 1 UMETA(DisplayName = "LZ4", ToolTip = "Compressing using LZ4"),
		Oodle = 2 UMETA(DisplayName = "Oodle", ToolTip = "Compressing using Oodle (default)"),
		Zlib = 3 UMETA(DisplayName = "ZLib", ToolTip = "Compressing using Zlib")
	};
}

UENUM()
namespace ELumenSoftwareTracingMode
{
	enum Type : int
	{
		DetailTracing = 1 UMETA(DisplayName = "Detail Tracing", ToolTip = "When using Software Ray Tracing, Lumen will trace against individual mesh's Distance Fields for highest quality.  Cost can be high in scenes with many overlapping instances."),
		GlobalTracing = 0 UMETA(DisplayName = "Global Tracing", ToolTip = "When using Software Ray Tracing, Lumen will trace against the Global Distance Field for fastest traces."),
	};
}

UENUM()
enum class ELumenRayLightingMode : uint8
{
	/* Use Lumen Surface Cache for ray hit lighting. This method gives the best GI and reflection performance, but quality will be limited by how well surface cache represents given scene. */
	SurfaceCache = 0				UMETA(DisplayName = "Surface Cache"),
	/* Calculate lighting at a hit point for reflections. This will improve reflection quality, but increases GPU cost, as full material needs to be evaluated and shadow rays traced. Lumen Surface Cache will still be used for GI and secondary bounces, including GI seen in reflections. */
	HitLightingForReflections = 2 	UMETA(DisplayName = "Hit Lighting for Reflections"),
	/* Calculate lighting at a hit point for GI and reflections. This will improve both GI and reflection quality, but greatly increases GPU cost, as full material and lighting will be evaluated at every hit point. Lumen Surface Cache will still be used for secondary bounces. */
	HitLighting = 1 				UMETA(DisplayName = "Hit Lighting"),
};

UENUM()
enum class ELumenScreenTracingSource : uint8
{
	/* Lumen screen traces will sample the previous frame's Scene Color. This will not contain translucency and can increase Lumen's noise from small emissive elements, but has less leaking. */
	SceneColor = 0				UMETA(DisplayName = "Scene Color"),
	/* Lumen screen traces will sample the previous frame's anti-aliased Scene Color (TSR output). This contains translucency and is prefiltered, reducing Lumen's noise from small emissive elements. */
	AntialiasedSceneColorWithTranslucency = 1 	UMETA(DisplayName = "Anti-aliased Scene Color, with Translucency"),
};

UENUM()
namespace EWorkingColorSpace
{
	enum Type : int
	{
		sRGB = 1 UMETA(DisplayName = "sRGB / Rec709", ToolTip = "sRGB / Rec709 (BT.709) color primaries, with D65 white point."),
		Rec2020 = 2 UMETA(DisplayName = "Rec2020", ToolTip = "Rec2020 (BT.2020) primaries with D65 white point."),
		ACESAP0 = 3 UMETA(DIsplayName = "ACES AP0", ToolTip = "ACES AP0 wide gamut primaries, with D60 white point."),
		ACESAP1 = 4 UMETA(DIsplayName = "ACES AP1 / ACEScg", ToolTip = "ACES AP1 / ACEScg wide gamut primaries, with D60 white point."),
		P3DCI = 5 UMETA(DisplayName = "P3DCI", ToolTip = "P3 (Theater) primaries, with DCI Calibration white point."),
		P3D65 = 6 UMETA(DisplayName = "P3D65", ToolTip = "P3 (Display) primaries, with D65 white point."),
		Custom = 7 UMETA(DisplayName = "Custom", ToolTip = "User defined color space and white point."),
	};
}

UENUM()
enum class ESlateMaterialShaderMode : uint8
{
	/* Use static specializations for all Slate material shaders. This results in the most shaders and PSOs compiled for each Slate material, but leads to optimal performance. */
	Static = 0		UMETA(DisplayName = "Static"),
	/* Use static specialization for Default Slate shader type and dynamic branch version for the rest. This balances performance and PSO/shader count. */
	Hybrid  = 1 	UMETA(DisplayName = "Hybrid"),
	/* Only use dynamic branch version of Slate material shaders. Leads to some performance overhead, but the least number of total shaders. */
	Dynamic = 2 	UMETA(DisplayName = "Dynamic"),
};

UENUM()
enum class ESkinningShaderSupportMode : uint8
{
	// Only compile vertex factories for inline skinning. 
	InlineShadersOnly = 0,
	// Compile shaders for both inline and skin cache skinning.
	InlineAndSkinCacheShaders = 1,
	// Only compile shaders for skin cache skinning.
	SkinCacheShadersOnly = 2,
};

/**
 * Rendering settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Rendering"), MinimalAPI)
class URendererSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.ShadingPath", DisplayName = "Mobile Shading",
		ToolTip = "The shading path to use on mobile platforms. Changing this setting requires restarting the editor. Forward shading will force MSAA. Mobile HDR is required for Deferred Shading.",
		ConfigRestartRequired = true))
	TEnumAsByte<EMobileShadingPath::Type> MobileShadingPath;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		EditCondition = "MobileShadingPath == EMobileShadingPath::Deferred",
		ConsoleVariable = "r.Mobile.ExtendedGBuffer", DisplayName = "Extended GBuffer",
		ToolTip = "Whether to enable 'Extended GBuffer' for mobile deferred shading.\nAdds support for non-default shading models with static lighting, additionally improves normal encoding and blending in projects without static lighting.\nUses 160 bits per pixel vs. the default 128",
		ConfigRestartRequired = true))
	bool bMobileExtendedGBuffer;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.AntiAliasing", DisplayName = "Anti-Aliasing Method",
		ToolTip = "The mobile default anti-aliasing method."))
	TEnumAsByte<EMobileAntiAliasingMethod::Type> MobileAntiAliasing;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.FloatPrecisionMode", DisplayName = "Float Precision Mode",
		ToolTip = "Project wide mobile float precision mode for shaders and materials. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	TEnumAsByte<EMobileFloatPrecisionMode::Type> MobileFloatPrecisionMode;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta=(
		ConsoleVariable="r.Mobile.AllowDitheredLODTransition", DisplayName="Allow Dithered LOD Transition",
		ToolTip="Whether to support 'Dithered LOD Transition' material option on mobile platforms. Enabling this may degrade performance as rendering will not benefit from Early-Z optimization.",
		ConfigRestartRequired=true))
	bool bMobileAllowDitheredLODTransition;

	UPROPERTY()
	uint32 bMobileVirtualTextures_DEPRECATED : 1; // Deprecated: override bVirtualTextures in per platform .ini files instead.

	UPROPERTY(config, EditAnywhere, Category = Materials, meta = (
		ConsoleVariable = "r.DiscardUnusedQuality", DisplayName = "Game Discards Unused Material Quality Levels",
		ToolTip = "When running in game mode, whether to keep shaders for all quality levels in memory or only those needed for the current quality level.\nUnchecked: Keep all quality levels in memory allowing a runtime quality level change. (default)\nChecked: Discard unused quality levels when loading content for the game, saving some memory."))
	uint32 bDiscardUnusedQualityLevels : 1;

	UPROPERTY()
	TEnumAsByte<EShaderCompressionFormat::Type> ShaderCompressionFormat_DEPRECATED;	// we now force oodle

	UPROPERTY(config, EditAnywhere, Category=Culling, meta=(
		ConsoleVariable="r.AllowOcclusionQueries",DisplayName="Occlusion Culling",
		ToolTip="Allows occluded meshes to be culled and not rendered."))
	uint32 bOcclusionCulling:1;

	UPROPERTY(config, EditAnywhere, Category=Culling, meta=(
		ConsoleVariable="r.MinScreenRadiusForLights",DisplayName="Min Screen Radius for Lights",
		ToolTip="Screen radius at which lights are culled. Larger values can improve performance but causes lights to pop off when they affect a small area of the screen."))
	float MinScreenRadiusForLights;

	UPROPERTY(config, EditAnywhere, Category=Culling, meta=(
		ConsoleVariable="r.MinScreenRadiusForDepthPrepass",DisplayName="Min Screen Radius for Early Z Pass",
		ToolTip="Screen radius at which objects are culled for the early Z pass. Larger values can improve performance but very large values can degrade performance if large occluders are not rendered."))
	float MinScreenRadiusForEarlyZPass;

	UPROPERTY(config, EditAnywhere, Category=Culling, meta=(
		ConsoleVariable="r.PrecomputedVisibilityWarning",DisplayName="Warn about no precomputed visibility",
		ToolTip="Displays a warning when no precomputed visibility data is available for the current camera location. This can be helpful if you are making a game that relies on precomputed visibility, e.g. a first person mobile game."))
	uint32 bPrecomputedVisibilityWarning:1;

	UPROPERTY(config, EditAnywhere, Category=Textures, meta=(
		ConsoleVariable="r.TextureStreaming",DisplayName="Texture Streaming",
		ToolTip="When enabled textures will stream in based on what is visible on screen."))
	uint32 bTextureStreaming:1;

	UPROPERTY(config, EditAnywhere, Category=Textures, meta=(
		ConsoleVariable="Compat.UseDXT5NormalMaps",DisplayName="Use DXT5 Normal Maps",
		ToolTip="Whether to use DXT5 for normal maps, otherwise BC5 will be used, which is not supported on all hardware. Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bUseDXT5NormalMaps:1;

	/**
	 * Virtual Texture
	 */
	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		ConsoleVariable = "r.VirtualTextures", DisplayName = "Enable virtual texture support",
		ToolTip = "When enabled, textures can be streamed using the virtual texture system. Can be overridden per platform in the <Platform>Engine.ini files. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bVirtualTextures : 1;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.VirtualTexturedLightmaps", DisplayName = "Enable virtual texture lightmaps",
		ToolTip = "When enabled, lightmaps will be streamed using the virtual texture system. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bVirtualTexturedLightmaps : 1;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.VT.TileSize", DisplayName = "Tile size",
		ToolTip = "Size in pixels for virtual texture tiles, will be rounded to next power-of-2. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 VirtualTextureTileSize;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.VT.TileBorderSize", DisplayName = "Tile border size",
		ToolTip = "Size in pixels for virtual texture tile borders, will be rounded to next multiple-of-2. Larger borders allow higher degree of anisotropic filtering, but uses more disk/cache memory. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 VirtualTextureTileBorderSize;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.VT.AnisotropicFiltering", DisplayName = "Enable virtual texture anisotropic filtering",
		ToolTip = "When enabled, virtual textures will use anisotropic filtering. This adds a cost to all shaders using virtual textures. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bVirtualTextureAnisotropicFiltering : 1;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.VT.EnableAutoImport", DisplayName = "Enable virtual texture on texture import",
		ToolTip = "Enable virtual texture streaming for imported textures based on the 'Auto Virtual Texturing Size' in the texture import settings.",
		ConfigRestartRequired = false))
	uint32 bVirtualTextureEnableAutoImport : 1;

	UPROPERTY(config, EditAnywhere, Category = VirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		DisplayName = "Enable virtual texture for opacity mask on material import.",
		ToolTip = "When virtual textures contribute to opacity mask it can sometimes lead to low resolution shadow edges."))
	uint32 bEnableVirtualTextureOpacityMask : 1;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.MeshPaintVirtualTexture.Support", DisplayName = "Enable virtual texture mesh painting",
		ToolTip = "When enabled, mesh paint virtual textures are supported. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bMeshPaintVirtualTextureSupport : 1;
		
	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.MeshPaintVirtualTexture.TileSize", DisplayName = "Mesh paint tile size",
		ToolTip = "Size in pixels for mesh paint virtual texture tiles. Will be rounded to next power-of-2. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 MeshPaintVirtualTextureTileSize;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.MeshPaintVirtualTexture.TileBorderSize", DisplayName = "Mesh paint tile border size",
		ToolTip = "Size in pixels for virtual texture tile borders. Will be rounded next to multiple-of-2. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 MeshPaintVirtualTextureTileBorderSize;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.MeshPaintVirtualTexture.UseCompression", DisplayName = "Mesh paint use compressed textures",
		ToolTip = "Whether to use a compressed texture format for storing mesh paint textures.",
		ConfigRestartRequired = true))
	bool MeshPaintVirtualTextureUseCompression;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.StaticMesh.DefaultMeshPaintTextureSupport", DisplayName = "Mesh paint default static mesh support",
		ToolTip = "Default setting for whether static mesh assets support mesh paint textures."))
	bool MeshPaintDefaultOnStaticMesh;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.MeshPaintVirtualTexture.DefaultTexelsPerVertex", DisplayName = "Mesh paint texels per vertex",
		ToolTip = "Default ratio of texels to vertices when creating a mesh paint texture for a mesh."))
	int32 MeshPaintDefaultTexelsPerVertex;

	UPROPERTY(config, EditAnywhere, Category = MeshPaintVirtualTextures, meta = (
		EditCondition = "bVirtualTextures && bMeshPaintVirtualTextureSupport",
		ConsoleVariable = "r.MeshPaintVirtualTexture.MaxTextureSize", DisplayName = "Mesh paint maximum texture size",
		ToolTip = "Maximum size in pixels for mesh paint virtual textures. Will be rounded to next power-of-2."))
	int32 MeshPaintVirtualTextureMaxTextureSize;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableBaseColor", DisplayName = "Enable 'Base Color'",
		ToolTip = "Disable material type when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTBaseColor;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableBaseColorRoughness", DisplayName = "Enable 'Base Color, Normal, Roughness'",
		ToolTip = "Disable material type when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTBaseColorRoughness;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableBaseColorSpecular", DisplayName = "Enable 'Base Color, Normal, Roughness, Specular' family",
		ToolTip = "Disable family of material types when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTBaseColorSpecular;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableMask4", DisplayName = "Enable 'Mask4'",
		ToolTip = "Disable material type when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTMask4;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableWorldHeight", DisplayName = "Enable 'World Height'",
		ToolTip = "Disable material type when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTWorldHeight;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures|Enable Material Types", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.EnableDisplacement", DisplayName = "Enable 'Displacement'",
		ToolTip = "Disable material type when not required by a project. This minimizes shader permutation count for materials with Runtime Virtual Texture Output nodes.",
		ConfigRestartRequired = true))
	bool bEnableRVTDisplacement;

	UPROPERTY(config, EditAnywhere, Category = "Runtime Virtual Textures", meta = (
		EditCondition = "bVirtualTextures",
		ConsoleVariable = "r.vt.rvt.HighQualityPerPixelHeight", DisplayName = "High quality landscape height sampling.",
		ToolTip = "Use higher quality sampling of per pixel heightmaps when rendering to Runtime Virtual Texture.",
		ConfigRestartRequired = true))
	bool bUseHighQualityRVTHeightSampling;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		DisplayName = "Working Color Space",
		ToolTip = "Choose from list of provided working color spaces, or custom to provide user-defined space.",
		ConfigRestartRequired = true))
	TEnumAsByte<EWorkingColorSpace::Type> WorkingColorSpaceChoice;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		EditCondition = "WorkingColorSpaceChoice == EWorkingColorSpace::Custom",
		ToolTip = "Working color space red chromaticity coordinates.",
		ConfigRestartRequired = true))
	FVector2D RedChromaticityCoordinate;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		EditCondition = "WorkingColorSpaceChoice == EWorkingColorSpace::Custom",
		ToolTip = "Working color space green chromaticity coordinates.",
		ConfigRestartRequired = true))
	FVector2D GreenChromaticityCoordinate;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		EditCondition = "WorkingColorSpaceChoice == EWorkingColorSpace::Custom",
		ToolTip = "Working color space blue chromaticity coordinates.",
		ConfigRestartRequired = true))
	FVector2D BlueChromaticityCoordinate;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		EditCondition = "WorkingColorSpaceChoice == EWorkingColorSpace::Custom",
		ToolTip = "Working color space white chromaticity coordinates.",
		ConfigRestartRequired = true))
	FVector2D WhiteChromaticityCoordinate;

	UPROPERTY(config, EditAnywhere, Category = WorkingColorSpace, meta = (
		ConsoleVariable = "r.LegacyLuminanceFactors",
		ToolTip = "Default luminance factors: 0 enables the working color space coefficients (default), 1 enables inaccurate legacy coefficients.",
		ConfigRestartRequired = true))
	uint32 bUseLegacyLuminanceFactors : 1;

	UPROPERTY(config, EditAnywhere, Category = Materials, meta =(
		ConfigRestartRequired = true,
		ConsoleVariable = "r.ClearCoatNormal",
		ToolTip = "Use a separate normal map for the bottom layer of a clear coat material. This is a higher quality feature that is expensive."))
	uint32 bClearCoatEnableSecondNormal : 1;
	
	UPROPERTY(config, EditAnywhere, Category = GlobalIllumination, meta=(
		ConsoleVariable="r.DynamicGlobalIlluminationMethod",DisplayName="Dynamic Global Illumination Method",
		ToolTip="Dynamic Global Illumination Method"))
	TEnumAsByte<EDynamicGlobalIlluminationMethod::Type> DynamicGlobalIllumination;

	UPROPERTY(config, EditAnywhere, Category=Reflections, meta=(
		ConsoleVariable="r.ReflectionMethod",DisplayName="Reflection Method",
		ToolTip="Reflection Method"))
	TEnumAsByte<EReflectionMethod::Type> Reflections;

	UPROPERTY(config, EditAnywhere, Category = Reflections, meta = (
		ConsoleVariable = "r.ReflectionCaptureResolution", DisplayName = "Reflection Capture Resolution",
		ToolTip = "The cubemap resolution for all reflection capture probes. Must be power of 2. Note that for very high values the memory and performance impact may be severe."))
	int32 ReflectionCaptureResolution;

	UPROPERTY(config, EditAnywhere, Category = Reflections, meta = (
		ConsoleVariable = "r.ReflectionEnvironmentLightmapMixBasedOnRoughness", DisplayName = "Reduce lightmap mixing on smooth surfaces",
		ToolTip = "Whether to reduce lightmap mixing with reflection captures for very smooth surfaces.  This is useful to make sure reflection captures match SSR / planar reflections in brightness."))
	uint32 ReflectionEnvironmentLightmapMixBasedOnRoughness : 1;

	UPROPERTY(config, EditAnywhere, Category = Lumen, meta = (
		EditCondition = "bEnableRayTracing",
		ConsoleVariable = "r.Lumen.HardwareRayTracing", DisplayName = "Use Hardware Ray Tracing when available",
		ToolTip = "Uses Hardware Ray Tracing for Lumen features, when supported by the video card + RHI + Operating System. Lumen will fall back to Software Ray Tracing otherwise. Note: Hardware ray tracing has significant scene update costs for scenes with more than 100k instances."))
	uint32 bUseHardwareRayTracingForLumen : 1;

	UPROPERTY(config, EditAnywhere, Category=Lumen, meta=(
		EditCondition = "bEnableRayTracing && bUseHardwareRayTracingForLumen",
		ConsoleVariable="r.Lumen.HardwareRayTracing.LightingMode", DisplayName = "Ray Lighting Mode",
		ToolTip="Controls how Lumen Reflection rays are lit when Lumen is using Hardware Ray Tracing.  By default, Lumen uses the Surface Cache for best performance, but can be set to 'Hit Lighting' for higher quality."))
	ELumenRayLightingMode LumenRayLightingMode;

	UPROPERTY(config, EditAnywhere, Category = Lumen, meta = (
		ConsoleVariable = "r.Lumen.TranslucencyReflections.FrontLayer.EnableForProject", DisplayName = "High Quality Translucency Reflections",
		ToolTip = "Whether to use high quality mirror reflections on the front layer of translucent surfaces.  Other layers will use the lower quality Radiance Cache method that can only produce glossy reflections.  Increases GPU cost when enabled."))
	uint32 LumenFrontLayerTranslucencyReflections : 1;

	UPROPERTY(config, EditAnywhere, Category=Lumen, meta=(
		EditCondition = "bGenerateMeshDistanceFields",
		ConsoleVariable="r.Lumen.TraceMeshSDFs", DisplayName = "Software Ray Tracing Mode",
		ToolTip="Controls which tracing method Lumen uses when using Software Ray Tracing."))
	TEnumAsByte<ELumenSoftwareTracingMode::Type> LumenSoftwareTracingMode;

	UPROPERTY(config, EditAnywhere, Category=Lumen, meta=(
		ConsoleVariable="r.Lumen.ScreenTracingSource", DisplayName = "Screen Tracing Source",
		ToolTip="Specifies which Scene Color texture Lumen's Screen Traces should read from."))
	ELumenScreenTracingSource LumenScreenTracingSource;

	UPROPERTY(config, EditAnywhere, Category = Lumen, meta = (
		ConsoleVariable = "r.Lumen.Reflections.HardwareRayTracing.Translucent.Refraction.EnableForProject", DisplayName = "Ray Traced Translucent Refractions",
		ToolTip = "Whether to use Lumen refraction tracing from surfaces when using harware ray tracing and hit lighting. This will require shader recompilation to compile of translucent card capture Lumen shaders. Increases GPU cost when enabled."))
	uint32 LumenRayTracedTranslucentRefractions : 1;

	UPROPERTY(config, EditAnywhere, Category = DirectLighting, meta = (
		ConsoleVariable = "r.MegaLights.EnableForProject", DisplayName = "MegaLights",
		ToolTip = "Whether to use MegaLights by default, but this can still be overridden by Post Process Volumes, or disabled per-light. MegaLights uses stochastic sampling to render many shadow casting lights efficiently, with a consistent low GPU cost. When MegaLights is enabled, other direct lighting algorithms like Deferred Shading will no longer be used, and other shadowing methods like Ray Traced Shadows, Distance Field Shadows and Shadow Maps will no longer be used. MegaLights requires Hardware Ray Tracing, and does not support Directional Lights. Experimental feature."))
	uint32 bEnableMegaLights : 1;

	UPROPERTY(config, EditAnywhere, Category = DirectLighting, meta = (
		ConsoleVariable = "r.MegaLights.FrontLayerTranslucency.EnableForProject", DisplayName = "MegaLights - High Quality Translucency",
		ToolTip = "Whether to use high quality direct lighting on the front layer of translucent surfaces.  Other layers will use the lower quality Translucency Lighting Volume.  Increases GPU cost when enabled."))
	uint32 bEnableMegaLightsFrontLayerTranslucency : 1;

	UPROPERTY(config, EditAnywhere, Category = DirectLighting, meta = (
		ConsoleVariable = "r.RayTracing.Shadows", DisplayName = "Ray Traced Shadows",
		ToolTip = "Controls whether Ray Traced Shadows are used by default. Lights can still override and force Ray Traced shadows on or off. Requires Hardware Ray Tracing to be enabled."))
	uint32 bEnableRayTracingShadows : 1;

	UPROPERTY(config, EditAnywhere, Category = DirectLighting, meta = (
		ConsoleVariable = "r.Shadow.Virtual.Enable", DisplayName = "Shadow Map Method",
		ToolTip = "Select the primary shadow mapping method. Automatically uses 'Shadow Maps' when Forward Shading is enabled for the project as Virtual Shadow Maps are not supported."))
	TEnumAsByte<EShadowMapMethod::Type> ShadowMapMethod;

	/**
	 * "Ray Tracing settings."
	 */
	UPROPERTY(config, EditAnywhere, Category = HardwareRayTracing, meta = (
		ConsoleVariable = "r.RayTracing", DisplayName = "Support Hardware Ray Tracing",
		ToolTip = "Support Hardware Ray Tracing features.  Requires 'Support Compute Skin Cache' before project is allowed to set this.",
		ConfigRestartRequired = true))
		uint32 bEnableRayTracing : 1;

	UPROPERTY(config, EditAnywhere, Category = HardwareRayTracing, meta = (
		ConsoleVariable = "r.RayTracing.RayTracingProxies.ProjectEnabled", DisplayName = "Generate Ray Tracing Proxies",
		ToolTip = "Whether to generate dedicated ray tracing proxies for Nanite meshes by default (instead of using the fallback mesh).\nIt's recommended to enable this setting as it guarantees that static meshes have multiple ray tracing LODs to support streaming.",
		ConfigRestartRequired = true))
		uint32 bGenerateRayTracingProxies : 1;

	UPROPERTY()
		uint32 bEnableRayTracingSkylight_DEPRECATED : 1;

	UPROPERTY(config, EditAnywhere, Category = HardwareRayTracing, meta = (
		ConsoleVariable = "r.RayTracing.UseTextureLod", DisplayName = "Texture LOD",
		ToolTip = "Enable automatic texture mip level selection in ray tracing material shaders. Unchecked: highest resolution mip level is used for all texture (default). Checked: texture LOD is approximated based on total ray length, output resolution and texel density at hit point (ray cone method).",
		ConfigRestartRequired = true))
		uint32 bEnableRayTracingTextureLOD : 1;

	UPROPERTY(config, EditAnywhere, Category = HardwareRayTracing, meta = (
		ConsoleVariable = "r.PathTracing", DisplayName = "Path Tracing",
		ToolTip = "Enables the Path Tracing renderer. This enables additional material permutations. Requires Hardware Ray Tracing to be enabled.",
		ConfigRestartRequired = true))
		uint32 bEnablePathTracing : 1;

	UPROPERTY(config, EditAnywhere, Category=SoftwareRayTracing, meta=(
		ConsoleVariable="r.GenerateMeshDistanceFields",
		ToolTip="Whether to build distance fields of static meshes, needed for Software Ray Tracing in Lumen, and distance field AO, which is used to implement Movable SkyLight shadows, and ray traced distance field shadows on directional lights.  Enabling will increase the build times, memory usage and disk size of static meshes.  Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bGenerateMeshDistanceFields:1;

	UPROPERTY(config, EditAnywhere, Category=SoftwareRayTracing, meta=(
		EditCondition = "bGenerateMeshDistanceFields",
		ConsoleVariable="r.DistanceFields.DefaultVoxelDensity",
		ClampMin = ".05", ClampMax = ".4",
		ToolTip="Determines how the default scale of a mesh converts into distance field voxel dimensions. Changing this will cause all distance fields to be rebuilt.  Large values can consume memory very quickly!  Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	float DistanceFieldVoxelDensity;

	UPROPERTY(config, EditAnywhere, Category = Nanite, meta = (
		ConsoleVariable = "r.Nanite.ProjectEnabled",
		ToolTip = "Whether to enable Nanite rendering.",
		ConfigRestartRequired = true))
	uint32 bNanite : 1;

	UPROPERTY(config, EditAnywhere, Category=MiscLighting, meta=(
		ConsoleVariable="r.AllowStaticLighting",
		ToolTip="Whether to allow any static lighting to be generated and used, like lightmaps and shadowmaps. Games that only use dynamic lighting should set this to 0 to save some static lighting overhead. Disabling allows Material AO and Material BentNormal to work with Lumen Global Illumination. When StaticLighting is disabled, it also enables certain effects related to first person rendering, such as player shadows on the ground. Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bAllowStaticLighting:1;

	UPROPERTY(config, EditAnywhere, Category=MiscLighting, meta=(
		ConsoleVariable="r.NormalMapsForStaticLighting",
		ToolTip="Whether to allow any static lighting to use normal maps for lighting computations."))
	uint32 bUseNormalMapsForStaticLighting:1;

	UPROPERTY(config, EditAnywhere, Category=ForwardRenderer, meta=(
		ConsoleVariable="r.ForwardShading",
		DisplayName = "Forward Shading",
		ToolTip="Whether to use forward shading on desktop platforms, requires Shader Model 5 hardware.  Forward shading requires MSAA and has lower default cost, but fewer features supported overall.  Materials have to opt-in to more expensive features like high quality reflections.  Deferred shading does not support MSAA.  Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bForwardShading:1;

	UPROPERTY(config, EditAnywhere, Category=ForwardRenderer, meta=(
		ConsoleVariable="r.VertexFoggingForOpaque",
		ToolTip="Causes opaque materials to use per-vertex fogging, which costs slightly less.  Only supported with forward shading. Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bVertexFoggingForOpaque:1;

	UPROPERTY(config, EditAnywhere, Category=Translucency, meta=(
		ConsoleVariable="r.SeparateTranslucency",
		ToolTip="Allow translucency to be rendered to a separate render targeted and composited after depth of field. Prevents translucency from appearing out of focus."))
	uint32 bSeparateTranslucency:1;

	UPROPERTY(config, EditAnywhere, Category=Translucency, meta=(
		ConsoleVariable="r.TranslucentSortPolicy",DisplayName="Translucent Sort Policy",
		ToolTip="The sort mode for translucent primitives, affecting how they are ordered and how they change order as the camera moves. Requires that Separate Translucency (under Postprocessing) is true."))
	TEnumAsByte<ETranslucentSortPolicy::Type> TranslucentSortPolicy;

	UPROPERTY(config, EditAnywhere, Category=Translucency, meta=(
		DisplayName="Translucent Sort Axis",
		ToolTip="The axis that sorting will occur along when Translucent Sort Policy is set to SortAlongAxis."))
	FVector TranslucentSortAxis;

	UPROPERTY(config, EditAnywhere, Category=Translucency, meta=( 
		ConsoleVariable="r.LocalFogVolume.ApplyOnTranslucent",
		ToolTip="Allow local fog volumes to be combined and rendered over translucent meshes. Only per vertex evaluation is supported as of today. It requires r.SupportLocalFogVolumes to be true",
		ConfigRestartRequired = true))
	uint32 bLocalFogVolumeApplyOnTranslucent:1;

	UPROPERTY(config, EditAnywhere, Category = VR, meta = (
		EditCondition = "bSupportHardwareVariableRateShading",
		ConsoleVariable = "xr.VRS.FoveationLevel", DisplayName = "Stereo Foveation Level (Experimental)",
		ToolTip = "Set the level of foveation to apply when generating the Variable Rate Shading attachment. This feature is currently experimental.\nThis can yield some fairly significant performance benefits on GPUs that support Tier 2 VRS.\nLower settings will result in almost no discernible artifacting on most HMDs; higher settings will show some artifacts towards the edges of the view."))
	TEnumAsByte<EFixedFoveationLevels::Type> FoveationLevel;

	UPROPERTY(config, EditAnywhere, Category = VR, meta = (
		EditCondition = "bSupportHardwareVariableRateShading && FoveationLevel != EFixedFoveationLevels::Disabled",
		ConsoleVariable = "xr.VRS.DynamicFoveation", DisplayName = "Dynamic Foveation (Experimental)",
		ToolTip = "Allows foveation level to adjust dynamically based on GPU utilization.\nLevel will range between none at the minimum, and the currently selected foveation level at the maximum."))
	uint32 bDynamicFoveation:1;

	UPROPERTY(config, EditAnywhere, Category=Postprocessing, meta=(
		ConsoleVariable="r.CustomDepth",DisplayName="Custom Depth-Stencil Pass",
		ToolTip="Whether the custom depth pass for tagging primitives for postprocessing passes is enabled. Enabling it on demand can save memory but may cause a hitch the first time the feature is used."))
	TEnumAsByte<ECustomDepthStencil::Type> CustomDepthStencil;

	UPROPERTY(config, EditAnywhere, Category = Postprocessing, meta = (
		ConsoleVariable = "r.CustomDepthTemporalAAJitter", DisplayName = "Custom Depth with TemporalAA Jitter",
		ToolTip = "Whether the custom depth pass has the TemporalAA jitter enabled. Disabling this can be useful when the result of the CustomDepth Pass is used after TAA (e.g. after Tonemapping)"))
	uint32 bCustomDepthTaaJitter : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.PostProcessing.PropagateAlpha", DisplayName = "Alpha Output",
		ToolTip = "Enable r.PostProcessing.PropagateAlpha to enforce alpha in scene color (overriding r.SceneColorFormat if necessary) and propagate it through the renderer's post-processing chain. The legacy \"Linear color space only\" and \"Allow through tonemapper\" options now map to True and the engine keeps full alpha precision throughout post-processing. This feature can now be toggled without an engine restart."))
	uint32 bEnableAlphaChannelInPostProcessing : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		EditCondition = "bEnableAlphaChannelInPostProcessing",
		ConsoleVariable = "r.Deferred.SupportPrimitiveAlphaHoldout", DisplayName = "Support Primitive Alpha Holdout (Deferred)",
		ToolTip = "Enable primitive alpha holdout support in multiple deferred renderer passes. If primitive holdout masks are not to be used, keep this setting disabled for increased performance. Requires \"Alpha Output\" to be enabled.",
		ConfigRestartRequired = true))
	uint32 bDeferredSupportPrimitiveAlphaHoldout : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.Bloom", DisplayName = "Bloom",
		ToolTip = "Whether the default for Bloom is enabled or not (postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureBloom : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AmbientOcclusion", DisplayName = "Ambient Occlusion",
		ToolTip = "Whether the default for AmbientOcclusion is enabled or not (postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureAmbientOcclusion : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AmbientOcclusionStaticFraction", DisplayName = "Ambient Occlusion Static Fraction (AO for baked lighting)",
		ToolTip = "Whether the default for AmbientOcclusionStaticFraction is enabled or not (only useful for baked lighting and if AO is on, allows to have SSAO affect baked lighting as well, costs performance, postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureAmbientOcclusionStaticFraction : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AutoExposure", DisplayName = "Auto Exposure",
		ToolTip = "Whether the default for AutoExposure is enabled or not (postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureAutoExposure : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AutoExposure.Method", DisplayName = "Auto Exposure",
		ToolTip = "The default method for AutoExposure(postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	TEnumAsByte<EAutoExposureMethodUI::Type> DefaultFeatureAutoExposure;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AutoExposure.Bias", DisplayName = "Auto Exposure Bias",
		ToolTip = "Default Value for auto exposure bias."))
	float DefaultFeatureAutoExposureBias;

	UE_DEPRECATED(4.26, "Extend Default Luminance Range is deprecated, and is forced to ON at all times in new projects.")
	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange", DisplayName = "Extend default luminance range in Auto Exposure settings",
		ToolTip = "Whether the default values for AutoExposure should support an extended range of scene luminance. Also changes the exposure settings to be expressed in EV100. Having this setting disabled is deprecated and can only be done manually using r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange.",
		ConfigRestartRequired=true))
	uint32 bExtendDefaultLuminanceRangeInAutoExposureSettings : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.LocalExposure.HighlightContrastScale", DisplayName = "Local Exposure Highlight Contrast",
		ToolTip = "Default Value for Local Exposure Highlight Contrast.", ClampMin = "0.0", ClampMax = "1.0"))
	float DefaultFeatureLocalExposureHighlightContrast;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.LocalExposure.ShadowContrastScale", DisplayName = "Local Exposure Shadow Contrast",
		ToolTip = "Default Value for Local Exposure Shadow Contrast.", ClampMin = "0.0", ClampMax = "1.0"))
	float DefaultFeatureLocalExposureShadowContrast;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.MotionBlur", DisplayName = "Motion Blur",
		ToolTip = "Whether the default for MotionBlur is enabled or not (postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureMotionBlur : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.LensFlare", DisplayName = "Lens Flares (Image based)",
		ToolTip = "Whether the default for LensFlare is enabled or not (postprocess volume/camera/game setting can still override and enable or disable it independently)"))
	uint32 bDefaultFeatureLensFlare : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		EditCondition = "DefaultFeatureAntiAliasing == EAntiAliasingMethod::AAM_TemporalAA",
		ConsoleVariable = "r.TemporalAA.Upsampling", DisplayName = "Temporal Upsampling",
		ToolTip = "Whether to do primary screen percentage upscale with Temporal AA pass or not."))
	uint32 bTemporalUpsampling : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.AntiAliasingMethod", DisplayName = "Anti-Aliasing Method",
		ToolTip = "Selects the anti-aliasing method to use.",
		EditCondition = "!bForwardShading"))
	TEnumAsByte<EAntiAliasingMethod> DefaultFeatureAntiAliasing;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.MSAACount", DisplayName = "MSAA Sample Count",
		ToolTip = "Default number of samples for MSAA."))
	TEnumAsByte<ECompositingSampleCount::Type> MSAASampleCount;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultFeature.LightUnits", DisplayName = "Light Units",
		ToolTip = "Which units to use for newly placed point, spot and rect lights"))
	ELightUnits DefaultLightUnits;

	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.DefaultBackBufferPixelFormat",DisplayName = "Frame Buffer Pixel Format",
		ToolTip = "Pixel format used for back buffer, when not specified. This setting is currently incompatible with variable DisplayGamma and HDR mode.",
		ConfigRestartRequired=true))
	TEnumAsByte<EDefaultBackBufferPixelFormat::Type> DefaultBackBufferPixelFormat;
	
	UPROPERTY(config, EditAnywhere, Category = DefaultScreenPercentage, meta = (
		ConsoleVariable = "r.ScreenPercentage.Default", DisplayName = "Manual Screen Percentage",
		ToolTip = ""))
	float DefaultManualScreenPercentage;

	UPROPERTY(config, EditAnywhere, Category = DefaultScreenPercentage, meta = (
		ConsoleVariable = "r.ScreenPercentage.Default.Desktop.Mode", DisplayName = "Screen Percentage Mode for Desktop renderer",
		ToolTip = ""))
	EScreenPercentageMode DefaultScreenPercentageDesktopMode;

	UPROPERTY(config, EditAnywhere, Category = DefaultScreenPercentage, meta = (
		ConsoleVariable = "r.ScreenPercentage.Default.Mobile.Mode", DisplayName = "Screen Percentage Mode for Mobile renderer",
		ToolTip = ""))
	EScreenPercentageMode DefaultScreenPercentageMobileMode;

	UPROPERTY(config, EditAnywhere, Category = DefaultScreenPercentage, meta = (
		ConsoleVariable = "r.ScreenPercentage.Default.VR.Mode", DisplayName = "Screen Percentage Mode for VR",
		ToolTip = ""))
	EScreenPercentageMode DefaultScreenPercentageVRMode;

	UPROPERTY(config, EditAnywhere, Category = DefaultScreenPercentage, meta = (
		ConsoleVariable = "r.ScreenPercentage.Default.PathTracer.Mode", DisplayName = "Screen Percentage Mode for PathTracer",
		ToolTip = ""))
	EScreenPercentageMode DefaultScreenPercentagePathTracerMode;

	UPROPERTY(config, EditAnywhere, Category=Optimizations, meta=(
		ConsoleVariable="r.Shadow.UnbuiltPreviewInGame",DisplayName="Render Unbuilt Preview Shadows in game",
		ToolTip="Whether to render unbuilt preview shadows in game.  When enabled and lighting is not built, expensive preview shadows will be rendered in game.  When disabled, lighting in game and editor won't match which can appear to be a bug."))
	uint32 bRenderUnbuiltPreviewShadowsInGame:1;

	UPROPERTY(config, EditAnywhere, Category=Optimizations, meta=(
		ConsoleVariable="r.StencilForLODDither",DisplayName="Use Stencil for LOD Dither Fading",
		ToolTip="Whether to use stencil for LOD dither fading.  This saves GPU time in the base pass for materials with dither fading enabled, but forces a full prepass. Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bStencilForLODDither:1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable="r.EarlyZPass",DisplayName="Early Z-pass",
		ToolTip="Whether to use a depth only pass to initialize Z culling for the base pass."))
	TEnumAsByte<EEarlyZPass::Type> EarlyZPass;

	UE_DEPRECATED(5.8, "This option will be removed, and its behavior always enabled in a future release.")
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		EditCondition = "EarlyZPass == EEarlyZPass::OpaqueAndMasked",
		ConsoleVariable = "r.EarlyZPassOnlyMaterialMasking", DisplayName = "Mask material only in early Z-pass",
		ToolTip = "Whether to compute materials' mask opacity only in early Z pass. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bEarlyZPassOnlyMaterialMasking : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Shadow.CSMCaching", DisplayName = "Enable CSM Caching",
		ToolTip = "Enable caching CSM to reduce draw calls for casting CSM and probably improve performance."))
		uint32 bEnableCSMCaching : 1;

	UPROPERTY(config, EditAnywhere, Category=MiscLighting, meta=(
		ConsoleVariable="r.DBuffer",DisplayName="DBuffer Decals",
		ToolTip="Whether to accumulate decal properties to a buffer before the base pass.  DBuffer decals correctly affect lightmap and sky lighting, unlike regular deferred decals.  DBuffer enabled forces a full prepass.  Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bDBuffer:1;

	UPROPERTY(config, EditAnywhere, Category=Optimizations, meta=(
		ConsoleVariable="r.ClearSceneMethod",DisplayName="Clear Scene",
		ToolTip="Select how the g-buffer is cleared in game mode (only affects deferred shading)."))
	TEnumAsByte<EClearSceneOptions::Type> ClearSceneMethod;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.VelocityOutputPass", DisplayName = "Velocity Pass",
		ToolTip = "When to write velocity buffer. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	TEnumAsByte<EVelocityOutputPass::Type> VelocityPass;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Water.SingleLayer.DepthPrepass",
		DisplayName = "Single Layer Water Depth Prepass",
		ToolTip = "Enable a depth prepass for single layer water. Required for prepass velocity output and Virtual Shadow Maps. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bWaterSingleLayerDepthPrepass : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Water.SingleLayer.VelocityOutputPass",
		DisplayName = "Single Layer Water Velocity Pass",
		ToolTip = "When to write velocity for Single Layer Water. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	TEnumAsByte<ESingleLayerWaterVelocityOutputPass::Type> SingleLayerWaterVelocityOutputPass;

	UPROPERTY(config, EditAnywhere, Category=Optimizations, meta=(
		DisplayName="Output velocities due to vertex deformation",
		ConsoleVariable="r.Velocity.EnableVertexDeformation",
		ToolTip="Enables materials with World Position Offset and/or World Displacement to output velocities during the velocity pass even when the actor has not moved. \nIf the VelocityPass is set to 'Write after base pass' this can incur a performance cost due to additional draw calls. \nThat performance cost is higher if many objects are using World Position Offset. A forest of trees for example." ))
	TEnumAsByte<EVertexDeformationOutputsVelocity::Type> VertexDeformationOutputsVelocity;

	UPROPERTY(config, EditAnywhere, Category=Optimizations, meta=(
		ConsoleVariable="r.SelectiveBasePassOutputs", DisplayName="Selectively output to the GBuffer rendertargets",
		ToolTip="Enables not exporting to the GBuffer rendertargets that are not relevant. Changing this setting requires restarting the editor.",
		ConfigRestartRequired=true))
	uint32 bSelectiveBasePassOutputs:1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		DisplayName = "Enable Particle Cutouts by default",
		ToolTip = "When enabled, after changing the material on a Required particle module a Particle Cutout texture will be chosen automatically from the Opacity Mask texture if it exists, if not the Opacity Texture will be used if it exists."))
	uint32 bDefaultParticleCutouts : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "fx.GPUSimulationTextureSizeX",
		DisplayName = "GPU Particle simulation texture size - X",
		ToolTip = "The X size of the GPU simulation texture size. SizeX*SizeY determines the maximum number of GPU simulated particles in an emitter. Potentially overridden by CVar settings in BaseDeviceProfile.ini.",
		ConfigRestartRequired = true))
	int32 GPUSimulationTextureSizeX;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "fx.GPUSimulationTextureSizeY",
		DisplayName = "GPU Particle simulation texture size - Y",
		ToolTip = "The Y size of the GPU simulation texture size. SizeX*SizeY determines the maximum number of GPU simulated particles in an emitter. Potentially overridden by CVar settings in BaseDeviceProfile.ini.",
		ConfigRestartRequired = true))
	int32 GPUSimulationTextureSizeY;

	UPROPERTY(config, EditAnywhere, Category = Reflections, meta = (
		ConsoleVariable = "r.AllowGlobalClipPlane", DisplayName = "Support global clip plane for Planar Reflections",
		ToolTip = "Whether to support the global clip plane needed for planar reflections.  Enabling this increases BasePass triangle cost by ~15% regardless of whether planar reflections are active. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bGlobalClipPlane : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.GBufferFormat", DisplayName = "GBuffer Format",
		ToolTip = "Selects which GBuffer format should be used. Affects performance primarily via how much GPU memory bandwidth used. This also controls Substrate normal quality and, in this case, a restart is required.",
		ConfigRestartRequired = true))
	TEnumAsByte<EGBufferFormat::Type> GBufferFormat;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.MorphTarget.Mode", DisplayName = "Use GPU for computing morph targets",
		ToolTip = "Whether to use original CPU method (loop per morph then by vertex) or use a GPU-based method on Shader Model 5 hardware."))
	uint32 bUseGPUMorphTargets : 1;

	UPROPERTY(config, EditAnywhere, Category = DefaultSettings, meta = (
		ConsoleVariable = "r.MorphTarget.MaxBlendWeight", DisplayName = "Maximum absolute value accepted as a morph target blend weight, positive or negative.",
		ToolTip = "Blend target weights will be checked against this value for validation. Absolue values greather than this number will be clamped to [-MorphTargetMaxBlendWeight, MorphTargetMaxBlendWeight]."))
	float MorphTargetMaxBlendWeight;

	/**
	"The sky atmosphere component requires extra samplers/textures to be bound to apply aerial perspective on transparent surfaces (and all surfaces on mobile via per vertex evaluation)."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportSkyAtmosphere", DisplayName = "Support Sky Atmosphere",
		ToolTip = "The sky atmosphere component requires extra samplers/textures to be bound to apply aerial perspective on transparent surfaces (and all surfaces on mobile via per vertex evaluation).",
		ConfigRestartRequired = true))
		uint32 bSupportSkyAtmosphere : 1;

	/**
	"The sky atmosphere component can light up the height fog but it requires extra samplers/textures to be bound to apply aerial perspective on transparent surfaces (and all surfaces on mobile via per vertex evaluation)."
	"It requires r.SupportSkyAtmosphere to be true."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportSkyAtmosphereAffectsHeightFog", DisplayName = "Support Sky Atmosphere Affecting Height Fog",
		ToolTip = "The sky atmosphere component can light up the height fog but it requires extra samplers/textures to be bound to apply aerial perspective on transparent surfaces (and all surfaces on mobile via per vertex evaluation). It requires r.SupportSkyAtmosphere to be true.",
		ConfigRestartRequired = true))
		uint32 bSupportSkyAtmosphereAffectsHeightFog : 1;

	/**
	"When enabled, the height fog scattering lob will match the volumetric fog phase function."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportExpFogMatchesVolumetricFog", DisplayName = "Support Exponential Fog Matches Volumetric Fog",
		ToolTip = "When enabled, the height fog scattering/ambient/emissive/phase will match the volumetric fog better.",
		ConfigRestartRequired = true))
		uint32 bSupportExpFogMatchesVolumetricFog : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportLocalFogVolumes", DisplayName = "Support Local Fog Volumes",
		ToolTip = "Enable local fog volume components rendering support. Disable that setting if local fog volumes are not used in a project for increased performance.",
		ConfigRestartRequired = true))
		uint32 bSupportLocalFogVolumes : 1;

	/**
	"Enable cloud shadow on translucent surface. This is evaluated per vertex to reduce GPU cost. The cloud system requires extra samplers/textures to be bound to vertex shaders."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportCloudShadowOnForwardLitTranslucent", DisplayName = "Support Cloud Shadow On Forward Lit Translucent",
		ToolTip = "Enable cloud shadow on translucent surface not relying on the translucenct lighting volume, e.g. using Forward lighting. This is evaluated per vertex to reduce GPU cost and requires extra samplers/textures to be bound to vertex shaders. This is not implemented on mobile as VolumetricClouds are not available on these platforms.",
		ConfigRestartRequired = true))
		uint32 bSupportCloudShadowOnForwardLitTranslucent : 1;
		
	/**
	"When enabled, the height fog scattering lob will match the volumetric fog phase function."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.ClusteredDeferredShading.EnableForProject", DisplayName = "Support Clustered Deferred Shading",
		ToolTip = "Whether to support Clustered Deferred Shading for the project. When enabled, more shaders are compiled and the feature can be toggled on/off at runtime using r.UseClusteredDeferredShading.",
		ConfigRestartRequired = true))
		uint32 bClusteredDeferredShadingenableForProject : 1;

	/**
	"Select the format of the light function atlas texture."
	*/
	UPROPERTY(config, EditAnywhere, Category = LightFunctionAtlas, meta = (
		ConsoleVariable = "r.LightFunctionAtlas.Format", DisplayName = "Light Function Atlas Format",
		ToolTip = "Select the format of the light function atlas texture.",
		ConfigRestartRequired = true))
		TEnumAsByte<ELightFunctionAtlasPixelFormat::Type> LightFunctionAtlasPixelFormat;

	/**
	"Enable support for light function on volumetric fog, when the light function atlas is enabled."
	*/
	UPROPERTY(config, EditAnywhere, Category = LightFunctionAtlas, meta = (
		ConsoleVariable = "r.VolumetricFog.LightFunction", DisplayName = "Volumetric Fog Uses Light Function Atlas.",
		ToolTip = "Enable support for light function on volumetric fog, when the light function atlas is enabled."))
		uint32 bVolumetricFogUsesLightFunctionAtlas : 1;

	/**
	"Enable support for light function on deferred lighting (multi-pass and clustered), when the light function atlas is enabled."
	*/
	UPROPERTY(config, EditAnywhere, Category = LightFunctionAtlas, meta = (
		ConsoleVariable = "r.Deferred.UsesLightFunctionAtlas", DisplayName = "Deferred Lighting Uses Light Function Atlas.",
		ToolTip = "Enable support for light function on deferred lighting (multi-pass and clustered), when the light function atlas is enabled."))
		uint32 bDeferredLightingUsesLightFunctionAtlas : 1;

	/**
	"Enable support for light function on Single Layer Water when the light function atlas is enabled."
	*/
	UPROPERTY(config, EditAnywhere, Category = LightFunctionAtlas, meta = (
		ConsoleVariable = "r.SingleLayerWater.UsesLightFunctionAtlas", DisplayName = "Single Layer Water Uses Light Function Atlas.",
		ToolTip = "Enable support for light function on Single Layer Water when the light function atlas is enabled.",
		ConfigRestartRequired = true))
		uint32 bSingleLayerWaterUsesLightFunctionAtlas : 1;

	/**
	"Enable support for light function on Translucent material using Forward Shading mode, when the light function atlas is enabled."
	*/
	UPROPERTY(config, EditAnywhere, Category = LightFunctionAtlas, meta = (
		ConsoleVariable = "r.Translucent.UsesLightFunctionAtlas", DisplayName = "Translucent Uses Light Function Atlas.",
		ToolTip = "Enable support for light function on Translucent material using Forward Shading mode, when the light function atlas is enabled.",
		ConfigRestartRequired = true))
		uint32 bTranslucentUsesLightFunctionAtlas : 1;

	/**
		"Enable IES profile evaluation on translucent materials when using the Forward Shading mode."
		*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Translucent.UsesIESProfiles", DisplayName = "Support IES profiles On Translucent Materials (When Using ForwardShading)",
		ToolTip = "Enable IES profile evaluation on translucent materials when using the Forward Shading mode.",
		ConfigRestartRequired = true))
	uint32 bSupportIESProfileOnTranslucent : 1;

	/**
	"Enable rect light evaluation on translucent materials when using the Forward Shading mode."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Translucent.UsesRectLights", DisplayName = "Support Rect Light On Translucent Materials (When Using ForwardShading)",
		ToolTip = "Enable rect light evaluation on translucent materials when using the Forward Shading mode.",
		ConfigRestartRequired = true))
		uint32 bSupportRectLightOnTranslucent : 1;

	/**
	"Enable local light shadow evaluation on translucent materials when using the Forward Shading mode."
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.Translucent.UsesShadowedLocalLights", DisplayName = "Support Local Light Shadow On Translucent Materials (When Using ForwardShading)",
		ToolTip = "Enable local light shadow evaluation on translucent materials when using the Forward Shading mode.",
		ConfigRestartRequired = true))
		uint32 bSupportShadowedLocalLightOnTranslucent : 1;

	UPROPERTY(config, EditAnywhere, Category = Debugging, meta = (
		ConsoleVariable = "r.GPUCrashDebugging", DisplayName = "Enable vendor specific and in-engine GPU crash analysis tools.",
		ToolTip = "Enables vendor specific and in-engine GPU crash analysis tools.",
		ConfigRestartRequired = true))
		uint32 bGpuCrashDebugging : 1;

	UPROPERTY(config, EditAnywhere, Category = VR, meta = (
		ConsoleVariable = "vr.InstancedStereo", DisplayName = "Instanced Stereo",
		ToolTip = "Enable single-pass stereoscopic rendering through view instancing or draw call instancing.",
		ConfigRestartRequired = true))
	uint32 bMultiView : 1;

	UPROPERTY(config, EditAnywhere, Category = VR, meta=(
		EditCondition = "MobileShadingPath == 0",
		ConsoleVariable="r.MobileHDR", DisplayName="Mobile HDR",
		ToolTip="If true, mobile pipelines include a full post-processing pass with tonemapping. Disable this setting for a performance boost and to enable stereoscopic rendering optimizations. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bMobilePostProcessing:1;

	UPROPERTY(config, EditAnywhere, Category = VR, meta = (
		ConsoleVariable = "vr.MobileMultiView", DisplayName = "Mobile Multi-View",
		ToolTip = "Enable single-pass stereoscopic rendering on mobile platforms.",
		ConfigRestartRequired = true))
		uint32 bMobileMultiView : 1;
	
	UPROPERTY(config, meta = (
		EditCondition = "!bMobilePostProcessing",
		ConsoleVariable = "r.Mobile.UseHWsRGBEncoding", DisplayName = "Single-pass linear rendering",
		ToolTip = "If true then mobile single-pass (without post-processing) rendering will use HW accelerated sRGB encoding/decoding. Available only on Oculus for now."))
		uint32 bMobileUseHWsRGBEncoding : 1;

	UPROPERTY(config, EditAnywhere, Category = VR, meta = (
		ConsoleVariable = "vr.RoundRobinOcclusion", DisplayName = "Round Robin Occlusion Queries",
		ToolTip = "Enable round-robin scheduling of occlusion queries for VR.",
		ConfigRestartRequired = false))
	uint32 bRoundRobinOcclusion : 1;

	UPROPERTY(config, EditAnywhere, Category = "Mesh Streaming", meta = (
		ConsoleVariable="r.MeshStreaming",DisplayName="Mesh Streaming",
		ToolTip="When enabled mesh LODs will stream in based on what is visible on screen.",
		ConfigRestartRequired = true))
		uint32 bMeshStreaming : 1;

	UPROPERTY(config, EditAnywhere, Category = "Heterogeneous Volumes", meta = (
		ConsoleVariable = "r.HeterogeneousVolumes", DisplayName = "Heterogeneous Volumes (Beta)",
		ToolTip = "Enable rendering with the heterogeneous volumes subsystem.",
		ConfigRestartRequired = false))
	uint32 bEnableHeterogeneousVolumes : 1;

	UPROPERTY(config, EditAnywhere, Category = "Heterogeneous Volumes", meta = (
		ConsoleVariable = "r.HeterogeneousVolumes.Shadows", DisplayName = "Shadow Casting",
		ToolTip = "Enable heterogeneous volumes to cast shadows onto the environment.",
		ConfigRestartRequired = false))
		uint32 bShouldHeterogeneousVolumesCastShadows : 1;

	UPROPERTY(config, EditAnywhere, Category = "Heterogeneous Volumes", meta = (
		ConsoleVariable = "r.Translucency.HeterogeneousVolumes", DisplayName = "Composite with Translucency",
		ToolTip = "Enable compositing with heterogeneous volumes when rendering translucency.",
		ConfigRestartRequired = true))
	uint32 bCompositeHeterogeneousVolumesWithTranslucency : 1;

	UPROPERTY(config, EditAnywhere, Category=Editor, meta=(
		ConsoleVariable="r.WireframeCullThreshold",DisplayName="Wireframe Cull Threshold",
		ToolTip="Screen radius at which wireframe objects are culled. Larger values can improve performance when viewing a scene in wireframe."))
	float WireframeCullThreshold;

	/** 
	 * Enables support for stationary skylight. 
	 * Disabling this option can simplify base pass shaders to increase GPU performance. 
	 * It's recommended to disable if your project does not require Stationary skylights.
	 */
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.SupportStationarySkylight", DisplayName = "Support Stationary Skylight",
		ConfigRestartRequired = true))
		uint32 bSupportStationarySkylight : 1;

	/**
	"Low quality lightmap requires permutations of the lightmap rendering shaders.  Disabling will reduce the number of shader permutations required per material. Note that the mobile renderer requires low quality lightmaps, so disabling this setting is not recommended for mobile titles using static lighting. Changing this setting requires restarting the editor."
	*/
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.SupportLowQualityLightmaps", DisplayName = "Support low quality lightmap shader permutations",
		ConfigRestartRequired = true))
		uint32 bSupportLowQualityLightmaps : 1;

	/**
	PointLight WholeSceneShadows requires many vertex and geometry shader permutations for cubemap rendering. Disabling will reduce the number of shader permutations required per material. Changing this setting requires restarting the editor."
	*/
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.SupportPointLightWholeSceneShadows", DisplayName = "Support PointLight WholeSceneShadows",
		ConfigRestartRequired = true))
		uint32 bSupportPointLightWholeSceneShadows : 1;

	/**
	""Enable translucent volumetric self-shadow, requires vertex and pixel shader permutations for all tranlucent materials even if not used by any light."
	*/
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.Shadow.TranslucentPerObject.ProjectEnabled", DisplayName = "Support Volumetric Translucent Self Shadowing",
		ToolTip = "Enable translucent volumetric self-shadow, requires vertex and pixel shader permutations for all tranlucent materials even if not used by any light.",
		ConfigRestartRequired = true))
		uint32 bSupportTranslucentPerObjectShadow : 1;

	/**
	"Enable cloud shadow on SingleLayerWater surface. This is evaluated per vertex to reduce GPU cost. The cloud system requires extra samplers/textures to be bound to vertex shaders."
	*/
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.Water.SingleLayerWater.SupportCloudShadow", DisplayName = "Support Cloud Shadow On SingleLayerWater",
		ToolTip = "Enable cloud shadow on SingleLayerWater. This is evaluated per vertex to reduce GPU cost and requires extra samplers/textures to be bound to vertex shaders. This is not implemented on mobile as VolumetricClouds are not available on these platforms.",
		ConfigRestartRequired = true))
		uint32 bSupportCloudShadowOnSingleLayerWater : 1;

	/**
	"Enable Substrate materials."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate", DisplayName = "Substrate materials",
		ToolTip = "Enable Substrate materials.",
		ConfigRestartRequired = true))
		uint32 bEnableSubstrate : 1;

	/**
	"Enable Substrate storage format."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.ProjectGBufferFormat",
		DisplayName = "Substrate GBuffer Format (Project)",
		ToolTip = "Select the GBuffer format used for this project.\n - Blendable GBuffer is supported on all platforms (=> 1 closure and 20 bytes per pixel maximum). \n - Adaptive GBuffer is supported on certains platforms. Unsupported platforms will fallback onto 'Blendable GBuffer' format.\nChanging this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	TEnumAsByte<ESubstrateStorageFormat::Type> SubstrateGBufferFormat;

	/**
	"Substrate maximum number of closure evaluated per pixel for this project."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.ProjectClosuresPerPixel", 
		DisplayName = "Substrate Closure Per Pixel (Project)",
		ToolTip = "Define the maximum number of closures evaluated per pixel for this project. Materials with a number of closures beyond this limit will be simplified until it fits the requirements. This only applies when Adaptive GBuffer is enabled. On platforms not supporting Adaptive GBuffer, the material will be simplified into a single closure",
		ClampMin = "1", ClampMax = "8", 
		EditCondition = "SubstrateGBufferFormat == 1",
		ConfigRestartRequired = true))
	uint32 SubstrateProjectClosuresPerPixel;

	/**
	"Substrate maximum number of closure evaluated per pixel override behavior."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.ProjectClosuresPerPixelOverride",
		DisplayName = "Substrate Closure-Per-Pixel Override",
		ToolTip = "Select how the maximum number of closures evaluated per pixel is defined for each platform.\n - Use Platform Closure Count: Use the minimum between the platform's closure count and the project's closure count.\n - Force Closure Count: Force the project's closure count on platforms supporting Adaptive GBuffer.",
		EditCondition = "SubstrateGBufferFormat == 1",
		ConfigRestartRequired = true))
	TEnumAsByte<ESubstrateClosureConfig::Type> SubstrateClosurePerPixelOverride;

	/**
	"Substrate maximum number of bytes allocated per pixels for storing GBuffer information for this project with Adaptive GBuffer."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.BytesPerPixel", 
		DisplayName = "Substrate Bytes Per Pixel",
		ToolTip = "Define the maximum number of bytes per pixel allocated. Materials requiring memory beyond this limit will be simplified until it fits the requirements. This only applies when Adaptive GBuffer is enabled. On platforms not supporting Adaptive GBuffer, the material will be simplified to a smaller fix limit.",
		ClampMin = "20", ClampMax = "256", 
		EditCondition = "SubstrateGBufferFormat == 1",
		ConfigRestartRequired = true))
	uint32 SubstrateProjectBytesPerPixel;

	/**
	"Enable Substrate opaque material rough refractions effect from top layers over layers below."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.OpaqueMaterialRoughRefraction", DisplayName = "Substrate opaque material rough refraction (Experimental)",
		ToolTip = "Enable Substrate opaque material rough refractions effect from top layers over layers below. (Experimental for now since this is not a path validated in production). Requires adaptive GBuffer format and at least 2 closures per pixel.",
		EditCondition = "SubstrateGBufferFormat == 1 && SubstrateProjectClosuresPerPixel > 1",
		ConfigRestartRequired = true))
		uint32 SubstrateOpaqueMaterialRoughRefraction : 1;
		
	/**
	"Enable Substrate translucent material rough refractions effect over background. Editor restart is not required."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Refraction.Blur", DisplayName = "Substrate translucent material rough refraction",
		ToolTip = "Enable Substrate translucent material rough refractions effect over background.",
		ConfigRestartRequired = false))
		uint32 SubstrateTranslucentMaterialRoughRefraction : 1;

	/**
	"Enable advanced Substrate material debug visualization shaders. Base pas shaders can output such advanced data."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.Debug.AdvancedVisualizationShaders", DisplayName = "Substrate advanced visualization shaders (Editor/Win64/DX12 Only)",
		ToolTip = "Enable advanced Substrate material debug visualization shaders. Base pass shaders can output such advanced data. Only available for Editors built for Win64 and running with the DX12 graphic API for now.",
		ConfigRestartRequired = true))
		uint32 SubstrateDebugAdvancedVisualizationShaders : 1;

	/**
	"Enable substrate layer support."
	*/
	UPROPERTY(config, EditAnywhere, Category = Substrate, meta = (
		ConsoleVariable = "r.Substrate.EnableLayerSupport", DisplayName = "Substrate enable material layer support (Experimental)",
		ToolTip = "Enable Substrate material layering and UI(Experimental).Note: This support is one way; the legacy layer materials auto-upgrade and can only be reversed manually after assets are re - saved.",
		ConfigRestartRequired = true,
		EditCondition = "bEnableSubstrate"))
		uint32 SubstrateEnableLayerSupport : 1;

	/**
	"Enable rough diffuse material."
	*/
	UPROPERTY(config, EditAnywhere, Category = Materials, meta = (
		ConsoleVariable = "r.Material.RoughDiffuse", DisplayName = "Enable Rough Diffuse Material",
		ToolTip = "Enable Rough Diffuse Material. Please note that when Substrate is enabled, energy conservation is forced to ENABLED.",
		ConfigRestartRequired = true))
		uint32 bMaterialRoughDiffuse : 1; 

	/**
	"Enable rough diffuse material."
	*/
	UPROPERTY(config, EditAnywhere, Category = Materials, meta = (
		ConsoleVariable = "r.Material.EnergyConservation", DisplayName = "Enable Energy Conservation on Material",
		ToolTip = "Enable Energy Conservation on Material. Please note that when Substrate is enabled, energy conservation is forced to ENABLED.",
		ConfigRestartRequired = true))
		uint32 bMaterialEnergyConservation : 1;

	/**
	"Automatically set Material usage flags in editor default."
	*/
	UPROPERTY(config, EditAnywhere, Category = Materials, meta = (
		ConsoleVariable = "r.Material.DefaultAutoMaterialUsage", DisplayName = "Automatically set Material usage flags in editor default",
		ToolTip = "Whether new Materials should automatically set usage flags in the Editor.",
		ConfigRestartRequired = false))
	uint32 bAutomaticallySetMaterialUsageInEditorDefault : 1;

	/**
	"Enable Order Independent Transparency (Experimental)."
	*/
	UPROPERTY(config, EditAnywhere, Category = Translucency, meta = (
		ConsoleVariable = "r.OIT.SortedPixels", DisplayName = "Enable Order Independent Transparency (Experimental)",
		ToolTip = "Enable support for Order-Independent-Transparency on translucent surfaces, which remove most of the sorting artifact among translucent surfaces.",
		ConfigRestartRequired = true))
		uint32 bOrderedIndependentTransparencyEnable : 1;

	/**
	"Enable hair strands Auto LOD mode by default."
	*/
	UPROPERTY(config, EditAnywhere, Category = HairStrands, meta = (
		ConsoleVariable = "r.HairStrands.LODMode", DisplayName = "Enable Hair Strands 'Auto' LOD mode",
		ToolTip = "Enable hair strands Auto LOD mode by default. Otherwise use Manual LOD mode. Auto LOD mode adapts hair curves based on screen coverage. Manual LOD mode relies on LODs manually setup per groom asset. This global behavior can be overridden per groom asset",
		ConfigRestartRequired = true))
	uint32 bUseHairStrandsAutoLODMode  : 1;

	/**
	"Support hardware variable rate shading.
	*/
	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.VRS.Support", DisplayName = "Support Hardware Variable Rate Shading",
		ToolTip = "Allows selectively shading certain portions of the image at lower rates, using one pixel shader invocation to shade multiple pixels. Rates are selected per-material, or in screenspace by enabling a shading rate image generator (such as Contrast Adaptive Shading or Stereo Foveation). Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bSupportHardwareVariableRateShading : 1;

	UPROPERTY(config, EditAnywhere, Category = MobileShaderPermutationReduction, meta = (
		ConsoleVariable = "r.Mobile.EnableStaticAndCSMShadowReceivers", DisplayName = "Support Combined Static and CSM Shadowing",
		ToolTip = "Allow primitives to receive both static and CSM shadows from a stationary light. Disabling will free a mobile texture sampler and reduce shader permutations. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileEnableStaticAndCSMShadowReceivers : 1;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.EnableMovableLightCSMShaderCulling", DisplayName = "Support movable light CSM shader culling",
		ToolTip = "Primitives lit by a movable directional light will render with the CSM shader only when determined to be within CSM range. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileEnableMovableLightCSMShaderCulling : 1;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.Forward.EnableLocalLights",
		DisplayName = "Local Light Setting (When using ForwardShading)",
		ToolTip = "Select which Local Light Setting to use for Mobile. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		TEnumAsByte<EMobileLocalLightSetting> MobileLocalLightSetting;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.Forward.EnableClusteredReflections",
		DisplayName = "Clustered reflections (When using ForwardShading)",
		ToolTip = "Whether to enable clustered reflections on mobile forward (including translucency in deferred). Always supported for opaque geometry on mobile deferred. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileForwardEnableClusteredReflections : 1;

	UPROPERTY(config, EditAnywhere, Category = MobileShaderPermutationReduction, meta = (
		ConsoleVariable = "r.Mobile.AllowDistanceFieldShadows",
		DisplayName = "Support Pre-baked Distance Field Shadow Maps",
		ToolTip = "Generate shaders for static primitives render Lightmass-baked distance field shadow maps from stationary directional lights. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileAllowDistanceFieldShadows : 1;

	UPROPERTY(config, EditAnywhere, Category = MobileShaderPermutationReduction, meta = (
		ConsoleVariable = "r.Mobile.EnableMovableSpotlightsShadow",
		DisplayName = "Support Movable SpotlightShadows",
		ToolTip = "Generate shaders for primitives to receive shadow from movable spotlights. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileAllowMovableSpotlightShadows : 1;

	// Deprecated 5.8: All platforms now support 16-bit bone indices. 8-bit indices are still used automatically for sections with fewer than 256 bones.
	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.GPUSkin.Support16BitBoneIndex", DisplayName = "Support 16-bit Bone Index",
		ToolTip = "Deprecated. All platforms now support 16-bit bone indices. This setting is preserved for backward compatibility.",
		DeprecatedProperty, DeprecationMessage = "All platforms now support 16-bit bone indices.",
		ConfigRestartRequired = true))
		uint32 bSupport16BitBoneIndex : 1;

	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.GPUSkin.Limit2BoneInfluences", DisplayName = "Limit GPU skinning to 2 bones influence",
		ToolTip = "Whether to use 2 bone influences instead of the default of 4 for GPU skinning. This does not change skeletal mesh assets but reduces the number of instructions required by the GPU skin vertex shaders. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bGPUSkinLimit2BoneInfluences : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportDepthOnlyIndexBuffers", DisplayName = "Support depth-only index buffers",
		ToolTip = "Support depth-only index buffers, which provide a minor rendering speedup at the expense of using twice the index buffer memory.",
		ConfigRestartRequired = true))
		uint32 bSupportDepthOnlyIndexBuffers : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "r.SupportReversedIndexBuffers", DisplayName = "Support reversed index buffers",
		ToolTip = "Support reversed index buffers, which provide a minor rendering speedup at the expense of using twice the index buffer memory.",
		ConfigRestartRequired = true))
		uint32 bSupportReversedIndexBuffers : 1;

	UPROPERTY(config, EditAnywhere, Category = Optimizations, meta = (
		ConsoleVariable = "slate.MaterialShaderMode", DisplayName = "Slate Material Shader Mode",
		ToolTip = "Specify how Slate material shaders should be compiled to balance GPU performance and number of shaders. This can be overridden using per-platform configuration files.",
		ConfigRestartRequired = true))
		ESlateMaterialShaderMode SlateMaterialShaderMode;


	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.AmbientOcclusion", DisplayName = "Ambient Occlusion (When using ForwardShading)",
		ToolTip = "Mobile Ambient Occlusion. Causion: An extra sampler will be occupied in mobile base pass pixel shader after enable the mobile ambient occlusion. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bMobileAmbientOcclusion : 1;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.ScreenSpaceReflections", DisplayName = "Screen Space Reflections (When using Forward Shading)",
		ToolTip = "Support Screen Space Reflections with mobile rendering. Screen Space Reflections on mobile require TAA Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
	uint32 bMobileScreenSpaceReflections : 1;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.DBuffer", DisplayName = "Mobile DBuffer Decals",
		ToolTip = "Whether to accumulate decal properties to a buffer before the base pass with mobile rendering. DBuffer enabled forces a full prepass. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true))
		uint32 bMobileDBuffer : 1;

	UPROPERTY(config, EditAnywhere, Category = Mobile, meta = (
		ConsoleVariable = "r.Mobile.PropagateAlpha", DisplayName = "Mobile Alpha Output",
		ToolTip = "Enable r.Mobile.PropagateAlpha to enforce alpha in scene color (overriding r.SceneColorFormat if necessary) and propagate it through the mobile renderer's post-processing chain.",
		ConfigRestartRequired = true))
	uint32 bMobileEnableAlphaChannelInPostProcessing : 1;

	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.GPUSkin.UnlimitedBoneInfluences", DisplayName = "Use Unlimited Bone Influences",
		ToolTip = "If enabled, a new mesh imported will use unlimited bone buffer instead of fixed MaxBoneInfluences for rendering.",
		ConfigRestartRequired = true))
		uint32 bUseUnlimitedBoneInfluences : 1;
		
	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.GPUSkin.AlwaysUseDeformerForUnlimitedBoneInfluences",
		ToolTip = "Any mesh LODs using Unlimited Bone Influences will always be rendered with a Mesh Deformer. This reduces the number of shader permutations needed for skeletal mesh materials, saving memory at the cost of performance. Has no effect if either Unlimited Bone Influences or Deformer Graph is disabled.",
		ConfigRestartRequired = true))
		uint32 bAlwaysUseDeformerForUnlimitedBoneInfluences : 1;
		
	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.GPUSkin.UnlimitedBoneInfluencesThreshold", DisplayName = "Unlimited Bone Influences Threshold",
		ToolTip = "When Unlimited Bone Influence is enabled, it still uses a fixed bone inflence buffer until the max bone influence of a mesh exceeds this value"))
		int32 UnlimitedBonInfluencesThreshold;

	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ToolTip = "When BoneInfluenceLimit on a skeletal mesh LOD is set to 0, this setting is used instead. If this setting is 0, no limit will be applied here and the max bone influences will be determined by other project settings. Changing this setting requires restarting the editor.",
		ConfigRestartRequired = true,
		ClampMin = "0", UIMin = "0"))
		FPerPlatformInt DefaultBoneInfluenceLimit;

	// Deprecated 5.8: Sections are no longer split based on bone count. This setting is preserved only for DDC key backward compatibility.
	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		DisplayName = "Maximum bones per Sections",
		ToolTip = "Deprecated. Sections are no longer split based on bone count.",
		DeprecatedProperty, DeprecationMessage = "Sections are no longer split based on bone count.",
		ConfigRestartRequired = true,
		ClampMin = "12", UIMin = "12"))
		FPerPlatformInt MaxSkinBones;

	/**
	 * Skin cache allows a compute shader to skin once each vertex, save those results into a new buffer and reuse those calculations when later running the depth, base and velocity passes. 
	 * This also allows opting into the 'recompute tangents' for skinned mesh instance feature. 
	 * Selecting only one shader path will reduce the number of shader permutations required per material. Changing this setting requires restarting the editor.
	 */
	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.SkinCache.CompileShaders", DisplayName = "Skin Cache Shader Support",
		ToolTip = "Select the skinning shader types to build. Selecting only one shader type will reduce the number of shader permutations required per material. Skin cache shaders are required if Ray Tracing is enabled.",
		ConfigRestartRequired = true))
	ESkinningShaderSupportMode SkinningShaderSupport;

	UPROPERTY(config, EditAnywhere, Category = Skinning, meta = (
		ConsoleVariable = "r.SkinCache.DefaultBehavior", DisplayName = "Skin Cache Default Behavior",
		ToolTip = "Default behavior for whether skeletal meshes use the skin cache. If Support Ray Tracing is enabled on a mesh, the skin cache will be used for Ray Tracing updates on that mesh regardless of this setting."))
	ESkinCacheDefaultBehavior DefaultSkinCacheBehavior;

	UE_DEPRECATED(5.6, "MobilePlanarReflectionMode is deprecated. The screen space reflections on mobile is supported and the pixel projected reflection is removed.")
	UPROPERTY()
	uint8 MobilePlanarReflectionMode;

	UPROPERTY(config, EditAnywhere, Category = Nanite, meta = (
		ConsoleVariable = "r.Nanite.Foliage", DisplayName = "Nanite Foliage (Experimental)",
		ToolTip = "Enables experimental features required to support Nanite Foliage.",
		ConfigRestartRequired = true))
	uint32 bEnableNaniteFoliage : 1;

	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = Nanite, meta=(		
		DisplayName="Cook Nanite streamable pages as optional",
		ToolTip="When true, cooks Nanite streamable bulk data as optional."))
	FPerPlatformBool bNaniteStreamableBulkDataOptional;

	UPROPERTY(config, EditAnywhere, Category="Mesh Streaming|Skeletal Mesh", meta=(
		EditCondition = "bMeshStreaming",
		DisplayName="Stream LODs by default",
		ToolTip="Whether to stream skeletal mesh LODs by default."))
	FPerPlatformBool bStreamSkeletalMeshLODs;

	UPROPERTY(config, EditAnywhere, Category="Mesh Streaming|Skeletal Mesh", meta=(
		DisplayName="Discard optional LODs",
		ToolTip="Whether to discard skeletal mesh LODs below minimum LOD levels at cook time."))
	FPerPlatformBool bDiscardSkeletalMeshOptionalLODs;

	/**
	" Visualize calibration material settings for post process calibration materials, used for setting full-screen images used for monitor calibration."
	*/
	UPROPERTY(config, EditAnywhere, Category = PostProcessCalibrationMaterials, meta = (AllowedClasses = "/Script/Engine.Material",
		DisplayName = "Visualize Calibration Color Material Path",
		ToolTip = "When the VisualizeCalibrationColor show flag is enabled, this path will be used as the post-process material to render. The post-process material's Blendable Location property must be set to \"After Tonemapping\" for proper calibration display.",
		ConfigRestartRequired = false))
	FSoftObjectPath VisualizeCalibrationColorMaterialPath;

	UPROPERTY(config, EditAnywhere, Category = PostProcessCalibrationMaterials, meta = (AllowedClasses = "/Script/Engine.Material",
		DisplayName = "Visualize Calibration Custom Material Path",
		ToolTip = "When the VisualizeCalibrationCustom show flag is enabled, this path will be used as the post-process material to render. The post-process material's Blendable Location property must be set to \"After Tonemapping\" for proper calibration display.",
		ConfigRestartRequired = false))
	FSoftObjectPath VisualizeCalibrationCustomMaterialPath;

	UPROPERTY(config, EditAnywhere, Category = PostProcessCalibrationMaterials, meta = (AllowedClasses = "/Script/Engine.Material",
		DisplayName = "Visualize Calibration Grayscale Material Path",
		ToolTip = "When the VisualizeCalibrationGrayscale show flag is enabled, this path will be used as the post-process material to render. The post-process material's Blendable Location property must be set to \"After Tonemapping\" for proper calibration display.",
		ConfigRestartRequired = false))
		FSoftObjectPath VisualizeCalibrationGrayscaleMaterialPath;

	UPROPERTY(config, EditAnywhere, Category = PSOPrecache, meta = (
		ConsoleVariable = "r.PSOPrecache.ProxyCreationStrategy", DisplayName = "PSOPrecache Proxy Creation Strategy",
		ToolTip = "Defines what needs to happen when PSOs are still compiling while when a render proxy is created for a component."))
	EPSOPrecacheProxyCreationStrategy PSOPrecacheProxyCreationStrategy;

	UPROPERTY(config, EditAnywhere, Category = PSOPrecache, meta = (AllowedClasses = "/Script/Engine.Material",
		DisplayName = "PSOPrecache Fallback Material",
		ToolTip = "Material to use when PSOs are still compiling for a component and UseFallbackMaterialUntilPSOPrecached is set for EPSOPrecacheProxyCreationStrategy.",
		AssetBundles="Client",
		ConfigRestartRequired = false))
	FSoftObjectPath PSOPrecacheFallbackMaterial;

public:

	//~ Begin UObject Interface

	ENGINE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	//~ End UObject Interface

private:
#if WITH_EDITOR
	/** shadow copy saved before effects of PostEditChange() to provide option to roll back edit. */
	int32 PreEditReflectionCaptureResolution = 128;

	// Notification about missing required shader models.
	TWeakPtr<class SNotificationItem> ShaderModelNotificationPtr;

	void PreInitPropertiesFixup();
	void CheckForMissingShaderModels();
	void FixAntiAliasingOnShadingPathChange(FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

	void SanitizeReflectionCaptureResolution();
	void UpdateWorkingColorSpaceAndChromaticities();
};

UCLASS(config = Engine, projectuserconfig, meta = (DisplayName = "Rendering Overrides (Local)"), MinimalAPI)
class URendererOverrideSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/**
		"Enabling will locally override all ShaderPermutationReduction settings from the Renderer section to be enabled.  Saved to local user config only.."
	*/
	UPROPERTY(config, EditAnywhere, Category = ShaderPermutationReduction, meta = (
		ConsoleVariable = "r.SupportAllShaderPermutations", DisplayName = "Force all shader permutation support",
		ConfigRestartRequired = true))
		uint32 bSupportAllShaderPermutations : 1;

public:

	//~ Begin UObject Interface

	ENGINE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
