// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Engine/NetSerialization.h"
#include "ReplicatedState.generated.h"

/** Describes rules for network replicating a vector efficiently */
UENUM()
enum class EVectorQuantization : uint8
{
	/** Each vector component will be rounded to the nearest whole number. */
	RoundWholeNumber,
	/** Each vector component will be rounded, preserving one decimal place. */
	RoundOneDecimal,
	/** Each vector component will be rounded, preserving two decimal places. */
	RoundTwoDecimals
};

/** Describes rules for network replicating a vector efficiently */
UENUM()
enum class ERotatorQuantization : uint8
{
	/** The rotator will be compressed to 8 bits per component. */
	ByteComponents,
	/** The rotator will be compressed to 16 bits per component. */
	ShortComponents
};

/** Handles attachment replication to clients.  */
USTRUCT()
struct FRepAttachment
{
	GENERATED_BODY()

	/** Actor we are attached to, movement replication will not happen while AttachParent is non-nullptr */
	UPROPERTY()
	TObjectPtr<class AActor> AttachParent;

	/** Location offset from attach parent */
	UPROPERTY()
	FVector_NetQuantize100 LocationOffset;

	/** Scale relative to attach parent */
	UPROPERTY()
	FVector_NetQuantize100 RelativeScale3D;

	/** Rotation offset from attach parent */
	UPROPERTY()
	FRotator RotationOffset;

	/** Specific socket we are attached to */
	UPROPERTY()
	FName AttachSocket;

	/** Specific component we are attached to */
	UPROPERTY()
	TObjectPtr<class USceneComponent> AttachComponent;

	FRepAttachment()
		: AttachParent(nullptr)
		, LocationOffset(ForceInit)
		, RelativeScale3D(ForceInit)
		, RotationOffset(ForceInit)
		, AttachSocket(NAME_None)
		, AttachComponent(nullptr)
	{ }
};

/** Describes extra state about a specific rigid body */
namespace ERigidBodyFlags
{
	enum Type
	{
		None = 0x00,
		Sleeping = 0x01,
		NeedsUpdate = 0x02,
		RepPhysics = 0x04 // This RigidBody was replicated from the server with simulating physics
	};
}

/** Describes the physical state of a rigid body. */
USTRUCT()
struct FRigidBodyState
{
	GENERATED_BODY()

	UPROPERTY()
	FVector_NetQuantize100 Position;

	UPROPERTY()
	FQuat Quaternion;

	UPROPERTY()
	FVector_NetQuantize100 LinVel;

	UPROPERTY()
	FVector_NetQuantize100 AngVel;

	UPROPERTY()
	uint8 Flags;

	FRigidBodyState()
		: Position(ForceInit)
		, Quaternion(ForceInit)
		, LinVel(ForceInit)
		, AngVel(ForceInit)
		, Flags(0)
	{ }
};

/** Replicated movement data of our RootComponent.
  * Struct used for efficient replication as velocity and location are generally replicated together (this saves a repindex) 
  * and velocity.Z is commonly zero (most position replications are for walking pawns). 
  */
USTRUCT()
struct FRepMovement
{
	GENERATED_BODY()

	/** Velocity of component in world space */
	UPROPERTY(Transient)
	FVector LinearVelocity;

	/** Velocity of rotation for component (only valid if bRepPhysics is set) */
	UPROPERTY(Transient)
	FVector AngularVelocity;
	
	/** Location in world space */
	UPROPERTY(Transient)
	FVector Location;

	/** Current rotation */
	UPROPERTY(Transient)
	FRotator Rotation;

	/** Acceleration of component in world space. Only valid if bRepAcceleration is set. */
	UPROPERTY(Transient)
	FVector Acceleration;

	/** If set, RootComponent should be sleeping. */
	UPROPERTY(Transient)
	uint8 bSimulatedPhysicSleep : 1;

	/** If set, additional physic data (angular velocity) will be replicated. */
	UPROPERTY(Transient)
	uint8 bRepPhysics : 1;

	/** If set, additional acceleration data will be replicated. */
	UPROPERTY(Transient)
	uint8 bRepAcceleration : 1;

	/** Server physics step */
	UPROPERTY(Transient)
	int32 ServerFrame;

	/** ID assigned by server used to ensure determinism by physics. */
	UPROPERTY(Transient)
	int32 ServerPhysicsHandle = INDEX_NONE;

	/** Allows tuning the compression level for the replicated location vector. You should only need to change this from the default if you see visual artifacts. */
	UPROPERTY(EditDefaultsOnly, Category=Replication, AdvancedDisplay)
	EVectorQuantization LocationQuantizationLevel;

	/** Allows tuning the compression level for the replicated velocity vectors. You should only need to change this from the default if you see visual artifacts. */
	UPROPERTY(EditDefaultsOnly, Category=Replication, AdvancedDisplay)
	EVectorQuantization VelocityQuantizationLevel;

	/** Allows tuning the compression level for replicated rotation. You should only need to change this from the default if you see visual artifacts. */
	UPROPERTY(EditDefaultsOnly, Category=Replication, AdvancedDisplay)
	ERotatorQuantization RotationQuantizationLevel;

	ENGINE_API FRepMovement();

	ENGINE_API bool SerializeQuantizedVector(FArchive& Ar, FVector& Vector, EVectorQuantization QuantizationLevel);

	ENGINE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	void FillFrom(const FRigidBodyState& RBState, const AActor* const Actor = nullptr, int32 InServerFrame = 0)
	{
		Location = RebaseOntoZeroOrigin(RBState.Position, Actor);
		Rotation = RBState.Quaternion.Rotator();
		LinearVelocity = RBState.LinVel;
		AngularVelocity = RBState.AngVel;
		bSimulatedPhysicSleep = (RBState.Flags & ERigidBodyFlags::Sleeping) != 0;
		bRepPhysics = true;
		ServerFrame = InServerFrame;
	}

	void FillFrom(const FRigidBodyState& RBState, const UObject* const Object = nullptr, int32 InServerFrame = 0)
	{
		//TODO: Get Rebase stuff working with Entities
		//Location = RebaseOntoZeroOrigin(RBState.Position, Actor);
		Location = RBState.Position;
		Rotation = RBState.Quaternion.Rotator();
		LinearVelocity = RBState.LinVel;
		AngularVelocity = RBState.AngVel;
		bSimulatedPhysicSleep = (RBState.Flags & ERigidBodyFlags::Sleeping) != 0;
		bRepPhysics = true;
		ServerFrame = InServerFrame;
	}

	void CopyTo(FRigidBodyState& RBState, const AActor* const Actor = nullptr) const
	{
		RBState.Position = RebaseOntoLocalOrigin(Location, Actor);
		RBState.Quaternion = Rotation.Quaternion();
		RBState.LinVel = LinearVelocity;
		RBState.AngVel = AngularVelocity;
		RBState.Flags = 
			(decltype(FRigidBodyState::Flags))(bSimulatedPhysicSleep ? ERigidBodyFlags::Sleeping : ERigidBodyFlags::None)
			| ERigidBodyFlags::NeedsUpdate
			| (decltype(FRigidBodyState::Flags))(bRepPhysics ? ERigidBodyFlags::RepPhysics : ERigidBodyFlags::None);
	}

	void CopyTo(FRigidBodyState& RBState, const UObject* const Object = nullptr) const
	{
		RBState.Position = Location;
		RBState.Quaternion = Rotation.Quaternion();
		RBState.LinVel = LinearVelocity;
		RBState.AngVel = AngularVelocity;
		RBState.Flags =
			(decltype(FRigidBodyState::Flags))(bSimulatedPhysicSleep ? ERigidBodyFlags::Sleeping : ERigidBodyFlags::None)
			| ERigidBodyFlags::NeedsUpdate
			| (decltype(FRigidBodyState::Flags))(bRepPhysics ? ERigidBodyFlags::RepPhysics : ERigidBodyFlags::None);
	}

	bool operator==(const FRepMovement& Other) const
	{
		if ( LinearVelocity != Other.LinearVelocity )
		{
			return false;
		}

		if ( AngularVelocity != Other.AngularVelocity )
		{
			return false;
		}

		if ( Location != Other.Location )
		{
			return false;
		}

		if ( Rotation != Other.Rotation )
		{
			return false;
		}

		if (bRepAcceleration && (Acceleration != Other.Acceleration))
		{
			return false;
		}

		if ( bSimulatedPhysicSleep != Other.bSimulatedPhysicSleep )
		{
			return false;
		}

		if ( bRepPhysics != Other.bRepPhysics )
		{
			return false;
		}

		if ( bRepPhysics && (ServerFrame != Other.ServerFrame))
		{
			return false;
		}

		if ( bRepAcceleration != Other.bRepAcceleration )
		{
			return false;
		}

		return true;
	}

	bool operator!=(const FRepMovement& Other) const
	{
		return !(*this == Other);
	}

	/** True if multiplayer rebasing is enabled, corresponds to p.EnableMultiplayerWorldOriginRebasing console variable */
	static ENGINE_API int32 EnableMultiplayerWorldOriginRebasing;

	/** Rebase zero-origin position onto local world origin value. */
	static ENGINE_API FVector RebaseOntoLocalOrigin(const FVector& Location, const FIntVector& LocalOrigin);

	/** Rebase local-origin position onto zero world origin value. */
	static ENGINE_API FVector RebaseOntoZeroOrigin(const FVector& Location, const FIntVector& LocalOrigin);

	/** Rebase zero-origin position onto an Actor's local world origin. */
	static ENGINE_API FVector RebaseOntoLocalOrigin(const FVector& Location, const AActor* const WorldContextActor);

	/** Rebase an Actor's local-origin position onto zero world origin value. */
	static ENGINE_API FVector RebaseOntoZeroOrigin(const FVector& Location, const AActor* const WorldContextActor);

	/** Rebase zero-origin position onto local world origin value based on an actor component's world. */
	static ENGINE_API FVector RebaseOntoLocalOrigin(const FVector& Location, const class UActorComponent* const WorldContextActorComponent);

	/** Rebase local-origin position onto zero world origin value based on an actor component's world.*/
	static ENGINE_API FVector RebaseOntoZeroOrigin(const FVector& Location, const class UActorComponent* const WorldContextActorComponent);
};

template<>
struct TStructOpsTypeTraits<FRepMovement> : public TStructOpsTypeTraitsBase2<FRepMovement>
{
	enum
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};
