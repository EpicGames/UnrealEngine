// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	
 *	===================== Replication Driver Interface =====================
 *
 *	Defines an interface for driving actor replication. That is, the system that determines what actors should replicate to what connections.
 *	This is server only (in the traditional server->clients model).
 *	
 *	How to setup a Replication Driver (two ways):
 *	1. Set ReplicationDriverClassName in DefaultEngine.ini
 *	
 *		[/Script/OnlineSubsystemUtils.IpNetDriver]
 *		ReplicationDriverClassName="/Script/MyGame.MyReplicationGraph"
 *
 *	2. Bind to UReplicationDriver::CreateReplicationDriverDelegate(). Do this if you have custom logic for instantiating the driver (e.g, conditional based on map/game mode or hot fix options, etc)
 *	
 *		UReplicationDriver::CreateReplicationDriverDelegate().BindLambda([](UNetDriver* ForNetDriver, const FURL& URL, UWorld* World) -> UReplicationDriver*
 *		{
 *			return NewObject<UMyReplicationDriverClass>(GetTransientPackage());
 *		});	
 *
 */

#pragma once

#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "ReplicationDriver.generated.h"

class AActor;
class UActorChannel;
class UNetConnection;
class UNetDriver;
class UReplicationDriver;
class UWorld;

struct FActorDestructionInfo;
struct FURL;

DECLARE_DELEGATE_RetVal_ThreeParams(UReplicationDriver*, FCreateReplicationDriver, UNetDriver*, const FURL&, UWorld*);

UCLASS(abstract, transient, config=Engine, MinimalAPI)
class UReplicationDriver :public UObject
{
	GENERATED_BODY()

public:

	ENGINE_API UReplicationDriver();

	/** This is the function UNetDriver calls to create its replication driver. It will invoke OnCreateReplicationDriver if set, otherwise will instantiate ReplicationDriverClassName from the NetDriver.  */
	static ENGINE_API UReplicationDriver* CreateReplicationDriver(UNetDriver* NetDriver, const FURL& URL, UWorld* World);

	/** Static delegate you can bind to override replication driver creation */
	static ENGINE_API FCreateReplicationDriver& CreateReplicationDriverDelegate();

	// -----------------------------------------------------------------------

	/** Called to associate a world with a rep driver. This will be called before  InitForNetDriver */
	virtual void SetRepDriverWorld(UWorld* InWorld) PURE_VIRTUAL(UReplicationDriver::SetRepDriverWorld, );

	/** Called to associate a netdriver with a rep driver. The rep driver can "get itself ready" here. SetRepDriverWorld() will have already been called */
	virtual void InitForNetDriver(UNetDriver* InNetDriver) PURE_VIRTUAL(UReplicationDriver::InitForNetDriver, );

	/** Called after World and NetDriver have been set. This is where RepDriver should possibly look at existing actors in the world */
	virtual void InitializeActorsInWorld(UWorld* InWorld) PURE_VIRTUAL(UReplicationDriver::InitializeActorsInWorld, );

	virtual void TearDown() { MarkAsGarbage(); }

	virtual void ResetGameWorldState() PURE_VIRTUAL(UReplicationDriver::ResetGameWorldState, );

	virtual void AddClientConnection(UNetConnection* NetConnection) PURE_VIRTUAL(UReplicationDriver::AddClientConnection, );

	virtual void RemoveClientConnection(UNetConnection* NetConnection) PURE_VIRTUAL(UReplicationDriver::RemoveClientConnection, );

	virtual void AddNetworkActor(AActor* Actor) PURE_VIRTUAL(UReplicationDriver::AddNetworkActor, );

	virtual void RemoveNetworkActor(AActor* Actor) PURE_VIRTUAL(UReplicationDriver::RemoveNetworkActor, );

	virtual void ForceNetUpdate(AActor* Actor) PURE_VIRTUAL(UReplicationDriver::ForceNetUpdate, );

	virtual void FlushNetDormancy(AActor* Actor, bool WasDormInitial) PURE_VIRTUAL(UReplicationDriver::FlushNetDormancy, );

	virtual void NotifyActorTearOff(AActor* Actor) PURE_VIRTUAL(UReplicationDriver::NotifyActorTearOff, );

	virtual void NotifyActorFullyDormantForConnection(AActor* Actor, UNetConnection* Connection) PURE_VIRTUAL(UReplicationDriver::NotifyActorFullyDormantForConnection, );

	virtual void NotifyActorDormancyChange(AActor* Actor, ENetDormancy OldDormancyState) PURE_VIRTUAL(UReplicationDriver::NotifyActorDormancyChange, );

	/** Called when a destruction info is created for an actor. Can be used to override some of the destruction info struct */
	virtual void NotifyDestructionInfoCreated(AActor* Actor, FActorDestructionInfo& DestructionInfo) PURE_VIRTUAL(UReplicationDriver::NotifyDestructionInfoCreated, );

	/** Called when an actor is renamed. The implementation should update any cached actor/level mapping if the actor's outer changed. */
	virtual void NotifyActorRenamed(AActor* Actor, UObject* PreviousOuter, FName PreviousName) PURE_VIRTUAL(UReplicationDriver::NotifyActorRenamed, );

	virtual void SetRoleSwapOnReplicate(AActor* Actor, bool bSwapRoles) PURE_VIRTUAL(UReplicationDriver::SetRoleSwapOnReplicate, );

	/** Handles an RPC. Returns true if it actually handled it. Returning false will cause the rep driver function to handle it instead */
	virtual bool ProcessRemoteFunction(class AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject ) { return false; }

	/** The main function that will actually replicate actors. Called every server tick. */
	virtual int32 ServerReplicateActors(float DeltaSeconds) PURE_VIRTUAL(UReplicationDriver::ServerReplicateActors, return 0; );

	/** Called after the netdriver has handled TickDispatch */
	virtual void PostTickDispatch() { }
};

/** Class/interface for replication extension that is per connection. It is up to the replication driver to create and associate these with a UNetConnection */
UCLASS(abstract, transient, MinimalAPI)
class UReplicationConnectionDriver : public UObject
{
	GENERATED_BODY()

public:

	virtual void NotifyActorChannelAdded(AActor* Actor, UActorChannel* Channel) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyActorChannelAdded, );

	virtual void NotifyActorChannelRemoved(AActor* Actor) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyActorChannelRemoved, );

	virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyActorChannelCleanedUp, );

	virtual void NotifyAddDestructionInfo(FActorDestructionInfo* DestructInfo) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyAddDestructionInfo, );

	virtual void NotifyAddDormantDestructionInfo(AActor* Actor) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyAddDormantDestructionInfo, );

	virtual void NotifyRemoveDestructionInfo(FActorDestructionInfo* DestructInfo) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyRemoveDestructionInfo, );

	virtual void NotifyResetDestructionInfo() PURE_VIRTUAL(UReplicationConnectionDriver::NotifyResetDestructionInfo, );

	virtual void NotifyClientVisibleLevelNamesAdd(FName LevelName, UWorld* StreamingWorld) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyClientVisibleLevelNamesAdd, );

	virtual void NotifyClientVisibleLevelNamesRemove(FName LevelName) PURE_VIRTUAL(UReplicationConnectionDriver::NotifyClientVisibleLevelNamesRemove, );

	virtual void TearDown() { MarkAsGarbage(); }
};
