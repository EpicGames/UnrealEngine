// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RuntimeOptionsBase.generated.h"

/**
 *
 * URuntimeOptionsBase: Base class designed to be subclassed in your game.
 *
 * Supports checking at runtime whether features are enabled/disabled, and changing
 * configuration parameters via console cheats or startup commands.
 *
 * Add new config properties to your subclasses which default to the desired normal state
 * This can be adjusted via the development-only tools (command line or cvar) or via an
 * override in the config hierarchy to adjust it as needed (e.g., via a hotfix).
 *
 * In non-Shipping builds, each property will be exposed both as a console variable and as a
 * command-line argument for easy testing during development.
 *
 * Debug console syntax (disabled in Shipping configurations):
 *   prefix.PropertyName Value
 * Command line syntax (disabled in Shipping configurations):
 *   -prefix.PropertyName=Value
 * DefaultRuntimeOptions.ini syntax (note that there is no prefix for these):
 *   [/Script/YourModule.YourRuntimeOptionsSubclass]
 *   PropertyName=Value
 *
 * Where the prefix is set by the value of OptionCommandPrefix (defaults to "ro" but can be overridden)
 *
 * You can also change the name of the ini file that settings are gathered from in your derived
 * UCLASS() declaration
 */

UCLASS(config=RuntimeOptions, BlueprintType, Abstract, MinimalAPI)
class URuntimeOptionsBase : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API URuntimeOptionsBase();

	//~UObject interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;
	//~End of UObject interface

	// Returns the runtime options for the specified type, typically you will make a non-templated overload of Get in your subclass
	template<typename TRuntimeOptionsSubclass>
	static const TRuntimeOptionsSubclass& Get()
	{
		return *GetDefault<TRuntimeOptionsSubclass>();
	}

protected:
	// The command prefix for all options declared in a subclass (defaults to "ro", so a property named TestProp would be exposed as ro.TestProp)
	FString OptionCommandPrefix;

protected:
	// Applies overrides from the command line
	ENGINE_API void ApplyCommandlineOverrides();
	ENGINE_API void RegisterSupportedConsoleVariables(bool bDuringReload);
	ENGINE_API void InitializeRuntimeOptions();
};
