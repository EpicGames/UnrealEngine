// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/Blueprint.h"
#include "SCS_Node.generated.h"

class UBlueprint;

DECLARE_DELEGATE_OneParam( FSCSNodeNameChanged, const FName& );

UCLASS(MinimalAPI)
class USCS_Node : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Component class */
	UPROPERTY()
	TObjectPtr<UClass> ComponentClass;

	/** Template for the component to create */
	UPROPERTY()
	TObjectPtr<class UActorComponent> ComponentTemplate;

	/** Cached data for faster runtime instancing (only used in cooked builds) */
	UPROPERTY()
	FBlueprintCookedComponentInstancingData CookedComponentInstancingData;

#if WITH_EDITORONLY_DATA
	/** If non-None, the assigned category name */
	UPROPERTY()
	FText CategoryName;
#endif //WITH_EDITORONLY_DATA

	/** Socket/Bone that Node might attach to */
	UPROPERTY()
	FName AttachToName;

	/** Component template or variable that Node might be parented to */
	UPROPERTY()
	FName ParentComponentOrVariableName;

	/** If the node is attached to another node inherited from a parent Blueprint, this contains the name of the Blueprint parent class that owns the component template */
	//@TODO: We can potentially remove this if/when inherited SCS component template instances are included in subobject serialization, as we could then infer that the owner class is always the same as the BP class.
	UPROPERTY()
	FName ParentComponentOwnerClassName;

	/** If the node is parented, this indicates whether or not the template is found in the CDO's Components array */
	UPROPERTY()
	bool bIsParentComponentNative;

	/** Set of child nodes */
	UPROPERTY()
	TArray<TObjectPtr<class USCS_Node>> ChildNodes;

	/** Metadata information for this Node */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	TArray<struct FBPVariableMetaDataEntry> MetaDataArray;

	UPROPERTY()
	FGuid VariableGuid;

#if WITH_EDITORONLY_DATA
	/** (DEPRECATED) Indicates if this is a native component or not */
	UPROPERTY()
	bool bIsNative_DEPRECATED;

	/* (DEPRECATED) If this is a native component, this is the name of the UActorComponent */
	UPROPERTY()
	FName NativeComponentName_DEPRECATED;

	/** (DEPRECATED) If true, the variable name was a autogenerated and is not presented to the user */
	UPROPERTY()
	bool bVariableNameAutoGenerated_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

#if WITH_EDITOR

	/** The scene component constructed for component editing in the SCS editor */
	TWeakObjectPtr<USceneComponent> EditorComponentInstance;

	/** Make sure, the guid is proper - backward compatibility */
	ENGINE_API void ValidateGuid();

	/** Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object. */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/**
	 * Create the specified component on the actor, then call action on children
	 *
	 * @param Actor					The actor instance for which to create a new component based on the template encapsulated by this node.
	 * @param ParentComponent		If non-NULL, the component to which the new component should be attached as a child. If NULL, the new component will not be attached to anything.
	 * @param RootTransform			The transform to apply if this node turns out to be the root component of the actor instance.
	 * @param RootRelativeRotationCache Optional rotation cache to use if this node turns out to be the root component of the actor instance.
	 * @param bIsDefaultTransform	Indicates whether or not the given transform is a "default" transform, in which case it can be overridden by template defaults.
	 * @return The new component instance that was created, or NULL on failure.
	 */
	ENGINE_API UActorComponent* ExecuteNodeOnActor(AActor* Actor, USceneComponent* ParentComponent, const FTransform* RootTransform, const struct FRotationConversionCache* RootRelativeRotationCache, bool bIsDefaultTransform, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::OverrideRootScale);

	/** Return the actual component template used in the BPGC. The template can be overridden in a child. */
	ENGINE_API UActorComponent* GetActualComponentTemplate(class UBlueprintGeneratedClass* ActualBPGC) const;

	/** Return component template instancing data if cooked for the BPGC, as overridden template data can be cooked out for a child. */
	ENGINE_API const FBlueprintCookedComponentInstancingData* GetActualComponentTemplateData(class UBlueprintGeneratedClass* ActualBPGC) const;

	/** Returns an array containing this node and all children below it */
	TArray<USCS_Node*> GetAllNodes();
	
	/** Returns an constant reference to the child nodes array of this node */
	const TArray<USCS_Node*>& GetChildNodes() const { return ChildNodes; }

	/** Adds the given node as a child node */
	ENGINE_API void AddChildNode(USCS_Node* InNode, bool bAddToAllNodes = true);

	/** Removes the child node at the given index */
	ENGINE_API void RemoveChildNode(USCS_Node* InNode, bool bRemoveFromAllNodes = true);

	/** Removes the child node at the given index */
	ENGINE_API void RemoveChildNodeAt(int32 ChildIndex, bool bRemoveFromAllNodes = true);

	/** Moves a list of nodes from their current list to this node's ChildNode list */
	ENGINE_API void MoveChildNodes(USCS_Node* SourceNode, int32 InsertLocation = INDEX_NONE);

	/** Returns an array containing this node and all children below it */
	TArray<const USCS_Node*> GetAllNodes() const;

	/** See if this node is a child of the supplied parent */
	ENGINE_API bool IsChildOf(USCS_Node* TestParent);

	/** Preloads the node, and all its child nodes recursively */
	ENGINE_API void PreloadChain();

	/** See if this node is the root */
	ENGINE_API bool IsRootNode() const;

	/** Get name of the variable we should create for this component instance */
	inline FName GetVariableName() const
	{
		return InternalVariableName;
	}

	/** Set name of the variable we should create for this component instance */
	ENGINE_API void SetVariableName(const FName& NewName, bool bRenameTemplate = true);

	/** Get the SimpleConstructionScript that owns this node */
	class USimpleConstructionScript* GetSCS() const
	{
		return CastChecked<USimpleConstructionScript>(GetOuter());
	}

#if WITH_EDITOR
	/** Set delegate to call when name is modified externally */
	ENGINE_API void SetOnNameChanged(const FSCSNodeNameChanged& OnChange);

	/** Signal back to the SCS that variable name was modified by an external part of program */
	ENGINE_API void NameWasModified();
#endif

	/** Set a metadata value on the SCS_Node */
	ENGINE_API void SetMetaData(FName Key, FString Value);

	/** Gets a metadata value on the SCS_Node; asserts if the value isn't present.  Check for validiy using FindMetaDataEntryIndexForKey. */
	ENGINE_API const FString& GetMetaData(FName Key) const;

	/** Clear metadata value on the SCS_Node */
	ENGINE_API void RemoveMetaData(FName Key);

	/** Find the index in the array of a SCS_Node entry */
	ENGINE_API int32 FindMetaDataEntryIndexForKey(FName Key) const;

#if WITH_EDITOR
	/** Sets parent component attributes based on the given SCS node */
	ENGINE_API void SetParent(USCS_Node* InParentNode);

	/** Sets parent component attributes based on the given component instance */
	ENGINE_API void SetParent(const USceneComponent* InParentComponent);

	/** Finds and returns the parent component template through the given Blueprint */
	ENGINE_API USceneComponent* GetParentComponentTemplate(UBlueprint* InBlueprint) const;

	/** Finds and returns the parent component template through the given BPGC */
	ENGINE_API USceneComponent* GetParentComponentTemplate(UBlueprintGeneratedClass* BPGC) const;

	/** Saves the current state of SimpleConstructionScript and its nodes to the transaction buffer. */
	ENGINE_API void SaveToTransactionBuffer();
#endif

protected:
	/** Helper method to rename the given component template along with any instances */
	static void RenameComponentTemplate(UActorComponent* ComponentTemplate, const FName& NewName);
		
private:
	/** Internal variable name. This is used for:
			a) Generating the component template (archetype) object name.
			b) A FObjectProperty in the generated Blueprint class. This holds a reference to the component instance created at Actor construction time.
			c) Archetype lookup through the generated Blueprint class. All instances route back to the archetype through the variable name (i.e. not the template name).
	 */
	UPROPERTY()
	FName InternalVariableName;

#if WITH_EDITOR
	/** Delegate to trigger when Variable name is modified outside of SCS */
	FSCSNodeNameChanged		  OnNameChangedExternal;
#endif
};

