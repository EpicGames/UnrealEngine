// Copyright Epic Games, Inc. All Rights Reserved.

//===============================s==============================================
// Scene - script exposed scene enums
//=============================================================================

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptInterface.h"
#include "Engine/BlendableInterface.h"
#include "SceneUtils.h"
#include "Engine/EngineTypes.h"
#include "Scene.generated.h"


struct FPostProcessSettings;


/** Used by FPostProcessSettings Depth of Fields */
UENUM()
enum EDepthOfFieldMethod : int
{
	DOFM_BokehDOF UMETA(DisplayName="BokehDOF"),
	DOFM_Gaussian UMETA(DisplayName="GaussianDOF"),
	DOFM_CircleDOF UMETA(DisplayName="CircleDOF"),
	DOFM_MAX,
};

/** Used by FPostProcessSettings Auto Exposure */
UENUM()
enum EAutoExposureMethod : int
{
	/** requires compute shader to construct 64 bin histogram */
	AEM_Histogram  UMETA(DisplayName = "Auto Exposure Histogram"),
	/** faster method that computes single value by downsampling */
	AEM_Basic      UMETA(DisplayName = "Auto Exposure Basic"),
	/** Uses camera settings. */
	AEM_Manual   UMETA(DisplayName = "Manual"),
	AEM_MAX,
};

UENUM()
enum class ELocalExposureMethod : uint8
{
	/** Decompose image into base and detail layers using a bilateral blur. Only the base layer contrast is reduced in order to preserve details. */
	Bilateral UMETA(DisplayName = "Bilateral"),
	/** Fuse multiple exposures according to quality measures. Local Exposure is calculated using the fused exposures. */
	Fusion UMETA(DisplayName = "Fusion (Experimental)"),
};

UENUM()
enum EBloomMethod : int
{
	/** Sum of Gaussian formulation */
	BM_SOG  UMETA(DisplayName = "Standard"),
	/** Fast Fourier Transform Image based convolution, intended for cinematics (too expensive for games)  */
	BM_FFT  UMETA(DisplayName = "Convolution"),
	BM_MAX,
};

/** Used by FPostProcessSettings to determine Temperature calculation method. */
UENUM()
enum ETemperatureMethod : int
{
	TEMP_WhiteBalance UMETA(DisplayName = "White Balance"),
	TEMP_ColorTemperature UMETA(DisplayName = "Color Temperature"),
	TEMP_MAX,
};

/** Used by FPostProcessSettings to determine the tonemapping method. */
UENUM()
enum class ETonemappingMethod : uint8
{
	Filmic UMETA(DisplayName = "Filmic"),
	StandardACES UMETA(DisplayName = "Standard ACES", ToolTip = "Film settings do not apply to this method."),
	MAX,
};


UENUM() 
enum class ELightUnits : uint8
{
	Unitless UMETA(DisplayName = "Unitless"), 
	Candelas UMETA(DisplayName = "Candelas - Luminous Intensity, Normalized"),
	Lumens   UMETA(DisplayName = "Lumens - Luminous Flux, Normalized"),
	EV       UMETA(DisplayName = "EV - Exposure Value, Normalized"),
	Nits     UMETA(DisplayName = "Nits - Luminance, Non-Normalized")
};

UENUM()
enum class EReflectionsType : uint8
{
	ScreenSpace	UMETA(DisplayName = "Screen Space"),
	RayTracing	UMETA(DisplayName = "Ray Tracing"),
};

UENUM()
enum class ELumenRayLightingModeOverride : uint8
{
	/* Use the project default method */
	Default						UMETA(DisplayName = "Project Default"),
	/* Use Lumen Surface Cache for ray hit lighting. This method gives the best GI and reflection performance, but quality will be limited by how well surface cache represents given scene. */
	SurfaceCache				UMETA(DisplayName = "Surface Cache"),
	/* Calculate lighting at a hit point for reflections. This will improve reflection quality, but increases GPU cost, as full material needs to be evaluated and shadow rays traced. Lumen Surface Cache will still be used for GI and secondary bounces, including GI seen in reflections. */
	HitLightingForReflections	UMETA(DisplayName = "Hit Lighting for Reflections"),
	/* Calculate lighting at a hit point for GI and reflections. This will improve both GI and reflection quality, but greatly increases GPU cost, as full material and lighting will be evaluated at every hit point. Lumen Surface Cache will still be used for secondary bounces. */
	HitLighting					UMETA(DisplayName = "Hit Lighting"),
};

UENUM()
enum class ETranslucencyType : uint8
{
	Raster = 0		 UMETA(DisplayName = "Raster"),
	RayTraced_Deprecated UE_DEPRECATED(5.6, "Use RayTraced instead") = 1 UMETA(DisplayName = "Legacy Ray Tracing (Deprecated)", ToolTip = "The legacy Ray Tracing Translucency Method is deprecated, use Ray Traced instead"),
	RayTracing UE_DEPRECATED(5.6, "Use RayTraced instead") = 1 UMETA(Hidden),
	RayTraced = 2 UMETA(DisplayName = "Ray Traced")
};


UENUM()
enum class EReflectedAndRefractedRayTracedShadows : uint8
{
	Disabled		UMETA(DisplayName = "Disabled"),
	Hard_shadows	UMETA(DisplayName = "Hard Shadows"),
	Area_shadows	UMETA(DisplayName = "Area Shadows"),
};

UENUM()
enum class EVignetteType : uint8
{
	CosineFourthLaw	= 0 UMETA(DisplayName = "Cosine Fourth Law (Default)"),
	Texture			= 1 UMETA(DisplayName = "Texture"),
	Oval			= 2 UMETA(DisplayName = "Oval")
};

USTRUCT(BlueprintType)
struct FColorGradePerRangeSettings
{
	GENERATED_USTRUCT_BODY()


	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation", SupportDynamicSliderMaxValue = "true", DisplayName = "Saturation"))
	FVector4 Saturation;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast", SupportDynamicSliderMaxValue = "true", DisplayName = "Contrast"))
	FVector4 Contrast;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.01", UIMax = "2.0", Delta = "0.01", ClampMin = "0.01", ColorGradingMode = "gamma", SupportDynamicSliderMaxValue = "true", DisplayName = "Gamma"))
	FVector4 Gamma;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gain", SupportDynamicSliderMaxValue = "true", DisplayName = "Gain"))
	FVector4 Gain;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true", DisplayName = "Offset"))
	FVector4 Offset;


	FColorGradePerRangeSettings()
	{
		Saturation = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
		Contrast = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
		Gamma = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
		Gain = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
		Offset = FVector4(0.0f, 0.0f, 0.0f, 0.0f);
	}
};

USTRUCT(BlueprintType)
struct FColorGradingSettings
{
	GENERATED_USTRUCT_BODY()


	UPROPERTY(Interp,BlueprintReadWrite, Category = "Color Grading")
	FColorGradePerRangeSettings Global;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading")
	FColorGradePerRangeSettings Shadows;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading")
	FColorGradePerRangeSettings Midtones;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading")
	FColorGradePerRangeSettings Highlights;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "-1.0", UIMax = "1.0", DisplayName = "ShadowsMax"))
	float ShadowsMax;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "-1.0", UIMax = "1.0", DisplayName = "HighlightsMin"))
	float HighlightsMin;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "1.0", UIMax = "10.0", DisplayName = "HighlightsMax"))
	float HighlightsMax;

	FColorGradingSettings()
	{
		ShadowsMax = 0.09f;
		HighlightsMin = 0.5f;
		HighlightsMax = 1.0f;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FFilmStockSettings
{
	GENERATED_USTRUCT_BODY()


	UPROPERTY(Interp, BlueprintReadWrite, Category = "Film Stock", meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "Slope"))
	float Slope;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Film Stock", meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "Toe"))
	float Toe;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Film Stock", meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "Shoulder"))
	float Shoulder;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Film Stock", meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "Black clip"))
	float BlackClip;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Film Stock", meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "White clip"))
	float WhiteClip;


	FFilmStockSettings()
	{
		Slope = 0.88f;
		Toe = 0.55f;
		Shoulder = 0.26f;
		BlackClip = 0.0f;
		WhiteClip = 0.04f;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FGaussianSumBloomSettings
{
	GENERATED_USTRUCT_BODY()


	/** Multiplier for Gaussian bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", meta=(ClampMin = "0.0", UIMax = "8.0", DisplayName = "Intensity"))
	float Intensity;

	/**
	 * minimum brightness the bloom starts having effect
	 * -1:all pixels affect bloom equally (physically correct, faster as a threshold pass is omitted), 0:all pixels affect bloom brights more, 1(default), >1 brighter
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", meta=(ClampMin = "-1.0", UIMax = "8.0", DisplayName = "Threshold"))
	float Threshold;

	/**
	 * Scale for all bloom sizes
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "64.0", DisplayName = "Size scale"))
	float SizeScale;

	/**
	 * Diameter size for the Bloom1 in percent of the screen width
	 * (is done in 1/2 resolution, larger values cost more performance, good for high frequency details)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "4.0", DisplayName = "#1 Size"))
	float Filter1Size;

	/**
	 * Diameter size for Bloom2 in percent of the screen width
	 * (is done in 1/4 resolution, larger values cost more performance)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "8.0", DisplayName = "#2 Size"))
	float Filter2Size;

	/**
	 * Diameter size for Bloom3 in percent of the screen width
	 * (is done in 1/8 resolution, larger values cost more performance)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "16.0", DisplayName = "#3 Size"))
	float Filter3Size;

	/**
	 * Diameter size for Bloom4 in percent of the screen width
	 * (is done in 1/16 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "32.0", DisplayName = "#4 Size"))
	float Filter4Size;

	/**
	 * Diameter size for Bloom5 in percent of the screen width
	 * (is done in 1/32 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "64.0", DisplayName = "#5 Size"))
	float Filter5Size;

	/**
	 * Diameter size for Bloom6 in percent of the screen width
	 * (is done in 1/64 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "128.0", DisplayName = "#6 Size"))
	float Filter6Size;

	/** Bloom1 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#1 Tint", HideAlphaChannel))
	FLinearColor Filter1Tint;

	/** Bloom2 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#2 Tint", HideAlphaChannel))
	FLinearColor Filter2Tint;

	/** Bloom3 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#3 Tint", HideAlphaChannel))
	FLinearColor Filter3Tint;

	/** Bloom4 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#4 Tint", HideAlphaChannel))
	FLinearColor Filter4Tint;

	/** Bloom5 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#5 Tint", HideAlphaChannel))
	FLinearColor Filter5Tint;

	/** Bloom6 tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(DisplayName = "#6 Tint", HideAlphaChannel))
	FLinearColor Filter6Tint;


	FGaussianSumBloomSettings()
	{
		Intensity = 1.0f;
		Threshold = -1.0f;
		// default is 4 to maintain old settings after fixing something that caused a factor of 4
		SizeScale = 4.0;
		Filter1Tint = FLinearColor(0.3465f, 0.3465f, 0.3465f);
		Filter1Size = 0.3f;
		Filter2Tint = FLinearColor(0.138f, 0.138f, 0.138f);
		Filter2Size = 1.0f;
		Filter3Tint = FLinearColor(0.1176f, 0.1176f, 0.1176f);
		Filter3Size = 2.0f;
		Filter4Tint = FLinearColor(0.066f, 0.066f, 0.066f);
		Filter4Size = 10.0f;
		Filter5Tint = FLinearColor(0.066f, 0.066f, 0.066f);
		Filter5Size = 30.0f;
		Filter6Tint = FLinearColor(0.061f, 0.061f, 0.061f);
		Filter6Size = 64.0f;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FConvolutionBloomSettings
{
	GENERATED_USTRUCT_BODY()


	/** Multiplier for convolution bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom|Convolution", meta=(ClampMin = "0.0", UIMax = "8.0", DisplayName = "Intensity"))
	float Intensity;

	/** Texture to replace default convolution bloom kernel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", meta = (DisplayName = "Convolution Kernel"))
	TObjectPtr<class UTexture2D> Texture;

	/** Intensity multiplier on the scatter dispersion energy of the kernel. 1.0 means exactly use the same energy as the kernel scatter dispersion. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "1.0", DisplayName = "Scatter Dispersion"))
	float ScatterDispersion;

	/** Relative size of the convolution kernel image compared to the minor axis of the viewport  */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "1.0", DisplayName = "Convolution Scale"))
	float Size;

	/** The UV location of the center of the kernel.  Should be very close to (.5,.5) */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (DisplayName = "Convolution Center"))
	FVector2D CenterUV;

	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables convolution boost */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (DisplayName = "Convolution Boost Min"))
	float PreFilterMin;

	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables convolution boost */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (DisplayName = "Convolution Boost Max"))
	float PreFilterMax;

	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables convolution boost */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (DisplayName = "Convolution Boost Mult"))
	float PreFilterMult;

	/** Implicit buffer region as a fraction of the screen size to insure the bloom does not wrap across the screen.  Larger sizes have perf impact.*/
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "1.0", DisplayName = "Convolution Buffer"))
	float BufferScale;


	FConvolutionBloomSettings()
	{
		Intensity = 1.0f;
		Texture = nullptr;
		ScatterDispersion = 1.0f;
		Size = 1.f;
		CenterUV = FVector2D(0.5f, 0.5f);
		PreFilterMin = 7.f;
		PreFilterMax = 15000.f;
		PreFilterMult = 15.f;
		BufferScale = 0.133f;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FLensBloomSettings
{
	GENERATED_USTRUCT_BODY()
	
	/** Multiplier for all bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Bloom", meta=(ClampMin = "0.0", UIMax = "8.0", DisplayName = "Intensity"))
	float Intensity;

	/** Bloom gaussian sum method specific settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Gaussian Sum Method")
	FGaussianSumBloomSettings GaussianSum;

	/** Bloom convolution method specific settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Convolution Method")
	FConvolutionBloomSettings Convolution;

	/** Bloom algorithm */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Bloom")
	TEnumAsByte<enum EBloomMethod> Method;


	FLensBloomSettings()
	{
		Intensity = 0.675f;
		Method = BM_SOG;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FLensBloomSettings> : public TStructOpsTypeTraitsBase2<FLensBloomSettings>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};
#endif

USTRUCT(BlueprintType)
struct FLensImperfectionSettings
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * Texture that defines the dirt on the camera lens where the light of very bright objects is scattered.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(DisplayName = "Dirt Mask Texture"))
	TObjectPtr<class UTexture> DirtMask;	
	
	/** BloomDirtMask intensity */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(ClampMin = "0.0", UIMax = "8.0", DisplayName = "Dirt Mask Intensity"))
	float DirtMaskIntensity;

	/** BloomDirtMask tint color */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(DisplayName = "Dirt Mask Tint", HideAlphaChannel))
	FLinearColor DirtMaskTint;


	FLensImperfectionSettings()
	{
		DirtMask = nullptr;
		DirtMaskIntensity = 0.0f;
		DirtMaskTint = FLinearColor(0.5f, 0.5f, 0.5f);
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FLensSettings
{
	GENERATED_USTRUCT_BODY()


	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens")
	FLensBloomSettings Bloom;

	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens")
	FLensImperfectionSettings Imperfections;

	/** in percent, Scene chromatic aberration / color fringe (camera imperfection) to simulate an artifact that happens in real-world lens, mostly visible in the image corners. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens", meta = (UIMin = "0.0", UIMax = "5.0"))
	float ChromaticAberration;


	FLensSettings()
	{
		ChromaticAberration = 0.0f;
	}

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FCameraExposureSettings
{
	GENERATED_USTRUCT_BODY()
		
	/** Luminance computation method */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Exposure", meta=(DisplayName = "Method"))
    TEnumAsByte<enum EAutoExposureMethod> Method;

	/**
	 * The eye adaptation will adapt to a value extracted from the luminance histogram of the scene color.
	 * The value is defined as having x percent below this brightness. Higher values give bright spots on the screen more priority
	 * but can lead to less stable results. Lower values give the medium and darker values more priority but might cause burn out of
	 * bright spots.
	 * >0, <100, good values are in the range 70 .. 80
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "100.0", DisplayName = "Low Percent"))
	float LowPercent;

	/**
	 * The eye adaptation will adapt to a value extracted from the luminance histogram of the scene color.
	 * The value is defined as having x percent below this brightness. Higher values give bright spots on the screen more priority
	 * but can lead to less stable results. Lower values give the medium and darker values more priority but might cause burn out of
	 * bright spots.
	 * >0, <100, good values are in the range 80 .. 95
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "100.0", DisplayName = "High Percent"))
	float HighPercent;

	/**
	 * A good value should be positive near 0. This is the minimum brightness the auto exposure can adapt to.
	 * It should be tweaked in a dark lighting situation (too small: image appears too bright, too large: image appears too dark).
	 * Note: Tweaking emissive materials and lights or tweaking auto exposure can look the same. Tweaking auto exposure has global
	 * effect and defined the HDR range - you don't want to change that late in the project development.
	 * Eye Adaptation is disabled if MinBrightness = MaxBrightness
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", meta=(ClampMin = "0.0", UIMax = "10.0", DisplayName = "Min Brightness"))
	float MinBrightness;

	/**
	 * A good value should be positive (2 is a good value). This is the maximum brightness the auto exposure can adapt to.
	 * It should be tweaked in a bright lighting situation (too small: image appears too bright, too large: image appears too dark).
	 * Note: Tweaking emissive materials and lights or tweaking auto exposure can look the same. Tweaking auto exposure has global
	 * effect and defined the HDR range - you don't want to change that late in the project development.
	 * Eye Adaptation is disabled if MinBrightness = MaxBrightness
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", meta=(ClampMin = "0.0", UIMax = "10.0", DisplayName = "Max Brightness"))
	float MaxBrightness;

	/** >0 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", meta=(ClampMin = "0.02", UIMax = "20.0", DisplayName = "Speed Up", tooltip = "In F-stops per second, should be >0"))
	float SpeedUp;

	/** >0 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", meta=(ClampMin = "0.02", UIMax = "20.0", DisplayName = "Speed Down", tooltip = "In F-stops per second, should be >0"))
	float SpeedDown;

	/**
	 * Logarithmic adjustment for the exposure. Only used if a tonemapper is specified.
	 * 0: no adjustment, -1:2x darker, -2:4x darker, 1:2x brighter, 2:4x brighter, ...
	 */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Exposure", meta = (UIMin = "-8.0", UIMax = "8.0", DisplayName = "Exposure Bias"))
	float Bias;

	/**
	 * Exposure compensation based on the scene EV100.
	 * Used to calibrate the final exposure differently depending on the average scene luminance.
	 * 0: no adjustment, -1:2x darker, -2:4x darker, 1:2x brighter, 2:4x brighter, ...
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Exposure Bias Curve"))
	TObjectPtr<class UCurveFloat> BiasCurve = nullptr;

	/**
	 * Exposure metering mask. Bright spots on the mask will have high influence on auto-exposure metering
	 * and dark spots will have low influence.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Exposure", meta=(DisplayName = "Exposure Metering Mask"))
	TObjectPtr<class UTexture> MeterMask = nullptr;	

	/** temporary exposed until we found good values, -8: 1/256, -10: 1/1024 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", AdvancedDisplay, meta=(UIMin = "-16", UIMax = "0.0"))
	float HistogramLogMin;

	/** temporary exposed until we found good values 4: 16, 8: 256 */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Exposure", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "16.0"))
	float HistogramLogMax;

	/** Calibration constant for 18% albedo. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Exposure", AdvancedDisplay, meta=(UIMin = "0", UIMax = "100.0", DisplayName = "Calibration Constant"))
	float CalibrationConstant;

	/** Enables physical camera exposure using ShutterSpeed/ISO/Aperture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Apply Physical Camera Exposure"))
	uint32 ApplyPhysicalCameraExposure : 1;

	ENGINE_API FCameraExposureSettings();

	/* Exports to post process settings with overrides. */
	ENGINE_API void ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const;
};

USTRUCT(BlueprintType)
struct FMatteBoxFlag
{
	GENERATED_USTRUCT_BODY()

	/** Angle of the panel with the lens barrel */
	UPROPERTY(interp, BlueprintReadWrite, Category=FMatteBoxFlag, meta=(ClampMin = "-90.0", ClampMax = "90.0"))
	float Pitch;

	/** Rotation of the panel around the lens barrel */
	UPROPERTY(interp, BlueprintReadWrite, Category=FMatteBoxFlag, meta=(ClampMin = "0.0", ClampMax = "360.0"))
	float Roll;

	/** How long is the panel */
	UPROPERTY(interp, BlueprintReadWrite, Category=FMatteBoxFlag, meta=(ClampMin = "0.0", UIMax = "100.0"))
	float Length;

	FMatteBoxFlag()
	: Pitch(0)
	, Roll(0)
	, Length(0)
	{ }
};

USTRUCT(BlueprintType)
struct FWeightedBlendable
{
	GENERATED_USTRUCT_BODY()

	/** 0:no effect .. 1:full effect */
	UPROPERTY(interp, BlueprintReadWrite, Category=FWeightedBlendable, meta=(ClampMin = "0.0", ClampMax = "1.0", Delta = "0.01"))
	float Weight;

	/** should be of the IBlendableInterface* type but UProperties cannot express that */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FWeightedBlendable, meta=( AllowedClasses="/Script/Engine.BlendableInterface", Keywords="PostProcess" ))
	TObjectPtr<UObject> Object;

	// default constructor
	FWeightedBlendable()
		: Weight(-1)
		, Object(0)
	{
	}

	// constructor
	// @param InWeight -1 is used to hide the weight and show the "Choose" UI, 0:no effect .. 1:full effect
	FWeightedBlendable(float InWeight, UObject* InObject)
		: Weight(InWeight)
		, Object(InObject)
	{
	}
};

// for easier detail customization, needed?
USTRUCT(BlueprintType)
struct FWeightedBlendables
{
	GENERATED_BODY()

public:

	FWeightedBlendables() { }
	FWeightedBlendables(const TArray<FWeightedBlendable>& InArray) : Array(InArray) { }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PostProcessSettings", meta=( Keywords="PostProcess" ))
	TArray<FWeightedBlendable> Array;
};

struct FPostProcessSettingsDebugInfo
{
	FString Name;

	float Priority;
	float CurrentBlendWeight;
	bool bIsEnabled;
	bool bIsUnbound;

	FString DebugPropertyValue;
};

/** To be able to use struct PostProcessSettings. */
// Each property consists of a bool to enable it (by default off),
// the variable declaration and further down the default value for it.
// The comment should include the meaning and usable range.
USTRUCT(BlueprintType, meta=(HiddenByDefault, DisableSplitPin))
struct FPostProcessSettings
{
	GENERATED_USTRUCT_BODY()

	// first all bOverride_... as they get grouped together into bitfields

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_TemperatureType:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_WhiteTemp:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_WhiteTint:1;

	// Color Correction controls
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorSaturation:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorContrast:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGamma:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGain:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorOffset:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorSaturationShadows : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorContrastShadows : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGammaShadows : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGainShadows : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorOffsetShadows : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorSaturationMidtones : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorContrastMidtones : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGammaMidtones : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGainMidtones : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorOffsetMidtones : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorSaturationHighlights : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorContrastHighlights : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGammaHighlights : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGainHighlights : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorOffsetHighlights : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionShadowsMax : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionHighlightsMin : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionHighlightsMax : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BlueCorrection : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ExpandGamut : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ToneCurveAmount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_TonemappingMethod:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmSlope:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmToe:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmShoulder:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmBlackClip:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmWhiteClip:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_SceneColorTint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_SceneFringeIntensity:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ChromaticAberrationStartOffset:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_bMegaLights:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MegaLightsFrontLayerTranslucency:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientCubemapTint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientCubemapIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomGaussianIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomThreshold:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom1Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom1Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom2Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom2Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom3Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom3Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom4Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom4Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom5Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom5Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom6Tint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Bloom6Size:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomSizeScale:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionTexture : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionScatterDispersion : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionSize : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionCenterUV : 1;

	UPROPERTY()
	uint8 bOverride_BloomConvolutionPreFilter_DEPRECATED : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionPreFilterMin : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionPreFilterMax : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionPreFilterMult : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomConvolutionBufferScale : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomDirtMaskIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomDirtMaskTint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BloomDirtMask:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
    uint8 bOverride_CameraShutterSpeed:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
    uint8 bOverride_CameraISO:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
    uint8 bOverride_AutoExposureMethod:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureLowPercent:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureHighPercent:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureMinBrightness:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureMaxBrightness:1;

	UPROPERTY()
	uint8 bOverride_AutoExposureCalibrationConstant_DEPRECATED:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureSpeedUp:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureSpeedDown:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureBias:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureBiasCurve:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureMeterMask:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureApplyPhysicalCameraExposure:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_HistogramLogMin:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_HistogramLogMax:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureMethod : 1;

	UPROPERTY()
	uint8 bOverride_LocalExposureContrastScale_DEPRECATED:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureHighlightContrastScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureShadowContrastScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureHighlightContrastCurve:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureShadowContrastCurve:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureHighlightThreshold:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureShadowThreshold :1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureDetailStrength:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureBlurredLuminanceBlend:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureBlurredLuminanceKernelSizePercent:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureHighlightThresholdStrength :1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureShadowThresholdStrength :1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LocalExposureMiddleGreyBias:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareTint:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareTints:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareBokehSize:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareBokehShape:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LensFlareThreshold:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteType:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteCenter:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteColor:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteSize:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteSoftness:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_VignetteTexture:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Sharpen:1;

	UPROPERTY()
	uint8 bOverride_GrainIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_GrainJitter_DEPRECATED:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainIntensityShadows : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainIntensityMidtones : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainIntensityHighlights : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainShadowsMax : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainHighlightsMin : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainHighlightsMax : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 bOverride_FilmGrainTexelSize_DEPRECATED:1;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_FilmGrainTexture : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionStaticFraction:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionRadius:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionFadeDistance:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionFadeRadius:1;

	UPROPERTY()
	uint8 bOverride_AmbientOcclusionDistance_DEPRECATED:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionRadiusInWS:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionPower:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionBias:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionQuality:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionMipBlend:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionMipScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionMipThreshold:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AmbientOcclusionTemporalBlendWeight : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingAO : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingAOSamplesPerPixel : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingAOIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingAORadius : 1;

	UPROPERTY()
	uint8 bOverride_LPVIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVDirectionalOcclusionIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVDirectionalOcclusionRadius_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVDiffuseOcclusionExponent_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVSpecularOcclusionExponent_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVDiffuseOcclusionIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVSpecularOcclusionIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVSize_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVSecondaryOcclusionIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVSecondaryBounceIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVGeometryVolumeBias_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVVplInjectionBias_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVEmissiveInjectionIntensity_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVFadeRange_DEPRECATED:1;

	UPROPERTY()
	uint8 bOverride_LPVDirectionalOcclusionFadeRange_DEPRECATED:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_IndirectLightingColor:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_IndirectLightingIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGradingIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorGradingLUT:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldEnabled:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldFocalDistance:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldFstop:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldMinFstop : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldBladeCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldSensorWidth:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldSqueezeFactor:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldDepthBlurRadius:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldUseHairDepth:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldPetzvalBokeh:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldPetzvalBokehFalloff:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldPetzvalExclusionBoxExtents:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldPetzvalExclusionBoxRadius:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldAspectRatioScalar:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldMatteBoxFlags:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldBarrelRadius:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldBarrelLength:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldDepthBlurAmount:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldFocalRegion:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldNearTransitionRegion:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldFarTransitionRegion:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldNearBlurSize:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldFarBlurSize:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MobileHQGaussian:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldOcclusion:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldSkyFocusDistance:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_DepthOfFieldVignetteSize:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MotionBlurAmount:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MotionBlurMax:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MotionBlurTargetFPS : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_MotionBlurPerObjectSize:1;

	UPROPERTY()
	uint8 bOverride_ScreenPercentage_DEPRECATED:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ReflectionMethod:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LumenReflectionQuality:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ScreenSpaceReflectionIntensity:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ScreenSpaceReflectionQuality:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ScreenSpaceReflectionMaxRoughness:1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint8 bOverride_ScreenSpaceReflectionRoughnessScale:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_UserFlags:1;

	// -----------------------------------------------------------------------

	// Ray Tracing
	
	UPROPERTY()
	uint32 bOverride_ReflectionsType_DEPRECATED : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingReflectionsMaxRoughness : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingReflectionsMaxBounces : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingReflectionsSamplesPerPixel : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingReflectionsShadows : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingReflectionsTranslucency : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_TranslucencyType : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyMaxRoughness : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyRefractionRays : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencySamplesPerPixel : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyShadows : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyRefraction : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyMaxPrimaryHitEvents : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyMaxSecondaryHitEvents : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_RayTracingTranslucencyUseRayTracedRefraction : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_DynamicGlobalIlluminationMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSceneLightingQuality : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSceneDetail : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSceneViewDistance : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSceneLightingUpdateSpeed : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenFinalGatherQuality : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenFinalGatherLightingUpdateSpeed : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenFinalGatherScreenTraces : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenMaxTraceDistance : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenDiffuseColorBoost : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenAmbientOcclusionIntensity : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSkylightLeaking : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenSkylightLeakingTint : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenFullSkylightLeakingDistance : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LumenRayLightingMode:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LumenReflectionsScreenTraces:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Overrides, meta=(PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LumenFrontLayerTranslucencyReflections:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenMaxRoughnessToTraceReflections : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenMaxReflectionBounces : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_LumenMaxRefractionBounces : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_LumenSurfaceCacheResolution : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingGI : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingGIMaxBounces : 1;

	UE_DEPRECATED(5.8, "Field will be removed")
	UPROPERTY()
	uint32 bOverride_RayTracingGISamplesPerPixel : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingMaxBounces : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingSamplesPerPixel : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingMaxPathIntensity : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingEnableEmissiveMaterials : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingEnableReferenceDOF : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingEnableReferenceAtmosphere : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingEnableDenoiser : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeEmissive : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeDiffuse : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeIndirectDiffuse : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeSpecular : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeIndirectSpecular : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeVolume : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverride_PathTracingIncludeIndirectVolume : 1;

	// -----------------------------------------------------------------------

	/** Enable HQ Gaussian on high end mobile platforms. (ES3_1) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Mobile Depth of Field", meta = (editcondition = "bOverride_MobileHQGaussian", DisplayName = "High Quality Gaussian DoF on Mobile"))
	uint8 bMobileHQGaussian:1;

	/** Preferred bloom algorithm. A different method may be used if the preferred method is not available on the target platform, e.g. due to performance limits. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Bloom", meta = (editcondition = "bOverride_BloomMethod", DisplayName = "Preferred Method"))
	TEnumAsByte<enum EBloomMethod> BloomMethod;

	/** Luminance computation method */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Exposure", meta=(editcondition = "bOverride_AutoExposureMethod", DisplayName = "Metering Mode"))
    TEnumAsByte<enum EAutoExposureMethod> AutoExposureMethod;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TEnumAsByte<enum EDepthOfFieldMethod> DepthOfFieldMethod_DEPRECATED;
#endif

	/**
	* Selects the type of temperature calculation.
	* White Balance uses the Temperature value to control the virtual camera's White Balance. This is the default selection.
	* Color Temperature uses the Temperature value to adjust the color temperature of the scene, which is the inverse of the White Balance operation.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Temperature", meta = (DisplayName = "Temperature Type", editcondition = "bOverride_TemperatureType" ))
	TEnumAsByte<enum ETemperatureMethod> TemperatureType;
	/** Controls the color temperature or white balance in degrees Kelvin which the scene considers as white light. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Color Grading|Temperature", meta=(UIMin = "1500.0", UIMax = "15000.0", editcondition = "bOverride_WhiteTemp", DisplayName = "Temp"))
	float WhiteTemp;
	/** Controls the color of the scene along the magenta - green axis (orthogonal to the color temperature).  This feature is equivalent to color tint in digital cameras. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Color Grading|Temperature", meta=(UIMin = "-1.0", UIMax = "1.0", editcondition = "bOverride_WhiteTint", DisplayName = "Tint"))
	float WhiteTint;

	/** Control the intensity of the color(hue) for the entire image.Higher values will result in more vibrant colors. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Global", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorSaturation", DisplayName = "Saturation"))
	FVector4 ColorSaturation;
	/** Control the range of light and dark values in your scene. Lower values will reduce the difference between bright and dark areas while higher values will increase the difference between the bright and dark areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Global", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorContrast", DisplayName = "Contrast"))
	FVector4 ColorContrast;
	/** Control the luminance curve of the scene. Raising or lowering this value will result brightening or darkening the mid-tones of the entire image. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Global", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gamma", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGamma", DisplayName = "Gamma"))
	FVector4 ColorGamma;
	/** This value multiplies the colors of the image.  Raising or lowering this value will result in brightening or darkening the entire scene. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Global", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gain", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGain", DisplayName = "Gain"))
	FVector4 ColorGain;
	/** This value is added to the colors of the scene.  Raising or lowering this value will result in the image being more or less washed-out. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Global", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true", editcondition = "bOverride_ColorOffset", DisplayName = "Offset"))
	FVector4 ColorOffset;

	/** Control the intensity of the colors (hue) in the shadow region of the image.  Higher values will result in more vibrant colors. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorSaturationShadows", DisplayName = "Saturation"))
	FVector4 ColorSaturationShadows;
	/** Control the range of light and dark values in your scene. Lower values will reduce the difference between bright and dark areas while higher values will increase the difference between the bright and dark areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorContrastShadows", DisplayName = "Contrast"))
	FVector4 ColorContrastShadows;
	/** Control the luminance curve of the shadow region. Raising or lowering this value will result brightening or darkening the mid-tones of the shadow region. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gamma", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGammaShadows", DisplayName = "Gamma"))
	FVector4 ColorGammaShadows;
	/** This value multiplies the colors in the shadow region.  Raising or lowering this value will result in brightening or darkening the affected region. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gain", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGainShadows", DisplayName = "Gain"))
	FVector4 ColorGainShadows;
	/** This value is added to the colors in the shadow region.  Raising or lowering this value will result in the shadows being more or less washed-out. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true", editcondition = "bOverride_ColorOffsetShadows", DisplayName = "Offset"))
	FVector4 ColorOffsetShadows;

	/** Control the intensity of the colors (hue) in the mid-tone region of the image.  Higher values will result in more vibrant colors. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Midtones", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorSaturationMidtones", DisplayName = "Saturation"))
	FVector4 ColorSaturationMidtones;
	/** Control the range of light and dark values in the mid-tone region. Lower values will reduce the difference between bright and dark areas while higher values will increase the difference between the bright and dark areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Midtones", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorContrastMidtones", DisplayName = "Contrast"))
	FVector4 ColorContrastMidtones;
	/** Control the luminance curve of the mid-tone region of the image. Raising or lowering this value will result brightening or darkening the mid-tones of the image. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Midtones", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gamma", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGammaMidtones", DisplayName = "Gamma"))
	FVector4 ColorGammaMidtones;
	/** This value multiplies the colors in the mid-tone region of the image.  Raising or lowering this value will result in brightening or darkening the affected region. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Midtones", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gain", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGainMidtones", DisplayName = "Gain"))
	FVector4 ColorGainMidtones;
	/** This value is added to the colors in the mid-tone region of the image.  Raising or lowering this value will result in the mid-tones being more or less washed-out. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Midtones", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true", editcondition = "bOverride_ColorOffsetMidtones", DisplayName = "Offset"))
	FVector4 ColorOffsetMidtones;

	/** Control the intensity of the color (hue) for the highlights region of the image.  Higher values will result in more vibrant colors. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorSaturationHighlights", DisplayName = "Saturation"))
	FVector4 ColorSaturationHighlights;
	/** Control the range of light and dark values in the highlights region. Lower values will reduce the difference between bright and dark areas while higher values will increase the difference between the bright and dark areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorContrastHighlights", DisplayName = "Contrast"))
	FVector4 ColorContrastHighlights;
	/** Control the luminance curve of the highlight region. Raising or lowering this value will result brightening or darkening the mid-tones of the highlight region. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gamma", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGammaHighlights", DisplayName = "Gamma"))
	FVector4 ColorGammaHighlights;
	/** This value multiplies the colors in the highlight region.  Raising or lowering this value will result in brightening or darkening the affected region. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gain", SupportDynamicSliderMaxValue = "true", editcondition = "bOverride_ColorGainHighlights", DisplayName = "Gain"))
	FVector4 ColorGainHighlights;
	/** This value is added to the colors in the highlight region.  Raising or lowering this value will result in the highlights being more or less washed-out. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true", editcondition = "bOverride_ColorOffsetHighlights", DisplayName = "Offset"))
	FVector4 ColorOffsetHighlights;
	/** This value sets the lower threshold for what is considered to be the highlight region of the image. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "-1.0", UIMax = "1.0", editcondition = "bOverride_ColorCorrectionHighlightsMin", DisplayName = "HighlightsMin"))
	float ColorCorrectionHighlightsMin;
	/** This value sets the upper threshold for what is considered to be the highlight region of the image.  This value should be larger than HighlightsMin. Default is 1.0, for backwards compatibility */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Highlights", meta = (UIMin = "1.0", UIMax = "10.0", editcondition = "bOverride_ColorCorrectionHighlightsMax", DisplayName = "HighlightsMax"))
	float ColorCorrectionHighlightsMax;

	/** This value sets the threshold for what is considered to be the shadow region of the image. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Shadows", meta = (UIMin = "-1.0", UIMax = "1.0", editcondition = "bOverride_ColorCorrectionShadowsMax", DisplayName = "ShadowsMax"))
	float ColorCorrectionShadowsMax;

	/** Correct for artifacts with "electric" blues due to the ACEScg color space. Bright blue desaturates instead of going to violet. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Misc", meta = (ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_BlueCorrection"))
	float BlueCorrection;
	/** Expand bright saturated colors outside the sRGB gamut to fake wide gamut rendering. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Misc", meta = (ClampMin = "0.0", UIMax = "1.0", editcondition = "bOverride_ExpandGamut"))
	float ExpandGamut;
	/** Allow effect of Tone Curve to be reduced (Set ToneCurveAmount and ExpandGamut to 0.0 to fully disable tone curve) */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Misc", meta = (ClampMin = "0.0", UIMax = "1.0", editcondition = "bOverride_ToneCurveAmount"))
	float ToneCurveAmount;

	/** Controls the method of tonemapping. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(editcondition = "bOverride_TonemappingMethod", DisplayName = "Method"))
	ETonemappingMethod TonemappingMethod;
	/** Controls the overall steepness of the tonemapper curve.  Larger values increase scene contrast and smaller values reduce contrast. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmSlope", DisplayName = "Slope"))
	float FilmSlope;
	/** Controls the contrast of the dark end of the tonemapper curve. Larger values increase contrast and smaller values decrease contrast. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmToe", DisplayName = "Toe"))
	float FilmToe;
	/**  Sometimes referred to as highlight rolloff.  Controls the contrast of the bright end of the tonemapper curve. Larger values increase contrast and smaller values decrease contrast.  */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmShoulder", DisplayName = "Shoulder"))
	float FilmShoulder;
	/** Lowers the toe of the tonemapper curve by this amount. Increasing this value causes more of the scene to clip to black.  For most purposes, this property should remain 0 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmBlackClip", DisplayName = "Black clip"))
	float FilmBlackClip;
	/** Controls the height of the tonemapper curve.  Raising this value can cause bright values to more quickly approach fully-saturated white. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Film", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmWhiteClip", DisplayName = "White clip"))
	float FilmWhiteClip;

	/** Scene tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Color Grading|Misc", meta=(editcondition = "bOverride_SceneColorTint", HideAlphaChannel))
	FLinearColor SceneColorTint;
	
	/** in percent, Scene chromatic aberration / color fringe (camera imperfection) to simulate an artifact that happens in real-world lens, mostly visible in the image corners. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Chromatic Aberration", meta = (UIMin = "0.0", UIMax = "5.0", editcondition = "bOverride_SceneFringeIntensity", DisplayName = "Intensity"))
	float SceneFringeIntensity;

	/** A normalized distance to the center of the framebuffer where the effect takes place. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Chromatic Aberration", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_ChromaticAberrationStartOffset", DisplayName = "Start Offset"))
	float ChromaticAberrationStartOffset;

	/** Multiplier for all bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom", meta=(ClampMin = "0.0", UIMax = "8.0", editcondition = "bOverride_BloomIntensity", DisplayName = "Intensity"))
	float BloomIntensity;
	
	/** Multiplier for Gaussian bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", meta=(ClampMin = "0.0", UIMax = "8.0", editcondition = "bOverride_BloomGaussianIntensity", DisplayName = "Intensity"))
	float BloomGaussianIntensity;

	/**
	 * minimum brightness the bloom starts having effect
	 * -1:all pixels affect bloom equally (physically correct, faster as a threshold pass is omitted), 0:all pixels affect bloom brights more, 1(default), >1 brighter
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", meta=(ClampMin = "-1.0", UIMax = "8.0", editcondition = "bOverride_BloomThreshold", DisplayName = "Threshold"))
	float BloomThreshold;

	/**
	 * Scale for all bloom sizes
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "64.0", editcondition = "bOverride_BloomSizeScale", DisplayName = "Size scale"))
	float BloomSizeScale;

	/**
	 * Diameter size for the Bloom1 in percent of the screen width
	 * (is done in 1/2 resolution, larger values cost more performance, good for high frequency details)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "4.0", editcondition = "bOverride_Bloom1Size", DisplayName = "#1 Size"))
	float Bloom1Size;
	/**
	 * Diameter size for Bloom2 in percent of the screen width
	 * (is done in 1/4 resolution, larger values cost more performance)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "8.0", editcondition = "bOverride_Bloom2Size", DisplayName = "#2 Size"))
	float Bloom2Size;
	/**
	 * Diameter size for Bloom3 in percent of the screen width
	 * (is done in 1/8 resolution, larger values cost more performance)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "16.0", editcondition = "bOverride_Bloom3Size", DisplayName = "#3 Size"))
	float Bloom3Size;
	/**
	 * Diameter size for Bloom4 in percent of the screen width
	 * (is done in 1/16 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "32.0", editcondition = "bOverride_Bloom4Size", DisplayName = "#4 Size"))
	float Bloom4Size;
	/**
	 * Diameter size for Bloom5 in percent of the screen width
	 * (is done in 1/32 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "64.0", editcondition = "bOverride_Bloom5Size", DisplayName = "#5 Size"))
	float Bloom5Size;
	/**
	 * Diameter size for Bloom6 in percent of the screen width
	 * (is done in 1/64 resolution, larger values cost more performance, best for wide contributions)
	 * >=0: can be clamped because of shader limitations
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "128.0", editcondition = "bOverride_Bloom6Size", DisplayName = "#6 Size"))
	float Bloom6Size;

	/** Bloom1 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom1Tint", DisplayName = "#1 Tint", HideAlphaChannel))
	FLinearColor Bloom1Tint;
	/** Bloom2 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom2Tint", DisplayName = "#2 Tint", HideAlphaChannel))
	FLinearColor Bloom2Tint;
	/** Bloom3 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom3Tint", DisplayName = "#3 Tint", HideAlphaChannel))
	FLinearColor Bloom3Tint;
	/** Bloom4 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom4Tint", DisplayName = "#4 Tint", HideAlphaChannel))
	FLinearColor Bloom4Tint;
	/** Bloom5 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom5Tint", DisplayName = "#5 Tint", HideAlphaChannel))
	FLinearColor Bloom5Tint;
	/** Bloom6 tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Gaussian", AdvancedDisplay, meta=(editcondition = "bOverride_Bloom6Tint", DisplayName = "#6 Tint", HideAlphaChannel))
	FLinearColor Bloom6Tint;
	
	/** Multiplier for Convolution bloom contributions >=0: off, 1(default), >1 brighter */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Bloom|Convolution", meta=(ClampMin = "0.0", UIMax = "1.0", editcondition = "bOverride_BloomConvolutionIntensity", DisplayName = "Intensity"))
	float BloomConvolutionIntensity;

	/** Intensity multiplier on the scatter dispersion energy of the kernel. 1.0 means exactly use the same energy as the kernel scatter dispersion. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", meta = (ClampMin = "0.0", UIMax = "20.0", editcondition = "bOverride_BloomConvolutionScatterDispersion", DisplayName = "Scatter Dispersion"))
	float BloomConvolutionScatterDispersion;

	/** Relative size of the convolution kernel image compared to the minor axis of the viewport  */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "1.0", editcondition = "bOverride_BloomConvolutionSize", DisplayName = "Scale"))
	float BloomConvolutionSize;

	/** Texture to replace default convolution bloom kernel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", meta = (editcondition = "bOverride_BloomConvolutionTexture", DisplayName = "Kernel"))
	TObjectPtr<class UTexture2D> BloomConvolutionTexture;

	/** The UV location of the center of the kernel.  Should be very close to (.5,.5) */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (editcondition = "bOverride_BloomConvolutionCenterUV", DisplayName = "Center"))
	FVector2D BloomConvolutionCenterUV;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FVector3f BloomConvolutionPreFilter_DEPRECATED;
#endif
	
	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (editcondition = "bOverride_BloomConvolutionPreFilterMin", DisplayName = "Boost Min"))
	float BloomConvolutionPreFilterMin;

	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (editcondition = "bOverride_BloomConvolutionPreFilterMax", DisplayName = "Boost Max"))
	float BloomConvolutionPreFilterMax;

	/** Boost intensity of select pixels  prior to computing bloom convolution (Min, Max, Multiplier).  Max < Min disables */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (editcondition = "bOverride_BloomConvolutionPreFilterMult", DisplayName = "Boost Mult"))
	float BloomConvolutionPreFilterMult;

	/** Implicit buffer region as a fraction of the screen size to insure the bloom does not wrap across the screen.  Larger sizes have perf impact.*/
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Bloom|Convolution", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "1.0", editcondition = "bOverride_BloomConvolutionBufferScale", DisplayName = "Buffer"))
	float BloomConvolutionBufferScale;
	
	/**
	 * Texture that defines the dirt on the camera lens where the light of very bright objects is scattered.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(editcondition = "bOverride_BloomDirtMask", DisplayName = "Dirt Mask Texture"))
	TObjectPtr<class UTexture> BloomDirtMask;	
	
	/** BloomDirtMask intensity */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(ClampMin = "0.0", UIMax = "8.0", editcondition = "bOverride_BloomDirtMaskIntensity", DisplayName = "Dirt Mask Intensity"))
	float BloomDirtMaskIntensity;

	/** BloomDirtMask tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Dirt Mask", meta=(editcondition = "bOverride_BloomDirtMaskTint", DisplayName = "Dirt Mask Tint", HideAlphaChannel))
	FLinearColor BloomDirtMaskTint;

	/** Chooses the Dynamic Global Illumination method.  Not compatible with Forward Shading. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination", meta = (editcondition = "bOverride_DynamicGlobalIlluminationMethod", DisplayName = "Method"))
	TEnumAsByte<EDynamicGlobalIlluminationMethod::Type> DynamicGlobalIlluminationMethod;

	/** Adjusts indirect lighting color. (1,1,1) is default. (0,0,0) to disable GI. The show flag 'Global Illumination' must be enabled to use this property. */
	UPROPERTY(interp, BlueprintReadWrite, AdvancedDisplay, Category="Global Illumination", meta=(editcondition = "bOverride_IndirectLightingColor", DisplayName = "Indirect Lighting Color", HideAlphaChannel))
	FLinearColor IndirectLightingColor;

	/** Scales the indirect lighting contribution. A value of 0 disables GI. Default is 1. The show flag 'Global Illumination' must be enabled to use this property. */
	UPROPERTY(interp, BlueprintReadWrite, AdvancedDisplay, Category="Global Illumination", meta=(ClampMin = "0", UIMax = "4.0", editcondition = "bOverride_IndirectLightingIntensity", DisplayName = "Indirect Lighting Intensity"))
	float IndirectLightingIntensity;
	
	/** Controls how Lumen rays are lit when Lumen is using Hardware Ray Tracing.  By default, Lumen uses the Surface Cache for best performance, but can be set to 'Hit Lighting' for higher quality. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (editcondition = "bOverride_LumenRayLightingMode", DisplayName = "Ray Lighting Mode"))
	ELumenRayLightingModeOverride LumenRayLightingMode;
	
	/** Scales Lumen Scene's quality.  Larger scales cause Lumen Scene to be calculated with a higher fidelity, which can be visible in reflections, but increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = ".25", UIMax = "2", editcondition = "bOverride_LumenSceneLightingQuality", DisplayName = "Lumen Scene Lighting Quality"))
	float LumenSceneLightingQuality;

	/** Controls the size of instances that can be represented in Lumen Scene.  Larger values will ensure small objects are represented, but increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = ".25", UIMax = "4", editcondition = "bOverride_LumenSceneDetail", DisplayName = "Lumen Scene Detail"))
	float LumenSceneDetail;

	/** Sets the maximum view distance of the scene that Lumen maintains for ray tracing against.  Larger values will increase the effective range of sky shadowing and Global Illumination, but increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = "1", UIMax = "2097152", editcondition = "bOverride_LumenSceneViewDistance", DisplayName = "Lumen Scene View Distance"))
	float LumenSceneViewDistance;

	/** Controls how much Lumen Scene is allowed to cache lighting results to improve performance.  Larger scales cause lighting changes to propagate faster, but increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = ".5", UIMax = "4", editcondition = "bOverride_LumenSceneLightingUpdateSpeed", DisplayName = "Lumen Scene Lighting Update Speed"))
	float LumenSceneLightingUpdateSpeed;

	/** Scales Lumen's Final Gather quality.  Larger scales reduce noise, but greatly increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = ".25", UIMax = "2", editcondition = "bOverride_LumenFinalGatherQuality", DisplayName = "Final Gather Quality"))
	float LumenFinalGatherQuality;

	/** Controls how much Lumen Final Gather is allowed to cache lighting results to improve performance.  Larger scales cause lighting changes to propagate faster, but increase GPU cost and noise. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = ".5", UIMax = "4", editcondition = "bOverride_LumenFinalGatherLightingUpdateSpeed", DisplayName = "Final Gather Lighting Update Speed"))
	float LumenFinalGatherLightingUpdateSpeed;

	/** Whether to use screen space traces for Lumen Global Illumination. Screen space traces bypass Lumen Scene and instead sample Scene Depth and Scene Color. This improves quality, as it bypasses Lumen Scene, but causes view dependent lighting. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (editcondition = "bOverride_LumenFinalGatherScreenTraces", DisplayName = "Screen Traces"))
	uint8 LumenFinalGatherScreenTraces : 1;

	/** Controls the maximum distance that Lumen should trace while solving lighting.  Values that are too small will cause lighting to leak into large caves, while values that are large will increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = "1", UIMax = "2097152", editcondition = "bOverride_LumenMaxTraceDistance", DisplayName = "Max Trace Distance"))
	float LumenMaxTraceDistance;

	/** Allows brightening indirect lighting by calculating material diffuse color for indirect lighting. Values above 1 (original diffuse color) aren't physically correct, but they can be useful as an art direction knob to increase the amount of bounced light in the scene. Best to keep below 2 as it also causes reflections to be brighter than the scene. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = "1.0", UIMax = ".01", ClampMax = "4", editcondition = "bOverride_LumenDiffuseColorBoost", DisplayName = "Diffuse Color Boost"))
	float LumenDiffuseColorBoost;

	/** Allows control over the strength of Lumen's Short Range Ambient Occlusion. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = "0", UIMax = "1", ClampMax = "1", editcondition = "bOverride_LumenAmbientOcclusionIntensity", DisplayName = "Ambient Occlusion Intensity"))
	float LumenAmbientOcclusionIntensity;

	/** Controls what fraction of the skylight intensity should be allowed to leak.  This can be useful as an art direction knob (non-physically based) to keep indoor areas from going fully black. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = "0", UIMax = ".02", ClampMax = "1", editcondition = "bOverride_LumenSkylightLeaking", DisplayName = "Skylight Leaking"))
	float LumenSkylightLeaking;

	/** Color tint for Lumen Skylight Leaking. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta=(editcondition = "bOverride_LumenSkylightLeakingTint", DisplayName = "Skylight Leaking Tint", HideAlphaChannel))
	FLinearColor LumenSkylightLeakingTint;

	/** Controls the distance from a receiving surface where skylight leaking reaches its full intensity.  Smaller values make the skylight leaking flatter, while larger values create an Ambient Occlusion effect. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", AdvancedDisplay, meta = (ClampMin = ".1", UIMax = "2000", editcondition = "bOverride_LumenFullSkylightLeakingDistance", DisplayName = "Full Skylight Leaking Distance"))
	float LumenFullSkylightLeakingDistance;

	/** Scale factor for Lumen Surface Cache resolution.  Smaller values save GPU memory, at a cost in quality.  Defaults to 0.5 for Scene Captures if not overridden. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Global Illumination|Lumen Global Illumination", meta = (ClampMin = ".5", ClampMax = "1", editcondition = "bOverride_LumenSurfaceCacheResolution"))
	float LumenSurfaceCacheResolution;

	/** Chooses the Reflection method. Not compatible with Forward Shading. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections", meta = (editcondition = "bOverride_ReflectionMethod", DisplayName = "Method"))
	TEnumAsByte<EReflectionMethod::Type> ReflectionMethod;

	UPROPERTY()
	EReflectionsType ReflectionsType_DEPRECATED;

	/** Scales the Reflection quality.  Larger scales reduce noise in reflections, but increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (ClampMin = ".25", UIMax = "2", editcondition = "bOverride_LumenReflectionQuality", DisplayName = "Quality"))
	float LumenReflectionQuality;

	/** Whether to use screen space traces for Lumen Reflections. Screen space traces bypass Lumen Scene and instead sample Scene Depth and Scene Color. This improves quality, as it bypasses Lumen Scene, but causes view dependent lighting. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (editcondition = "bOverride_LumenReflectionsScreenTraces", DisplayName = "Screen Traces"))
	uint8 LumenReflectionsScreenTraces : 1;

	/** Whether to use high quality mirror reflections on the front layer of translucent surfaces.  Other layers will use the lower quality Radiance Cache method that can only produce glossy reflections.  Increases GPU cost when enabled. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (ClampMin = ".25", UIMax = "2", editcondition = "bOverride_LumenFrontLayerTranslucencyReflections", DisplayName = "High Quality Translucency Reflections"))
	uint8 LumenFrontLayerTranslucencyReflections : 1;

	/** Sets the maximum roughness value for which Lumen still traces dedicated reflection rays. Higher values improve reflection quality, but greatly increase GPU cost. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (ClampMin = "0", ClampMax = "1", editcondition = "bOverride_LumenMaxRoughnessToTraceReflections", DisplayName = "Max Roughness To Trace"))
	float LumenMaxRoughnessToTraceReflections;

	/** Sets the maximum number of recursive reflection bounces. 1 means a single reflection ray (no secondary reflections in mirrors). Currently only supported by Hardware Ray Tracing with Hit Lighting. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (ClampMin = "1", ClampMax = "8", editcondition = "bOverride_LumenMaxReflectionBounces", DisplayName = "Max Reflection Bounces"))
	int32 LumenMaxReflectionBounces;

	/** The maximum count of refraction event to trace. When hit lighting is used, Translucent meshes will be traced when LumenMaxRefractionBounces > 0, making the reflection tracing more expenssive. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Reflections|Lumen Reflections", meta = (ClampMin = "0", ClampMax = "64", editcondition = "bOverride_LumenMaxRefractionBounces", DisplayName = "Max Refraction Bounces"))
	int32 LumenMaxRefractionBounces;

	/** Enable/Fade/disable the Screen Space Reflection feature, in percent, avoid numbers between 0 and 1 fo consistency */
	UPROPERTY(interp, BlueprintReadWrite, Category="Reflections|Screen Space Reflections", meta=(ClampMin = "0.0", ClampMax = "100.0", editcondition = "bOverride_ScreenSpaceReflectionIntensity", DisplayName = "Intensity"))
	float ScreenSpaceReflectionIntensity;

	/** 0=lowest quality..100=maximum quality, only a few quality levels are implemented, no soft transition, 50 is the default for better performance. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Reflections|Screen Space Reflections", meta=(ClampMin = "0.0", UIMax = "100.0", editcondition = "bOverride_ScreenSpaceReflectionQuality", DisplayName = "Quality"))
	float ScreenSpaceReflectionQuality;

	/** Until what roughness we fade the screen space reflections, 0.8 works well, smaller can run faster */
	UPROPERTY(interp, BlueprintReadWrite, Category="Reflections|Screen Space Reflections", meta=(ClampMin = "0.01", ClampMax = "1.0", editcondition = "bOverride_ScreenSpaceReflectionMaxRoughness", DisplayName = "Max Roughness"))
	float ScreenSpaceReflectionMaxRoughness;

	/** 
	 * Allows forcing MegaLights on or off for this volume, regardless of the project setting for MegaLights. 
	 * MegaLights will stochastically sample lights, which allows many shadow casting lights to be rendered efficiently, with a consistent and low GPU cost.
	 * When MegaLights is enabled, other direct lighting algorithms like Deferred Shading will no longer be used, and other shadowing methods like Ray Traced Shadows, Distance Field Shadows and Shadow Maps will no longer be used.
	 * MegaLights requires Hardware Ray Tracing and Shader Model 6.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering Features", meta = (editcondition = "bOverride_bMegaLights", DisplayName = "MegaLights"))
	uint8 bMegaLights : 1;
	
	/** Whether to use high quality direct lighting on the front layer of translucent surfaces.  Other layers will use the lower quality Translucency Lighting Volume.  Increases GPU cost when enabled. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features", meta = (editcondition = "bOverride_MegaLightsFrontLayerTranslucency", DisplayName = "MegaLights - High Quality Translucency"))
	uint8 MegaLightsFrontLayerTranslucency : 1;

	/** AmbientCubemap tint color */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Cubemap", meta=(editcondition = "bOverride_AmbientCubemapTint", DisplayName = "Tint", HideAlphaChannel))
	FLinearColor AmbientCubemapTint;

	/**
	 * To scale the Ambient cubemap brightness
	 * >=0: off, 1(default), >1 brighter
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Cubemap", meta=(ClampMin = "0.0", UIMax = "4.0", editcondition = "bOverride_AmbientCubemapIntensity", DisplayName = "Intensity"))
	float AmbientCubemapIntensity;

	/** The Ambient cubemap (Affects diffuse and specular shading), blends additively which if different from all other settings here */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Rendering Features|Ambient Cubemap", meta=(DisplayName = "Cubemap Texture"))
	TObjectPtr<class UTextureCube> AmbientCubemap;

	/** The camera shutter in 1/seconds.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Camera", meta=(ClampMin = "0.01", ClampMax = "8000.0", editcondition = "bOverride_CameraShutterSpeed", DisplayName = "Shutter Speed (1/s)"))
    float CameraShutterSpeed;

	/** The camera sensor sensitivity in ISO.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Camera", meta=(ClampMin = "1.0", tooltip = "The camera sensor sensitivity", editcondition = "bOverride_CameraISO", DisplayName = "ISO"))
    float CameraISO;

	/** Defines the opening of the camera lens, Aperture is 1/fstop, typical lens go down to f/1.2 (large opening), larger numbers reduce the DOF effect */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Camera", meta=(ClampMin = "1.0", ClampMax = "32.0", editcondition = "bOverride_DepthOfFieldFstop", DisplayName = "Aperture (F-stop)"))
	float DepthOfFieldFstop;

	/** Defines the maximum opening of the camera lens to control the curvature of blades of the diaphragm. Set it to 0 to get straight blades. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Camera", meta=(ClampMin = "0.0", ClampMax = "32.0", editcondition = "bOverride_DepthOfFieldMinFstop", DisplayName = "Maximum Aperture (min F-stop)"))
	float DepthOfFieldMinFstop;

	/** Defines the number of blades of the diaphragm within the lens (between 4 and 16). */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Camera", meta=(ClampMin = "4", ClampMax = "16", editcondition = "bOverride_DepthOfFieldBladeCount", DisplayName = "Number of diaphragm blades"))
	int32 DepthOfFieldBladeCount;

	/**
	 * Logarithmic adjustment for the exposure. Only used if a tonemapper is specified.
	 * 0: no adjustment, -1:2x darker, -2:4x darker, 1:2x brighter, 2:4x brighter, ...
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Exposure", meta = (UIMin = "-15.0", UIMax = "15.0", editcondition = "bOverride_AutoExposureBias", DisplayName = "Exposure Compensation "))
	float AutoExposureBias;

	/**
	 * With the auto exposure changes, we are changing the AutoExposureBias inside the serialization code. We are 
	 * storing that value before conversion here as a backup. Hopefully it will not be needed, and removed in the next engine revision.
	 */
	UPROPERTY()
	float AutoExposureBiasBackup;

	/**
	 * With the auto exposure changes, we are also changing the auto exposure override value, so we are storing 
	 * that backup as well.
	 */
	UPROPERTY()
	uint8 bOverride_AutoExposureBiasBackup : 1;

	/** Enables physical camera exposure using ShutterSpeed/ISO/Aperture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Exposure", meta = (editcondition = "bOverride_AutoExposureApplyPhysicalCameraExposure", DisplayName = "Apply Physical Camera Exposure", tooltip = "Only affects Manual exposure mode."))
	uint32 AutoExposureApplyPhysicalCameraExposure : 1;

	/**
	 * Exposure compensation based on the scene EV100.
	 * Used to calibrate the final exposure differently depending on the average scene luminance.
	 * 0: no adjustment, -1:2x darker, -2:4x darker, 1:2x brighter, 2:4x brighter, ...
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Exposure", meta = (editcondition = "bOverride_AutoExposureBiasCurve", DisplayName = "Exposure Compensation Curve"))
	TObjectPtr<class UCurveFloat> AutoExposureBiasCurve = nullptr;

	/**
	 * Exposure metering mask. Bright spots on the mask will have high influence on auto-exposure metering
	 * and dark spots will have low influence.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Exposure", meta=(editcondition = "bOverride_AutoExposureMeterMask", DisplayName = "Exposure Metering Mask"))
	TObjectPtr<class UTexture> AutoExposureMeterMask = nullptr;	

	/**
	 * The eye adaptation will adapt to a value extracted from the luminance histogram of the scene color.
	 * The value is defined as having x percent below this brightness. Higher values give bright spots on the screen more priority
	 * but can lead to less stable results. Lower values give the medium and darker values more priority but might cause burn out of
	 * bright spots.
	 * >0, <100, good values are in the range 70 .. 80
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "100.0", editcondition = "bOverride_AutoExposureLowPercent", DisplayName = "Low Percent"))
	float AutoExposureLowPercent;

	/**
	 * The eye adaptation will adapt to a value extracted from the luminance histogram of the scene color.
	 * The value is defined as having x percent below this brightness. Higher values give bright spots on the screen more priority
	 * but can lead to less stable results. Lower values give the medium and darker values more priority but might cause burn out of
	 * bright spots.
	 * >0, <100, good values are in the range 80 .. 95
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "100.0", editcondition = "bOverride_AutoExposureHighPercent", DisplayName = "High Percent"))
	float AutoExposureHighPercent;

	/**
	 * Auto-Exposure minimum adaptation. Eye Adaptation is disabled if Min = Max. 
	 * Auto-exposure is implemented by choosing an exposure value for which the average luminance generates a pixel brightness equal to the Constant Calibration value.
	 * The Min/Max are expressed in pixel luminance (cd/m2) or in EV100 when using ExtendDefaultLuminanceRange (see project settings).
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", meta=(ClampMin = "-10.0", UIMax = "20.0", editcondition = "bOverride_AutoExposureMinBrightness", DisplayName = "Min Brightness"))
	float AutoExposureMinBrightness;

	/**
	 * Auto-Exposure maximum adaptation. Eye Adaptation is disabled if Min = Max. 
	 * Auto-exposure is implemented by choosing an exposure value for which the average luminance generates a pixel brightness equal to the Constant Calibration value.
	 * The Min/Max are expressed in pixel luminance (cd/m2) or in EV100 when using ExtendDefaultLuminanceRange (see project settings).
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", meta=(ClampMin = "-10.0", UIMax = "20.0", editcondition = "bOverride_AutoExposureMaxBrightness", DisplayName = "Max Brightness"))
	float AutoExposureMaxBrightness;

	/** >0 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", meta=(ClampMin = "0.02", UIMax = "20.0", editcondition = "bOverride_AutoExposureSpeedUp", DisplayName = "Speed Up", tooltip = "In F-stops per second, should be >0"))
	float AutoExposureSpeedUp;

	/** >0 */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", meta=(ClampMin = "0.02", UIMax = "20.0", editcondition = "bOverride_AutoExposureSpeedDown", DisplayName = "Speed Down", tooltip = "In F-stops per second, should be >0"))
	float AutoExposureSpeedDown;

	/** Histogram Min value. Expressed in Log2(Luminance) or in EV100 when using ExtendDefaultLuminanceRange (see project settings) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", AdvancedDisplay, meta=(UIMin = "-16", UIMax = "0.0", editcondition = "bOverride_HistogramLogMin"))
	float HistogramLogMin;

	/** Histogram Max value. Expressed in Log2(Luminance) or in EV100 when using ExtendDefaultLuminanceRange (see project settings) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Exposure", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "16.0", editcondition = "bOverride_HistogramLogMax"))
	float HistogramLogMax;

	/** Calibration constant for 18% albedo, deprecating this value. */
	UPROPERTY()
	float AutoExposureCalibrationConstant_DEPRECATED;

	/** Local Exposure algorithm */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Local Exposure", meta = (editcondition = "bOverride_LocalExposureMethod", DisplayName = "Method"))
	ELocalExposureMethod LocalExposureMethod;

	UPROPERTY()
	float LocalExposureContrastScale_DEPRECATED;

	/** 
	 * Local Exposure decomposes luminance of the frame into a base layer and a detail layer.
	 * Contrast of the base layer is reduced based on this value.
	 * Value less than 1 will enable local exposure.
	 * Good values are usually in the range 0.6 .. 1.0.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_LocalExposureHighlightContrastScale", DisplayName = "Highlight Contrast"))
	float LocalExposureHighlightContrastScale;

	/** 
	 * Local Exposure decomposes luminance of the frame into a base layer and a detail layer.
	 * Contrast of the base layer is reduced based on this value.
	 * Value less than 1 will enable local exposure.
	 * Good values are usually in the range 0.6 .. 1.0.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_LocalExposureShadowContrastScale", DisplayName = "Shadow Contrast"))
	float LocalExposureShadowContrastScale;

	/**
	 * Local Exposure Highlight Contrast based on the scene EV100.
	 * Used to calibrate Local Exposure differently depending on the average scene luminance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Local Exposure", meta = (editcondition = "bOverride_LocalExposureHighlightContrastCurve", DisplayName = "Highlight Contrast Curve"))
	TObjectPtr<class UCurveFloat> LocalExposureHighlightContrastCurve = nullptr;

	/**
	 * Local Exposure Shadow Contrast based on the scene EV100.
	 * Used to calibrate Local Exposure differently depending on the average scene luminance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Local Exposure", meta = (editcondition = "bOverride_LocalExposureShadowContrastCurve", DisplayName = "Shadow Contrast Curve"))
	TObjectPtr<class UCurveFloat> LocalExposureShadowContrastCurve = nullptr;

	/** 
	 * Threshold used to determine which regions of the screen are considered highlights.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "4.0", editcondition = "bOverride_LocalExposureHighlightThreshold", DisplayName = "Highlight Threshold"))
	float LocalExposureHighlightThreshold;

	/**
	 * Threshold used to determine which regions of the screen are considered shadows.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "4.0", editcondition = "bOverride_LocalExposureShadowThreshold", DisplayName = "Shadow Threshold"))
	float LocalExposureShadowThreshold;

	/**
	 * Local Exposure decomposes luminance of the frame into a base layer and a detail layer.
	 * Value different than 1 will enable local exposure.
	 * This value should be set to 1 in most cases.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "4.0", editcondition = "bOverride_LocalExposureDetailStrength", DisplayName = "Detail Strength"))
	float LocalExposureDetailStrength;

	/**
	 * Local Exposure decomposes luminance of the frame into a base layer and a detail layer.
	 * Blend between bilateral filtered and blurred luminance as the base layer.
	 * Blurred luminance helps preserve image appearance and specular highlights, and reduce ringing.
	 * Good values are usually in the range 0.4 .. 0.6
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_LocalExposureBlurredLuminanceBlend", DisplayName = "Blurred Luminance Blend"))
	float LocalExposureBlurredLuminanceBlend;

	/**
	 * Kernel size (percentage of screen) used to blur frame luminance.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", meta=(UIMin = "0.0", UIMax = "100.0", editcondition = "bOverride_LocalExposureBlurredLuminanceKernelSizePercent", DisplayName = "Blurred Luminance Kernel Size Percent"))
	float LocalExposureBlurredLuminanceKernelSizePercent;

	/** 
	 * Strength of the highlight threshold.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_LocalExposureHighlightThresholdStrength", DisplayName = "Highlight Threshold Strength"))
	float LocalExposureHighlightThresholdStrength;

	/**
	 * Strength of the shadow threshold.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Local Exposure", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_LocalExposureShadowThresholdStrength", DisplayName = "Shadow Threshold Strength"))
	float LocalExposureShadowThresholdStrength;

	/**
	 * Logarithmic adjustment for the local exposure middle grey.
	 * 0: no adjustment, -1:2x darker, -2:4x darker, 1:2x brighter, 2:4x brighter, ...
	 */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Local Exposure", AdvancedDisplay, meta = (UIMin = "-15.0", UIMax = "15.0", editcondition = "bOverride_LocalExposureMiddleGreyBias", DisplayName = "Middle Grey Bias"))
	float LocalExposureMiddleGreyBias;

	/** Brightness scale of the image cased lens flares (linear) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Lens Flares", meta=(UIMin = "0.0", UIMax = "16.0", editcondition = "bOverride_LensFlareIntensity", DisplayName = "Intensity"))
	float LensFlareIntensity;

	/** Tint color for the image based lens flares. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Lens Flares", meta=(editcondition = "bOverride_LensFlareTint", DisplayName = "Tint", HideAlphaChannel))
	FLinearColor LensFlareTint;

	/** Size of the Lens Blur (in percent of the view width) that is done with the Bokeh texture (note: performance cost is radius*radius) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Lens Flares", meta=(UIMin = "0.0", UIMax = "32.0", editcondition = "bOverride_LensFlareBokehSize", DisplayName = "BokehSize"))
	float LensFlareBokehSize;

	/** Minimum brightness the lens flare starts having effect (this should be as high as possible to avoid the performance cost of blurring content that is too dark too see) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Lens Flares", meta=(UIMin = "0.1", UIMax = "32.0", editcondition = "bOverride_LensFlareThreshold", DisplayName = "Threshold"))
	float LensFlareThreshold;

	/** Defines the shape of the Bokeh when the image base lens flares are blurred, cannot be blended */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Lens Flares", meta=(editcondition = "bOverride_LensFlareBokehShape", DisplayName = "BokehShape"))
	TObjectPtr<class UTexture> LensFlareBokehShape;

	/** RGB defines the lens flare color, A it's position. This is a temporary solution. */
	UPROPERTY(EditAnywhere, Category="Lens|Lens Flares", meta=(editcondition = "bOverride_LensFlareTints", DisplayName = "Tints"))
	FLinearColor LensFlareTints[8];

	/** 0..1 0=off/no vignette .. 1=strong vignette */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Image Effects", meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_VignetteIntensity"))
	float VignetteIntensity;

	/** The shape type of the vignette */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(editcondition = "bOverride_VignetteType"))
	EVignetteType VignetteType;

	/** Position of the vignette center in UV space.  */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_VignetteCenter"))
	FVector2f VignetteCenter;

	/** Color multiplied with areas that are masked by the vignette. Default is black. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(editcondition = "bOverride_VignetteColor"))
	FLinearColor VignetteColor;

	/** Scale of the vignette shape in screenspace. Can be used to scale vignette anamorphically. Default is 1. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(ClampMin = "0.0", UIMin = "0.1", UIMax = "2.0", editcondition = "bOverride_VignetteSize"))
	FVector2f VignetteSize;

	/** The softness of the transition from opaque to transparant on the edge of the vignette shape. */
	UPROPERTY(interp, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(ClampMin = "0.001", UIMax = "10.0", editcondition = "bOverride_VignetteSoftness"))
	float VignetteSoftness;

	/** Defines the shape of the vignette when VignetteType is set to Texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Image Effects", AdvancedDisplay, meta=(editcondition = "bOverride_VignetteTexture", DisplayName = "Vignette Texture"))
	TObjectPtr<class UTexture> VignetteTexture;

	/** Controls the strength of image sharpening applied during tonemapping. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Lens|Image Effects", meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "2.0", editcondition = "bOverride_Sharpen"))
	float Sharpen;

	UPROPERTY()
	float GrainJitter_DEPRECATED;

	UPROPERTY()
	float GrainIntensity_DEPRECATED;

	/** 0..1 Film grain intensity to apply. LinearSceneColor *= lerp(1.0, DecodedFilmGrainTexture, FilmGrainIntensity) */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainIntensity"))
	float FilmGrainIntensity;

	/** Control over the grain intensity in the regions of the image considered shadow areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainIntensityShadows"))
	float FilmGrainIntensityShadows;

	/** Control over the grain intensity in the mid-tone region of the image. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainIntensityMidtones"))
	float FilmGrainIntensityMidtones;

	/** Control over the grain intensity in the regions of the image considered highlight areas. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainIntensityHighlights"))
	float FilmGrainIntensityHighlights;

	/** Sets the upper bound used for Film Grain Shadow Intensity. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainShadowsMax"))
	float FilmGrainShadowsMax;
	
	/** Sets the lower bound used for Film Grain Highlight Intensity. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "1.0", editcondition = "bOverride_FilmGrainHighlightsMin"))
	float FilmGrainHighlightsMin;

	/** Sets the upper bound used for Film Grain Highlight Intensity. This value should be larger than HighlightsMin.. Default is 1.0, for backwards compatibility */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "1.0", UIMax = "10.0", editcondition = "bOverride_FilmGrainHighlightsMax"))
	float FilmGrainHighlightsMax;

#if WITH_EDITORONLY_DATA
	/** Controls the size of the film grain. Size of texel of FilmGrainTexture on screen. */
	UE_DEPRECATED(5.8, "Use FilmGrainScale instead")
	UPROPERTY()
	float FilmGrainTexelSize_DEPRECATED;
#endif

	/** Controls the size of the film grain. Size of texel of FilmGrainTexture on screen. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Film Grain", meta = (UIMin = "0.0", UIMax = "4.0", editcondition = "bOverride_FilmGrainScale"))
	FVector2f FilmGrainScale;

	/** Defines film grain texture to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Film Grain", meta = (editcondition = "bOverride_FilmGrainTexture"))
	TObjectPtr<class UTexture2D> FilmGrainTexture;

	/** 0..1 0=off/no ambient occlusion .. 1=strong ambient occlusion, defines how much it affects the non direct lighting after base pass */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_AmbientOcclusionIntensity", DisplayName = "Intensity"))
	float AmbientOcclusionIntensity;

	/** 0..1 0=no effect on static lighting .. 1=AO affects the stat lighting, 0 is free meaning no extra rendering pass */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_AmbientOcclusionStaticFraction", DisplayName = "Static Fraction"))
	float AmbientOcclusionStaticFraction;

	/** >0, in unreal units, bigger values means even distant surfaces affect the ambient occlusion */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", meta=(ClampMin = "0.1", UIMax = "500.0", editcondition = "bOverride_AmbientOcclusionRadius", DisplayName = "Radius"))
	float AmbientOcclusionRadius;

	/** true: AO radius is in world space units, false: AO radius is locked the view space in 400 units */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(editcondition = "bOverride_AmbientOcclusionRadiusInWS", DisplayName = "Radius in WorldSpace"))
	uint32 AmbientOcclusionRadiusInWS:1;

	/** >0, in unreal units, at what distance the AO effect disppears in the distance (avoding artifacts and AO effects on huge object) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "20000.0", editcondition = "bOverride_AmbientOcclusionFadeDistance", DisplayName = "Fade Out Distance"))
	float AmbientOcclusionFadeDistance;
	
	/** >0, in unreal units, how many units before AmbientOcclusionFadeOutDistance it starts fading out */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "20000.0", editcondition = "bOverride_AmbientOcclusionFadeRadius", DisplayName = "Fade Out Radius"))
	float AmbientOcclusionFadeRadius;

	/** >0, in unreal units, how wide the ambient occlusion effect should affect the geometry (in depth), will be removed - only used for non normal method which is not exposed */
	UPROPERTY()
	float AmbientOcclusionDistance_DEPRECATED;

	/** >0, in unreal units, bigger values means even distant surfaces affect the ambient occlusion */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.1", UIMax = "8.0", editcondition = "bOverride_AmbientOcclusionPower", DisplayName = "Power"))
	float AmbientOcclusionPower;

	/** >0, in unreal units, default (3.0) works well for flat surfaces but can reduce details */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "10.0", editcondition = "bOverride_AmbientOcclusionBias", DisplayName = "Bias"))
	float AmbientOcclusionBias;

	/** 0=lowest quality..100=maximum quality, only a few quality levels are implemented, no soft transition */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "100.0", editcondition = "bOverride_AmbientOcclusionQuality", DisplayName = "Quality"))
	float AmbientOcclusionQuality;

	/** Affects the blend over the multiple mips (lower resolution versions) , 0:fully use full resolution, 1::fully use low resolution, around 0.6 seems to be a good value */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.1", UIMax = "1.0", editcondition = "bOverride_AmbientOcclusionMipBlend", DisplayName = "Mip Blend"))
	float AmbientOcclusionMipBlend;

	/** Affects the radius AO radius scale over the multiple mips (lower resolution versions) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.5", UIMax = "4.0", editcondition = "bOverride_AmbientOcclusionMipScale", DisplayName = "Mip Scale"))
	float AmbientOcclusionMipScale;

	/** to tweak the bilateral upsampling when using multiple mips (lower resolution versions) */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Ambient Occlusion", AdvancedDisplay, meta=(ClampMin = "0.0", UIMax = "0.1", editcondition = "bOverride_AmbientOcclusionMipThreshold", DisplayName = "Mip Threshold"))
	float AmbientOcclusionMipThreshold;

	/** How much to blend the current frame with previous frames when using GTAO with temporal accumulation */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Rendering Features|Ambient Occlusion", AdvancedDisplay, meta = (ClampMin = "0.0", UIMax = "0.5", editcondition = "bOverride_AmbientOcclusionTemporalBlendWeight", DisplayName = "Temporal Blend Weight"))
	float AmbientOcclusionTemporalBlendWeight;

	/** Enables ray tracing ambient occlusion. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Ray Tracing Ambient Occlusion", meta = (editcondition = "bOverride_RayTracingAO", DisplayName = "Enabled"))
	uint32 RayTracingAO : 1;

	/** Sets the samples per pixel for ray tracing ambient occlusion. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Ray Tracing Ambient Occlusion", meta = (ClampMin = "1", ClampMax = "65536", editcondition = "bOverride_RayTracingAOSamplesPerPixel", DisplayName = "Samples Per Pixel"))
	int32 RayTracingAOSamplesPerPixel;

	/** Scalar factor on the ray-tracing ambient occlusion score. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Ray Tracing Ambient Occlusion", meta = (ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_RayTracingAOIntensity", DisplayName = "Intensity"))
	float RayTracingAOIntensity;

	/** Defines the world-space search radius for occlusion rays. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Ray Tracing Ambient Occlusion", meta = (ClampMin = "0.0", ClampMax = "10000.0", editcondition = "bOverride_RayTracingAORadius", DisplayName = "Radius"))
	float RayTracingAORadius;

	/** Color grading lookup table intensity. 0 = no intensity, 1=full intensity */
	UPROPERTY(interp, BlueprintReadWrite, Category="Color Grading|Misc", meta=(ClampMin = "0", ClampMax = "1.0", editcondition = "bOverride_ColorGradingIntensity", DisplayName = "Color Grading LUT Intensity"))
	float ColorGradingIntensity;

	/** Look up table texture to use or none of not used*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Color Grading|Misc", meta=(editcondition = "bOverride_ColorGradingLUT", DisplayName = "Color Grading LUT"))
	TObjectPtr<class UTexture> ColorGradingLUT;

	/** When false, post-process depth of field will not be used. When true, DoF may be used depending on lens and scalability settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(editcondition = "bOverride_DepthOfFieldEnabled", DisplayName = "Enabled"))
	bool DepthOfFieldEnabled;

	/** Width of the camera sensor to assume, in mm. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ForceUnits=mm, ClampMin = "0.1", UIMin="0.1", UIMax= "1000.0", editcondition = "bOverride_DepthOfFieldSensorWidth", DisplayName = "Sensor Width (mm)"))
	float DepthOfFieldSensorWidth;

	/** This is the squeeze factor for the DOF, which emulates the properties of anamorphic lenses. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens|Depth of Field", meta = (ClampMin = "1.0", ClampMax = "2.0", editcondition = "bOverride_DepthOfFieldSqueezeFactor", DisplayName = "Squeeze Factor"))
	float DepthOfFieldSqueezeFactor;

	/** Distance in which the Depth of Field effect should be sharp, in unreal units (cm) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", UIMin = "1.0", UIMax = "10000.0", editcondition = "bOverride_DepthOfFieldFocalDistance", DisplayName = "Focal Distance"))
	float DepthOfFieldFocalDistance;

	/** CircleDOF only: Depth blur km for 50% */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.000001", ClampMax = "100.0", editcondition = "bOverride_DepthOfFieldDepthBlurAmount", DisplayName = "Depth Blur km for 50%"))
	float DepthOfFieldDepthBlurAmount;

	/** CircleDOF only: Depth blur radius in pixels at 1920x */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", UIMax = "4.0", editcondition = "bOverride_DepthOfFieldDepthBlurRadius", DisplayName = "Depth Blur Radius"))
	float DepthOfFieldDepthBlurRadius;

	/** For depth of field to use the hair depth for computing circle of confusion size. Otherwise use an interpolated distance between the hair depth and the scene depth based on the hair coverage (default). */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", AdvancedDisplay, meta=(editcondition = "bOverride_DepthOfFieldUseHairDepth", DisplayName = "Use Hair Depth"))
	uint32 DepthOfFieldUseHairDepth:1;

	/** Simulate stretching in blur and bokeh. Positive values for sagittal (swirly bokeh), negative values for tangential. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Lens|Depth of Field", meta = (ClampMin = "-10.0", ClampMax = "10.0", editcondition = "bOverride_DepthOfFieldPetzvalBokeh", DisplayName = "Petzval Bokeh"))
	float DepthOfFieldPetzvalBokeh;

	/** How quickly does the Petzval bokeh effect increase towards the edge of the image */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", UIMax = "10.0", editcondition = "bOverride_DepthOfFieldPetzvalBokehFalloff", DisplayName = "Petzval Bokeh Falloff"))
	float DepthOfFieldPetzvalBokehFalloff;

	/** Box, centered on screen, around which the Petzval effect is applied. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_DepthOfFieldPetzvalExclusionBoxExtents", DisplayName = "Petzval Bokeh Exclusion Box"))
	FVector2f DepthOfFieldPetzvalExclusionBoxExtents;

	/** Corner radius */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_DepthOfFieldPetzvalExclusionBoxRadius", DisplayName = "Petzval Bokeh Exclusion Box Radius"))
	float DepthOfFieldPetzvalExclusionBoxRadius;

	/** Amount to scale the output viewport's aspect ratio by when computing depth of field properties.  */
	UPROPERTY(Interp, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", editcondition = "bOverride_DepthOfFieldAspectRatioScalar", DisplayName = "Aspect Ratio Scalar"))
	float DepthOfFieldAspectRatioScalar;
		
	/** The lens barrel creates vignetting and occludes bokeh, i.e. cat's eye bokeh */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", UIMax = "100.0", editcondition = "bOverride_DepthOfFieldBarrelRadius", DisplayName = "Barrel Radius"))
	float DepthOfFieldBarrelRadius;

	/** The lens barrel creates vignetting and occludes bokeh, i.e. cat's eye bokeh */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Depth of Field", meta=(ClampMin = "0.0", UIMax = "150.0", editcondition = "bOverride_DepthOfFieldBarrelLength", DisplayName = "Barrel Length"))
	float DepthOfFieldBarrelLength;

	/** Panels around the front of the lens barrel that occlude bokeh */
	UPROPERTY(EditAnywhere, Category="Lens|Depth of Field", meta=(editcondition = "bOverride_DepthOfFieldMatteBoxFlags", DisplayName = "Matte Box Flags"))
	FMatteBoxFlag DepthOfFieldMatteBoxFlags[3];
	
	/** Artificial region where all content is in focus, starting after DepthOfFieldFocalDistance, in unreal units  (cm) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(UIMin = "0.0", UIMax = "10000.0", editcondition = "bOverride_DepthOfFieldFocalRegion", DisplayName = "Focal Region"))
	float DepthOfFieldFocalRegion;

	/** To define the width of the transition region next to the focal region on the near side (cm) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(UIMin = "0.0", UIMax = "10000.0", editcondition = "bOverride_DepthOfFieldNearTransitionRegion", DisplayName = "Near Transition Region"))
	float DepthOfFieldNearTransitionRegion;

	/** To define the width of the transition region next to the focal region on the near side (cm) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(UIMin = "0.0", UIMax = "10000.0", editcondition = "bOverride_DepthOfFieldFarTransitionRegion", DisplayName = "Far Transition Region"))
	float DepthOfFieldFarTransitionRegion;

	/** SM5: BokehDOF only: To amplify the depth of field effect (like aperture)  0=off 
	    ES3_1: Used to blend DoF. 0=off
	*/
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(ClampMin = "0.0", ClampMax = "2.0", editcondition = "bOverride_DepthOfFieldScale", DisplayName = "Scale"))
	float DepthOfFieldScale;

	/** Gaussian only: Maximum size of the Depth of Field blur (in percent of the view width) (note: performance cost scales with size) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(UIMin = "0.0", UIMax = "32.0", editcondition = "bOverride_DepthOfFieldNearBlurSize", DisplayName = "Near Blur Size"))
	float DepthOfFieldNearBlurSize;

	/** Gaussian only: Maximum size of the Depth of Field blur (in percent of the view width) (note: performance cost scales with size) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", meta=(UIMin = "0.0", UIMax = "32.0", editcondition = "bOverride_DepthOfFieldFarBlurSize", DisplayName = "Far Blur Size"))
	float DepthOfFieldFarBlurSize;

	/** Occlusion tweak factor 1 (0.18 to get natural occlusion, 0.4 to solve layer color leaking issues) */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_DepthOfFieldOcclusion", DisplayName = "Occlusion"))
	float DepthOfFieldOcclusion;

	/** Artificial distance to allow the skybox to be in focus (e.g. 200000), <=0 to switch the feature off, only for GaussianDOF, can cost performance */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", AdvancedDisplay, meta=(ClampMin = "0.0", ClampMax = "200000.0", editcondition = "bOverride_DepthOfFieldSkyFocusDistance", DisplayName = "Sky Distance"))
	float DepthOfFieldSkyFocusDistance;

	/** Artificial circular mask to (near) blur content outside the radius, only for GaussianDOF, diameter in percent of screen width, costs performance if the mask is used, keep Feather can Radius on default to keep it off */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category="Lens|Mobile Depth of Field", AdvancedDisplay, meta=(UIMin = "0.0", UIMax = "100.0", editcondition = "bOverride_DepthOfFieldVignetteSize", DisplayName = "Vignette Size"))
	float DepthOfFieldVignetteSize;

	/** Strength of motion blur, 0:off */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Motion Blur", meta=(ClampMin = "0.0", ClampMax = "1.0", editcondition = "bOverride_MotionBlurAmount", DisplayName = "Amount"))
	float MotionBlurAmount;
	/** max distortion caused by motion blur, in percent of the screen width, 0:off */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Motion Blur", meta=(ClampMin = "0.0", ClampMax = "100.0", editcondition = "bOverride_MotionBlurMax", DisplayName = "Max"))
	float MotionBlurMax;
	/**
	 * Defines the target FPS for motion blur. Makes motion blur independent of actual frame rate and relative
	 * to the specified target FPS instead. Higher target FPS results in shorter frames, which means shorter
	 * shutter times and less motion blur. Lower FPS means more motion blur. A value of zero makes the motion
	 * blur dependent on the actual frame rate.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Motion Blur", meta=(ClampMin = "0", ClampMax = "120", editcondition = "bOverride_MotionBlurTargetFPS", DisplayName = "Target FPS"))
	int32 MotionBlurTargetFPS;

	/** The minimum projected screen radius for a primitive to be drawn in the velocity pass, percentage of screen width. smaller numbers cause more draw calls, default: 4% */
	UPROPERTY(interp, BlueprintReadWrite, Category="Rendering Features|Motion Blur", meta=(ClampMin = "0.0", UIMax = "100.0", editcondition = "bOverride_MotionBlurPerObjectSize", DisplayName = "Per Object Size"))
	float MotionBlurPerObjectSize;

	UPROPERTY()
	float LPVIntensity_DEPRECATED;

	UPROPERTY()
	float LPVVplInjectionBias_DEPRECATED;

	UPROPERTY()
	float LPVSize_DEPRECATED;

	UPROPERTY()
	float LPVSecondaryOcclusionIntensity_DEPRECATED;

	UPROPERTY()
	float LPVSecondaryBounceIntensity_DEPRECATED;

	UPROPERTY()
	float LPVGeometryVolumeBias_DEPRECATED;

	UPROPERTY()
	float LPVEmissiveInjectionIntensity_DEPRECATED;

	UPROPERTY()
	float LPVDirectionalOcclusionIntensity_DEPRECATED;

	UPROPERTY()
	float LPVDirectionalOcclusionRadius_DEPRECATED;

	UPROPERTY()
	float LPVDiffuseOcclusionExponent_DEPRECATED;

	UPROPERTY()
	float LPVSpecularOcclusionExponent_DEPRECATED;

	UPROPERTY()
	float LPVDiffuseOcclusionIntensity_DEPRECATED;

	UPROPERTY()
	float LPVSpecularOcclusionIntensity_DEPRECATED;

	/** Sets the translucency type */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (editcondition = "bOverride_TranslucencyType", DisplayName = "Translucency Method"))
	ETranslucencyType TranslucencyType;

	/** Sets the maximum roughness until which ray tracing translucency will be visible (lower value is faster). Translucency contribution is smoothly faded when close to roughness threshold. This parameter behaves similarly to ScreenSpaceReflectionMaxRoughness. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (ClampMin = "0.01", ClampMax = "1.0", editcondition = "bOverride_RayTracingTranslucencyMaxRoughness", DisplayName = "Max Roughness", TranslucencyType = "RayTraced_Deprecated"))
	float RayTracingTranslucencyMaxRoughness;

	/** Sets the maximum number of ray tracing refraction rays. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (ClampMin = "0", ClampMax = "50", editcondition = "bOverride_RayTracingTranslucencyRefractionRays", DisplayName = "Max. Refraction Rays", TranslucencyType = "RayTraced_Deprecated"))
	int32 RayTracingTranslucencyRefractionRays;

	/** Sets the samples per pixel for ray traced translucency. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (ClampMin = "1", ClampMax = "65536", editcondition = "bOverride_RayTracingTranslucencySamplesPerPixel", DisplayName = "Samples Per Pixel", TranslucencyType = "RayTraced_Deprecated"))
	int32 RayTracingTranslucencySamplesPerPixel;

	/** Maximum number of hit events allowed on primary ray paths */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (ClampMin = "1", ClampMax = "64", editcondition = "bOverride_RayTracingTranslucencyMaxPrimaryHitEvents", DisplayName = "Max. Primary Hit Events", TranslucencyType = "RayTraced"))
	int32 RayTracingTranslucencyMaxPrimaryHitEvents;

	/** Maximum number of hit events allowed on secondary ray paths */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (ClampMin = "0", ClampMax = "64", editcondition = "bOverride_RayTracingTranslucencyMaxSecondaryHitEvents", DisplayName = "Max. Secondary Hit Events", TranslucencyType = "RayTraced"))
	int32 RayTracingTranslucencyMaxSecondaryHitEvents;

	/** Sets the translucency shadows type. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (editcondition = "bOverride_RayTracingTranslucencyShadows", DisplayName = "Shadows", TranslucencyType = "RayTraced_Deprecated"))
	EReflectedAndRefractedRayTracedShadows RayTracingTranslucencyShadows;

	/** Sets whether refraction should be enabled or not (if not rays will not scatter and only travel in the same direction as before the intersection event). */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (editcondition = "bOverride_RayTracingTranslucencyRefraction", DisplayName = "Refraction", TranslucencyType = "RayTraced_Deprecated"))
	uint8 RayTracingTranslucencyRefraction : 1;

	/** Whether to use ray traced refraction which currently doesn't work well with rough refraction or simulate it using a screen space effect */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Rendering Features|Translucency", meta = (editcondition = "bOverride_RayTracingTranslucencyUseRayTracedRefraction", DisplayName = "Use Ray Traced Refraction", TranslucencyType = "RayTraced"))
	uint8 RayTracingTranslucencyUseRayTracedRefraction : 1;

	// Path Tracing
	/** Sets the path tracing maximum bounces */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (ClampMin = "0", ClampMax = "100", editcondition = "bOverride_PathTracingMaxBounces", DisplayName = "Max. Bounces"))
	int32 PathTracingMaxBounces;

	/** Sets the samples per pixel for the path tracer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (ClampMin = "1", UIMax = "65536", editcondition = "bOverride_PathTracingSamplesPerPixel", DisplayName = "Samples Per Pixel"))
	int32 PathTracingSamplesPerPixel;

	/** Sets the maximum intensity of indirect samples to reduce fireflies. Lowering this value reduces noise at the expense of accuracy. Increasing it is more accurate but may lead to more noise. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (ClampMin = "1.0", ClampMax = "65504.0", editcondition = "bOverride_PathTracingMaxPathIntensity", DisplayName = "Max Path Intensity"))
	float PathTracingMaxPathIntensity;

	/** Should emissive materials contribute to scene lighting? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (editcondition = "bOverride_PathTracingEnableEmissiveMaterials", DisplayName = "Emissive Materials"))
	uint32 PathTracingEnableEmissiveMaterials : 1;

	/** Enables a reference quality depth-of-field which replaces the post-process effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (editcondition = "bOverride_PathTracingEnableReferenceDOF", DisplayName = "Reference Depth Of Field"))
	uint32 PathTracingEnableReferenceDOF : 1;

	/** Enables path tracing in the atmosphere instead of baking the sky atmosphere contribution into a skylight. Any skylight present in the scene will be automatically ignored when this is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (editcondition = "bOverride_PathTracingEnableReferenceAtmosphere", DisplayName = "Reference Atmosphere"))
	uint32 PathTracingEnableReferenceAtmosphere : 1;

	/** Run the currently loaded denoiser plugin on the last sample to remove noise from the output. Has no effect if a plug-in is not loaded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing", meta = (editcondition = "bOverride_PathTracingEnableDenoiser", DisplayName = "Denoiser"))
	uint32 PathTracingEnableDenoiser : 1;

	/** Should the render include directly visible emissive elements? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeEmissive", DisplayName = "Emissive"))
	uint32 PathTracingIncludeEmissive : 1;

	/** Should the render include diffuse lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeDiffuse", DisplayName = "Diffuse"))
	uint32 PathTracingIncludeDiffuse : 1;

	/** Should the render include indirect diffuse lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeIndirectDiffuse", DisplayName = "Indirect Diffuse"))
	uint32 PathTracingIncludeIndirectDiffuse : 1;

	/** Should the render include specular lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeSpecular", DisplayName = "Specular"))
	uint32 PathTracingIncludeSpecular : 1;

	/** Should the render include indirect specular lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeIndirectSpecular", DisplayName = "Indirect Specular"))
	uint32 PathTracingIncludeIndirectSpecular : 1;

	/** Should the render include volume lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeVolume", DisplayName = "Volume"))
	uint32 PathTracingIncludeVolume : 1;

	/** Should the render include volume lighting contributions? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Path Tracing|Lighting Components", meta = (editcondition = "bOverride_PathTracingIncludeIndirectVolume", DisplayName = "Indirect Volume"))
	uint32 PathTracingIncludeIndirectVolume : 1;

	UPROPERTY()
	float LPVFadeRange_DEPRECATED;

	UPROPERTY()
	float LPVDirectionalOcclusionFadeRange_DEPRECATED;

	UPROPERTY()
	float ScreenPercentage_DEPRECATED;

	/** Per-view user flags accessible in materials via TestPostVolumeUserFlag node, allowing per-view overrides of material behavior. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering Features", meta = (editcondition = "bOverride_UserFlags", DisplayName = "User Flags"))
	int32 UserFlags;



	// Note: Adding properties before this line require also changes to the OverridePostProcessSettings() function and 
	// FPostProcessSettings constructor and possibly the SetBaseValues() method.
	// -----------------------------------------------------------------------
	
	/**
	 * Allows custom post process materials to be defined, using a MaterialInstance with the same Material as its parent to allow blending.
	 * For materials this needs to be the "PostProcess" domain type. This can be used for any UObject object implementing the IBlendableInterface (e.g. could be used to fade weather settings).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Rendering Features", meta=( Keywords="PostProcess", DisplayName = "Post Process Materials" ))
	FWeightedBlendables WeightedBlendables;

#if WITH_EDITORONLY_DATA
	/**
	 * For editor material preview windows, we need to support visualizing the output of a blendable that writes to a UserSceneTexture.  Stores
	 * a pointer to a blendable that's being previewed, forcing its output to write to SceneColor instead of the UserSceneTexture, making it visible.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UObject> PreviewBlendable;

	// for backwards compatibility
	UPROPERTY()
	TArray<TObjectPtr<UObject>> Blendables_DEPRECATED;

	// for backwards compatibility
	void OnAfterLoad()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for(int32 i = 0, count = Blendables_DEPRECATED.Num(); i < count; ++i)
		{
			if(Blendables_DEPRECATED[i])
			{
				FWeightedBlendable Element(1.0f, Blendables_DEPRECATED[i]);
				WeightedBlendables.Array.Add(Element);
			}
		}
		Blendables_DEPRECATED.Empty();

		if (bOverride_BloomConvolutionPreFilter_DEPRECATED)
		{
			bOverride_BloomConvolutionPreFilterMin  = bOverride_BloomConvolutionPreFilter_DEPRECATED;
			bOverride_BloomConvolutionPreFilterMax  = bOverride_BloomConvolutionPreFilter_DEPRECATED;
			bOverride_BloomConvolutionPreFilterMult = bOverride_BloomConvolutionPreFilter_DEPRECATED;
		}
		if (BloomConvolutionPreFilter_DEPRECATED.X > -1.f)
		{
			BloomConvolutionPreFilterMin = BloomConvolutionPreFilter_DEPRECATED.X;
			BloomConvolutionPreFilterMax = BloomConvolutionPreFilter_DEPRECATED.Y;
			BloomConvolutionPreFilterMult = BloomConvolutionPreFilter_DEPRECATED.Z;
		}

		if (bOverride_LocalExposureContrastScale_DEPRECATED)
		{
			bOverride_LocalExposureHighlightContrastScale = bOverride_LocalExposureContrastScale_DEPRECATED;
			bOverride_LocalExposureShadowContrastScale = bOverride_LocalExposureContrastScale_DEPRECATED;
		}
		if (LocalExposureContrastScale_DEPRECATED != 1.0f)
		{
			LocalExposureHighlightContrastScale = LocalExposureContrastScale_DEPRECATED;
			LocalExposureShadowContrastScale = LocalExposureContrastScale_DEPRECATED;
		}

		if (bOverride_FilmGrainTexelSize_DEPRECATED)
		{
			bOverride_FilmGrainScale = true;
			FilmGrainScale = FVector2f(FilmGrainTexelSize_DEPRECATED, FilmGrainTexelSize_DEPRECATED);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	// Adds an Blendable (implements IBlendableInterface) to the array of Blendables (if it doesn't exist) and update the weight
	// @param InBlendableObject silently ignores if no object is referenced
	// @param 0..1 InWeight, values outside of the range get clampled later in the pipeline
	void AddBlendable(TScriptInterface<IBlendableInterface> InBlendableObject, float InWeight)
	{
		// update weight, if the Blendable is already in the array
		if(UObject* Object = InBlendableObject.GetObject())
		{
			for (int32 i = 0, count = WeightedBlendables.Array.Num(); i < count; ++i)
			{
				if (WeightedBlendables.Array[i].Object == Object)
				{
					WeightedBlendables.Array[i].Weight = InWeight;
					// We assumes we only have one
					return;
				}
			}

			// add in the end
			WeightedBlendables.Array.Add(FWeightedBlendable(InWeight, Object));
		}
	}

	// removes one or multiple blendables from the array
	void RemoveBlendable(TScriptInterface<IBlendableInterface> InBlendableObject)
	{
		if(UObject* Object = InBlendableObject.GetObject())
		{
			for (int32 i = 0, count = WeightedBlendables.Array.Num(); i < count; ++i)
			{
				if (WeightedBlendables.Array[i].Object == Object)
				{
					// this might remove multiple
					WeightedBlendables.Array.RemoveAt(i);
					--i;
					--count;
				}
			}
		}
	}

	// good start values for a new volume, by default no value is overriding
	ENGINE_API FPostProcessSettings();
	
	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPostProcessSettings(const FPostProcessSettings&) = default;
	FPostProcessSettings(FPostProcessSettings&&) = default;
	FPostProcessSettings& operator=(const FPostProcessSettings&) = default;
	FPostProcessSettings& operator=(FPostProcessSettings&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
		* Used to define the values before any override happens.
		* Should be as neutral as possible.
		*/		
	void SetBaseValues()
	{
		*this = FPostProcessSettings();

		AmbientCubemapIntensity = 0.0f;
		ColorGradingIntensity = 0.0f;
	}


	// Default number of blade of the diaphragm to simulate in depth of field.
	static constexpr int32 kDefaultDepthOfFieldBladeCount = 5;

	// Get the default post-process settings
	static ENGINE_API const FPostProcessSettings& GetDefault();
	
#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif

}; // struct FPostProcessSettings

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FPostProcessSettings>
	: public TStructOpsTypeTraitsBase2<FPostProcessSettings>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};
#endif

UCLASS()
class UScene : public UObject
{
	GENERATED_UCLASS_BODY()
};
