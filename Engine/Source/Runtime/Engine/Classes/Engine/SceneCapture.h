// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for all SceneCapture actors
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "SceneCapture.generated.h"

UCLASS(abstract, hidecategories=(Collision, Attachment, Actor), MinimalAPI)
class ASceneCapture : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** To display the 3d camera in the editor. */
	UE_DEPRECATED(4.22, "SceneCapture's mesh and frustum components should now be accessed through the SceneCaptureComponent instead of the Actor")
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> MeshComp_DEPRECATED;

	UPROPERTY(Category = Components, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USceneComponent> SceneComponent;

public:
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

	/** Returns MeshComp subobject **/
	UE_DEPRECATED(4.22, "SceneCapture's mesh and frustum components should now be accessed through the SceneCaptureComponent instead of the Actor")
	class UStaticMeshComponent* GetMeshComp() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MeshComp_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
};



