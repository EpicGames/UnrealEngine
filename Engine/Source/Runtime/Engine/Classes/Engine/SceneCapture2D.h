// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * 
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SceneCapture.h"
#include "SceneCapture2D.generated.h"

UCLASS(hidecategories=(Collision, Material, Attachment, Actor), MinimalAPI)
class ASceneCapture2D : public ASceneCapture
{
	GENERATED_UCLASS_BODY()

private:
	/** Scene capture component. */
	UPROPERTY(Category = DecalActor, VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"))
	TObjectPtr<class USceneCaptureComponent2D> CaptureComponent2D;

public:
	UFUNCTION(BlueprintCallable, Category="Rendering")
	void OnInterpToggle(bool bEnable);

	ENGINE_API virtual void CalcCamera(float DeltaTime, struct FMinimalViewInfo& OutMinimalViewInfo) override;

	/** Returns CaptureComponent2D subobject **/
	class USceneCaptureComponent2D* GetCaptureComponent2D() const { return CaptureComponent2D; }
};
