// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * 
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SceneCapture.h"
#include "SceneCaptureCube.generated.h"

UCLASS(hidecategories = (Collision, Material, Attachment, Actor), MinimalAPI)
class ASceneCaptureCube : public ASceneCapture
{
	GENERATED_UCLASS_BODY()

private:
	/** Scene capture component. */
	UPROPERTY(Category = DecalActor, VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"))
	TObjectPtr<class USceneCaptureComponentCube> CaptureComponentCube;

public:
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void OnInterpToggle(bool bEnable);

	/** Returns CaptureComponentCube subobject **/
	class USceneCaptureComponentCube* GetCaptureComponentCube() const { return CaptureComponentCube; }
};



