// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for FViewportClients that are also UObjects
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameplayViewportClient.h"
#include "ScriptViewportClient.generated.h"

#define UE_API ENGINE_API

UCLASS(Transient, MinimalAPI)
class UScriptViewportClient : public UObject, public FGameplayViewportClient
{
	GENERATED_BODY()
public:
	UE_API explicit UScriptViewportClient(const FObjectInitializer& ObjectInitializer);

	UE_API virtual UWorld* GetWorld() const override;
};

#undef UE_API
