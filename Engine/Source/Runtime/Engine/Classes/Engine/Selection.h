// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
// TODO: Deprecate this adapter include in favor of using the UnrealEd version directly
#include "Editor/UnrealEd/Public/Selection.h"
#endif
