// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "ServerStatReplicator.generated.h"

/**
 * Class used to replicate server "stat net" data over. For server only values, the client data is
 * is overwritten when bUpdateStatNet == true. For data that both the client and server set, the server
 * data will only overwrite if bUpdateStatNet == true && bOverwriteClientStats == true. 
 */
UCLASS(BlueprintType, MinimalAPI)
class AServerStatReplicator :
	public AInfo
{
	GENERATED_UCLASS_BODY()

public:
	/** Whether to update stat net with data from the server or not */
	UPROPERTY(EditAnywhere, Category=ServerStats)
	bool bUpdateStatNet;

	/** Whether to overwrite client data stat net with data from the server or not */
	UPROPERTY(EditAnywhere, Category=ServerStats)
	bool bOverwriteClientStats;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 Channels;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InRate;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutRate;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 MaxPacketOverhead;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InRateClientMax;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InRateClientMin;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InRateClientAvg;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InPacketsClientMax;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InPacketsClientMin;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InPacketsClientAvg;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutRateClientMax;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutRateClientMin;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutRateClientAvg;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutPacketsClientMax;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutPacketsClientMin;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutPacketsClientAvg;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NetNumClients;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InPackets;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutPackets;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InBunches;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutBunches;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 OutLoss;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 InLoss;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 VoiceBytesSent;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 VoiceBytesRecv;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 VoicePacketsSent;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 VoicePacketsRecv;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 PercentInVoice;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 PercentOutVoice;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumActorChannels;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumConsideredActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 PrioritizedActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumRelevantActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumRelevantDeletedActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumReplicatedActorAttempts;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumReplicatedActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumNetActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumDormantActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumInitiallyDormantActors;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumNetGUIDsAckd;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumNetGUIDsPending;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NumNetGUIDsUnAckd;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 ObjPathBytes;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NetGUIDOutRate;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NetGUIDInRate;

	/** @see Network stats counters in EngineStats.h */
	UPROPERTY(Replicated)
	uint32 NetSaturated;

	ENGINE_API virtual void Tick(float DeltaSeconds) override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void Destroyed() override;
};
