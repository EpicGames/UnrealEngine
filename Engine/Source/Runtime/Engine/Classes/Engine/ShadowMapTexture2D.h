// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture2D.h"
#include "ShadowMapTexture2D.generated.h"

UCLASS(MinimalAPI)
class UShadowMapTexture2D : public UTexture2D
{
	GENERATED_UCLASS_BODY()
	
	/** Bit-field with shadowmap flags. */
	UPROPERTY()
	TEnumAsByte<enum EShadowMapFlags> ShadowmapFlags;
};
