// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Async/TransactionallySafeRecursiveMutex.h"
#include "Misc/Guid.h"
#include "GameFramework/Actor.h"
#include "SimpleConstructionScript.generated.h"

class USCS_Node;
class FStaticMeshComponentBulkReregisterContext;

UCLASS(MinimalAPI)
class USimpleConstructionScript : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Suffix used for component template object name. */
	ENGINE_API static const FString ComponentTemplateNameSuffix;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	void PreloadChain();

	/** Ensures that all root node parent references are still valid and clears the reference if not */
	ENGINE_API void FixupRootNodeParentReferences();

	/** Helper method to register instanced components post-construction */
	static void RegisterInstancedComponent(UActorComponent* Component);

	/**
	 * Execute this script on the supplied actor, creating components 
	 *
	 * @param Actor					The actor instance to execute the script on.
	 * @param NativeSceneComponents	The set of native scene components from the actor that SCS nodes can parent to when instanced.
	 * @param RootTransform			The transform to apply to the root scene component of the actor instance if defined in this script.
	 * @param RootRelativeRotationCache Optional rotation cache to use when applying the transform to the root component.
	 * @param bIsDefaultTransform	Indicates whether or not the given transform is a "default" transform, in which case it can be overridden by template defaults.
	 */
	void ExecuteScriptOnActor(AActor* Actor, const TInlineComponentArray<USceneComponent*>& NativeSceneComponents, const FTransform& RootTransform, const FRotationConversionCache* RootRelativeRotationCache, bool bIsDefaultTransform, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::OverrideRootScale);

	/** Create the map from names to SCS_Nodes to improve FindSCSNode performance during construction script execution */
	ENGINE_API void CreateNameToSCSNodeMap();

	/** Remove the map from names to SCS_Nodes */
	ENGINE_API void RemoveNameToSCSNodeMap();

#if WITH_EDITOR
	/** Return the Blueprint associated with this SCS instance */
	ENGINE_API class UBlueprint* GetBlueprint() const;

	/** 
	 * Helper function to find the current scene root component template and optionally the owning SCS node 
	 * @param bShouldUseDefaultRoot  Whether the default scene root, if already in the root node set, should be used (note this will return null in that case because of the expectations of other calling code)
	 * @param OutSCSNode             If not-null, the SCSNode for the scene root component will be returned in this variable
	 */
	ENGINE_API USceneComponent* GetSceneRootComponentTemplate(bool bShouldUseDefaultRoot = false, USCS_Node** OutSCSNode = nullptr) const;

	/** Saves the current state of SimpleConstructionScript and its nodes to the transaction buffer. */
	ENGINE_API void SaveToTransactionBuffer();
#endif

	/** Return the Blueprint associated with this SCS instance */
	ENGINE_API class UClass* GetOwnerClass() const;

	/** Return the parent class of the SCS owner class */
	ENGINE_API class UClass* GetParentClass() const;

#if WITH_EDITOR
	/** Return all nodes in tree as a flat list */
	ENGINE_API const TArray<USCS_Node*>& GetAllNodes() const;
#else
	const TArray<USCS_Node*>& GetAllNodes() const { return AllNodes; }
#endif	

	/** Provides read-only access to the root node set */
	const TArray<USCS_Node*>& GetRootNodes() const { return RootNodes; }

	/** Provides read-only access to the default scene root node */
	const class USCS_Node* GetDefaultSceneRootNode() const { return DefaultSceneRootNode; }

	class USCS_Node* GetDefaultSceneRootNode() { return DefaultSceneRootNode; }

	/** Adds this node to the root set */
	ENGINE_API void AddNode(USCS_Node* Node);

	/** 
	 * Remove this node from the script
	 * @param Node							The node to remove.
	 * @param bValidateSceneRootNodes		Whether to validate scene root nodes after removing the node. Set to false if there are other operations that need to occur before validating.
	 */
	ENGINE_API void RemoveNode(USCS_Node* Node, bool bValidateSceneRootNodes = true);

	/** Remove this node from the script and if it's the root, promote its first child to replace it */
	ENGINE_API void RemoveNodeAndPromoteChildren(USCS_Node* Node);

	/** Find the parent node of this one. Returns NULL if node is not in tree or if is root */
	ENGINE_API USCS_Node* FindParentNode(USCS_Node* InNode) const;

	/** Find the SCS_Node node by name and return it if found */
	ENGINE_API USCS_Node* FindSCSNode(const FName InName) const;

	/** Find the SCS_Node node by name and return it if found */
	ENGINE_API USCS_Node* FindSCSNodeByGuid(const FGuid Guid) const;

	/** Checks the root node set for scene components and ensures that it is valid (e.g. after a removal) */
	ENGINE_API void ValidateSceneRootNodes();

#if WITH_EDITOR
	/** Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object. */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	FStaticMeshComponentBulkReregisterContext* GetReregisterContext() const { return ReregisterContext; }

private:
	mutable UE::FTransactionallySafeRecursiveMutex Mutex;

	/** Root nodes of the construction script */
	UPROPERTY()
	TArray<TObjectPtr<class USCS_Node>> RootNodes;

	/** All nodes that exist in the hierarchy of this SimpleConstructionScript */
	UPROPERTY()
	TArray<TObjectPtr<class USCS_Node>> AllNodes;

	/** Default scene root node; used when no other nodes are available to use as the root */
	UPROPERTY()
	TObjectPtr<class USCS_Node> DefaultSceneRootNode;

#if WITH_EDITORONLY_DATA
	/** (DEPRECATED) Root node of the construction script */
	UPROPERTY()
	TObjectPtr<class USCS_Node> RootNode_DEPRECATED;

	/** (DEPRECATED) Actor Component based nodes are stored here.  They cannot be in the tree hierarchy */
	UPROPERTY()
	TArray<TObjectPtr<USCS_Node>> ActorComponentNodes_DEPRECATED;
#endif

	/** Helper function to find an appropriate child node index that can be promoted to the parent's level */
	int32 FindPromotableChildNodeIndex(USCS_Node* InParentNode) const;

	/** 
	 * Helper function to repair invalid SCS hierarchies (like when a 
	 * Blueprint's super class has had all it components removed, and one of  
	 * this Blueprint's components is promoted to scene root). 
	 */
	void FixupSceneNodeHierarchy();

#if WITH_EDITOR
	/** Helper function for generating list of currently used names */
	void GenerateListOfExistingNames(TSet<FName>& CurrentNames) const;

	/** Helper function to perform actions that all node creation functions have in common */
	USCS_Node* CreateNodeImpl(UActorComponent* NewComponentTemplate, FName ComponentVariableName);

public:
	/** Helper function for generating a name for a new node, DesiredName can be used to optionally request a name, ComponentClass is mandatory */
	ENGINE_API FName GenerateNewComponentName(const UClass* ComponentClass, FName DesiredName = NAME_None) const;

	/** Creates a new SCS node using the given class to create the component template */
	ENGINE_API USCS_Node* CreateNode(class UClass* NewComponentClass, FName NewComponentVariableName = NAME_None);

	/** Creates a new SCS node using the given component template instance */
	ENGINE_API USCS_Node* CreateNodeAndRenameComponent(UActorComponent* ExistingTemplate);

	/** Ensures that all nodes in the SCS have valid names for compilation/replication */
	ENGINE_API void ValidateNodeVariableNames(class FCompilerResultsLog& MessageLog);

	/** Ensures that all nodes in the SCS have valid templates */
	ENGINE_API void ValidateNodeTemplates(class FCompilerResultsLog& MessageLog);

	/** Called by the SCS editor to clear all SCS editor component references */
	ENGINE_API void ClearEditorComponentReferences();

	/** Called by the SCS editor to prepare for constructing editable components */
	ENGINE_API void BeginEditorComponentConstruction();

	/** Called by the SCS editor to clean up after constructing editable components */
	ENGINE_API void EndEditorComponentConstruction();

	/** Find out whether or not we're constructing components in the SCS editor */
	bool IsConstructingEditorComponents() const
	{
		return bIsConstructingEditorComponents;
	}

	/** Called by the SCS editor to set the actor instance for component editing */
	void SetComponentEditorActorInstance(class AActor* InActor)
	{
		EditorActorInstancePtr = InActor;
	}

	/** Gets the SCS editor actor instance that's being used for component editing */
	class AActor* GetComponentEditorActorInstance() const
	{
		return EditorActorInstancePtr.Get();
	}

private:
	/** Actor instance used to host components in the SCS editor */
	TWeakObjectPtr<class AActor> EditorActorInstancePtr;

	/** True if we're constructing editable components in the SCS editor */
	bool bIsConstructingEditorComponents;
#endif

	/** Reference count to allow nesting of Create/RemoveNameToSCSNodeMap calls */
	int32 NameToSCSNodeMapRefCount;

	/** Quick lookup from name to SCS Node when executing the script */
	TMap<FName, USCS_Node*> NameToSCSNodeMap;

	/** Reregister context, which allows bulk handling of render commands */
	FStaticMeshComponentBulkReregisterContext* ReregisterContext;
	friend FStaticMeshComponentBulkReregisterContext;

	friend struct FSCSAllNodesHelper;
};

/** Helper struct to allow USCS_Node to manage USimpleConstructionScript's AllNodes array */
struct FSCSAllNodesHelper
{
private:
	static void Remove(USimpleConstructionScript* SCS, USCS_Node* SCSNode);
	static void Add(USimpleConstructionScript* SCS, USCS_Node* SCSNode);

	friend class USCS_Node;
};
