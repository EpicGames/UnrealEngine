// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Contains the shared data that is used by all SkeletalMeshComponents (instances).
 */

#include "Animation/MorphTarget.h"
#include "Animation/NodeMappingProviderInterface.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "Engine/StreamableRenderAsset.h"
#include "EngineDefines.h"
#include "EngineTypes.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"
#include "ReferenceSkeleton.h"
#include "RenderCommandFence.h"
#include "SkeletalMeshSampling.h"
#include "SkeletalMeshSourceModel.h"
#include "SkinnedAsset.h"
#include "SkinnedAssetCommon.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SkeletalMesh.generated.h"

#define UE_API ENGINE_API

class UAnimInstance;
class UAnimSequence;
class UAssetUserData;
class UBlueprint;
class UBodySetup;
class UClothingAssetBase;
class UMeshDeformer;
class UNodeMappingContainer;
class UPhysicsAsset;
class USkeletalMeshEditorData;
class FSkeletalMeshImportData;
class FSkeletalMeshLODModel;
class FSkeletalMeshLODRenderData;
class FSkeletalMeshModel;
class FSkeletalMeshRenderData;
class USkeletalMeshSocket;
class FSkeletalMeshUpdate;
class FSkinnedAssetAsyncBuildScope;
class FSkinnedAssetAsyncBuildWorker;
class FSkinnedAssetCompilationContext;
class FSkinnedAssetBuildContext;
class FSkinnedAssetPostLoadContext;
class USkeletalMeshLODSettings;
class USkeleton;
class UThumbnailInfo;
class FEvent;
class USkeletalMesh;
struct FMeshUVChannelInfo;
struct FSkeletalMaterial;
struct FSkinWeightProfileInfo;
class FSkinWeightVertexBuffer;
enum class ESkeletalMeshGeoImportVersions : uint8;
enum class ESkeletalMeshSkinningImportVersions : uint8;
struct FMeshDescription;

/*-----------------------------------------------------------------------------
	Async Skeletal Mesh Compilation
-----------------------------------------------------------------------------*/

UENUM()
enum class ESkeletalMeshAsyncProperties : uint64
{
	None = 0,
	Materials = 1 << 0,
	Skeleton = 1 << 1,
	RefSkeleton = 1 << 2,
	RetargetBasePose = 1 << 3,
	RefBasesInvMatrix = 1 << 4,
	MeshClothingAssets = 1 << 5,
	SourceModels = 1 << 6,	
	HasActiveClothingAssets = 1 << 7,
	LODSettings = 1 << 8,
	HasVertexColors = 1 << 9,
	VertexColorGuid = 1 << 10,
	MorphTargets = 1 << 11,
	SkeletalMeshRenderData = 1 << 12,
	MeshEditorDataObject = 1 << 13,
	NeverStream = 1 << 14,
	OverrideLODStreamingSettings = 1 << 15,
	SupportLODStreaming = 1 << 16,
	MaxNumStreamedLODs = 1 << 17,
	MaxNumOptionalLODs = 1 << 18,
	ImportedModel = 1 << 19,
	// Spare = 1 << 20,
	SkinWeightProfiles = 1 << 21,
	CachedComposedRefPoseMatrices = 1 << 22,
	SamplingInfo = 1 << 23,
	NodeMappingData = 1 << 24,
	ShadowPhysicsAsset = 1 << 25,
	SkelMirrorTable = 1 << 26,
	MinLod = 1 << 27,
	DisableBelowMinLodStripping = 1 << 28,
	SkelMirrorAxis = 1 << 29,
	SkelMirrorFlipAxis = 1 << 30,
	DefaultAnimationRig = 1llu << 31,
	NegativeBoundsExtension = 1llu << 32,
	PositiveBoundsExtension = 1llu << 33,
	ExtendedBounds = 1llu << 34,
	// Spare = 1llu << 35,
	EnablePerPolyCollision = 1llu << 36,
	BodySetup = 1llu << 37,
	MorphTargetIndexMap = 1llu << 38,
	FloorOffset = 1llu << 39,
	ImportedBounds = 1llu << 40,
	PhysicsAsset = 1llu << 41,
	AssetImportData = 1llu << 42,
	ThumbnailInfo = 1llu << 43,
	HasCustomDefaultEditorCamera = 1llu << 44,
	DefaultEditorCameraLocation = 1llu << 45,
	DefaultEditorCameraRotation = 1llu << 46,
	RequiresLODScreenSizeConversion = 1llu << 47,
	PostProcessAnimBlueprint = 1llu << 48,
	DefaultEditorCameraLookAt = 1llu << 49,
	PreviewAttachedAssetContainer = 1llu << 50,
	DefaultEditorCameraOrthoZoom = 1llu << 51,
	RequiresLODHysteresisConversion = 1llu << 52,
	bSupportRayTracing = 1llu << 53,
	RayTracingMinLOD = 1llu << 54,
	ClothLODBiasMode = 1llu << 55,
	DefaultMeshDeformer = 1llu << 56,
	OverlayMaterial = 1llu << 57,
	OverlayMaterialMaxDrawDistance = 1llu << 58,
	TargetMeshDeformers = 1llu << 59,
	All = MAX_uint64
};

ENUM_CLASS_FLAGS(ESkeletalMeshAsyncProperties);


using FSkeletalMeshCompilationContext UE_DEPRECATED(5.1, "Use FSkinnedAssetCompilationContext instead.") = FSkinnedAssetCompilationContext;
using FSkeletalMeshPostLoadContext UE_DEPRECATED(5.1, "Use FSkinnedAssetPostLoadContext instead.") = FSkinnedAssetPostLoadContext;
using FSkeletalMeshBuildContext UE_DEPRECATED(5.1, "Use FSkinnedAssetBuildContext instead.") = FSkinnedAssetBuildContext;
#if WITH_EDITOR
using FSkeletalMeshAsyncBuildScope UE_DEPRECATED(5.1, "Use FSkinnedAssetAsyncBuildScope instead.") = FSkinnedAssetAsyncBuildScope;
using FSkeletalMeshAsyncBuildWorker UE_DEPRECATED(5.1, "Use FSkinnedAssetAsyncBuildWorker instead.") = FSkinnedAssetAsyncBuildWorker;
using FSkeletalMeshAsyncBuildTask UE_DEPRECATED(5.1, "Use FSkinnedAssetAsyncBuildTask instead.") = FSkinnedAssetAsyncBuildTask;
#endif

struct UE_DEPRECATED(5.0, "FBoneMirrorInfo is deprecated. Please use UMirrorDataTable for mirroring support.") FBoneMirrorInfo;
USTRUCT()
struct FBoneMirrorInfo
{
	GENERATED_USTRUCT_BODY()

	/** The bone to mirror. */
	UPROPERTY(EditAnywhere, Category=BoneMirrorInfo, meta=(ArrayClamp = "RefSkeleton"))
	int32 SourceIndex;

	/** Axis the bone is mirrored across. */
	UPROPERTY(EditAnywhere, Category=BoneMirrorInfo)
	TEnumAsByte<EAxis::Type> BoneFlipAxis;

	FBoneMirrorInfo()
		: SourceIndex(0)
		, BoneFlipAxis(0)
	{
	}

};

struct UE_DEPRECATED(5.0, "FBoneMirrorExport is deprecated. Please use UMirrorDataTable for mirroring support.") FBoneMirrorExport;
USTRUCT()
struct FBoneMirrorExport
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=BoneMirrorExport)
	FName BoneName;

	UPROPERTY(EditAnywhere, Category=BoneMirrorExport)
	FName SourceBoneName;

	UPROPERTY(EditAnywhere, Category=BoneMirrorExport)
	TEnumAsByte<EAxis::Type> BoneFlipAxis;


	FBoneMirrorExport()
		: BoneFlipAxis(0)
	{
	}

};

/** Struct holding parameters needed when creating a new clothing asset or sub asset (LOD) */
USTRUCT()
struct FSkeletalMeshClothBuildParams
{
	GENERATED_BODY()

	ENGINE_API FSkeletalMeshClothBuildParams();

	// Target asset when importing LODs
	UPROPERTY(EditAnywhere, Category = Target)
	TWeakObjectPtr<UClothingAssetBase> TargetAsset;

	// Target LOD to import to when importing LODs
	UPROPERTY(EditAnywhere, Category = Target)
	int32 TargetLod;

	// If reimporting, this will map the old LOD parameters to the new LOD mesh.
	// If adding a new LOD this will map the parameters from the preceeding LOD.
	UPROPERTY(EditAnywhere, Category = Target)
	bool bRemapParameters;

	// Name of the clothing asset 
	UPROPERTY(EditAnywhere, Category = Basic)
	FString AssetName;

	// LOD to extract the section from
	UPROPERTY(EditAnywhere, Category = Basic)
	int32 LodIndex;

	// Section within the specified LOD to extract
	UPROPERTY(EditAnywhere, Category = Basic)
	int32 SourceSection;

	// Whether or not to leave this section behind (if driving a mesh with itself). Enable this if driving a high poly mesh with a low poly
	UPROPERTY(EditAnywhere, Category = Basic)
	bool bRemoveFromMesh;

	// Physics asset to extract collisions from, note this will only extract spheres and Sphyls, as that is what the simulation supports.
	UPROPERTY(EditAnywhere, Category = Collision)
	TSoftObjectPtr<UPhysicsAsset> PhysicsAsset;
};

/**
 * Legacy object for back-compat loading, no longer used by clothing system
 */
USTRUCT()
struct FClothPhysicsProperties_Legacy
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	float VerticalResistance;
	UPROPERTY()
	float HorizontalResistance;
	UPROPERTY()
	float BendResistance;
	UPROPERTY()
	float ShearResistance;
	UPROPERTY()
	float Friction;
	UPROPERTY()
	float Damping;
	UPROPERTY()
	float TetherStiffness;
	UPROPERTY()
	float TetherLimit;
	UPROPERTY()
	float Drag;
	UPROPERTY()
	float StiffnessFrequency;
	UPROPERTY()
	float GravityScale;
	UPROPERTY()
	float MassScale;
	UPROPERTY()
	float InertiaBlend;
	UPROPERTY()
	float SelfCollisionThickness;
	UPROPERTY()
	float SelfCollisionSquashScale;
	UPROPERTY()
	float SelfCollisionStiffness;
	UPROPERTY()
	float SolverFrequency;
	UPROPERTY()
	float FiberCompression;
	UPROPERTY()
	float FiberExpansion;
	UPROPERTY()
	float FiberResistance;

	FClothPhysicsProperties_Legacy()
		: VerticalResistance(0.f)
		, HorizontalResistance(0.f)
		, BendResistance(0.f)
		, ShearResistance(0.f)
		, Friction(0.f)
		, Damping(0.f)
		, TetherStiffness(0.f)
		, TetherLimit(0.f)
		, Drag(0.f)
		, StiffnessFrequency(0.f)
		, GravityScale(0.f)
		, MassScale(0.f)
		, InertiaBlend(0.f)
		, SelfCollisionThickness(0.f)
		, SelfCollisionSquashScale(0.f)
		, SelfCollisionStiffness(0.f)
		, SolverFrequency(0.f)
		, FiberCompression(0.f)
		, FiberExpansion(0.f)
		, FiberResistance(0.f)
	{
	}
};


// Legacy struct for handling back compat serialization
USTRUCT()
struct FClothingAssetData_Legacy
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName AssetName;
	UPROPERTY()
	FString	ApexFileName;
	UPROPERTY()
	bool bClothPropertiesChanged;
	UPROPERTY()
	FClothPhysicsProperties_Legacy PhysicsProperties;

	FClothingAssetData_Legacy()
		: bClothPropertiesChanged(false), PhysicsProperties()
	{
	}

	// serialization
	friend FArchive& operator<<(FArchive& Ar, FClothingAssetData_Legacy& A);
};

/**
 * Strategy used for storing additional cloth deformer mappings depending on the
 * desired use of the RaytracingMinLOD value and of the LODBias console variable.
 */
UENUM()
enum class EClothLODBiasMode : uint8
{
	// Only store the strict minimum amount of cloth deformer mappings to save on memory usage.
	// Raytracing of cloth elements must never be of a different LOD to the one being rendered when using this mode.
	MappingsToSameLOD,

	// Store additional cloth deformer mappings to allow raytracing of the cloth elements at RayTracingMinLOD.
	// Raytracing of cloth elements must never be of a different LOD to the one being rendered, or to the one set in RayTracingMinLOD when using this mode.
	MappingsToMinLOD,

	// Store all cloth deformer mappings at the expense of memory usage, to allow raytracing of the cloth elements at any higher LOD.
	// Use this mode when the RayTracing LODBias console variable is in use.
	MappingsToAnyLOD,
};

#if WITH_EDITOR
/** delegate type for pre skeletal mesh build events */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostMeshCache, class USkeletalMesh*);

#endif

#if WITH_EDITORONLY_DATA
namespace NSSkeletalMeshSourceFileLabels
{
	inline FText GeoAndSkinningText()
	{
		static FText GeoAndSkinningText = (NSLOCTEXT("FBXReimport", "ImportContentTypeAll", "Geometry and Skinning Weights"));
		return GeoAndSkinningText;
	}

	inline FText GeometryText()
	{
		static FText GeometryText = (NSLOCTEXT("FBXReimport", "ImportContentTypeGeometry", "Geometry"));
		return GeometryText;
	}
	inline FText SkinningText()
	{
		static FText SkinningText = (NSLOCTEXT("FBXReimport", "ImportContentTypeSkinning", "Skinning Weights"));
		return SkinningText;
	}

	inline const FString& GeoAndSkinningMetaDataValue()
	{
		static FString GeoAndSkinningName("All");
		return GeoAndSkinningName;
	}

	inline const FString& GeometryMetaDataValue()
	{
		static FString GeometryName("Geometry");
		return GeometryName;
	}

	inline const FString& SkinningMetaDataValue()
	{
		static FString SkinningName("SkinningWeights");
		return SkinningName;
	}

	inline FName GetSkeletalMeshLastImportContentTypeMetadataKey()
	{
		static FName SkeletalMeshLastImportContentTypeMetadataKey("SkeletalMeshLastImportContentTypeMetadataKey");
		return SkeletalMeshLastImportContentTypeMetadataKey;
	}
}
#endif

/* Scoped SkeletalMeshRenderData
 * All to get access to SkeletalMeshRenderData for a given platform an ensuring the data are thread-safe during the entire scope
 */
struct FScopedSkeletalMeshRenderData
{
public:
	UE_DEPRECATED(5.4, "This class is deprecated, there is no replacement because this is not a safe async design.")
	UE_API FScopedSkeletalMeshRenderData(USkeletalMesh* Mesh);
	
	UE_API ~FScopedSkeletalMeshRenderData();

	UE_DEPRECATED(5.4, "This class is deprecated, there is no replacement because this is not a safe async design.")
	UE_API const FSkeletalMeshRenderData* GetData() const;
private:
	FEvent* Lock = nullptr;
	USkeletalMesh* Mesh = nullptr;
	FSkeletalMeshRenderData* Data = nullptr;

	friend class USkeletalMesh;
};

/**
 * SkeletalMesh is geometry bound to a hierarchical skeleton of bones which can be animated for the purpose of deforming the mesh.
 * Skeletal Meshes are built up of two parts; a set of polygons composed to make up the surface of the mesh, and a hierarchical skeleton which can be used to animate the polygons.
 * The 3D models, rigging, and animations are created in an external modeling and animation application (3DSMax, Maya, Softimage, etc).
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Content/Types/SkeletalMeshes/
 */
UCLASS(hidecategories=Object, BlueprintType, MinimalAPI)
class USkeletalMesh : public USkinnedAsset, public IInterface_CollisionDataProvider, public IInterface_AssetUserData, public INodeMappingProviderInterface
{
	GENERATED_UCLASS_BODY()

	// This is declared so we can use TUniquePtr<FSkeletalMeshRenderData> with just a forward declare of that class
	ENGINE_API USkeletalMesh(FVTableHelper& Helper);
	ENGINE_API virtual ~USkeletalMesh() override;

#if WITH_EDITOR
	/** Notification when something is about to be changed */
	DECLARE_MULTICAST_DELEGATE(FOnPreMeshChange);
	/** Notification when anything changed */
	DECLARE_MULTICAST_DELEGATE(FOnMeshChanged);
#endif
private:
	/** The imported/editable geometric source data for generating renderable data. */
	UE_DEPRECATED(5.4, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, EditFixedSize, Category=LevelOfDetailModels)
	TArray<FSkeletalMeshSourceModel> SourceModels;

#if WITH_EDITORONLY_DATA
	/** Processed skeletal mesh geometry information (not used at runtime). */
	/** NOTE: This data is slated for removal in future versions of UE. Do not rely on it. Use either the
	 *  render data, or source models, as needed.
	 */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	TSharedPtr<FSkeletalMeshModel> ImportedModel;
#endif

	/** Rendering resources used at runtime */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	TUniquePtr<FSkeletalMeshRenderData> SkeletalMeshRenderData;

	ENGINE_API FSkeletalMeshRenderData* GetSkeletalMeshRenderData() const;

	ENGINE_API void SetSkeletalMeshRenderData(TUniquePtr<FSkeletalMeshRenderData>&& InSkeletalMeshRenderData);

	/** Returns true if the mesh has valid Nanite render data. */
	ENGINE_API bool HasValidNaniteData() const;

public:

	/** Add a new source model and set the reduction setting according to the state of the bAutoGenerated argument. 
	 * \param bAutoGenerated If set to true, the reduction settings are updated to point to a previous LOD with
	 *    editable data and the triangle reduction set to 50% for each LOD between the two (e.g. if this adds LOD 2,
	 *    and the first non-autogenerated LOD is 0, then the reduction settings will be at 25%).
	 * \return The newly created source model.
	 */
	ENGINE_API FSkeletalMeshSourceModel& AddSourceModel(const bool bAutoGenerated = false);
		
	/** Set the number of source models. If adding more LODs than currently exist, empty LODs will be created
	 *  to fill in the gap. If the new size is smaller than the existing LODs, the source geometry for LODs
	 *  at and higher than the number given will be thrown away.
	 */
	ENGINE_API void SetNumSourceModels(const int32 InNumSourceModels);

	/** Remove the source model for the given LOD. LODs above it will be adjusted down.
	 */
	ENGINE_API void RemoveSourceModel(const int32 InLODIndex);

	/** Returns the number of source models.
	 */
	ENGINE_API int32 GetNumSourceModels() const;

	/** Returns a boolean indicating whether the given LOD index is valid or not.
	 * \param InLODIndex The potential LOD index to check for validity.
	 * \return \c true if the given LOD index in InLODIndex is valid.
	*/
	ENGINE_API bool IsSourceModelValid(const int32 InLODIndex) const;	
	
	/** Returns all source models as immutable. See GetSourceModel for description of what each source model contains.
	 *  \return The list of all source models. 
	 */
	ENGINE_API TConstArrayView<FSkeletalMeshSourceModel> GetAllSourceModels() const;

	/** Returns all source models. See GetSourceModel for description of what each source model contains.
	 *  \return The list of all source models. 
	 */
	ENGINE_API TArrayView<FSkeletalMeshSourceModel> GetAllSourceModels();
	
	/** Returns the source model object for the skeletal mesh of a given LOD. This source models stores
	 *  an optional mesh description that the renderable data is generated from. If there is no mesh description
	 *  object, then the geometry of this LOD is automatically generated from an earlier LOD.
	 *  \param InLODIndex The LOD index to retrieve a FSkeletalMeshSourceModel object for. 
	 */
	ENGINE_API const FSkeletalMeshSourceModel& GetSourceModel(const int32 InLODIndex) const;
	
	/** Returns the source model object for the skeletal mesh of a given LOD. This source models stores
	 *  an optional mesh description that the renderable data is generated from. If there is no mesh description
	 *  object, then the geometry of this LOD is automatically generated from an earlier LOD.
	 *  \param InLODIndex The LOD index to retrieve a FSkeletalMeshSourceModel object for. 
	 */
	ENGINE_API FSkeletalMeshSourceModel& GetSourceModel(const int32 InLODIndex);

#if WITH_EDITORONLY_DATA
	/** Returns the MeshDescription object associated with the given LOD index. If there's no import data
	 *  at this LOD, and the LOD is auto-generated, then a \c nullptr is returned.
	 *  \param InLODIndex The LOD index to retrieve a FMeshDescription object for. 
	 */
	ENGINE_API FMeshDescription* GetMeshDescription(const int32 InLODIndex) const;
	
	/** Returns the MeshDescription object associated with the given LOD index. If there's no import data
	 *  at this LOD, and the LOD is auto-generated, then a \c nullptr is returned.
	 *  \param InLODIndex The LOD index to clone a FMeshDescription object from.
	 *  \param OutMeshDescription The target mesh description object to copy the source data into.
	 *  \return \c true if valid mesh description was copied in. \c false if there was no mesh description
	 *     the LOD is automatically generated.
	 */
	ENGINE_API bool CloneMeshDescription(const int32 InLODIndex, FMeshDescription& OutMeshDescription) const;

	/** Returns \c true if this LOD has a mesh description to generate render data from.
	 *  If not, then render data is automatically generated from a lower LOD level.
	 *  Use this rather than GetMeshDescription if the only requirement is to check for existence, since
	 *  this function won't unpack a full mesh description object from bulk storage if it doesn't have to.
	 *  \param InLODIndex The LOD index to check for existence of a mesh description object.
	 *  \return \c This LOD index contains a valid mesh description object.
	 */
	ENGINE_API bool HasMeshDescription(const int32 InLODIndex) const;

	/** Creates a new, empty mesh description at the LOD index given. If there was a mesh description there
	 *  prior, that object is destroyed.
	 *  \param InLODIndex The LOD index to create a new mesh description object at.
	 *  \return \c A pointer to the newly created object, or \c nullptr if the method was unable to create anything.
	 */
	ENGINE_API FMeshDescription* CreateMeshDescription(const int32 InLODIndex);

	/** Creates a new mesh description at the LOD index given, moving the contents of the given mesh description
	 *  into it. If there was a mesh description there prior, that object is destroyed.
	 *  \param InLODIndex The LOD index to create a new mesh description object at.
	 *  \param InMeshDescription The mesh description object to initialize from as a copy.
	 *  \return \c A pointer to the newly created object, or \c nullptr if the method was unable to create anything.
	 */
	ENGINE_API FMeshDescription* CreateMeshDescription(const int32 InLODIndex, FMeshDescription&& InMeshDescription);

	
	/* Structure that defines parameters passed into the commit mesh description function */
	struct FCommitMeshDescriptionParams
	{
		FCommitMeshDescriptionParams() {}
		
		/** Ensure that there is a placeholder on the skeletal mesh to hold the morph target delta data prior to
		 *  the mesh rebuild. Set this to off if the morph target placeholder creation is not necessary. If the
		 *  UMorphTarget already exists, then nothing is done.
		 */
		bool bUpdateMorphTargets = true;
		
		/** Update the alternate skin weight profile list based on presence of non-default skin weight attributes in the  
		 *  committed mesh. If a skin weight attribute exists on the mesh but not in the list of profiles, then the profile
		 *  is added. Likewise, if a profile exists but there's no corresponding skin weight attribute then that profile
		 *  is removed.
		 */
		bool bUpdateSkinWeightProfiles = true;
		
		/** Update the LOD's vertex attribute list based on presence of vertex attributes in the committed mesh. 
		 *  If a vertex attribute is defined on the committed mesh but doesn't exist in the list of vertex attributes 
		 *  on the LOD, then that vertex attribute is added to the LOD. Likewise, vertex attribute exists on the LOD, 
		 *  but not on the committed mesh, then the attribute is removed from the LOD. 
		 */
		bool bUpdateVertexAttributes = true;
		
		/** Update the vertex color enabled state/GUID if the mesh description contains non-default vertex colors */
		bool bUpdateVertexColors = true;
		
		/** Mark the package as dirty. If calling CommitMeshDescription from a non-game thread,
		 *  this value should be set to \c false.
		 */
		bool bMarkPackageDirty = true;

		/** Force the render data to update. By default, the render data uses the hash of the mesh to check if
		 *  an update is required, this forces this hash to be unique, causing the render data to update whether
		 *  the mesh has changed or not.
		 */
		bool bForceUpdate = false;
	};

	/** Commits the stored mesh description object to bulk storage. This also forces the imported bounds to update
	 *  if the mesh committed is on LOD 0. If there is no stored mesh description, the bulk storage will be emptied.
	 *  \note It is thread-safe to commit multiple meshes simultaneously, as long as they're all on different LODs. 
	 *  \param InLODIndex The LOD index at which to commit the mesh description to bulk storage.
	 *  \param InParams An optional object to control what happens during the commit.
	 *  \return \c true if the commit was successful. Even if there was no mesh description to store, the commit
	 *    is still successful.
	 */
	ENGINE_API bool CommitMeshDescription(const int32 InLODIndex, const FCommitMeshDescriptionParams& InParams = FCommitMeshDescriptionParams());

	/** Mark the bulk data as dirty so that it can be transacted for undo/redo.
	 *  \param InLODIndex The LOD index at which to mark the bulk data dirty.
	 *  \param bInAlwaysMarkPackageDirty Always mark the owning package dirty, even if there was not a transaction open to store into.
	 *  \return \c true if the bulk data was stored in the transaction buffer.
	 */
	ENGINE_API bool ModifyMeshDescription(const int32 InLODIndex, const bool bInAlwaysMarkPackageDirty = true);

	/** Clears a stored mesh description and releases its memory. Leaves the bulk data in place if the mesh
	 *  description is needed again.
	 *  \param InLODIndex The LOD index at which to clear out the mesh description.
	 */
	ENGINE_API void ClearMeshDescription(const int32 InLODIndex);

	/** Clears a all stored mesh descriptions on all LODs and releases their memory. Leaves the bulk data in 
	 *  place if the mesh descriptions are needed again.
	 */
	ENGINE_API void ClearAllMeshDescriptions();

	/** Clears the stored mesh description _and_ the bulk data. Use this to mark the LOD as being auto-generated.
	 *  Once this function is called, the LOD settings for this level should be updated to reflect how the
	 *  geometry should be autogenerated from a prior LOD.
	 *  \param InLODIndex The LOD index at which to clear out the mesh description and bulk data.
	 */
	ENGINE_API void ClearMeshDescriptionAndBulkData(const int32 InLODIndex);


	/** Generally clear data (source models, skeleton, physics, materials, etc) off the skeletal mesh */
	ENGINE_API void Clear();

	//////////////////////////////////////////////////////////////////////////
	// USkeletalMeshEditorData public skeletalmesh API
	// We do not want skeletal mesh client to use directly the asset(function GetMeshEditorData)
	// We have to maintain some sync between the LODModels and the asset to avoid loading the asset when
	// building the DDC key. That is why the asset accessor are private. the data we keep in sync in the LODModels is:
	// IsLODImportedDataBuildAvailable
	// IsLODImportedDataEmpty
	// Raw mesh data DDC string ID, there is no API to retrieve it, since only the LODModels need this value
	

	UE_DEPRECATED(5.4, "Use GetMeshDescription instead.")
	ENGINE_API void LoadLODImportedData(const int32 LODIndex, FSkeletalMeshImportData& OutMesh) const;

	UE_DEPRECATED(5.4, "Use CommitMeshDescription instead.")
	ENGINE_API void SaveLODImportedData(const int32 LODIndex, const FSkeletalMeshImportData& InMesh);
	
	UE_DEPRECATED(5.4, "Use HasMeshDescription instead.")
	ENGINE_API bool IsLODImportedDataBuildAvailable(const int32 LODIndex) const;
	
	UE_DEPRECATED(5.4, "Use HasMeshDescription instead.")
	ENGINE_API bool IsLODImportedDataEmpty(const int32 LODIndex) const;

	UE_DEPRECATED(5.4, "No equivalent provided since versioning is not surfaced for mesh description bulk data.")
	ENGINE_API void GetLODImportedDataVersions(const int32 LODIndex, ESkeletalMeshGeoImportVersions& OutGeoImportVersion, ESkeletalMeshSkinningImportVersions& OutSkinningImportVersion) const;

	UE_DEPRECATED(5.4, "No equivalent provided since versioning is not surfaced for mesh description bulk data.")
	ENGINE_API void SetLODImportedDataVersions(const int32 LODIndex, const ESkeletalMeshGeoImportVersions& InGeoImportVersion, const ESkeletalMeshSkinningImportVersions& InSkinningImportVersion);

	UE_DEPRECATED(5.4, "Use GetMeshDescription and CreateMeshDescription instead.")
	static ENGINE_API void CopyImportedData(int32 SrcLODIndex, USkeletalMesh* SrcSkeletalMesh, int32 DestLODIndex, USkeletalMesh* DestSkeletalMesh);

	UE_DEPRECATED(5.4, "No equivalent provided. All LODs should be added up-front for multi-threaded use.")
	ENGINE_API void ReserveLODImportData(int32 MaxLODIndex);
	
	UE_DEPRECATED(5.4, "No equivalent provided.")
	ENGINE_API void ForceBulkDataResident(const int32 LODIndex);

	UE_DEPRECATED(5.4, "Use ClearMeshDescriptionAndBulkData instead.")
	ENGINE_API void EmptyLODImportData(const int32 LODIndex);

	UE_DEPRECATED(5.4, "Use ClearMeshDescriptionAndBulkData instead.")
	ENGINE_API void EmptyAllImportData();

	// End USkeletalMeshEditorData public skeletalmesh API
	//////////////////////////////////////////////////////////////////////////

	/** Get the number of imported vertices. This returns 0 if GetImportedModel() returns a nullptr.
	  * This is the number of vertices as they appear in the source asset, for example 8 for a cube. */
	ENGINE_API int32 GetNumImportedVertices() const;

	/** Get the imported data for this skeletal mesh. */
	virtual FSkeletalMeshModel* GetImportedModel() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ImportedModel);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ImportedModel.Get();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
private:
	void SetImportedModel(TSharedPtr<FSkeletalMeshModel> InImportedModel)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ImportedModel);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ImportedModel = InImportedModel;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
public:
#endif //WITH_EDITORONLY_DATA


#if WITH_EDITOR
    /** Warn if the platform supports the minimal number of per vertex bone weights */
	ENGINE_API void ValidateBoneWeights(const ITargetPlatform* TargetPlatform);
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
#endif

	/** Get the data to use for rendering. USkinnedAsset interface. */
	ENGINE_API virtual FSkeletalMeshRenderData* GetResourceForRendering() const override;

	/** Skeleton of this skeletal mesh **/
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetSkeleton() or USkeletalMesh::SetSkeleton().")
	UPROPERTY(Category=Mesh, AssetRegistrySearchable, VisibleAnywhere, BlueprintGetter = GetSkeleton, BlueprintSetter = SetSkeleton)
	TObjectPtr<USkeleton> Skeleton;

	static FName GetSkeletonMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, Skeleton);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual USkeleton* GetSkeleton() override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::Skeleton);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Skeleton;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	UFUNCTION(BlueprintGetter)
	virtual const USkeleton* GetSkeleton() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::Skeleton, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Skeleton;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	void SetSkeleton(USkeleton* InSkeleton)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::Skeleton);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Skeleton = InSkeleton;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

protected:
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void UpdateMaterialUVChannelData(FMeshUVChannelInfo& UVChannelData, int32 MaterialIndex) override;
#endif

private:
	/** Original imported mesh bounds */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(transient, duplicatetransient)
	FBoxSphereBounds ImportedBounds;

	/** Bounds extended by user values below */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(transient, duplicatetransient)
	FBoxSphereBounds ExtendedBounds;

	const FBoxSphereBounds& GetExtendedBounds() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ExtendedBounds, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ExtendedBounds;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetExtendedBounds(const FBoxSphereBounds& InExtendedBounds)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ExtendedBounds);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ExtendedBounds = InExtendedBounds;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

protected:
	// The properties below are protected to force the use of the Set* methods for this data
	// in code so we can keep the extended bounds up to date after changing the data.
	// Property editors will trigger property events to correctly recalculate the extended bounds.

	/** Bound extension values in addition to imported bound in the positive direction of XYZ, 
	 *	positive value increases bound size and negative value decreases bound size. 
	 *	The final bound would be from [Imported Bound - Negative Bound] to [Imported Bound + Positive Bound]. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mesh)
	FVector PositiveBoundsExtension;

	static FName GetPositiveBoundsExtensionMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, PositiveBoundsExtension);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Bound extension values in addition to imported bound in the negative direction of XYZ, 
	 *	positive value increases bound size and negative value decreases bound size. 
	 *	The final bound would be from [Imported Bound - Negative Bound] to [Imported Bound + Positive Bound]. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mesh)
	FVector NegativeBoundsExtension;

	static FName GetNegativeBoundsExtensionMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, NegativeBoundsExtension);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

public:
	/** Get the extended bounds of this mesh (imported bounds plus bounds extension). USkinnedAsset interface. */
	UFUNCTION(BlueprintCallable, Category = Mesh)
	ENGINE_API virtual FBoxSphereBounds GetBounds() const override;

	/** Get the original imported bounds of the skel mesh */
	UFUNCTION(BlueprintCallable, Category = Mesh)
	ENGINE_API FBoxSphereBounds GetImportedBounds() const;

	/** Set the original imported bounds of the skel mesh, will recalculate extended bounds */
	ENGINE_API void SetImportedBounds(const FBoxSphereBounds& InBounds);

	/** Set bound extension values in the positive direction of XYZ, positive value increases bound size */
	ENGINE_API void SetPositiveBoundsExtension(const FVector& InExtension);

	/** Get bound extension values in the positive direction of XYZ **/
	const FVector& GetPositiveBoundsExtension() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ExtendedBounds, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PositiveBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Set bound extension values in the negative direction of XYZ, positive value increases bound size */
	ENGINE_API void SetNegativeBoundsExtension(const FVector& InExtension);

	/** Get bound extension values in the negative direction of XYZ **/
	const FVector& GetNegativeBoundsExtension() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ExtendedBounds, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NegativeBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Calculate the extended bounds based on the imported bounds and the extension values */
	ENGINE_API void CalculateExtendedBounds();

	/** Alters the bounds extension values to fit correctly into the current bounds (so negative values never extend the bounds etc.) */
	ENGINE_API void ValidateBoundsExtension();

#if WITH_EDITOR
	/** This is a bit hacky. If you are inherriting from SkeletalMesh you can opt out of using the skeletal mesh actor factory. Note that this only works for one level of inherritence and is not a good long term solution */
	virtual bool HasCustomActorFactory() const
	{
		return false;
	}

	/** This is a bit hacky. If you are inherriting from SkeletalMesh you can opt out of using the skeletal mesh actor factory. Note that this only works for one level of inherritence and is not a good long term solution */
	virtual bool HasCustomActorReimportFactory() const
	{
		return false;
	}

	ENGINE_API bool IsReadyToRenderInThumbnail() const;

	/* Return true if this skeletalmesh was never build since its creation. USkinnedAsset interface. */
	ENGINE_API virtual bool IsInitialBuildDone() const override;

	/* Return true if the reduction settings are setup to reduce a LOD*/
	ENGINE_API bool IsReductionActive(int32 LODIndex) const;

	/* Get a copy of the reduction settings for a specified LOD index. */
	ENGINE_API struct FSkeletalMeshOptimizationSettings GetReductionSettings(int32 LODIndex) const;

#endif

	/** List of materials applied to this mesh. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMaterials() or USkeletalMesh::SetMaterials().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetMaterials, BlueprintSetter = SetMaterials, transient, duplicatetransient, Category=SkeletalMesh)
	TArray<FSkeletalMaterial> Materials;

	TArray<FSkeletalMaterialMinimalInfo> SkeletalMaterialsInfoCache;
	mutable FCriticalSection MaterialsInfoCriticalSection;
	ENGINE_API void UpdateSkeletalMaterialsInfoCache(const TArray<FSkeletalMaterial>& InMaterials);
	ENGINE_API const FString ExportSkeletalMaterialsInfo() const;
	
	static FName GetMaterialsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, Materials);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const TArray<FSkeletalMaterialMinimalInfo> GetCopyOfSkeletalMaterialsInfo() const
	{
		FScopeLock Lock(&MaterialsInfoCriticalSection);
		return SkeletalMaterialsInfoCache;
	}

	ENGINE_API static const TArray<FSkeletalMaterialMinimalInfo> ImportSkeletalMaterialsInfo(const FString& InData);

	/** USkinnedAsset interface. */
	virtual TArray<FSkeletalMaterial>& GetMaterials() override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::Materials);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Materials;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	UFUNCTION(BlueprintGetter)
	virtual const TArray<FSkeletalMaterial>& GetMaterials() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::Materials, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Materials;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetMaterials(const TArray<FSkeletalMaterial>& InMaterials);

#if WITH_EDITORONLY_DATA
	/** List of bones that should be mirrored. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(4.27, "Please do not access this member directly; Use UMirrorDataTable for mirroring support")
	UPROPERTY(EditAnywhere, editfixedsize, Category=Mirroring)
	TArray<struct FBoneMirrorInfo> SkelMirrorTable;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Settings related to building Nanite data. */
	UPROPERTY(EditAnywhere, Category = Mesh)
	FMeshNaniteSettings NaniteSettings;
	/**
	 * Returns true if this skeletal mesh should have Nanite built for it.
	 */
	ENGINE_API bool IsNaniteEnabled() const;

	const FMeshNaniteSettings& GetNaniteSettings() const
	{
		return NaniteSettings;
	}
	void SetNaniteSettings(const FMeshNaniteSettings& Settings)
	{
		NaniteSettings = Settings;
	}

	void NotifyNaniteSettingsChanged()
	{
		FProperty* ChangedProperty = FindFProperty<FProperty>(USkeletalMesh::StaticClass(), GET_MEMBER_NAME_CHECKED(USkeletalMesh, NaniteSettings));
		FPropertyChangedEvent Event(ChangedProperty);
		PostEditChangeProperty(Event);
	}

	/**
	 * Returns true if this skeletal mesh is a Nanite Assembly
	 */
	ENGINE_API bool IsNaniteAssembly() const;

	/**
	 * Caches the meshes referenced by the Nanite Assembly data for the mesh to be used for build. Must
	 * be called on the game thread before any asynchronous building. You should only need to call this externally
	 * if caching render data manually (as opposed to using CacheDerivedData on the game thread or async mesh build).
	 */
	ENGINE_API void RecacheNaniteAssemblyReferences(bool bWaitForAsyncCompile = true);
	
	/**
	 * Clears the cache of static meshes referenced by the Nanite Assembly data for this mesh. Must be called from the
	 * game thread.
	 */
	ENGINE_API void ClearCachedNaniteAssemblyReferences();

	/**
	 * Returns whether or not Nanite Assembly reference meshes are adequately cached for build (always true for non-assemblies).
	 */
	ENGINE_API bool HasCachedNaniteAssemblyReferences() const;

	/**
	 * Retrieves the cached array of static meshes referenced by the Nanite Assembly data.
	 * NOTE: This array is only meant to contain anything during static mesh build.
	 */
	ENGINE_API const TArray<TObjectPtr<USkeletalMesh>>& GetCachedNaniteAssemblyReferences() const;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API virtual TArray<USkinnedAsset*> GetSkinnedAssetDependencies() const override;
	ENGINE_API virtual bool HasCompileDependencies() const override;
	ENGINE_API virtual bool HasAnyDependenciesCompiling() const override;
private:
	void OnNaniteAssemblyReferenceCached(USkeletalMesh* Reference);
	void CacheClothDependencies();

	/** Cache of SkinnedAsset dependencies accessible during the async build. */
	TArray<TWeakObjectPtr<USkinnedAsset>> ClothDependencyCache;
#endif

private:

#if WITH_EDITOR
	FSimpleMulticastDelegate OnVertexAttributesArrayChanged;
#endif
	
#if !WITH_EDITOR
	/** Acceleration struct used for faster socket lookups */
	struct FSocketInfo
	{
		FSocketInfo(const USkeletalMesh* InSkeletalMesh, USkeletalMeshSocket* InSocket, int32 InSocketIndex);

		FTransform SocketLocalTransform;
		USkeletalMeshSocket* Socket;
		int32 SocketIndex;
		int32 SocketBoneIndex;
	};

	/** Map used for faster lookups of sockets/indices */
	TMap<FName, FSocketInfo> SocketMap;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TArray<TObjectPtr<USkeletalMesh>> AssemblyReferenceCache;
#endif

public:
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Quality Level Minimum LOD"))
	FPerQualityLevelInt MinQualityLevelLOD;

	UFUNCTION(BlueprintCallable, Category = StaticMesh, Meta = (ToolTip = "Allow to override min lod quality levels on a skeletalMesh and it Default value (-1 value for Default dont override its value)."))
	void SetMinLODForQualityLevels(const TMap<EPerQualityLevels, int32>& QualityLevelMinimumLODs, int32 Default = -1)
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MinLod);
		MinQualityLevelLOD.PerQuality = QualityLevelProperty::ConvertQualityLevelData(QualityLevelMinimumLODs);
		MinQualityLevelLOD.Default = Default >=0 ? Default : MinQualityLevelLOD.Default;
#endif
	}

	UFUNCTION(BlueprintPure, Category = StaticMesh)
	void GetMinLODForQualityLevels(TMap<EPerQualityLevels, int32>& QualityLevelMinimumLODs, int32& Default) const
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MinLod, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		QualityLevelMinimumLODs = QualityLevelProperty::ConvertQualityLevelData(MinQualityLevelLOD.PerQuality);
		Default = MinQualityLevelLOD.Default;
#endif
	}

	static FName GetQualityLevelMinLodMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MinQualityLevelLOD);
	}

	const FPerQualityLevelInt& GetQualityLevelMinLod() const
	{
		return MinQualityLevelLOD;
	}

	void SetQualityLevelMinLod(FPerQualityLevelInt InMinLod)
	{
			MinQualityLevelLOD = MoveTemp(InMinLod);
	}

	/** Check the QualitLevel property is enabled for MinLod. USkinnedAsset interface. */
	ENGINE_API virtual bool IsMinLodQualityLevelEnable() const override;

	/** USkinnedAsset interface */
	ENGINE_API virtual int32 GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform) const override;
	ENGINE_API virtual void SetSkinWeightProfilesData(int32 LODIndex, FSkinWeightProfilesData& SkinWeightProfilesData) override;	
	ENGINE_API virtual FSkinWeightProfilesData* GetSkinWeightProfilesData(int32 LODIndex) override;

	static ENGINE_API void OnLodStrippingQualityLevelChanged(IConsoleVariable* Variable);

	/*Choose either PerPlatform or PerQuality override. Note: Enable PerQuality override in the Project Settings/ General Settings/ UseSkeletalMeshMinLODPerQualityLevels*/
	ENGINE_API virtual int32 GetMinLodIdx(bool bForceLowestLODIdx = false) const override;
	ENGINE_API virtual int32 GetDefaultMinLod() const override;
	ENGINE_API void SetMinLodIdx(int32 InMinLOD);

	/** Minimum LOD to render. Can be overridden per component as well as set here for all mesh instances here */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMinLod() or USkeletalMesh::SetMinLod().")
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Minimum LOD"))
	FPerPlatformInt MinLod;

	static FName GetMinLodMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MinLod);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual const FPerPlatformInt& GetMinLod() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MinLod, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MinLod;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetMinLod(FPerPlatformInt InMinLod)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MinLod);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MinLod = MoveTemp(InMinLod);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** when true all lods below minlod will still be cooked */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDisableBelowMinLodStripping() or USkeletalMesh::SetDisableBelowMinLodStripping().")
	UPROPERTY(EditAnywhere, Category = LODSettings)
	FPerPlatformBool DisableBelowMinLodStripping;

	static FName GetDisableBelowMinLodStrippingMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DisableBelowMinLodStripping);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual const FPerPlatformBool& GetDisableBelowMinLodStripping() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DisableBelowMinLodStripping, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DisableBelowMinLodStripping;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetDisableBelowMinLodStripping(FPerPlatformBool InDisableBelowMinLodStripping)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DisableBelowMinLodStripping);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DisableBelowMinLodStripping = MoveTemp(InDisableBelowMinLodStripping);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if WITH_EDITORONLY_DATA
	/** Whether this skeletal mesh overrides default LOD streaming settings. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetOverrideLODStreamingSettings() or USkeletalMesh::SetOverrideLODStreamingSettings().")
	UPROPERTY(EditAnywhere, Category=LODSettings)
	bool bOverrideLODStreamingSettings;
	static FName GetOverrideLODStreamingSettingsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bOverrideLODStreamingSettings);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool GetOverrideLODStreamingSettings() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverrideLODStreamingSettings, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bOverrideLODStreamingSettings;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetOverrideLODStreamingSettings(bool bInOverrideLODStreamingSettings)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverrideLODStreamingSettings);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bOverrideLODStreamingSettings = bInOverrideLODStreamingSettings;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Whether we can stream the LODs of this mesh */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetSupportLODStreaming() or USkeletalMesh::SetSupportLODStreaming().")
	UPROPERTY(EditAnywhere, Category=LODSettings, meta=(DisplayName="Stream LODs", EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformBool bSupportLODStreaming;
	static FName GetSupportLODStreamingMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bSupportLODStreaming);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FPerPlatformBool& GetSupportLODStreaming() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SupportLODStreaming, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bSupportLODStreaming;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetSupportLODStreaming(FPerPlatformBool bInSupportLODStreaming)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SupportLODStreaming);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bSupportLODStreaming = MoveTemp(bInSupportLODStreaming);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Maximum number of LODs that can be streamed */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMaxNumStreamedLODs() or USkeletalMesh::SetMaxNumStreamedLODs().")
	UPROPERTY(EditAnywhere, Category=LODSettings, meta=(EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformInt MaxNumStreamedLODs;
	static FName GetMaxNumStreamedLODsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MaxNumStreamedLODs);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FPerPlatformInt& GetMaxNumStreamedLODs() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MaxNumStreamedLODs, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MaxNumStreamedLODs;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetMaxNumStreamedLODs(FPerPlatformInt InMaxNumStreamedLODs)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MaxNumStreamedLODs);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MaxNumStreamedLODs = MoveTemp(InMaxNumStreamedLODs);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Maximum number of LODs below min LOD level that can be saved to optional pak (currently, need to be either 0 or > num of LODs below MinLod) */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMaxNumOptionalLODs() or USkeletalMesh::SetMaxNumOptionalLODs().")
	UPROPERTY(EditAnywhere, Category=LODSettings, meta=(EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformInt MaxNumOptionalLODs;
	static FName GetMaxNumOptionalLODsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MaxNumOptionalLODs);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FPerPlatformInt& GetMaxNumOptionalLODs() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MaxNumOptionalLODs, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MaxNumOptionalLODs;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetMaxNumOptionalLODs(FPerPlatformInt InMaxNumOptionalLODs)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MaxNumOptionalLODs);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MaxNumOptionalLODs = MoveTemp(InMaxNumOptionalLODs);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetLODSettings() or USkeletalMesh::SetLODSettings().")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, BlueprintGetter = GetLODSettings, BlueprintSetter = SetLODSettings, Category = LODSettings)
	TObjectPtr<USkeletalMeshLODSettings> LODSettings;

	static FName GetLODSettingsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, LODSettings);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** The Default Control Rig To Animate with when used in Sequnecer. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDefaultAnimatingRig() or USkeletalMesh::SetDefaultAnimatingRig().")
	UPROPERTY(EditAnywhere, Category = AnimationRig, BlueprintGetter = GetDefaultAnimatingRig, BlueprintSetter = SetDefaultAnimatingRig, meta = (AllowedClasses = "/Script/ControlRigDeveloper.ControlRigEditorAssetInterface,/Script/ControlRig.ControlRigRuntimeAssetInterface"))
	TSoftObjectPtr<UObject> DefaultAnimatingRig;
	static FName GetDefaultAnimatingRigMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DefaultAnimatingRig);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
#endif // WITH_EDITORONLY_DATA

	USkeletalMeshLODSettings* GetLODSettings()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::LODSettings);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
		return LODSettings;
#else
		const bool bCallOutsideOf_WithEditorOnlyData = false;
		ensure(bCallOutsideOf_WithEditorOnlyData);
		return nullptr;
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintGetter)
	const USkeletalMeshLODSettings* GetLODSettings() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::LODSettings, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
		return LODSettings;
#else
		const bool bCallOutsideOf_WithEditorOnlyData = false;
		ensure(bCallOutsideOf_WithEditorOnlyData);
		return nullptr;
#endif // WITH_EDITORONLY_DATA
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetLODSettings(USkeletalMeshLODSettings* InLODSettings);

#if WITH_EDITOR
	/** Get whether this mesh should use LOD streaming for the given platform. Do not use bSupportLODStreaming directly. Call this method instead. USkinnedAsset Interface. */
	ENGINE_API virtual bool GetEnableLODStreaming(const class ITargetPlatform* TargetPlatform) const override;

	/** Get the maximum number of LODs that can be streamed. Do not use MaxNumStreamedLODs directly. Call this method instead. USkinnedAsset Interface. */
	ENGINE_API virtual int32 GetMaxNumStreamedLODs(const class ITargetPlatform* TargetPlatform) const override;

	/** Get the maximum number of optional LODs. Do not use MaxNumOptionalLODs directly. Call this method instead. USkinnedAsset Interface. */
	ENGINE_API virtual int32 GetMaxNumOptionalLODs(const class ITargetPlatform* TargetPlatform) const override;

	/* Build a LOD model for the given render data. USkinnedAsset Interface. */
	ENGINE_API virtual void BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex) override;
#endif

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetDefaultAnimatingRig(TSoftObjectPtr<UObject> InAnimatingRig);

	UFUNCTION(BlueprintGetter)
	ENGINE_API TSoftObjectPtr<UObject> GetDefaultAnimatingRig() const;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetSkelMirrorAxis() or USkeletalMesh::SetSkelMirrorAxis().")
	UPROPERTY(EditAnywhere, Category=Mirroring)
	TEnumAsByte<EAxis::Type> SkelMirrorAxis;


	UE_DEPRECATED(4.27, "Please do not access this member directly; Use UMirrorDataTable for mirroring support")
	UPROPERTY(EditAnywhere, Category=Mirroring)
	TEnumAsByte<EAxis::Type> SkelMirrorFlipAxis;
#endif 

	/** If true, use 32 bit UVs. If false, use 16 bit UVs to save memory */
	UPROPERTY()
	uint8 bUseFullPrecisionUVs_DEPRECATED :1;

	/** If true, tangents will be stored at 16 bit vs 8 bit precision */
	UPROPERTY()
	uint8 bUseHighPrecisionTangentBasis_DEPRECATED : 1;

	UE_DEPRECATED(5.5, "This functionality has been removed. Use FSkeletalMeshLODInfo::bHasBeenSimplified instead.")
	static FName GetHasBeenSimplifiedMemberName()
	{
		return NAME_None;
	}

	UE_DEPRECATED(5.5, "This functionality has been removed. Use FSkeletalMeshLODInfo::bHasBeenSimplified instead.")
	bool GetHasBeenSimplified() const;

	UE_DEPRECATED(5.5, "This functionality has been removed. Use FSkeletalMeshLODInfo::bHasBeenSimplified instead.")
	static void SetHasBeenSimplified(bool bInHasBeenSimplified)
	{
	}

	/** Whether or not the mesh has vertex colors */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetHasVertexColors() or USkeletalMesh::SetHasVertexColors().")
	UPROPERTY()
	uint8 bHasVertexColors:1;

	static FName GetbHasVertexColorsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bHasVertexColors);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Return whether the mesh has vertex colors. USkinnedAsset interface. */
	UFUNCTION(BlueprintPure, Category = Mesh, meta = (DisplayName = "Has Vertex Colors", ScriptName = "HasVertexColors", Keywords = "vertex color"))
	virtual bool GetHasVertexColors() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::HasVertexColors, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bHasVertexColors != 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetHasVertexColors(bool InbHasVertexColors)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::HasVertexColors);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bHasVertexColors = InbHasVertexColors;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	//caching optimization to avoid recalculating in non-editor builds
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::HasActiveClothingAssets() or USkeletalMesh::SetHasActiveClothingAssets().")
	uint8 bHasActiveClothingAssets : 1;

	static FName GetHasActiveClothingAssetsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bHasActiveClothingAssets);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetHasActiveClothingAssets(const bool InbHasActiveClothingAssets)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::HasActiveClothingAssets);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bHasActiveClothingAssets = InbHasActiveClothingAssets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Uses skinned data for collision data. Per poly collision cannot be used for simulation, in most cases you are better off using the physics asset */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetEnablePerPolyCollision() or USkeletalMesh::SetEnablePerPolyCollision().")
	UPROPERTY(EditAnywhere, Category = Physics)
	uint8 bEnablePerPolyCollision : 1;

	static FName GetEnablePerPolyCollisionMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bEnablePerPolyCollision);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool GetEnablePerPolyCollision() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::EnablePerPolyCollision, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bEnablePerPolyCollision != 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetEnablePerPolyCollision(bool bInEnablePerPolyCollision)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::EnablePerPolyCollision);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bEnablePerPolyCollision = bInEnablePerPolyCollision;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
#if WITH_EDITORONLY_DATA
	/** The guid to compute the ddc key, it must be dirty when we change the vertex color. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetVertexColorGuid() or USkeletalMesh::SetVertexColorGuid().")
	UPROPERTY()
	FGuid VertexColorGuid;

	static FName GetVertexColorGuidMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, VertexColorGuid);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	FGuid GetVertexColorGuid() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::VertexColorGuid, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return VertexColorGuid;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetVertexColorGuid(FGuid InVertexColorGuid)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::VertexColorGuid);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VertexColorGuid = InVertexColorGuid;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#endif

	// Physics data for the per poly collision case. In 99% of cases you will not need this and are better off using simple ragdoll collision (physics asset)
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetBodySetup() or USkeletalMesh::SetBodySetup().")
	UPROPERTY(transient)
	TObjectPtr<class UBodySetup> BodySetup;
	static FName GetBodySetupMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, BodySetup);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	class UBodySetup* GetBodySetup() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::BodySetup);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BodySetup;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetBodySetup(class UBodySetup* InBodySetup)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::BodySetup);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BodySetup = InBodySetup;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 *	Physics and collision information used for this USkeletalMesh, set up in Physics Asset Editor.
	 *	This is used for per-bone hit detection, accurate bounding box calculation and ragdoll physics for example.
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetPhysicsAsset() or USkeletalMesh::SetPhysicsAsset().")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, BlueprintGetter = GetPhysicsAsset, Category=Physics)
	TObjectPtr<class UPhysicsAsset> PhysicsAsset;
	
	static FName GetPhysicsAssetMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, PhysicsAsset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	UFUNCTION(BlueprintGetter)
	virtual class UPhysicsAsset* GetPhysicsAsset() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PhysicsAsset);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PhysicsAsset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetPhysicsAsset(class UPhysicsAsset* InPhysicsAsset)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PhysicsAsset);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PhysicsAsset = InPhysicsAsset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Physics asset whose shapes will be used for shadowing when components have bCastCharacterCapsuleDirectShadow or bCastCharacterCapsuleIndirectShadow enabled.
	 * Only spheres and sphyl shapes in the physics asset can be supported.  The more shapes used, the higher the cost of the capsule shadows will be.
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetShadowPhysicsAsset() or USkeletalMesh::SetShadowPhysicsAsset().")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, BlueprintGetter = GetShadowPhysicsAsset, Category=Lighting)
	TObjectPtr<class UPhysicsAsset> ShadowPhysicsAsset;

	static FName GetShadowPhysicsAssetMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, ShadowPhysicsAsset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	UFUNCTION(BlueprintGetter)
	virtual class UPhysicsAsset* GetShadowPhysicsAsset() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ShadowPhysicsAsset);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ShadowPhysicsAsset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetShadowPhysicsAsset(class UPhysicsAsset* InShadowPhysicsAsset)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ShadowPhysicsAsset);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ShadowPhysicsAsset = InShadowPhysicsAsset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Mapping data that is saved */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetNodeMappingData() or USkeletalMesh::SetNodeMappingData().")
	UPROPERTY(EditAnywhere, editfixedsize, BlueprintGetter = GetNodeMappingData, Category=Animation)
	TArray<TObjectPtr<class UNodeMappingContainer>> NodeMappingData;
	static FName GetNodeMappingDataMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, NodeMappingData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TArray<TObjectPtr<class UNodeMappingContainer>>& GetNodeMappingData()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::NodeMappingData);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NodeMappingData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintGetter)
	const TArray<class UNodeMappingContainer*>& GetNodeMappingData() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::NodeMappingData, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NodeMappingData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetNodeMappingData(const TArray<class UNodeMappingContainer*>& InNodeMappingData)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::NodeMappingData);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NodeMappingData = InNodeMappingData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API class UNodeMappingContainer* GetNodeMappingContainer(class UBlueprint* SourceAsset) const;

#if WITH_EDITORONLY_DATA

	/** Importing data and options used for this mesh */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetAssetImportData() or USkeletalMesh::SetAssetImportData().")
	UPROPERTY(EditAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	static FName GetAssetImportDataMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, AssetImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	class UAssetImportData* GetAssetImportData() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::AssetImportData);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AssetImportData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetAssetImportData(class UAssetImportData* InAssetImportData)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::AssetImportData);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AssetImportData = InAssetImportData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static ENGINE_API FText GetSourceFileLabelFromIndex(int32 SourceFileIndex);

	/** Path to the resource used to construct this skeletal mesh */
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	/** Date/Time-stamp of the file from the last import */
	UPROPERTY()
	FString SourceFileTimestamp_DEPRECATED;

	/** Information for thumbnail rendering */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetThumbnailInfo() or USkeletalMesh::SetThumbnailInfo().")
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Getter=GetThumbnailInfo, Setter=SetThumbnailInfo, Category = Thumbnail)
	TObjectPtr<UThumbnailInfo> ThumbnailInfo;
	
	static FName GetThumbnailInfoMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, ThumbnailInfo);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UThumbnailInfo* GetThumbnailInfo() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ThumbnailInfo);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ThumbnailInfo;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	void SetThumbnailInfo(UThumbnailInfo* InThumbnailInfo)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ThumbnailInfo);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ThumbnailInfo = InThumbnailInfo;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Should we use a custom camera transform when viewing this mesh in the tools */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetHasCustomDefaultEditorCamera() or USkeletalMesh::SetHasCustomDefaultEditorCamera().")
	UPROPERTY()
	bool bHasCustomDefaultEditorCamera;

	static FName GetHasCustomDefaultEditorCameraMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, bHasCustomDefaultEditorCamera);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool GetHasCustomDefaultEditorCamera() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::HasCustomDefaultEditorCamera, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bHasCustomDefaultEditorCamera;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetHasCustomDefaultEditorCamera(bool bInHasCustomDefaultEditorCamera)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::HasCustomDefaultEditorCamera);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bHasCustomDefaultEditorCamera = bInHasCustomDefaultEditorCamera;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Default camera location */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDefaultEditorCameraLocation() or USkeletalMesh::SetDefaultEditorCameraLocation().")
	UPROPERTY()
	FVector DefaultEditorCameraLocation;

	static FName GetDefaultEditorCameraLocationMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DefaultEditorCameraLocation);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FVector& GetDefaultEditorCameraLocation() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraLocation, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DefaultEditorCameraLocation;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetDefaultEditorCameraLocation(FVector InDefaultEditorCameraLocation)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraLocation);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultEditorCameraLocation = InDefaultEditorCameraLocation;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Default camera rotation */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDefaultEditorCameraRotation() or USkeletalMesh::SetDefaultEditorCameraRotation().")
	UPROPERTY()
	FRotator DefaultEditorCameraRotation;

	static FName GetDefaultEditorCameraRotationMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DefaultEditorCameraRotation);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FRotator& GetDefaultEditorCameraRotation() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraRotation, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DefaultEditorCameraRotation;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetDefaultEditorCameraRotation(FRotator InDefaultEditorCameraRotation)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraRotation);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultEditorCameraRotation = InDefaultEditorCameraRotation;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Default camera look at */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDefaultEditorCameraLookAt() or USkeletalMesh::SetDefaultEditorCameraLookAt().")
	UPROPERTY()
	FVector DefaultEditorCameraLookAt;

	static FName GetDefaultEditorCameraLookAtMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DefaultEditorCameraLookAt);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FVector& GetDefaultEditorCameraLookAt() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraLookAt, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DefaultEditorCameraLookAt;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetDefaultEditorCameraLookAt(FVector InDefaultEditorCameraLookAt)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraLookAt);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultEditorCameraLookAt = InDefaultEditorCameraLookAt;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Default camera ortho zoom */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetDefaultEditorCameraOrthoZoom() or USkeletalMesh::SetDefaultEditorCameraOrthoZoom().")
	UPROPERTY()
	float DefaultEditorCameraOrthoZoom;

	static FName GetDefaultEditorCameraOrthoZoomMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, DefaultEditorCameraOrthoZoom);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	float GetDefaultEditorCameraOrthoZoom() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraOrthoZoom, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DefaultEditorCameraOrthoZoom;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetDefaultEditorCameraOrthoZoom(float InDefaultEditorCameraOrthoZoom)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultEditorCameraOrthoZoom);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultEditorCameraOrthoZoom = InDefaultEditorCameraOrthoZoom;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

 
	/* Attached assets component for this mesh */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetPreviewAttachedAssetContainer() or USkeletalMesh::SetPreviewAttachedAssetContainer().")
	UPROPERTY()
	FPreviewAssetAttachContainer PreviewAttachedAssetContainer;

	FPreviewAssetAttachContainer& GetPreviewAttachedAssetContainer()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PreviewAttachedAssetContainer);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PreviewAttachedAssetContainer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FPreviewAssetAttachContainer& GetPreviewAttachedAssetContainer() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PreviewAttachedAssetContainer, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PreviewAttachedAssetContainer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetPreviewAttachedAssetContainer(const FPreviewAssetAttachContainer& InPreviewAttachedAssetContainer)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PreviewAttachedAssetContainer);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PreviewAttachedAssetContainer = InPreviewAttachedAssetContainer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * If true on post load we need to calculate resolution independent Display Factors from the
	 * loaded LOD screen sizes.
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetRequiresLODScreenSizeConversion() or USkeletalMesh::SetRequiresLODScreenSizeConversion().")
	uint32 bRequiresLODScreenSizeConversion : 1;

	bool GetRequiresLODScreenSizeConversion() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RequiresLODScreenSizeConversion, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bRequiresLODScreenSizeConversion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetRequiresLODScreenSizeConversion(bool bInRequiresLODScreenSizeConversion)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RequiresLODScreenSizeConversion);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bRequiresLODScreenSizeConversion = bInRequiresLODScreenSizeConversion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * If true on post load we need to calculate resolution independent LOD hysteresis from the
	 * loaded LOD hysteresis.
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetRequiresLODHysteresisConversion() or USkeletalMesh::SetRequiresLODHysteresisConversion().")
	uint32 bRequiresLODHysteresisConversion : 1;

	bool GetRequiresLODHysteresisConversion() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RequiresLODHysteresisConversion, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bRequiresLODHysteresisConversion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetRequiresLODHysteresisConversion(bool bInRequiresLODHysteresisConversion)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RequiresLODHysteresisConversion);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bRequiresLODHysteresisConversion = bInRequiresLODHysteresisConversion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#endif // WITH_EDITORONLY_DATA

	/**
	 * If true, a ray tracing acceleration structure will be built for this mesh and it may be used in ray tracing effects
	 */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = RayTracing)
	uint8 bSupportRayTracing : 1;

	/** USkinnedAsset interface. */
	virtual bool GetSupportRayTracing() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::bSupportRayTracing, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bSupportRayTracing;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetSupportRayTracing(bool InSupportRayTracing)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::bSupportRayTracing);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bSupportRayTracing = InSupportRayTracing;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * LOD bias for ray tracing. When non-zero, a different LOD level other than the predicted LOD level will be used for ray tracing. Advanced features like morph targets and cloth simulation may not work properly.
	 */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = RayTracing)
	int32 RayTracingMinLOD;

	/** USkinnedAsset interface. */
	virtual int32 GetRayTracingMinLOD() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RayTracingMinLOD, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RayTracingMinLOD;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetRayTracingMinLOD(int32 InRayTracingMinLOD)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RayTracingMinLOD);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RayTracingMinLOD = InRayTracingMinLOD;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Set the strategy used for storing the additional cloth deformer mappings depending on the desired use of Raytracing LOD bias.
	 * This parameter is only used in relation to raytracing of the cloth sections.
	 */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = RayTracing)
	EClothLODBiasMode ClothLODBiasMode;

	EClothLODBiasMode GetClothLODBiasMode() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ClothLODBiasMode, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClothLODBiasMode;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetClothLODBiasMode(EClothLODBiasMode InClothLODBiasMode)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::ClothLODBiasMode);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ClothLODBiasMode = InClothLODBiasMode;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMorphTargets() or USkeletalMesh::SetMorphTargets().")
	UPROPERTY(BlueprintGetter = GetMorphTargetsPtrConv, BlueprintSetter = SetMorphTargets, Category = Mesh)
	TArray<TObjectPtr<UMorphTarget>> MorphTargets;

	static FName GetMorphTargetsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MorphTargets);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual TArray<TObjectPtr<UMorphTarget>>& GetMorphTargets() override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargets);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MorphTargets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual const TArray<TObjectPtr<UMorphTarget>>& GetMorphTargets() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargets, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MorphTargets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// NOTE: BP compiler doesn't support TObjectPtr as an argument type for UFUNCTION so this converting call is
	// required. For many morphs, this can be expensive, since it needs to resolve _all_ TObjectPtrs and construct a new
	// array for it.
	UFUNCTION(BlueprintGetter)
	TArray<UMorphTarget*> GetMorphTargetsPtrConv() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargets, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MorphTargets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	void SetMorphTargets(const TArray<UMorphTarget*>& InMorphTargets)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargets);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MorphTargets = InMorphTargets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 *	Returns the list of all morph targets of this skeletal mesh
	 *  @return	The list of morph targets
	 */
	UFUNCTION(BlueprintPure, Category = Mesh, meta = (DisplayName = "Get All Morph Target Names", ScriptName = "GetAllMorphTargetNames", Keywords = "morph shape"))
	ENGINE_API TArray<FString> K2_GetAllMorphTargetNames() const;

	/** A fence which is used to keep track of the rendering thread releasing the static mesh resources. */
	FRenderCommandFence ReleaseResourcesFence;

	/** New Reference skeleton type **/
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetRefSkeleton() or USkeletalMesh::SetRefSkeleton().")
	FReferenceSkeleton RefSkeleton;

	static FName GetRefSkeletonMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, RefSkeleton);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual FReferenceSkeleton& GetRefSkeleton() override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RefSkeleton);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RefSkeleton;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual const FReferenceSkeleton& GetRefSkeleton() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RefSkeleton, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RefSkeleton;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	ENGINE_API void SetRefSkeleton(const FReferenceSkeleton& InRefSkeleton);

	/** Map of morph target name to index into USkeletalMesh::MorphTargets**/
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMorphTargetIndexMap() or USkeletalMesh::SetMorphTargetIndexMap().")
	TMap<FName, int32> MorphTargetIndexMap;

	static FName GetMorphTargetIndexMapMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MorphTargetIndexMap);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TMap<FName, int32>& GetMorphTargetIndexMap()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargetIndexMap);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MorphTargetIndexMap;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const TMap<FName, int32>& GetMorphTargetIndexMap() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargetIndexMap, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MorphTargetIndexMap;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetMorphTargetIndexMap(const TMap<FName, int32>& InMorphTargetIndexMap)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MorphTargetIndexMap);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MorphTargetIndexMap = InMorphTargetIndexMap;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Reference skeleton precomputed bases. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetRefBasesInvMatrix() or USkeletalMesh::SetRefBasesInvMatrix().")
	TArray<FMatrix44f> RefBasesInvMatrix;

	static FName GetRefBasesInvMatrixMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, RefBasesInvMatrix);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual TArray<FMatrix44f>& GetRefBasesInvMatrix() override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RefBasesInvMatrix);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return RefBasesInvMatrix;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** USkinnedAsset interface. */
	virtual const TArray<FMatrix44f>& GetRefBasesInvMatrix() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RefBasesInvMatrix, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RefBasesInvMatrix;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetRefBasesInvMatrix(const TArray<FMatrix44f>& InRefBasesInvMatrix)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::RefBasesInvMatrix);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RefBasesInvMatrix = InRefBasesInvMatrix;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if WITH_EDITORONLY_DATA

	/** Height offset for the floor mesh in the editor */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetFloorOffset() or USkeletalMesh::SetFloorOffset().")
	UPROPERTY()
	float FloorOffset;

	static FName GetFloorOffsetMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, FloorOffset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	float GetFloorOffset() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::FloorOffset, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FloorOffset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetFloorOffset(float InFloorOffset)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::FloorOffset);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FloorOffset = InFloorOffset;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Legacy clothing asset data, will be converted to new assets after loading */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use USkeletalMesh::GetClothingAssets_DEPRECATED().")
	UPROPERTY()
	TArray<FClothingAssetData_Legacy>		ClothingAssets_DEPRECATED;

	TArray<FClothingAssetData_Legacy>& GetClothingAssets_DEPRECATED()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClothingAssets_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	// The visual size of the bones in the viewport (saved between sessions). This is set from the viewport Character>Bones menu
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use USkeletalMesh::GetBoneDrawSize() or USkeletalMesh::SetBoneDrawSize().")
	UPROPERTY()
	float BoneDrawSize = 1.0f;

	float GetBoneDrawSize() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BoneDrawSize;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetBoneDrawSize(float Size)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BoneDrawSize = Size;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	/** Animation Blueprint class to run as a post process for this mesh.
	 *  This blueprint will be ran before physics, but after the main
	 *  anim instance for any skeletal mesh component using this mesh.
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetPostProcessAnimBlueprint() or USkeletalMesh::SetPostProcessAnimBlueprint().")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Animation)
	TSubclassOf<UAnimInstance> PostProcessAnimBlueprint;

private:
	/*
	 * Max LOD level that post-process anim graphs are evaluated.
	 * For example if you have the threshold set to 2, it will evaluate until including LOD 2 (based on 0 index). In case the LOD level gets set to 3, it will stop evaluating the post-process anim graph.
	 * Setting it to -1 will always evaluate it and disable LODing.
	 */
	UPROPERTY(EditAnywhere, Category = Animation, meta = (DisplayName = "Post-Process LOD Threshold"))
	int32 PostProcessAnimBPLODThreshold = INDEX_NONE;

public:
	UE_DEPRECATED(5.6, "Use GetPostProcessAnimGraphLODThreshold() instead.")
	int32 GetPostProcessAnimBPLODThreshold() const { return GetPostProcessAnimGraphLODThreshold(); }
	UE_DEPRECATED(5.6, "Use SetPostProcessAnimGraphLODThreshold() instead.")
	void SetPostProcessAnimBPLODThreshold(int32 InPostProcessAnimBPLODThreshold) { SetPostProcessAnimGraphLODThreshold(InPostProcessAnimBPLODThreshold); }
	UE_DEPRECATED(5.6, "Use ShouldEvaluatePostProcessAnimGraph() instead.")
	bool ShouldEvaluatePostProcessAnimBP(int32 LODLevel) const { return ShouldEvaluatePostProcessAnimGraph(LODLevel); }

	ENGINE_API int32 GetPostProcessAnimGraphLODThreshold() const;
	ENGINE_API void SetPostProcessAnimGraphLODThreshold(int32 LODThreshold);
	ENGINE_API bool ShouldEvaluatePostProcessAnimGraph(int32 LODLevel) const;

	static FName GetPostProcessAnimBlueprintMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, PostProcessAnimBlueprint);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TSubclassOf<UAnimInstance> GetPostProcessAnimBlueprint() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PostProcessAnimBlueprint, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PostProcessAnimBlueprint;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetPostProcessAnimBlueprint(TSubclassOf<UAnimInstance> InPostProcessAnimBlueprint)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::PostProcessAnimBlueprint);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PostProcessAnimBlueprint = InPostProcessAnimBlueprint;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if WITH_EDITOR
	/** If the given section of the specified LOD has a clothing asset, unbind it and remove it from the asset array if it is no longer used. */
	ENGINE_API void RemoveClothingAsset(int32 InLodIndex, int32 InSectionIndex);

	/**
	* Clothing used to require the original section to be hidden and duplicated to a new rendered
	* section. This was mainly due to an older requirement that we use new render data so the
	* duplicated section allowed us not to destroy the original data. This method will undo this
	* process and restore the mesh now that this is no longer necessary.
	*/
	ENGINE_API void RemoveLegacyClothingSections();

	/*
	* Handle some common preparation steps between async post load and async build. USkinnedAsset interface.
	*/
	ENGINE_API virtual void PrepareForAsyncCompilation() override;

	/** Returns false if there is currently an async task running. USkinnedAsset interface. */
	ENGINE_API virtual bool IsAsyncTaskComplete() const override;

#endif // WITH_EDITOR

	/**
	 * Given an LOD and section index, retrieve a clothing asset bound to that section.
	 * If no clothing asset is in use, returns nullptr
	 */
	ENGINE_API UClothingAssetBase* GetSectionClothingAsset(int32 InLodIndex, int32 InSectionIndex);
	ENGINE_API const UClothingAssetBase* GetSectionClothingAsset(int32 InLodIndex, int32 InSectionIndex) const;

	/**
	 * Clothing assets imported to this mesh. May or may not be in use currently on the mesh.
	 * Ordering not guaranteed, use the provided getters to access elements in this array
	 * whenever possible
	 */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use USkeletalMesh::GetMeshClothingAssets() or USkeletalMesh::SetMeshClothingAssets().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetMeshClothingAssets, BlueprintSetter = SetMeshClothingAssets, Category = Clothing)
	TArray<TObjectPtr<UClothingAssetBase>> MeshClothingAssets;

	static FName GetMeshClothingAssetsMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, MeshClothingAssets);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TArray<TObjectPtr<UClothingAssetBase>>& GetMeshClothingAssets()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MeshClothingAssets);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MeshClothingAssets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintGetter)
	const TArray<UClothingAssetBase*>& GetMeshClothingAssets() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MeshClothingAssets, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MeshClothingAssets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	void SetMeshClothingAssets(const TArray<UClothingAssetBase*>& InMeshClothingAssets)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::MeshClothingAssets);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MeshClothingAssets = InMeshClothingAssets;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}



	/** Get a clothing asset from its associated GUID (returns nullptr if no match is found) */
	ENGINE_API UClothingAssetBase* GetClothingAsset(const FGuid& InAssetGuid) const;

	/* Get the index in the clothing asset array for a given asset (INDEX_NONE if InAsset isn't in the array) */
	ENGINE_API int32 GetClothingAssetIndex(UClothingAssetBase* InAsset) const;

	/* Get the index in the clothing asset array for a given asset GUID (INDEX_NONE if there is no match) */
	ENGINE_API int32 GetClothingAssetIndex(const FGuid& InAssetGuid) const;

	/* Get whether or not any bound clothing assets exist for this mesh **/
	ENGINE_API bool HasActiveClothingAssets() const;

	/* Get whether or not any bound clothing assets exist for this mesh's given LOD**/
	ENGINE_API bool HasActiveClothingAssetsForLOD(int32 LODIndex) const;

	/* Compute whether or not any bound clothing assets exist for this mesh **/
	ENGINE_API bool ComputeActiveClothingAssets() const;

	/** Populates OutClothingAssets with all clothing assets that are mapped to sections in the mesh. */
	ENGINE_API void GetClothingAssetsInUse(TArray<UClothingAssetBase*>& OutClothingAssets) const;

	/** Adds an asset to this mesh with validation and event broadcast */
	ENGINE_API void AddClothingAsset(UClothingAssetBase* InNewAsset);

	static FName GetSamplingInfoMemberName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(USkeletalMesh, SamplingInfo);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FSkeletalMeshSamplingInfo& GetSamplingInfo() const 
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SamplingInfo, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SamplingInfo;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	FSkeletalMeshSamplingInfo& GetSamplingInfoInternal()
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SamplingInfo);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SamplingInfo;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

public:
#if WITH_EDITOR
	void SetSamplingInfo(const FSkeletalMeshSamplingInfo& InSamplingInfo)
	{ 
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SamplingInfo);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SamplingInfo = InSamplingInfo;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	const FOnPreMeshChange& GetOnPreMeshChange() const { return OnPreMeshChange; }
	FOnPreMeshChange& GetOnPreMeshChange() { return OnPreMeshChange; }
	const FOnMeshChanged& GetOnMeshChanged() const { return OnMeshChanged; }
	FOnMeshChanged& GetOnMeshChanged() { return OnMeshChanged; }
#endif

	/** 
	True if this mesh LOD needs to keep it's data on CPU. USkinnedAsset interface.
	*/
	ENGINE_API virtual bool NeedCPUData(int32 LODIndex)const override;

protected:

	/** Defines if and how to generate a set of precomputed data allowing targeted and fast sampling of this mesh on the CPU. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = "Sampling", meta=(ShowOnlyInnerProperties))
	FSkeletalMeshSamplingInfo SamplingInfo;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category=SkeletalMesh)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of user data stored with the asset */
	UE_DEPRECATED(5.7, "EditorOnly data will become private in the future. Use the GetAssetUserDataEditorOnly() accessor instead.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category=SkeletalMesh)
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;

	TArray<TObjectPtr<UAssetUserData>>& GetAssetUserDataEditorOnly()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AssetUserDataEditorOnly;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const TArray<TObjectPtr<UAssetUserData>>& GetAssetUserDataEditorOnly() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AssetUserDataEditorOnly;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
#if WITH_EDITOR
	FOnPreMeshChange OnPreMeshChange;
	FOnMeshChanged OnMeshChanged;
#endif


	friend struct FSkeletalMeshUpdateContext;
	friend class FSkeletalMeshUpdate;

private:
	/** 
	 *	Array of named socket locations, set up in editor and used as a shortcut instead of specifying 
	 *	everything explicitly to AttachComponent in the SkeletalMeshComponent. 
	 */
	UPROPERTY()
	TArray<TObjectPtr<class USkeletalMeshSocket>> Sockets;

	/** Cached matrices from GetComposedRefPoseMatrix */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	TArray<FMatrix> CachedComposedRefPoseMatrices;

	TArray<FMatrix>& GetCachedComposedRefPoseMatrices() 
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::CachedComposedRefPoseMatrices);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CachedComposedRefPoseMatrices;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const TArray<FMatrix>& GetCachedComposedRefPoseMatrices() const
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::CachedComposedRefPoseMatrices, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CachedComposedRefPoseMatrices;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

public:
	/**
	* Initialize the mesh's render resources.
	*/
	ENGINE_API void InitResources();

	/**
	* Releases the mesh's render resources.
	*/
	ENGINE_API void ReleaseResources();

	/**
	* Flush current render state
	*/
	ENGINE_API void FlushRenderState();

	/** Release CPU access version of buffer */
	ENGINE_API void ReleaseCPUResources();

	/** Allocate a new FSkeletalMeshRenderData and assign to SkeletalMeshRenderData member.  */
	ENGINE_API void AllocateResourceForRendering();

	/**
	 * Returns the UV channel data for a given material index. Used by the texture streamer.
	 * This data applies to all lod-section using the same material. USkinnedAsset interface.
	 *
	 * @param MaterialIndex		the material index for which to get the data for.
	 * @return the data, or null if none exists.
	 */
	ENGINE_API virtual const FMeshUVChannelInfo* GetUVChannelData(int32 MaterialIndex) const override;

	//~ Begin UObject Interface.
#if WITH_EDITOR
private:
	int32 PostEditChangeStackCounter;

	//When loading a legacy asset (saved before the skeletalmesh build refactor), we need to create the user sections data.
	//This function should be call only in the PostLoad
	void CreateUserSectionsDataForLegacyAssets();


	/*
	 * This function will enforce the user section data is coherent with the sections.
	 */
	void PostLoadValidateUserSectionData();

	/*
	 * This function will ensure we have valid tangent in all LODs. If we found an invalid tangent axis, we will try to set it with the cross product of the two other axis.
	 * If the two other axes are also bad, it will simply apply the triangle normals, which will facet the mesh.
	 * It will validate tangents only for assets that do not have source build data. (This means assets imported before the build refactor that was done in UE 4.24.)
	 * @note - If it finds a bad normal, it will LOG a warning to let the user know they have to re-import their mesh.
	 */
	void PostLoadVerifyAndFixBadTangent();


	/** After all loading is done and fixups completed, attempt to recover any LOD models into source model slots that
	 *  are still empty.
	 */
	void PostLoadRecoverConvertLODModelsToMeshDescription();

	/** 
	 *  Check if Quality and Platform MinLOD settings are valid 
	 */
	void CheckForValidMinLODs(FPerQualityLevelInt& QualityLocalMinLOD, FPerPlatformInt& LocalMinLOD, int32& OutMinAvailableLOD, TArray<TPair<int32, FName>>& OutInvalidMinLODs) const;
		
public:
	/*
	 * This function will enforce valid material index in the sections and the LODMaterialMap of all LOD.
	 */
	ENGINE_API void ValidateAllLodMaterialIndexes();
	
private:
	//We want to avoid calling post edit change multiple time during import and build process.

	/*
	 * This function will increment the PostEditChange stack counter.
	 * It will return the stack counter value. (the value should be >= 1)
	 */
	ENGINE_API int32 StackPostEditChange();
	
	/*
	 * This function will decrement the stack counter.
	 * It will return the stack counter value. (the value should be >= 0)
	 */
	ENGINE_API int32 UnStackPostEditChange();

	friend class FScopedSkeletalMeshPostEditChange;
public:
	/**
	* If derive data cache key do not match, regenerate derived data and re-create any render state based on that.
	*/
	ENGINE_API void Build();

	/**
	* Lock the skeletalmesh properties until the caller trigger the Event parameter.
	* The returned Event memory is all handle by the LockPropertiesUntil function, do not  return the event to the pool
	* 
	* @Return - Return the Event caller must trigger to unlock the skeletal mesh properties
	*/
	ENGINE_API FEvent* LockPropertiesUntil();

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:	
	bool bTransacting = false;
public:
	bool IsTransacting() const;
	
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;

	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;

	ENGINE_API void UpdateGenerateUpToData();

	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

#endif // WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass);
#endif
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	ENGINE_API virtual FString GetDesc() override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	//~ End UObject Interface.

	//~ Begin UStreamableRenderAsset Interface.
	ENGINE_API virtual int32 CalcCumulativeLODSize(int32 NumLODs) const override;
	ENGINE_API virtual FIoFilenameHash GetMipIoFilenameHash(const int32 MipIndex) const override;
	ENGINE_API virtual bool DoesMipDataExist(const int32 MipIndex) const override;
	ENGINE_API virtual bool StreamOut(int32 NewMipCount) override;
	ENGINE_API virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) override;
	ENGINE_API virtual bool HasPendingRenderResourceInitialization() const;
	virtual EStreamableRenderAssetType GetRenderAssetType() const final override { return EStreamableRenderAssetType::SkeletalMesh; }
	//~ End UStreamableRenderAsset Interface.

	/**
	* Cancels any pending static mesh streaming actions if possible.
	* Returns when no more async loading requests are in flight.
	*/
	static ENGINE_API void CancelAllPendingStreamingActions();


	/** Setup-only routines - not concerned with the instance. */

	ENGINE_API void CalculateInvRefMatrices();

#if WITH_EDITORONLY_DATA
	// Asset registry information for morph target names
	ENGINE_API static const FName MorphNamesTag;
	ENGINE_API static const FString MorphNamesTagDelimiter;

	// Asset registry information for scalar material param names
	ENGINE_API static const FName MaterialParamNamesTag;
	ENGINE_API static const FString MaterialParamNamesTagDelimiter;
#endif

#if WITH_EDITOR
	/** Calculate the required bones for a Skeletal Mesh LOD, including possible extra influences */
	static ENGINE_API void CalculateRequiredBones(FSkeletalMeshLODModel& LODModel, const struct FReferenceSkeleton& RefSkeleton, const TMap<FBoneIndexType, FBoneIndexType> * BonesToRemove);

	/**
	 *	Add a skeletal socket object to this SkeletalMesh, and optionally promotes it to USkeleton socket.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void AddSocket(USkeletalMeshSocket* InSocket, bool bAddToSkeleton=false);
#endif // WITH_EDITOR

	/** 
	 *	Find a socket object in this SkeletalMesh by name. 
	 *	Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	 */
	ENGINE_API virtual USkeletalMeshSocket* FindSocket(FName InSocketName) const override;

	/**
	*	Find a socket object in this SkeletalMesh by name.
	*	Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	*   Also returns the index for the socket allowing for future fast access via GetSocketByIndex()
	*/
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API USkeletalMeshSocket* FindSocketAndIndex(FName InSocketName, int32& OutIndex) const;

	/**
	*	Find a socket object and asscociated info in this SkeletalMesh by name.
	*	Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	*	Also returns the index for the socket allowing for future fast access via GetSocketByIndex()
	*	Also rteturns the socket loca transform and the bone index (if any)
	*/
	ENGINE_API virtual USkeletalMeshSocket* FindSocketInfo(FName InSocketName, FTransform& OutTransform, int32& OutBoneIndex, int32& OutIndex) const override;

	/** Returns the number of sockets available. Both on this mesh and it's skeleton. */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API int32 NumSockets() const;

	/** Returns a socket by index. Max index is NumSockets(). The meshes sockets are accessed first, then the skeletons.  */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API USkeletalMeshSocket* GetSocketByIndex(int32 Index) const;

	/**
	 * Returns vertex color data by position.
	 * For matching to reimported meshes that may have changed or copying vertex paint data from mesh to mesh.
	 *
	 *	@return	VertexColorData		Returns a map of vertex position and their associated color.
	 */
	ENGINE_API TMap<FVector3f, FColor> GetVertexColorData(const uint32 PaintingMeshLODIndex = 0) const;

	/** Called to rebuild an out-of-date or invalid socket map */
	ENGINE_API void RebuildSocketMap();

	// @todo document
	ENGINE_API FMatrix GetRefPoseMatrix( int32 BoneIndex ) const;

	/** 
	 * Get the component orientation of a bone or socket. Transforms by parent bones.
	 * USkinnedAsset interface.
	 */
	ENGINE_API virtual FMatrix GetComposedRefPoseMatrix( FName InBoneName ) const override;
	ENGINE_API virtual FMatrix GetComposedRefPoseMatrix( int32 InBoneIndex ) const override;


	/**
	 * Returns the mesh only socket list - this ignores any sockets in the skeleton
	 * Return value is a non-const reference so the socket list can be changed
	 */
	ENGINE_API TArray<TObjectPtr<USkeletalMeshSocket>>& GetMeshOnlySocketList();

	/**
	 * Const version
	 * Returns the mesh only socket list - this ignores any sockets in the skeleton
	 * Return value is a non-const reference so the socket list can be changed
	 */
	ENGINE_API const TArray<USkeletalMeshSocket*>& GetMeshOnlySocketList() const;

	/**
	* Returns the "active" socket list - all sockets from this mesh plus all non-duplicates from the skeleton
	* Const ref return value as this cannot be modified externally. USkinnedAsset interface.
	*/
	ENGINE_API virtual TArray<USkeletalMeshSocket*> GetActiveSocketList() const override;

#if WITH_EDITOR
	/**
	* Makes sure all attached objects are valid and removes any that aren't.
	*
	* @return		NumberOfBrokenAssets
	*/
	ENGINE_API int32 ValidatePreviewAttachedObjects();

	/**
	 * Removes a specified section from the skeletal mesh, this is a destructive action
	 *
	 * @param InLodIndex Lod index to remove section from
	 * @param InSectionIndex Section index to remove
	 */
	ENGINE_API void RemoveMeshSection(int32 InLodIndex, int32 InSectionIndex);

#endif // #if WITH_EDITOR

	/**
	* Verify SkeletalMeshLOD is set up correctly	
	*/
	UE_DEPRECATED(5.5, "Use GetLODInfo()->ScreenSize to manually verify a LOD's screen size settings.")
	ENGINE_API void DebugVerifySkeletalMeshLOD();

	/**
	 * Find a named MorphTarget from the MorphSets array in the SkinnedMeshComponent.
	 * This searches the array in the same way as FindAnimSequence. USkinnedAsset interface.
	 *
	 * @param MorphTargetName Name of MorphTarget to look for.
	 *
	 * @return Pointer to found MorphTarget. Returns NULL if could not find target with that name.
	 */
	ENGINE_API virtual UMorphTarget* FindMorphTarget(FName MorphTargetName) const override;
	ENGINE_API UMorphTarget* FindMorphTargetAndIndex(FName MorphTargetName, int32& OutIndex) const;

	/* Initialize morph targets and rebuild the render data */
	ENGINE_API void InitMorphTargetsAndRebuildRenderData();

	/** if name conflicts, it will overwrite the reference */
	ENGINE_API bool RegisterMorphTarget(UMorphTarget* MorphTarget, bool bInvalidateRenderData = true);

	ENGINE_API void UnregisterMorphTarget(UMorphTarget* MorphTarget, bool bInvalidateRenderData = true);

	ENGINE_API void UnregisterAllMorphTarget();

	/** Initialize MorphSets look up table : MorphTargetIndexMap */
	ENGINE_API void InitMorphTargets(bool bInKeepEmptyMorphTargets = false);

#if WITH_EDITOR
	/** Remove the morph targets with the specified names */
	ENGINE_API bool RemoveMorphTargets(TConstArrayView<FName> InMorphTargetNames, bool bInRebuildRenderMesh = false);

	/**
	 * Remove a single morph target from all LODs on this mesh.
	 * @param InMorphTargetName The name of the morph target to remove.
	 * @param bInRebuildRenderMesh Set to true if rebuilding the render mesh as a part of the operation is desired. 
	 *    This is needed if the result of the morph removal on the mesh is to be visible afterward. 
	 * @return true if the morph target was successfully removed. 
	 */
	UFUNCTION(BlueprintCallable, Category = Mesh, meta=(DevelopmentOnly))
	bool RemoveMorphTarget(const FName InMorphTargetName, bool bInRebuildRenderMesh = false)
	{
		return RemoveMorphTargets({InMorphTargetName}, bInRebuildRenderMesh);
	}
	
	/** Rename the morph target named InOldName to InNewName */
	UFUNCTION(BlueprintCallable, Category = Mesh, meta=(DevelopmentOnly))
	ENGINE_API bool RenameMorphTarget(FName InOldName, FName InNewName);
#endif

	/** 
	 * Checks whether the provided section is using APEX cloth. if bCheckCorrespondingSections is true
	 * disabled sections will defer to correspond sections to see if they use cloth (non-cloth sections
	 * are disabled and another section added when cloth is enabled, using this flag allows for a check
	 * on the original section to succeed)
	 * @param InSectionIndex Index to check
	 * @param bCheckCorrespondingSections Whether to check corresponding sections for disabled sections
	 */
	UFUNCTION(BlueprintCallable, Category="Clothing Simulation")
	ENGINE_API bool IsSectionUsingCloth(int32 InSectionIndex, bool bCheckCorrespondingSections = true) const;

	ENGINE_API void CreateBodySetup();

#if WITH_EDITOR
	/** Trigger a physics build to ensure per poly collision is created */
	ENGINE_API void BuildPhysicsData();
	ENGINE_API void AddBoneToReductionSetting(int32 LODIndex, const TArray<FName>& BoneNames);
	ENGINE_API void AddBoneToReductionSetting(int32 LODIndex, FName BoneName);
#endif
	
#if WITH_EDITORONLY_DATA
	/** Convert legacy screen size (based on fixed resolution) into screen size (diameter in screen units) */
	ENGINE_API void ConvertLegacyLODScreenSize();
#endif
	

	//~ Begin Interface_CollisionDataProvider Interface
	ENGINE_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	ENGINE_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	ENGINE_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool WantsNegXTriMesh() override
	{
		return true;
	}
	virtual void GetMeshId(FString& OutMeshId) override { OutMeshId = TEXT("659304F7CF5D4F31BCC54A9FBB0B49E0"); }
	//~ End Interface_CollisionDataProvider Interface

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITOR
private:	
	/** Called after derived mesh data is cached */
	FOnPostMeshCache PostMeshCached;
public:
	/** Get multicast delegate broadcast post to mesh data caching */
	FOnPostMeshCache& OnPostMeshCached() { return PostMeshCached; }

	/**
	* Force the creation of a new GUID use to build the derive data cache key.
	* Next time a build happen the whole skeletal mesh will be rebuild.
	* Use this when you change stuff not in the skeletal mesh ddc key, like the geometry (import, re-import)
	* Every big data should not be in the ddc key and should use this function, because its slow to create a key with big data.
	*/
	ENGINE_API void InvalidateDeriveDataCacheGUID();

	/** Generate the derived data key for the given platform. USkinnedAsset interface. */
	ENGINE_API virtual FString BuildDerivedDataKey(const ITargetPlatform* TargetPlatform) override;

	/** Generate the derived data key used to fetch derived data */
	ENGINE_API FString GetDerivedDataKey();

	/** Get a skeletal mesh render data for requested platform. */
	ENGINE_API static void GetPlatformSkeletalMeshRenderData(const ITargetPlatform* Platform, FScopedSkeletalMeshRenderData& Out);
#endif 

private:

#if WITH_EDITOR
	/** Generate SkeletalMeshRenderData from ImportedModel */
	ENGINE_API void CacheDerivedData(FSkinnedAssetCompilationContext* ContextPtr);
	
	/**
	 * Caches derived renderable for cooked platforms currently active.
	 */
	ENGINE_API void PrepareDerivedDataForActiveTargetPlatforms();

	/**
	 * Initial step for the building process - Can't be done in parallel. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void BeginBuildInternal(FSkinnedAssetBuildContext& Context) override;

	/**
	 * Thread-safe part. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void ExecuteBuildInternal(FSkinnedAssetBuildContext& Context) override;

	/**
	 * Complete the building process - Can't be done in parallel. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void FinishBuildInternal(FSkinnedAssetBuildContext& Context) override;

	/**
	 * Copy build/load context result data to the skeletalmesh member on the game thread - Can't be done in parallel.
	 */
	ENGINE_API void ApplyFinishBuildInternalData(FSkinnedAssetCompilationContext* ContextPtr);



	/**
	 * Initial step for the building process - Can't be done in parallel. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void BeginAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) override;

	/**
	 * Thread-safe part. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void ExecuteAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) override;

	/**
	 * Complete the building process - Can't be done in parallel. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void FinishAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) override;

#endif

	/** Utility function to help with building the combined socket list */
	ENGINE_API bool IsSocketOnMesh( const FName& InSocketName ) const;

	/**
	* Create a new GUID for the source Model data, regenerate derived data and re-create any render state based on that.
	*/
	ENGINE_API void InvalidateRenderData();

#if WITH_EDITORONLY_DATA
	/**
	* In older data, the bEnableShadowCasting flag was stored in LODInfo
	* so it needs moving over to materials
	*/
	ENGINE_API void MoveDeprecatedShadowFlagToMaterials();

	/*
	* Ask the reference skeleton to rebuild the NameToIndexMap array. This is use to load old package before this array was created.
	*/
	ENGINE_API void RebuildRefSkeletonNameToIndexMap();

	/*
	* In version prior to FEditorObjectVersion::RefactorMeshEditorMaterials
	* The material slot is containing the "Cast Shadow" and the "Recompute Tangent" flag
	* We move those flag to sections to allow artist to control those flag at section level
	* since its a section flag.
	*/
	ENGINE_API void MoveMaterialFlagsToSections();

#endif // WITH_EDITORONLY_DATA

	/**
	* Test whether all the flags in an array are identical (could be moved to Array.h?)
	*/
	ENGINE_API bool AreAllFlagsIdentical( const TArray<bool>& BoolArray ) const;

#if WITH_EDITOR
public:
	/** Delegates for asset editor events */

	ENGINE_API FDelegateHandle RegisterOnClothingChange(const FSimpleMulticastDelegate::FDelegate& InDelegate);
	ENGINE_API void UnregisterOnClothingChange(const FDelegateHandle& InHandle);

private:

	/** Called to notify a change to the clothing object array */
	FSimpleMulticastDelegate OnClothingChange;
#endif // WITH_EDITOR
	// INodeMappingProviderInterface
	ENGINE_API virtual void GetMappableNodeData(TArray<FName>& OutNames, TArray<FNodeItem>& OutTransforms) const override;

	/**
	 * Wait for the asset to finish compilation to protect internal skinned asset data from race conditions during async build.
	 * This should be called before accessing all async accessible properties.
	 */
	ENGINE_API void WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite) const;

	/** Convert async property from enum value to string. USkinnedAsset interface. */
	ENGINE_API virtual FString GetAsyncPropertyName(uint64 Property) const override;

public:
	/*
	 * Add New LOD info entry to LODInfo array
	 * 
	 * This adds one entry with correct setting
	 * If it's using LODSettings, it will copy from that setting
	 * If not, it will auto calculate based on what is previous LOD setting
	 *
	 */
	ENGINE_API FSkeletalMeshLODInfo& AddLODInfo();
	/*
	 * Add New LOD info entry with entry
	 * 
	 * This is used by  import code, where they want to override this
	 *
	 * @param NewLODInfo : new LOD info to be added
	 */
	ENGINE_API int32 AddLODInfo(const FSkeletalMeshLODInfo& NewLODInfo);
	
	/* 
	 * Remove LOD info of given index
	 */
	UE_DEPRECATED(5.7, "Use RemoveSourceModel() instead.")
	ENGINE_API void RemoveLODInfo(int32 Index);
	
	/*
	 * Reset whole entry
	 */
	UE_DEPRECATED(5.7, "Use SetNumSourceModels(0) instead.")
	ENGINE_API void ResetLODInfo();

	/*
	 * Returns whole array of LODInfo non-const. USkinnedAsset interface.
	 */
	UE_DEPRECATED(5.5, "Use GetLODInfo and GetLODNum instead.")
	ENGINE_API virtual TArray<FSkeletalMeshLODInfo>& GetLODInfoArray() override;

	/*
	 * Returns whole array of LODInfo const. USkinnedAsset interface.
	 */
	UE_DEPRECATED(5.5, "Use GetLODInfo and GetLODNum instead.")
	ENGINE_API virtual const TArray<FSkeletalMeshLODInfo>& GetLODInfoArray() const override;

	/* 
	 * Get LODInfo of the given index non-const. USkinnedAsset interface.
	 */
	ENGINE_API virtual FSkeletalMeshLODInfo* GetLODInfo(int32 Index) override;
	
	/* 
	 * Get LODInfo of the given index const. USkinnedAsset interface.
	 */	
	ENGINE_API virtual const FSkeletalMeshLODInfo* GetLODInfo(int32 Index) const override;

private:

	/** Update the given LOD based on a previous LOD, setting it to auto-generated based on the
	 *  first non-autogenerated LOD, with 50% reduction in triangle count per-LOD.
	 */
	void CopyAutogeneratedLODInfoFromPreviousLOD(int32 InLODIndex);
public:
	
#if WITH_EDITOR
	ENGINE_API FSimpleMulticastDelegate& GetOnVertexAttributesArrayChanged();
#endif
	
	/**
	 *	Get BakePose for the given LOD
	 */
	ENGINE_API const UAnimSequence* GetBakePose(int32 LODIndex) const;

	/* 
	 * Get Default LOD Setting of this mesh
	 */
	ENGINE_API const USkeletalMeshLODSettings* GetDefaultLODSetting() const; 

	/* 
	 * Return true if given index's LOD is valid. USkinnedAsset interface.
	 */
	ENGINE_API virtual bool IsValidLODIndex(int32 Index) const override;

	/* 
	 * Returns total number of LOD. USkinnedAsset interface.
	 */
	ENGINE_API virtual int32 GetLODNum() const override;

	/** Returns the number of vertices of the Nanite representation of this mesh. */
	ENGINE_API int32 GetNumNaniteVertices() const;

	/** Returns the number of triangles of the Nanite representation of this mesh. */
	ENGINE_API int32 GetNumNaniteTriangles() const;

	/** USkinnedAsset interface. */
	ENGINE_API virtual bool IsMaterialUsed(int32 MaterialIndex) const override;

	const TArray<FSkinWeightProfileInfo>& GetSkinWeightProfiles() const 
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SkinWeightProfiles, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SkinWeightProfiles; 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TArray<FSkinWeightProfileInfo>& GetSkinWeightProfiles() 
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SkinWeightProfiles); 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SkinWeightProfiles; 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	ENGINE_API void AddSkinWeightProfile(const FSkinWeightProfileInfo& Profile);
	
#if WITH_EDITOR
	ENGINE_API bool RemoveSkinWeightProfiles(TConstArrayView<FName> InProfileName, bool bInRebuildRenderMesh = false);
 	
	/** Remove the named skin weight profile from this skeletal mesh. 
	 *  @param InProfileName The name of the profile to remove.
	 *  @param bInRebuildRenderMesh Set to true if rebuilding the render mesh as a part of the operation is desired. 
	 *    This is needed if the result of the skin weight profile removal on the mesh is to be visible afterward. 
	 *  @return true if the named profile was found and removed. 
	 */
	UFUNCTION(BlueprintCallable, Category = Mesh, meta=(DevelopmentOnly))
	bool RemoveSkinWeightProfile(const FName InProfileName, bool bInRebuildRenderMesh = false)
	{
		return RemoveSkinWeightProfiles({InProfileName}, bInRebuildRenderMesh);
	}
#endif

	int32 GetNumSkinWeightProfiles() const 
	{ 
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::SkinWeightProfiles, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SkinWeightProfiles.Num(); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Releases all allocated Skin Weight Profile resources, assumes none are currently in use */
	ENGINE_API void ReleaseSkinWeightProfileResources();

	/**
	 *	Returns the list of the names of all the skin weight profile on this skeletal mesh
	 *  @return	The list of skin weight profiles.
	 */
	UFUNCTION(BlueprintPure, Category = Mesh, meta = (DisplayName = "Get All Skin Weight Profile Names", ScriptName = "GetAllSkinWeightProfileNames", Keywords = "skin weight profile"))
	ENGINE_API TArray<FString> K2_GetAllSkinWeightProfileNames() const;

protected:
	/** Set of skin weight profiles associated with this mesh */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = SkinWeights, EditFixedSize, Meta=(NoResetToDefault))
	TArray<FSkinWeightProfileInfo> SkinWeightProfiles;

	/** Default mesh deformer to use with this mesh. */
	UE_DEPRECATED(5.1, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer", BlueprintGetter=GetDefaultMeshDeformer)
	TObjectPtr<UMeshDeformer> DefaultMeshDeformer;
	
	/** Additional deformers that will be applied to this mesh at runtime. Skeletal Mesh needs this collection of deformers to make sure it cooks any extra data required by these deformers */
	UE_DEPRECATED(5.8, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer", BlueprintGetter=GetTargetMeshDeformers)
	TObjectPtr<UMeshDeformerCollection> TargetMeshDeformers;

	/** Default translucent material to blend on top of this mesh. Mesh will be rendered twice - once with a base material and once with overlay material */
	UE_DEPRECATED(5.3, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Rendering)
	TObjectPtr<UMaterialInterface> OverlayMaterial;

	/** Default max draw distance for overlay material. A distance of 0 indicates that overlay will be culled using primitive max distance. */
	UE_DEPRECATED(5.3, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Rendering)
	float OverlayMaterialMaxDrawDistance;

	/**
	 * Axis that the skeletal mesh is facing. Default is the Y axis.
	 * The facing axis represents the MeshDescription's orientation and is set on import.
	 * Therefore, if this property is modified elsewhere, the associated MeshDescription should be appropriately modified to reflect the new orientation.
	 */
	UPROPERTY(VisibleAnywhere, Category = Mesh, BlueprintGetter = GetForwardAxis)
	TEnumAsByte<EAxis::Type> ForwardAxis = EAxis::Y;

public:
	/** Set the default mesh deformer used by this mesh. A mesh deformer is used to deform the skeletal mesh at runtime.
	 *  The default deformer can be overridden by the USkinnedMeshComponent object. */
	void SetDefaultMeshDeformer(UMeshDeformer* InMeshDeformer)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultMeshDeformer, ESkinnedAssetAsyncPropertyLockType::WriteOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultMeshDeformer = InMeshDeformer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS		
	}
	
	/** Get the default mesh deformer used by this mesh. A mesh deformer is used to deform the skeletal mesh at runtime */
	UFUNCTION(BlueprintCallable, Category="Rendering|Deformer")
	virtual UMeshDeformer* GetDefaultMeshDeformer() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::DefaultMeshDeformer, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DefaultMeshDeformer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	/** Set the collection of mesh deformers that may be applied to this mesh. Skeletal Mesh use these deformers to determined if any extra data needs to be cooked */
	void SetTargetMeshDeformers(UMeshDeformerCollection* InMeshDeformerCollection)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::TargetMeshDeformers, ESkinnedAssetAsyncPropertyLockType::WriteOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TargetMeshDeformers = InMeshDeformerCollection;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS	
	}
	
	/** Get the collection of mesh deformers that may be applied to this mesh. Skeletal Mesh use these deformers to determined if any extra data needs to be cooked*/
	UFUNCTION(BlueprintCallable, Category="Rendering|Deformer")
	virtual UMeshDeformerCollection* GetTargetMeshDeformers() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::TargetMeshDeformers, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return TargetMeshDeformers;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** True if this mesh LOD builds the half edge buffer. USkinnedAsset interface.	*/
	ENGINE_API virtual bool HasHalfEdgeBuffer(int32 LODIndex)const override;
	
	/** Get the default overlay material used by this mesh */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	virtual class UMaterialInterface* GetOverlayMaterial() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverlayMaterial, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OverlayMaterial;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Change the default overlay material used by this mesh */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	void SetOverlayMaterial(class UMaterialInterface* NewOverlayMaterial)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverlayMaterial);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OverlayMaterial = NewOverlayMaterial;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	/** Get the default overlay material max draw distance used by this mesh */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	virtual float GetOverlayMaterialMaxDrawDistance() const override
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverlayMaterialMaxDrawDistance, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OverlayMaterialMaxDrawDistance;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Change the default overlay material max draw distance used by this mesh */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	void SetOverlayMaterialMaxDrawDistance(float InMaxDrawDistance)
	{
		WaitUntilAsyncPropertyReleased(ESkeletalMeshAsyncProperties::OverlayMaterialMaxDrawDistance);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OverlayMaterialMaxDrawDistance = InMaxDrawDistance;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Get the forward axis used by this mesh */
	UFUNCTION(BlueprintCallable, Category = Mesh)
	TEnumAsByte<EAxis::Type> GetForwardAxis() const
	{
		return ForwardAxis;
	}

	/** Set the forward axis used by this mesh */
	void SetForwardAxis(TEnumAsByte<EAxis::Type> InForwardAxis)
	{
		ForwardAxis = InForwardAxis;
	}

private:
	/**
	 * Initial step for the Post Load process - Can't be done in parallel. USkinnedAsset Interface.
	 */
	ENGINE_API virtual void BeginPostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;

	/**
	 * Thread-safe part of the Post Load. USkinnedAsset interface.
	 */
	ENGINE_API virtual void ExecutePostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;

	/**
	 * Complete the postload process - Can't be done in parallel. USkinnedAsset interface.
	 */
	ENGINE_API virtual void FinishPostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;

private:
	/**
	 * Fully deprecated properties.
	 */
	 
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<USkeletalMeshEditorData> MeshEditorDataObject_DEPRECATED;

#endif //WITH_EDITORONLY_DATA
	
	UPROPERTY()
	TArray<FSkeletalMeshLODInfo> LODInfo_DEPRECATED;
	
};

struct FSkeletalMeshBuildParameters
{
	FSkeletalMeshBuildParameters(USkeletalMesh* InSkeletalMesh, const ITargetPlatform* InTargetPlatform, int32 InLODIndex, bool bInRegenDepLODs)
		: SkeletalMesh(InSkeletalMesh)
		, TargetPlatform(InTargetPlatform)
		, LODIndex(InLODIndex)
		, bRegenDepLODs(bInRegenDepLODs)
	{}

	USkeletalMesh* SkeletalMesh;
	const ITargetPlatform* TargetPlatform;
	const int32 LODIndex;
	const bool bRegenDepLODs;
};

/**
 * Refresh Physics Asset Change
 * 
 * Physics Asset has been changed, so it will need to recreate physics state to reflect it
 * Utilities functions to propagate new Physics Asset for InSkeletalMesh
 *
 * @param	InSkeletalMesh	SkeletalMesh that physics asset has been changed for
 */
ENGINE_API void RefreshSkelMeshOnPhysicsAssetChange(const USkeletalMesh* InSkeletalMesh);

ENGINE_API FVector GetSkeletalMeshRefVertLocation(const USkeletalMesh* Mesh, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const int32 VertIndex);
ENGINE_API void GetSkeletalMeshRefTangentBasis(const USkeletalMesh* Mesh, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const int32 VertIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ);

#undef UE_API
