// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Contains the shared data that is used by all SkeletalMeshComponents (instances).
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "EngineDefines.h"
#include "Engine/EngineTypes.h"

#if WITH_EDITORONLY_DATA

#include "Rendering/SkeletalMeshLODImporterData.h"

#endif //WITH_EDITORONLY_DATA

#include "SkeletalMeshEditorData.generated.h"

/**
 * SkeletalMeshEditorData is a container for non editable editor data.
 * An example of data is the source import data that get updated only when we reimport an asset, but is needed if the asset need to be build. If the ddc key is uptodate the data do not have to be loaded
 *
 */
UCLASS(hidecategories=Object, MinimalAPI)
class USkeletalMeshEditorData : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA

/*
 * Raw skeletal mesh editor data
 */
public:

	ENGINE_API virtual void PostLoad() override;

	/**
	 * Return the imported source LOD data, create one if there is nothing.
	 * Previous reference get by this function can be invalid if we have to add items
	 */
	ENGINE_API FRawSkeletalMeshBulkData& GetLODImportedData(int32 LODIndex);

	ENGINE_API bool IsLODImportDataValid(int32 LODIndex);

	ENGINE_API bool RemoveLODImportedData(int32 LODIndex);

	/* UObject overrides */

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	
	/* End of UObject overrides */

private:
	friend class USkeletalMesh;
	
	/** Imported raw mesh data. Optional, only the imported mesh LOD has this, generated LOD or old asset will be empty. */
	TArray<TSharedRef<FRawSkeletalMeshBulkData>> RawSkeletalMeshBulkDatas;


#endif //WITH_EDITORONLY_DATA
};
