// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "SkeletalMeshReductionSettings.h"
#include "DataAsset.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"
#include "SkeletalMeshLODSettings.generated.h"

UENUM()
enum class EBoneFilterActionOption : uint8
{
	/** Remove list of joints specified and children. All the other joints will be kept. */
	Remove UMETA(DisplayName = "Remove the joints specified and children"), 

	/** Only keep the joints specified and parents. All the other joints will be removed. */
	Keep UMETA(DisplayName = "Only keep the joints specified and parents"), 

	/** Invalid*/
	Invalid UMETA(Hidden), 
};

USTRUCT()
struct FBoneFilter
{
	GENERATED_USTRUCT_BODY()

	/*
	 * Do not include the joint specified
	 *
	 * This option will work differently based on EBoneFilterActionOption
	 * If EBoneFilterActionOption is Remove, it will exclude itself and only remove children
	 * For example, if you specify hand, it will only include children of hand(all fingers), 
	 * not the hand itself if this is true
	 * 
	 * But if the EBoneFilterActionOption is Keep, it will exclude itself but include all parents of it 
	 * You can't remove joint without children removed, and you can't keep without your parents 
	 */
	UPROPERTY(EditAnywhere, Category = FBoneFilter)
	bool bExcludeSelf = false;

	/* Name of Bone Name */
	UPROPERTY(EditAnywhere, Category = FBoneFilter)
	FName BoneName;
};

USTRUCT()
struct FSkeletalMeshLODGroupSettings
{
	GENERATED_USTRUCT_BODY()

	FSkeletalMeshLODGroupSettings()
		: ScreenSize(0.3f)
		, LODHysteresis(0.0f)
		, BoneFilterActionOption(EBoneFilterActionOption::Remove)
		, WeightOfPrioritization(1.f)
		, BakePose(nullptr)
	{}
	
	/** Get Skeletal mesh optimizations setting structure for the given LOD level */
	ENGINE_API FSkeletalMeshOptimizationSettings GetReductionSettings() const;

	/** Get Skeletal mesh optimizations setting structure for the given LOD level */
	UE_DEPRECATED(4.20, "Please use GetReductionSettings instead")
	FSkeletalMeshOptimizationSettings GetSettings() const
	{
		return GetReductionSettings();
	}

	/** Get the correct screen size for the given LOD level */
	ENGINE_API const float GetScreenSize() const;

	/** FSkeletalMeshLODSettings initializes group entries. */
	friend class USkeletalMeshLODSettings;

	/** The screen sizes to use for the respective LOD level */
	UPROPERTY(EditAnywhere, Category = LODSetting)
	FPerPlatformFloat ScreenSize;

	/**	Used to avoid 'flickering' when on LOD boundary. Only taken into account when moving from complex->simple. */
	UPROPERTY(EditAnywhere, Category = LODSetting)
	float LODHysteresis;

	/** Bones which should be removed from the skeleton for the LOD level */
	UPROPERTY(EditAnywhere, Category = Reduction)
	EBoneFilterActionOption BoneFilterActionOption;

	/** Bones which should be removed from the skeleton for the LOD level */
	UPROPERTY(EditAnywhere, Category = Reduction)
	TArray<FBoneFilter> BoneList;

	/** Bones which should be prioritized for the quality, this will be weighted toward keeping source data. Use WeightOfPrioritization to control the value. */
	UPROPERTY(EditAnywhere, Category = Reduction)
	TArray<FName> BonesToPrioritize;

	/** Sections which should be prioritized for the quality, this will be weighted toward keeping source data. Use WeightOfPrioritization to control the value. */
	UPROPERTY(EditAnywhere, Category = Reduction)
	TArray<int32> SectionsToPrioritize;

	/** How much to consideration to give BonesToPrioritize and SectionsToPrioritize.  The weight is an additional vertex simplification penalty where 0 means nothing. */
	UPROPERTY(EditAnywhere, Category = Reduction, meta = (UIMin = "0.0", ClampMin = "0.0"))
	float WeightOfPrioritization;

	/** Pose which should be used to reskin vertex influences for which the bones will be removed in this LOD level, uses ref-pose by default */
	UPROPERTY(EditAnywhere, Category = Reduction)
	TObjectPtr<class UAnimSequence> BakePose;

	/** The optimization settings to use for the respective LOD level */
	UPROPERTY(EditAnywhere, Category = Reduction)
	FSkeletalMeshOptimizationSettings ReductionSettings;

	/** Whether a Mesh Deformer applied to the mesh asset or Skinned Mesh Component should be used on this LOD or not */
	UPROPERTY(EditAnywhere, Category = LODSetting)
	bool bAllowMeshDeformer = true;
};

UCLASS(config = Engine, defaultconfig, BlueprintType, MinimalAPI)
class USkeletalMeshLODSettings : public UDataAsset
{
	GENERATED_UCLASS_BODY()
protected:
	/** Minimum Quality Level LOD to render. Can be overridden per mesh as well as set here for all mesh instances */
	UPROPERTY(globalconfig, EditAnywhere, Category = LODGroups, meta = (DisplayName = "Quality Level Minimum LOD"))
	FPerQualityLevelInt MinQualityLevelLod;

	/** Minimum LOD to render. Can be overridden per mesh as well as set here for all mesh instances */
	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups, meta = (DisplayName = "Minimum LOD"))
	FPerPlatformInt MinLod;

	/** When true LODs below MinLod will not be stripped during cook. */
	UPROPERTY(globalconfig, EditAnywhere, Category = LODGroups)
	FPerPlatformBool DisableBelowMinLodStripping;

	/** Whether meshes in this group override default LOD streaming settings. */
	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups)
	bool bOverrideLODStreamingSettings;

	/** Whether meshes in this group stream LODs by default */
	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups, meta=(DisplayName="Stream LODs", EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformBool bSupportLODStreaming;

	/** Default maximum number of streamed LODs for meshes in this group */
	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups, meta=(EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformInt MaxNumStreamedLODs;

	/** Default maximum number of optional LODs for meshes in this group (currently, need to be either 0 or > num of LODs below MinLod) */
	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups, meta=(EditCondition="bOverrideLODStreamingSettings"))
	FPerPlatformInt MaxNumOptionalLODs;

	/** Default LOD bias for no-ref meshes. */
	UPROPERTY(globalconfig, EditAnywhere, Category = LODGroups)
	FPerQualityLevelInt NoRefStreamingLODBias;

	UPROPERTY(globalconfig, EditAnywhere, Category=LODGroups)
	TArray<FSkeletalMeshLODGroupSettings> LODGroups;

	friend class FSkeletalMeshReductionSettingsDetails;
public:
	/** Retrieves the Skeletal mesh LOD group settings for the given name */
	ENGINE_API const FSkeletalMeshLODGroupSettings& GetSettingsForLODLevel(const int32 LODIndex) const;

	/** Returns whether or not valid settings were retrieved from the ini file */
	const bool HasValidSettings() const
	{
		return LODGroups.Num() > 0;
	}

	/** Returns the number of settings parsed from the ini file */
	ENGINE_API int32 GetNumberOfSettings() const;

	/*
	* Set InMesh->LODInfo with this LOD Settings for LODIndex
	* return true if succeed
	*/
	ENGINE_API bool SetLODSettingsToMesh(USkeletalMesh* InMesh, int32 LODIndex) const;

	/*
	* Set InMesh->LODInfo with this LOD Settings
	* return # of settings that are set. Return N for 0-(N-1).
	*/
	ENGINE_API int32 SetLODSettingsToMesh(USkeletalMesh* InMesh) const;
	/*
	* Set this LOD Settings from InMesh->LODInfo 
	* return # of settings that are set. Return N for 0-(N-1).
	*/
	ENGINE_API int32 SetLODSettingsFromMesh(USkeletalMesh* InMesh);
	
	// BEGIN UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITOR
	virtual void Serialize(FArchive& Ar) override;
	// END UObject interface
};
