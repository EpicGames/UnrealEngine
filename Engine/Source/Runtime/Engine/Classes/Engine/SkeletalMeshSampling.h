// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WeightedRandomSampler.h"

#include "SkeletalMeshSampling.generated.h"

class USkeletalMesh;

/** Allows area weighted sampling of triangles on a skeletal mesh. */
struct FSkeletalMeshAreaWeightedTriangleSampler : public FWeightedRandomSampler
{
	ENGINE_API FSkeletalMeshAreaWeightedTriangleSampler();
	ENGINE_API void Init(USkeletalMesh* InOwner, int32 LODIndex, TArray<int32>* InTriangleIndices);
	ENGINE_API virtual float GetWeights(TArray<float>& OutWeights)override;

protected:
	//Data used in initialization of the sampler. Not serialized.
	USkeletalMesh* Owner;
	TArray<int32>* TriangleIndices;
	int32 LODIndex;
};

/** Built data for sampling a single region of a skeletal mesh. */
USTRUCT()
struct FSkeletalMeshSamplingRegionBuiltData
{
	GENERATED_USTRUCT_BODY()

	/** Triangles included in this region. */
	TArray<int32> TriangleIndices;

	/** Vertices included in this region. */
	TArray<int32> Vertices;
	
	/** Bones included in this region. */
	TArray<int32> BoneIndices;

	/** Provides random area weighted sampling of the TriangleIndices array. */
	FSkeletalMeshAreaWeightedTriangleSampler AreaWeightedSampler;

	ENGINE_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshSamplingRegionBuiltData& A)
	{
		A.Serialize(Ar);
		return Ar;
	}
};

template<> struct TStructOpsTypeTraits<FSkeletalMeshSamplingRegionBuiltData> : public TStructOpsTypeTraitsBase2<FSkeletalMeshSamplingRegionBuiltData>
{
	enum{ WithSerializer = true };
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** Built data for sampling a the whole mesh at a particular LOD. */
USTRUCT()
struct FSkeletalMeshSamplingLODBuiltData
{
	GENERATED_USTRUCT_BODY()

	/** Area weighted sampler for the whole mesh at this LOD.*/
	FSkeletalMeshAreaWeightedTriangleSampler AreaWeightedTriangleSampler;

	ENGINE_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshSamplingLODBuiltData& A)
	{
		A.Serialize(Ar);
		return Ar;
	}
};

template<> struct TStructOpsTypeTraits<FSkeletalMeshSamplingLODBuiltData> : public TStructOpsTypeTraitsBase2<FSkeletalMeshSamplingLODBuiltData>
{
	enum{ WithSerializer = true };
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

USTRUCT()
struct FSkeletalMeshSamplingBuiltData
{
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY()
	TArray<FSkeletalMeshSamplingLODBuiltData> WholeMeshBuiltData;

	UPROPERTY()
	TArray<FSkeletalMeshSamplingRegionBuiltData> RegionBuiltData;
};

/** Filter to include or exclude bones and their associated triangles from a sampling region. */
USTRUCT()
struct FSkeletalMeshSamplingRegionBoneFilter
{
	GENERATED_USTRUCT_BODY()

	FSkeletalMeshSamplingRegionBoneFilter();

	UPROPERTY(EditAnywhere, Category = "Bone Filter")
	FName BoneName;

	/** If true, this filter will include bones. If false, it will exclude them. */
	UPROPERTY(EditAnywhere, Category = "Bone Filter")
	uint32 bIncludeOrExclude : 1;
	
	/** If true, this filter will apply to all children of this bone as well. */
	UPROPERTY(EditAnywhere, Category = "Bone Filter")
	uint32 bApplyToChildren : 1;
};

/** Filter to include triangles in a sampling region based on their material. */
USTRUCT()
struct FSkeletalMeshSamplingRegionMaterialFilter
{
	GENERATED_USTRUCT_BODY()

	FSkeletalMeshSamplingRegionMaterialFilter();

	UPROPERTY(EditAnywhere, Category = "Material Filter")
	FName MaterialName;
};

/** Defined a named region on a mesh that will have associated triangles and bones for fast sampling at each enabled LOD. */
USTRUCT()
struct FSkeletalMeshSamplingRegion
{
	GENERATED_USTRUCT_BODY()
		
	ENGINE_API FSkeletalMeshSamplingRegion();

	/** Name of this region that users will reference. */
	UPROPERTY(EditAnywhere, Category = "Region")
	FName Name;

	/** The LOD of the mesh that this region applies to. */
	UPROPERTY(EditAnywhere, Category = "Region")
	int32 LODIndex;

	/**
	Mesh supports uniformly distributed sampling in constant time.
	Memory cost is 8 bytes per triangle.
	*/
	UPROPERTY(EditAnywhere, Category = "Region")
	uint32 bSupportUniformlyDistributedSampling : 1;
	
	/** Filters to determine which triangles to include in this region based on material. */
	UPROPERTY(EditAnywhere, Category = "Region")
	TArray<FSkeletalMeshSamplingRegionMaterialFilter> MaterialFilters;

	/** Filters to determine which triangles and bones to include in this region based on bone. */
	UPROPERTY(EditAnywhere, Category = "Region")
	TArray<FSkeletalMeshSamplingRegionBoneFilter> BoneFilters;

	/** Directly included vertices from a future painting tool. */
// 	UPROPERTY()
// 	TArray<int32> PaintedTriangles;

/** Directly included bones from a future painting tool. */
// 	UPROPERTY()
// 	TArray<int32> PaintedBones;
	
	ENGINE_API void GetFilteredBones(struct FReferenceSkeleton& RefSkel, TArray<int32>& OutIncludedBoneIndices, TArray<int32>& OutExcludedBoneIndices);
	ENGINE_API bool IsMaterialAllowed(FName MaterialName);
};

USTRUCT()
struct FSkeletalMeshSamplingInfo
{
	GENERATED_USTRUCT_BODY()
			
	/** Info defining sampling of named regions on this mesh. */
	UPROPERTY(EditAnywhere, Category="Sampling")
	TArray<FSkeletalMeshSamplingRegion> Regions;

#if WITH_EDITORONLY_DATA
	/** Rebuilds all data required for sampling. */
	ENGINE_API void BuildRegions(USkeletalMesh* SkeletalMesh);
	ENGINE_API void BuildWholeMesh(USkeletalMesh* SkeletalMesh);
#endif

	/** True if this LOD is enabled for the whole mesh or a named region. */
	ENGINE_API bool IsSamplingEnabled(const USkeletalMesh* OwnerMesh, int32 LODIndex)const;

	inline const FSkeletalMeshSamplingLODBuiltData& GetWholeMeshLODBuiltData(int32 LODIndex)const
	{
		return BuiltData.WholeMeshBuiltData[LODIndex];
	}

	inline int32 IndexOfRegion(FName RegionName)const
	{
		return Regions.IndexOfByPredicate([&RegionName](const FSkeletalMeshSamplingRegion& CheckRegion)
		{
			return CheckRegion.Name == RegionName;
		});
	}

	inline const FSkeletalMeshSamplingRegion& GetRegion(int32 RegionIdx) const
	{
		return Regions[RegionIdx]; 
	}

	inline const FSkeletalMeshSamplingRegion* GetRegion(FName RegionName) const
	{
		int32 RegionIndex = IndexOfRegion(RegionName);
		if (RegionIndex != INDEX_NONE)
		{
			return &GetRegion(RegionIndex);
		}
		return nullptr;
	}

	inline const FSkeletalMeshSamplingBuiltData& GetBuiltData()const
	{
		return BuiltData;
	}

	inline const FSkeletalMeshSamplingRegionBuiltData& GetRegionBuiltData(int32 RegionIndex)const
	{
		return BuiltData.RegionBuiltData[RegionIndex];
	}

	inline const FSkeletalMeshSamplingRegionBuiltData* GetRegionBuiltData(FName RegionName)const
	{
		int32 RegionIndex = IndexOfRegion(RegionName);
		if (RegionIndex != INDEX_NONE)
		{
			return &GetRegionBuiltData(RegionIndex);
		}
		return nullptr;
	}
	
private:

	UPROPERTY()
	FSkeletalMeshSamplingBuiltData BuiltData;
};
