// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "SkeletalMeshSimplificationSettings.generated.h"

/**
* Controls the selection of the system used to simplify skeletal meshes.
*/
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Skeletal Mesh Simplification"), MinimalAPI)
class USkeletalMeshSimplificationSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual FName GetContainerName() const override;
	ENGINE_API virtual FName GetCategoryName() const override;

	ENGINE_API void SetSkeletalMeshReductionModuleName(FName InSkeletalMeshReductionModuleName);

public:
	/** Mesh reduction plugin to use when simplifying skeletal meshes */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (ConsoleVariable = "r.SkeletalMeshReductionModule", DisplayName = "Skeletal Mesh Reduction Plugin", ConfigRestartRequired = true))
	FName SkeletalMeshReductionModuleName;

	ENGINE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};
