// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "SkeletalMeshSocket.generated.h"

UCLASS(hidecategories=Object, hidecategories=Actor, BlueprintType, MinimalAPI)
class USkeletalMeshSocket : public UObject
{
	GENERATED_BODY()

public:
	/** 
	 *	Defines a named attachment location on the USkeletalMesh. 
	 *	These are set up in editor and used as a shortcut instead of specifying 
	 *	everything explicitly to AttachComponent in the SkeletalMeshComponent.
	 *	The Outer of a SkeletalMeshSocket should always be the USkeletalMesh.
	 */
	UPROPERTY(Category="Socket Parameters", VisibleAnywhere, BlueprintReadOnly)
	FName SocketName;

	UPROPERTY(Category="Socket Parameters", VisibleAnywhere, BlueprintReadOnly)
	FName BoneName;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Socket Parameters")
	FVector RelativeLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Socket Parameters")
	FRotator RelativeRotation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Socket Parameters")
	FVector RelativeScale = FVector::OneVector;

	/** If true then the hierarchy of bones this socket is attached to will always be 
	    evaluated, even if it had previously been removed due to the current lod setting */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Socket Parameters")
	bool bForceAlwaysAnimated = true;

#if WITH_EDITOR
	/** Change the sockets parent to a new bone. The skeleton is used to validate that the bone exists */
	UFUNCTION(BlueprintCallable, Category="Socket Parameters")
	ENGINE_API void SetSocketParent(USkeletalMesh* InSkeletalMesh, FName InBoneName);
#endif
	
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API FVector GetSocketLocation(const class USkeletalMeshComponent* SkelComp) const;

	/** Sets BoneName, RelativeLocation and RelativeRotation based on closest bone to WorldLocation and WorldNormal */
	UFUNCTION(BlueprintCallable, Category="Components|SkeletalMesh")
	ENGINE_API void InitializeSocketFromLocation(const class USkeletalMeshComponent* SkelComp, FVector WorldLocation, FVector WorldNormal);

	/** Utility that returns the current matrix for this socket. Returns false if socket was not valid (bone not found etc) */
	ENGINE_API bool GetSocketMatrix(FMatrix& OutMatrix, const class USkeletalMeshComponent* SkelComp) const;

	/** returns FTransform of Socket local transform */
	UFUNCTION(BlueprintCallable, Category="Socket Parameters")
	ENGINE_API FTransform GetSocketLocalTransform() const;

#if WITH_EDITOR
	/** Sets the relative transform parameters of the socket to the given local FTransform */
	UFUNCTION(BlueprintCallable, Category="Socket Parameters")
	ENGINE_API void SetSocketLocalTransform(FTransform InTransform);
#endif

	/** Utility that returns the current transform for this socket. */
	ENGINE_API FTransform GetSocketTransform( const class USkeletalMeshComponent* SkelComp ) const;

	/** 
	 *	Utility that returns the current matrix for this socket with an offset. Returns false if socket was not valid (bone not found etc) 
	 *	
	 *	@param	OutMatrix		The resulting socket matrix
	 *	@param	SkelComp		The skeletal mesh component that the socket comes from
	 *	@param	InOffset		The additional offset to apply to the socket location
	 *	@param	InRotation		The additional rotation to apply to the socket rotation
	 *
	 *	@return	bool			true if successful, false if not
	 */
	bool GetSocketMatrixWithOffset(FMatrix& OutMatrix, class USkeletalMeshComponent* SkelComp, const FVector& InOffset, const FRotator& InRotation) const;

	/** 
	 *	Utility that returns the current position of this socket with an offset. Returns false if socket was not valid (bone not found etc)
	 *	
	 *	@param	OutPosition		The resulting position
	 *	@param	SkelComp		The skeletal mesh component that the socket comes from
	 *	@param	InOffset		The additional offset to apply to the socket location
	 *	@param	InRotation		The additional rotation to apply to the socket rotation
	 *
	 *	@return	bool			true if successful, false if not
	 */
	bool GetSocketPositionWithOffset(FVector& OutPosition, class USkeletalMeshComponent* SkelComp, const FVector& InOffset, const FRotator& InRotation) const;

	/** 
	 *	Utility to associate an actor with a socket
	 *	
	 *	@param	Actor			The actor to attach to the socket
	 *	@param	SkelComp		The skeletal mesh component associated with this socket.
	 *
	 *	@return	bool			true if successful, false if not
	 */
	ENGINE_API bool AttachActor (class AActor* Actor, class USkeletalMeshComponent* SkelComp) const;

	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	/** Broadcasts a notification whenever the socket property has changed. */
	DECLARE_EVENT_TwoParams(USkeletalMeshSocket, FSocketChangedEvent, const class USkeletalMeshSocket*, const class FProperty*);
	FSocketChangedEvent& OnPropertyChanged() { return ChangedEvent; }

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	// Utility function to copy socket 
	ENGINE_API void CopyFrom(const class USkeletalMeshSocket* OtherSocket);

private: 
	/** Broadcasts a notification whenever the socket property has changed. */
	FSocketChangedEvent ChangedEvent;

#endif // WITH_EDITOR
};



