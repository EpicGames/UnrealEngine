// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDescription.h"
#include "MeshDescriptionBaseBulkData.h"
#include "SkinnedAssetCommon.h"

#include "SkeletalMeshSourceModel.generated.h"

class FRawSkeletalMeshBulkData;
class USkeletalMesh;
class USkeletalMeshDescription;
struct FMeshDescription;

UCLASS(MinimalAPI)
class USkeletalMeshDescriptionBulkData : public UMeshDescriptionBaseBulkData
{
	GENERATED_BODY()
public:
	ENGINE_API USkeletalMeshDescriptionBulkData();

	void Initialize();
};


USTRUCT()
struct FSkeletalMeshSourceModel :
	public FSkeletalMeshLODInfo
{
	GENERATED_BODY()

	friend class USkeletalMesh;
	
	ENGINE_API FSkeletalMeshSourceModel();

	// No copying allowed.
	FSkeletalMeshSourceModel(const FSkeletalMeshSourceModel&) = delete;
	FSkeletalMeshSourceModel& operator=(const FSkeletalMeshSourceModel&) = delete;

	// Moving is permitted.
	FSkeletalMeshSourceModel(FSkeletalMeshSourceModel&&);
	FSkeletalMeshSourceModel& operator=(FSkeletalMeshSourceModel&&);

	/** Initialize all sub-objects and other owned data that depends on knowing the skeletal mesh owner */
	void Initialize(USkeletalMesh* InOwner);
	
#if WITH_EDITOR
	/**
	 * Returns \c true if there is mesh description data available on this bulk data object.
	 * This says nothing about the validity of the actual mesh description data, only that it exists.
	 * It's can be used as a quick way to verify without unpacking the bulk data into an accessible 
	 * mesh description object.
	*/ 
	ENGINE_API bool HasMeshDescription() const;

	/**
	 * Create a new mesh description object on this container, obliterating whatever mesh might have
	 * been there before. Does not invalidate the bulk data unless CommitMeshDescription is called.
	 * Use this if wanting to start afresh.
	 */
	ENGINE_API FMeshDescription* CreateMeshDescription();

	/**
	 * Gets the cached mesh description, if there was one, otherwise tries to load it from the bulk data.
	 * If there is no bulk data, returns nullptr.
	 */
	ENGINE_API FMeshDescription* GetMeshDescription() const;

	/**
	 * Commits the currently cached mesh description (as retrieved via GetMeshDescription or TryGetMeshDescription)
	 * into bulk storage so that it will be persisted.
	 * \param bInUseHashAsGuid If set to \c true, then the DDC hash will be computed from the contents of the
	 *   mesh itself, ensuring that if the same mesh is committed again, the DDC will not be updated. If set
	 *   to \c false, then a new, random GUID is generated each time, causing the DDC to be updated regardless
	 *   of whether the mesh changed or not. 
	 */
	ENGINE_API void CommitMeshDescription(bool bInUseHashAsGuid);
	
	/**
	 * Clones the current mesh description into the output object. Clones it from bulk storage if needed.
	 */
	ENGINE_API bool CloneMeshDescription(FMeshDescription& OutMeshDescription) const;

	/**
	 * Clears the currently stored mesh description objects, leaving the bulk data as-is. 
	 * Call this to free up some memory.
	 */
	ENGINE_API void ClearMeshDescription();

	/**
	 * Call this to clear all geometry from this source data. Useful if falling back to using auto-generated
	 * geometry for this LOD.
	 */
	ENGINE_API void ClearAllMeshData();

	/**
	 * Return a read-only version of the bulk data.
	 */
	ENGINE_API const FMeshDescriptionBulkData* GetMeshDescriptionBulkData() const;
#endif

	/** Returns the number of triangles on the stored mesh description, without having to load
	 *  it up from bulk storage.
	 */
	int32 GetTriangleCountFast() const
	{
		return TriangleCount;
	}

	/** Returns the number of vertices on the stored mesh description, without having to load
	 *  it up from bulk storage.
	 */
	int32 GetVertexCountFast() const
	{
		return VertexCount;
	}

	/**
	 * Returns the bounds of the stored mesh description, without having to load it up from
	 * bulk storage.
	 */
	const FBoxSphereBounds& GetBoundsFast() const
	{
		return Bounds;
	}

	/**
	 * Returns the list of names of alternate skin weight profiles stored on the mesh, if any.
	 */
	TConstArrayView<FName> GetSkinWeightProfileNames() const;

	/**
	 * Returns the list names of morph targets stored on the mesh, if any.
	 */
	TConstArrayView<FName> GetMorphTargetNames() const;

	/** Get the vertex color GUID as an optional value. If the mesh has no vertex colors defined, then
	 *  an empty optional is returned.
	 */
	TOptional<FGuid> GetVertexColorGuid() const;
	
private:
#if WITH_EDITOR
	USkeletalMesh* GetOwner() const;
	
	/**
	 * Ensures that any old raw mesh bulk data has been converted to new data. Used prior to saving out an
	 * asset, since the old bulk data will not be saved out again and would otherwise be lost.
	 */
	void EnsureRawMeshBulkDataIsConvertedToNew();
	
	/**
	 * Tries to load an accessible mesh description from the bulk data. If the old bulk data exists and
	 * the new mesh description bulk data doesn't, a conversion will be performed and the converted
	 * old bulk data stored as new. Completely bypasses any cached mesh description object.  
	 */
	bool LoadMeshDescriptionFromBulkData(FMeshDescription& OutMeshDescription) const;

	/**
	 * If necessary, convert morph targets from the old FVector3f[2] storage to the split representation
	 * where we store position per vertex, and normal per vertex instance
	 */
	static void UpgradeMorphTargets(FMeshDescription& InOutMeshDescription);

	/**
	 * Convert already stored old raw import format to mesh description and commit to bulk data.
	 * The new bulk data will default to using the mesh hash as the GUID, so that any future
	 * commits on an unchanged mesh will not cause a DDC invalidation.
	 */
	void ConvertRawMeshToMeshDescriptionBulkData();

	/**
	 * Update the cached triangle/vertex count and bounds values from the given mesh.
	 * \param InMeshDescription The mesh description to update from. If \c nullptr then
	 *   the statistics are reset to zero, for the counts, and an empty bounds.
	 */
	void UpdateCachedMeshStatistics(const FMeshDescription* InMeshDescription);

	/**
	 *  Update the cached mesh statistics from either resident mesh, or by loading
	 *  and then unloading the bulkd data, to conserve memory.
	 */
	void UpdateCachedMeshStatisticsFromBulkIfNeeded();
	
	/**
	 *  Update the MeshDescription bones data if needed so that it matches the skeletal mesh ref skeleton.  
     */
	void UpdateBonesDataIfNeeded(FMeshDescription& InOutMeshDescription) const;
    
#endif

	/** The number of triangles on the mesh, stored here so that users can quickly query this information
	 *  without having to unpack the entire mesh. This is set when CommitMeshDescription is called. */
	UPROPERTY()
	int32 TriangleCount = 0;
	
	/** The number of vertices on the mesh, stored here so that users can quickly query this information
	 *  without having to unpack the entire mesh. This is set when CommitMeshDescription is called. */
	UPROPERTY()
	int32 VertexCount = 0;

	/** The bounds of the stored mesh, here so that users can quickly query this information
	 *  without having to unpack the entire mesh. This is set when CommitMeshDescription is called. */
	UPROPERTY()
	FBoxSphereBounds Bounds;

	// List of all alternate skin weight profiles stored on the mesh,
	// NOTE: The default value is single-entry array with NAME_None to mark
	// that this value has not been initialized yet / TOptional does not work
	// with containers. 
	UPROPERTY()
	TArray<FName> CachedSkinWeightProfileNames{ NAME_None };

	// List of all morph targets stored on the mesh.
	UPROPERTY()
	TArray<FName> CachedMorphTargetNames{ NAME_None };
	
	UPROPERTY()
	TOptional<bool> CachedHasVertexColors;

	UPROPERTY()
	FGuid CachedVertexColorGuid;
	
#if WITH_EDITORONLY_DATA
	/**
	 * Bulk data containing mesh description from imported or modeled geometry.
	 * If the bulk data within is empty, the LOD is autogenerated (for LOD1+).
	 */
	UPROPERTY()
	TObjectPtr<USkeletalMeshDescriptionBulkData> MeshDescriptionBulkData;

	// Accessor mutex for the bulk data, since it can be accessed from any thread. 
	mutable FCriticalSection MeshDescriptionBulkDataMutex;
#endif

#if WITH_EDITOR
	/** The old mesh bulk data. When calling LoadMeshDescription and this object is non-null
	 *  a conversion will take place to convert this bulk data to new mesh description object.
	 */
	TSharedPtr<FRawSkeletalMeshBulkData> RawMeshBulkData;
	int32 RawMeshBulkDataLODIndex = INDEX_NONE;
#endif
};


template<> struct TStructOpsTypeTraits<FSkeletalMeshSourceModel> :
	TStructOpsTypeTraitsBase2<FSkeletalMeshSourceModel>
{
	enum { WithCopy = false };
};
