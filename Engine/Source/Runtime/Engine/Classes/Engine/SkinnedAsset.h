// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for common skinned mesh assets.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/StreamableRenderAsset.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "ReferenceSkeleton.h"
#include "UObject/PerPlatformProperties.h"
#include "SkinnedAsset.generated.h"

struct FSkeletalMaterial;
struct FSkeletalMeshLODInfo;
class FSkinnedAssetBuildContext;
class FSkinnedAssetPostLoadContext;
class FSkinnedAsyncTaskContext;
class FVertexFactoryType;
class ITargetPlatform;
class UMeshDeformer;
class UMeshDeformerCollection;
class UMorphTarget;
class UPhysicsAsset;
class USkeleton;
struct FPSOPrecacheVertexFactoryDataPerMaterialIndex;
struct FSkinnedAssetAsyncBuildTask;
enum class ESkeletalMeshVertexFlags : uint8;
typedef TArray<FPSOPrecacheVertexFactoryDataPerMaterialIndex, TInlineAllocator<4> > FPSOPrecacheVertexFactoryDataPerMaterialIndexList;

DECLARE_LOG_CATEGORY_EXTERN(LogSkinnedAsset, Log, All)

enum class ESkinnedAssetAsyncPropertyLockType
{
	None = 0,
	ReadOnly = 1,
	WriteOnly = 2,
	ReadWrite = 3
};
ENUM_CLASS_FLAGS(ESkinnedAssetAsyncPropertyLockType);

UCLASS(hidecategories = Object, config = Engine, editinlinenew, abstract, MinimalAPI)
class USkinnedAsset : public UStreamableRenderAsset, public IInterface_AsyncCompilation
{
	GENERATED_BODY()

public:
	ENGINE_API USkinnedAsset(const FObjectInitializer& ObjectInitializer);
	ENGINE_API virtual ~USkinnedAsset();

	/** Return the reference skeleton. */
	virtual struct FReferenceSkeleton& GetRefSkeleton()
	PURE_VIRTUAL(USkinnedAsset::GetRefSkeleton, static FReferenceSkeleton Dummy; return Dummy;);
	virtual const struct FReferenceSkeleton& GetRefSkeleton() const
	PURE_VIRTUAL(USkinnedAsset::GetRefSkeleton, static const FReferenceSkeleton Dummy; return Dummy;);

	/** Return the LOD information for the specified LOD index. */
	virtual FSkeletalMeshLODInfo* GetLODInfo(int32 Index)
	PURE_VIRTUAL(USkinnedAsset::GetLODInfo, return nullptr;);
	virtual const FSkeletalMeshLODInfo* GetLODInfo(int32 Index) const
	PURE_VIRTUAL(USkinnedAsset::GetLODInfo, return nullptr;);

	/** Return if the material index is valid. */
	ENGINE_API virtual bool IsValidMaterialIndex(int32 Index) const;

	/** Return the number of materials of this mesh. */
	ENGINE_API virtual int32 GetNumMaterials() const;

	/** Return the physics asset whose shapes will be used for shadowing. */
	virtual UPhysicsAsset* GetShadowPhysicsAsset() const
	PURE_VIRTUAL(USkinnedAsset::GetShadowPhysicsAsset, return nullptr;);

	/** Return the component orientation of a bone or socket. */
	virtual FMatrix GetComposedRefPoseMatrix(FName InBoneName) const
	PURE_VIRTUAL(USkinnedAsset::GetComposedRefPoseMatrix, return FMatrix::Identity;);

	/** Return the component orientation of a bone or socket. */
	virtual FMatrix GetComposedRefPoseMatrix(int32 InBoneIndex) const
	PURE_VIRTUAL(USkinnedAsset::GetComposedRefPoseMatrix, return FMatrix::Identity;);

	/**
	 * Returns the UV channel data for a given material index. Used by the texture streamer.
	 * This data applies to all lod-section using the same material.
	 *
	 * @param MaterialIndex		the material index for which to get the data for.
	 * @return the data, or null if none exists.
	 */
	virtual const FMeshUVChannelInfo* GetUVChannelData(int32 MaterialIndex) const
	PURE_VIRTUAL(USkinnedAsset::GetUVChannelData, return nullptr;);

	/** Return whether ray tracing is supported on this mesh. */
	virtual bool GetSupportRayTracing() const
	PURE_VIRTUAL(USkinnedAsset::GetSupportRayTracing, return false;);

	/** Return the minimum ray tracing LOD of this mesh. */
	virtual int32 GetRayTracingMinLOD() const
	PURE_VIRTUAL(USkinnedAsset::GetRayTracingMinLOD, return 0;);

	/** Return the reference skeleton precomputed bases. */
	virtual TArray<FMatrix44f>& GetRefBasesInvMatrix()
	PURE_VIRTUAL(USkinnedAsset::GetRefBasesInvMatrix, static TArray<FMatrix44f> Dummy; return Dummy;);
	virtual const TArray<FMatrix44f>& GetRefBasesInvMatrix() const
	PURE_VIRTUAL(USkinnedAsset::GetRefBasesInvMatrix, static const TArray<FMatrix44f> Dummy; return Dummy;);

	/** Return the whole array of LOD info. */
	UE_DEPRECATED(5.5, "Use GetLODInfo and GetLODNum instead.")
	virtual TArray<FSkeletalMeshLODInfo>& GetLODInfoArray()
	PURE_VIRTUAL(USkinnedAsset::GetLODInfoArray, return GetMeshLodInfoDummyArray(););
	UE_DEPRECATED(5.5, "Use GetLODInfo and GetLODNum instead.")
	virtual const TArray<FSkeletalMeshLODInfo>& GetLODInfoArray() const
	PURE_VIRTUAL(USkinnedAsset::GetLODInfoArray, return GetMeshLodInfoDummyArray(););

	/** Get the data to use for rendering. */
	virtual class FSkeletalMeshRenderData* GetResourceForRendering() const
	PURE_VIRTUAL(USkinnedAsset::GetResourceForRendering, return nullptr;);

	virtual int32 GetDefaultMinLod() const
	PURE_VIRTUAL(USkinnedAsset::GetDefaultMinLod, return 0;);

	virtual const FPerPlatformInt& GetMinLod() const
	PURE_VIRTUAL(USkinnedAsset::GetMinLod, static const FPerPlatformInt Dummy;  return Dummy;);

	/** Check the QualitLevel property is enabled for MinLod. */
	virtual bool IsMinLodQualityLevelEnable() const { return false; }

	virtual UPhysicsAsset* GetPhysicsAsset() const
	PURE_VIRTUAL(USkinnedAsset::GetPhysicsAsset, return nullptr;);

	virtual TArray<FSkeletalMaterial>& GetMaterials()
	PURE_VIRTUAL(USkinnedAsset::GetMaterials, return GetSkeletalMaterialDummyArray(););
	virtual const TArray<FSkeletalMaterial>& GetMaterials() const
	PURE_VIRTUAL(USkinnedAsset::GetMaterials, return GetSkeletalMaterialDummyArray(););

	virtual int32 GetLODNum() const
	PURE_VIRTUAL(USkinnedAsset::GetLODNum, return 0;);

	virtual bool IsMaterialUsed(int32 MaterialIndex) const
	PURE_VIRTUAL(USkinnedAsset::IsMaterialUsed, return false;);

	virtual FBoxSphereBounds GetBounds() const
	PURE_VIRTUAL(USkinnedAsset::GetBounds, return FBoxSphereBounds(););

	/**
	 * Returns the "active" socket list - all sockets from this mesh plus all non-duplicates from the skeleton
	 * Const ref return value as this cannot be modified externally
	 */
	virtual TArray<class USkeletalMeshSocket*> GetActiveSocketList() const
	PURE_VIRTUAL(USkinnedAsset::GetActiveSocketList, static TArray<class USkeletalMeshSocket*> Dummy; return Dummy;);

	/**
	 * Find a socket object in this SkeletalMesh by name.
	 * Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	virtual USkeletalMeshSocket* FindSocket(FName InSocketName) const
	PURE_VIRTUAL(USkinnedAsset::FindSocket, return nullptr;);

	/**
	 * Find a socket object and associated info in this SkeletalMesh by name.
	 * Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	 * Also returns the index for the socket allowing for future fast access via GetSocketByIndex()
	 * Also returns the socket transform and the bone index (if any)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation")
	virtual USkeletalMeshSocket* FindSocketInfo(FName InSocketName, FTransform& OutTransform, int32& OutBoneIndex, int32& OutIndex) const
	PURE_VIRTUAL(USkinnedAsset::FindSocketInfo, return nullptr;);

	virtual USkeleton* GetSkeleton()
	PURE_VIRTUAL(USkinnedAsset::GetSkeleton, return nullptr;);
	virtual const USkeleton* GetSkeleton() const
	PURE_VIRTUAL(USkinnedAsset::GetSkeleton, return nullptr;);
	virtual void SetSkeleton(USkeleton* InSkeleton)
	PURE_VIRTUAL(USkinnedAsset::SetSkeleton,);

	virtual UMeshDeformer* GetDefaultMeshDeformer() const
	PURE_VIRTUAL(USkinnedAsset::GetDefaultMeshDeformer, return nullptr;);
	
	virtual UMeshDeformerCollection* GetTargetMeshDeformers() const
	PURE_VIRTUAL(USkinnedAsset::GetTargetMeshDeformers, return nullptr;);
	
	virtual bool HasHalfEdgeBuffer(int32 LODIndex) const
	PURE_VIRTUAL(USkinnedAsset::HasHalfEdgeBuffer, return false;);
	
	virtual class UMaterialInterface* GetOverlayMaterial() const
	PURE_VIRTUAL(USkinnedAsset::GetOverlayMaterial, return nullptr;);
	virtual float GetOverlayMaterialMaxDrawDistance() const
	PURE_VIRTUAL(USkinnedAsset::GetOverlayMaterialMaxDrawDistance, return 0.f;);
	
	/** Return true if given index's LOD is valid */
	virtual bool IsValidLODIndex(int32 Index) const
	PURE_VIRTUAL(USkinnedAsset::IsValidLODIndex, return false;);

	virtual int32 GetMinLodIdx(bool bForceLowestLODIdx = false) const
	PURE_VIRTUAL(USkinnedAsset::GetMinLodIdx, return 0;);

	/** Return the morph targets. */
	virtual TArray<TObjectPtr<UMorphTarget>>& GetMorphTargets()
	{ static TArray<TObjectPtr<UMorphTarget>> Dummy; return Dummy; }
	virtual const TArray<TObjectPtr<UMorphTarget>>& GetMorphTargets() const
	{ static const TArray<TObjectPtr<UMorphTarget>> Dummy; return Dummy; }

	/**Find a named MorphTarget from the morph targets */
	virtual UMorphTarget* FindMorphTarget(FName MorphTargetName) const
	{ return nullptr; }

	/** True if this mesh LOD needs to keep it's data on CPU. USkinnedAsset interface. */
	virtual bool NeedCPUData(int32 LODIndex) const
	PURE_VIRTUAL(USkinnedAsset::NeedCPUData, return false;);

	/** Return whether or not the mesh has vertex colors. */
	virtual bool GetHasVertexColors() const
	PURE_VIRTUAL(USkinnedAsset::GetHasVertexColors, return false;);

	virtual int32 GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform) const
	PURE_VIRTUAL(USkinnedAsset::GetPlatformMinLODIdx, return 0;);

	virtual const FPerPlatformBool& GetDisableBelowMinLodStripping() const
	PURE_VIRTUAL(USkinnedAsset::GetDisableBelowMinLodStripping, static const FPerPlatformBool Dummy; return Dummy;);

	virtual void SetSkinWeightProfilesData(int32 LODIndex, struct FSkinWeightProfilesData& SkinWeightProfilesData) {}
	virtual FSkinWeightProfilesData* GetSkinWeightProfilesData(int32 LODIndex) { return nullptr; }

	/** Computes flags for building vertex buffers. */
	ENGINE_API virtual ESkeletalMeshVertexFlags GetVertexBufferFlags() const;

	/**
	 * Take the BoneSpaceTransforms array (translation vector, rotation quaternion and scale vector) and update the array of component-space bone transformation matrices (ComponentSpaceTransforms).
	 * It will work down hierarchy multiplying the component-space transform of the parent by the relative transform of the child.
	 * This code also applies any per-bone rotators etc. as part of the composition process
	 */
	ENGINE_API void FillComponentSpaceTransforms(const TArray<FTransform>& InBoneSpaceTransforms, const TArray<FBoneIndexType>& InFillComponentSpaceTransformsRequiredBones, TArray<FTransform>& OutComponentSpaceTransforms) const;

	//~ Begin UObject Interface
	/**
	* This will return detail info about this specific object. (e.g. AudioComponent will return the name of the cue,
	* ParticleSystemComponent will return the name of the ParticleSystem)  The idea here is that in many places
	* you have a component of interest but what you really want is some characteristic that you can use to track
	* down where it came from.
	*/
	virtual FString GetDetailedInfoInternal() const override
	{ return GetPathName(nullptr); }

	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface
		
	ENGINE_API void GetVertexFactoryTypesPerMaterialIndex(FPSOPrecacheVertexFactoryDataPerMaterialIndexList& OutList, USkinnedMeshComponent* SkinnedMeshComponent, int32 MinLODIndex, bool bCPUSkin, ERHIFeatureLevel::Type FeatureLevel);

	/** Helper function for resource tracking, construct a string using the skinned asset's path name and LOD index . */
	static ENGINE_API FString GetLODPathName(const USkinnedAsset* Mesh, int32 LODIndex);

	/** IInterface_AsyncCompilation begin*/
#if WITH_EDITOR
	ENGINE_API virtual bool IsCompiling() const override;
#else
	inline bool IsCompiling() const { return false; }
#endif
	/** IInterface_AsyncCompilation end*/

#if WITH_EDITOR
	virtual FString BuildDerivedDataKey(const ITargetPlatform* TargetPlatform)
	PURE_VIRTUAL(USkinnedAsset::BuildDerivedDataKey, return TEXT(""););

	/* Return true if this asset was never build since its creation. */
	virtual bool IsInitialBuildDone() const
	PURE_VIRTUAL(USkinnedAsset::IsInitialBuildDone, return false;);

	/* Build a LOD model before creating its render data. */
	UE_DEPRECATED(5.6, "Use or override the overload that takes FSkeletalMeshRenderData instead")
	virtual void BuildLODModel(const ITargetPlatform* TargetPlatform, int32 LODIndex) final {}
	
	/* Build a LOD model for the given render data. */
	virtual void BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex) {}

	/** Get whether this mesh should use LOD streaming for the given platform. */
	virtual bool GetEnableLODStreaming(const ITargetPlatform* TargetPlatform) const
	PURE_VIRTUAL(USkinnedAsset::GetEnableLODStreaming, return false;);

	/** Get the maximum number of LODs that can be streamed. */
	virtual int32 GetMaxNumStreamedLODs(const ITargetPlatform* TargetPlatform) const
	PURE_VIRTUAL(USkinnedAsset::GetMaxNumStreamedLODs, return 0;);

	virtual int32 GetMaxNumOptionalLODs(const ITargetPlatform* TargetPlatform) const
	PURE_VIRTUAL(USkinnedAsset::GetMaxNumOptionalLODs, return 0;);

	virtual ENGINE_API int64 GetRequiredMemoryForCompile() const;
	virtual bool HasCompileDependencies() const { return false; }
	virtual bool HasAnyDependenciesCompiling() const { return false; }
	virtual TArray<USkinnedAsset*> GetSkinnedAssetDependencies() const { return {}; }
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "This function and associated functionality is no longer supported.")
	static bool GetUseLegacyMeshDerivedDataKey()	{ return false; }

	/** Get the source mesh data. */
	virtual class FSkeletalMeshModel* GetImportedModel() const
	PURE_VIRTUAL(USkinnedAsset::GetImportedModel, return nullptr;);
#endif // WITH_EDITORONLY_DATA

	/**
	 * Update the material UV channel data used by the texture streamer.
	 *
	 * @param bResetOverrides True if overridden values should be reset.
	 */
	ENGINE_API void UpdateUVChannelData(bool bResetOverrides);

protected:
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void UpdateMaterialUVChannelData(FMeshUVChannelInfo& UVChannelData, int32 MaterialIndex);
#endif

	/** Lock properties that should not be modified/accessed during async build. */
	ENGINE_API void AcquireAsyncProperty(uint64 AsyncProperties = MAX_uint64, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite);
	/** Release properties that should not be modified/accessed during async build. */
	ENGINE_API void ReleaseAsyncProperty(uint64 AsyncProperties = MAX_uint64, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite);

	/**
	 * Wait for the asset to finish compilation to protect internal skinned asset data from race conditions during async build.
	 * A derived class may want to define its async properties as enums and cast them to uint64 when calling this function.
	 */
	ENGINE_API void WaitUntilAsyncPropertyReleasedInternal(uint64 AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite) const;

#if WITH_EDITOR
	/** Initial step for the building process - Can't be done in parallel. */
	virtual void BeginBuildInternal(FSkinnedAssetBuildContext& Context) {}
	/** Thread-safe part. */
	virtual void ExecuteBuildInternal(FSkinnedAssetBuildContext& Context) {}
	/** Complete the building process - Can't be done in parallel. */
	virtual void FinishBuildInternal(FSkinnedAssetBuildContext& Context) {}

	/** Initial step for the async task process - Can't be done in parallel. */
	virtual void BeginAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) {}
	/** Thread-safe part. */
	virtual void ExecuteAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) {}
	/** Complete the async task process - Can't be done in parallel. */
	virtual void FinishAsyncTaskInternal(FSkinnedAsyncTaskContext& Context) {}

	/** Try to cancel any pending async tasks.
	 *  Returns true if there is no more async tasks pending, false otherwise.
	 */
	ENGINE_API virtual bool TryCancelAsyncTasks();

	/** Holds the pointer to an async task if one exists. */
	TUniquePtr<FSkinnedAssetAsyncBuildTask> AsyncTask;
#endif // WITH_EDITOR

private:
	/** Initial step for the Post Load process - Can't be done in parallel. */
	virtual void BeginPostLoadInternal(FSkinnedAssetPostLoadContext& Context) {}
	/** Thread-safe part of the Post Load */
	virtual void ExecutePostLoadInternal(FSkinnedAssetPostLoadContext& Context) {}
	/** Complete the postload process - Can't be done in parallel. */
	virtual void FinishPostLoadInternal(FSkinnedAssetPostLoadContext& Context) {}

	/** Convert async property from enum value to string. */
	virtual FString GetAsyncPropertyName(uint64 Property) const { return TEXT(""); }

#if WITH_EDITOR
	/** Handle some common preparation steps between async post load and async build */
	virtual void PrepareForAsyncCompilation() {}
	/** Returns false if there is currently an async task running */
	virtual bool IsAsyncTaskComplete() const { return true; }

	/** Used as a bit-field indicating which properties are read by async compilation. */
	std::atomic<uint64> AccessedProperties;
	/** Used as a bit-field indicating which properties are written to by async compilation. */
	std::atomic<uint64> ModifiedProperties;
#endif // WITH_EDITOR

	friend class FSkinnedAssetCompilingManager;
	friend class FSkinnedAssetAsyncBuildWorker;

	static ENGINE_API TArray<FSkeletalMeshLODInfo>& GetMeshLodInfoDummyArray();
	static ENGINE_API TArray<FSkeletalMaterial>& GetSkeletalMaterialDummyArray();
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "PSOPrecache.h"
#endif
