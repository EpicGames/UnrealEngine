// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Contains the helper functions to compile skinned asset asynchronously.
 */

#if WITH_EDITOR
#include "Animation/MorphTarget.h"
#endif //WITH_EDITOR
#include "Async/AsyncWork.h"
#include "SkeletalMeshTypes.h"

class USkinnedAsset;

class FSkinnedAssetCompilationContext
{
public:
	FSkinnedAssetCompilationContext() = default;
	// Non-copyable
	FSkinnedAssetCompilationContext(const FSkinnedAssetCompilationContext&) = delete;
	FSkinnedAssetCompilationContext& operator=(const FSkinnedAssetCompilationContext&) = delete;
	// Movable
	FSkinnedAssetCompilationContext(FSkinnedAssetCompilationContext&&) = default;
	FSkinnedAssetCompilationContext& operator=(FSkinnedAssetCompilationContext&&) = default;
	
	//True if this compilation context is start from a serialize save
	bool bIsSerializeSaving = false;
#if WITH_EDITOR
	TUniquePtr<FFinishBuildMorphTargetData> FinishBuildMorphTargetData;
#endif //WITH_EDITOR
};

class FSkinnedAssetPostLoadContext : public FSkinnedAssetCompilationContext
{
public:
	bool bHasCachedDerivedData = false;
};

class FSkinnedAssetBuildContext : public FSkinnedAssetCompilationContext
{
public:
	TUniquePtr<class FSkinnedMeshComponentRecreateRenderStateContext> RecreateRenderStateContext;
};

class FSkinnedAsyncTaskContext : public FSkinnedAssetCompilationContext
{
public:
	FSkinnedAsyncTaskContext(FEvent* InEvent)
		: Event(InEvent)
	{
	}
	FEvent* Event = nullptr;
};

#if WITH_EDITOR

// Any thread implicated in the build must have a valid scope to be granted access to protected properties without causing any stalls.
class FSkinnedAssetAsyncBuildScope
{
public:
	ENGINE_API FSkinnedAssetAsyncBuildScope(const USkinnedAsset* SkinnedAsset);
	ENGINE_API ~FSkinnedAssetAsyncBuildScope();
	ENGINE_API static bool ShouldWaitOnLockedProperties(const USkinnedAsset* SkinnedAsset);

private:
	const USkinnedAsset* PreviousScope = nullptr;
	// Only the thread(s) compiling this Skeletal mesh will have full access to protected properties without causing any stalls.
	static thread_local const USkinnedAsset* SkinnedAssetBeingAsyncCompiled;
};

/**
 * Worker used to perform async static mesh compilation.
 */
class FSkinnedAssetAsyncBuildWorker : public FNonAbandonableTask
{
public:
	USkinnedAsset* SkinnedAsset;
	TOptional<FSkinnedAssetPostLoadContext> PostLoadContext;
	TOptional<FSkinnedAssetBuildContext> BuildContext;
	TOptional<FSkinnedAsyncTaskContext> AsyncTaskContext;

	/** Initialization constructor. */
	FSkinnedAssetAsyncBuildWorker(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAssetBuildContext&& InBuildContext)
		: SkinnedAsset(InSkinnedAsset)
		, BuildContext(MoveTemp(InBuildContext))
	{
	}

	/** Initialization constructor. */
	FSkinnedAssetAsyncBuildWorker(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAssetPostLoadContext&& InPostLoadContext)
		: SkinnedAsset(InSkinnedAsset)
		, PostLoadContext(MoveTemp(InPostLoadContext))
	{
	}

	FSkinnedAssetAsyncBuildWorker(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAsyncTaskContext&& InAsyncTaskContext)
		: SkinnedAsset(InSkinnedAsset)
		, AsyncTaskContext(MoveTemp(InAsyncTaskContext))
	{
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSkinnedAssetAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void ENGINE_API DoWork();
};

struct FSkinnedAssetAsyncBuildTask : public FAsyncTask<FSkinnedAssetAsyncBuildWorker>
{
	FSkinnedAssetAsyncBuildTask(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAssetPostLoadContext&& InPostLoadContext)
		: FAsyncTask<FSkinnedAssetAsyncBuildWorker>(InSkinnedAsset, MoveTemp(InPostLoadContext))
		, SkinnedAsset(InSkinnedAsset)
	{
	}

	FSkinnedAssetAsyncBuildTask(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAssetBuildContext&& InBuildContext)
		: FAsyncTask<FSkinnedAssetAsyncBuildWorker>(InSkinnedAsset, MoveTemp(InBuildContext))
		, SkinnedAsset(InSkinnedAsset)
	{
	}

	FSkinnedAssetAsyncBuildTask(
		USkinnedAsset* InSkinnedAsset,
		FSkinnedAsyncTaskContext&& InAsyncTaskContext)
		: FAsyncTask<FSkinnedAssetAsyncBuildWorker>(InSkinnedAsset, MoveTemp(InAsyncTaskContext))
		, SkinnedAsset(InSkinnedAsset)
	{
	}

	const USkinnedAsset* SkinnedAsset;
};

#endif // #if WITH_EDITOR
