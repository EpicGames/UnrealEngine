// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Contains the shared data that is used by skinned assets
 */

#include "BoneContainer.h"
#include "Components.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/PerPlatformProperties.h"
#include "SkeletalMeshReductionSettings.h"
#include "Animation/SkeletalMeshVertexAttribute.h"
#include "Async/Mutex.h"

#include "SkinnedAssetCommon.generated.h"


class FSkeletalMeshLODModel;
class UAnimSequence;
class UMaterialInterface;
struct FSkelMeshSection;
struct FSkeletalMeshLODGroupSettings;


//This is the total cloth time split up among multiple computation (updating gpu, updating sim, etc...)
DECLARE_CYCLE_STAT_EXTERN(TEXT("Cloth Total"), STAT_ClothTotalTime, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Cloth Writeback"), STAT_ClothWriteback, STATGROUP_Physics, ENGINE_API);

UENUM()
enum class ESkinCacheUsage : uint8
{
	// Auto will defer to child or global behavior based on context
	Auto		= 0,

	// Mesh will not use the skin cache. However, if Support Ray Tracing is enabled on the mesh, the skin cache will still be used for Ray Tracing updates
	Disabled	= uint8(-1),

	// Mesh will use the skin cache
	Enabled		= 1,
};

UENUM()
enum class ESkinCacheDefaultBehavior : uint8
{
	// All skeletal meshes are excluded from the skin cache. Each must opt in individually. If Support Ray Tracing is enabled on a mesh, will force inclusive behavior on that mesh
	Exclusive = 0,

	// All skeletal meshes are included into the skin cache. Each must opt out individually
	Inclusive = 1,
};

USTRUCT()
struct FSectionReference
{
	GENERATED_USTRUCT_BODY()

	/** Index of the section we reference. **/
	UPROPERTY(EditAnywhere, Category = SectionReference)
	int32 SectionIndex;

	FSectionReference()
		: SectionIndex(INDEX_NONE)
	{
	}

	FSectionReference(const int32& InSectionIndex)
		: SectionIndex(InSectionIndex)
	{
	}

	bool operator==(const FSectionReference& Other) const
	{
		return SectionIndex == Other.SectionIndex;
	}

#if WITH_EDITOR
	/** return true if it has a valid section index for LodModel parameter **/
	ENGINE_API bool IsValidToEvaluate(const FSkeletalMeshLODModel& LodModel) const;

	const FSkelMeshSection* GetMeshLodSection(const FSkeletalMeshLODModel& LodModel) const;
	int32 GetMeshLodSectionIndex(const FSkeletalMeshLODModel& LodModel) const;
#endif

	bool Serialize(FArchive& Ar)
	{
		Ar << SectionIndex;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FSectionReference& B)
	{
		B.Serialize(Ar);
		return Ar;
	}
};

USTRUCT()
struct FMorphTargetImportedSourceFileInfo
{
	GENERATED_USTRUCT_BODY()

	/** String that was used to mark generated by engine morph targets. */
	static FString GeneratedByEnginePrefix;

	ENGINE_API const FString& GetSourceFilename() const;

	ENGINE_API void SetSourceFilename(const FString& Filename);

	ENGINE_API bool IsGeneratedByEngine() const;

	ENGINE_API void SetGeneratedByEngine(bool bInGeneratedByEngine);

private:
	UPROPERTY(EditAnywhere, Category = MorphTargetInfo)
	FString SourceFilename;

	
	UPROPERTY()
	FGuid DerivedDataHash;

	friend FArchive& operator<<(FArchive& Ar, FMorphTargetImportedSourceFileInfo& MorphTargetImportedSourceFileInfo);
};

/** Struct containing information for a particular LOD level, such as materials and info for when to use it. */
USTRUCT()
struct FSkeletalMeshLODInfo
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * ScreenSize to display this LOD.
	 * The screen size is based around the projected diameter of the bounding
	 * sphere of the model. i.e. 0.5 means half the screen's maximum dimension.
	 */
	UPROPERTY(EditAnywhere, Category=SkeletalMeshLODInfo)
	FPerPlatformFloat ScreenSize{ 1.0 };

	/**	Used to avoid 'flickering' when on LOD boundary. Only taken into account when moving from complex->simple. */
	UPROPERTY(EditAnywhere, Category=SkeletalMeshLODInfo, meta=(DisplayName="LOD Hysteresis"))
	float LODHysteresis = 0.0f;

	/** Mapping table from this LOD's materials to the USkeletalMesh materials array.
	 * section index is the key
	 * remapped material index is the value, can be INDEX_NONE for no remapping
	 */
	UPROPERTY()
	TArray<int32> LODMaterialMap;

#if WITH_EDITORONLY_DATA
	/** Per-section control over whether to enable shadow casting. */
	UPROPERTY()
	TArray<bool> bEnableShadowCasting_DEPRECATED;

	/** This has been removed in editor. We could re-apply this in import time or by mesh reduction utilities*/
	UPROPERTY()
	TArray<FName> RemovedBones_DEPRECATED;
#endif

	/** build settings to apply when building render data. */
	UPROPERTY(EditAnywhere, Category = BuildSettings)
	FSkeletalMeshBuildSettings BuildSettings;

	/** Reduction settings to apply when building render data. */
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	FSkeletalMeshOptimizationSettings ReductionSettings;

	/** Bones which should be removed from the skeleton for the LOD level */
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	TArray<FBoneReference> BonesToRemove;

	/** Bones which should be prioritized for the quality, this will be weighted toward keeping source data. Use WeightOfPrioritization to control the value. */
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	TArray<FBoneReference> BonesToPrioritize;

	/** Sections which should be prioritized for the quality, this will be weighted toward keeping source data. Use WeightOfPrioritization to control the value. */
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	TArray<FSectionReference> SectionsToPrioritize;

	/** How much to consideration to give BonesToPrioritize and SectionsToPrioritize.  The weight is an additional vertex simplification penalty where 0 means nothing. */
	UPROPERTY(EditAnywhere, Category = ReductionSettings, meta = (UIMin = "0.0", ClampMin = "0.0"))
	float WeightOfPrioritization = 1.0f;

	/** Pose which should be used to reskin vertex influences for which the bones will be removed in this LOD level, uses ref-pose by default */
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	TObjectPtr<UAnimSequence> BakePose;

	/** This is used when you are sharing the LOD settings, but you'd like to override the BasePose. This precedes prior to BakePose*/
	UPROPERTY(EditAnywhere, Category = ReductionSettings)
	TObjectPtr<UAnimSequence> BakePoseOverride;

#if WITH_EDITORONLY_DATA
	/** The filename of the file tha was used to import this LOD if it was not auto generated. */
	UPROPERTY(VisibleAnywhere, Category= SkeletalMeshLODInfo, AdvancedDisplay)
	FString SourceImportFilename;
#endif

	/**
	 * How this LOD uses the skin cache feature. Auto will defer to the default project global option. If Support Ray Tracing is enabled on the mesh, will imply Enabled
	 */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo)
	ESkinCacheUsage SkinCacheUsage = ESkinCacheUsage::Auto;

	/** The Morph target position error tolerance in microns. Larger values result in better compression and lower memory footprint, but also lower quality. */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo, meta = (UIMin = "0.01", ClampMin = "0.01", UIMax = "10000.0", ClampMax = "10000.0"))
	float MorphTargetPositionErrorTolerance = 20.0f;

#if WITH_EDITORONLY_DATA

	/** Store the custom import morph target source file. The key of the map is the morph target name and the value is the source file path. */
	UPROPERTY(VisibleAnywhere, Category = SkeletalMeshLODInfo)
	TMap<FString, FMorphTargetImportedSourceFileInfo> ImportedMorphTargetSourceFilename;

#endif //WITH_EDITORONLY_DATA

	/** List of vertex attributes to include for rendering and what type they should be */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo, AdvancedDisplay, EditFixedSize, Meta=(NoResetToDefault))
	TArray<FSkeletalMeshVertexAttributeInfo> VertexAttributes;	
	
	/** Whether to disable morph targets for this LOD. */
	UPROPERTY()
	uint8 bHasBeenSimplified:1;

	UPROPERTY()
	uint8 bHasPerLODVertexColors : 1;

	/** Keeps this LODs data on the CPU so it can be used for things such as sampling in FX. */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo)
	uint8 bAllowCPUAccess : 1;
	
	/**
	 * Whether to force cache/cook half edge data that provides vertex connectivity information across material sections, which
 	 * may be useful for other systems like Mesh Deformer. Please note that Half Edge buffers are always built if the Skeletal Mesh Asset
 	 * has Default Mesh Deformer assigned, and the LOD has AllowMeshDeformer enabled, 
	 */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo, meta=(DisplayName="Force Build Half Edge Buffers", EditCondition=bAllowMeshDeformer))
	uint8 bBuildHalfEdgeBuffers : 1;

	/** Whether a Mesh Deformer applied to the mesh asset or Skinned Mesh Component should be used on this LOD or not */
	UPROPERTY(EditAnywhere, Category = SkeletalMeshLODInfo)
	uint8 bAllowMeshDeformer : 1;
	
	/**
	Mesh supports uniformly distributed sampling in constant time.
	Memory cost is 8 bytes per triangle.
	Example usage is uniform spawning of particles.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = SkeletalMeshLODInfo, meta=(EditCondition="bAllowCPUAccess"))
	uint8 bSupportUniformlyDistributedSampling : 1;

#if WITH_EDITORONLY_DATA
	/*
	 * This boolean specify if the LOD was imported with the base mesh or not.
	 */
	UPROPERTY()
	uint8 bImportWithBaseMesh:1;

	// Protects access to the build guid.
	struct FThreadSafeBuildGUID
	{
	private:
		FGuid   BuildGuid;
		mutable UE::FMutex Mutex;
	public:
		FThreadSafeBuildGUID() = default;
		FThreadSafeBuildGUID(const FGuid& InGuid)
			: BuildGuid(InGuid)
		{
		}

		FThreadSafeBuildGUID(const FThreadSafeBuildGUID& Other)
		{
			UE::TUniqueLock OtherLock(Other.Mutex);
			UE::TUniqueLock Lock(Mutex);
			BuildGuid = Other.BuildGuid;
		}

		FThreadSafeBuildGUID& operator=(const FThreadSafeBuildGUID& Other)
		{
			if (&Other != this)
			{
				UE::TUniqueLock OtherLock(Other.Mutex);
				UE::TUniqueLock Lock(Mutex);
				BuildGuid = Other.BuildGuid;
			}
			return *this;
		}

		bool operator==(const FGuid& Other) const
		{
			UE::TUniqueLock Lock(Mutex);
			return BuildGuid == Other;
		}

		bool operator==(const FThreadSafeBuildGUID& Other) const
		{
			if (&Other == this)
			{
				return true;
			}

			UE::TUniqueLock OtherLock(Other.Mutex);
			UE::TUniqueLock Lock(Mutex);
			return BuildGuid == Other.BuildGuid;
		}

		void operator=(const FGuid& Other)
		{
			UE::TUniqueLock Lock(Mutex);
			BuildGuid = Other;
		}

		void Invalidate()
		{
			UE::TUniqueLock Lock(Mutex);
			BuildGuid.Invalidate();
		}

		operator FGuid() const
		{
			UE::TUniqueLock Lock(Mutex);
			return BuildGuid;
		}

		bool IsValid() const
		{
			UE::TUniqueLock Lock(Mutex);
			return BuildGuid.IsValid();
		}

		FString ToString(EGuidFormats Format = EGuidFormats::Digits) const
		{
			UE::TUniqueLock Lock(Mutex);
			FString Out;
			BuildGuid.AppendString(Out, Format);
			return Out;
		}

		FThreadSafeBuildGUID(FThreadSafeBuildGUID&&) = delete;
		FThreadSafeBuildGUID& operator=(FThreadSafeBuildGUID&&) = delete;
	};

	//Temporary build GUID data
	//We use this GUID to store the LOD Key so we can know if the LOD needs to be rebuilt
	//This GUID is set when we Cache the render data (build function)
	FThreadSafeBuildGUID BuildGUID;

	ENGINE_API FGuid ComputeDeriveDataCacheKey(const FSkeletalMeshLODGroupSettings* SkeletalMeshLODGroupSettings);
#endif

	FSkeletalMeshLODInfo()
		: bHasBeenSimplified(false)
		, bHasPerLODVertexColors(false)
		, bAllowCPUAccess(false)
		, bBuildHalfEdgeBuffers(false)
		, bAllowMeshDeformer(true)
		, bSupportUniformlyDistributedSampling(false)
#if WITH_EDITORONLY_DATA
		, bImportWithBaseMesh(false)
#endif
	{
	}

};

//~ Begin Material Interface for USkeletalMesh
USTRUCT(BlueprintType)
struct FSkeletalMaterial
{
	GENERATED_USTRUCT_BODY()

	FSkeletalMaterial()
		: MaterialInterface(nullptr)
		, MaterialSlotName(NAME_None)
#if WITH_EDITORONLY_DATA
		, bEnableShadowCasting_DEPRECATED(true)
		, bRecomputeTangent_DEPRECATED(false)
		, ImportedMaterialSlotName(NAME_None)
#endif
		, OverlayMaterialInterface(nullptr)
	{

	}

	FSkeletalMaterial(
		UMaterialInterface* InMaterialInterface,
		FName InMaterialSlotName,
		FName InImportedMaterialSlotName = NAME_None,
		UMaterialInterface* InOverlayMaterialInterface = nullptr)
		: MaterialInterface( InMaterialInterface )
		, MaterialSlotName(InMaterialSlotName)
	#if WITH_EDITORONLY_DATA
		, bEnableShadowCasting_DEPRECATED(true)
		, bRecomputeTangent_DEPRECATED(false)
		, ImportedMaterialSlotName(InImportedMaterialSlotName)
	#endif
		, OverlayMaterialInterface(InOverlayMaterialInterface)
	{

	}

	FSkeletalMaterial( UMaterialInterface* InMaterialInterface
						, bool bInEnableShadowCasting = true
						, bool bInRecomputeTangent = false
						, FName InMaterialSlotName = NAME_None
						, FName InImportedMaterialSlotName = NAME_None
						, UMaterialInterface* InOverlayMaterialInterface = nullptr)
		: MaterialInterface( InMaterialInterface )
		, MaterialSlotName(InMaterialSlotName)
	#if WITH_EDITORONLY_DATA
		, bEnableShadowCasting_DEPRECATED(bInEnableShadowCasting)
		, bRecomputeTangent_DEPRECATED(bInRecomputeTangent)
		, ImportedMaterialSlotName(InImportedMaterialSlotName)
	#endif
		, OverlayMaterialInterface(InOverlayMaterialInterface)
	{

	}

	friend FArchive& operator<<( FArchive& Ar, FSkeletalMaterial& Elem );
#if WITH_EDITORONLY_DATA
	static void DeclareCustomVersions(FArchive& Ar);
#endif

	ENGINE_API friend bool operator==( const FSkeletalMaterial& LHS, const FSkeletalMaterial& RHS );
	ENGINE_API friend bool operator==( const FSkeletalMaterial& LHS, const UMaterialInterface& RHS );
	ENGINE_API friend bool operator==( const UMaterialInterface& LHS, const FSkeletalMaterial& RHS );

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMesh)
	TObjectPtr<UMaterialInterface> 	MaterialInterface;
	
	/*This name should be use by the gameplay to avoid error if the skeletal mesh Materials array topology change*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SkeletalMesh)
	FName						MaterialSlotName;
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool						bEnableShadowCasting_DEPRECATED;
	UPROPERTY()
	bool						bRecomputeTangent_DEPRECATED;
	/*This name should be use when we re-import a skeletal mesh so we can order the Materials array like it should be*/
	UPROPERTY(VisibleAnywhere, Category = SkeletalMesh)
	FName						ImportedMaterialSlotName;
#endif

	/** Data used for texture streaming relative to each UV channels. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = SkeletalMesh)
	FMeshUVChannelInfo			UVChannelData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMesh)
	TObjectPtr<UMaterialInterface> 	OverlayMaterialInterface;
};

USTRUCT()
struct FSkeletalMaterialMinimalInfo
{
	GENERATED_BODY()

	FSkeletalMaterialMinimalInfo(FName InMaterialSlotName = NAME_None, FString InMaterialObjectPath = FString())
		: MaterialSlotName(InMaterialSlotName), MaterialObjectPath(InMaterialObjectPath)
	{}

	FName MaterialSlotName;
	FString MaterialObjectPath;
};
