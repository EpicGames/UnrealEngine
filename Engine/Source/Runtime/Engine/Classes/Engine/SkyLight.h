// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "SkyLight.generated.h"

UCLASS(ClassGroup=Lights, hidecategories=(Input,Collision,Replication,Info), showcategories=(Rendering, DataLayers, "Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass, ConversionRoot, Blueprintable, MinimalAPI)
class ASkyLight : public AInfo
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	/** @todo document */
	UPROPERTY(Category = Light, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Light,Rendering,Rendering|Components|SkyLight", AllowPrivateAccess = "true"))
	TObjectPtr<class USkyLightComponent> LightComponent;
public:
	/** replicated copy of LightComponent's bEnabled property */
	UPROPERTY(replicatedUsing=OnRep_bEnabled)
	uint32 bEnabled:1;

	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_bEnabled();

	/** Returns LightComponent subobject **/
	class USkyLightComponent* GetLightComponent() const { return LightComponent; }
};



