// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SoftWorldReference.generated.h"

class UWorld;

/** A simple wrapper type to enable content-defined structs to hold soft references to UWorld assets **/
USTRUCT(BlueprintType)
struct FSoftWorldReference
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Asset")
	TSoftObjectPtr<UWorld> WorldAsset;
};
