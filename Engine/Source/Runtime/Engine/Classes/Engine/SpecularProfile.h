// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "RenderResource.h"
#include "Curves/CurveLinearColor.h"
#include "SpecularProfile.generated.h"

#define MAX_SPECULAR_PROFILE_COUNT 256

class UTexture2D;
class FRDGBuilder;
class FTextureReference;

/** List of niagara solvers */
UENUM(BlueprintType)
enum class ESpecularProfileFormat : uint8
{
	ViewLightVector UMETA(DisplayName = "View / Light", ToolTip="The specular LUT will be sampled based on NoV (angle between the Normal and View direction) for the view color axis and NoL (angle between the Normal and Light direction) for the light color axis."),
	HalfVector UMETA(DisplayName = "Half Angle", ToolTip = "The specular LUT will be sampled based on VoH (angle between the View and the Half vector) for the view color axis and NoH (angle between the Normal and the Half vector) for the light color axis."),
};

// struct with all the settings we want in USpecularProfile, separate to make it easer to pass this data around in the engine.
USTRUCT(BlueprintType)
struct FSpecularProfileStruct
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * Define the format driving the sampling of the specular LUT.
	 */
	UPROPERTY(Category = "Common", EditAnywhere, BlueprintReadOnly, meta=(DisplayName="LUT Format"))
	ESpecularProfileFormat Format;

	/**
	* Define the view facing color.
	* Exemple with View/Light mode: color at 0 is applied when NoV=0 (view grazing angle)  while color at 1 is applied when NoV=1 (view facing angle).
	*/
	UPROPERTY(Category = "Procedural", EditAnywhere, meta = (AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers="false", ShowOutputGridNumbers="false"))
	FRuntimeCurveLinearColor ViewColor;

	/**
	* Define the light facing color
	* Exemple with View/Light mode: color at 0 is applied when NoL=0 (light hit the surface at grazing angle)  while color at 1 is applied when NoV=1 (light hit the surface at facing angle).
	*/
	UPROPERTY(Category = "Procedural", EditAnywhere, meta = (AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers="false", ShowOutputGridNumbers="false"))
	FRuntimeCurveLinearColor LightColor;

	/**
	 * Define the texture used as a specular profile
	 */
	UPROPERTY(Category  ="Texture", EditAnywhere, BlueprintReadOnly, meta=(DisplayName="Texture"))
	TObjectPtr<UTexture2D> Texture;

	FSpecularProfileStruct();

	bool IsProcedural() const { return Texture == nullptr; }

	void Invalidate()
	{
		*this = FSpecularProfileStruct();
	}
};

/**
 * Specular profile asset, can be specified at a material. 
 * Don't change at runtime. All properties in here are per material.
 */
UCLASS(autoexpandcategories = SpecularProfile, MinimalAPI)
class USpecularProfile : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = USpecularProfile, EditAnywhere, meta = (ShowOnlyInnerProperties))
	struct FSpecularProfileStruct Settings;

	UPROPERTY()
	FGuid Guid;

	//~ Begin UObject Interface
	virtual void BeginDestroy();
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode);
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif
	//~ End UObject Interface
};

namespace SpecularProfile
{
// Atlas - Initializes or updates the contents of the specular profile texture.
ENGINE_API void UpdateSpecularProfileTextureAtlas(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);

// Atlas - Returns the specular profile texture if it exists, or null.
ENGINE_API FRHITexture* GetSpecularProfileTextureAtlas();

// Atlas - Returns the specular profile texture if it exists, or black.
ENGINE_API FRHITexture* GetSpecularProfileTextureAtlasWithFallback();

// Profile - Initializes or updates the contents of the specular profile texture.
ENGINE_API int32 AddOrUpdateProfile(const USpecularProfile* InProfile, const FGuid& InGuid, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture);

// Profile - Returns the specular profile ID shader parameter name
ENGINE_API FName GetSpecularProfileParameterName(const USpecularProfile* InProfile);

// Profile - Returns the specular profile ID for a given Specular Profile object
ENGINE_API float GetSpecularProfileId(const USpecularProfile* In);

// Profile - Returns the shader parameter name for a Specular profile.
ENGINE_API FName CreateSpecularProfileParameterName(USpecularProfile* InProfile);
}