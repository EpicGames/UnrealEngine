// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/ReflectionCapture.h"
#include "SphereReflectionCapture.generated.h"

class UDrawSphereComponent;

/** 
 *	Actor used to capture the scene for reflection in a sphere shape.
 *	@see https://docs.unrealengine.com/latest/INT/Resources/ContentExamples/Reflections/1_4
 */
UCLASS(hidecategories = (Collision, Attachment, Actor), MinimalAPI)
class ASphereReflectionCapture : public AReflectionCapture
{
	GENERATED_UCLASS_BODY()

private:
	/** Sphere component used to visualize the capture radius */
	UPROPERTY()
	TObjectPtr<UDrawSphereComponent> DrawCaptureRadius;

public:

#if WITH_EDITOR
	//~ Begin AActor Interface.
	virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif

	/** Returns DrawCaptureRadius subobject **/
	UDrawSphereComponent* GetDrawCaptureRadius() const { return DrawCaptureRadius; }
};



