// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "SplineMeshActor.generated.h"

/**
 * SplineMeshActor is an actor with a SplineMeshComponent.
 *
 * @see USplineMeshComponent
 */
UCLASS(hideCategories = (Input), showCategories = ("Input|MouseInput", "Input|TouchInput"), ConversionRoot, ComponentWrapperClass, meta = (ChildCanTick), MinimalAPI)
class ASplineMeshActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = SplineMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh,Components|SplineMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class USplineMeshComponent> SplineMeshComponent;

public:

	/** Function to change mobility type */
	ENGINE_API void SetMobility(EComponentMobility::Type InMobility);

#if WITH_EDITOR
	//~ Begin AActor Interface
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	//~ End AActor Interface
#endif // WITH_EDITOR	

protected:
	//~ Begin UObject Interface.
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	//~ End UObject Interface.

public:
	/** Returns SplineMeshComponent subobject **/
	ENGINE_API class USplineMeshComponent* GetSplineMeshComponent() const;
};



