// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/EngineTypes.h"
#include "PhysicsEngine/BodyInstance.h"

#include "SplineMeshComponentDescriptor.generated.h"

// Inspired in large part from ISMComponentDescriptor.h

class USplineMeshComponent;
class UStaticMesh;
enum class ERuntimeVirtualTextureMainPassType : uint8;
enum class ERendererStencilMask : uint8;

USTRUCT()
struct FSplineMeshComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSplineMeshComponentDescriptorBase();
	ENGINE_API explicit FSplineMeshComponentDescriptorBase(ENoInit);
	ENGINE_API virtual ~FSplineMeshComponentDescriptorBase();

	ENGINE_API USplineMeshComponent* CreateComponent(UObject* Outer, FName Name = NAME_None, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) const;

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true);
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(USplineMeshComponent* SplineMeshComponent) const;

	ENGINE_API void PostLoadFixup(UObject* Loader);

	ENGINE_API bool operator!=(const FSplineMeshComponentDescriptorBase& Other) const;
	ENGINE_API bool operator==(const FSplineMeshComponentDescriptorBase& Other) const;

	friend inline uint32 GetTypeHash(const FSplineMeshComponentDescriptorBase& Key)
	{
		return Key.GetTypeHash();
	}

	uint32 GetTypeHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

public:
	UPROPERTY()
	mutable uint32 Hash = 0;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TSubclassOf<USplineMeshComponent> ComponentClass;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<EComponentMobility::Type> Mobility;
		
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ELightmapType LightmapType;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	FLightingChannels LightingChannels;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 RayTracingGroupId;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERayTracingGroupCullingPriority RayTracingGroupCullingPriority;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<EHasCustomNavigableGeometry::Type> bHasCustomNavigableGeometry;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERendererStencilMask CustomDepthStencilWriteMask;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	FBodyInstance BodyInstance;
		
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 VirtualTextureCullMips;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 TranslucencySortPriority;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 OverriddenLightMapRes;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 CustomDepthStencilValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	EHLODBatchingPolicy HLODBatchingPolicy;
#endif
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEmissiveLightSource : 1;
		
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastDynamicShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastStaticShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastContactShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastShadowAsTwoSided : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastHiddenShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDynamicIndirectLighting : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDynamicIndirectLightingWhileHidden : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDistanceFieldLighting : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bReceivesDecals : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bOverrideLightMapRes : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bUseAsOccluder : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEnableDiscardOnLoad : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRenderCustomDepth : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisibleInRayTracing : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisibleInReflections : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bHiddenInGame : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bIsEditorOnly : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisible : 1;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEvaluateWorldPositionOffset : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bReverseCulling : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bIncludeInHLOD : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bConsiderForActorPlacementWhenHidden : 1;
#endif

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "bGenerateOverlapEvents"))
	uint8 bUseDefaultCollision : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "CustomDepthStencilWriteMask"))
	uint8 bGenerateOverlapEvents : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bOverrideNavigationExport : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bForceNavigationObstacle : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bFillCollisionUnderneathForNavmesh : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bNeverNeedsCookedCollisionData : 1;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 WorldPositionOffsetDisableDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<enum EDetailMode> DetailMode;
};

USTRUCT()
struct FSplineMeshComponentDescriptor : public FSplineMeshComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSplineMeshComponentDescriptor();
	ENGINE_API explicit FSplineMeshComponentDescriptor(const FSoftSplineMeshComponentDescriptor& Other);
	ENGINE_API ~FSplineMeshComponentDescriptor();
	static ENGINE_API FSplineMeshComponentDescriptor CreateFrom(const TSubclassOf<UStaticMeshComponent>& ComponentClass);

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(USplineMeshComponent* SplineMeshComponent) const override;
		
	ENGINE_API bool operator!=(const FSplineMeshComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FSplineMeshComponentDescriptor& Other) const;

	friend inline bool operator<(const FSplineMeshComponentDescriptor& Lhs, const FSplineMeshComponentDescriptor& Rhs)
	{
		return Lhs.Hash < Rhs.Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TObjectPtr<UMaterialInterface> OverlayMaterial;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;
};

USTRUCT()
struct FSoftSplineMeshComponentDescriptor : public FSplineMeshComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSoftSplineMeshComponentDescriptor();
	ENGINE_API explicit FSoftSplineMeshComponentDescriptor(const FSplineMeshComponentDescriptor& Other);
	ENGINE_API ~FSoftSplineMeshComponentDescriptor();
	static ENGINE_API FSoftSplineMeshComponentDescriptor CreateFrom(const TSubclassOf<UStaticMeshComponent>& ComponentClass);

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(USplineMeshComponent* SplineMeshComponent) const override;

	ENGINE_API bool operator!=(const FSoftSplineMeshComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FSoftSplineMeshComponentDescriptor& Other) const;

	friend inline bool operator<(const FSoftSplineMeshComponentDescriptor& Lhs, const FSoftSplineMeshComponentDescriptor& Rhs)
	{
		return Lhs.Hash < Rhs.Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TSoftObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TSoftObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TSoftObjectPtr<UMaterialInterface> OverlayMaterial;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TSoftObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;
};
