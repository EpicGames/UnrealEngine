// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Light.h"
#include "SpotLight.generated.h"

UCLASS(ClassGroup=(Lights, SpotLights), MinimalAPI, ComponentWrapperClass, meta=(ChildCanTick))
class ASpotLight : public ALight
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Light", meta=(ExposeFunctionCategories="SpotLight,Rendering|Lighting"))
	TObjectPtr<class USpotLightComponent> SpotLightComponent;

#if WITH_EDITORONLY_DATA
	// Reference to editor arrow component visualization 
private:
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
public:
#endif

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	void SetInnerConeAngle(float NewInnerConeAngle);
	UFUNCTION(BlueprintCallable, Category="Rendering|Lighting", meta=(DeprecatedFunction))
	void SetOuterConeAngle(float NewOuterConeAngle);
	// END DEPRECATED

	// Disable this for now
	//UFUNCTION(BlueprintCallable, Category="Rendering|Lighting")
	//void SetLightShaftConeAngle(float NewLightShaftConeAngle);

#if WITH_EDITOR
	//~ Begin AActor Interface.
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface


public:
#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif
};



