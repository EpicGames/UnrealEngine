// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMath.h"
#include "HAL/IConsoleManager.h"
#include "SpringInterpolator.generated.h"

struct FRK4SpringConstants
{
	float SpringStiffness;
	float DampingConstant;

	FRK4SpringConstants()
		: SpringStiffness(1.f)
		, DampingConstant(1.f)
	{}

	FRK4SpringConstants(float InSpringStiffness, float InDampingConstant)
		: SpringStiffness(InSpringStiffness)
		, DampingConstant(InDampingConstant)
	{}
};

// Runge-Kutta order 4 integrator
namespace RK4Integrator
{
	template <typename T>
	struct FRK4State
	{
		T Position;
		T Velocity;

		FRK4State()
			: Position(T(0.f))
			, Velocity(T(0.f))
		{}

		FRK4State(T InPosition, T InVelocity)
			: Position(InPosition)
			, Velocity(InVelocity)
		{}
	};

	template <typename T>
	struct FRK4StateDerivative
	{
		T Velocity;
		T Acceleration;

		FRK4StateDerivative()
			: Velocity(T(0.f))
			, Acceleration(T(0.f))
		{}

		FRK4StateDerivative(T InVelocity, T InAcceleration)
			: Velocity(InVelocity)
			, Acceleration(InAcceleration)
		{}
	};

	template <typename T>
	static T Accelerate(const FRK4State<T>& InCurrentState, const T& InTargetPosition, const FRK4SpringConstants& InSpringConstants)
	{
		return (InTargetPosition - InCurrentState.Position) * InSpringConstants.SpringStiffness - InCurrentState.Velocity * InSpringConstants.DampingConstant;
	}

	template <typename T>
	static FRK4StateDerivative<T> Evaluate(const FRK4State<T>& InCurrentState, const FRK4StateDerivative<T>& InDerivativeState, const T& InTargetPosition, const FRK4SpringConstants& InSpringConstants, const float& InDeltaTime)
	{
		FRK4State<T> NewState;
		NewState.Position = InCurrentState.Position + InDerivativeState.Velocity * InDeltaTime;
		NewState.Velocity = InCurrentState.Velocity + InDerivativeState.Acceleration * InDeltaTime;

		FRK4StateDerivative<T> NewStateDerivative;
		NewStateDerivative.Velocity = NewState.Velocity;
		NewStateDerivative.Acceleration = RK4Integrator::Accelerate(NewState, InTargetPosition, InSpringConstants);

		return NewStateDerivative;
	}

	template <typename T>
	static void Integrate(FRK4State<T>& InCurrentState, const T& InTargetPosition, const FRK4SpringConstants& InSpringConstants, const float& InDeltaTime)
	{
		const FRK4StateDerivative<T> z = FRK4StateDerivative<T>();
		const FRK4StateDerivative<T> a = Evaluate(InCurrentState, z, InTargetPosition, InSpringConstants, 0.f);
		const FRK4StateDerivative<T> b = Evaluate(InCurrentState, a, InTargetPosition, InSpringConstants, InDeltaTime*0.5f);
		const FRK4StateDerivative<T> c = Evaluate(InCurrentState, b, InTargetPosition, InSpringConstants, InDeltaTime*0.5f);
		const FRK4StateDerivative<T> d = Evaluate(InCurrentState, c, InTargetPosition, InSpringConstants, InDeltaTime);

		const T Velocity = (1.0f / 6.0f) * (a.Velocity + 2.0f*(b.Velocity + c.Velocity) + d.Velocity);
		const T Acceleration = (1.0f / 6.0f) * (a.Acceleration + 2.0f*(b.Acceleration + c.Acceleration) + d.Acceleration);

		InCurrentState.Position += Velocity * InDeltaTime;
		InCurrentState.Velocity += Acceleration * InDeltaTime;
	}
}

extern ENGINE_API float RK4_SPRING_INTERPOLATOR_UPDATE_RATE;
extern ENGINE_API int32 RK4_SPRING_INTERPOLATOR_MAX_ITER;

// Max Spring Value to prevent system from exploding and creating NaNs
#define RK4_SPRING_INTERPOLATOR_MAX_VALUE (1e+16f)

// Tweaked so Spring comes to rest in a reasonable manner.
#define RK4_SPRING_INTERPOLATOR_POSITION_TOLERANCE 0.001f
#define RK4_SPRING_INTERPOLATOR_VELOCITY_TOLERANCE 0.01f

namespace FRK4SpringInterpolatorUtils
{
	static inline bool IsValidValue(float Value, float MaxAbsoluteValue = RK4_SPRING_INTERPOLATOR_MAX_VALUE)
	{
		return FMath::Abs(Value) < MaxAbsoluteValue;
	}

	static inline bool IsValidValue(FVector Value, float MaxAbsoluteValue = RK4_SPRING_INTERPOLATOR_MAX_VALUE)
	{
		return Value.GetAbsMax() < MaxAbsoluteValue;
	}

	static inline bool AreEqual(float A, float B, float ErrorTolerance = UE_KINDA_SMALL_NUMBER)
	{
		return FMath::IsNearlyEqual(A, B, ErrorTolerance);
	}

	static inline bool AreEqual(FVector A, FVector B, float ErrorTolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Equals(B, ErrorTolerance);
	}
}

template <typename T>
struct FRK4SpringInterpolator
{
protected:
	float StiffnessConstant;
	float DampeningRatio;

	bool bIsInitialized;
	bool bIsInMotion;
	float TimeRemaining;
	FRK4SpringConstants SpringConstants;

	T LastPosition;
	RK4Integrator::FRK4State<T> State;

public:
	FRK4SpringInterpolator(float InSpringStiffness = 1.f, float InDampeningRatio = 1.f)
		: StiffnessConstant(InSpringStiffness)
		, DampeningRatio(InDampeningRatio)
		, bIsInitialized(false)
		, bIsInMotion(false)
		, TimeRemaining(0.f)
		, LastPosition(0.f)
	{}

	/** @param InDampeningRatio	0 = Undamped, <1 = Underdamped, 1 = Critically damped, >1 = Overdamped */
	void SetDefaultSpringConstants(const float& InSpringStiffness, const float& InDampeningRatio = 1.f)
	{
		StiffnessConstant = InSpringStiffness;
		DampeningRatio = InDampeningRatio;
	}

	/** @param InDampeningRatio	0 = Undamped, <1 = Underdamped, 1 = Critically damped, >1 = Overdamped */
	void SetSpringConstants(const float& InSpringStiffness, const float& InDampeningRatio = 1.f)
	{
		SpringConstants.SpringStiffness = InSpringStiffness;
		SpringConstants.DampingConstant = InDampeningRatio * 2.f * FMath::Sqrt(InSpringStiffness);
		bIsInitialized = true;
	}

	float GetDefaultStiffnessConstant() const
	{
		return StiffnessConstant;
	}

	bool IsInMotion() const
	{
		return bIsInMotion;
	}

	T GetPosition() const
	{
		return State.Position;
	}

	void SetPosition(const T& NewPosition)
	{
		State.Position = NewPosition;
	}

	void OffsetPosition(const T& Offset)
	{
		State.Position += Offset;
	}

	T GetLastDeltaPosition() const
	{
		return (State.Position - LastPosition);
	}

	T GetVelocity() const
	{
		return State.Velocity;
	}

	void SetVelocity(const T& NewVelocity)
	{
		State.Velocity = NewVelocity;
		bIsInMotion = bIsInMotion || !IsVelocityZero();
	}

	void OffsetVelocity(const T& Offset)
	{
		SetVelocity(State.Velocity + Offset);
	}

	bool IsPositionEqualTo(const T& OtherPosition) const
	{
		return FRK4SpringInterpolatorUtils::AreEqual(GetPosition(), OtherPosition, RK4_SPRING_INTERPOLATOR_POSITION_TOLERANCE);
	}

	bool IsVelocityZero() const
	{
		return FRK4SpringInterpolatorUtils::AreEqual(GetVelocity(), T(0.f), RK4_SPRING_INTERPOLATOR_VELOCITY_TOLERANCE);
	}

	void Reset(const T& DefaultPosition = T(0.f))
	{
		State.Position = DefaultPosition;
		State.Velocity = T(0.f);
		bIsInMotion = false;
		TimeRemaining = 0.f;
	}

	T Update(const T& InTargetPosition, const float& InDeltaTime)
	{
		if (InDeltaTime > 0.f)
		{
			LastPosition = State.Position;
			bIsInMotion = bIsInMotion || !IsPositionEqualTo(InTargetPosition);
			if (bIsInMotion)
			{
				if (!bIsInitialized)
				{
					SetSpringConstants(StiffnessConstant, DampeningRatio);
					bIsInitialized = true;
				}

				TimeRemaining = FMath::Min(TimeRemaining + InDeltaTime, 0.1f); // hard cap at 10Hz

				const float StepTime = 1.f / FMath::Max(RK4_SPRING_INTERPOLATOR_UPDATE_RATE, 1.f);

				int32 NumIterations = FMath::Min(FMath::TruncToInt(TimeRemaining / StepTime), RK4_SPRING_INTERPOLATOR_MAX_ITER);
				TimeRemaining -= float(NumIterations) * StepTime;

				while (NumIterations-- > 0)
				{
					RK4Integrator::Integrate(State, InTargetPosition, SpringConstants, StepTime);
				}

				// See if we can take one last update to have an accurate position this frame
				if ((TimeRemaining > 0.f) && (TimeRemaining < StepTime))
				{
					RK4Integrator::Integrate(State, InTargetPosition, SpringConstants, TimeRemaining);
					TimeRemaining = 0.f;
				}

				const bool bReachedPosition = IsPositionEqualTo(InTargetPosition);
				const bool bVelocityIsZero = IsVelocityZero();
				if ((bReachedPosition && bVelocityIsZero)
					|| !FRK4SpringInterpolatorUtils::IsValidValue(State.Velocity)
					|| !FRK4SpringInterpolatorUtils::IsValidValue(State.Position))
				{
					Reset(InTargetPosition);
				}
			}
		}

		return State.Position;
	}

	T Update(const T& InNewPosition, const T& InTargetPosition, const float& InDeltaTime)
	{
		SetPosition(InNewPosition);
		return Update(InTargetPosition, InDeltaTime);
	}
};

struct FFloatRK4SpringInterpolator : FRK4SpringInterpolator<float>
{
private:
	typedef FRK4SpringInterpolator<float> Super;

public:
	FFloatRK4SpringInterpolator()
		: Super()
	{}

	FFloatRK4SpringInterpolator(const Super& Other)
		: Super(Other)
	{}
};

struct FVectorRK4SpringInterpolator : FRK4SpringInterpolator<FVector>
{
private:
	typedef FRK4SpringInterpolator<FVector> Super;

public:
	FVectorRK4SpringInterpolator()
		: Super()
	{}

	FVectorRK4SpringInterpolator(const Super& Other)
		: Super(Other)
	{}
};

#if !CPP	// begin noexport class

USTRUCT(noexport, BlueprintType)
struct FFloatRK4SpringInterpolator
{
	UPROPERTY(EditAnywhere, Category = "FloatRK4SpringInterpolator")
	float StiffnessConstant;

	/** 0 = Undamped, <1 = Underdamped, 1 = Critically damped, >1 = Over damped */
	UPROPERTY(EditAnywhere, Category = "FloatRK4SpringInterpolator")
	float DampeningRatio;

	bool bIsInitialized;
	bool bIsInMotion;
	float TimeRemaining;
	FRK4SpringConstants SpringConstants;

	float LastPosition;
	RK4Integrator::FRK4State<float> State;
};

USTRUCT(noexport, BlueprintType)
struct FVectorRK4SpringInterpolator
{
	UPROPERTY(EditAnywhere, Category = "VectorRK4SpringInterpolator")
	float StiffnessConstant;

	/** 0 = Undamped, <1 = Underdamped, 1 = Critically damped, >1 = Over damped */
	UPROPERTY(EditAnywhere, Category = "VectorRK4SpringInterpolator")
	float DampeningRatio;

	bool bIsInitialized;
	bool bIsInMotion;
	float TimeRemaining;
	FRK4SpringConstants SpringConstants;

	FVector LastPosition;
	RK4Integrator::FRK4State<FVector> State;
};

#endif	// end noexport class
