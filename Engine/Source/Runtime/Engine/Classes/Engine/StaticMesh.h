// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Engine/EngineTypes.h"
#include "UObject/ScriptMacros.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "RenderCommandFence.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "MeshUVChannelInfo.h"
#include "Engine/StreamableRenderAsset.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "StaticMeshSourceData.h"
#include "UObject/PerPlatformProperties.h"
#include "MeshTypes.h"
#include "PerQualityLevelProperties.h"

#include "StaticMesh.generated.h"

class FSpeedTreeWind;
class FStaticMeshLODGroup;
class UAssetUserData;
class UMaterialInterface;
class UNavCollisionBase;
class UStaticMeshComponent;
class UStaticMeshDescription;
class FStaticMeshUpdate;
class UPackage;
struct FMeshDescription;
struct FStaticMeshLODResources;
class IStaticMeshComponent;
struct FStaticMeshSection;

/*-----------------------------------------------------------------------------
	Async Static Mesh Compilation
-----------------------------------------------------------------------------*/

enum class EStaticMeshAsyncProperties : uint32
{
	None                    = 0,
	RenderData              = 1 << 0,
	//OccluderData            = 1 << 1,
	SourceModels            = 1 << 2,
	SectionInfoMap          = 1 << 3,
	OriginalSectionInfoMap  = 1 << 4,
	NavCollision            = 1 << 5,
	LightmapUVVersion       = 1 << 6,
	BodySetup               = 1 << 7,
	LightingGuid            = 1 << 8,
	ExtendedBounds          = 1 << 9,
	NegativeBoundsExtension = 1 << 10,
	PositiveBoundsExtension = 1 << 11,
	StaticMaterials         = 1 << 12,
	LightmapUVDensity       = 1 << 13,
	IsBuiltAtRuntime        = 1 << 14,
	MinLOD                  = 1 << 15,
	LightMapCoordinateIndex = 1 << 16,
	LightMapResolution      = 1 << 17,
	HiResSourceModel		= 1 << 18,
	UseLegacyTangentScaling = 1 << 19,

	All                     = MAX_uint32
};

inline const TCHAR* ToString(EStaticMeshAsyncProperties Value)
{
	switch (Value)
	{
		case EStaticMeshAsyncProperties::None: 
			return TEXT("None");
		case EStaticMeshAsyncProperties::RenderData: 
			return TEXT("RenderData");
		case EStaticMeshAsyncProperties::SourceModels: 
			return TEXT("SourceModels");
		case EStaticMeshAsyncProperties::SectionInfoMap: 
			return TEXT("SectionInfoMap");
		case EStaticMeshAsyncProperties::OriginalSectionInfoMap:
			return TEXT("OriginalSectionInfoMap");
		case EStaticMeshAsyncProperties::NavCollision: 
			return TEXT("NavCollision");
		case EStaticMeshAsyncProperties::LightmapUVVersion: 
			return TEXT("LightmapUVVersion");
		case EStaticMeshAsyncProperties::BodySetup: 
			return TEXT("BodySetup");
		case EStaticMeshAsyncProperties::LightingGuid: 
			return TEXT("LightingGuid");
		case EStaticMeshAsyncProperties::ExtendedBounds: 
			return TEXT("ExtendedBounds");
		case EStaticMeshAsyncProperties::NegativeBoundsExtension:
			return TEXT("NegativeBoundsExtension");
		case EStaticMeshAsyncProperties::PositiveBoundsExtension:
			return TEXT("PositiveBoundsExtension");
		case EStaticMeshAsyncProperties::StaticMaterials: 
			return TEXT("StaticMaterials");
		case EStaticMeshAsyncProperties::LightmapUVDensity: 
			return TEXT("LightmapUVDensity");
		case EStaticMeshAsyncProperties::IsBuiltAtRuntime: 
			return TEXT("IsBuiltAtRuntime");
		case EStaticMeshAsyncProperties::MinLOD:
			return TEXT("MinLOD");
		case EStaticMeshAsyncProperties::LightMapCoordinateIndex:
			return TEXT("LightMapCoordinateIndex");
		case EStaticMeshAsyncProperties::LightMapResolution:
			return TEXT("LightMapResolution");
		case EStaticMeshAsyncProperties::HiResSourceModel:
			return TEXT("HiResSourceModel");
		case EStaticMeshAsyncProperties::UseLegacyTangentScaling:
			return TEXT("UseLegacyTangentScaling");
		case EStaticMeshAsyncProperties::All:
		default: 
			check(false); 
			return TEXT("Unknown");
	}
}

ENUM_CLASS_FLAGS(EStaticMeshAsyncProperties);

class FStaticMeshPostLoadContext;
class FStaticMeshBuildContext;

#if WITH_EDITOR

// Any thread implicated in the static mesh build must have a valid scope to be granted access to protected properties without causing any stalls.
class FStaticMeshAsyncBuildScope
{
public:
	FStaticMeshAsyncBuildScope(const UStaticMesh* StaticMesh)
	{
		PreviousScope = StaticMeshBeingAsyncCompiled;
		StaticMeshBeingAsyncCompiled = StaticMesh;
	}

	~FStaticMeshAsyncBuildScope()
	{
		check(StaticMeshBeingAsyncCompiled);
		StaticMeshBeingAsyncCompiled = PreviousScope;
	}

	static bool ShouldWaitOnLockedProperties(const UStaticMesh* StaticMesh)
	{
		return StaticMeshBeingAsyncCompiled != StaticMesh;
	}

private:
	const UStaticMesh* PreviousScope = nullptr;
	// Only the thread(s) compiling this static mesh will have full access to protected properties without causing any stalls.
	static thread_local const UStaticMesh* StaticMeshBeingAsyncCompiled;
};

/**
 * Worker used to perform async static mesh compilation.
 */
class FStaticMeshAsyncBuildWorker : public FNonAbandonableTask
{
public:
	UStaticMesh* StaticMesh;
	TUniquePtr<FStaticMeshPostLoadContext> PostLoadContext;
	TUniquePtr<FStaticMeshBuildContext> BuildContext;

	/** Initialization constructor. */
	FStaticMeshAsyncBuildWorker(
		UStaticMesh* InStaticMesh,
		TUniquePtr<FStaticMeshBuildContext>&& InBuildContext)
		: StaticMesh(InStaticMesh)
		, PostLoadContext(nullptr)
		, BuildContext(MoveTemp(InBuildContext))
	{
	}

	/** Initialization constructor. */
	FStaticMeshAsyncBuildWorker(
		UStaticMesh* InStaticMesh,
		TUniquePtr<FStaticMeshPostLoadContext>&& InPostLoadContext)
		: StaticMesh(InStaticMesh)
		, PostLoadContext(MoveTemp(InPostLoadContext))
		, BuildContext(nullptr)
	{
	}

	void MarkAsCanceled();

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FStaticMeshAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
};

struct FStaticMeshAsyncBuildTask : public FAsyncTask<FStaticMeshAsyncBuildWorker>
{
	FStaticMeshAsyncBuildTask(
		UStaticMesh* InStaticMesh,
		TUniquePtr<FStaticMeshPostLoadContext>&& InPostLoadContext)
		: FAsyncTask<FStaticMeshAsyncBuildWorker>(InStaticMesh, MoveTemp(InPostLoadContext))
		, StaticMesh(InStaticMesh)
	{
	}

	FStaticMeshAsyncBuildTask(
		UStaticMesh* InStaticMesh,
		TUniquePtr<FStaticMeshBuildContext>&& InBuildContext)
		: FAsyncTask<FStaticMeshAsyncBuildWorker>(InStaticMesh, MoveTemp(InBuildContext))
		, StaticMesh(InStaticMesh)
	{
	}

	const UStaticMesh* StaticMesh;
};
#endif // #if WITH_EDITOR

/*-----------------------------------------------------------------------------
	Legacy mesh optimization settings.
-----------------------------------------------------------------------------*/

/** Optimization settings used to simplify mesh LODs. */
UENUM()
enum ENormalMode : int
{
	NM_PreserveSmoothingGroups,
	NM_RecalculateNormals,
	NM_RecalculateNormalsSmooth,
	NM_RecalculateNormalsHard,
	TEMP_BROKEN,
	ENormalMode_MAX,
};

UENUM()
enum EImportanceLevel : int
{
	IL_Off,
	IL_Lowest,
	IL_Low,
	IL_Normal,
	IL_High,
	IL_Highest,
	TEMP_BROKEN2,
	EImportanceLevel_MAX,
};

/** Enum specifying the reduction type to use when simplifying static meshes. */
UENUM()
enum EOptimizationType : int
{
	OT_NumOfTriangles,
	OT_MaxDeviation,
	OT_MAX,
};

/** Old optimization settings. */
USTRUCT()
struct FStaticMeshOptimizationSettings
{
	GENERATED_USTRUCT_BODY()

	/** The method to use when optimizing the skeletal mesh LOD */
	UPROPERTY()
	TEnumAsByte<enum EOptimizationType> ReductionMethod;

	/** If ReductionMethod equals SMOT_NumOfTriangles this value is the ratio of triangles [0-1] to remove from the mesh */
	UPROPERTY()
	float NumOfTrianglesPercentage;

	/**If ReductionMethod equals SMOT_MaxDeviation this value is the maximum deviation from the base mesh as a percentage of the bounding sphere. */
	UPROPERTY()
	float MaxDeviationPercentage;

	/** The welding threshold distance. Vertices under this distance will be welded. */
	UPROPERTY()
	float WeldingThreshold;

	/** Whether Normal smoothing groups should be preserved. If false then NormalsThreshold is used **/
	UPROPERTY()
	bool bRecalcNormals;

	/** If the angle between two triangles are above this value, the normals will not be
	smooth over the edge between those two triangles. Set in degrees. This is only used when PreserveNormals is set to false*/
	UPROPERTY()
	float NormalsThreshold;

	/** How important the shape of the geometry is (EImportanceLevel). */
	UPROPERTY()
	uint8 SilhouetteImportance;

	/** How important texture density is (EImportanceLevel). */
	UPROPERTY()
	uint8 TextureImportance;

	/** How important shading quality is. */
	UPROPERTY()
	uint8 ShadingImportance;


	FStaticMeshOptimizationSettings()
	: ReductionMethod( OT_MaxDeviation )
	, NumOfTrianglesPercentage( 1.0f )
	, MaxDeviationPercentage( 0.0f )
	, WeldingThreshold( 0.1f )
	, bRecalcNormals( true )
	, NormalsThreshold( 60.0f )
	, SilhouetteImportance( IL_Normal )
	, TextureImportance( IL_Normal )
	, ShadingImportance( IL_Normal )
	{
	}

	/** Serialization for FStaticMeshOptimizationSettings. */
	inline friend FArchive& operator<<( FArchive& Ar, FStaticMeshOptimizationSettings& Settings )
	{
		Ar << Settings.ReductionMethod;
		Ar << Settings.MaxDeviationPercentage;
		Ar << Settings.NumOfTrianglesPercentage;
		Ar << Settings.SilhouetteImportance;
		Ar << Settings.TextureImportance;
		Ar << Settings.ShadingImportance;
		Ar << Settings.bRecalcNormals;
		Ar << Settings.NormalsThreshold;
		Ar << Settings.WeldingThreshold;

		return Ar;
	}

};

/*-----------------------------------------------------------------------------
	UStaticMesh
-----------------------------------------------------------------------------*/

/**
 * Per-section settings.
 */
USTRUCT()
struct FMeshSectionInfo
{
	GENERATED_USTRUCT_BODY()

	/** Index in to the Materials array on UStaticMesh. */
	UPROPERTY()
	int32 MaterialIndex;

	/** If true, collision is enabled for this section. */
	UPROPERTY()
	bool bEnableCollision;

	/** If true, this section will cast shadows. */
	UPROPERTY()
	bool bCastShadow;

	/** If true, this section will be visible in ray tracing Geometry. */
	UPROPERTY()
	bool bVisibleInRayTracing;

	/** If true, this section will affect lighting methods that use Distance Fields. */
	UPROPERTY()
	bool bAffectDistanceFieldLighting;

	/** If true, this section will always considered opaque in ray tracing Geometry. */
	UPROPERTY()
	bool bForceOpaque;

	/** Default values. */
	FMeshSectionInfo()
		: MaterialIndex(0)
		, bEnableCollision(true)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, bAffectDistanceFieldLighting(true)
		, bForceOpaque(false)
	{
	}

	/** Default values with an explicit material index. */
	explicit FMeshSectionInfo(int32 InMaterialIndex)
		: MaterialIndex(InMaterialIndex)
		, bEnableCollision(true)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, bAffectDistanceFieldLighting(true)
		, bForceOpaque(false)
	{
	}

	/** Comparison for mesh section info. */
	friend bool operator==(const FMeshSectionInfo& A, const FMeshSectionInfo& B);
	friend bool operator!=(const FMeshSectionInfo& A, const FMeshSectionInfo& B);
};

/**
 * Map containing per-section settings for each section of each LOD.
 */
USTRUCT()
struct FMeshSectionInfoMap
{
	GENERATED_USTRUCT_BODY()

	/** Maps an LOD+Section to the material it should render with. */
	UPROPERTY()
	TMap<uint32,FMeshSectionInfo> Map;

	/** Serialize. */
	void Serialize(FArchive& Ar);

	/** Clears all entries in the map resetting everything to default. */
	ENGINE_API void Clear();

	/** Get the number of section for a LOD. */
	ENGINE_API int32 GetSectionNumber(int32 LODIndex) const;

	/** Return true if the section exist, false otherwise. */
	ENGINE_API bool IsValidSection(int32 LODIndex, int32 SectionIndex) const;

	/** Gets per-section settings for the specified LOD + section. */
	ENGINE_API FMeshSectionInfo Get(int32 LODIndex, int32 SectionIndex) const;

	/** Sets per-section settings for the specified LOD + section. */
	ENGINE_API void Set(int32 LODIndex, int32 SectionIndex, FMeshSectionInfo Info);

	/** Resets per-section settings for the specified LOD + section to defaults. */
	ENGINE_API void Remove(int32 LODIndex, int32 SectionIndex);

	/** Copies per-section settings from the specified section info map. */
	ENGINE_API void CopyFrom(const FMeshSectionInfoMap& Other);

	/** Returns true if any section of the specified LOD has collision enabled. */
	bool AnySectionHasCollision(int32 LodIndex) const;
};

USTRUCT()
struct FAssetEditorOrbitCameraPosition
{
	GENERATED_USTRUCT_BODY()

	FAssetEditorOrbitCameraPosition()
		: bIsSet(false)
		, CamOrbitPoint(ForceInitToZero)
		, CamOrbitZoom(ForceInitToZero)
		, CamOrbitRotation(ForceInitToZero)
	{
	}

	FAssetEditorOrbitCameraPosition(const FVector& InCamOrbitPoint, const FVector& InCamOrbitZoom, const FRotator& InCamOrbitRotation)
		: bIsSet(true)
		, CamOrbitPoint(InCamOrbitPoint)
		, CamOrbitZoom(InCamOrbitZoom)
		, CamOrbitRotation(InCamOrbitRotation)
	{
	}

	/** Whether or not this has been set to a valid value */
	UPROPERTY()
	bool bIsSet;

	/** The position to orbit the camera around */
	UPROPERTY()
	FVector	CamOrbitPoint;

	/** The distance of the camera from the orbit point */
	UPROPERTY()
	FVector CamOrbitZoom;

	/** The rotation to apply around the orbit point */
	UPROPERTY()
	FRotator CamOrbitRotation;
};

#if WITH_EDITOR
/** delegate type for pre mesh build events */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreMeshBuild, class UStaticMesh*);
/** delegate type for post mesh build events */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostMeshBuild, class UStaticMesh*);
#endif

//~ Begin Material Interface for UStaticMesh - contains a material and other stuff
USTRUCT(BlueprintType)
struct FStaticMaterial
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FStaticMaterial();

	ENGINE_API FStaticMaterial(class UMaterialInterface* InMaterialInterface
		, FName InMaterialSlotName = NAME_None
#if WITH_EDITORONLY_DATA
		, FName InImportedMaterialSlotName = NAME_None
#endif
		, class UMaterialInterface* InOverlayMaterialInterface = nullptr
		);

	friend FArchive& operator<<(FArchive& Ar, FStaticMaterial& Elem);

	ENGINE_API friend bool operator==(const FStaticMaterial& LHS, const FStaticMaterial& RHS);
	ENGINE_API friend bool operator==(const FStaticMaterial& LHS, const UMaterialInterface& RHS);
	ENGINE_API friend bool operator==(const UMaterialInterface& LHS, const FStaticMaterial& RHS);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	TObjectPtr<class UMaterialInterface> MaterialInterface;

	/*This name should be use by the gameplay to avoid error if the skeletal mesh Materials array topology change*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	FName MaterialSlotName;

	/*This name should be use when we re-import a skeletal mesh so we can order the Materials array like it should be*/
	UPROPERTY(VisibleAnywhere, Category = StaticMesh)
	FName ImportedMaterialSlotName;

	/** Data used for texture streaming relative to each UV channels. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = StaticMesh)
	FMeshUVChannelInfo			UVChannelData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	TObjectPtr<class UMaterialInterface> OverlayMaterialInterface;
};

USTRUCT()
struct FStaticMaterialMinimalInfo
{
	GENERATED_USTRUCT_BODY()

	FStaticMaterialMinimalInfo(FName InMaterialSlotName = NAME_None, FString InMaterialObjectPath = FString())
		: MaterialSlotName(InMaterialSlotName), MaterialObjectPath(InMaterialObjectPath)
		{}

	FName MaterialSlotName;
	FString MaterialObjectPath;
};

enum EImportStaticMeshVersion
{
	// Before any version changes were made
	BeforeImportStaticMeshVersionWasAdded,
	// Remove the material re-order workflow
	RemoveStaticMeshSkinxxWorkflow,
	StaticMeshVersionPlusOne,
	LastVersion = StaticMeshVersionPlusOne - 1
};

USTRUCT()
struct FMaterialRemapIndex
{
	GENERATED_USTRUCT_BODY()

	FMaterialRemapIndex()
	{
		ImportVersionKey = 0;
	}

	FMaterialRemapIndex(uint32 VersionKey, TArray<int32> RemapArray)
	: ImportVersionKey(VersionKey)
	, MaterialRemap(RemapArray)
	{
	}

	UPROPERTY()
	uint32 ImportVersionKey;

	UPROPERTY()
	TArray<int32> MaterialRemap;
};

UENUM(BlueprintType)
enum class EStaticMeshPaintSupport : uint8
{
	/** Use the default project setting for whether texture color mesh painting is supported. */
	Default,
	/** Enable texture color mesh painting support. */
	Enabled,
	/** Disable texture color mesh painting support. */
	Disabled,
};

UENUM()
enum class EStaticMeshLODStreaming : uint8
{
	/** Use the default behavior (LOD Group controls LOD streaming). */
	Default,
	/** Enable LOD streaming. */
	Enabled,
	/** Disable LOD streaming. */
	Disabled,
};

/**
 * A StaticMesh is a piece of geometry that consists of a static set of polygons.
 * Static Meshes can be translated, rotated, and scaled, but they cannot have their vertices animated in any way. As such, they are more efficient
 * to render than other types of geometry such as USkeletalMesh, and they are often the basic building block of levels created in the engine.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Content/Types/StaticMeshes/
 * @see AStaticMeshActor, UStaticMeshComponent
 */
UCLASS(hidecategories=Object, customconstructor, MinimalAPI, BlueprintType, config=Engine)
class UStaticMesh : public UStreamableRenderAsset, public IInterface_CollisionDataProvider, public IInterface_AssetUserData, public IInterface_AsyncCompilation
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	/** Notification when bounds changed */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExtendedBoundsChanged, const FBoxSphereBounds&);

	/** Notification when anything changed */
	DECLARE_MULTICAST_DELEGATE(FOnMeshChanged);
#endif
public:
	ENGINE_API ~UStaticMesh();

private:

#if WITH_EDITOR
	/** Used as a bit-field indicating which properties are currently accessed/modified by async compilation. */
	std::atomic<uint32> LockedProperties;

	void AcquireAsyncProperty(EStaticMeshAsyncProperties AsyncProperties = EStaticMeshAsyncProperties::All);
	void ReleaseAsyncProperty(EStaticMeshAsyncProperties AsyncProperties = EStaticMeshAsyncProperties::All);
	ENGINE_API void WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties AsyncProperties) const;
#else
	inline void AcquireAsyncProperty(EStaticMeshAsyncProperties AsyncProperties = EStaticMeshAsyncProperties::All) {};
	inline void ReleaseAsyncProperty(EStaticMeshAsyncProperties AsyncProperties = EStaticMeshAsyncProperties::All) {};
	inline void WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties AsyncProperties) const {}
#endif

	/** Pointer to the data used to render this static mesh. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	TUniquePtr<class FStaticMeshRenderData> RenderData;

public:
#if WITH_EDITOR
	bool IsCompiling() const override { return AsyncTask != nullptr || LockedProperties.load(std::memory_order_relaxed) != 0; }
	ENGINE_API bool HasAnyDependenciesCompiling() const;
#else
	inline bool IsCompiling() const { return false; }
	bool HasAnyDependenciesCompiling() const { return false; }
#endif
	
	ENGINE_API FStaticMeshRenderData* GetRenderData();
	ENGINE_API const FStaticMeshRenderData* GetRenderData() const;
	ENGINE_API void SetRenderData(TUniquePtr<class FStaticMeshRenderData>&& InRenderData);

	void RequestUpdateCachedRenderState() const;

#if WITH_EDITORONLY_DATA
	static const float MinimumAutoLODPixelError;

private:
	/** Imported raw mesh bulk data. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(Setter = None, Getter = None)
	TArray<FStaticMeshSourceModel> SourceModels;

	/** Optional hi-res source data */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(Setter = None, Getter = None)
	FStaticMeshSourceModel HiResSourceModel;

	void SetLightmapUVVersion(int32 InLightmapUVVersion)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightmapUVVersion);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LightmapUVVersion = InLightmapUVVersion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Map of LOD+Section index to per-section info. */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY()
	FMeshSectionInfoMap SectionInfoMap;

	/**
	 * We need the OriginalSectionInfoMap to be able to build mesh in a non destructive way. Reduce has to play with SectionInfoMap in case some sections disappear.
	 * This member will be update in the following situation
	 * 1. After a static mesh import/reimport
	 * 2. Postload, if the OriginalSectionInfoMap is empty, we will fill it with the current SectionInfoMap
	 *
	 * We do not update it when the user shuffle section in the staticmesh editor because the OriginalSectionInfoMap must always be in sync with the saved rawMesh bulk data.
	 */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY()
	FMeshSectionInfoMap OriginalSectionInfoMap;

public:
	static FName GetSectionInfoMapName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, SectionInfoMap);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** The LOD group to which this mesh belongs. */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::Get/SetLODGroup() or, if absolutely needed, UStaticMesh::SetLODGroup_Unsafe() which bypasses the conditional processing in SetLODGroup(). The Unsafe version is likely to be removed in the near future.")
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category=LodSettings)
	FName LODGroup;

	/**
	 * If non-negative, specify the maximum number of streamed LODs. Only has effect if
	 * mesh LOD streaming is enabled for the target platform.
	 */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetNumStreamedLODs().")
	UPROPERTY()
	FPerPlatformInt NumStreamedLODs;

	/* The last import version */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetImportVersion() or UStaticMesh::SetImportVersion().")
	UPROPERTY()
	int32 ImportVersion;

	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetMaterialRemapIndexPerImportVersion()")
	UPROPERTY()
	TArray<FMaterialRemapIndex> MaterialRemapIndexPerImportVersion;

	/** If true, the screen sizes at which LODs swap are computed automatically. */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetAutoComputeLODScreenSize() or UStaticMesh::SetAutoComputeLODScreenSize().")
	UPROPERTY()
	uint8 bAutoComputeLODScreenSize : 1;

	/**
	* If true on post load we need to calculate Display Factors from the
	* loaded LOD distances.
	*/
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetRequiresLODDistanceConversion() or UStaticMesh::SetRequiresLODDistanceConversion().")
	uint8 bRequiresLODDistanceConversion : 1;

	/**
	 * If true on post load we need to calculate resolution independent Display Factors from the
	 * loaded LOD screen sizes.
	 */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetRequiresLODScreenSizeConversion() or UStaticMesh::SetRequiresLODScreenSizeConversion().")
	uint8 bRequiresLODScreenSizeConversion : 1;

	/** Materials used by this static mesh. Individual sections index in to this array. */
	UE_DEPRECATED(5.7, "Do not access this member. It is not meant to be used outside of UStaticMesh.")
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> Materials_DEPRECATED;

	/** Settings related to building Nanite data. */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use the various NaniteSettings accessor functions.")
	UPROPERTY(EditAnywhere, Category = NaniteSettings)
	FMeshNaniteSettings NaniteSettings;


private:
	/* The lightmap UV generation version used during the last derived data build */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
		UPROPERTY()
	int32 LightmapUVVersion;

	UPROPERTY(EditAnywhere, Category = LodSettings)
	EStaticMeshLODStreaming LODStreaming;

public:

	int32 GetLightmapUVVersion() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightmapUVVersion);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return LightmapUVVersion;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FName GetLODGroup() const
	{
		return LODGroup;
	}

	void SetLODGroup_Unsafe(FName InLODGroup)
	{
		LODGroup = InLODGroup;
	}

	void SetLODStreaming(EStaticMeshLODStreaming InLODStreaming)
	{
		LODStreaming = InLODStreaming;
	}

	static FName GetLODGroupMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, LODGroup);
	}

	const FPerPlatformInt& GetNumStreamedLODs() const
	{
		return NumStreamedLODs;
	}

	FPerPlatformInt& GetNumStreamedLODs()
	{
		return NumStreamedLODs;
	}

	int32 GetImportVersion()
	{
		return ImportVersion;
	}

	void SetImportVersion(int32 Version)
	{
		ImportVersion = Version;
	}
		
	const TArray<FMaterialRemapIndex>& GetMaterialRemapIndexPerImportVersion()
	{
		return MaterialRemapIndexPerImportVersion;
	}

	void AddMaterialRemapIndex(const FMaterialRemapIndex& Index)
	{
		MaterialRemapIndexPerImportVersion.Add(Index);
	}

	bool GetAutoComputeLODScreenSize() const
	{
		return bAutoComputeLODScreenSize;
	}

	void SetAutoComputeLODScreenSize(bool bShouldCompute)
	{
		bAutoComputeLODScreenSize = bShouldCompute;
	}

	static FName GetAutoComputeLODScreenSizeMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, bAutoComputeLODScreenSize);
	}

	bool GetRequiresLODDistanceConversion() const
	{
		return bRequiresLODDistanceConversion;
	}

	void SetRequiresLODDistanceConversion(bool bRequiresConversion)
	{
		bRequiresLODDistanceConversion = bRequiresConversion;
	}

	bool GetRequiresLODScreenSizeConversion() const
	{
		return bRequiresLODScreenSizeConversion;
	}

	void SetRequiresLODScreenSizeConversion(bool bRequiresConversion)
	{
		bRequiresLODScreenSizeConversion = bRequiresConversion;
	}

	const FMeshNaniteSettings& GetNaniteSettings() const
	{
		return NaniteSettings;
	}
	FMeshNaniteSettings& GetNaniteSettings()
	{
		return NaniteSettings;
	}

	void SetNaniteSettings(const FMeshNaniteSettings& Settings)
	{
		NaniteSettings = Settings;
	}

	static FName GetNaniteSettingsMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, NaniteSettings);
	}

	void NotifyNaniteSettingsChanged()
	{
		FProperty* ChangedProperty = FindFProperty<FProperty>(UStaticMesh::StaticClass(), GET_MEMBER_NAME_CHECKED(UStaticMesh, NaniteSettings));
		FPropertyChangedEvent Event(ChangedProperty);
		PostEditChangeProperty(Event);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // #if WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = StaticMesh)
	bool IsLODScreenSizeAutoComputed() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bAutoComputeLODScreenSize;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	/** Check the QualitLevel property is enabled for MinLod. */
	bool IsMinLodQualityLevelEnable() const;

	UPROPERTY()
	/*PerQuality override. Note: Enable PerQuality override in the Project Settings/ General Settings/ UseStaticMeshMinLODPerQualityLevels*/
	/* Allow more flexibility to set various values driven by the Scalability or Device Profile.*/
	FPerQualityLevelInt MinQualityLevelLOD;

	static FName GetQualityLevelMinLODMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, MinQualityLevelLOD);
	}

	const FPerQualityLevelInt& GetQualityLevelMinLOD() const
	{
		return MinQualityLevelLOD;
	}

	void SetQualityLevelMinLOD(FPerQualityLevelInt InMinLOD)
	{
		MinQualityLevelLOD = MoveTemp(InMinLOD);
	}

	UFUNCTION(BlueprintPure, Category = StaticMesh)
	void GetMinimumLODForQualityLevels(TMap<FName, int32>& QualityLevelMinimumLODs) const
	{
#if WITH_EDITORONLY_DATA
		for (const TPair<int32, int32>& Pair : GetQualityLevelMinLOD().PerQuality)
		{
			QualityLevelMinimumLODs.Add(QualityLevelProperty::QualityLevelToFName(Pair.Key), Pair.Value);
		}
#endif
	}

	UFUNCTION(BlueprintPure, Category = StaticMesh)
	int32 GetMinimumLODForQualityLevel(const FName& QualityLevel) const
	{
#if WITH_EDITORONLY_DATA
		int32 QualityLevelKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
		if (const int32* Result = GetQualityLevelMinLOD().PerQuality.Find(QualityLevelKey))
		{
			return *Result;
		}
#endif
		return INDEX_NONE;
	}

	UFUNCTION(BlueprintCallable, Category = StaticMesh, Meta = (ToolTip = "Allow to override min lod quality levels on a staticMesh and it Default value (-1 value for Default dont override its value)."))
	void SetMinLODForQualityLevels(const TMap<EPerQualityLevels, int32>& QualityLevelMinimumLODs, int32 Default = -1)
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		MinQualityLevelLOD.PerQuality = QualityLevelProperty::ConvertQualityLevelData(QualityLevelMinimumLODs);
		MinQualityLevelLOD.Default = Default >= 0 ? Default : MinQualityLevelLOD.Default;
#endif
	}

	UFUNCTION(BlueprintPure, Category = StaticMesh)
	void GetMinLODForQualityLevels(TMap<EPerQualityLevels, int32>& QualityLevelMinimumLODs, int32& Default) const
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		QualityLevelMinimumLODs = QualityLevelProperty::ConvertQualityLevelData(MinQualityLevelLOD.PerQuality);
		Default = MinQualityLevelLOD.Default;
#endif
	}

	/*Choose either PerPlatform or PerQuality override. Note: Enable PerQuality override in the Project Settings/ General Settings/ UseStaticMeshMinLODPerQualityLevels*/
	ENGINE_API int32 GetMinLODIdx(bool bForceLowestLODIdx = false) const;
	ENGINE_API int32 GetDefaultMinLOD() const;
	ENGINE_API void SetMinLODIdx(int32 InMinLOD);
	
	/** Computes the LOD Screen Size based on the LODIndex while keeping it within tolerance from the previous LODScreenSize*/
	ENGINE_API static float ComputeLODScreenSize(int32 LODIndex, float PreviousLODScreenSize = -1.0f);
	
	ENGINE_API static void OnLodStrippingQualityLevelChanged(IConsoleVariable* Variable);

	/** Minimum LOD to use for rendering.  This is the default setting for the mesh and can be overridden by component settings. */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use UStaticMesh::GetMinLOD() or UStaticMesh::SetMinLOD().")
	UPROPERTY()
	FPerPlatformInt MinLOD;

	const FPerPlatformInt& GetMinLOD() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MinLOD;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetMinLOD(FPerPlatformInt InMinLOD)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MinLOD = MoveTemp(InMinLOD);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintPure, Category=StaticMesh)
	void GetMinimumLODForPlatforms(TMap<FName, int32>& PlatformMinimumLODs) const
	{
#if WITH_EDITORONLY_DATA
		PlatformMinimumLODs = GetMinLOD().PerPlatform;
#endif
	}

	UFUNCTION(BlueprintPure, Category=StaticMesh)
	int32 GetMinimumLODForPlatform(const FName& PlatformName) const
	{
#if WITH_EDITORONLY_DATA
		if (const int32* Result = GetMinLOD().PerPlatform.Find(PlatformName))
		{
			return *Result;
		}
#endif
		return INDEX_NONE;
	}

	UFUNCTION(BlueprintCallable, Category=StaticMesh)
	void SetMinimumLODForPlatforms(const TMap<FName, int32>& PlatformMinimumLODs)
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MinLOD.PerPlatform = PlatformMinimumLODs;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	UFUNCTION(BlueprintCallable, Category=StaticMesh)
	void SetMinimumLODForPlatform(const FName& PlatformName, int32 InMinLOD)
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::MinLOD);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MinLOD.PerPlatform.Add(PlatformName, InMinLOD);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

protected:
	/**
	 * Index of an element to ignore while gathering streaming texture factors.
	 * This is useful to disregard automatically generated vertex data which breaks texture factor heuristics.
	 */
	UPROPERTY(meta = (DisplayAfter = "ExtendedBounds"))
	int32 ElementToIgnoreForTexFactor;

public:

#if WITH_EDITORONLY_DATA
	/**
	 * Returns true if this SM should have Nanite built for it.
	 * This also includes the result of IsNaniteForceEnabled().
	 */
	ENGINE_API bool IsNaniteEnabled() const;

	/**
	 * Returns true if this SM should always have Nanite data built.
	 * This forces the SM to be Nanite even if the flag in the editor is set to false.
	 */
	ENGINE_API bool IsNaniteForceEnabled() const;

	/**
	 * Returns true if this SM is comprised of instanced copies of other SMs for at least
	 * part of its own internal structure.
	 **/
	ENGINE_API bool IsNaniteAssembly() const;

	/**
	 * Returns whether or not Nanite Assembly reference meshes are adequately cached for build (always true for non-assemblies).
	 */
	ENGINE_API bool HasCachedNaniteAssemblyReferences() const;

	/**
	 * Retrieves the cached array of static meshes referenced by the Nanite Assembly data.
	 * NOTE: This array is only meant to contain anything during static mesh build.
	 */
	ENGINE_API const TArray<TObjectPtr<UStaticMesh>>& GetCachedNaniteAssemblyReferences() const;

	/**
	 * Caches the static meshes referenced by the Nanite Assembly data for the mesh to be used for build. Must
	 * be called on the game thread before any asynchronous building. You should only need to call this externally
	 * if caching render data manually (as opposed to using CacheDerivedData on the game thread or async mesh build).
	 */
	ENGINE_API void RecacheNaniteAssemblyReferences(bool bWaitForAsyncCompile = true);

	/**
	 * Removes references previously cached using RecacheNaniteAssemblyReferences(). Must be called from the game thread.
	 */
	ENGINE_API void ClearCachedNaniteAssemblyReferences();
#endif

	// TODO: Temp/deprecated hack - Do not call
	inline bool IsNaniteLandscape() const
	{
		return GetName().StartsWith(TEXT("LandscapeNaniteMesh"));
	}

private:
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(BlueprintGetter = GetStaticMaterials, BlueprintSetter = SetStaticMaterials, Category = StaticMesh)
	TArray<FStaticMaterial> StaticMaterials;

	TArray<FStaticMaterialMinimalInfo> StaticMaterialsInfoCache;
	mutable FCriticalSection MaterialsInfoCriticalSection;
	ENGINE_API void UpdateStaticMaterialsInfoCache(const TArray<FStaticMaterial>& InMaterials);
	ENGINE_API const FString ExportStaticMaterialsInfo() const;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TArray<TObjectPtr<UStaticMesh>> AssemblyReferenceCache;
#endif
#if WITH_EDITOR
	void OnNaniteAssemblyReferenceBuilt(UStaticMesh* Reference);
#endif

public:
	static FName GetStaticMaterialsName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, StaticMaterials);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const TArray<FStaticMaterialMinimalInfo> GetCopyOfStaticMaterialsInfo() const
	{
		FScopeLock Lock(&MaterialsInfoCriticalSection);
		return StaticMaterialsInfoCache;
	}

	ENGINE_API static const TArray<FStaticMaterialMinimalInfo> ImportStaticMaterialsInfo(const FString& InData);

	TArray<FStaticMaterial>& GetStaticMaterials()
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::StaticMaterials);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return StaticMaterials;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintGetter)
	const TArray<FStaticMaterial>& GetStaticMaterials() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::StaticMaterials);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return StaticMaterials;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintSetter)
	void SetStaticMaterials(const TArray<FStaticMaterial>& InStaticMaterials)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::StaticMaterials);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		StaticMaterials = InStaticMaterials;
		UpdateStaticMaterialsInfoCache(StaticMaterials);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY()
	float LightmapUVDensity;
public:
	void SetLightmapUVDensity(float InLightmapUVDensity)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightmapUVDensity);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LightmapUVDensity = InLightmapUVDensity;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	float GetLightmapUVDensity() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightmapUVDensity);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LightmapUVDensity;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(4.27, "Please do not access this member directly; use UStaticMesh::GetLightMapResolution() or UStaticMesh::SetLightMapResolution().")
	UPROPERTY(EditAnywhere, Category=StaticMesh, meta=(ClampMax = 4096, ToolTip="The light map resolution", FixedIncrement="4.0"))
	int32 LightMapResolution;

	int32 GetLightMapResolution() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightMapResolution);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LightMapResolution;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetLightMapResolution(int32 InLightMapResolution)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightMapResolution);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LightMapResolution = InLightMapResolution;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetLightMapResolutionName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, LightMapResolution);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** The light map coordinate index */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use UStaticMesh::GetLightMapCoordinateIndex() or UStaticMesh::SetLightMapCoordinateIndex().")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=StaticMesh, meta=(ToolTip="The light map coordinate index", UIMin = "0", UIMax = "3"))
	int32 LightMapCoordinateIndex;

	int32 GetLightMapCoordinateIndex() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightMapCoordinateIndex);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LightMapCoordinateIndex;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetLightMapCoordinateIndex(int32 InLightMapCoordinateIndex)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightMapCoordinateIndex);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LightMapCoordinateIndex = InLightMapCoordinateIndex;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetLightMapCoordinateIndexName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, LightMapCoordinateIndex);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Whether to support per instance texture color mesh painting on components using this mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh, meta = (DisplayName = "Support Texture Color Mesh Painting"))
	EStaticMeshPaintSupport StaticMeshPaintSupport;

	/** Whether to support per instance texture color mesh painting on components using this mesh. */
	ENGINE_API bool CanMeshPaintTextureColors() const;

	/** The default coordinate index to use when texture color painting on this mesh. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh, meta = (UIMin = "0", UIMax = "3"))
	int32 MeshPaintTextureCoordinateIndex;

	/** 
	 * The resolution of texture color mesh paint textures on this mesh.
	 * The final size will be rounded up to a power of 2 and a multiple of the "Mesh Paint Tile Size" project setting.
	 * A default value of 0 will auto calculate the size using the "Mesh paint texels per vertex" project setting.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh, meta = (UIMin = "0", UIMax = "4096", ClampMax = "4096"))
	int32 MeshPaintTextureResolution;

	/** Useful for reducing self shadowing from distance field methods when using world position offset to animate the mesh's vertices. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	float DistanceFieldSelfShadowBias;

private:
	// Physics data.
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, transient, duplicatetransient, Instanced, Category = StaticMesh)
	TObjectPtr<class UBodySetup> BodySetup;
public:
	UBodySetup* GetBodySetup() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::BodySetup);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BodySetup;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetBodySetup(UBodySetup* InBodySetup)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::BodySetup);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BodySetup = InBodySetup;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetBodySetupName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, BodySetup);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** 
	 *	Specifies which mesh LOD to use for complex (per-poly) collision. 
	 *	Sometimes it can be desirable to use a lower poly representation for collision to reduce memory usage, improve performance and behaviour.
	 *	Collision representation does not change based on distance to camera.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh, meta=(DisplayName="LOD For Collision"))
	int32 LODForCollision;

	/** 
	 * Whether to generate a distance field for this mesh, which can be used by DistanceField Indirect Shadows.
	 * This is ignored if the project's 'Generate Mesh Distance Fields' setting is enabled.
	 */
	UPROPERTY(EditAnywhere, Category=StaticMesh)
	uint8 bGenerateMeshDistanceField : 1;

	/** If true, strips unwanted complex collision data aka kDOP tree when cooking for consoles.
		On the Playstation 3 data of this mesh will be stored in video memory. */
	UPROPERTY()
	uint8 bStripComplexCollisionForConsole_DEPRECATED:1;

	/** If true, mesh will have NavCollision property with additional data for navmesh generation and usage.
	    Set to false for distant meshes (always outside navigation bounds) to save memory on collision data. */
	UPROPERTY(EditAnywhere, Category=Navigation)
	uint8 bHasNavigationData:1;

	/**	
		Mesh supports uniformly distributed sampling in constant time.
		Memory cost is 8 bytes per triangle.
		Example usage is uniform spawning of particles.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	uint8 bSupportUniformlyDistributedSampling : 1;

	/** 
		If true, complex collision data will store UVs and face remap table for use when performing
	    PhysicalMaterialMask lookups in cooked builds. Note the increased memory cost for this
		functionality.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	uint8 bSupportPhysicalMaterialMasks : 1;

#if WITH_EDITORONLY_DATA
private:
	// If true, will incorrectly scale tangents when applying a non-uniform BuildScale to match what legacy code did. Only use for consistency on old assets.
	UE_DEPRECATED(5.4, "Please do not access this member directly; use UStaticMesh::GetLegacyTangentScaling() or UStaticMesh::SetLegacyTangentScaling().")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	uint8 bUseLegacyTangentScaling : 1;
public:

	bool GetLegacyTangentScaling() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::UseLegacyTangentScaling);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bUseLegacyTangentScaling;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetLegacyTangentScaling(bool bInUseLegacyTangentScaling)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::UseLegacyTangentScaling);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bUseLegacyTangentScaling = bInUseLegacyTangentScaling;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	/** Settings related to building Ray Tracing Proxy data. */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use the various RayTracingProxySettings accessors.")
	UPROPERTY(EditAnywhere, Category = RayTracingProxySettings)
	FMeshRayTracingProxySettings RayTracingProxySettings;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FMeshRayTracingProxySettings& GetRayTracingProxySettings() const
	{
		return RayTracingProxySettings;
	}
	FMeshRayTracingProxySettings& GetRayTracingProxySettings()
	{
		return RayTracingProxySettings;
	}
	void SetRayTracingProxySettings(const FMeshRayTracingProxySettings& Settings)
	{
		RayTracingProxySettings = Settings;
	}

	static FName GetRayTracingProxySettingsMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, RayTracingProxySettings);
	}

	void NotifyRayTracingProxySettingsChanged()
	{
		FProperty* ChangedProperty = FindFProperty<FProperty>(UStaticMesh::StaticClass(), UStaticMesh::GetRayTracingProxySettingsMemberName());
		FPropertyChangedEvent Event(ChangedProperty);
		PostEditChangeProperty(Event);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS


#endif // WITH_EDITORONLY_DATA

	/**
	 * If true, a ray tracing acceleration structure will be built for this mesh and it may be used in ray tracing effects
	 */
	UPROPERTY(EditAnywhere, Category = RayTracing)
	uint8 bSupportRayTracing : 1;

	static FName GetSupportRayTracingMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, bSupportRayTracing);
	}

	UPROPERTY()
	uint8 bDoFastBuild : 1;

private:

	UPROPERTY()
	uint8 bIsBuiltAtRuntime_DEPRECATED : 1;

public:

	UE_DEPRECATED(5.0, "IsBuiltAtRuntime() is no longer used.")
	bool IsBuiltAtRuntime() const
	{
		return false;
	}

	UE_DEPRECATED(5.0, "SetIsBuiltAtRuntime() is no longer used.")
	void SetIsBuiltAtRuntime(bool InIsBuiltAtRuntime)
	{
	}
protected:
	/** Tracks whether InitResources has been called, and rendering resources are initialized. */
	uint8 bRenderingResourcesInitialized:1;

public:
	/** 
	 *	If true, will keep geometry data CPU-accessible in cooked builds, rather than uploading to GPU memory and releasing it from CPU memory.
	 *	This is required if you wish to access StaticMesh geometry data on the CPU at runtime in cooked builds (e.g. to convert StaticMesh to ProceduralMeshComponent)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	uint8 bAllowCPUAccess:1;

	/**
	 * If true, a GPU buffer containing required data for uniform mesh surface sampling will be created at load time.
	 * It is created from the cpu data so bSupportUniformlyDistributedSampling is also required to be true.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = StaticMesh)
	uint8 bSupportGpuUniformlyDistributedSampling : 1;

	/** A fence which is used to keep track of the rendering thread releasing the static mesh resources. */
	FRenderCommandFence ReleaseResourcesFence;

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this mesh */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetAssetImportData() or UStaticMesh::SetAssetImportData().")
	UPROPERTY(EditAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** Path to the resource used to construct this static mesh */
	UE_DEPRECATED(5.7, "Do not access this member. It is not meant to be used outside of UStaticMesh.")
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	/** Date/Time-stamp of the file from the last import */
	UE_DEPRECATED(5.7, "Do not access this member. It is not meant to be used outside of UStaticMesh.")
	UPROPERTY()
	FString SourceFileTimestamp_DEPRECATED;

	/** Information for thumbnail rendering */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetThumbnailInfo() or UStaticMesh::SetThumbnailInfo().")
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Category=StaticMesh)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	/** The stored camera position to use as a default for the static mesh editor */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetEditorCameraPosition() or UStaticMesh::SetEditorCameraPosition().")
	UPROPERTY()
	FAssetEditorOrbitCameraPosition EditorCameraPosition;

	/** If the user has modified collision in any way or has custom collision imported. Used for determining if to auto generate collision on import */
	UE_DEPRECATED(5.7, "Please do not access this member directly, it will become private soon; use UStaticMesh::GetCustomizedCollision() or UStaticMesh::SetCustomizedCollision().")
	UPROPERTY(EditAnywhere, Category = Collision)
	bool bCustomizedCollision;
#endif // WITH_EDITORONLY_DATA

private:
	/** Unique ID for tracking/caching this mesh during distributed lighting */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	FGuid LightingGuid;
public:

	const FGuid& GetLightingGuid() const
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightingGuid);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LightingGuid;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
		static const FGuid NullGuid( 0, 0, 0, 0 );
		return NullGuid;
#endif // WITH_EDITORONLY_DATA
	}

	void SetLightingGuid(const FGuid& InLightingGuid = FGuid::NewGuid())
	{
#if WITH_EDITORONLY_DATA
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::LightingGuid);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LightingGuid = InLightingGuid;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 *	Array of named socket locations, set up in editor and used as a shortcut instead of specifying
	 *	everything explicitly to AttachComponent in the StaticMeshComponent.
	 */
	UPROPERTY()
	TArray<TObjectPtr<class UStaticMeshSocket>> Sockets;

	/** Data that is only available if this static mesh is an imported SpeedTree */
	TSharedPtr<class FSpeedTreeWind> SpeedTreeWind;

	/** Bound extension values in the positive direction of XYZ, positive value increases bound size */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use UStaticMesh::GetPositiveBoundsExtension() or UStaticMesh::SetPositiveBoundsExtension.")
	UPROPERTY(EditDefaultsOnly, AdvancedDisplay, Category = StaticMesh)
	FVector PositiveBoundsExtension;

	const FVector& GetPositiveBoundsExtension() const
	{
		// No need for WaitUntilAsyncPropertyReleased here as this is only read during async Build/Postload
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PositiveBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetPositiveBoundsExtension(FVector InPositiveBoundsExtension)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::PositiveBoundsExtension);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PositiveBoundsExtension = InPositiveBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetPositiveBoundsExtensionName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, PositiveBoundsExtension);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Bound extension values in the negative direction of XYZ, positive value increases bound size */
	UE_DEPRECATED(4.27, "Please do not access this member directly; use UStaticMesh::GetNegativeBoundsExtension() or UStaticMesh::SetNegativeBoundsExtension.")
	UPROPERTY(EditDefaultsOnly, AdvancedDisplay, Category = StaticMesh)
	FVector NegativeBoundsExtension;
	
	const FVector& GetNegativeBoundsExtension() const
	{
		// No need for WaitUntilAsyncPropertyReleased here as this is not modified during async Build/Postload
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NegativeBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetNegativeBoundsExtension(FVector InNegativeBoundsExtension)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::NegativeBoundsExtension);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NegativeBoundsExtension = InNegativeBoundsExtension;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static FName GetNegativeBoundsExtensionName()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, NegativeBoundsExtension);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Original mesh bounds extended with Positive/NegativeBoundsExtension */
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY()
	FBoxSphereBounds ExtendedBounds;
public:

	const FBoxSphereBounds& GetExtendedBounds() const
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::ExtendedBounds);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ExtendedBounds;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetExtendedBounds(const FBoxSphereBounds& InExtendedBounds)
	{
		WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::ExtendedBounds);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ExtendedBounds = InExtendedBounds;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
		OnExtendedBoundsChanged.Broadcast(InExtendedBounds);
#endif
	}

#if WITH_EDITOR
	FOnExtendedBoundsChanged OnExtendedBoundsChanged;
	FOnMeshChanged OnMeshChanged;

	/** This transient guid is use by the automation framework to modify the DDC key to force a build. */
	FGuid BuildCacheAutomationTestGuid;
#endif

protected:

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = StaticMesh)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	friend class FStaticMeshCompilingManager;
	friend class FStaticMeshAsyncBuildWorker;
	friend struct FStaticMeshUpdateContext;
	friend class FStaticMeshUpdate;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Instanced)
	TObjectPtr<class UObject> EditableMesh_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = Collision)
	TObjectPtr<class UStaticMesh> ComplexCollisionMesh;
#endif

	/**
	 * Registers the mesh attributes required by the mesh description for a static mesh.
	 */
	UE_DEPRECATED(4.24, "Please use FStaticMeshAttributes::Register to do this.")
	ENGINE_API static void RegisterMeshAttributes( FMeshDescription& MeshDescription );

#if WITH_EDITORONLY_DATA
	/*
	 * Return the MeshDescription associate to the LODIndex. The mesh description can be created on the fly if it was null
	 * and there is a FRawMesh data for this LODIndex.
	 */
	ENGINE_API FMeshDescription* GetMeshDescription(int32 LodIndex) const;

	/* 
	 * Clone the MeshDescription associated to the LODIndex.
	 *
	 * This will make a copy of any pending mesh description that hasn't been committed or will deserialize
	 * from the bulkdata or rawmesh directly if no current working copy exists.
	 */
	ENGINE_API bool CloneMeshDescription(int32 LodIndex, FMeshDescription& OutMeshDescription) const;

	ENGINE_API bool IsMeshDescriptionValid(int32 LodIndex) const;
	ENGINE_API FMeshDescription* CreateMeshDescription(int32 LodIndex);
	ENGINE_API FMeshDescription* CreateMeshDescription(int32 LodIndex, FMeshDescription MeshDescription);

	/** Structure that defines parameters passed into the commit mesh description function */
	struct FCommitMeshDescriptionParams
	{
		FCommitMeshDescriptionParams()
			: bMarkPackageDirty(true)
			, bUseHashAsGuid(false)
		{}

		/**
		* If set to false, the caller can be from any thread but will have the
		* responsability to call MarkPackageDirty() from the main thread.
		*/
		bool bMarkPackageDirty;

		/**
		* Uses a hash as the GUID, useful to prevent recomputing content already in cache.
		*/
		bool bUseHashAsGuid;
	};

	/**
	 * Serialize the mesh description into its more optimized form.
	 *
	 * @param	LodIndex	Index of the StaticMesh LOD.
	 * @param	Params		Different options to use when committing mesh description
	 */
	ENGINE_API void CommitMeshDescription(int32 LodIndex, const FCommitMeshDescriptionParams& Params = FCommitMeshDescriptionParams());

	/**
	 * Clears the cached mesh description for the given LOD.
	 * Note that this does not empty the bulk data.
	 */
	ENGINE_API void ClearMeshDescription(int32 LodIndex);

	/**
	 * Clears cached mesh descriptions for all LODs.
	 */
	ENGINE_API void ClearMeshDescriptions();

	ENGINE_API bool LoadHiResMeshDescription(FMeshDescription& OutMeshDescription) const;
	ENGINE_API bool CloneHiResMeshDescription(FMeshDescription& OutMeshDescription) const;
	ENGINE_API FMeshDescription* CreateHiResMeshDescription();
	ENGINE_API FMeshDescription* CreateHiResMeshDescription(FMeshDescription MeshDescription);
	ENGINE_API FMeshDescription* GetHiResMeshDescription() const;
	ENGINE_API bool IsHiResMeshDescriptionValid() const;
	ENGINE_API void CommitHiResMeshDescription(const FCommitMeshDescriptionParams& Params = FCommitMeshDescriptionParams());
	ENGINE_API void ClearHiResMeshDescription();

	/**
	 * Performs a Modify on the StaticMeshDescription object pertaining to the given LODIndex
	 */
	ENGINE_API bool ModifyMeshDescription(int32 LodIndex, bool bAlwaysMarkDirty = true);

	/**
	 * Performs a Modify on StaticMeshDescription objects for all LODs
	 */
	ENGINE_API bool ModifyAllMeshDescriptions(bool bAlwaysMarkDirty = true);

	/**
	 * Performs a Modify on the hi-res StaticMeshDescription
	 */
	ENGINE_API bool ModifyHiResMeshDescription(bool bAlwaysMarkDirty = true);


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/**
	 * Get AssetImportData for the static mesh
	 */
	class UAssetImportData* GetAssetImportData() const
	{
		return AssetImportData;
	}

	/**
	 * Set AssetImportData for the static mesh
	 */
	void SetAssetImportData(class UAssetImportData* InAssetImportData)
	{
		AssetImportData = InAssetImportData;
	}

	static FName GetAssetImportDataMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, AssetImportData);
	}

	/**
	 * Get ThumbnailInfo for the static mesh
	 */
	class UThumbnailInfo* GetThumbnailInfo() const
	{
		return ThumbnailInfo;
	}

	/**
	 * Set ThumbnailInfo for the static mesh
	 */
	void SetThumbnailInfo(class UThumbnailInfo* InThumbnailInfo)
	{
		ThumbnailInfo = InThumbnailInfo;
	}

	static FName GetThumbnailInfoMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UStaticMesh, ThumbnailInfo);
	}

	const FAssetEditorOrbitCameraPosition& GetEditorCameraPosition()
	{
		return EditorCameraPosition;
	}

	void SetEditorCameraPosition(const FAssetEditorOrbitCameraPosition& Position)
	{
		EditorCameraPosition = Position;
	}

	bool GetCustomizedCollision()
	{
		return bCustomizedCollision;
	}

	void SetCustomizedCollision(bool bInCustomizedCollision)
	{
		bCustomizedCollision = bInCustomizedCollision;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Adds an empty UV channel at the end of the existing channels on the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @return true if a UV channel was added.
	 */
	ENGINE_API bool AddUVChannel(int32 LODIndex);

	/**
	 * Inserts an empty UV channel at the specified channel index on the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to insert the UV channel.
	 * @return true if a UV channel was added.
	 */
	ENGINE_API bool InsertUVChannel(int32 LODIndex, int32 UVChannelIndex);

	/**
	 * Removes the UV channel at the specified channel index on the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to remove the UV channel.
	 * @return true if the UV channel was removed.
	 */
	ENGINE_API bool RemoveUVChannel(int32 LODIndex, int32 UVChannelIndex);

	/**
	 * Sets the texture coordinates at the specified UV channel index on the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to remove the UV channel.
	 * @param	TexCoords			The texture coordinates to set on the UV channel.
	 * @return true if the UV channel could be set.
	 */
	ENGINE_API bool SetUVChannel(int32 LODIndex, int32 UVChannelIndex, const TMap<FVertexInstanceID, FVector2D>& TexCoords);

	/**
	 * Sets the texture coordinates at the specified UV channel index on the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to remove the UV channel.
	 * @param	TexCoords			The texture coordinates to set on the UV channel.
	 * @return true if the UV channel could be set.
	 */
	ENGINE_API bool SetUVChannel(int32 LODIndex, int32 UVChannelIndex, const TMap<FVertexInstanceID, FVector2f>& TexCoords);

#endif

	/** Create an empty StaticMeshDescription object, to describe a static mesh at runtime */
	UFUNCTION(BlueprintCallable, Category="StaticMesh")
	static ENGINE_API UStaticMeshDescription* CreateStaticMeshDescription(UObject* Outer = nullptr);

	/** Builds static mesh LODs from the array of StaticMeshDescriptions passed in */
	UFUNCTION(BlueprintCallable, Category="StaticMesh")
	ENGINE_API void BuildFromStaticMeshDescriptions(const TArray<UStaticMeshDescription*>& StaticMeshDescriptions, bool bBuildSimpleCollision = false, bool bFastBuild = true);

	/** Return a new StaticMeshDescription referencing the MeshDescription of the given LOD */
	UFUNCTION(BlueprintCallable, Category="StaticMesh")
	ENGINE_API UStaticMeshDescription* GetStaticMeshDescription(int32 LODIndex);

	struct FBuildMeshDescriptionsLODParams
	{
		/**
		 * If true, Tangents will be stored at 16 bit vs 8 bit precision.
		 */
		bool bUseHighPrecisionTangentBasis = false;

		/**
		 * If true, UVs will be stored at full floating point precision.
		 */
		bool bUseFullPrecisionUVs = false;
	};

	 /** Structure that defines parameters passed into the build mesh description function */
	struct FBuildMeshDescriptionsParams
	{
		FBuildMeshDescriptionsParams()
			: bMarkPackageDirty(true)
			, bUseHashAsGuid(false)
			, bBuildSimpleCollision(false)
			, bCommitMeshDescription(true)
			, bFastBuild(false)
			, bAllowCpuAccess(false)
		{}

		/**
		 * If set to false, the caller can be from any thread but will have the
		 * responsibility to call MarkPackageDirty() from the main thread.
		 */
		bool bMarkPackageDirty;

		/**
		 * Uses a hash as the GUID, useful to prevent recomputing content already in cache.
		 * Set to false by default.
		 */
		bool bUseHashAsGuid;

		/**
		 * Builds simple collision as part of the building process. Set to false by default.
		 */
		bool bBuildSimpleCollision;
	
		/**
		 * Commits the MeshDescription as part of the building process. Set to true by default.
		 */
		bool bCommitMeshDescription;

		/**
		 * Specifies that the mesh will be built by the fast path (mandatory in non-editor builds).
		 * Set to false by default.
		 */
		bool bFastBuild;

		/**
		 * Ored with the value of bAllowCpuAccess on the static mesh. Set to false by default.
		 */
		bool bAllowCpuAccess;

		/**
		 * Extra optional LOD params. Overrides any previous settings from source model build settings.
		 */
		TArray<FBuildMeshDescriptionsLODParams> PerLODOverrides;
	};

	/**
	 * Builds static mesh render buffers from a list of MeshDescriptions, one per LOD.
	 */
	ENGINE_API bool BuildFromMeshDescriptions(const TArray<const FMeshDescription*>& MeshDescriptions, const FBuildMeshDescriptionsParams& Params = FBuildMeshDescriptionsParams());
	
	/** Builds a LOD resource from a MeshDescription */
	ENGINE_API void BuildFromMeshDescription(const FMeshDescription& MeshDescription, FStaticMeshLODResources& LODResources);

	/** Exports static mesh LOD render data to a MeshDescription */
	ENGINE_API void ExportStaticMeshLOD(const FStaticMeshLODResources& InLODResources, FMeshDescription& OutMeshDescription) const;

	/**
	 * Returns the number of UV channels for the given LOD of a StaticMesh.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @return the number of UV channels.
	 */
	ENGINE_API int32 GetNumUVChannels(int32 LODIndex) const;

	/** Pre-build navigation collision */
private:
	UE_DEPRECATED(5.0, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(VisibleAnywhere, transient, duplicatetransient, Instanced, Category = Navigation, meta = (EditCondition = "bHasNavigationData"))
	TObjectPtr<UNavCollisionBase> NavCollision;

public:
	ENGINE_API void SetNavCollision(UNavCollisionBase*);
	ENGINE_API UNavCollisionBase* GetNavCollision() const;
	ENGINE_API FBox GetNavigationBounds(const FTransform& LocalToWorld) const;
	ENGINE_API bool IsNavigationRelevant() const;

	/**
	 * Default constructor
	 */
	ENGINE_API UStaticMesh(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	ENGINE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	
	ENGINE_API virtual void WillNeverCacheCookedPlatformDataAgain() override;
	ENGINE_API virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API void SetLODGroup(FName NewGroup, bool bRebuildImmediately = true, bool bAllowModify = true);
	ENGINE_API void BroadcastNavCollisionChange();

	FOnExtendedBoundsChanged& GetOnExtendedBoundsChanged() { return OnExtendedBoundsChanged; }
	FOnMeshChanged& GetOnMeshChanged() { return OnMeshChanged; }

	/*
	 * Add or change the LOD data specified by LodIndex with the content of the sourceStaticMesh.
	 *
	 * @Param SourceStaticMesh - The data we want to use to add or modify the specified lod
	 * @Param LodIndex - The lod index we want to add or modify.
	 * @Param SourceDataFilename - The source filename thta need to set into the LOD to allow re-import.
	 */
	ENGINE_API bool SetCustomLOD(const UStaticMesh* SourceStaticMesh, int32 LodIndex, const FString& SourceDataFilename);

	/*
	 * Static function that remove any trailing unused material.
	 *
	 * @Param StaticMesh - The static mesh we want to remove the trailing materials.
	 */
	ENGINE_API static void RemoveUnusedMaterialSlots(UStaticMesh* StaticMesh);

	//SourceModels API
	ENGINE_API FStaticMeshSourceModel& AddSourceModel();

	UFUNCTION(BlueprintCallable, Category="StaticMesh")
	ENGINE_API void SetNumSourceModels(int32 Num);

	ENGINE_API void RemoveSourceModel(int32 Index);
	ENGINE_API const TArray<FStaticMeshSourceModel>& GetSourceModels() const;
	ENGINE_API FStaticMeshSourceModel& GetSourceModel(int32 Index);
	ENGINE_API const FStaticMeshSourceModel& GetSourceModel(int32 Index) const;
	ENGINE_API int32 GetNumSourceModels() const;
	ENGINE_API bool IsSourceModelValid(int32 Index) const;
	ENGINE_API TArray<FStaticMeshSourceModel>&& MoveSourceModels();
	ENGINE_API void SetSourceModels(TArray<FStaticMeshSourceModel>&& SourceModels);

	ENGINE_API FStaticMeshSourceModel& GetHiResSourceModel();
	ENGINE_API const FStaticMeshSourceModel& GetHiResSourceModel() const;
	ENGINE_API FStaticMeshSourceModel&& MoveHiResSourceModel();
	ENGINE_API void SetHiResSourceModel(FStaticMeshSourceModel&& SourceModel);

	ENGINE_API FMeshSectionInfoMap& GetSectionInfoMap();
	ENGINE_API const FMeshSectionInfoMap& GetSectionInfoMap() const;
	ENGINE_API FMeshSectionInfoMap& GetOriginalSectionInfoMap();
	ENGINE_API const FMeshSectionInfoMap& GetOriginalSectionInfoMap() const;

	ENGINE_API bool IsAsyncTaskComplete() const;
	
	/** Try to cancel any pending async tasks.
	 *  Returns true if there is no more async tasks pending, false otherwise.
	 */
	ENGINE_API bool TryCancelAsyncTasks();

	TUniquePtr<FStaticMeshAsyncBuildTask> AsyncTask;
#endif // WITH_EDITOR

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
	ENGINE_API static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	ENGINE_API virtual FString GetDesc() override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual bool CanBeClusterRoot() const override;
	//~ End UObject Interface.

	//~ Begin UStreamableRenderAsset Interface
	ENGINE_API virtual int32 CalcCumulativeLODSize(int32 NumLODs) const final override;
	ENGINE_API virtual FIoFilenameHash GetMipIoFilenameHash(const int32 MipIndex) const final override;
	ENGINE_API virtual bool DoesMipDataExist(const int32 MipIndex) const final override;
	ENGINE_API virtual bool HasPendingRenderResourceInitialization() const final override;
	ENGINE_API virtual bool StreamOut(int32 NewMipCount) final override;
	ENGINE_API virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) final override;
	ENGINE_API virtual EStreamableRenderAssetType GetRenderAssetType() const final override;
	//~ End UStreamableRenderAsset Interface

	/**
	* Cancels any pending static mesh streaming actions if possible.
	* Returns when no more async loading requests are in flight.
	*/
	ENGINE_API static void CancelAllPendingStreamingActions();

	/**
	 * Contains all the parameters required to build the mesh.
	 */
	struct FBuildParameters
	{
		// Required to work around a Clang bug
#if PLATFORM_COMPILER_CLANG
		FBuildParameters()
			: bInSilent(false)
			, OutErrors(nullptr)
			, bInRebuildUVChannelData(false)
			, bInEnforceLightmapRestrictions(false)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			bIgnoreBoundsDiff = false;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
#else
		FBuildParameters() = default;
#endif

		FBuildParameters(const FBuildParameters& Other)
		{
			bInSilent = Other.bInSilent;
			OutErrors = Other.OutErrors;
			bInRebuildUVChannelData = Other.bInRebuildUVChannelData;
			bInEnforceLightmapRestrictions = Other.bInEnforceLightmapRestrictions;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			bIgnoreBoundsDiff = Other.bIgnoreBoundsDiff;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// If true will not popup a progress dialog.
		bool bInSilent = false;

		// If provided, will contain the errors that occurred during this process.This will prevent async static mesh compilation because OutErrors could get out of scope.
		TArray<FText>* OutErrors = nullptr;

		// Whether to completely rebuild the UV Channel data after the render data has been computed.
		bool bInRebuildUVChannelData = false;

		// Whether to call EnforceLightmapRestrictions as part of the build process.
		bool bInEnforceLightmapRestrictions = false;

		// Whether to avoid outputting a warning if the render data and MeshDescription bounds differ too much.
		UE_DEPRECATED(5.7, "This option no longer has any effect and will be removed in a future release.")
		bool bIgnoreBoundsDiff = false;
	};

	/**
	 * Rebuilds renderable data for this static mesh, automatically made async if enabled.
	 * @param		bInSilent	If true will not popup a progress dialog.
	 * @param [out]	OutErrors	If provided, will contain the errors that occurred during this process. This will prevent async static mesh compilation because OutErrors could get out of scope.
	 */
	ENGINE_API void Build(bool bInSilent, TArray<FText>* OutErrors = nullptr);

	/**
	 * Rebuilds renderable data for a batch of static meshes.
	 * @param		InStaticMeshes		The list of all static meshes to build.
	 * @param		bInSilent			If true will not popup a progress dialog.
	 * @param		InProgressCallback	If provided, will be used to abort task and report progress to higher level functions (should return true to continue, false to abort).
	 * @param [out]	OutErrors			If provided, will contain the errors that occurred during this process. This will prevent async static mesh compilation because OutErrors could get out of scope.
	 */
	ENGINE_API static void BatchBuild(const TArray<UStaticMesh*>& InStaticMeshes, bool bInSilent, TFunction<bool(UStaticMesh*)> InProgressCallback = nullptr, TArray<FText>* OutErrors = nullptr);

	/**
	 * Rebuilds renderable data for this static mesh, automatically made async if enabled.
	 * @param		BuildParameters	  Contains all the information required to build the mesh.
	 */
	ENGINE_API void Build(const FBuildParameters& BuildParameters = FBuildParameters());

	/**
	 * Rebuilds renderable data for a batch of static meshes.
	 * @param		InStaticMeshes		The list of all static meshes to build.
	 * @param		BuildParameters	    Contains all the parameters required to build the mesh.
	 * @param		InProgressCallback	If provided, will be used to abort task and report progress to higher level functions (should return true to continue, false to abort).
	 */
	ENGINE_API static void BatchBuild(const TArray<UStaticMesh*>& InStaticMeshes, const FBuildParameters& BuildParameters = FBuildParameters(), TFunction<bool(UStaticMesh*)> InProgressCallback = nullptr);

	/**
	 * Initialize the static mesh's render resources.
	 */
	ENGINE_API virtual void InitResources();

	/**
	 * Releases the static mesh's render resources.
	 */
	ENGINE_API virtual void ReleaseResources();

	/**
	 * Update missing material UV channel data used for texture streaming. 
	 *
	 * @param bRebuildAll		If true, rebuild everything and not only missing data.
	 */
	ENGINE_API void UpdateUVChannelData(bool bRebuildAll);

	/**
	 * Returns the material bounding box. Computed from all lod-section using the material index.
	 *
	 * @param MaterialIndex			Material Index to look at
	 * @param TransformMatrix		Matrix to be applied to the position before computing the bounds
	 *
	 * @return false if some parameters are invalid
	 */
	ENGINE_API FBox GetMaterialBox(int32 MaterialIndex, const FTransform& Transform) const;

	/**
	 * Returns the UV channel data for a given material index. Used by the texture streamer.
	 * This data applies to all lod-section using the same material.
	 *
	 * @param MaterialIndex		the material index for which to get the data for.
	 * @return the data, or null if none exists.
	 */
	ENGINE_API const FMeshUVChannelInfo* GetUVChannelData(int32 MaterialIndex) const;

	/**
	 * Returns the number of vertices for the specified LOD.
	 */
	UFUNCTION(BlueprintPure, Category = StaticMesh)
	ENGINE_API int32 GetNumVertices(int32 LODIndex) const;

	/**
	 * Returns the number of triangles in the render data for the specified LOD.
	 */
	UFUNCTION(BlueprintPure, Category = StaticMesh)
	ENGINE_API int32 GetNumTriangles(int32 LODIndex) const;

	/**
	 * Returns the number of tex coords for the specified LOD.
	 */
	UFUNCTION(BlueprintPure, Category = StaticMesh)
	ENGINE_API int32 GetNumTexCoords(int32 LODIndex) const;

	/**
	 * Returns the number of vertices of the Nanite representation of this mesh.
	 */
	UFUNCTION(BlueprintPure, Category = StaticMesh)
	ENGINE_API int32 GetNumNaniteVertices() const;

	/**
	 * Returns the number of triangles of the Nanite representation of this mesh.
	 */
	UFUNCTION(BlueprintPure, Category = StaticMesh)
	ENGINE_API int32 GetNumNaniteTriangles() const;

	/**
	 * Returns the number of LODs used by the mesh.
	 */
	UFUNCTION(BlueprintPure, Category = "StaticMesh", meta=(ScriptName="GetNumLods"))
	ENGINE_API int32 GetNumLODs() const;

	/**
	 * Returns true if the mesh has data that can be rendered.
	 */
	ENGINE_API bool HasValidRenderData(bool bCheckLODForVerts = true, int32 LODIndex = INDEX_NONE) const;

	/**
	 * Returns true if the mesh has valid Nanite render data.
	 */
	ENGINE_API bool HasValidNaniteData() const;

	/**
	 * Returns true if the mesh has Nanite fallback on a specific shader platform.
	 */
	ENGINE_API bool HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const;

	/**
	 * Returns the number of bounds of the mesh.
	 *
	 * @return	The bounding box represented as box origin with extents and also a sphere that encapsulates that box
	 */
	UFUNCTION( BlueprintPure, Category="StaticMesh" )
	ENGINE_API FBoxSphereBounds GetBounds() const;

	/** Returns the bounding box, in local space including bounds extension(s), of the StaticMesh asset */
	UFUNCTION(BlueprintPure, Category="StaticMesh")
	ENGINE_API FBox GetBoundingBox() const;

	/** Returns number of Sections that this StaticMesh has, in the supplied LOD (LOD 0 is the highest) */
	UFUNCTION(BlueprintPure, Category = "StaticMesh")
	ENGINE_API int32 GetNumSections(int32 InLOD) const;

	/**
	 * Gets a Material given a Material Index and an LOD number
	 *
	 * @return Requested material
	 */
	UFUNCTION(BlueprintPure, Category = "StaticMesh")
	ENGINE_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const;

	/**
	 * Adds a new material and return its slot name
	 */
	UFUNCTION(BlueprintCallable, Category = "StaticMesh")
	ENGINE_API FName AddMaterial(UMaterialInterface* Material);

	/**
	 * Gets a Material index given a slot name
	 *
	 * @return Requested material
	 */
	UFUNCTION(BlueprintPure, Category = "StaticMesh")
	ENGINE_API int32 GetMaterialIndex(FName MaterialSlotName) const;

	ENGINE_API int32 GetMaterialIndexFromImportedMaterialSlotName(FName ImportedMaterialSlotName) const;


	ENGINE_API void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, TFunctionRef<UMaterialInterface*(int32)> OverrideMaterial) const;

	/**
	 * Returns the render data to use for exporting the specified LOD. This method should always
	 * be called when exporting a static mesh.
	 */
	ENGINE_API const FStaticMeshLODResources& GetLODForExport(int32 LODIndex) const;

	/**
	 * Static: Processes the specified static mesh for light map UV problems
	 *
	 * @param	InStaticMesh					Static mesh to process
	 * @param	InOutAssetsWithMissingUVSets	Array of assets that we found with missing UV sets
	 * @param	InOutAssetsWithBadUVSets		Array of assets that we found with bad UV sets
	 * @param	InOutAssetsWithValidUVSets		Array of assets that we found with valid UV sets
	 * @param	bInVerbose						If true, log the items as they are found
	 */
	ENGINE_API static void CheckLightMapUVs( UStaticMesh* InStaticMesh, TArray< FString >& InOutAssetsWithMissingUVSets, TArray< FString >& InOutAssetsWithBadUVSets, TArray< FString >& InOutAssetsWithValidUVSets, bool bInVerbose = true );

	//~ Begin Interface_CollisionDataProvider Interface
	ENGINE_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	ENGINE_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	ENGINE_API virtual bool PollAsyncPhysicsTriMeshData(bool InUseAllTriData) const override;
	ENGINE_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;

private:
	bool GetPhysicsTriMeshDataCheckComplex(struct FTriMeshCollisionData* CollisionData, bool bInUseAllTriData, bool bInCheckComplexCollisionMesh);
	bool PollAsyncPhysicsTriMeshDataCheckComplex(bool InUseAllTriData, bool bInCheckComplexCollisionMesh) const;
	bool ContainsPhysicsTriMeshDataCheckComplex(bool InUseAllTriData, bool bInCheckComplexCollisionMesh) const;
	bool SectionHasCollisionEnabled(const FStaticMeshSection& Section, int32 LODIndex, int32 SectionIndex) const;

public:

	virtual bool WantsNegXTriMesh() override
	{
		return true;
	}
	ENGINE_API virtual void GetMeshId(FString& OutMeshId) override;
	//~ End Interface_CollisionDataProvider Interface

	/** Return the number of sections of the StaticMesh with collision enabled */
	int32 GetNumSectionsWithCollision() const;

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface


	/**
	 * Create BodySetup for this staticmesh if it doesn't have one
	 */
	ENGINE_API void CreateBodySetup();

	/**
	 * Calculates navigation collision for caching
	 */
	ENGINE_API void CreateNavCollision(const bool bIsUpdate = false);

	/**
	 * Delete current NavCollision and create a new one if needed
	 */
	ENGINE_API void RecreateNavCollision();

	/** Configures this SM as bHasNavigationData = false and clears stored NavCollision */
	ENGINE_API void MarkAsNotHavingNavigationData();


	/**
	 *	Add a socket object in this StaticMesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "StaticMesh")
	ENGINE_API void AddSocket(UStaticMeshSocket* Socket);

	/**
	 *	Find a socket object in this StaticMesh by name.
	 *	Entering NAME_None will return NULL. If there are multiple sockets with the same name, will return the first one.
	 */
	UFUNCTION(BlueprintPure, Category = "StaticMesh")
	ENGINE_API class UStaticMeshSocket* FindSocket(FName InSocketName) const;

	/**
	 *	Remove a socket object in this StaticMesh by providing it's pointer. Use FindSocket() if needed.
	 */
	UFUNCTION(BlueprintCallable, Category = "StaticMesh")
	ENGINE_API void RemoveSocket(UStaticMeshSocket* Socket);

	/**
	 * Returns a list of sockets with the provided tag.
	 */
	UFUNCTION(BlueprintPure, Category = "StaticMesh")
	ENGINE_API TArray<UStaticMeshSocket*> GetSocketsByTag(const FString& InSocketTag) const;

	/**
	 * Returns vertex color data by position.
	 * For matching to reimported meshes that may have changed or copying vertex paint data from mesh to mesh.
	 *
	 *	@param	VertexColorData		(out)A map of vertex position data and its color. The method fills this map.
	 */
	ENGINE_API void GetVertexColorData(TMap<FVector3f, FColor>& VertexColorData);

	/**
	 * Sets vertex color data by position.
	 * Map of vertex color data by position is matched to the vertex position in the mesh
	 * and nearest matching vertex color is used.
	 *
	 *	@param	VertexColorData		A map of vertex position data and color.
	 */
	ENGINE_API void SetVertexColorData(const TMap<FVector3f, FColor>& VertexColorData);

	/** Removes all vertex colors from this mesh and rebuilds it (Editor only */
	ENGINE_API void RemoveVertexColors();

	/** Make sure the Lightmap UV point on a valid UVChannel */
	ENGINE_API void EnforceLightmapRestrictions(bool bUseRenderData = true);

	/** Calculates the extended bounds */
	ENGINE_API void CalculateExtendedBounds();

	inline bool AreRenderingResourcesInitialized() const { return bRenderingResourcesInitialized; }

	/** Helper function for resource tracking, construct a name using the mesh's path name and LOD index . */
	static FName GetLODPathName(const UStaticMesh* Mesh, int32 LODIndex);

#if WITH_EDITOR

	/**
	 * Sets a Material given a Material Index
	 */
	UFUNCTION(BlueprintCallable, Category = "StaticMesh")
	ENGINE_API void SetMaterial(int32 MaterialIndex, UMaterialInterface* NewMaterial);

	/**
	 * Returns true if LODs of this static mesh may share texture lightmaps.
	 */
	ENGINE_API bool CanLODsShareStaticLighting() const;

	/**
	 * Retrieves the names of all LOD groups.
	 */
	ENGINE_API static void GetLODGroups(TArray<FName>& OutLODGroups);

	/**
	 * Retrieves the localized display names of all LOD groups.
	 */
	ENGINE_API static void GetLODGroupsDisplayNames(TArray<FText>& OutLODGroupsDisplayNames);

	ENGINE_API void GenerateLodsInPackage();

	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	/** Get multicast delegate broadcast prior to mesh building */
	FOnPreMeshBuild& OnPreMeshBuild() { return PreMeshBuild; }

	/** Get multicast delegate broadcast after mesh building */
	FOnPostMeshBuild& OnPostMeshBuild() { return PostMeshBuild; }
	

	/* Return true if the reduction settings are setup to reduce a LOD*/
	ENGINE_API bool IsReductionActive(int32 LODIndex) const;

	/* Get a copy of the reduction settings for a specified LOD index. */
	ENGINE_API struct FMeshReductionSettings GetReductionSettings(int32 LODIndex) const;

	/** Get whether this mesh should use LOD streaming for the given platform. */
	bool GetEnableLODStreaming(const class ITargetPlatform* TargetPlatform) const;

	/* Get a static mesh render data for requested platform. */
	static FStaticMeshRenderData& GetPlatformStaticMeshRenderData(UStaticMesh* Mesh, const ITargetPlatform* Platform);

private:
	/**
	 * Converts legacy LODDistance in the source models to Display Factor
	 */
	void ConvertLegacyLODDistance();

	/**
	 * Converts legacy LOD screen area in the source models to resolution-independent screen size
	 */
	void ConvertLegacyLODScreenArea();

	/**
	 * Fixes up static meshes that were imported with sections that had zero triangles.
	 */
	void FixupZeroTriangleSections();

	/**
	* Converts legacy RawMesh to MeshDescription.
	*/
	void ConvertLegacySourceData();
	
	/**
	 * Verify if the static mesh can be built.
	 */
	bool CanBuild() const;

	/**
	 * Initial step for the static mesh building process - Can't be done in parallel.
	 */
	void BeginBuildInternal(FStaticMeshBuildContext* Context = nullptr);

	/**
	 * Build the static mesh
	 */
	bool ExecuteBuildInternal(const FBuildParameters& BuildParameters);

	/**
	 * Complete the static mesh building process - Can't be done in parallel.
	 */
	void FinishBuildInternal(const TArray<IStaticMeshComponent*>& InAffectedComponents, bool bHasRenderDataChanged, bool bShouldComputeExtendedBounds = true);

#if WITH_EDITORONLY_DATA
	/**
	 * Deserialize MeshDescription for the specified LodIndex from BulkData, DDC or RawMesh.
	 */
	bool LoadMeshDescription(int32 LodIndex, FMeshDescription& OutMeshDescription) const;
#endif

public:
	/**
	 * Caches derived renderable data.
	 */
	ENGINE_API void CacheDerivedData();

	/**
	 * Get an estimate of the peak amount of memory required to build this mesh.
	 */
	ENGINE_API int64 GetBuildRequiredMemoryEstimate() const;

	/**
	 * Caches derived renderable for cooked platforms currently active.
	 */
	ENGINE_API void PrepareDerivedDataForActiveTargetPlatforms();

private:
	// Filled at CommitDescription time and reused during build
	TOptional<FBoxSphereBounds> CachedMeshDescriptionBounds;

	// Notification about missing Nanite required shader models.
	TWeakPtr<class SNotificationItem> ShaderModelNotificationPtr;

	void CheckForValidMinLODs(FPerQualityLevelInt& QualityLocalMinLOD, FPerPlatformInt& LocalMinLOD, int32& OutMinAvailableLOD, TArray<TPair<int32, FName>>& OutInvalidMinLODs) const;
	void CheckForMissingShaderModels();

	FOnPreMeshBuild PreMeshBuild;
	FOnPostMeshBuild PostMeshBuild;

	/**
	 * Fixes up the material when it was converted to the new staticmesh build process
	 */
	bool bCleanUpRedundantMaterialPostLoad;

	/**
	 * Guard to ignore re-entrant PostEditChange calls.
	 */
	bool bIsInPostEditChange = false;
#endif // #if WITH_EDITOR

	/**
	 * Initial step for the Post Load process - Can't be done in parallel.
	 */
	void BeginPostLoadInternal(FStaticMeshPostLoadContext& Context);

	/**
	 * Thread-safe part of the Post Load
	 */
	void ExecutePostLoadInternal(FStaticMeshPostLoadContext& Context);

	/**
	 * Complete the static mesh postload process - Can't be done in parallel.
	 */
	void FinishPostLoadInternal(FStaticMeshPostLoadContext& Context);
};

class FStaticMeshCompilationContext
{
public:
	FStaticMeshCompilationContext();
	// Non-copyable
	FStaticMeshCompilationContext(const FStaticMeshCompilationContext&) = delete;
	FStaticMeshCompilationContext& operator=(const FStaticMeshCompilationContext&) = delete;
	// Movable
	FStaticMeshCompilationContext(FStaticMeshCompilationContext&&) = delete;
	FStaticMeshCompilationContext& operator=(FStaticMeshCompilationContext&&) = delete;

	bool bShouldComputeExtendedBounds = false;
	bool bIsEditorLoadingPackage = false;
	UE::Tasks::FCancellationToken CancellationToken;
};

class FStaticMeshPostLoadContext : public FStaticMeshCompilationContext
{
public:
	bool bNeedsMeshUVDensityFix = false;
	bool bNeedsMaterialFixup = false;
	bool bIsCookedForEditor = false;
};

class FStaticMeshBuildContext : public FStaticMeshCompilationContext
{
public:
	FStaticMeshBuildContext(const UStaticMesh::FBuildParameters& InBuildParameters)
		: BuildParameters(InBuildParameters)
	{
	}

	UStaticMesh::FBuildParameters BuildParameters;
	bool bHasRenderDataChanged = false;
};

struct FStaticMeshBuildParameters
{
	FStaticMeshBuildParameters(UStaticMesh* InStaticMesh, const ITargetPlatform* InTargetPlatform, const FStaticMeshLODGroup& InLODGroup)
		: StaticMesh(InStaticMesh)
		, TargetPlatform(InTargetPlatform)
		, LODGroup(InLODGroup)
	{}

	UStaticMesh* StaticMesh;
	const ITargetPlatform* TargetPlatform;
	const FStaticMeshLODGroup& LODGroup;
};

namespace UE::Private::StaticMesh
{
#if WITH_EDITOR
	ENGINE_API FString BuildStaticMeshDerivedDataKey(const ITargetPlatform* TargetPlatform, UStaticMesh* Mesh, const FStaticMeshLODGroup& LODGroup);
#endif
}
