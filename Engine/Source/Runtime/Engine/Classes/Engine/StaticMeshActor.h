// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavDataGatheringMode.h"
#include "StaticMeshActor.generated.h"

/**
 * StaticMeshActor is an instance of a UStaticMesh in the world.
 * Static meshes are geometry that do not animate or otherwise deform, and are more efficient to render than other types of geometry.
 * Static meshes dragged into the level from the Content Browser are automatically converted to StaticMeshActors.
 *
 * @see https://docs.unrealengine.com/latest/INT/Engine/Actors/StaticMeshActor/
 * @see UStaticMesh
 */
UCLASS(hidecategories=(Input), showcategories=("Input|MouseInput", "Input|TouchInput"), ConversionRoot, ComponentWrapperClass, meta=(ChildCanTick), MinimalAPI)
class AStaticMeshActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = StaticMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class UStaticMeshComponent> StaticMeshComponent;

public:
	ENGINE_API virtual void PostInitializeComponents() override;

	/** This static mesh should replicate movement. Automatically sets the RemoteRole and bReplicateMovement flags. Meant to be edited on placed actors (those other two properties are not) */
	UPROPERTY(Category=Actor, EditAnywhere, AdvancedDisplay)
	bool bStaticMeshReplicateMovement;

	/** Set which replication mode to use for this static mesh instance if it both replicates and simulates physics. */
	UPROPERTY(Category = Actor, EditAnywhere, AdvancedDisplay)
	EPhysicsReplicationMode StaticMeshPhysicsReplicationMode;

	UE_DEPRECATED(5.0, "Unused property. The actor will use the DefaultGeometryGatheringMode set in FNavigationOctree (see virtual ENavDataGatheringMode GetGeometryGatheringMode()).")
	UPROPERTY(EditAnywhere, Category = Navigation, AdvancedDisplay)
	ENavDataGatheringMode NavigationGeometryGatheringMode;

	/** Function to change mobility type */
	UFUNCTION(BlueprintCallable, Category = Mobility)
	ENGINE_API void SetMobility(EComponentMobility::Type InMobility);

	//~ Begin AActor Interface
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const override;
	ENGINE_API virtual void PostLoad() override;	
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End AActor Interface

	// INavRelevantInterface begin
	virtual ENavDataGatheringMode GetGeometryGatheringMode() const { return ENavDataGatheringMode::Default; }
	// INavRelevantInterface end

protected:
	//~ Begin UObject Interface.
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR	
	//~ End UObject Interface.
	
public:
	/** Returns StaticMeshComponent subobject **/
	class UStaticMeshComponent* GetStaticMeshComponent() const { return StaticMeshComponent; }

	/** Name of the StaticMeshComponent.  Use this name if you want to use a different class (with ObjectInitializer.SetDefaultSubobjectClass). */
	static ENGINE_API FName StaticMeshComponentName;
};



