// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "StaticMeshSocket.generated.h"

class AActor;
struct FPropertyChangedEvent;

UCLASS(hidecategories=Object, hidecategories=Actor, MinimalAPI)
class UStaticMeshSocket : public UObject
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	Defines a named attachment location on the UStaticMesh. 
	 *	These are set up in editor and used as a shortcut instead of specifying 
	 *	everything explicitly to AttachComponent in the StaticMeshComponent.
	 *	The Outer of a StaticMeshSocket should always be the UStaticMesh.
	 */
	UPROPERTY(Category=StaticMeshSocket, BlueprintReadOnly)
	FName SocketName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=StaticMeshSocket)
	FVector RelativeLocation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=StaticMeshSocket)
	FRotator RelativeRotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=StaticMeshSocket)
	FVector RelativeScale;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=StaticMeshSocket)
	FString Tag;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=StaticMeshSocket)
	TObjectPtr<class UStaticMesh> PreviewStaticMesh;

	/* Whether the socket was imported with the asset or created in the editor. Importer will remove/modify only imported socket and will not touch any editor created socket. */
	UPROPERTY()
	bool bSocketCreatedAtImport;
#endif // WITH_EDITORONLY_DATA

	/** Utility that returns the current matrix for this socket. Returns false if socket was not valid */
	ENGINE_API bool GetSocketMatrix(FMatrix& OutMatrix, class UStaticMeshComponent const* MeshComp) const;

	/** Utility that returns the current transform for this socket. Returns false if socket was not valid */
	ENGINE_API bool GetSocketTransform(FTransform& OutTransform, class UStaticMeshComponent const* MeshComp) const;

	/** 
	 *	Utility to associate an actor with a socket
	 *	
	 *	@param	Actor			The actor to attach to the socket
	 *	@param	MeshComp		The static mesh component associated with this socket.
	 *
	 *	@return	bool			true if successful, false if not
	 */
	ENGINE_API bool AttachActor(AActor* Actor, class UStaticMeshComponent* MeshComp) const;

	virtual void Serialize(FArchive& Ar) override;

public:
#if WITH_EDITOR
	/** Broadcasts a notification whenever the socket property has changed. */
	DECLARE_EVENT_TwoParams( UStaticMeshSocket, FChangedEvent, const class UStaticMeshSocket*, const class FProperty* );
	FChangedEvent& OnPropertyChanged() { return ChangedEvent; }

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) override;
	//~ End UObject Interface

private:
	/** Broadcasts a notification whenever the socket property has changed. */
	FChangedEvent ChangedEvent;
#endif // WITH_EDITOR
};



