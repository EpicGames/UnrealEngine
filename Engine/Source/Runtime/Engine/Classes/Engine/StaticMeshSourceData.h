// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescriptionBaseBulkData.h"
#include "Engine/EngineTypes.h"
#include "MeshReductionSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"

#include "StaticMeshSourceData.generated.h"


struct FMeshDescription;
class UStaticMeshDescription;


/**
 * UObject wrapper for FMeshDescriptionBulkData
 */
UCLASS(MinimalAPI)
class UStaticMeshDescriptionBulkData : public UMeshDescriptionBaseBulkData
{
	GENERATED_BODY()

public:
	ENGINE_API UStaticMeshDescriptionBulkData();
};


/**
 * Source model from which a renderable static mesh is built.
 */
USTRUCT()
struct FStaticMeshSourceModel
{
	GENERATED_BODY()

	friend class UStaticMesh;

public:
#if WITH_EDITOR
	/**
	 * Imported raw mesh data. Optional for all but the first LOD.
	 *
	 * This is a member for legacy assets only.
	 * If it is non-empty, this means that it has been de-serialized from the asset, and
	 * the asset hence pre-dates MeshDescription.
	 */
	class FRawMeshBulkData* RawMeshBulkData;
	
	/*
	 * Accessor to Load and save the raw mesh or the mesh description depending on the editor settings.
	 * Temporary until we deprecate the RawMesh.
	 */
	ENGINE_API bool IsRawMeshEmpty() const;
	ENGINE_API void LoadRawMesh(struct FRawMesh& OutRawMesh) const;
	ENGINE_API void SaveRawMesh(struct FRawMesh& InRawMesh, bool bConvertToMeshdescription = true);
	
	void FillMaterialName(TMap<int32, FName>& OutMaterialMap) const;


#endif // #if WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/*
	 * We used to maintain the UStaticMesh which owned the source model here.
	 * Now this is not necessary as we can get it from the Outer of the StaticMeshDescriptionBulkData.
	 * This means less to maintain
	 */
	UE_DEPRECATED(5.0, "StaticMeshOwner is now deprecated.")
	class UStaticMesh* StaticMeshOwner;

	/**
	 * Bulk data containing mesh description.
	 * New assets store their source data here instead of in the RawMeshBulkData.
	 * If the bulk data within is empty, the LOD is autogenerated (for LOD1+).
	 */
	UPROPERTY()
	TObjectPtr<UStaticMeshDescriptionBulkData> StaticMeshDescriptionBulkData;

	// Only allow safe read of the mesh description from the game thread while a build is running (not sure about the rest. the UStaticMeshDescription stuff look particularly dangerous)
	mutable FCriticalSection StaticMeshDescriptionBulkDataCS;
#endif

#if WITH_EDITOR
	/**
	 * Returns the static mesh which owns this source model.
	 */
	ENGINE_API UStaticMesh* GetStaticMeshOwner() const;

	/**
	 * Verify that this SourceModel has been fully initialized. It may still not contain valid data ( use IsMeshDescriptionValid() for that )
	 * Currently this checks if StaticMeshDescriptionBulkData is allocated, which the other functions below all require via checks.
	 */
	ENGINE_API bool IsSourceModelInitialized() const;

	/**
	 * Create a new MeshDescription object
	 */
	ENGINE_API FMeshDescription* CreateMeshDescription();

	/**
	 * Tries to initialize a MeshDescription into the given FMeshDescription object, returning whether it was successful.
	 * It will try the following steps in order to obtain it:
	 * - if there's valid StaticMeshDescriptionBulkData, unpack it from that
	 * - if there's a valid DDC entry from a previously converted RawMesh, load it and unpack it
	 * - if there's a valid RawMesh, convert it to MeshDescription and return that.
	 * If all the above failed, then return failure.
	 */
	ENGINE_API bool LoadMeshDescription(FMeshDescription& OutMeshDescription) const;

	/**
	 * If there's a valid cached MeshDescription, return a copy of it.
	 * Otherwise unpack the mesh description from bulk data and return the result (without caching it).
	 */
	ENGINE_API bool CloneMeshDescription(FMeshDescription& OutMeshDescription) const;

	/**
	 * Gets the cached mesh description if there was one, otherwise loads, caches and returns it.
	 */
	ENGINE_API FMeshDescription* GetOrCacheMeshDescription();

	/**
	 * Gets the cached mesh description, or nullptr if there wasn't one
	 */
	ENGINE_API FMeshDescription* GetCachedMeshDescription() const;

	/**
	 * Gets the cached StaticMeshDescription if there is one, or nullptr if not.
	 */
	ENGINE_API UStaticMeshDescription* GetCachedStaticMeshDescription() const;

	/**
	 * Gets the mesh description bulk data, or nullptr if not valid 
	 */
	ENGINE_API FMeshDescriptionBulkData* GetMeshDescriptionBulkData() const;

	/**
	 * Returns whether the cached mesh description is valid or not
	 */
	ENGINE_API bool IsMeshDescriptionValid() const;

	/**
	 * Commits the cached mesh description to bulk data
	 */
	ENGINE_API void CommitMeshDescription(bool bUseHashAsGuid);

	/**
	 * Clears the cached mesh description
	 */
	ENGINE_API void ClearMeshDescription();

	ENGINE_API void ResetReductionSetting();
#endif

public:
	/** Settings applied when building the mesh. */
	UPROPERTY(EditAnywhere, Category=BuildSettings)
	FMeshBuildSettings BuildSettings;

	/** Reduction settings to apply when building render data. */
	UPROPERTY(EditAnywhere, Category=ReductionSettings)
	FMeshReductionSettings ReductionSettings; 

	UPROPERTY()
	uint32 CacheMeshDescriptionTrianglesCount = MAX_uint32;
	
	UPROPERTY()
	uint32 CacheMeshDescriptionVerticesCount = MAX_uint32;

	UPROPERTY()
	float LODDistance_DEPRECATED;

	/** 
	 * ScreenSize to display this LOD.
	 * The screen size is based around the projected diameter of the bounding
	 * sphere of the model. i.e. 0.5 means half the screen's maximum dimension.
	 */
	UPROPERTY(EditAnywhere, Category=ReductionSettings)
	FPerPlatformFloat ScreenSize;

	/** The file path that was used to import this LOD. */
	UPROPERTY(VisibleAnywhere, Category = StaticMeshSourceModel, AdvancedDisplay)
	FString SourceImportFilename;

#if WITH_EDITORONLY_DATA
	/** Whether this LOD was imported in the same file as the base mesh. */
	UPROPERTY()
	bool bImportWithBaseMesh;
#endif

	/** Default constructor. */
	ENGINE_API FStaticMeshSourceModel();

	/** Destructor. */
	ENGINE_API ~FStaticMeshSourceModel();

	/** Copying not allowed */
	FStaticMeshSourceModel(const FStaticMeshSourceModel&) = delete;
	FStaticMeshSourceModel& operator=(const FStaticMeshSourceModel&) = delete;

	/** Handle moving gracefully */
	ENGINE_API FStaticMeshSourceModel(FStaticMeshSourceModel&& Other);
	ENGINE_API FStaticMeshSourceModel& operator=(FStaticMeshSourceModel&& Other);

	/** Initialize */
	ENGINE_API void CreateSubObjects(UStaticMesh* Owner);

#if WITH_EDITOR
	/** Serializes bulk data. */
	void SerializeBulkData(FArchive& Ar, UObject* Owner);

	void ConvertRawMesh(int32 LodIndex);
#endif
};


// Make FStaticMeshSourceModel non-assignable
template<> struct TStructOpsTypeTraits<FStaticMeshSourceModel> : public TStructOpsTypeTraitsBase2<FStaticMeshSourceModel>
{
	enum { WithCopy = false };
};
