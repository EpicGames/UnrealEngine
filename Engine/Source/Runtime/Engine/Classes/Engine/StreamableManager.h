// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "IO/IoStoreOnDemand.h"
#include "Math/NumericLimits.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/SourceLocation.h"
#include "Misc/SourceLocationUtils.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "Experimental/StreamableManagerError.h"

#ifndef UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
#define UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING 0
#endif

class IStreamableAsyncLoader;

enum class EStreamableProgressType : uint8
{
	/** Load progress */
	Load,
	/** Progress of downloading packages that were not installed when the request was issued. */
	DownloadRelative,
	/** Progress of the total percentage of installed packages for the request. */
	DownloadAbsolute,
};


/** Defines FStreamableDelegate delegate interface */
using FStreamableDelegate = TDelegate<void()>;
using FStreamableDelegateWithHandle = TDelegate<void(TSharedPtr<struct FStreamableHandle>)>;
using FStreamableUpdateDelegate = TDelegate<void(TSharedRef<struct FStreamableHandle>)>;

/** EXPERIMENTAL - download priority */
using FStreamableDownloadPriority = int16;

namespace UE::StreamableManager::Private
{
	// Default priority for all async loads
	static constexpr TAsyncLoadPriority DefaultAsyncLoadPriority	= 0;
	// Priority to try and load immediately
	static constexpr TAsyncLoadPriority AsyncLoadHighPriority		= 100;

	static constexpr FStreamableDownloadPriority DownloadVeryLowPriority	= -200;
	static constexpr FStreamableDownloadPriority DownloadLowPriority		= -100;
	static constexpr FStreamableDownloadPriority DownloadDefaultPriority	= 0;
	static constexpr FStreamableDownloadPriority DownloadHighPriority		= 100;
	static constexpr FStreamableDownloadPriority DownloadVeryHighPriority	= 200;

	inline FStreamableDelegateWithHandle WrapDelegate(const FStreamableDelegate& InDelegate)
	{
		if (InDelegate.IsBound())
		{
			return FStreamableDelegateWithHandle::CreateLambda(
				[InDelegate](TSharedPtr<FStreamableHandle>)
				{
					// Delegates may get tick delayed so may not be safe to call if bound to a GC'd object
					InDelegate.ExecuteIfBound();
				});
		}

		return FStreamableDelegateWithHandle();
	}

	inline FStreamableDelegateWithHandle WrapDelegate(FStreamableDelegate&& InDelegate)
	{
		if (InDelegate.IsBound())
		{
			return FStreamableDelegateWithHandle::CreateLambda(
				[InDelegate = MoveTemp(InDelegate)](TSharedPtr<FStreamableHandle>)
				{
					// Delegates may get tick delayed so may not be safe to call if bound to a GC'd object
					InDelegate.ExecuteIfBound();
				});
		}

		return FStreamableDelegateWithHandle();
	}

	template<class> inline constexpr bool always_false_v = false;

	struct FDownloadContext;
}

/** Handle that pins downloaded packages in the local cache, may be shared between requests */
struct FStreamableDownloadCachePin
{
	ENGINE_API FStreamableDownloadCachePin();
	ENGINE_API ~FStreamableDownloadCachePin();

	template<typename DebugNameType = UE::FSourceLocation>
	static FStreamableDownloadCachePin Create(DebugNameType&& DebugNameOrLocation)
	{
		FStreamableDownloadCachePin Result;
		if constexpr (std::is_same_v<std::decay_t<DebugNameType>, UE::FSourceLocation>)
		{
			Result.ContentHandle = UE::IoStore::FOnDemandContentHandle::Create(
				UE::SourceLocation::FileAndLine(DebugNameOrLocation).ToString());
		}
		else
		{
			Result.ContentHandle = UE::IoStore::FOnDemandContentHandle::Create(
				Forward<DebugNameType>(DebugNameOrLocation));
		}

		return Result;
	}

	ENGINE_API FString GetDebugName() const;

	ENGINE_API void Release();
	ENGINE_API bool IsValid() const;
	operator bool() const { return IsValid(); }
	ENGINE_API bool operator==(FStreamableDownloadCachePin& Other) const;
private:
	UE::IoStore::FOnDemandContentHandle ContentHandle;
	friend struct UE::StreamableManager::Private::FDownloadContext;
};

#if WITH_EDITOR
enum class ECookLoadType : uint8;
#endif

/** Storage class of the per-class id used in FStreamableHandleContextDataBase */
typedef uint8 TStreamableHandleContextDataTypeID;

enum { TStreamableHandleContextDataTypeIDInvalid = TNumericLimits<TStreamableHandleContextDataTypeID>::Max() };

/** struct to hold a TStreamableHandleContextDataTypeID that is invalid until assigned. */
struct TStreamableHandleContextDataTypeIDStorage
{
private:
	TStreamableHandleContextDataTypeID Value = TStreamableHandleContextDataTypeIDInvalid;
	template <typename SubClassType> friend struct TStreamableHandleContextData;
};

/** Base struct to hold type tag data for TStreamableHandleContextData. */
struct FStreamableHandleContextDataBase : public TSharedFromThis<FStreamableHandleContextDataBase, ESPMode::ThreadSafe>
{
	TStreamableHandleContextDataTypeID GetInstanceTypeId() const
	{
		return InstanceTypeId;
	}
	template <typename SubClassType> bool IsType() const
	{
		return GetInstanceTypeId() == SubClassType::GetClassTypeId();
	}

protected:
	FStreamableHandleContextDataBase(TStreamableHandleContextDataTypeID TypeId)
		: InstanceTypeId(TypeId)
	{}

	ENGINE_API static TStreamableHandleContextDataTypeID AllocateClassTypeId();

private:
	TStreamableHandleContextDataTypeID InstanceTypeId;
};

/**
 * Templated struct that provides type tags (no RTTI) for internal data used on StreamableHandle
 * Type IDs are not stable across multiple processes, so should never be directly serialized or stored.
 */
template <typename SubClassType>
struct TStreamableHandleContextData : public FStreamableHandleContextDataBase
{
	TStreamableHandleContextData()
		: FStreamableHandleContextDataBase(GetClassTypeId())
	{}

	static TStreamableHandleContextDataTypeID GetClassTypeId()
	{
		// Subclasses of TStreamableHandleContextData must implement 
		//     MODULE_API static TStreamableHandleContextDataTypeIDStorage&
		//     TypeIdCrossModuleStorage() { static TStreamableHandleContextDataTypeIDStorage Id; return Id; }
		// With MODULE_API being e.g. ENGINE_API.
		// MODULE_API is required for classes used across modules, to ensure that all modules see the same value.
		TStreamableHandleContextDataTypeIDStorage& ClassTypeId = SubClassType::TypeIdCrossModuleStorage();
		if (ClassTypeId.Value == TStreamableHandleContextDataTypeIDInvalid)
		{
			ClassTypeId.Value = AllocateClassTypeId();
		}
		return ClassTypeId.Value;
	}
};

/** A handle to a synchronous or async load. As long as the handle is Active, loaded assets will stay in memory */
struct FStreamableHandle : public TSharedFromThis<FStreamableHandle>
{
	/** 
	 * If this request has finished loading, meaning all available assets were loaded
	 * Any assets that failed to load will still be null
	 * This can be true before the completion callback has happened as it may be in the delayed callback queue
	 */
	bool HasLoadCompleted() const
	{
		return bLoadCompleted;
	}

	/** If this request was cancelled. Assets may still have been loaded, but completion delegate was not called */
	bool WasCanceled() const
	{
		return bCanceled;
	}

	/** True if load is still ongoing and we haven't been cancelled */
	bool IsLoadingInProgress() const
	{
		return !bLoadCompleted && !bCanceled;
	}

	/** If this handle is still active, meaning it wasn't canceled or released */
	bool IsActive() const
	{
		return !bCanceled && !bReleased;
	}

	/** If this handle is stalled and waiting for another event to occur before it is actually requested */
	bool IsStalled() const
	{
		return bStalled;
	}

	/** Returns true if this is a combined handle that depends on child handles. */
	bool IsCombinedHandle() const
	{
		return bIsCombinedHandle;
	}

	bool IsUsingJustInTimeAsyncLoader()
	{
		return bUseJustInTimeAsyncLoader;
	}

	/** True if an error occured */
	bool HasError() const
	{
		return Error.IsSet();
	}

	/** Get error details */
	const TOptional<UE::UnifiedError::FError>& GetError() const
	{
		return Error;
	}

	/** Get download cache pin for this handle */
	FStreamableDownloadCachePin GetDownloadCachePin() const
	{
		return DownloadCachePin;
	}

	/** Get download cache pins for this handle and all childern */
	TArray<FStreamableDownloadCachePin> GetDownloadCachePins() const;

	/** Returns true if we've done all the loading we can now, ie all handles are either completed or stalled */
	ENGINE_API bool HasLoadCompletedOrStalled() const;

	/** Allows user code to provide a more detailed name in the cases where a name is autogenerated or otherwise default. */
	ENGINE_API void SetDebugNameIfEmptyOrDefault(const FString& NewName);

	/** Returns the debug name for this handle. */
	const FString& GetDebugName() const
	{
		return DebugName;
	}

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FName GetReferencerPackage() const
	{
		return ReferencerPackage;
	}
	FName GetRefencerPackageOp() const
	{
		return ReferencerPackageOp;
	}
#endif
#if WITH_EDITOR
	ECookLoadType GetCookLoadType() const
	{
		return CookLoadType;
	}
#endif

	/** Returns the streaming priority. */
	TAsyncLoadPriority GetPriority() const
	{
		return Priority;
	}

	/**
	 * Set priority (useful to update remaining async loads to a different priority than the handle was created with).
	 * This only takes affect when using just in time async loading.
	 * @see bUseJustInTimeAsyncLoader
	 */
	ENGINE_API void SetPriority(TAsyncLoadPriority InPriority);

	/** Returns download priority. Only valid if a download has been requested and is not yet complete. */
	ENGINE_API TOptional<FStreamableDownloadPriority> GetDownloadPriority() const;

	/** 
	 * Set download priority. Only valid if a download has been requested and is not yet complete. 
	 * @return true if the priority could be set.
	 */
	ENGINE_API bool SetDownloadPriority(FStreamableDownloadPriority Priority);

	/**
	 * Release this handle. This can be called from normal gameplay code to indicate that the loaded assets are no longer needed
	 * This will be called implicitly if all shared pointers to this handle are destroyed
	 * If called before the completion delegate, the release will be delayed until after completion
	 */
	ENGINE_API void ReleaseHandle();

	/**
	 * Cancel a request, callable from within the manager or externally
	 * This will immediately release the handle even if it is still in progress, and call the cancel callback if bound
	 * This stops the completion callback from happening, even if it is in the delayed callback queue
	 */
	ENGINE_API void CancelHandle();

	/** Tells a stalled handle to start its actual request. */
	ENGINE_API void StartStalledHandle();

	/** Check to see whether or not the Complete delegate is bound; useful for doing extra work before/without stomping the existing delegate. */
	ENGINE_API bool HasCompleteDelegate() const;

	/** Check to see whether or not the Cancel delegate is bound; useful for doing extra work before/without stomping the existing delegate. */
	ENGINE_API bool HasCancelDelegate() const;

	/** Check to see whether or not the Update delegate is bound; useful for doing extra work before/without stomping the existing delegate. */
	ENGINE_API bool HasUpdateDelegate() const;

	/** Bind delegate that is called when load completes, only works if loading is in progress. This will overwrite any already bound delegate! */
	ENGINE_API bool BindCompleteDelegate(FStreamableDelegateWithHandle NewDelegate);
	ENGINE_API bool BindCompleteDelegate(FStreamableDelegate NewDelegate);

	/** Bind delegate that is called if handle is canceled, only works if loading is in progress. This will overwrite any already bound delegate! */
	ENGINE_API bool BindCancelDelegate(FStreamableDelegateWithHandle NewDelegate);
	ENGINE_API bool BindCancelDelegate(FStreamableDelegate NewDelegate);

	/** Bind delegate that is called periodically as request updates, only works if loading is in progress. This will overwrite any already bound delegate! */
	ENGINE_API bool BindUpdateDelegate(FStreamableUpdateDelegate NewDelegate);

	/**
	 * Blocks until the requested assets have loaded. This pushes the requested asset to the top of the priority list,
	 * but does not flush all async loading, usually resulting in faster completion than a LoadObject call
	 *
	 * @param Timeout				Maximum time to wait, if this is 0 it will wait forever
	 * @param StartStalledHandles	If true it will force all handles waiting on external resources to try and load right now
	 */
	ENGINE_API EAsyncPackageState::Type WaitUntilComplete(float Timeout = 0.0f, bool bStartStalledHandles = true);

	/** 
	 * Gets list of assets references this load was started with. This will be the paths before redirectors, and not all of these are guaranteed to be loaded 
	 * 
	 * @param AssetList  Output array for requested assets. Array is Reset before adding values.
	 * @param bIncludeChildren Whether the child handles should be included in the search.
	 */
	ENGINE_API void GetRequestedAssets(TArray<FSoftObjectPath>& AssetList, bool bIncludeChildren = true) const;

	/** Adds all loaded assets if load has succeeded. Some entries will be null if loading failed */
	ENGINE_API void GetLoadedAssets(TArray<UObject *>& LoadedAssets) const;

	/** Templated version of above. Entries that fail to cast will also be null. */
	template<class T>
	void GetLoadedAssets(TArray<T*>& LoadedAssets) const
	{
		ForEachLoadedAsset([&LoadedAssets](UObject* LoadedAsset)
		{
			LoadedAssets.Add(Cast<T>(LoadedAsset));
		});
	}

	/** Returns first asset in requested asset list, if it's been successfully loaded. This will fail if the asset failed to load */
	ENGINE_API UObject* GetLoadedAsset() const;

	/** Templated version of above */
	template<class T>
	T* GetLoadedAsset() const
	{
		UObject* LoadedAsset = GetLoadedAsset();
		return Cast<T>(LoadedAsset);
	}

	/** Returns number of assets that have completed loading out of initial list, failed loads will count as loaded */
	ENGINE_API void GetLoadedCount(int32& LoadedCount, int32& RequestedCount) const;

	/**
	 * Invokes a callable for each loaded asset if load has succeeded. Some entries will be null if loading failed
	 *
	 * @param Callable	Callable object
	 */
	template <typename CallableT>
	void ForEachLoadedAsset(CallableT Callable) const;

	/** Returns progress as a value between 0.0 and 1.0. */
	ENGINE_API float GetProgress(EStreamableProgressType Type = EStreamableProgressType::Load) const;

	/** Returns progress as a value between 0.0 and 1.0. */
	float GetLoadProgress() const { return GetProgress(EStreamableProgressType::Load); }

	/** Returns progress as a value between 0.0 and 1.0 representing progress of downloading packages that were not installed when the request was issued. */
	float GetRelativeDownloadProgress() const { return GetProgress(EStreamableProgressType::DownloadRelative); }

	/** Returns progress as a value between 0.0 and 1.0 representing the total percentage of installed packages for the request. */
	float GetAbsoluteDownloadProgress() const { return GetProgress(EStreamableProgressType::DownloadAbsolute); }

	/** Get the StreamableManager for this handle */
	ENGINE_API struct FStreamableManager* GetOwningManager() const;

	/** 
	 * Calls a StreamableDelegate, this will add to the delayed callback queue depending on s.StreamableDelegateDelayFrames
	 *
	 * @param Delegate			Primary delegate to execute
	 * @param AssociatedHandle	Streamable handle associated with this delegate, may be null
	 * @param CancelDelegate	If handle gets cancelled before primary delegate executes, this delegate will be called instead
	 */
	static ENGINE_API void ExecuteDelegate(const FStreamableDelegateWithHandle& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle = nullptr, const FStreamableDelegateWithHandle& CancelDelegate = FStreamableDelegateWithHandle());
	/** Overload taking the Delegate by rvalue. Using this will prevent any copies of FStreamableDelegate from being made if the callback is deferred. */
	static ENGINE_API void ExecuteDelegate(FStreamableDelegateWithHandle&& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle = nullptr, FStreamableDelegateWithHandle&& CancelDelegate = FStreamableDelegateWithHandle());

	/**
	 * Calls a StreamableDelegate, this will add to the delayed callback queue depending on s.StreamableDelegateDelayFrames
	 *
	 * @param Delegate			Primary delegate to execute
	 * @param AssociatedHandle	Streamable handle associated with this delegate, may be null
	 * @param CancelDelegate	If handle gets cancelled before primary delegate executes, this delegate will be called instead
	 */
	static ENGINE_API void ExecuteDelegate(const FStreamableDelegate& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle = nullptr, const FStreamableDelegate& CancelDelegate = FStreamableDelegate());
	/** Overload taking the Delegate by rvalue. Using this will prevent any copies of FStreamableDelegate from being made if the callback is deferred. */
	static ENGINE_API void ExecuteDelegate(FStreamableDelegate&& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle = nullptr, FStreamableDelegate&& CancelDelegate = FStreamableDelegate());

	/**
	 * Return a TSharedPtr of the first handle among this and descendants which satisfies the predicate.
	 * 
	 * @param Predicate	Conditional applied to each candidate handle
	 * @return Either the first matching handle, or nullptr
	 * 
	 * @see GetOutermostHandle
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> FindMatchingHandle(TFunction<bool(const FStreamableHandle&)> Predicate) const;

	/**
	 * Creates a new merged handle which contains this handle as well as any external handles.
	 * No delegates are moved, and the new handle is returned for convenience. Parent handles no longer directly track `this`,
	 * but will be correctly linked with the newly merged handle (and `this` handle is a child of the merged hande).
	 * 
	 * @param OtherHandles	Container of handles which should be merged with this handle
	 * @return Either the new merged handle or nullptr when merging isn't feasible
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> CreateCombinedHandle(const TConstArrayView<TSharedPtr<FStreamableHandle>>& OtherHandles);

	/**
	 * Crawls out to the outermost handle and returns it. Since we could have multiple Parent handles, we might have a tie. 
	 * In cases of a tie, the eldest parent handle (so, lower indices) will win.
	 * 
	 * @return The single parent which is the most outer, or this if no Parent Handle
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> GetOutermostHandle();

	/**
	 * Gives you a mutable reference to a contextual data struct of the specified type.
	 *
	 * @return Either a newly constructed T, or one which was previously added to this handle.
	 */
	template <typename T>
	T& FindOrAddContextData()
	{
		TSharedPtr<T> FoundInternally = FindFirstContextDataOfType<T>();
		if (FoundInternally.IsValid())
		{
			return *FoundInternally.Get();
		}

		TSharedPtr<T> FreshlyAdded = MakeShared<T>();
		AddContextData(FreshlyAdded);

		return *FreshlyAdded.Get();
	}

	/**
	 * Unconditionally adds a contextual data struct to this handle. Uniqueness is not enforced.
	 * 
	 * @see FindFirstContextDataOfType
	 */
	template <typename T>
	void AddContextData(const TSharedPtr<T>& NewData)
	{
		AdditionalContextData.Add(NewData);
	}

	/**
	 * Finds the first instance of a contextual data struct which is attached to this handle.
	 *
	 * @return Either nullptr, or TSharedPtr to the first instance of a T on this handle.
	 * @see AddContextData
	 */
	template <typename T>
	TSharedPtr<T> FindFirstContextDataOfType() const
	{
		for (const TSharedPtr<FStreamableHandleContextDataBase>& InternalPtr : AdditionalContextData)
		{
			if (InternalPtr.IsValid() && InternalPtr->IsType<T>())
			{
				return StaticCastSharedPtr<T>(InternalPtr);
			}
		}
		
		return nullptr;
	}

	/**
	 * Finds all instances of a contextual data struct which are attached to this handle
	 *
	 * @return Array filled with all instances of T currently attached to this handle
	 */
	template <typename T>
	TArray<TSharedPtr<T>> GetContextDataOfType() const
	{
		TArray<TSharedPtr<T>> OfType;

		for (const TSharedPtr<FStreamableHandleContextDataBase>& InternalPtr : AdditionalContextData)
		{
			if (InternalPtr.IsValid() && InternalPtr->IsType<T>())
			{
				OfType.Add(StaticCastSharedPtr<T>(InternalPtr));
			}
		}

		return OfType;
	}

	/** Constructor is private, but this allows use by MakeShared */
private:
	struct FPrivateToken { explicit FPrivateToken() = default; };
	FStreamableHandle() = delete;
public:
	explicit FStreamableHandle(FPrivateToken, UE::FSourceLocation&& InLocation);

	/** Destructor */
	ENGINE_API ~FStreamableHandle();

	/** Not safe to copy or duplicate */
	FStreamableHandle(const FStreamableHandle&) = delete;
	FStreamableHandle& operator=(const FStreamableHandle&) = delete;

	static ENGINE_API const FString HandleDebugName_Preloading;
	static ENGINE_API const FString HandleDebugName_AssetList;
	static ENGINE_API const FString HandleDebugName_CombinedHandle;
	static ENGINE_API const FString HandleDebugName_Error;

private:
	friend struct FStreamableManager;
	friend struct FStreamable;
	friend struct UE::StreamableManager::Private::FDownloadContext;

	/** Called from self as part of SetDebugNameIfEmptyOrDefault */
	bool IsHandleNameEmptyOrDefault() const;

	/** Called from manager to complete the request */
	void CompleteLoad();

	/** Callback when async load finishes, it's here so we can use a shared pointer for callback safety */
	void AsyncLoadCallbackWrapper(const FName& PackageName, UPackage* LevelPackage, EAsyncLoadingResult::Type Result, FSoftObjectPath TargetName);

	/** Notify all parents that a child completed loading */
	void NotifyParentsOfCompletion();

	/** Notify all parents that a child was canceled */
	void NotifyParentsOfCancellation();

	/** Called on meta handle when a child handle has completed/canceled */
	void UpdateCombinedHandle();

	/** Call to call the update delegate if bound, will propagate to parents */
	void CallUpdateDelegate();

	/** Unbind all of our delegates, called after a cancel/load */
	void UnbindDelegates();

	/** Record any errors that occured with this handle */
	void SetError(const UE::UnifiedError::FError& Error);
	void SetErrorOnParents(const UE::UnifiedError::FError& Error);

	/** Delegate to call when streaming is completed */
	FStreamableDelegateWithHandle CompleteDelegate;

	/** Delegate to call when streaming is canceled */
	FStreamableDelegateWithHandle CancelDelegate;

	/** Called periodically during streaming to update progress UI */
	FStreamableUpdateDelegate UpdateDelegate;

	/** Name of this handle, passed in by caller to help in debugging */
	FString DebugName;

	/** Holds a location where this asset list was requested from */
	UE::FSourceLocation Location;

	/** Error details in case of an error */
	TOptional<UE::UnifiedError::FError> Error;

	/** Handle that holds packages in the download cache */
	FStreamableDownloadCachePin DownloadCachePin;

	/** Download book-keeping, released when download finishes */
	TUniquePtr<UE::StreamableManager::Private::FDownloadContext> DownloadContext;

	/** The async priority for this request */
	TAsyncLoadPriority Priority;

	/** How many FStreamables is this waiting on to finish loading */
	int32 StreamablesLoading;

	/** How many of our children that have been completed */
	int32 CompletedChildCount = 0;

	/** How many of our children that have been canceled */
	int32 CanceledChildCount = 0;

	/** List of assets that were referenced by this handle */
	TArray<FSoftObjectPath> RequestedAssets;

	/** List of handles this depends on, these will keep the child references alive */
	TArray<TSharedPtr<FStreamableHandle> > ChildHandles;

	/** Backpointer to handles that depend on this */
	TArray<TWeakPtr<FStreamableHandle> > ParentHandles;

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FName ReferencerPackage;
	FName ReferencerPackageOp;
#endif

	/** This is set at the time of creation, and will be cleared when request completes or is canceled */
	struct FStreamableManager* OwningManager;

	/** Array of contextual data added by game/engine code */
	TArray<TSharedPtr<FStreamableHandleContextDataBase>> AdditionalContextData;

#if WITH_EDITOR
	ECookLoadType CookLoadType;
#endif

	/** True if this request ever started loading from disk. bLoadCompleted may be set without a load starting if cancels or errors occur during download. */
	bool bLoadStarted;

	/** True if this request has finished loading. It may still be active, or it may have been released */
	bool bLoadCompleted;

	/** True if this request was released, which will stop it from keeping hard GC references */
	bool bReleased;

	/** True if this request was explicitly canceled, which stops it from calling the completion delegate and immediately releases it */
	bool bCanceled;

	/** True if this handle has been created but not yet actually requested. This handle is probably waiting for a resource like a chunk to be available */
	bool bStalled;

	/** If true, this handle will be released when it finishes loading */
	bool bReleaseWhenLoaded;

	/** If true, this is a combined handle that depends on child handles. */
	bool bIsCombinedHandle;

	/** If true, use the JIT async loader when queueing requests */
	bool bUseJustInTimeAsyncLoader = false;
};

enum class EStreamableManagerCombinedHandleOptions : uint8 
{
	None = 0,
	/** If present, the DebugName of the merged handle will concatenate DebugName of all merged handles, otherwise it will be set to DebugName argument. */
	MergeDebugNames = 0x01,
	/** If present, existing parent handles will be redirected to point to the merged handle, otherwise they will continue pointing to the child handles directly. */ 
	RedirectParents = 0x02,
	/** If present, nullptr handles are ignored (and will not be present in the merged handle), otherwise merged handle creation will fail when nullptr handles are present. */
	SkipNulls = 0x04,
};

ENUM_CLASS_FLAGS(EStreamableManagerCombinedHandleOptions);

/** EXPERIMENTAL -  Additional parameters for downloading packages */
struct FStreamableDownloadParams
{
	/** Handle to keep packages in the local cache. If no cache pin is provided, a new one will be created. */
	FStreamableDownloadCachePin CachePin;
	/** Download priority */
	FStreamableDownloadPriority Priority = UE::StreamableManager::Private::DownloadDefaultPriority;
	/** If set, any soft references will also be installed */
	bool bInstallSoftReferences = false;
	/** If set, request will complete after download, no assets will be loaded */
	bool bDownloadOnly = false;

	bool bAllowMissingDependencies = false;
};

/** Parameters for an async load */
struct FStreamableAsyncLoadParams
{
	/** Assets to load off disk */
	TArray<FSoftObjectPath> TargetsToStream;
	/** [optional] Delegate to call when load finishes. Will be called on the next tick if asset is already loaded, or many seconds later */
	FStreamableDelegateWithHandle OnComplete;
	/** [optional] Delegate to call when the load is canceled. Will be called on the next tick or many seconds later */
	FStreamableDelegateWithHandle OnCancel;
	/** [optional] delegate that is called periodically as request updates */
	FStreamableUpdateDelegate OnUpdate;
	/** [optional] Priority to pass to the streaming system, higher priority will be loaded first */
	TAsyncLoadPriority Priority = UE::StreamableManager::Private::DefaultAsyncLoadPriority;
	/** [optional] If true, the manager will keep the streamable handle active until explicitly released */
	bool bManageActiveHandle = false;
	/** [optional] If true, the handle will start in a stalled state and will not attempt to actually async load until StartStalledHandle is called on it */
	bool bStartStalled = false;
	/** [optional] If true, async load requests will be trickled to the async loader instead of queueing all at once, while continuing to queue subsequent requests as processed on the AsyncLoadingThread */
	/** Useful for not overloading the async loading queue and if improved control over prioritization/cancellation is needed (remaining requests are queued at updated handle priorities and only if necessary/not cancelled) */
	bool bUseJustInTimeAsyncLoader = false;
	/** [optional] EXPERIMENTAL - If present, will attempt to download necessary packages with IoStoreOnDemand */
	TOptional<FStreamableDownloadParams> DownloadParams;
};

/** A native class for managing streaming assets in and keeping them in memory. AssetManager is the global singleton version of this with blueprint access */
struct FStreamableManager : public FGCObject
{
	// Default priority for all async loads
	static inline constexpr TAsyncLoadPriority DefaultAsyncLoadPriority		= UE::StreamableManager::Private::DefaultAsyncLoadPriority;
	// Priority to try and load immediately
	static inline constexpr TAsyncLoadPriority AsyncLoadHighPriority		= UE::StreamableManager::Private::AsyncLoadHighPriority;

	static inline constexpr FStreamableDownloadPriority DownloadVeryLowPriority		= UE::StreamableManager::Private::DownloadVeryLowPriority;
	static inline constexpr FStreamableDownloadPriority DownloadLowPriority			= UE::StreamableManager::Private::DownloadLowPriority;
	static inline constexpr FStreamableDownloadPriority DownloadDefaultPriority		= UE::StreamableManager::Private::DownloadDefaultPriority;
	static inline constexpr FStreamableDownloadPriority DownloadHighPriority		= UE::StreamableManager::Private::DownloadHighPriority;
	static inline constexpr FStreamableDownloadPriority DownloadVeryHighPriority	= UE::StreamableManager::Private::DownloadVeryHighPriority;

	/** 
	 * This is the primary streamable operation. Requests streaming of one or more target objects. When complete, a delegate function is called. Returns a Streamable Handle.
	 *
	 * @param TargetsToStream		Assets to load off disk
	 * @param DelegateToCall		[optional] Delegate to call when load finishes. Will be called on the next tick if asset is already loaded, or many seconds later
	 * @param Priority				[optional] Priority to pass to the streaming system, higher priority will be loaded first
	 * @param bManageActiveHandle	[optional] If true, the manager will keep the streamable handle active until explicitly released
	 * @param bStartStalled			[optional] If true, the handle will start in a stalled state and will not attempt to actually async load until StartStalledHandle is called on it
	 * @param DebugName				[optional] Name of this handle, either FString or anything that can construct FString
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */
	template< typename PathContainerType = TArray<FSoftObjectPath>, typename FuncType = FStreamableDelegateWithHandle, typename = std::enable_if_t<!std::is_same_v<std::decay_t<PathContainerType>, FStreamableAsyncLoadParams>, void> >
	TSharedPtr<FStreamableHandle> RequestAsyncLoad(
		PathContainerType&& TargetsToStream,
		FuncType&& DelegateToCall = FStreamableDelegateWithHandle(),
		TAsyncLoadPriority Priority = DefaultAsyncLoadPriority,
		bool bManageActiveHandle = false,
		bool bStartStalled = false,
		FString DebugName = FString(),
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * This is the primary streamable operation. Requests streaming of one or more target objects. When complete, a delegate function is called. Returns a Streamable Handle.
	 *
	 * @param Params				Load Parameters
	 * @param DebugName				[optional] Name of this handle, either FString or anything that can construct FString, will be reported in debug tools
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */ 
	ENGINE_API TSharedPtr<FStreamableHandle> RequestAsyncLoad(
		FStreamableAsyncLoadParams&& Params,
		FString DebugName = FString(),
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Synchronously load a set of assets, and return a handle. This can be very slow and may stall the game thread for several seconds.
	 * 
	 * @param TargetsToStream		Assets to load off disk
	 * @param bManageActiveHandle	[optional] If true, the manager will keep the streamable handle active until explicitly released
	 * @param DebugName				[optional] Name of this handle, either FString or anything that can construct FString
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */
	template< typename PathContainerType = TArray<FSoftObjectPath>>
	TSharedPtr<FStreamableHandle> RequestSyncLoad(
		PathContainerType&& TargetsToStream,
		bool bManageActiveHandle = false,
		FString DebugName = FString(),
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Synchronously load the referred asset and return the loaded object, or nullptr if it can't be found. This can be very slow and may stall the game thread for several seconds.
	 * 
	 * @param Target				Specific asset to load off disk
	 * @param bManageActiveHandle	[optional] If true, the manager will keep the streamable handle active until explicitly released
	 * @param RequestHandlePointer	[optional] If non-null, this will set the handle to the handle used to make this request. This useful for later releasing the handle
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */
	ENGINE_API UObject* LoadSynchronous(
		const FSoftObjectPath& Target,
		bool bManageActiveHandle = false,
		TSharedPtr<FStreamableHandle>* RequestHandlePointer = nullptr,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** Typed wrappers */
	template< typename T >
	T* LoadSynchronous(
		const FSoftObjectPath& Target,
		bool bManageActiveHandle = false,
		TSharedPtr<FStreamableHandle>* RequestHandlePointer = nullptr,
		UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		return Cast<T>(LoadSynchronous(Target, bManageActiveHandle, RequestHandlePointer, MoveTemp(Location)) );
	}

	template< typename T >
	T* LoadSynchronous(
		const TSoftObjectPtr<T>& Target,
		bool bManageActiveHandle = false,
		TSharedPtr<FStreamableHandle>* RequestHandlePointer = nullptr,
		UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		return Cast<T>(LoadSynchronous(Target.ToSoftObjectPath(), bManageActiveHandle, RequestHandlePointer, MoveTemp(Location)));
	}

	template< typename T >
	TSubclassOf<T> LoadSynchronous(
		const TSoftClassPtr<T>& Target,
		bool bManageActiveHandle = false,
		TSharedPtr<FStreamableHandle>* RequestHandlePointer = nullptr,
		UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		TSubclassOf<T> ReturnClass;
		ReturnClass = Cast<UClass>(LoadSynchronous(Target.ToSoftObjectPath(), bManageActiveHandle, RequestHandlePointer, MoveTemp(Location)));
		return ReturnClass;
	}

	/** 
	 * Creates a combined handle, which will wait for other handles to complete before completing.
	 * The child handles will be held as hard references as long as this handle is active.
	 * 
	 * @param ChildHandles			List of handles to wrap into this one
	 * @param DebugName				[optional] Name of this handle, will be reported in debug tools
	 * @param Options				[optional] Additional flags that let you specify how the combined handle will be constructed
	 * @param Params				[optional] Load Parameters. TargetsToStream, Priority, bManageActiveHandle, bStartStalled and DownloadParams are ignored
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> CreateCombinedHandle(
		TConstArrayView<TSharedPtr<FStreamableHandle>> ChildHandles,
		FString DebugName = FStreamableHandle::HandleDebugName_CombinedHandle,
		EStreamableManagerCombinedHandleOptions Options = EStreamableManagerCombinedHandleOptions::None,
		FStreamableAsyncLoadParams&& Params = FStreamableAsyncLoadParams{},
		UE::FSourceLocation Location = UE::FSourceLocation::Current()
	);

	/**
	 * Creates a canceled handle with an error set. Does not do any loading. Does not manage the handle.
	 * Useful in situations where a handle needs to be returned with an error without starting an actual load occuring.
	 *
	 * @param Error					The error to set on the handle
	 * @param Params				Load Parameters
	 * @param DownloadParams		Additional Parameters to control downloading
	 * @param DebugName				[optional] Name of this handle, will be reported in debug tools
	 * @param Location				[optional] Is not intended for direct use, the parameter catches call site source location. That location is used for debug tools output.
	 */
	ENGINE_API TSharedPtr<FStreamableHandle> CreateErrorHandle(
		UE::UnifiedError::FError&& Error,
		FStreamableAsyncLoadParams&& Params,
		FString DebugName = FStreamableHandle::HandleDebugName_CombinedHandle,
		UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/** 
	 * Gets list of handles that are directly referencing this asset, returns true if any found.
	 * Combined Handles will not be returned by this function.
	 *
	 * @param Target					Asset to get active handles for 
	 * @param HandleList				Fill in list of active handles
	 * @param bOnlyManagedHandles		If true, only return handles that are managed by this manager, other active handles are skipped
	 */
	ENGINE_API bool GetActiveHandles(const FSoftObjectPath& Target, TArray<TSharedRef<FStreamableHandle>>& HandleList, bool bOnlyManagedHandles = false) const;

	/** Returns true if all pending async loads have finished for all targets */
	ENGINE_API bool AreAllAsyncLoadsComplete() const;

	/** Returns true if all pending async loads have finished for this target */
	ENGINE_API bool IsAsyncLoadComplete(const FSoftObjectPath& Target) const;

	/** This will release any managed active handles pointing to the target soft object path, even if they include other requested assets in the same load */
	ENGINE_API void Unload(const FSoftObjectPath& Target);

	/** Checks for any redirectors that were previously loaded, and returns the redirected target if found. This will not handle redirects that it doesn't yet know about */
	ENGINE_API FSoftObjectPath ResolveRedirects(const FSoftObjectPath& Target) const;

	/** Returns the debug name for this manager */
	ENGINE_API const FString& GetManagerName() const;

	/** Modifies the debug name of this manager, used for debugging GC references */
	ENGINE_API void SetManagerName(FString InName);

	/** Add referenced objects to stop them from GCing */
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return ManagerName; }
	ENGINE_API virtual bool GetReferencerPropertyName(UObject* Object, FString& OutPropertyName) const override;

	ENGINE_API FStreamableManager();
	ENGINE_API ~FStreamableManager();

	/** Not safe to copy or duplicate */
	FStreamableManager(const FStreamableManager&) = delete;
	FStreamableManager& operator=(const FStreamableManager&) = delete;
private:
	friend FStreamableHandle;
	friend UE::StreamableManager::Private::FDownloadContext;

	void RemoveReferencedAsset(const FSoftObjectPath& Target, TSharedRef<FStreamableHandle> Handle);
	void StartHandleRequests(TSharedRef<FStreamableHandle> Handle);
	TArray<int32> GetAsyncLoadRequestIds(TSharedRef<FStreamableHandle> Handle);
	void FindInMemory(FSoftObjectPath& InOutTarget, struct FStreamable* Existing, UPackage* Package = nullptr);
	FSoftObjectPath HandleLoadedRedirector(UObjectRedirector* LoadedRedirector, FSoftObjectPath RequestedPath, struct FStreamable* RequestedStreamable);
	struct FStreamable* FindStreamable(const FSoftObjectPath& Target) const;
	struct FStreamable* StreamInternal(const FSoftObjectPath& Target, TAsyncLoadPriority Priority, TSharedRef<FStreamableHandle> Handle, TSharedRef<IStreamableAsyncLoader> AsyncLoader);

	void CancelJITLoadsRecursively(TSharedRef<FStreamableHandle> Handle);

	ENGINE_API UObject* GetStreamed(const FSoftObjectPath& Target) const;
	void CheckCompletedRequests(const FSoftObjectPath& Target, struct FStreamable* Existing);

	void OnPreGarbageCollect();
	void AsyncLoadCallback(FSoftObjectPath Request, UPackage* Package, const TOptional<UE::UnifiedError::FError> AsyncLoadError);

	ENGINE_API static bool ShouldStripDebugName();
	ENGINE_API TSharedPtr<FStreamableHandle> RequestSyncLoadInternal(TArray<FSoftObjectPath>&& TargetsToStream, bool bManageActiveHandle, FString&& DebugName, UE::FSourceLocation&& Location);

	/** Map of paths to streamable objects, this will be the post-redirector name */
	typedef TMap<FSoftObjectPath, struct FStreamable*> TStreamableMap;
	TStreamableMap StreamableItems;

	/** Map of redirected paths */
	struct FRedirectedPath
	{
		/** The path of the non-redirector object loaded */
		FSoftObjectPath NewPath;

		/** The redirector that was loaded off disk, need to keep this around for path resolves until this redirect is freed */
		TObjectPtr<UObjectRedirector> LoadedRedirector;

		FRedirectedPath() : LoadedRedirector(nullptr) {}
		FRedirectedPath(const FSoftObjectPath& InNewPath, UObjectRedirector* InLoadedRedirector) : NewPath(InNewPath),LoadedRedirector(InLoadedRedirector) {}
	};
	typedef TMap<FSoftObjectPath, FRedirectedPath> TStreamableRedirects;
	TStreamableRedirects StreamableRedirects;

	/** List of explicitly held handles */
	TArray<TSharedRef<FStreamableHandle>> ManagedActiveHandles;

	/** List of combined handles that are still loading, these need to be here to avoid them being deleted */
	TArray<TSharedRef<FStreamableHandle>> PendingCombinedHandles;

	/** If True, temporarily force synchronous loading */
	bool bForceSynchronousLoads;

	/** Debug name of this manager */
	FString ManagerName;
};

template <typename CallableT>
void FStreamableHandle::ForEachLoadedAsset(CallableT Callable) const
{
	if (HasLoadCompleted())
	{
		for (const FSoftObjectPath& Ref : RequestedAssets)
		{
			// Try manager, should be faster and will handle redirects better
			if (IsActive())
			{
				Invoke(Callable, OwningManager->GetStreamed(Ref));
			}
			else
			{
				Invoke(Callable, Ref.ResolveObject());
			}
		}

		// Check child handles
		for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
		{
			for (const FSoftObjectPath& Ref : ChildHandle->RequestedAssets)
			{
				// Try manager, should be faster and will handle redirects better
				if (IsActive())
				{
					Invoke(Callable, OwningManager->GetStreamed(Ref));
				}
				else
				{
					Invoke(Callable, Ref.ResolveObject());
				}
			}
		}
	}
}

template< typename PathContainerType, typename FuncType, typename >
TSharedPtr<FStreamableHandle> FStreamableManager::RequestAsyncLoad(
	PathContainerType&& TargetsToStream,
	FuncType&& Callback,
	TAsyncLoadPriority Priority,
	bool bManageActiveHandle,
	bool bStartStalled,
	FString DebugName,
	UE::FSourceLocation Location)
{
	FStreamableAsyncLoadParams Params;
	Params.TargetsToStream = TArray<FSoftObjectPath>{ Forward<PathContainerType>(TargetsToStream) };

	if constexpr (std::is_constructible_v<FStreamableDelegateWithHandle, FuncType>)
	{
		Params.OnComplete = FStreamableDelegateWithHandle(Forward<FuncType>(Callback));
	}
	else if constexpr (std::is_constructible_v<FStreamableDelegate, FuncType>)
	{
		Params.OnComplete = UE::StreamableManager::Private::WrapDelegate(
			FStreamableDelegate(Forward<FuncType>(Callback)));
	}
	else if constexpr (std::is_invocable_v<FuncType, TSharedPtr<FStreamableHandle>>)
	{
		Params.OnComplete = FStreamableDelegateWithHandle::CreateLambda(Forward<FuncType>(Callback));
	}
	else if constexpr (std::is_invocable_v<FuncType>)
	{
		Params.OnComplete = UE::StreamableManager::Private::WrapDelegate(
			FStreamableDelegate::CreateLambda(Forward<FuncType>(Callback)));
	}
	else
	{
		// Static assertions are evaluated even in non-instantiable static branches unless a dependent type is used. C++23 fixes this.
		static_assert(UE::StreamableManager::Private::always_false_v<FuncType>, "Cannot bind callback");
	}

	Params.Priority = Priority;
	Params.bManageActiveHandle = bManageActiveHandle;
	Params.bStartStalled = bStartStalled;

	return RequestAsyncLoad(
		MoveTemp(Params),
		MoveTemp(DebugName),
		MoveTemp(Location));
}

template< typename PathContainerType>
TSharedPtr<FStreamableHandle> FStreamableManager::RequestSyncLoad(
	PathContainerType&& TargetsToStream,
	bool bManageActiveHandle,
	FString DebugName,
	UE::FSourceLocation Location)
{
	return RequestSyncLoadInternal(
		TArray<FSoftObjectPath>{ Forward<PathContainerType>(TargetsToStream) },
		bManageActiveHandle,
		MoveTemp(DebugName),
		MoveTemp(Location));
}
