// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/App.h"
#include "Engine/TextureStreamingTypes.h"
#include "Serialization/BulkData.h"
#include "Templates/RefCounting.h"
#include "Streaming/SimpleStreamableAssetManagerHandle.h"
#include "Streaming/StreamableRenderResourceState.h"
#include "PerQualityLevelProperties.h"

#include "StreamableRenderAsset.generated.h"

#define STREAMABLERENDERASSET_NODEFAULT(FuncName) LowLevelFatalError(TEXT("UStreamableRenderAsset::%s has no default implementation"), TEXT(#FuncName))
 // Allows yield to lower priority threads
#define RENDER_ASSET_STREAMING_SLEEP_DT (0.010f)

namespace Nanite
{
	class FCoarseMeshStreamingManager;
}

enum class EStreamableRenderAssetType : uint8
{
	None,
	Texture,
	StaticMesh,
	SkeletalMesh,
	LandscapeMeshMobile UE_DEPRECATED(5.1, "LandscapeMeshMobile is now deprecated and will be removed."),
	NaniteCoarseMesh UE_DEPRECATED(5.8, "NaniteCoarseMesh is now deprecated and will be removed."),
};

UCLASS(Abstract, MinimalAPI)
class UStreamableRenderAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Destructor */
	ENGINE_API virtual ~UStreamableRenderAsset();

	/** Get an integer representation of the LOD group */
	virtual int32 GetLODGroupForStreaming() const
	{
		return 0;
	}

	virtual int32 CalcCumulativeLODSize(int32 NumLODs) const
	{
		STREAMABLERENDERASSET_NODEFAULT(CalcCumulativeLODSize);
		return -1;
	}

	virtual FIoFilenameHash GetMipIoFilenameHash(const int32 MipIndex) const
	{
		STREAMABLERENDERASSET_NODEFAULT(GetMipIoFilenameHash);
		return INVALID_IO_FILENAME_HASH;
	}

	virtual bool DoesMipDataExist(const int32 MipIndex) const
	{
		STREAMABLERENDERASSET_NODEFAULT(DoesMipDataExist);
		return false;
	}

	/** Tries to cancel a pending LOD change request. Requests cannot be canceled if they are in the finalization phase. */
	ENGINE_API void CancelPendingStreamingRequest();

	/** Whether a stream in / out request is pending. Only one request can exists at anytime for a streamable asset. This also includes the resource initialization! */
	ENGINE_API bool HasPendingInitOrStreaming() const;

	UE_DEPRECATED(5.8, "HasPendingInitOrStreaming with the optional bool argument is deprecated. bWaitForLODTransition is unused. Remove it from the arguments list.")
	bool HasPendingInitOrStreaming(bool /*bWaitForLODTransition*/) const
	{
		return HasPendingInitOrStreaming();
	}

	/** Whether there is a pending update and it is locked within an update step. Used to prevent dealocks in SuspendRenderAssetStreaming(). */
	bool IsPendingStreamingRequestLocked() const;

	/**
	* Unload some mips from memory. Only usable if the asset is streamable.
	*
	* @param NewMipCount - The desired mip count after the mips are unloaded.
	* @return Whether any mips were requested to be unloaded.
	*/
	virtual bool StreamOut(int32 NewMipCount)
	{
		STREAMABLERENDERASSET_NODEFAULT(StreamOut);
		return false;
	}

	/**
	* Loads mips from disk to memory. Only usable if the asset is streamable.
	*
	* @param NewMipCount - The desired mip count after the mips are loaded.
	* @param bHighPrio   - true if the load request is of high priority and must be issued before other asset requests.
	* @return Whether any mips were resquested to be loaded.
	*/
	virtual bool StreamIn(int32 NewMipCount, bool bHighPrio)
	{
		STREAMABLERENDERASSET_NODEFAULT(StreamIn);
		return false;
	}

	/**
	* Invalidates per-asset last render time. Mainly used to opt in UnknownRefHeuristic
	* during LOD index calculation. See FStreamingRenderAsset::bUseUnknownRefHeuristic
	*/
	virtual void InvalidateLastRenderTimeForStreaming() {}

	/**
	* Get the per-asset last render time. FLT_MAX means never use UnknownRefHeuristic
	* and the asset will only keep non-streamable LODs when there is no instance/reference
	* in the scene
	*/
	virtual float GetLastRenderTimeForStreaming() const { return FLT_MAX; }

	/**
	* Returns whether miplevels should be forced resident.
	*
	* @return true if either transient or serialized override requests miplevels to be resident, false otherwise
	*/
	virtual bool ShouldMipLevelsBeForcedResident() const
	{
		return bGlobalForceMipLevelsToBeResident
			|| bForceMiplevelsToBeResident
			|| ForceMipLevelsToBeResidentTimestamp >= FApp::GetCurrentTime();
	}

	/**
	* Register a callback to get notified when a certain mip or LOD is resident or evicted.
	* @param Component The context component
	* @param LODIdx The mip or LOD level to wait for
	* @param TimeoutSecs Timeout in seconds
	* @param bOnStreamIn Whether to get notified when the specified level is resident or evicted
	* @param Callback The callback to call
	*/
	ENGINE_API void RegisterMipLevelChangeCallback(IPrimitiveComponent* Component, int32 LODIdx, float TimeoutSecs, bool bOnStreamIn, FLODStreamingCallback&& Callback);

	/**
	* Register a set of callbacks to get notified when new mips/LODs start streaming in and when streaming is complete.
	* If the target mips/LODs are already streamed in, CallbackStreamingDone is called immediately and CallbackStreamingStart is not called.
	* @param Component The context component
	* @param TimeoutStartSecs Timeout in seconds for streaming to start
	* @param CallbackStreamingStart The callback to call when the streamer begins changing LOD/mip target. This callback will not be called if the asset is not streamable, or is already streamed in
	* @param TimeoutDoneSecs Timeout in seconds for streaming to be done
	* @param CallbackStreamingDone The callback to call when the desired mip/LOD level has been streamed in or when the timeout period has elapsed. This callback will not be called if the start timeout has expired.
	*/
	ENGINE_API void RegisterMipLevelChangeCallback(IPrimitiveComponent* Component, float TimeoutStartSecs, FLODStreamingCallback&& CallbackStreamingStart, float TimeoutDoneSecs, FLODStreamingCallback&& CallbackStreamingDone);

	/**
	* Remove mip level change callbacks registered by a component.
	* @param Component The context component
	*/
	ENGINE_API void RemoveMipLevelChangeCallback(IPrimitiveComponent* Component);
	ENGINE_API void RemoveMipLevelChangeCallback(UPrimitiveComponent* Component);

	/**
	* Not thread safe and must be called on GT timeline but not necessarily on GT (e.g. ParallelFor called from GT).
	*/
	ENGINE_API void RemoveAllMipLevelChangeCallbacks();

	/**
	* Invoke registered mip level change callbacks.
	* @param DeferredTickCBAssets Non-null if not called on GT. An array that collects assets with CBs that need to be called later on GT
	*/
	ENGINE_API void TickMipLevelChangeCallbacks(TArray<UStreamableRenderAsset*>* DeferredTickCBAssets);

	/**
	* Tells the streaming system that it should force all mip-levels to be resident for a number of seconds.
	* @param Seconds					Duration in seconds
	* @param CinematicTextureGroups	Bitfield indicating which texture groups that use extra high-resolution mips
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API void SetForceMipLevelsToBeResident(float Seconds, int32 CinematicLODGroupMask = 0);

	/**
	* Returns the combined LOD bias based on texture LOD group and LOD bias.
	* Function name is legacy and incorrect, it is no longer cached
	* @return	LOD bias
	*/
	virtual int32 GetCachedLODBias() const 
	{ 
		return 0; 
	}

	/** Return the streaming state of the render resources. Mirrors the state and lowers cache misses. Cleared if there are no resources. */
	inline const FStreamableRenderResourceState& GetStreamableResourceState() const 
	{ 
		return CachedSRRState; 
	}

	/** Whether the current asset render resource is streamable from a gamethread timeline.*/
	inline bool RenderResourceSupportsStreaming() const { return CachedSRRState.bSupportsStreaming && CachedSRRState.MaxNumLODs > CachedSRRState.NumNonStreamingLODs; }

#if !UE_BUILD_SHIPPING
	/** Return the streaming manager's last computed WantedMips for this asset (debug visualization only). */
	uint8 GetCachedNumWantedLODs() const { return CachedNumWantedLODs; }
#endif

	/** Whether the current asset render resource is streamable from a gamethread timeline.*/
	inline bool IsStreamable() const { return StreamingIndex != INDEX_NONE; }

	/** Links texture to the texture streaming manager. */
	ENGINE_API void LinkStreaming();
	/** Unlinks texture from the texture streaming manager. */
	ENGINE_API void UnlinkStreaming();

	/** Whether all miplevels of this texture have been fully streamed in, LOD settings permitting. Note that if optional mips are not mounted it will always return false. */
	ENGINE_API bool IsFullyStreamedIn();

	inline int32 GetStreamingIndex() const { return StreamingIndex; }

	enum class ETickStreamingFlags
	{
		None = 0,
		SendCompletionEvents = 1 << 0
	};
	FRIEND_ENUM_CLASS_FLAGS(ETickStreamingFlags);

	/** Wait for any pending operation (like InitResource and Streaming) to complete. */
	ENGINE_API void WaitForPendingInitOrStreaming(ETickStreamingFlags Flags = ETickStreamingFlags::None);

	/** Wait for any pending operation and make sure that the asset streamer has performed requested ops on this asset. */
	ENGINE_API void WaitForStreaming(ETickStreamingFlags Flags = ETickStreamingFlags::None);

	void TickStreaming(ETickStreamingFlags Flags = ETickStreamingFlags::None, TArray<UStreamableRenderAsset*>* DeferredTickCBAssets = nullptr);

	UE_DEPRECATED(5.8, "WaitForPendingInitOrStreaming with a single optional bool argument is deprecated. bWaitForLODTransition is unused. Call WaitForPendingInitOrStreaming with the optional ETickStreamingFlags argument.")
	void WaitForPendingInitOrStreaming(bool /*bWaitForLODTransition*/)
	{
		WaitForPendingInitOrStreaming(ETickStreamingFlags::None);
	}

	UE_DEPRECATED(5.8, "WaitForPendingInitOrStreaming with two optional bool arguments is deprecated. bWaitForLODTransition is unused. Call WaitForPendingInitOrStreaming with the optional ETickStreamingFlags argument.")
	void WaitForPendingInitOrStreaming(bool /*bWaitForLODTransition*/, bool bSendCompletionEvents)
	{
		WaitForPendingInitOrStreaming(bSendCompletionEvents
			? ETickStreamingFlags::SendCompletionEvents
			: ETickStreamingFlags::None);
	}

	UE_DEPRECATED(5.8, "WaitForStreaming with a single optional bool argument is deprecated. bWaitForLODTransition is unused. Call WaitForStreaming with the optional ETickStreamingFlags argument.")
	void WaitForStreaming(bool /*bWaitForLODTransition*/)
	{
		WaitForStreaming(ETickStreamingFlags::None);
	}

	UE_DEPRECATED(5.8, "WaitForStreaming with two optional bool arguments is deprecated. bWaitForLODTransition is unused. Call WaitForStreaming with the optional ETickStreamingFlags argument.")
	void WaitForStreaming(bool /*bWaitForLODTransition*/, bool bSendCompletionEvents)
	{
		WaitForStreaming(bSendCompletionEvents
			? ETickStreamingFlags::SendCompletionEvents
			: ETickStreamingFlags::None);
	}

	UE_DEPRECATED(5.8, "WaitForStreaming with the optional bSendCompletionEvents bool argument is deprecated. Use the overload of TickStreaming which takes an ETickStreamingFlags argument.")
	void TickStreaming(bool bSendCompletionEvents, TArray<UStreamableRenderAsset*>* DeferredTickCBAssets = nullptr)
	{
		TickStreaming(bSendCompletionEvents
			? ETickStreamingFlags::SendCompletionEvents
			: ETickStreamingFlags::None,
			DeferredTickCBAssets);
	}

	virtual EStreamableRenderAssetType GetRenderAssetType() const { return EStreamableRenderAssetType::None; }

	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;

	const FPerQualityLevelInt& GetNoRefStreamingLODBias() const
	{
		return NoRefStreamingLODBias;
	}

	void SetNoRefStreamingLODBias(FPerQualityLevelInt NewValue)
	{
		NoRefStreamingLODBias = MoveTemp(NewValue);
	}

	ENGINE_API int32 GetCurrentNoRefStreamingLODBias() const;

protected:
	
	// Also returns false if the render resource is non existent, to prevent stalling on an event that will never complete.
	virtual bool HasPendingRenderResourceInitialization() const { return false; }

	UE_DEPRECATED(5.8, "HasPendingLODTransition is deprecated and no longer called. Remove any remaining uses of this function.")
	virtual bool HasPendingLODTransition() const { return false; }

	struct FLODStreamingCallbackPayload
	{
		IPrimitiveComponent* Component;
		double DeadlineStart;
		double DeadlineDone;
		int32 ExpectedResidentMips;
		bool bOnStreamIn;
		bool bIsExpectedResidentMipPayload;
		FLODStreamingCallback CallbackStart;
		FLODStreamingCallback CallbackDone;

		FLODStreamingCallbackPayload(IPrimitiveComponent* InComponent, double InDeadlineDone, int32 ExpectedResidentMips, bool bInOnStreamIn, FLODStreamingCallback&& InCallbackStreamingDone)
			: Component(InComponent)
			, DeadlineStart(InDeadlineDone)
			, DeadlineDone(InDeadlineDone)
			, ExpectedResidentMips(ExpectedResidentMips)
			, bOnStreamIn(bInOnStreamIn)
			, bIsExpectedResidentMipPayload(true)
			, CallbackStart()
			, CallbackDone(MoveTemp(InCallbackStreamingDone))
		{}

		FLODStreamingCallbackPayload(IPrimitiveComponent* InComponent, double InDeadlineStart, FLODStreamingCallback&& InCallbackStreamingStart, double InDeadlineDone, FLODStreamingCallback&& InCallbackStreamingDone)
			: Component(InComponent)
			, DeadlineStart(InDeadlineStart)
			, DeadlineDone(InDeadlineDone)
			, ExpectedResidentMips()
			, bOnStreamIn()
			, bIsExpectedResidentMipPayload(false)
			, CallbackStart(MoveTemp(InCallbackStreamingStart))
			, CallbackDone(MoveTemp(InCallbackStreamingDone))
		{}
	};

	TArray<FLODStreamingCallbackPayload> MipChangeCallbacks;

	/** An update structure used to handle streaming strategy. */
	TRefCountPtr<class FRenderAssetUpdate> PendingUpdate;

	/** WorldSettings timestamp that tells the streamer to force all miplevels to be resident up until that time. */
	UPROPERTY(transient)
	double ForceMipLevelsToBeResidentTimestamp;

public:
	/** Number of mip-levels to use for cinematic quality. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LevelOfDetail, AdvancedDisplay)
	int32 NumCinematicMipLevels;

protected:
	UPROPERTY()
	FPerQualityLevelInt NoRefStreamingLODBias;

	/** FStreamingRenderAsset index used by the texture streaming system. */
	UPROPERTY(transient, duplicatetransient, NonTransactional)
	int32 StreamingIndex = INDEX_NONE;
	
	friend class FAsyncRenderAssetStreamingData;
	friend class FSimpleStreamableAssetManager;

private:
	/** SimpleStreamableAssetManager (SSAM) handle for this asset. Mutated during the asset's
	 *  lifetime (UnregisterAsset) while workers may be concurrently reading it from
	 *  FRegister::FRegister(const FPrimitiveSceneProxy*, ...). Private so every external reader must go 
	 *  through GetSSAMAssetHandle(), which enforces the acquire load via the wrapper's implicit conversion. */
	FSSAMAtomicAssetHandle SSAMAssetHandle;

public:
	/** Thread-safe SSAM handle accessor. Callable from any thread including render and worker threads. */
	FSSAMAssetHandle GetSSAMAssetHandle() const
	{
		return SSAMAssetHandle;
	}
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LevelOfDetail, AssetRegistrySearchable, AdvancedDisplay)
	uint8 NeverStream : 1;

	/** Global and serialized version of ForceMiplevelsToBeResident.				*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = LevelOfDetail, meta = (DisplayName = "Global Force Resident Mip Levels"), AdvancedDisplay)
	uint8 bGlobalForceMipLevelsToBeResident : 1;

	/** Whether some mips might be streamed soon. If false, the texture is not planned resolution will be stable. */
	UPROPERTY(transient, NonTransactional)
	uint8 bHasStreamingUpdatePending : 1;

	/** Override whether to fully stream even if texture hasn't been rendered.	*/
	UPROPERTY(transient)
	uint8 bForceMiplevelsToBeResident : 1;

	/** When forced fully resident, ignores the streaming mip bias used to accommodate memory constraints. */
	UPROPERTY(transient)
	uint8 bIgnoreStreamingMipBias : 1;

	/** Object marked as being part of Editors Pool */
	UPROPERTY(transient)
	uint8 bMarkAsEditorStreamingPool : 1;
	
protected:
	/** Whether to use the extra cinematic quality mip-levels, when we're forcing mip-levels to be resident. */
	UPROPERTY(transient)
	uint8 bUseCinematicMipLevels : 1;

	/** Gamethread coherent state of the resource. Updated whenever graphic resources would from enqueued render commands. */
	FStreamableRenderResourceState CachedSRRState;

#if !UE_BUILD_SHIPPING
	/** Cached copy of the streaming manager's computed WantedMips, mirrored for debug visualization. */
	uint8 CachedNumWantedLODs = 0;
#endif

	friend struct FRenderAssetStreamingManager;
	friend struct FStreamingRenderAsset;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	friend class Nanite::FCoarseMeshStreamingManager;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

ENUM_CLASS_FLAGS(UStreamableRenderAsset::ETickStreamingFlags);
