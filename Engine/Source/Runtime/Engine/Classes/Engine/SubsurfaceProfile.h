// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "RenderResource.h"
#include "SubsurfaceProfile.generated.h"

#define MAX_SUBSURFACE_PROFILE_COUNT 256

UENUM()
enum class ESubsurfaceImplementationTechniqueHint : uint8
{
	/** 
	* Best for high quality like skin. Perform direct subsurface scattering with acceleration from adaptive filtered importance sampling (AFIS). It is expensive but more accurate. 
	* Will fallback to Separable if lower quality (better performance) is preferred (r.SSS.Burley.Quality 0) or force separable in half resolution (r.SSS.HalfRes 1 and r.SSS.HalfRes.ForceSeparable 1) for performance.
	*/
	SIH_AFIS UMETA(DisplayName = "Adaptive Filtered Importance Sampling"),

	/** Best for high performance if large ice/snow covering. Perform subsurface scattering with vertial and horizontal separable filters.*/
	SIH_Separable UMETA(DisplayName = "Separable"),
};

// struct with all the settings we want in USubsurfaceProfile, separate to make it easer to pass this data around in the engine.
USTRUCT(BlueprintType)
struct FSubsurfaceProfileStruct
{
	GENERATED_USTRUCT_BODY()
	
	/**
	* It should match The base color of the corresponding material as much as possible.
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.01", UIMax = "1.0", ClampMax = "1.0", HideAlphaChannel, editcondition = "bEnableBurley"))
	FLinearColor SurfaceAlbedo;

	/**
	* Controls how far light goes into the subsurface in the Red, Green and Blue channel. It is scaled by Mean Free path distance.
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.001", UIMax = "1.0", ClampMax = "1.0", HideAlphaChannel, editcondition = "bEnableBurley"))
	FLinearColor MeanFreePathColor; //MeanFreePathLength;
	
	/**
	* Subsurface mean free path distance in world/unreal units (cm)
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BluePrintReadOnly, meta = (ClampMin = "0.1", UIMax = "50.0", ClampMax = "50.0", editcondition = "bEnableBurley"))
	float MeanFreePathDistance;

	/**
	* Control the scale of world/unreal units (cm)
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.1", UIMax = "50.0", ClampMax = "50.0", editcondition = "bEnableBurley"))
	float WorldUnitScale;

	/**
	* Effective only when Burley subsurface scattering is enabled in cmd.
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (editcondition = "false", EditConditionHides))
	bool  bEnableBurley;

	/**
	 * Switch to use Mean Free Path, otherwise use diffuse mean free path.
	 */
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (editcondition = "false", EditConditionHides))
	bool  bEnableMeanFreePath;

	/**
	* Specifies the how much of the diffuse light gets into the material,
	* can be seen as a per-channel mix factor between the original image,
	* and the SSS-filtered image. It introduces Non-PBR looks.
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (HideAlphaChannel, editcondition = "bEnableBurley"))
	FLinearColor Tint;

	/** in world/unreal units (cm) */
	UPROPERTY(Category = "SubsurfaceProfileStruct", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.1", UIMax = "50.0", ClampMax = "1000.0", DeprecatedProperty, editcondition = "!bEnableBurley", EditConditionHides))
	float ScatterRadius;

	/**
	* Specifies the how much of the diffuse light gets into the material,
	* can be seen as a per-channel mix factor between the original image,
	* and the SSS-filtered image (called "strength" in SeparableSSS, default there: 0.48, 0.41, 0.28)
	*/
	UPROPERTY(Category = "SubsurfaceProfileStruct", EditAnywhere, BlueprintReadOnly, meta = (HideAlphaChannel, DeprecatedProperty, editcondition = "!bEnableBurley", EditConditionHides))
	FLinearColor SubsurfaceColor;

	/**
	* defines the per-channel falloff of the gradients
	* produced by the subsurface scattering events, can be used to fine tune the color of the gradients
	* (called "falloff" in SeparableSSS, default there: 1, 0.37, 0.3)
	*/
	UPROPERTY(Category = "SubsurfaceProfileStruct", EditAnywhere, BlueprintReadOnly, meta = (HideAlphaChannel, DeprecatedProperty, editcondition = "!bEnableBurley", EditConditionHides))
	FLinearColor FalloffColor;

	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly, meta = (HideAlphaChannel))
	FLinearColor BoundaryColorBleed;

	/**
	* This allows users to use mixed implementations for best quality and performance (e.g., High quality for skin with AFIS, and high performance on ice with Separable in the same scene).
	*/
	UPROPERTY(Category = "Burley Normalized", EditAnywhere, BlueprintReadOnly)
	ESubsurfaceImplementationTechniqueHint Implementation;

	/** A value of 1 indicates physically correct extinction. */
	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, AdvancedDisplay, meta = (ClampMin = "0.01", UIMax = "100.0", ClampMax = "100.0", SliderExponent="24"))
	float ExtinctionScale;
	
	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, AdvancedDisplay, meta = (ClampMin = "0.01", UIMax = "0.99", ClampMax = "1.0"))
	float NormalScale;

	/**
	* Scale the transmission in addition to scattering. Majorly used for backward compatability. A value of 1 indicates physically correct scale - No additional scaling.
	*/
	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, AdvancedDisplay, meta = (ClampMin = "0.1", UIMax = "10.0", ClampMax = "10.0"))
	float DistanceScale;

	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.01", UIMax = "0.99", ClampMax = "1.0"))
	float ScatteringDistribution;

	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "1.0", UIMax = "3.0", ClampMax = "3.0"))
	float IOR;

	UPROPERTY(Category = "Dual Specular", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.5", UIMax = "2.0", ClampMax = "2.0"))
	float Roughness0;

	UPROPERTY(Category = "Dual Specular", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.5", UIMax = "2.0", ClampMax = "2.0"))
	float Roughness1;

	UPROPERTY(Category = "Dual Specular", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.1", UIMax = "0.9", ClampMax = "0.9"))
	float LobeMix;

	/**
	* Transmission tint control. It is multiplied on the transmission results. Works only when Burley is enabled.
	*/
	UPROPERTY(Category = "Transmission", EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.001", UIMax = "1.0", ClampMax = "1.0", HideAlphaChannel))
	FLinearColor TransmissionTintColor;

	// constructor
	FSubsurfaceProfileStruct()
	{
		// defaults from SeparableSSS.h and the demo
		ScatterRadius = 1.2f;
		SubsurfaceColor = FLinearColor(0.48f, 0.41f, 0.28f);
		FalloffColor = FLinearColor(1.0f, 0.37f, 0.3f);
		BoundaryColorBleed = FLinearColor::White;
		ExtinctionScale = 1.0f;
		ScatteringDistribution = 0.93f;
		NormalScale = 0.08f;
		IOR = 1.55f;
		Roughness0 = 0.75f;
		Roughness1 = 1.30f;
		LobeMix = 0.85f;
		bEnableBurley = false;
		//match FalloffColor exactly for the default setting
		SurfaceAlbedo = FLinearColor(0.91058f,0.338275f,0.2718f);
		MeanFreePathColor = FLinearColor(1.0f,0.1983f/2.229f,0.1607f/2.229f);
		MeanFreePathDistance = 1.2f*2.229f;
		WorldUnitScale = 0.1f;
		TransmissionTintColor = FLinearColor(1.0f, 1.0f, 1.0f);
		Tint = FLinearColor(1.0f, 1.0f, 1.0f);
		Implementation = ESubsurfaceImplementationTechniqueHint::SIH_AFIS;
		bEnableMeanFreePath = false;
		DistanceScale = 1.0f;
	}

	void Invalidate()
	{
		*this = FSubsurfaceProfileStruct();
	}
};

/**
 * Subsurface Scattering profile asset, can be specified at the material. Only for "Subsurface Profile" materials, is use during Screenspace Subsurface Scattering
 * Don't change at runtime. All properties in here are per material - texture like variations need to come from properties that are in the GBuffer.
 */
UCLASS(autoexpandcategories = SubsurfaceProfile, MinimalAPI)
class USubsurfaceProfile : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = USubsurfaceProfile, EditAnywhere, meta = (ShowOnlyInnerProperties))
	struct FSubsurfaceProfileStruct Settings;

	UPROPERTY()
	FGuid Guid;

	//~ Begin UObject Interface
	virtual void BeginDestroy();
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode);
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif

	// Upgrade parameters from Separable to Burley.
	virtual void PostLoad();
	//~ End UObject Interface
};

// render thread
class FSubsurfaceProfileTexture : public FRenderResource
{
public:
	// constructor
	FSubsurfaceProfileTexture();

	// destructor
	~FSubsurfaceProfileTexture();

	// convenience, can be optimized 
	// @param Profile must not be 0, game thread pointer, do not dereference, only for comparison
	int32 AddOrUpdateProfile(const FSubsurfaceProfileStruct Settings, const USubsurfaceProfile* Profile)
	{
		check(Profile);

		int32 AllocationId = FindAllocationId(Profile); 
		
		if (AllocationId != -1)
		{
			UpdateProfile(AllocationId, Settings);
		}
		else
		{
			AllocationId = AddProfile(Settings, Profile);
		}

		return AllocationId;
	}

	// O(n) n is a small number
	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return AllocationId -1: no allocation, should be deallocated with DeallocateSubsurfaceProfile()
	int32 AddProfile(const FSubsurfaceProfileStruct Settings, const USubsurfaceProfile* InProfile);

	// O(n) to find the element, n is the SSProfile count and usually quite small
	void RemoveProfile(const USubsurfaceProfile* InProfile);

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(const FSubsurfaceProfileStruct Settings, const USubsurfaceProfile* Profile) { int32 AllocationId = FindAllocationId(Profile); UpdateProfile(AllocationId, Settings); }

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(int32 AllocationId, const FSubsurfaceProfileStruct Settings);

	// @return can be 0 if there is no SubsurfaceProfile
	struct IPooledRenderTarget* GetTexture(FRHICommandListImmediate& RHICmdList);
	struct IPooledRenderTarget* GetTexture();

	struct IPooledRenderTarget* GetSSProfilesPreIntegratedTexture(class FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);

	//~ Begin FRenderResource Interface.
	/**
	* Release textures when device is lost/destroyed.
	*/
	virtual void ReleaseRHI() override;

	// for debugging, can be removed
	void Dump();

	// for debugging / VisualizeSSS
	ENGINE_API bool GetEntryString(uint32 Index, FString& Out) const;

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return -1 if not found
	int32 FindAllocationId(const USubsurfaceProfile* InProfile) const;

private:

	struct FSubsurfaceProfileEntry
	{
		// @param InProfile game thread pointer, do not dereference, only for comparison, 0 if the entry can be reused or it's [0] which is used as default
		FSubsurfaceProfileEntry(FSubsurfaceProfileStruct InSubsurfaceProfileStruct, const USubsurfaceProfile* InProfile)
			: Settings(InSubsurfaceProfileStruct)
			, Profile(InProfile)
		{
		}

		FSubsurfaceProfileStruct Settings;

		// 0 if the entry can be reused or it's [0] which is used as default
		// game thread pointer, do not dereference, only for comparison
		const USubsurfaceProfile* Profile;
	};

	//
	TArray<FSubsurfaceProfileEntry> SubsurfaceProfileEntries;

	// Could be optimized but should not happen too often (during level load or editor operations).
	void CreateTexture(FRHICommandListImmediate& RHICmdList);

};

// If you change this you need to recompile the SSS shaders.
// Required if we use a texture format with limited size but want to express a larger radius
static const int32 SUBSURFACE_RADIUS_SCALE = 1024;

 // The kernels range from -3 to 3
static const int32 SUBSURFACE_KERNEL_SIZE = 3;

// lives on the render thread
extern ENGINE_API TGlobalResource<FSubsurfaceProfileTexture> GSubsurfaceProfileTextureObject;

// Initializes or updates the contents of the subsurface profile texture.
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API void UpdateSubsurfaceProfileTexture(class FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);

// Returns the subsurface profile texture if it exists, or null.
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API FRHITexture* GetSubsurfaceProfileTexture();

// Returns the subsurface profile texture if it exists, or black.
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API FRHITexture* GetSubsurfaceProfileTextureWithFallback();

// Returns the Preintegrated texture if it exists, or black.
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API FRHITexture* GetSSProfilesPreIntegratedTextureWithFallback();

// Returns the subsurface profile ID shader parameter name
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API FName GetSubsurfaceProfileParameterName();

// Returns the subsurface profile ID for a given Sub-surface Profile object
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API float GetSubsurfaceProfileId(const USubsurfaceProfile* In);

// Returns the shader parameter name for a Subsurface profile.
UE_DEPRECATED(5.5, "Please, use the name space SubsurfaceProfile::")
ENGINE_API FName CreateSubsurfaceProfileParameterName(const USubsurfaceProfile* InProfile);

namespace SubsurfaceProfile
{
// Initializes or updates the contents of the subsurface profile texture.
ENGINE_API void UpdateSubsurfaceProfileTexture(class FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);

// Returns the subsurface profile texture if it exists, or null.
ENGINE_API FRHITexture* GetSubsurfaceProfileTexture();

// Returns the subsurface profile texture if it exists, or black.
ENGINE_API FRHITexture* GetSubsurfaceProfileTextureWithFallback();

// Returns the Preintegrated texture if it exists, or black.
ENGINE_API FRHITexture* GetSSProfilesPreIntegratedTextureWithFallback();

// Returns the subsurface profile ID shader parameter name
ENGINE_API FName GetSubsurfaceProfileParameterName();

// Returns the subsurface profile ID for a given Sub-surface Profile object
ENGINE_API float GetSubsurfaceProfileId(const USubsurfaceProfile* In);

// Returns the shader parameter name for a Subsurface profile.
ENGINE_API FName CreateSubsurfaceProfileParameterName(const USubsurfaceProfile* InProfile);
};