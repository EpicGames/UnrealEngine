// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"


namespace SubsurfaceScattering
{
	namespace BurleyNormalized
	{

		FORCEINLINE float DiffusionProfile(float RadiusInMm, float A, float S, float L)
		{
			if (L <= 0.0f)
			{
				return 0.0f;
			}

			const float Inv8Pi = 1.0f / (8 * UE_PI);

			float D = 1.0f / S;
			float R = RadiusInMm / L;
			float NegRbyD = -R / D;
			float RrDotR = A * FMath::Max((FMath::Exp(NegRbyD) + FMath::Exp(NegRbyD / 3.0f)) / (D * L) * Inv8Pi, 0.0f);
			return RrDotR;
		}

		FORCEINLINE float TransmissionProfile(float RadiusInMm, float A, float S, float L)
		{
			if (L <= 0.0f)
			{
				return 0.0f;
			}

			//integration from RadiusInMm to infty
			return 0.25f * A * (FMath::Exp(-S * RadiusInMm / L) + 3.0f * FMath::Exp(-S * RadiusInMm / (3.0f * L)));
		}

		FORCEINLINE FVector DiffusionProfile(float RadiusInMm, FLinearColor SurfaceAlbedo, FVector ScalingFactor, FLinearColor DiffuseMeanFreePathInMm)
		{
			return FVector(DiffusionProfile(RadiusInMm, SurfaceAlbedo.R, ScalingFactor.X, DiffuseMeanFreePathInMm.R),
				           DiffusionProfile(RadiusInMm, SurfaceAlbedo.G, ScalingFactor.Y, DiffuseMeanFreePathInMm.G),
				           DiffusionProfile(RadiusInMm, SurfaceAlbedo.B, ScalingFactor.Z, DiffuseMeanFreePathInMm.B));
		}

		FORCEINLINE FVector TransmissionProfile(float RadiusInMm, FLinearColor SurfaceAlbedo, FVector ScalingFactor, FLinearColor DiffuseMeanFreePathInMm)
		{
			return FVector(TransmissionProfile(RadiusInMm, SurfaceAlbedo.R, ScalingFactor.X, DiffuseMeanFreePathInMm.R),
				           TransmissionProfile(RadiusInMm, SurfaceAlbedo.G, ScalingFactor.Y, DiffuseMeanFreePathInMm.G),
				           TransmissionProfile(RadiusInMm, SurfaceAlbedo.B, ScalingFactor.Z, DiffuseMeanFreePathInMm.B));
		}

		//-----------------------------------------------------------------
		// Functions should be identical on both cpu and gpu
		// Method 1: The light directly goes into the volume in a direction perpendicular to the surface.
		// Average relative error: 5.5% (reference to MC)
		FORCEINLINE float GetPerpendicularScalingFactor(float SurfaceAlbedo)
		{
			return 1.85f - SurfaceAlbedo + 7.0f * FMath::Pow(FMath::Abs(SurfaceAlbedo - 0.8f), 3.0f);
		}

		FORCEINLINE FVector GetPerpendicularScalingFactor(FLinearColor SurfaceAlbedo)
		{
			return FVector(GetPerpendicularScalingFactor(SurfaceAlbedo.R),
				           GetPerpendicularScalingFactor(SurfaceAlbedo.G),
				           GetPerpendicularScalingFactor(SurfaceAlbedo.B)
			);
		}

		// Method 2: Ideal diffuse transmission at the surface. More appropriate for rough surface.
		// Average relative error: 3.9% (reference to MC)
		FORCEINLINE float GetDiffuseSurfaceScalingFactor(float SurfaceAlbedo)
		{
			return 1.9f - SurfaceAlbedo + 3.5f * FMath::Pow(SurfaceAlbedo - 0.8f, 2.0f);
		}

		FORCEINLINE FVector GetDiffuseSurfaceScalingFactor(FLinearColor SurfaceAlbedo)
		{
			return FVector(GetDiffuseSurfaceScalingFactor(SurfaceAlbedo.R),
				           GetDiffuseSurfaceScalingFactor(SurfaceAlbedo.G),
				           GetDiffuseSurfaceScalingFactor(SurfaceAlbedo.B)
			);
		}

		// Method 3: The spectral of diffuse mean free path on the surface.
		// Avergate relative error: 7.7% (reference to MC)
		FORCEINLINE float GetSearchLightDiffuseScalingFactor(float SurfaceAlbedo)
		{
			return 3.5f + 100.0f * FMath::Pow(SurfaceAlbedo - 0.33f, 4.0f);
		}

		FORCEINLINE FVector GetSearchLightDiffuseScalingFactor(FLinearColor SurfaceAlbedo)
		{
			return FVector(GetSearchLightDiffuseScalingFactor(SurfaceAlbedo.R),
				           GetSearchLightDiffuseScalingFactor(SurfaceAlbedo.G),
				           GetSearchLightDiffuseScalingFactor(SurfaceAlbedo.B)
			);
		}

		// Match the magic number in BurleyNormalizedSSSCommon.ush
		FORCEINLINE FLinearColor GetMeanFreePathFromDiffuseMeanFreePath(FLinearColor SurfaceAlbedo, FLinearColor DiffuseMeanFreePath, float Dmfp2MfpMagicNumber = 0.6f)
		{
			return DiffuseMeanFreePath * FLinearColor(GetPerpendicularScalingFactor(SurfaceAlbedo) / GetSearchLightDiffuseScalingFactor(SurfaceAlbedo)) * Dmfp2MfpMagicNumber;
		}

		FORCEINLINE FLinearColor GetDiffuseMeanFreePathFromMeanFreePath(FLinearColor SurfaceAlbedo, FLinearColor MeanFreePath, float Dmfp2MfpMagicNumber = 0.6f)
		{
			return MeanFreePath * FLinearColor(GetSearchLightDiffuseScalingFactor(SurfaceAlbedo) / (GetPerpendicularScalingFactor(SurfaceAlbedo) * Dmfp2MfpMagicNumber));
		}
	}
}
