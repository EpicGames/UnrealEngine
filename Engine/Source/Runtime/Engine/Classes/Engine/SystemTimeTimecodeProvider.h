// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimecodeProvider.h"
#include "SystemTimeTimecodeProvider.generated.h"

/**
 * Converts the current system time to timecode, relative to a provided frame rate.
 */
UCLASS(config=Engine, Blueprintable, editinlinenew, MinimalAPI)
class USystemTimeTimecodeProvider : public UTimecodeProvider
{
	GENERATED_BODY()

public:

	/** The frame rate at which the timecode value will be generated. */
	UPROPERTY(EditAnywhere, Category = Timecode)
	FFrameRate FrameRate;

	/** When generating frame time, should we generate full frame without subframe value.*/
	UPROPERTY(EditAnywhere, Category = Timecode)
	bool bGenerateFullFrame;

	/**
	 * Use the high performance clock instead of the system time to generate the timecode value.
	 * Using the high performance clock is faster but will make the value drift over time.
	 */
	UPROPERTY(AdvancedDisplay, EditAnywhere, Category = Timecode)
	bool bUseHighPerformanceClock;

private:

	/** Current state of the provider */
	ETimecodeProviderSynchronizationState State;

public:

	ENGINE_API USystemTimeTimecodeProvider();

	/** Generate a frame time value, including subframe, using the system clock. */
	static ENGINE_API FFrameTime GenerateFrameTimeFromSystemTime(FFrameRate Rate);

	/** Generate a timecode value using the system clock. */
	static ENGINE_API FTimecode GenerateTimecodeFromSystemTime(FFrameRate Rate);

	/**
	 * Generate a frame time value, including subframe, using the high performance clock
	 * Using the high performance clock is faster but will make the value drift over time.
	 * This is an optimized version. Prefer GenerateTimecodeFromSystemTime, if the value need to be accurate.
	 **/
	static ENGINE_API FFrameTime GenerateFrameTimeFromHighPerformanceClock(FFrameRate Rate);

	/**
	 * Generate a timecode value using the high performance clock
	 * Using the high performance clock is faster but will make the value drift over time.
	 * This is an optimized version. Prefer GenerateTimecodeFromSystemTime, if the value need to be accurate.
	 **/
	static ENGINE_API FTimecode GenerateTimecodeFromHighPerformanceClock(FFrameRate Rate);

	//~ Begin UTimecodeProvider Interface
	ENGINE_API virtual FQualifiedFrameTime GetQualifiedFrameTime() const override;
	
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override
	{
		return State;
	}

	virtual bool Initialize(class UEngine* InEngine) override
	{
		State = ETimecodeProviderSynchronizationState::Synchronized;
		return true;
	}

	virtual void Shutdown(class UEngine* InEngine) override
	{
		State = ETimecodeProviderSynchronizationState::Closed;
	}
	//~ End UTimecodeProvider Interface
};
