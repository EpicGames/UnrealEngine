// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "TargetPoint.generated.h"

UCLASS(MinimalAPI)
class ATargetPoint : public AActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Display, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UBillboardComponent> SpriteComponent;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Display, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UArrowComponent> ArrowComponent;

public:
	/** Returns SpriteComponent subobject **/
	ENGINE_API class UBillboardComponent* GetSpriteComponent() const;
	/** Returns ArrowComponent subobject **/
	ENGINE_API class UArrowComponent* GetArrowComponent() const;
#endif
};



