// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * To render text in the 3d world (quads with UV assignment to render text with material support).
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "TextRenderActor.generated.h"

class UBillboardComponent;

UCLASS(MinimalAPI, ComponentWrapperClass, hideCategories = (Collision, Attachment))
class ATextRenderActor : public AActor
{
	GENERATED_UCLASS_BODY()

	friend class UActorFactoryTextRender;

private:
	/** Component to render a text in 3d with a font */
	UPROPERTY(Category = TextRenderActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Rendering|Components|TextRender", AllowPrivateAccess = "true"))
	TObjectPtr<class UTextRenderComponent> TextRender;

#if WITH_EDITORONLY_DATA
	// Reference to the billboard component
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif

public:
	/** Returns TextRender subobject **/
	class UTextRenderComponent* GetTextRender() const { return TextRender; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};



