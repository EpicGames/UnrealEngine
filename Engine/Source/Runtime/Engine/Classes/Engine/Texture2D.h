// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "ImageCoreBP.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Engine/Texture.h"
#include "Engine/TextureAllMipDataProviderFactory.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Serialization/BulkData.h"
#endif

#include "Texture2D.generated.h"

class FTexture2DResourceMem;
class FTexture2DResource;
struct FTexture2DMipMap;
struct FUpdateTextureRegion2D;

UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class UTexture2D : public UTexture
{
	GENERATED_UCLASS_BODY()

public:

	/** keep track of first mip level used for ResourceMem creation */
	UPROPERTY()
	int32 FirstResourceMemMip;

public:
	/**
	 * Retrieves the size of the source image from which the texture was created.
	 */
#if WITH_EDITOR
	ENGINE_API FIntPoint GetImportedSize() const;
#else // #if WITH_EDITOR
	inline FIntPoint GetImportedSize() const
	{
		return ImportedSize;
	}
#endif // #if WITH_EDITOR

private:
	/** True if streaming is temporarily disabled so we can update subregions of this texture's resource 
	without streaming clobbering it. Automatically cleared before saving. */
	UPROPERTY(transient)
	uint8 bTemporarilyDisableStreaming:1;

public:
#if WITH_EDITORONLY_DATA
	/** Whether the texture has been painted in the editor.						*/
	UPROPERTY()
	uint8 bHasBeenPaintedInEditor:1;
#endif // WITH_EDITORONLY_DATA

	/** The addressing mode to use for the X axis.								*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Texture, meta=(DisplayName="X-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis.								*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Texture, meta=(DisplayName="Y-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressY;

private:
	/**
	 * The imported size of the texture. Only valid on cooked builds when texture source is not
	 * available. Access ONLY via the GetImportedSize() accessor!
	 */
	UPROPERTY()
	FIntPoint ImportedSize;

	/** The derived data for this texture on this platform. */
	FTexturePlatformData* PrivatePlatformData;

public:

	/** Set the derived data for this texture on this platform. */
	ENGINE_API void SetPlatformData(FTexturePlatformData* PlatformData);
	/** Get the derived data for this texture on this platform. */
	ENGINE_API FTexturePlatformData* GetPlatformData();
	/** Get the const derived data for this texture on this platform. */
	ENGINE_API const FTexturePlatformData* GetPlatformData() const;

	UTextureAllMipDataProviderFactory* GetAllMipProvider() const
	{
		return const_cast<UTexture2D*>(this)->GetAssetUserData<UTextureAllMipDataProviderFactory>();
	}

#if WITH_EDITOR
	/* cooked platform data for this texture */
	TMap<FString, FTexturePlatformData*> CookedPlatformData;
#endif

	/** memory used for directly loading bulk mip data */
	FTexture2DResourceMem*		ResourceMem;

protected:

	/** Helper to manage the current pending update following a call to StreamIn() or StreamOut(). */
	friend class FTexture2DUpdate;

public:

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;	
#if WITH_EDITOR
	virtual void PostLinkerChange() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsDefaultTexture() const override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;
	virtual bool IsReadyForAsyncPostLoad() const override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
#if WITH_EDITOR
	virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif
	virtual FString GetDesc() override;
	//~ End UObject Interface.

	//~ Begin UTexture Interface.
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::TwoD; }
	virtual float GetSurfaceWidth() const override { return static_cast<float>(GetSizeX()); }
	virtual float GetSurfaceHeight() const override { return static_cast<float>(GetSizeY()); }
	virtual float GetSurfaceDepth() const override { return 0.0f; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual TextureAddress GetTextureAddressX() const override { return AddressX; }
	virtual TextureAddress GetTextureAddressY() const override { return AddressY; }
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override;
	virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual float GetAverageBrightness(bool bIgnoreTrueBlack, bool bUseGrayscale) override;
	virtual FTexturePlatformData** GetRunningPlatformData() final override;
#if WITH_EDITOR
	virtual TMap<FString,FTexturePlatformData*>* GetCookedPlatformData() override { return &CookedPlatformData; }
#endif
	//~ End UTexture Interface.

	//~ Begin UStreamableRenderAsset Interface
	virtual int32 CalcCumulativeLODSize(int32 NumLODs) const final override { return CalcTextureMemorySize(NumLODs); }
	ENGINE_API int32 GetNumResidentMips() const;
	ENGINE_API virtual bool StreamOut(int32 NewMipCount) final override;
	ENGINE_API virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) final override;
	//~ End UStreamableRenderAsset Interface

	/** Trivial accessors. */
	ENGINE_API int32 GetSizeX() const;
	ENGINE_API int32 GetSizeY() const;
	ENGINE_API int32 GetNumMips() const;
	ENGINE_API EPixelFormat GetPixelFormat(uint32 LayerIndex = 0u) const;
	ENGINE_API int32 GetMipTailBaseIndex() const;
	ENGINE_API const TIndirectArray<FTexture2DMipMap>& GetPlatformMips() const;
	ENGINE_API int32 GetExtData() const;



	/**
	 * Calculates the maximum number of mips that will be in this texture after cooking
	 *   (eg. after the "drop mip" lod bias is applied).
	 * This function is not correct and should not be used. 
	 *
	 * The cinematic mips will be considered as loadable, streaming enabled or not.
	 * Note that in the cooking process, mips smaller than the min residency count
	 * can be stripped out by the cooker.
	 *
	 * @param bIgnoreMinResidency - Whether to ignore min residency limitations.
	 * @return The maximum allowed number mips for this texture.
	 */
	ENGINE_API int32 GetNumMipsAllowed(bool bIgnoreMinResidency) const;

public:
	/** Returns the minimum number of mips that must be resident in memory (cannot be streamed). 
	This does not correctly account for NonStreaming mips and other constraints.
	This function is not correct and should not be used. */
	inline int32 GetMinTextureResidentMipCount() const
	{
		return FMath::Max(GMinTextureResidentMipCount, GetPlatformData() ? (int32)GetPlatformData()->GetNumMipsInTail() : 0);
	}

	/**
	 * Get the PlatformData mip data starting with the specified mip index.
	 *
	 * @param FirstMipToLoad - The first mip index to cache.
	 * @param OutMipData -	Must point to an array of pointers with at least
	 *						Mips.Num() - FirstMipToLoad + 1 entries. Upon
	 *						return those pointers will contain mip data.
	 *
	 * prefer TryLoadMipsWithSizes
	 *
	 * todo: deprecate this API when possible
	 * this API is also duplicated in Texture2DArray and TextureCube
	 * unify and fix them all
	 * also don't use "GetMipData" as that is the name used for TextureSource
	 */
	ENGINE_API void GetMipData(int32 FirstMipToLoad, void** OutMipData);

	/**
	 * Retrieve initial texel data for mips, starting from FirstMipToLoad, up to the last mip in the texture.
	 *    This function will only return mips that are currently loaded, or available via derived data.
	 *    THIS FUNCTION WILL FAIL if you include in the requested range a streaming mip that is NOT currently loaded.
	 *    Also note that ALL mip bulk data buffers are discarded (either returned or freed).
	 *    So... in short, this function should only ever be called once.
	 *
	 * @param FirstMipToLoad - The first mip index to load.
	 * @param OutMipData -	A pre-allocated array of pointers, that correspond to [FirstMipToLoad, .... LastMip]
	 *						Upon successful return, each of those pointers will point to allocated memory containing the corresponding mip's data.
	 *						Caller takes responsibility to free that memory.
	 * @param OutMipSize -  A pre-allocated array of int64, that should be the same size as OutMipData (or zero size if caller does not require the sizes to be returned)
	 *						Upon successful return, each element contains the size of the corresponding mip's data buffer.
	 * @returns true if the requested mip data has been successfully returned.
	 */
	virtual bool GetInitialMipData(int32 InFirstMipToLoad, TArrayView<void*> OutMipData, TArrayView<int64> OutMipSize);

	/**
	 * Calculates the size of this texture in bytes if it had MipCount miplevels streamed in.
	 *
	 * @param	MipCount	Number of mips to calculate size for, counting from the smallest 1x1 mip-level and up.
	 * @return	Size of MipCount mips in bytes
	 */
	int32 CalcTextureMemorySize( int32 MipCount ) const;

	/**
	 * Calculates the size of this texture if it had MipCount miplevels streamed in.
	 *
	 * @param	Enum	Which mips to calculate size for.
	 * @return	Total size of all specified mips, in bytes
	 */
	virtual uint32 CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const override;

	/**
	 *	Get the CRC of the source art pixels.
	 *
	 *	@param	[out]	OutSourceCRC		The CRC value of the source art pixels.
	 *
	 *	@return			bool				true if successful, false if failed (or no source art)
	 */
	ENGINE_API bool GetSourceArtCRC(uint32& OutSourceCRC);

	/**
	 *	See if the source art of the two textures matches...
	 *
	 *	@param		InTexture		The texture to compare it to
	 *
	 *	@return		bool			true if they matche, false if not
	 */
	ENGINE_API bool HasSameSourceArt(UTexture2D* InTexture);
	
	/**
	 * Returns true if the runtime texture has an alpha channel that is not completely white.
	 */
	ENGINE_API bool HasAlphaChannel() const;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @return size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	/**
	 * Returns the global mip map bias applied as an offset for 2d textures.
	 */
	ENGINE_API static float GetGlobalMipMapLODBias();

	/**
	 * Calculates and returns the corresponding ResourceMem parameters for this texture.
	 *
	 * @param FirstMipIdx		Index of the largest mip-level stored within a seekfree (level) package
	 * @param OutSizeX			[out] Width of the stored largest mip-level
	 * @param OutSizeY			[out] Height of the stored largest mip-level
	 * @param OutNumMips		[out] Number of stored mips
	 * @param OutTexCreateFlags	[out] ETextureCreateFlags bit flags
	 * @return					true if the texture should use a ResourceMem. If false, none of the out parameters will be filled in.
	 */
	bool GetResourceMemSettings(int32 FirstMipIdx, int32& OutSizeX, int32& OutSizeY, int32& OutNumMips, uint32& OutTexCreateFlags);

	/**
	*	Asynchronously update a set of regions of a texture with new data.
	*	@param MipIndex - the mip number to update
	*	@param NumRegions - number of regions to update
	*	@param Regions - regions to update
	*	@param SrcPitch - the pitch of the source data in bytes
	*	@param SrcBpp - the size one pixel data in bytes
	*	@param SrcData - the source data
	*	@param DataCleanupFunc - a function that will be called to clean up the SrcData and Regions after the update.
	*/
	ENGINE_API void UpdateTextureRegions(int32 MipIndex, uint32 NumRegions, const FUpdateTextureRegion2D* Regions, uint32 SrcPitch, uint32 SrcBpp, uint8* SrcData, TFunction<void(uint8* SrcData, const FUpdateTextureRegion2D* Regions)> DataCleanupFunc = [](uint8*, const FUpdateTextureRegion2D*){});

#if WITH_EDITOR
	
	/**
	 * Temporarily disable streaming so we update subregions of this texture without streaming clobbering it. 
	 */
	ENGINE_API void TemporarilyDisableStreaming();

	/** Called after an editor or undo operation is formed on texture
	*/
	virtual void PostEditUndo() override;

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** If we ever show the CPU accessible image in the editor we'll need a transient texture with
	* those bits in it. If we don't have a CPU copy, then this is null. Use the GetCPUCopyTexture to
	* access as it's created on demand.
	*/
	UPROPERTY();
	TObjectPtr<UTexture2D> CPUCopyTexture;

	/**
	* Creates and returns a 2d texture that holds the CPU copy. This is just so that we have something to show
	* in the editor for the cpu texture and should never be used at runtime or for game data. This function
	* stalls while the texture is encoding due to GetPlatformData, and will return nullptr if the texture
	* is not cpu accessible.
	* 
	* For game code access to the cpu copy image, use GetCPUCopy().
	*/
	ENGINE_API UTexture2D* GetCPUCopyTexture();

	/**
	 * Returns true if the Downscale and DownscaleOptions properties should be editable in the UI.
	 * Currently downscaling is only supported for 2d textures without mipmaps.
	 */
	virtual bool AreDownscalePropertiesEditable() const override { return MipGenSettings == TMGS_NoMipmaps || MipGenSettings == TMGS_FromTextureGroup; }
#endif

	friend struct FRenderAssetStreamingManager;
	friend struct FStreamingRenderAsset;
	
	/** creates and initializes a new Texture2D with the requested settings. The texture will have 1 mip level of the given size, optionally filled with the provided data. */
	ENGINE_API static class UTexture2D* CreateTransient(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat = PF_B8G8R8A8, const FName InName = NAME_None, TConstArrayView64<uint8> InImageData = TConstArrayView64<uint8>());

	/** creates a new texture2d from the first slice of the given image. If the image format isn't supported a texture isn't created. */
	ENGINE_API static class UTexture2D* CreateTransientFromImage(const FImage* InImage, const FName InName = NAME_None);

	/**
	* If the texture has a cpu accessible copy, this returns that copy. It will wait for encoding in the editor,
	* but otherwise may return an invalid reference if the texture is not ready or doesn't have a cpu copy.
	*/
	ENGINE_API FSharedImageConstRef GetCPUCopy() const;

	UFUNCTION(BlueprintCallable, meta=(DisplayName = "GetCPUCopy"), Category = "Rendering|Texture")
	FSharedImageConstRefBlueprint Blueprint_GetCPUCopy() const;


	/**
	 * Gets the X size of the texture, in pixels
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "GetSizeX"), Category="Rendering|Texture")
	int32 Blueprint_GetSizeX() const;

	/**
	 * Gets the Y size of the texture, in pixels
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "GetSizeY"), Category="Rendering|Texture")
	int32 Blueprint_GetSizeY() const;

	/**
	 * Update the offset for mip map lod bias.
	 * This is added to any existing mip bias values.
	 */
	virtual void RefreshSamplerStates();

	/**
	 * Returns if the texture is actually being rendered using virtual texturing right now.
	 * Unlike the 'VirtualTextureStreaming' property which reflects the user's desired state
	 * this reflects the actual current state on the renderer depending on the platform, VT
	 * data being built, project settings, ....
	 */
	virtual bool IsCurrentlyVirtualTextured() const override;

	/**
	 * Returns true if this virtual texture requests round-robin updates of the virtual texture pages. 
	 */
	virtual bool IsVirtualTexturedWithContinuousUpdate() const { return false; }

	/**
	 * Returns true if this virtual texture uses a single physical space all of its texture layers.
	 * This can reduce page table overhead but potentially increase the number of physical pools allocated.
	 */
	virtual bool IsVirtualTexturedWithSinglePhysicalSpace() const { return false;  }

	/**
	 * Returns true if this virtual texture requires that its physical pool is never split by the r.VT.PageTableMode=2 setting.
	 * This is necessary if the texture is part of a system that assumes that all textures in the system are in the same physical pool.
	 * Note that textures with this setting will necessarily be put in different physical spaces to those without the setting.
	 */
	virtual bool IsVirtualTexturedWithSinglePhysicalPool() const { return false; }
};
