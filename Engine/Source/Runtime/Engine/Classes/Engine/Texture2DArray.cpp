// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2DArray.cpp: UTexture2DArray implementation.
=============================================================================*/

#include "Engine/Texture2DArray.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "ImageUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderUtils.h"
#include "Rendering/Texture2DArrayResource.h"
#include "Streaming/Texture2DArrayStreaming.h"
#include "Streaming/Texture2DMipDataProvider_DDC.h"
#include "Streaming/TextureStreamIn.h"
#include "Streaming/Texture2DMipDataProvider_IO.h"
#include "Streaming/TextureStreamOut.h"
#include "TextureCompiler.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "ImageCoreUtils.h"

#if WITH_EDITOR
#include "AsyncCompilationHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Texture2DArray)

#define LOCTEXT_NAMESPACE "UTexture2DArray"

// Externed global switch to control whether streaming is enabled for texture2darray. 
bool GSupportsTexture2DArrayStreaming = true;
static FAutoConsoleVariableRef CVarSupportsTexture2DArrayStreaming(
	TEXT("r.SupportsTexture2DArrayStreaming"),
	GSupportsTexture2DArrayStreaming,
	TEXT("Enable Support of Texture2DArray Streaming\n")
);

static TAutoConsoleVariable<int32> CVarAllowTexture2DArrayAssetCreation(
	TEXT("r.AllowTexture2DArrayCreation"),
	1,
	TEXT("Enable UTexture2DArray assets"),
	ECVF_Default
);

UTexture2DArray* UTexture2DArray::CreateTransient(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat, const FName InName)
{
	UTexture2DArray* NewTexture = nullptr;
	if (InSizeX > 0 && InSizeY > 0 && InArraySize > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0)
	{
		NewTexture = NewObject<UTexture2DArray>(
			GetTransientPackage(),
			InName,
			RF_Transient
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->SetNumSlices(InArraySize);
		NewTexture->GetPlatformData()->PixelFormat = InFormat;
		NewTexture->bNotOfflineProcessed = true;

		// Allocate first mipmap.
		int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY, InArraySize);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc((int64)GPixelFormats[InFormat].BlockBytes * NumBlocksX * NumBlocksY * InArraySize);
		Mip->BulkData.Unlock();
	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UTexture2DArray::CreateTransient()");
	}
	return NewTexture;
}

/**
 * Get the optimal placeholder to use during texture compilation
 */
static UTexture2DArray* GetDefaultTexture2DArray(const UTexture2DArray* Texture)
{
	static TStrongObjectPtr<UTexture2DArray> CheckerboardTexture;
	if (!CheckerboardTexture.IsValid())
	{
		CheckerboardTexture.Reset(FImageUtils::CreateCheckerboardTexture2DArray(FColor(200, 200, 200, 128), FColor(128, 128, 128, 128)));
	}
	return CheckerboardTexture.Get();
}

UTexture2DArray::UTexture2DArray(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PrivatePlatformData(nullptr)
{
#if WITH_EDITORONLY_DATA
	SRGB = true;
	MipGenSettings = TMGS_NoMipmaps;
#endif
}

FTexturePlatformData** UTexture2DArray::GetRunningPlatformData()
{
	// @todo DC GetRunningPlatformData is fundamentally unsafe but almost unused... should we replace it with Get/SetRunningPlatformData directly in the base class
	return &PrivatePlatformData;
}

void UTexture2DArray::SetPlatformData(FTexturePlatformData* InPlatformData)
{
	if (PrivatePlatformData)
	{
		ReleaseResource();
		delete PrivatePlatformData;
	}

	PrivatePlatformData = InPlatformData;
}

// Any direct access to GetPlatformData() will stall until the structure
// is safe to use. It is advisable to replace those use case with
// async aware code to avoid stalls where possible.
const FTexturePlatformData* UTexture2DArray::GetPlatformData() const
{
#if WITH_EDITOR
	if (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTexture2DArray::GetPlatformDataStall);
		UE_LOGF(LogTexture, Log, "Call to GetPlatformData() is forcing a wait on data that is not yet ready.");

		FText Msg = FText::Format(LOCTEXT("WaitOnTextureCompilation", "Waiting on texture compilation {0} ..."), FText::FromString(GetName()));
		FScopedSlowTask Progress(1.f, Msg, true);
		Progress.MakeDialog(true);
		uint64 StartTime = FPlatformTime::Cycles64();
		PrivatePlatformData->FinishCache();
		AsyncCompilationHelpers::SaveStallStack(FPlatformTime::Cycles64() - StartTime);
	}
#endif // #if WITH_EDITOR

	return PrivatePlatformData;
}

FTexturePlatformData* UTexture2DArray::GetPlatformData()
{
	// For now, this is the same implementation as the const version.
	const UTexture2DArray* ConstThis = this;
	return const_cast<FTexturePlatformData*>(ConstThis->GetPlatformData());
}

FTextureResource* UTexture2DArray::CreateResource()
{
#if WITH_EDITOR
	if (PrivatePlatformData)
	{
		if (PrivatePlatformData->IsAsyncWorkComplete())
		{
			// Make sure AsyncData has been destroyed in case it still exists to avoid
			// IsDefaultTexture thinking platform data is still being computed.
			PrivatePlatformData->FinishCache();
		}
		else
		{
			FTextureCompilingManager::Get().AddTextures({ this });
			UnlinkStreaming();
			return new FTexture2DArrayResource(this, (const FTexture2DArrayResource*)GetDefaultTexture2DArray(this)->GetResource());
		}
	}
#endif

	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat()];
	if (GetNumMips() > 0 && FormatInfo.Supported)
	{
		return new FTexture2DArrayResource(this, GetResourcePostInitState(GetPlatformData(), GSupportsTexture2DArrayStreaming));
	}
#if WITH_EDITORONLY_DATA
	else if (!SourceTextures.Num())
	{
		// empty arrays don't have built mips
		return nullptr;
	}
#endif
	else if (GetNumMips() == 0)
	{
		UE_LOGF(LogTexture, Warning, "%ls contains no miplevels! Please delete.", *GetFullName());
	}
	else if (!FormatInfo.Supported)
	{
		UE_LOGF(LogTexture, Warning, "%ls cannot be created, rhi does not support format %ls.", *GetFullName(), FormatInfo.Name);
	}
	return nullptr;
}

void UTexture2DArray::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
#if WITH_EDITOR
	bool bAsyncFlags = false;
	// Recache platform data if the source has changed.
	if (!EnumHasAnyFlags(InFlags, EUpdateResourceFlags::Synchronous) &&
		FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		bAsyncFlags = true;
	}

	CachePlatformData(bAsyncFlags, bAsyncFlags, bAsyncFlags, nullptr, EnumHasAnyFlags(InFlags, EUpdateResourceFlags::ForceRebuild));
#endif // #if WITH_EDITOR
	
#if WITH_EDITORONLY_DATA
	bSourceGeneratedFromSourceTexturesArray = !Source.GetNumSlices() || SourceTextures.Num();
#endif

	Super::UpdateResourceWithParams(InFlags);
}

uint32 UTexture2DArray::CalcTextureMemorySize(int32 MipCount) const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTexture2DArray(this)->CalcTextureMemorySize(MipCount);
	}
#endif

	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat()];

	uint32 Size = 0;
	if (FormatInfo.Supported && GetPlatformData())
	{
		const EPixelFormat Format = GetPixelFormat();
		if (Format != PF_Unknown)
		{
			const ETextureCreateFlags Flags = (SRGB ? TexCreate_SRGB : TexCreate_None)  | (bNotOfflineProcessed ? TexCreate_None : TexCreate_OfflineProcessed) | (bNoTiling ? TexCreate_NoTiling : TexCreate_None);
			const int32 NumMips = GetNumMips();
			const int32 FirstMip = FMath::Max(0, NumMips - MipCount);

			// Must be consistent with the logic in FTexture2DResource::InitRHI
			const FIntPoint MipExtents = CalcMipMapExtent(GetSizeX(), GetSizeY(), Format, FirstMip);

			const FRHITextureDesc Desc =
				FRHITextureCreateDesc::Create2DArray(TEXT("Temp"), MipExtents, GetArraySize(), Format)
				.SetNumMips(FMath::Max(1, MipCount))
				.SetFlags(Flags)
				.SetExtData(GetPlatformData()->GetExtData());

			Size = RHICalcTexturePlatformSize(Desc).Size;
		}
	}
	return Size;
}

uint32 UTexture2DArray::CalcTextureMemorySizeEnum(ETextureMipCount Enum) const
{
	if (Enum == TMC_ResidentMips)
	{
		return CalcTextureMemorySize(GetNumResidentMips());
	}
	else if (Enum == TMC_AllMipsBiased) 
	{
		return CalcTextureMemorySize(GetNumMips() - GetCachedLODBias());
	}

	return CalcTextureMemorySize(GetNumMips());
}

// While compiling the platform data in editor, we will return the 
// placeholders value to ensure rendering works as expected and that
// there are no thread-unsafe access to the platform data being built.
// Any process requiring a fully up-to-date platform data is expected to
// call FTextureCompiler:Get().FinishCompilation on UTexture first.
int32 UTexture2DArray::GetSizeX() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2DArray(this)->GetSizeX();
		}
#endif
		return PrivatePlatformData->SizeX;
	}
	return 0;
}

int32 UTexture2DArray::GetSizeY() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2DArray(this)->GetSizeY();
		}
#endif
		return PrivatePlatformData->SizeY;
	}
	return 0;
}

int32 UTexture2DArray::GetArraySize() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2DArray(this)->GetArraySize();
		}
#endif
		return PrivatePlatformData->GetNumSlices();
	}
	return 0;
}

int32 UTexture2DArray::GetNumMips() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2DArray(this)->GetNumMips();
		}
#endif
		return PrivatePlatformData->Mips.Num();
	}
	return 0;
}

EPixelFormat UTexture2DArray::GetPixelFormat() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2DArray(this)->GetPixelFormat();
		}
#endif
		return PrivatePlatformData->PixelFormat;
	}
	return PF_Unknown;
}

#if WITH_EDITOR

ENGINE_API bool UTexture2DArray::CheckArrayTexturesCompatibility()
{
	for (UTexture2D* SourceTexture : SourceTextures)
	{
		if (!SourceTexture || !SourceTexture->Source.IsValid())
		{
			// Do not create array till all texture slots are filled.
			return false;
		}
	}
	return true;
}


ENGINE_API bool UTexture2DArray::UpdateSourceFromSourceTextures(bool bCreatingNewTexture)
{
	if (!CheckArrayTexturesCompatibility()) 
	{
		return false;
	}

	if (SourceTextures.Num() > 0)
	{
		Modify();

		int32 SizeX = SourceTextures[0]->Source.GetSizeX();
		int32 SizeY = SourceTextures[0]->Source.GetSizeY();
		ETextureSourceFormat Format = SourceTextures[0]->Source.Format;
		bool bSRGB = SourceTextures[0]->Source.GetGammaSpace(0) == EGammaSpace::sRGB;

		bool bMismatchedSize = false;
		bool bMismatchedAspectRatio = false;
		bool bMismatchedFormats = false;
		bool bMismatchedGammaSpace = false;

		for (int32 SourceTextureIndex = 1; SourceTextureIndex < SourceTextures.Num(); ++SourceTextureIndex)
		{
			if (SourceTextures[SourceTextureIndex]->Source.GetSizeX() != SizeX || SourceTextures[SourceTextureIndex]->Source.GetSizeY() != SizeY)
			{
				bMismatchedSize = true;
				if (SourceTextures[SourceTextureIndex]->Source.GetSizeX() * SizeY != SourceTextures[SourceTextureIndex]->Source.GetSizeY() * SizeX)
				{
					bMismatchedAspectRatio = true;
				}
				// Dimensions of the texture array source are set to the maximum SizeX and SizeY among the array element's sources in order to minimize quality loss.
				SizeX = FMath::Max(SizeX, SourceTextures[SourceTextureIndex]->Source.GetSizeX());
				SizeY = FMath::Max(SizeY, SourceTextures[SourceTextureIndex]->Source.GetSizeY());
			}

			if ((SourceTextures[SourceTextureIndex]->Source.GetGammaSpace(0) == EGammaSpace::sRGB) != bSRGB)
			{
				bMismatchedGammaSpace = true;
				bSRGB = false;
				Format = TSF_RGBA32F;
			}

			if (Format != SourceTextures[SourceTextureIndex]->Source.Format)
			{
				bMismatchedFormats = true;
				Format = FImageCoreUtils::GetCommonSourceFormat(Format, SourceTextures[SourceTextureIndex]->Source.Format);
			}
		}

		if (bMismatchedSize)
		{
			UE_LOGF(LogTexture, Log, "Mismatched sizes of the source textures, resizing all the array elements to %dx%d%ls ...",
				SizeX, SizeY, bMismatchedAspectRatio ? TEXT(" (this will also affect aspect ratio of some source textures)") : TEXT(""));
		}

		if (bMismatchedGammaSpace)
		{
			UE_LOGF(LogTexture, Log, "Mismatched source gamma spaces, converting all to RGBA32F/Linear ...");
		}
		else if (bMismatchedFormats)
		{
			UE_LOGF(LogTexture, Log, "Mismatched source pixel formats, converting all to %ls/%ls ...",
				ERawImageFormat::GetName(FImageCoreUtils::ConvertToRawImageFormat(Format)), bSRGB ? TEXT("sRGB") : TEXT("Linear"));
		}

		int32 FormatDataSize = FTextureSource::GetBytesPerPixel(Format);
		uint32 ArraySize = SourceTextures.Num();

		// This should be false when texture is updated to avoid overriding user settings.
		if (bCreatingNewTexture)
		{
			CompressionSettings = SourceTextures[0]->CompressionSettings;
			MipGenSettings = TMGS_NoMipmaps;
			PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
			LODGroup = SourceTextures[0]->LODGroup;
			NeverStream = true;
		}

		SRGB = bSRGB;

		// Create the source texture for this Texture2DArray.
		// Currently only single-mip Texture2DArray's are supported, therefore only the first mip is copied from each element source.
		Source.Init(SizeX, SizeY, ArraySize, 1, Format);

		uint8* DestMipData = Source.LockMip(0);
		int64 MipSizeBytes = Source.CalcMipSize(0) / ArraySize;

		for (int32 SourceTexIndex = 0; SourceTexIndex < SourceTextures.Num(); ++SourceTexIndex)
		{
			FTextureSource& TextureSource = SourceTextures[SourceTexIndex]->Source;
			void* DestSliceData = DestMipData + MipSizeBytes * SourceTexIndex;
			if (TextureSource.SizeX == SizeX && TextureSource.SizeY == SizeY && TextureSource.Format == Format)
			{
				const uint8* SourceData = TextureSource.LockMipReadOnly(0);
				check(TextureSource.CalcMipSize(0) == MipSizeBytes);
				FMemory::Memcpy(DestSliceData, SourceData, MipSizeBytes);
				TextureSource.UnlockMip(0);
			}
			else
			{
				FImage SourceImage;
				SourceTextures[SourceTexIndex]->Source.GetMipImage(SourceImage, 0);

				if (TextureSource.Format != Format)
				{
					// note: there is no need to check if the gamma space is different, because in such case we would fallback to TSF_RGBA32F, which is always linear
					FImage ConvertedSourceImage;
					SourceImage.CopyTo(ConvertedSourceImage, FImageCoreUtils::ConvertToRawImageFormat(Format), bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
					ConvertedSourceImage.Swap(SourceImage);
				}

				if (TextureSource.SizeX != SizeX || TextureSource.SizeY != SizeY)
				{
					FImage ResizedSourceImage;
					SourceImage.ResizeTo(ResizedSourceImage, SizeX, SizeY, SourceImage.Format, SourceImage.GammaSpace);
					ResizedSourceImage.Swap(SourceImage);
				}
				
				check(SourceImage.RawData.Num() == MipSizeBytes);
				FMemory::Memcpy(DestSliceData, SourceImage.RawData.GetData(), MipSizeBytes);
			}
		}

		Source.UnlockMip(0);

		ValidateSettingsAfterImportOrEdit();
		SetLightingGuid();
		UpdateResource();
	}

	return true;
}

ENGINE_API void UTexture2DArray::InvalidateTextureSource()
{
	Modify();

	SetPlatformData(nullptr);

	Source = FTextureSource();
	Source.SetOwner(this);
	
	UpdateResource();
}
#endif

void UTexture2DArray::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTexture2DArray::Serialize"), STAT_Texture2DArray_Serialize, STATGROUP_LoadTime);
	
	Super::Serialize(Ar);

	FStripDataFlags StripFlags(Ar);
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (bCooked || Ar.IsCooking())
	{
		SerializeCookedPlatformData(Ar);
	}
}

void UTexture2DArray::PostLoad() 
{
#if WITH_EDITOR
	if (FApp::CanEverRender())
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
			BeginCachePlatformData();
		}
		else
		{
			FinishCachePlatformData();
		}
	}
#endif // #if WITH_EDITOR
#if WITH_EDITORONLY_DATA
	bSourceGeneratedFromSourceTexturesArray = !Source.GetNumSlices() || SourceTextures.Num();
#endif

	Super::PostLoad();
};

void UTexture2DArray::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UTexture2DArray::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	int32 SizeX = Source.GetSizeX();
	int32 SizeY = Source.GetSizeY();
	int32 ArraySize = Source.GetNumSlices();
#else
	int32 SizeX = 0;
	int32 SizeY = 0;
	int32 ArraySize = 0;
#endif
	const FString Dimensions = FString::Printf(TEXT("%dx%d*%d"), SizeX, SizeY, ArraySize);
	Context.AddTag(FAssetRegistryTag("Dimensions", Dimensions, FAssetRegistryTag::TT_Dimensional));
	Context.AddTag(FAssetRegistryTag("Format", GPixelFormats[GetPixelFormat()].Name, FAssetRegistryTag::TT_Alphabetical));

	Super::GetAssetRegistryTags(Context);
}

FString UTexture2DArray::GetDesc() 
{
	return FString::Printf(TEXT("Array: %dx%d*%d [%s]"),
		GetSizeX(),
		GetSizeY(),
		GetArraySize(),
		GPixelFormats[GetPixelFormat()].Name
	);
}

void UTexture2DArray::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) 
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddUnknownMemoryBytes(CalcTextureMemorySizeEnum(TMC_ResidentMips));
}

#if WITH_EDITOR

bool UTexture2DArray::IsDefaultTexture() const
{
	return (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete()) || (GetResource() && GetResource()->IsProxy());
}

uint32 UTexture2DArray::GetMaximumDimension() const
{
	return GetMax2DTextureDimension();

}

ENGINE_API void UTexture2DArray::PostEditChangeProperty(FPropertyChangedEvent & PropertyChangedEvent)
{	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture2DArray, SourceTextures))
	{

		for (int i=0;i<SourceTextures.Num();i++)
		{
			UTexture2D * SourceTexture = SourceTextures[i];
			if (SourceTexture && !SourceTexture->Source.IsValid())
			{
				UE_LOGF(LogTexture, Warning, "Texture has no Source, cannot be used [%ls]", *SourceTexture->GetFullName());
				SourceTextures[i] = nullptr;
			}
		}		

		// Empty SourceTextures, remove any resources if present.
		if (SourceTextures.Num() == 0) 
		{
			InvalidateTextureSource();
		}
		// First entry into an empty texture array.
		else if (SourceTextures.Num() == 1)
		{
			UpdateSourceFromSourceTextures(true);
		}
		// Couldn't add to non-empty array (Error msg logged).
		else if (UpdateSourceFromSourceTextures(false) == false)
		{
			int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
			int32 LastIndex = SourceTextures.Num() - 1;

			// But don't remove an empty texture, only an incompatible one.
			if (SourceTextures[LastIndex] != nullptr && ChangedIndex == LastIndex)
			{
				SourceTextures.RemoveAt(LastIndex);
			}
		}
	}
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTexture2DArray, AddressX)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UTexture2DArray, AddressY)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UTexture2DArray, AddressZ))
	{
		UpdateResource();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // #if WITH_EDITOR

bool UTexture2DArray::StreamOut(int32 NewMipCount)
{
	FTexture2DArrayResource* Texture2DArrayResource = GetResource() ? GetResource()->GetTexture2DArrayResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamOut(NewMipCount) && ensure(Texture2DArrayResource))
	{
		FTextureMipAllocator* MipAllocator = new FTexture2DArrayMipAllocator_Reallocate(this);
		PendingUpdate = new FTextureStreamOut(this, MipAllocator);
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

bool UTexture2DArray::StreamIn(int32 NewMipCount, bool bHighPrio)
{
	FTexture2DArrayResource* Texture2DArrayResource = GetResource() ? GetResource()->GetTexture2DArrayResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamIn(NewMipCount) && ensure(Texture2DArrayResource))
	{
		FTextureMipDataProvider* CustomMipDataProvider = nullptr;
		for (UAssetUserData* UserData : AssetUserData)
		{
			UTextureMipDataProviderFactory* CustomMipDataProviderFactory = Cast<UTextureMipDataProviderFactory>(UserData);
			if (CustomMipDataProviderFactory)
			{
				CustomMipDataProvider = CustomMipDataProviderFactory->AllocateMipDataProvider(this);
				if (CustomMipDataProvider)
				{
					break;
				}
			}
		}

		FTextureMipAllocator* MipAllocator = new FTexture2DArrayMipAllocator_Reallocate(this);
		FTextureMipDataProvider* DefaultMipDataProvider = nullptr;

#if WITH_EDITORONLY_DATA
		if (FPlatformProperties::HasEditorOnlyData() && !GetOutermost()->bIsCookedForEditor)
		{
			DefaultMipDataProvider = new FTexture2DArrayMipDataProvider_DDC(this);
		}
		else
#endif
		{
			DefaultMipDataProvider = new FTexture2DArrayMipDataProvider_IO(this, bHighPrio);
		}

		PendingUpdate = new FTextureStreamIn(this, MipAllocator, CustomMipDataProvider, DefaultMipDataProvider);
		return !PendingUpdate->IsCancelled();
	}
	return false;
}


int32 UTexture2DArray::GetNumResidentMips() const
{
	if (GetResource())
	{
		if (CachedSRRState.IsValid())
		{
			return CachedSRRState.NumResidentLODs;
		}
		else
		{
			return GetResource()->GetCurrentMipCount();
		}
	}
	return 0;
}

#undef LOCTEXT_NAMESPACE
