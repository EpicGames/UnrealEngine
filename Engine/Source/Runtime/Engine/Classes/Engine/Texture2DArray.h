// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture2D.h"
#include "Texture2DArray.generated.h"

extern bool GSupportsTexture2DArrayStreaming;

#define MAX_ARRAY_SLICES 512

UCLASS(HideCategories = Object, MinimalAPI, BlueprintType)
class UTexture2DArray : public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The derived data for this texture on this platform. */
	FTexturePlatformData* PrivatePlatformData;

public:
	/** Set the derived data for this texture on this platform. */
	ENGINE_API void SetPlatformData(FTexturePlatformData* PlatformData);
	/** Get the derived data for this texture on this platform. */
	ENGINE_API FTexturePlatformData* GetPlatformData();
	/** Get the const derived data for this texture on this platform. */
	ENGINE_API const FTexturePlatformData* GetPlatformData() const;

#if WITH_EDITOR
	TMap<FString, FTexturePlatformData*> CookedPlatformData;
#endif

	ENGINE_API int32 GetSizeX() const;
	ENGINE_API int32 GetSizeY() const;
	ENGINE_API int32 GetArraySize() const;
	ENGINE_API int32 GetNumMips() const;
	ENGINE_API EPixelFormat GetPixelFormat() const;

	/**
	*  Makes a copy of the mip chain for the texture array starting with InFirstMipToLoad.
	* 
	*  In the editor this can potentially wait for texture builds to complete.
	*/
	bool GetMipData(int32 InFirstMipToLoad, TArray<FUniqueBuffer, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>>& OutMipData);

	//~ Begin UTexture Interface
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::Array; }
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual FString GetDesc() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual float GetSurfaceWidth() const override { return static_cast<float>(GetSizeX()); }
	virtual float GetSurfaceHeight() const override { return static_cast<float>(GetSizeY()); }
	virtual float GetSurfaceDepth() const override { return 0.0f; }
	virtual uint32 GetSurfaceArraySize() const override { return GetArraySize(); }
	virtual TextureAddress GetTextureAddressX() const override { return AddressX; }
	virtual TextureAddress GetTextureAddressY() const override { return AddressY; }
	virtual TextureAddress GetTextureAddressZ() const override{ return AddressZ; }
	virtual FTextureResource* CreateResource() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API bool UpdateSourceFromSourceTextures(bool bCreatingNewTexture = true);
	ENGINE_API void InvalidateTextureSource();
	ENGINE_API bool CheckArrayTexturesCompatibility();
#endif // WITH_EDITOR
	virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2DArray; }
	virtual FTexturePlatformData** GetRunningPlatformData() override;
#if WITH_EDITOR
	virtual bool IsDefaultTexture() const override;
	virtual TMap<FString, FTexturePlatformData*>* GetCookedPlatformData() override { return &CookedPlatformData; }
#endif // WITH_EDITOR
	//~ End UTexture Interface

	/** The addressing mode to use for the X axis.*/
	UPROPERTY(EditAnywhere, Category = Source2D, meta = (DisplayName = "Address X"))
	TEnumAsByte<enum TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis.*/
	UPROPERTY(EditAnywhere, Category = Source2D, meta = (DisplayName = "Address Y"))
	TEnumAsByte<enum TextureAddress> AddressY;

	/** The addressing mode to use for the Z axis.*/
	UPROPERTY(EditAnywhere, Category = Source2D, meta = (DisplayName = "Address Z"))
	TEnumAsByte<enum TextureAddress> AddressZ;

#if WITH_EDITORONLY_DATA
	/** Add Textures*/
	UPROPERTY(EditAnywhere, Category = Source2D, meta = (DisplayName = "Source Textures", EditCondition = bSourceGeneratedFromSourceTexturesArray, EditConditionHides, HideEditConditionToggle, RequiredAssetDataTags = "IsSourceValid=True"))
	TArray<TObjectPtr<UTexture2D>> SourceTextures;

	/** 
	* Is set to true if the source texture was generated from the SourceTextures array
	* (which is not always the case, i.e. the source texture could be imported from a DDS file containing multiple slices).
	* This transient property is used to control access to the SourceTextures array from UI using EditCondition mechanism
	* (as any operation with the SourceTextures array would invalidate the originally imported source texture).
	*/
	UPROPERTY(Transient, SkipSerialization)
	bool bSourceGeneratedFromSourceTexturesArray = true;
#endif

	/** Creates and initializes a new Texture2DArray with the requested settings */
	ENGINE_API static class UTexture2DArray* CreateTransient(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat = PF_B8G8R8A8, const FName InName = NAME_None);

	/**
	 * Calculates the size of this texture in bytes if it had MipCount mip-levels streamed in.
	 *
	 * @param	MipCount Number of mips to calculate size for, counting from the smallest 1x1 mip-level and up.
	 * @return	Size of MipCount mips in bytes
	 */
	uint32 CalcTextureMemorySize(int32 MipCount) const;

	/**
	* Calculates the size of this texture if it had MipCount mip levels streamed in.
	*
	* @param	Enum which mips to calculate size for.
	* @return	Total size of all specified mips, in bytes
	*/
	virtual uint32 CalcTextureMemorySizeEnum(ETextureMipCount Enum) const override;

#if WITH_EDITOR
	/**
	* Return maximum dimension for this texture type.
	*/
	virtual uint32 GetMaximumDimension() const override;

#endif

	//~ Begin UStreamableRenderAsset Interface
	virtual int32 CalcCumulativeLODSize(int32 NumLODs) const final override { return CalcTextureMemorySize(NumLODs); }
	ENGINE_API int32 GetNumResidentMips() const;
	virtual bool StreamOut(int32 NewMipCount) final override;
	virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) final override;
	//~ End UStreamableRenderAsset Interface
};