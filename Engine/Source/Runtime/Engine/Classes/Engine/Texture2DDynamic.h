// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture.h"
#include "Texture2DDynamic.generated.h"

class FTextureResource;

// Helper to set properties on the UTexture2DDynamic so it doesn't need to be reinitialized.
struct FTexture2DDynamicCreateInfo
{
	FTexture2DDynamicCreateInfo(EPixelFormat InFormat = PF_B8G8R8A8, bool InIsResolveTarget = false, bool InSRGB = true, TextureFilter InFilter = TF_Default, ESamplerAddressMode InSamplerAddressMode = AM_Wrap)
	:	Format(InFormat)
	,	bIsResolveTarget(InIsResolveTarget)
	,	bSRGB(InSRGB)
	,	Filter(InFilter)
	,	SamplerAddressMode(InSamplerAddressMode)
	{}

	EPixelFormat Format;
	bool bIsResolveTarget;
	bool bSRGB;
	TextureFilter Filter;
	ESamplerAddressMode SamplerAddressMode;
};

// note : UTexture2DDynamic derives directly from UTexture not from UTexture2D
// UTexture2DDynamic is a base for textures that don't have a TextureSource
UCLASS(hidecategories=Object, MinimalAPI)
class UTexture2DDynamic : public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The width of the texture. */
	int32 SizeX;

	/** The height of the texture. */
	int32 SizeY;

	/** The format of the texture. */
	UPROPERTY(transient)
	TEnumAsByte<enum EPixelFormat> Format;

	/** Whether the texture can be used as a resolve target. */
	uint8 bIsResolveTarget : 1;

	/** The number of mip-maps in the texture. */
	int32 NumMips;

	/** The sampler default address mode for this texture. */
	ESamplerAddressMode SamplerAddressMode;
	
public:
	//~ Begin UTexture Interface.
	virtual ETextureClass GetTextureClass() const { return ETextureClass::TwoDDynamic; }
	ENGINE_API virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	ENGINE_API virtual float GetSurfaceWidth() const override;
	ENGINE_API virtual float GetSurfaceHeight() const override;
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	//~ End UTexture Interface.
	
	/**
	 * Initializes the texture with 1 mip-level and creates the render resource.
	 *
	 * @param InSizeX			- Width of the texture, in texels
	 * @param InSizeY			- Height of the texture, in texels
	 * @param InFormat			- Format of the texture, defaults to PF_B8G8R8A8
	 * @param InIsResolveTarget	- Whether the texture can be used as a resolve target
	 */
	ENGINE_API void Init(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat = PF_B8G8R8A8, bool InIsResolveTarget = false);

	UE_DEPRECATED(4.20, "Please use UTexture2DDynamic::Create() with FTexture2DDynamicCreateInfo initialization")
	ENGINE_API static UTexture2DDynamic* Create(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat);

	UE_DEPRECATED(4.20, "Please use UTexture2DDynamic::Create() with FTexture2DDynamicCreateInfo initialization")
	ENGINE_API static UTexture2DDynamic* Create(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat, bool InIsResolveTarget);

	/** Creates and initializes a new Texture2DDynamic with the requested settings */
	ENGINE_API static UTexture2DDynamic* Create(int32 InSizeX, int32 InSizeY, const FTexture2DDynamicCreateInfo& InCreateInfo = FTexture2DDynamicCreateInfo());
};
