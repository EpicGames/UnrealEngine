// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipDataProviderFactory.h: base class to create custom FTextureMipDataProvider.
=============================================================================*/

#pragma once
#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "Streaming/StreamableRenderResourceState.h"
#include "TextureAllMipDataProviderFactory.generated.h"

class UTexture;
class FTextureMipDataProvider;

/**
* UTextureAllMipDataProviderFactory defines an interface to create instances of FTextureMipDataProvider.
* Derived classes from UTextureAllMipDataProviderFactory can be attached to UTexture2D
* to define a new source for all of the mip data (instead of the default disk file or ddc mips). 
* Use cases include custom texture compression.
*/

UCLASS(abstract, hidecategories=Object, MinimalAPI)
class UTextureAllMipDataProviderFactory : public UTextureMipDataProviderFactory
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Retrieve initial texel data for mips, starting from FirstMipToLoad, up to the last mip in the texture.
	 *    This function will only return mips that are currently loaded in Bulk Data.
	 *    THIS FUNCTION WILL FAIL if you include in the requested range a bulk data mip that is NOT currently loaded.
	 *    Also note that only inline mips are loaded initially, and that ALL mip data buffers are discarded (either returned or freed).
	 *    So... in short, this function should only ever be called once.
	 *
	 * @param FirstMipToLoad - The first mip index to load.
	 * @param OutMipData -	A pre-allocated array of pointers, that correspond to [FirstMipToLoad, .... LastMip]
	 *						Upon successful return, each of those pointers will point to allocated memory containing the corresponding mip's data.
	 *						Caller takes responsibility to free that memory.
	 * @param OutMipSize -  A pre-allocated array of int64, that should be the same size as OutMipData (or zero size if caller does not require the sizes to be returned)
	 *						Upon successful return, each element contains the size of the corresponding mip's data buffer.
	 * @param DebugContext - A string used for debug tracking and logging. Usually Texture->GetPathName()
	 * @returns true if the requested mip data has been successfully returned.
	 */
	virtual bool GetInitialMipData(int32 FirstMipToLoad, TArrayView<void*> OutMipData, TArrayView<int64> OutMipSize, FStringView DebugContext)
		PURE_VIRTUAL(UTextureAllMipDataProviderFactory::GetInitialMipData, return false;);

	/**
	  * Get the initial streaming state (after texture is first loaded)
	  */
	virtual FStreamableRenderResourceState GetResourcePostInitState(const UTexture* Owner, bool bAllowStreaming)
		PURE_VIRTUAL(UTextureAllMipDataProviderFactory::GetResourcePostInitState, return FStreamableRenderResourceState(););
};
