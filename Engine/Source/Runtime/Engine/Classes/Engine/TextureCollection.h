// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "RenderResource.h"
#include "TextureCollection.generated.h"

class FTextureResource;
class UTexture;
class UTextureCollection;

struct FTextureCollectionResource : public FRenderResource
{
	FTextureCollectionResource(UTextureCollection* InParent);

	/** Is this a bindless collection? */
	bool bIsBindless = false;
	
	/** Name of the parent collection */
	FName CollectionName;
};

struct FBindlessTextureCollectionResource : public FTextureCollectionResource
{
	FBindlessTextureCollectionResource(UTextureCollection* InParent);
	virtual ~FBindlessTextureCollectionResource() override = default;

	// FRenderResource Interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) final;
	virtual void ReleaseRHI() final;
	// ~FRenderResource

	FRHIResourceCollection* GetResourceCollectionRHI() const
	{
		check(ResourceCollectionRHI);
		return ResourceCollectionRHI.GetReference();
	}
	
private:
	TArray<FTextureResource*> InputTextureResources;
	FRHIResourceCollectionRef ResourceCollectionRHI;
};

UCLASS(MinimalAPI, Experimental)
class UTextureCollection : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=TextureCollection)
	TArray<TObjectPtr<UTexture>> Textures;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	//~ End UObject Interface.

	ENGINE_API void SetResource(FTextureCollectionResource* InResource);
	ENGINE_API FTextureCollectionResource* GetResource() const;
	ENGINE_API FTextureCollectionResource* GetResource();

	virtual ENGINE_API FTextureCollectionResource* CreateResource();
	ENGINE_API void ReleaseResource();
	ENGINE_API void UpdateResource(bool bNotifyMaterials = true);

	virtual bool IsVirtualCollection() const { return false; }
	
protected:
#if WITH_EDITOR
	void NotifyMaterials();
#endif

	FTextureCollectionResource* PrivateResource = nullptr;
	FTextureCollectionResource* PrivateResourceRenderThread = nullptr;
};
