// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture.h"
#include "TextureCube.generated.h"

class FTextureResource;

UCLASS(hidecategories=Object, MinimalAPI)
class UTextureCube : public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The derived data for this texture on this platform. */
	FTexturePlatformData* PrivatePlatformData;

public:
	/** Set the derived data for this texture on this platform. */
	ENGINE_API void SetPlatformData(FTexturePlatformData* PlatformData);
	/** Get the derived data for this texture on this platform. */
	ENGINE_API FTexturePlatformData* GetPlatformData();
	/** Get the const derived data for this texture on this platform. */
	ENGINE_API const FTexturePlatformData* GetPlatformData() const;

#if WITH_EDITOR
	TMap<FString, FTexturePlatformData*> CookedPlatformData;
#endif

	/** Destructor */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~UTextureCube() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	ENGINE_API virtual FString GetDesc() override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;
	//~ End UObject Interface.

	ENGINE_API int32 GetSizeX() const;
	ENGINE_API int32 GetSizeY() const;
	ENGINE_API int32 GetNumMips() const;
	ENGINE_API EPixelFormat GetPixelFormat() const;

	/**
	 * Get mip data starting with the specified mip index.
	 * @param FirstMipToLoad - The first mip index to cache.
	 * @param OutMipData -	Must point to an array of pointers with at least
	 *						Mips.Num() - FirstMipToLoad + 1 entries. Upon
	 *						return those pointers will contain mip data.
	 */
	ENGINE_API void GetMipData(int32 FirstMipToLoad, void** OutMipData);
	
	//~ Begin UTexture Interface
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::Cube; }
	virtual float GetSurfaceWidth() const override { return (float)GetSizeX(); }
	virtual float GetSurfaceHeight() const override { return (float)GetSizeY(); }
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 6; }
	ENGINE_API virtual FTextureResource* CreateResource() override;
	ENGINE_API virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_TextureCube; }
	ENGINE_API virtual FTexturePlatformData** GetRunningPlatformData() override;

#if WITH_EDITOR
	ENGINE_API virtual bool IsDefaultTexture() const override;
	virtual TMap<FString, FTexturePlatformData*> *GetCookedPlatformData() override { return &CookedPlatformData; }
#endif
	//~ End UTexture Interface

	/** Creates and initializes a new TextureCube with the requested settings */
	ENGINE_API static class UTextureCube* CreateTransient(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat = PF_B8G8R8A8, const FName InName = NAME_None);

	/**
	 * Calculates the size of this texture in bytes if it had MipCount miplevels streamed in.
	 *
	 * @param	MipCount	Number of mips to calculate size for, counting from the smallest 1x1 mip-level and up.
	 * @return	Size of MipCount mips in bytes
	 */
	uint32 CalcTextureMemorySize( int32 MipCount ) const;

	/**
	 * Calculates the size of this texture if it had MipCount miplevels streamed in.
	 *
	 * @param	Enum	Which mips to calculate size for.
	 * @return	Total size of all specified mips, in bytes
	 */
	ENGINE_API virtual uint32 CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const override;

#if WITH_EDITOR
	/**
	* Return maximum dimension for this texture type.
	*/
	ENGINE_API virtual uint32 GetMaximumDimension() const override;
#endif
};



