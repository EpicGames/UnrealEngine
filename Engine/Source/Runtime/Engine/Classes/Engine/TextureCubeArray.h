// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TextureCube.h"
#include "TextureCubeArray.generated.h"

UCLASS(HideCategories = Object, MinimalAPI, BlueprintType)
class UTextureCubeArray : public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The derived data for this texture on this platform. */
	FTexturePlatformData* PrivatePlatformData;

public:
	/** Set the derived data for this texture on this platform. */
	ENGINE_API void SetPlatformData(FTexturePlatformData* PlatformData);
	/** Get the derived data for this texture on this platform. */
	ENGINE_API FTexturePlatformData* GetPlatformData();
	/** Get the const derived data for this texture on this platform. */
	ENGINE_API const FTexturePlatformData* GetPlatformData() const;

#if WITH_EDITOR
	TMap<FString, FTexturePlatformData*> CookedPlatformData;
#endif

	ENGINE_API int32 GetSizeX() const;
	ENGINE_API int32 GetSizeY() const;
	ENGINE_API int32 GetNumSlices() const;
	ENGINE_API int32 GetNumMips() const;
	ENGINE_API EPixelFormat GetPixelFormat() const;

	//~ Begin UTexture Interface
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::CubeArray; }
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual FString GetDesc() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual float GetSurfaceWidth() const override { return static_cast<float>(GetSizeX()); }
	virtual float GetSurfaceHeight() const override { return static_cast<float>(GetSizeY()); }
	virtual float GetSurfaceDepth() const override { return 0.0f; }
	virtual uint32 GetSurfaceArraySize() const override { return GetNumSlices(); }
	virtual FTextureResource* CreateResource() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API bool UpdateSourceFromSourceTextures(bool bCreatingNewTexture = true);
	ENGINE_API void InvalidateTextureSource();
	ENGINE_API bool CheckArrayTexturesCompatibility();
	ENGINE_API bool CheckLastTextureCompatibility();
#endif // WITH_EDITOR
	virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_TextureCubeArray; }
	virtual FTexturePlatformData** GetRunningPlatformData() override;
#if WITH_EDITOR
	virtual bool IsDefaultTexture() const override;
	virtual TMap<FString, FTexturePlatformData*>* GetCookedPlatformData() override { return &CookedPlatformData; }
#endif // WITH_EDITOR
	//~ End UTexture Interface

#if WITH_EDITORONLY_DATA
	/** Add Textures*/
	UPROPERTY(EditAnywhere, Category = SourceCube, meta = (DisplayName = "Source Textures", EditCondition = bSourceGeneratedFromSourceTexturesArray, EditConditionHides, HideEditConditionToggle, RequiredAssetDataTags = "IsSourceValid=True"))
	TArray<TObjectPtr<UTextureCube>> SourceTextures;

	/**
	* Is set to true if the source texture was generated from the SourceTextures array
	* (which is not always the case, i.e. the source texture could be imported from a DDS file containing multiple cubemaps).
	* This transient property is used to control access to the SourceTextures array from UI using EditCondition mechanism
	* (as any operation with the SourceTextures array would invalidate the originally imported source texture).
	*/
	UPROPERTY(Transient, SkipSerialization)
	bool bSourceGeneratedFromSourceTexturesArray = true;
#endif

	/** Creates and initializes a new TextureCubeArray with the requested settings */
	ENGINE_API static class UTextureCubeArray* CreateTransient(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat = PF_B8G8R8A8, const FName InName = NAME_None);

	/**
	 * Calculates the size of this texture in bytes if it had MipCount mip-levels streamed in.
	 *
	 * @param	MipCount Number of mips to calculate size for, counting from the smallest 1x1 mip-level and up.
	 * @return	Size of MipCount mips in bytes
	 */
	uint32 CalcTextureMemorySize(int32 MipCount) const;

	/**
	* Calculates the size of this texture if it had MipCount mip levels streamed in.
	*
	* @param	Enum which mips to calculate size for.
	* @return	Total size of all specified mips, in bytes
	*/
	virtual uint32 CalcTextureMemorySizeEnum(ETextureMipCount Enum) const override;

#if WITH_EDITOR
	/**
	* Return maximum dimension for this texture type.
	*/
	virtual uint32 GetMaximumDimension() const override;

#endif
};