// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

// TextureDefines.h is used from the TextureCompressor module as an "include only"
// dependency to get visibility of these enum values without linking to the engine module
// to facilitate that the generated TextureDefines header is only conditionally included
// if we're compiling with the engine.
#if WITH_ENGINE
#include "TextureDefines.generated.h"
#endif // WITH_ENGINE

enum EPixelFormat : uint8;

/**
 * @warning: if this is changed:
 *     update BaseEngine.ini [SystemSettings]
 *     you might have to update the update Game's DefaultEngine.ini [SystemSettings]
 *     order and actual name can never change (order is important!)
 *
 * TEXTUREGROUP_Cinematic: should be used for Cinematics which will be baked out
 *                         and want to have the highest settings
 */
UENUM()
enum TextureGroup : int
{
	TEXTUREGROUP_World UMETA(DisplayName="ini:World"),
	TEXTUREGROUP_WorldNormalMap UMETA(DisplayName="ini:WorldNormalMap"),
	TEXTUREGROUP_WorldSpecular UMETA(DisplayName="ini:WorldSpecular"),
	TEXTUREGROUP_Character UMETA(DisplayName="ini:Character"),
	TEXTUREGROUP_CharacterNormalMap UMETA(DisplayName="ini:CharacterNormalMap"),
	TEXTUREGROUP_CharacterSpecular UMETA(DisplayName="ini:CharacterSpecular"),
	TEXTUREGROUP_Weapon UMETA(DisplayName="ini:Weapon"),
	TEXTUREGROUP_WeaponNormalMap UMETA(DisplayName="ini:WeaponNormalMap"),
	TEXTUREGROUP_WeaponSpecular UMETA(DisplayName="ini:WeaponSpecular"),
	TEXTUREGROUP_Vehicle UMETA(DisplayName="ini:Vehicle"),
	TEXTUREGROUP_VehicleNormalMap UMETA(DisplayName="ini:VehicleNormalMap"),
	TEXTUREGROUP_VehicleSpecular UMETA(DisplayName="ini:VehicleSpecular"),
	TEXTUREGROUP_Cinematic UMETA(DisplayName="ini:Cinematic"),
	TEXTUREGROUP_Effects UMETA(DisplayName="ini:Effects"),
	TEXTUREGROUP_EffectsNotFiltered UMETA(DisplayName="ini:EffectsNotFiltered"),
	TEXTUREGROUP_Skybox UMETA(DisplayName="ini:Skybox"),
	TEXTUREGROUP_UI UMETA(DisplayName="ini:UI"),
	TEXTUREGROUP_Lightmap UMETA(DisplayName="ini:Lightmap"),
	TEXTUREGROUP_RenderTarget UMETA(DisplayName="ini:RenderTarget"),
	TEXTUREGROUP_MobileFlattened UMETA(DisplayName="ini:MobileFlattened"),
	/** Obsolete - kept for backwards compatibility. */
	TEXTUREGROUP_ProcBuilding_Face UMETA(DisplayName="ini:ProcBuilding_Face"),
	/** Obsolete - kept for backwards compatibility. */
	TEXTUREGROUP_ProcBuilding_LightMap UMETA(DisplayName="ini:ProcBuilding_LightMap"),
	TEXTUREGROUP_Shadowmap UMETA(DisplayName="ini:Shadowmap"),
	/** No compression, no mips. */
	TEXTUREGROUP_ColorLookupTable UMETA(DisplayName="ini:ColorLookupTable"),
	TEXTUREGROUP_Terrain_Heightmap UMETA(DisplayName="ini:Terrain_Heightmap"),
	TEXTUREGROUP_Terrain_Weightmap UMETA(DisplayName="ini:Terrain_Weightmap"),
	/** Using this TextureGroup triggers special mip map generation code only useful for the BokehDOF post process. */
	TEXTUREGROUP_Bokeh UMETA(DisplayName="ini:Bokeh"),
	/** No compression, created on import of a .IES file. */
	TEXTUREGROUP_IESLightProfile UMETA(DisplayName="ini:IESLightProfile"),
	/** Non-filtered, useful for 2D rendering. */
	TEXTUREGROUP_Pixels2D UMETA(DisplayName="ini:2D Pixels (unfiltered)"),
	/** Hierarchical LOD generated textures*/
	TEXTUREGROUP_HierarchicalLOD UMETA(DisplayName="ini:Hierarchical LOD"),
	/** Impostor Color Textures*/
	TEXTUREGROUP_Impostor UMETA(DisplayName="ini:Impostor Color"),
	/** Impostor Normal and Depth, use default compression*/
	TEXTUREGROUP_ImpostorNormalDepth UMETA(DisplayName="ini:Impostor Normal and Depth"),
	/** 8 bit data stored in textures */
	TEXTUREGROUP_8BitData UMETA(DisplayName="ini:8 Bit Data"),
	/** 16 bit data stored in textures */
	TEXTUREGROUP_16BitData UMETA(DisplayName="ini:16 Bit Data"),
	/** Project specific group, rename in Engine.ini, [EnumRemap] TEXTUREGROUP_Project**.DisplayName=My Fun Group */
	TEXTUREGROUP_Project01 UMETA(DisplayName="ini:Project Group 01"),
	TEXTUREGROUP_Project02 UMETA(DisplayName="ini:Project Group 02"),
	TEXTUREGROUP_Project03 UMETA(DisplayName="ini:Project Group 03"),
	TEXTUREGROUP_Project04 UMETA(DisplayName="ini:Project Group 04"),
	TEXTUREGROUP_Project05 UMETA(DisplayName="ini:Project Group 05"),
	TEXTUREGROUP_Project06 UMETA(DisplayName="ini:Project Group 06"),
	TEXTUREGROUP_Project07 UMETA(DisplayName="ini:Project Group 07"),
	TEXTUREGROUP_Project08 UMETA(DisplayName="ini:Project Group 08"),
	TEXTUREGROUP_Project09 UMETA(DisplayName="ini:Project Group 09"),
	TEXTUREGROUP_Project10 UMETA(DisplayName="ini:Project Group 10"),
	TEXTUREGROUP_Project11 UMETA(DisplayName="ini:Project Group 11"),
	TEXTUREGROUP_Project12 UMETA(DisplayName="ini:Project Group 12"),
	TEXTUREGROUP_Project13 UMETA(DisplayName="ini:Project Group 13"),
	TEXTUREGROUP_Project14 UMETA(DisplayName="ini:Project Group 14"),
	TEXTUREGROUP_Project15 UMETA(DisplayName="ini:Project Group 15"),
	TEXTUREGROUP_Project16 UMETA(DisplayName="ini:Project Group 16"),
	TEXTUREGROUP_Project17 UMETA(DisplayName="ini:Project Group 17"),
	TEXTUREGROUP_Project18 UMETA(DisplayName="ini:Project Group 18"),
	TEXTUREGROUP_Project19 UMETA(DisplayName="ini:Project Group 19"),
	TEXTUREGROUP_Project20 UMETA(DisplayName="ini:Project Group 20"),
	TEXTUREGROUP_Project21 UMETA(DisplayName="ini:Project Group 21"),
	TEXTUREGROUP_Project22 UMETA(DisplayName="ini:Project Group 22"),
	TEXTUREGROUP_Project23 UMETA(DisplayName="ini:Project Group 23"),
	TEXTUREGROUP_Project24 UMETA(DisplayName="ini:Project Group 24"),
	TEXTUREGROUP_Project25 UMETA(DisplayName="ini:Project Group 25"),
	TEXTUREGROUP_Project26 UMETA(DisplayName="ini:Project Group 26"),
	TEXTUREGROUP_Project27 UMETA(DisplayName="ini:Project Group 27"),
	TEXTUREGROUP_Project28 UMETA(DisplayName="ini:Project Group 28"),
	TEXTUREGROUP_Project29 UMETA(DisplayName="ini:Project Group 29"),
	TEXTUREGROUP_Project30 UMETA(DisplayName="ini:Project Group 30"),
	TEXTUREGROUP_Project31 UMETA(DisplayName="ini:Project Group 31"),
	TEXTUREGROUP_Project32 UMETA(DisplayName="ini:Project Group 32"),
	TEXTUREGROUP_Project33 UMETA(DisplayName="ini:Project Group 33"),
	TEXTUREGROUP_Project34 UMETA(DisplayName="ini:Project Group 34"),
	TEXTUREGROUP_Project35 UMETA(DisplayName="ini:Project Group 35"),
	TEXTUREGROUP_Project36 UMETA(DisplayName="ini:Project Group 36"),
	TEXTUREGROUP_Project37 UMETA(DisplayName="ini:Project Group 37"),
	TEXTUREGROUP_Project38 UMETA(DisplayName="ini:Project Group 38"),
	TEXTUREGROUP_Project39 UMETA(DisplayName="ini:Project Group 39"),
	TEXTUREGROUP_Project40 UMETA(DisplayName="ini:Project Group 40"),
	TEXTUREGROUP_Project41 UMETA(DisplayName="ini:Project Group 41"),
	TEXTUREGROUP_Project42 UMETA(DisplayName="ini:Project Group 42"),
	TEXTUREGROUP_Project43 UMETA(DisplayName="ini:Project Group 43"),
	TEXTUREGROUP_Project44 UMETA(DisplayName="ini:Project Group 44"),
	TEXTUREGROUP_Project45 UMETA(DisplayName="ini:Project Group 45"),
	TEXTUREGROUP_Project46 UMETA(DisplayName="ini:Project Group 46"),
	TEXTUREGROUP_Project47 UMETA(DisplayName="ini:Project Group 47"),
	TEXTUREGROUP_Project48 UMETA(DisplayName="ini:Project Group 48"),
	TEXTUREGROUP_MAX,
};

UENUM()
enum TextureMipGenSettings : int
{
	/** Default for the "texture". */
	TMGS_FromTextureGroup UMETA(DisplayName="FromTextureGroup"),
	/** 2x2 average, default for the "texture group". */
	TMGS_SimpleAverage UMETA(DisplayName="SimpleAverage"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen0 UMETA(DisplayName="Sharpen0"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen1 UMETA(DisplayName="Sharpen1"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen2 UMETA(DisplayName="Sharpen2"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen3 UMETA(DisplayName="Sharpen3"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen4 UMETA(DisplayName="Sharpen4"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen5 UMETA(DisplayName="Sharpen5"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen6 UMETA(DisplayName="Sharpen6"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen7 UMETA(DisplayName="Sharpen7"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen8 UMETA(DisplayName="Sharpen8"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen9 UMETA(DisplayName="Sharpen9"),
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	TMGS_Sharpen10 UMETA(DisplayName="Sharpen10"),
	TMGS_NoMipmaps UMETA(DisplayName="NoMipmaps"),
	/** Do not touch existing mip chain as it contains generated data. */
	TMGS_LeaveExistingMips UMETA(DisplayName="LeaveExistingMips"),
	/** Blur further (useful for image based reflections). */
	TMGS_Blur1 UMETA(DisplayName="Blur1"),
	TMGS_Blur2 UMETA(DisplayName="Blur2"),
	TMGS_Blur3 UMETA(DisplayName="Blur3"),
	TMGS_Blur4 UMETA(DisplayName="Blur4"),
	TMGS_Blur5 UMETA(DisplayName = "Blur5"),
	/** Use the first texel of each 2x2 (or 2x2x2) group. */
	TMGS_Unfiltered UMETA(DisplayName = "Unfiltered"),
	/** Introduce significant amount of blur using angular filtering (only applies to cubemaps, useful for ambient lighting). */
	TMGS_Angular UMETA(DisplayName = "Angular"),
	TMGS_MAX,

	// Note: These are serialized as as raw values in the texture DDC key, so additional entries
	// should be added at the bottom; reordering or removing entries will require changing the GUID
	// in the texture compressor DDC key
};

/** Options for texture padding mode. */
UENUM()
namespace ETexturePowerOfTwoSetting
{
	enum Type : int
	{
		/** Do not modify the texture dimensions. */
		None,

		/** Pad the texture to the nearest power of two size. */
		PadToPowerOfTwo,

		/** Pad the texture to the nearest square power of two size. */
		PadToSquarePowerOfTwo,

		/** Stretch the texture to the nearest power of two size. */
		StretchToPowerOfTwo,

		/** Stretch the texture to the nearest square power of two size. */
		StretchToSquarePowerOfTwo,

		/** Resize the texture to specific user defined resolution. */
		ResizeToSpecificResolution

		// Note: These are serialized as as raw values in the texture DDC key, so additional entries
		// should be added at the bottom; reordering or removing entries will require changing the GUID
		// in the texture compressor DDC key
	};
}

// Must match enum ESamplerFilter in RHIDefinitions.h
UENUM()
enum class ETextureSamplerFilter : uint8
{
	Point,
	Bilinear,
	Trilinear,
	AnisotropicPoint,
	AnisotropicLinear,
};

UENUM()
enum class ETextureMipLoadOptions : uint8
{
	// Fallback to the LODGroup settings
	Default,
	// Load all mips.
	AllMips,
	// Load only the first mip.
	OnlyFirstMip,
};

UENUM()
enum class ETextureAvailability : uint8
{
	GPU,
	CPU
};

UENUM()
enum class ETextureDownscaleOptions : uint8
{
	/** Fallback to the "texture group" settings */
	Default,
	/** Unfiltered */
	Unfiltered,
	/** Average, default for the "texture group" */
	SimpleAverage,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen0,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen1,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen2,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen3,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen4,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen5,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen6,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen7,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen8,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen9,
	/** 8x8 with sharpening: 0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
	Sharpen10
};

UENUM()
enum ETextureLossyCompressionAmount : int
{
	TLCA_Default		UMETA(DisplayName = "Default"),
	TLCA_None			UMETA(DisplayName = "No lossy compression (Oodle RDO disabled)"),
	TLCA_Lowest			UMETA(DisplayName = "Lowest (Best image quality, largest filesize) (Oodle RDO 1)"),
	TLCA_Low			UMETA(DisplayName = "Low (Oodle RDO 10)"),
	TLCA_Medium			UMETA(DisplayName = "Medium (Oodle RDO 20)"),
	TLCA_High			UMETA(DisplayName = "High (Oodle RDO 30)"),
	TLCA_Highest		UMETA(DisplayName = "Highest (Worst image quality, smallest filesize) (Oodle RDO 40)"),
};

// Certain settings can be changed to facilitate how fast a texture build takes. This
// controls which of those settings is used. It is resolved prior to the settings reaching
// the encoder.
//
// In many places where this is used, FinalIfAvailable is invalid.
UENUM()
enum class ETextureEncodeSpeed : uint8
{
	// Use the "Final" encode speed settings in UTextureEncodingProjectSettings
	Final = 0,
	// Try and fetch the final encode speed settings, but if they don't exist, encode
	// with Fast.
	FinalIfAvailable = 1,
	// Use the "Fast" encode settings in UTextureEncodingProjectSettings
	Fast = 2
};

UENUM()
enum class ETextureClass : uint8
{
	Invalid,
	// Engine types with source data :
	TwoD,
	Cube,
	Array,
	CubeArray,
	Volume,
	
	// Engine types without source data :
	TwoDDynamic,
	RenderTarget, // can be 2D or Cube

	// User types :
	Other2DNoSource, // Media, Web, etc. that should have derived from TwoDDynamic but didn't
	OtherUnknown
};

UENUM()
enum ECompositeTextureMode : int
{
	CTM_Disabled UMETA(DisplayName="Disabled"),
	/** CompositingTexture needs to be a normal map with the same or larger size. */
	CTM_NormalRoughnessToRed UMETA(DisplayName="Add Normal Roughness To Red"),
	/** CompositingTexture needs to be a normal map with the same or larger size. */
	CTM_NormalRoughnessToGreen UMETA(DisplayName="Add Normal Roughness To Green"),
	/** CompositingTexture needs to be a normal map with the same or larger size. */
	CTM_NormalRoughnessToBlue UMETA(DisplayName="Add Normal Roughness To Blue"),
	/** CompositingTexture needs to be a normal map with the same or larger size. */
	CTM_NormalRoughnessToAlpha UMETA(DisplayName="Add Normal Roughness To Alpha"),
	CTM_MAX,

	// Note: These are serialized as as raw values in the texture DDC key, so additional entries
	// should be added at the bottom; reordering or removing entries will require changing the GUID
	// in the texture compressor DDC key
};

UENUM()
enum ETextureSourceCompressionFormat : int
{
	TSCF_None	UMETA(DisplayName = "None"),
	TSCF_PNG	UMETA(DisplayName = "PNG"),
	TSCF_JPEG	UMETA(DisplayName = "JPEG"),
	TSCF_UEJPEG	UMETA(DisplayName = "UE JPEG"),
	TSCF_UEDELTA UMETA(DisplayName = "UE Delta"),

	TSCF_MAX
};

// ETextureSourceFormat should map one-to-one to ImageCore ERawImageFormat::Type
UENUM()
enum ETextureSourceFormat : int
{
	TSF_Invalid,
	TSF_G8,
	TSF_BGRA8,
	TSF_BGRE8,
	TSF_RGBA16,
	TSF_RGBA16F,

	// these are changed to TSF_BGRA8/TSF_BGRE8 on load, so the runtime will never see them after loading :
	// keep them here to preserve enum values
	TSF_RGBA8_DEPRECATED,
	TSF_RGBE8_DEPRECATED,

	TSF_G16,
	TSF_RGBA32F,
	TSF_R16F,
	TSF_R32F,

	TSF_MAX
};

/**
* Information about a texture source format
*/
struct FTextureSourceFormatInfo
{
	FTextureSourceFormatInfo() = delete;
	FTextureSourceFormatInfo(ETextureSourceFormat InTextureSourceFormat, EPixelFormat InPixelFormat, const TCHAR* InName, int32 InNumComponents, int32 InBytesPerPixel);

	ETextureSourceFormat TextureSourceFormat;
	EPixelFormat PixelFormat;
	const TCHAR* Name;
	int32 NumComponents;
	int32 BytesPerPixel;
};
extern ENGINE_API FTextureSourceFormatInfo GTextureSourceFormats[TSF_MAX];		// Maps members of ETextureSourceFormat to a FTextureSourceFormatInfo describing the format.

// This needs to be mirrored in EditorFactories.cpp.
// TC_EncodedReflectionCapture is no longer used and could be deleted
UENUM()
enum TextureCompressionSettings : int
{
	TC_Default					UMETA(DisplayName = "Default (BC1 or BC3 with A)"),
	TC_Normalmap				UMETA(DisplayName = "Normalmap (BC5)"),
	TC_Masks					UMETA(DisplayName = "Masks (no sRGB)", ToolTip = "Same compression as Default"),
	TC_Grayscale				UMETA(DisplayName = "Grayscale (G8/16) from source R", ToolTip = "If SRGB is set, becomes uncompressed RGBA8; prefer BC4"),
	TC_Displacementmap			UMETA(DisplayName = "Displacementmap (G8/16) from source A", ToolTip = "For Nanite displacement use TC_Alpha"),
	TC_VectorDisplacementmap	UMETA(DisplayName = "VectorDisplacementmap (RGBA8)"),
	TC_HDR						UMETA(DisplayName = "HDR (RGBA16F, no sRGB)", ToolTip = "Prefer HDR Compressed"),
	TC_EditorIcon				UMETA(DisplayName = "Uncompressed (RGBA8)", ToolTip = "Very high memory use; prefer BC7 or Default instead"),
	TC_Alpha					UMETA(DisplayName = "Alpha (no sRGB, BC4) from source R", ToolTip = "Single channel compressed; can also use for displacement and gray"),
	TC_DistanceFieldFont		UMETA(DisplayName = "DistanceFieldFont (G8) from source A"),
	TC_HDR_Compressed			UMETA(DisplayName = "HDR Compressed (BC6H)"),
	TC_BC7						UMETA(DisplayName = "BC7 Compressed (BC7)", ToolTip = "High quality compressed texture, same size as BC3"),
	TC_HalfFloat				UMETA(DisplayName = "Half Float (R16F)"),
	TC_LQ				        UMETA(Hidden, DisplayName = "Low Quality (BGR565/BGR555A1)", ToolTip = "BGR565/BGR555A1, fallback to DXT1/DXT5 on Mac platform"),
	TC_EncodedReflectionCapture	UMETA(Hidden), 
	TC_SingleFloat				UMETA(DisplayName = "Single Float (R32F)"),
	TC_HDR_F32					UMETA(DisplayName = "HDR High Precision (RGBA32F)"),
	TC_MAX,
};

/** List of (advanced) texture source encodings, matching the list in ColorManagementDefines.h. */
UENUM()
enum class ETextureSourceEncoding : uint8
{
	TSE_None		= 0 UMETA(DisplayName = "None", ToolTip = "The source encoding is not overridden."),
	TSE_Linear		= 1 UMETA(DisplayName = "Linear", ToolTip = "The source encoding is considered linear (before optional sRGB encoding is applied)."),
	TSE_sRGB		= 2 UMETA(DisplayName = "sRGB", ToolTip = "sRGB source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_ST2084		= 3 UMETA(DisplayName = "ST 2084/PQ", ToolTip = "SMPTE ST 2084/PQ source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_Gamma22		= 4 UMETA(DisplayName = "Gamma 2.2", ToolTip = "Gamma 2.2 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_BT1886		= 5 UMETA(DisplayName = "BT1886/Gamma 2.4", ToolTip = "BT1886/Gamma 2.4 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_Gamma26		= 6 UMETA(DisplayName = "Gamma 2.6", ToolTip = "Gamma 2.6 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_Cineon		= 7 UMETA(DisplayName = "Cineon", ToolTip = "Cineon source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_REDLog		= 8 UMETA(DisplayName = "REDLog", ToolTip = "RED Log source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_REDLog3G10	= 9 UMETA(DisplayName = "REDLog3G10", ToolTip = "RED Log3G10 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_SLog1		= 10 UMETA(DisplayName = "SLog1", ToolTip = "Sony SLog1 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_SLog2		= 11 UMETA(DisplayName = "SLog2", ToolTip = "Sony SLog2 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_SLog3		= 12 UMETA(DisplayName = "SLog3", ToolTip = "Sony SLog3 source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_AlexaV3LogC	= 13 UMETA(DisplayName = "AlexaV3LogC", ToolTip = "ARRI Alexa V3 LogC source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_CanonLog	= 14 UMETA(DisplayName = "CanonLog", ToolTip = "Canon Log source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_ProTune		= 15 UMETA(DisplayName = "ProTune", ToolTip = "GoPro ProTune source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_VLog		= 16 UMETA(DisplayName = "V-Log", ToolTip = "Panasonic V-Log source encoding to be linearized (before optional sRGB encoding is applied)."),
	TSE_HLG			= 17 UMETA(DisplayName = "HLG", ToolTip = "HLG (BT.2100): linearized to the [0, 1] range (before optional sRGB encoding is applied)."),
	TSE_MAX			= 18,
};

//TODO: Rename nearly-colliding ETextureSourceColorSpace enum in TextureFactory.h
/** List of (source) texture color spaces, matching the list in ColorManagementDefines.h. */
UENUM()
enum class ETextureColorSpace : uint8
{
	TCS_None				= 0 UMETA(DisplayName = "None", ToolTip = "No explicit color space definition."),
	TCS_sRGB				= 1 UMETA(DisplayName = "sRGB / Rec709", ToolTip = "sRGB / Rec709 (BT.709) color primaries, with D65 white point."),
	TCS_Rec2020				= 2 UMETA(DisplayName = "Rec2020", ToolTip = "Rec2020 (BT.2020) primaries with D65 white point."),
	TCS_ACESAP0				= 3 UMETA(DIsplayName = "ACES AP0", ToolTip = "ACES AP0 wide gamut primaries, with D60 white point."),
	TCS_ACESAP1				= 4 UMETA(DIsplayName = "ACES AP1 / ACEScg", ToolTip = "ACES AP1 / ACEScg wide gamut primaries, with D60 white point."),
	TCS_P3DCI				= 5 UMETA(DisplayName = "P3DCI", ToolTip = "P3 (Theater) primaries, with DCI Calibration white point."),
	TCS_P3D65				= 6 UMETA(DisplayName = "P3D65", ToolTip = "P3 (Display) primaries, with D65 white point."),
	TCS_REDWideGamut		= 7 UMETA(DisplayName = "RED Wide Gamut", ToolTip = "RED Wide Gamut primaries, with D65 white point."),
	TCS_SonySGamut3			= 8 UMETA(DisplayName = "Sony S-Gamut3", ToolTip = "Sony S-Gamut/S-Gamut3 primaries, with D65 white point."),
	TCS_SonySGamut3Cine		= 9 UMETA(DisplayName = "Sony S-Gamut3 Cine", ToolTip = "Sony S-Gamut3 Cine primaries, with D65 white point."),
	TCS_AlexaWideGamut		= 10 UMETA(DisplayName = "Alexa Wide Gamut", ToolTip = "Alexa Wide Gamut primaries, with D65 white point."),
	TCS_CanonCinemaGamut	= 11 UMETA(DisplayName = "Canon Cinema Gamut", ToolTip = "Canon Cinema Gamut primaries, with D65 white point."),
	TCS_GoProProtuneNative	= 12 UMETA(DisplayName = "GoPro Protune Native", ToolTip = "GoPro Protune Native primaries, with D65 white point."),
	TCS_PanasonicVGamut		= 13 UMETA(DisplayName = "Panasonic V-Gamut", ToolTip = "Panasonic V-Gamut primaries, with D65 white point."),
	TCS_Custom				= 99 UMETA(DisplayName = "Custom", ToolTip = "User defined color space and white point."),
	TCS_MAX,
};

UENUM()
enum TextureCookPlatformTilingSettings : uint8
{
	/** Get the tiling setting from the texture's group CookPlatformTilingDisabled setting. By default it's to tile during cook, unless it has been changed in the texture group */
	TCPTS_FromTextureGroup UMETA(DisplayName = "FromTextureGroup"),
	/** The texture will be tiled during the cook process if the platform supports it. */
	TCPTS_Tile UMETA(DisplayName = "Tile during cook"),
	/** The texture will not be tiled during the cook process, and will be tiled when uploaded to the GPU if the platform supports it. */
	TCPTS_DoNotTile UMETA(DisplayName = "Do not tile during cook"),
	TCPTS_MAX,
};

/** List of chromatic adaptation methods, matching the list in ColorManagementDefines.h. */
UENUM()
enum class ETextureChromaticAdaptationMethod : uint8
{
	TCAM_None		= 0 UMETA(DisplayName = "None", ToolTip = "No chromatic adaptation is applied."),
	TCAM_Bradford	= 1 UMETA(DisplayName = "Bradford", ToolTip = "Chromatic adaptation is applied using the Bradford method."),
	TCAM_CAT02		= 2 UMETA(DisplayName = "CAT02", ToolTip = "Chromatic adaptation is applied using the CAT02 method."),
	TCAM_MAX,
};


UENUM()
enum TextureFilter : int
{
	TF_Nearest UMETA(DisplayName="Nearest"),
	TF_Bilinear UMETA(DisplayName="Bi-linear"),
	TF_Trilinear UMETA(DisplayName="Tri-linear"),
	/** Use setting from the Texture Group. */
	TF_Default UMETA(DisplayName="Default (from Texture Group)"),
	TF_MAX,
};

UENUM()
enum TextureAddress : int
{
	TA_Wrap UMETA(DisplayName="Wrap"),
	TA_Clamp UMETA(DisplayName="Clamp"),
	TA_Mirror UMETA(DisplayName="Mirror"),
	TA_MAX,
};

UENUM()
enum ETextureMipCount : int
{
	TMC_ResidentMips,
	TMC_AllMips,
	TMC_AllMipsBiased,
	TMC_MAX,
};

// TextureCompressionQuality is used for ASTC
UENUM()
enum ETextureCompressionQuality : int
{
	TCQ_Default = 0		UMETA(DisplayName="Default"),
	TCQ_Lowest = 1		UMETA(DisplayName="Lowest (ASTC 12x12)"),
	TCQ_Low = 2			UMETA(DisplayName="Low (ASTC 10x10)"),
	TCQ_Medium = 3		UMETA(DisplayName="Medium (ASTC 8x8)"),
	TCQ_High= 4			UMETA(DisplayName="High (ASTC 6x6)"),
	TCQ_Highest = 5		UMETA(DisplayName="Highest (ASTC 4x4)"),
	TCQ_MAX,
};

UENUM()
enum ETextureStreamingMethod : int
{
	TSM_Invalid = 0,   // Unknown / not set
	TSM_NotStreamed,   // not streamed (fully resident all the time)
	TSM_Streamed,      // traditional whole-mip streaming
	TSM_VirtualStreamed  // virtual texture (VT)
};


namespace UE
{
namespace TextureDefines
{

inline bool IsHDR(ETextureSourceFormat Format)
{
	return (Format == TSF_BGRE8 || Format == TSF_RGBA16F || Format == TSF_RGBA32F || Format == TSF_R16F || Format == TSF_R32F);
}

inline bool IsHDR(TextureCompressionSettings CompressionSettings)
{
	switch(CompressionSettings)
	{
	case TC_HDR:
	case TC_HDR_F32:
	case TC_HDR_Compressed:
	case TC_HalfFloat:
	case TC_SingleFloat:
		return true;
	default:
		return false;
	}
}

inline bool IsUncompressed(TextureCompressionSettings CompressionSettings)
{
	return (CompressionSettings == TC_Grayscale ||
			CompressionSettings == TC_Displacementmap ||
			CompressionSettings == TC_VectorDisplacementmap ||
			CompressionSettings == TC_HDR ||
			CompressionSettings == TC_HDR_F32 ||
			CompressionSettings == TC_EditorIcon ||
			CompressionSettings == TC_DistanceFieldFont ||
			CompressionSettings == TC_HalfFloat ||
			CompressionSettings == TC_SingleFloat
		);
}

inline bool ShouldUseGreyScaleEditorVisualization(TextureCompressionSettings CompressionSettings)
{
	// these formats should do R -> RGB red to gray replication in Editor viz
	return (CompressionSettings == TC_Grayscale ||
			CompressionSettings == TC_Alpha ||
			CompressionSettings == TC_Displacementmap ||
			CompressionSettings == TC_DistanceFieldFont );

	// ?? maybe these too ??
	//		CompressionSettings == TC_HalfFloat ||
	//		CompressionSettings == TC_SingleFloat
}


} // TextureDefines
} // UE

