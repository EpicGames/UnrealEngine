// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureEnum.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/TextureDefines.h"
#include "TextureLODSettings.generated.h"

class UTexture;
enum class EVTProducerPriority : uint8;

/** LOD settings for a single texture group. */
USTRUCT()
struct FTextureLODGroup
{
	GENERATED_USTRUCT_BODY()

	FTextureLODGroup()
	{
		SetupGroup();
	}

	/** Group ID.																	*/
	UPROPERTY()
	TEnumAsByte<TextureGroup> Group = TEXTUREGROUP_World;
	
	/** Sampler filter state.														*/
	ETextureSamplerFilter Filter;

	/** Defines how the the mip-map generation works, e.g. sharpening				*/
	UPROPERTY()
	TEnumAsByte<TextureMipGenSettings> MipGenSettings = TextureMipGenSettings::TMGS_SimpleAverage;
	
	UPROPERTY()
	ETextureMipLoadOptions MipLoadOptions = ETextureMipLoadOptions::AllMips;

	/** Maximum LOD mip count. Bias will be adjusted so texture won't go above.		*/
	int32 MaxLODMipCount = 32;
	
	/** Group LOD bias.																*/
	UPROPERTY()
	int32 LODBias = 0;

	UPROPERTY()
	int32 LODBias_Smaller = -1;

	UPROPERTY()
	int32 LODBias_Smallest = -1;
	
	UPROPERTY()
	int32 LODBias_VT = 0;

	/** Number of mip-levels that can be streamed. -1 means all mips can stream.	*/
	UPROPERTY()
	int32 NumStreamedMips = -1;

	/** Prevent LODBias from making the textures smaller than this value. Note that this does _not_ affect the smallest mip level size. */
	UPROPERTY()
	int32 MinLODSize = 1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize = 4096;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used for platforms with the "Smaller" memory bucket. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_Smaller = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used for platforms with the "Smallest" memory bucket. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_Smallest = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used in combination with r.TextureGroup.QualityLevel, r.TextureGroup.MaxQualityLevel and TextureGroupOptionalQualityLevel. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_Low = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used in combination with r.TextureGroup.QualityLevel, r.TextureGroup.MaxQualityLevel and TextureGroupOptionalQualityLevel. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_Medium = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used in combination with r.TextureGroup.QualityLevel, r.TextureGroup.MaxQualityLevel and TextureGroupOptionalQualityLevel. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_High = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used in combination with r.TextureGroup.QualityLevel, r.TextureGroup.MaxQualityLevel and TextureGroupOptionalQualityLevel. Not used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_Epic = -1;

	/** Cap the number of mips such that the largest mip is this big. Has no effect for textures with no mip chain. Used for virtual textures. */
	UPROPERTY()
	int32 MaxLODSize_VT = 0;

	/** If this is greater than 0 will put that number of mips into an optional bulkdata package */
	UPROPERTY()
	int32 OptionalLODBias = 0;

	/** Put all the mips which have a width / height larger then OptionalLODSize into an optional bulkdata package */
	UPROPERTY()
	int32 OptionalMaxLODSize = 4096;

	UPROPERTY()
	FName MinMagFilter = NAME_Aniso;

	UPROPERTY()
	FName MipFilter = NAME_Point;

	/** Whether those assets should be loaded with higher load order and higher IO priority. Allows ProjectXX texture groups to behave as character textures. */
	UPROPERTY()
	uint8 HighPriorityLoad : 1 = false;

	UPROPERTY()
	uint8 DuplicateNonOptionalMips : 1 = false;
	
	/** If true textures with CookPlatformTilingSettings set to TCPTS_FromTextureGroup will not be tiled during cook. They will be tiled when uploaded to the GPU if necessary */
	UPROPERTY()
	uint8 CookPlatformTilingDisabled : 1 = false;

	UPROPERTY()
	ETextureDownscaleOptions DownscaleOptions = ETextureDownscaleOptions::Default;
	
	UPROPERTY()
	TEnumAsByte<ETextureLossyCompressionAmount> LossyCompressionAmount = TLCA_Default;
	
	UPROPERTY()
	float Downscale = 1.0;

	UPROPERTY()
	int32 VirtualTextureTileCountBias = 0;

	UPROPERTY()
	int32 VirtualTextureTileSizeBias = 0;

	UPROPERTY()
	EVTProducerPriority VirtualTextureStreamingPriority = EVTProducerPriority::Normal;

	/** Allows us to override max anisotropy. If unspecified, uses r.MaxAnisotropy */
	UPROPERTY()
	int32 MaxAniso = 0;

	ENGINE_API void SetupGroup();

	ENGINE_API bool operator==(const FTextureLODGroup& Other) const;

	ENGINE_API void ComputeMaxSizesForQualityLevels(int32 OutMaxSizes[4]) const;
};

/**
 * Structure containing all information related to an LOD group and providing helper functions to calculate
 * the LOD bias of a given group.
 */
UCLASS(config=DeviceProfiles, perObjectConfig, MinimalAPI)
class UTextureLODSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Calculates and returns the LOD bias based on texture LOD group, LOD bias and minimum/maximum size.
	 *
	 * @param	Texture				Texture object to calculate LOD bias for.
	 * @param	bIncCinematicMips	If true, cinematic mips will also be included in consideration
	 * @return	LOD bias
	 */
	ENGINE_API int32 CalculateLODBias(const UTexture* Texture, bool bIncCinematicMips = true) const;

	/**
	 * Calculates and returns the LOD bias based on the information provided.
	 *
	 * @param	Width						Width of the texture
	 * @param	Height						Height of the texture
	 * @param	MinSize						Min size to clamp width or height of the texture
	 * @param	MaxSize						Max size to clamp width or height of the texture
	 * @param	LODGroup					Which LOD group the texture belongs to
	 * @param	LODBias						LOD bias to include in the calculation
	 * @param	NumCinematicMipLevels		The texture cinematic mip levels to include in the calculation
	 * @param	MipGenSetting				Mip generation setting
	 * @param	bVirtualTexture				If VT is enabled (in this case group's max LOD is ignored)
	* @param	bCurrentlyStreamableTexture	If texture is currently streamable (if it's not, then quality levels don't apply to it)
	 * @return	LOD bias
	 */
	ENGINE_API int32 CalculateLODBias(int32 Width, int32 Height, int32 MinSize, int32 MaxSize, int32 LODGroup, int32 LODBias, int32 NumCinematicMipLevels, TextureMipGenSettings MipGenSetting, bool bVirtualTexture, bool bCurrentlyStreamableTexture = false) const;

	UE_DEPRECATED(5.8, "Call newer version of function that takes MinSize.")
	int32 CalculateLODBias(int32 Width, int32 Height, int32 MaxSize, int32 LODGroup, int32 LODBias, int32 NumCinematicMipLevels, TextureMipGenSettings MipGenSetting, bool bVirtualTexture, bool bCurrentlyStreamableTexture = false) const
	{
		return CalculateLODBias(Width, Height, 0, MaxSize, LODGroup, LODBias, NumCinematicMipLevels, MipGenSetting, bVirtualTexture, bCurrentlyStreamableTexture);
	}

	/**
	 * Calculate num optional mips
	 *
	 * @param	LODGroup					Which LOD group the texture belongs to
	 * @param	MipGenSetting				Mip generation setting
	 * @return	Num optional mips counted from higest mip
	 */
	ENGINE_API int32 CalculateNumOptionalMips(int32 LODGroup, const int32 Width, const int32 Height, const int32 NumMips, const int32 MinMipToInline, TextureMipGenSettings InMipGenSetting) const;

#if WITH_EDITORONLY_DATA
	ENGINE_API void GetMipGenSettings( const UTexture& Texture, TextureMipGenSettings& OutMipGenSettings, float& OutSharpen, uint32& OutKernelSize, bool& bOutDownsampleWithAverage, bool& bOutSharpenWithoutColorShift, bool &bOutBorderColorBlack ) const;
	ENGINE_API void GetDownscaleOptions(const UTexture& Texture, const ITargetPlatform& CurrentPlatform, float& Downscale, ETextureDownscaleOptions& DownscaleOptions) const;
#endif // #if WITH_EDITORONLY_DATA

	/**
	 * Returns the filter state that should be used for the passed in texture, taking
	 * into account other system settings.
	 *
	 * @param	Texture		Texture to retrieve filter state for, must not be 0
	 * @return	Filter sampler state for passed in texture
	 */
	ENGINE_API ETextureSamplerFilter GetSamplerFilter(const UTexture* Texture) const;

	ENGINE_API ETextureSamplerFilter GetSamplerFilter(int32 InLODGroup) const;

	/**
	 * Returns the mip load options of a texture.
	 *
	 * @param	Texture		Texture to retrieve the mip load option, must not be 0
	 * @return	The mip load option
	 */
	ENGINE_API ETextureMipLoadOptions GetMipLoadOptions(const UTexture* Texture) const;

	/**
	 * Returns the LODGroup mip gen settings
	 *
	 * @param	InLODGroup		The LOD Group ID 
	 * @return	TextureMipGenSettings for lod group
	 */
	ENGINE_API const TextureMipGenSettings GetTextureMipGenSettings( int32 InLODGroup ) const; 


	/**
	 * Returns the texture group names, sorted like enum.
	 *
	 * @return array of texture group names
	 */
	static ENGINE_API TArray<FString> GetTextureGroupNames();

	/**
	 * TextureLODGroups access with bounds check
	 *
	 * @param   GroupIndex      usually from Texture.LODGroup
	 * @return                  A handle to the indexed LOD group. 
	 */
	ENGINE_API FTextureLODGroup& GetTextureLODGroup(TextureGroup GroupIndex);

	/**
	* TextureLODGroups access with bounds check
	*
	* @param   GroupIndex      usually from Texture.LODGroup
	* @return                  A handle to the indexed LOD group.
	*/
	ENGINE_API const FTextureLODGroup& GetTextureLODGroup(TextureGroup GroupIndex) const;

protected:
	ENGINE_API void SetupLODGroup(int32 GroupId);

public:

	/** If set to a valid value, use MaxLODSize for the given quality instead of OptionalMaxLODSize during cook time. */
	UPROPERTY(EditAnywhere, config, Category="Texture LOD Settings")
	int32 TextureGroupOptionalQualityLevel = -1;

	/** Array of LOD settings with entries per group. */
	UPROPERTY(EditAnywhere, config, Category="Texture LOD Settings")
	TArray<FTextureLODGroup> TextureLODGroups;
};
