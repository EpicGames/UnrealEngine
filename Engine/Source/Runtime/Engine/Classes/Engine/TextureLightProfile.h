// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture2D.h"
#include "TextureLightProfile.generated.h"

UCLASS(hidecategories=(Object,CompositeTexture,Texture2D), MinimalAPI, BlueprintType)
class UTextureLightProfile : public UTexture2D
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Light brightness in Candelas, imported from IES profile, <= 0 if the profile is used for masking only. Use with InverseSquareFalloff. */
	UPROPERTY(EditAnywhere, Category=TextureLightProfile, AssetRegistrySearchable)
	float Brightness;

	/** Multiplier to map texture value to result to integrate over the sphere to 1.0f */
	UPROPERTY(VisibleAnywhere, Category=TextureLightProfile)
	float TextureMultiplier;
};
