// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipDataProviderFactory.h: base class to create custom FTextureMipDataProvider.
=============================================================================*/

#pragma once
#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"
#include "TextureMipDataProviderFactory.generated.h"

class UTexture;
class FTextureMipDataProvider;

/**
* UTextureMipDataProviderFactory defines an interface to create instances of FTextureMipDataProvider.
* Derived classes from UTextureMipDataProviderFactory can be attached to UTexture::MipDataProviderFactory
* to define a new source for mip content (instead of the default disk file or ddc mips). 
* Usecases include dynamic textures that need to be driven by the texture streaming or textures that 
* get they data over the network.
*/
UCLASS(abstract, hidecategories=Object, MinimalAPI)
class UTextureMipDataProviderFactory : public UAssetUserData
{
	GENERATED_UCLASS_BODY()

public:

	/**
	* Create a FTextureMipDataProvider to handle a single StreamIn mip operation.
	* The object lifetime will be managed by FRenderAssetUpdate.
	* 
	* @param InAsset - the texture on which the stream in operation will be performed.
	*/
	virtual FTextureMipDataProvider* AllocateMipDataProvider(UTexture* Asset) { return nullptr; }

	/**
	* Returns true if TextureMipDataProviders allocated by this factory can provide MipData by themselves,
	* even without loading from disk at all, so streaming can be enabled for their textures.
	*/
	virtual bool WillProvideMipDataWithoutDisk() const { return false; }

	/**
	  * Return whether the mip data provided by this provider is in "offline processed" format (i.e. pre-tiled specifically for the target platform)
	  */
	virtual bool ShouldAllowPlatformTiling(const UTexture* Owner) const { return false; }
};
