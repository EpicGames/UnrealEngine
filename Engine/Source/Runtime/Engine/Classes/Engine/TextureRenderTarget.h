// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * TextureRenderTarget
 *
 * Base for all render target texture resources
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture.h"
#include "RenderUtils.h"
#include "TextureRenderTarget.generated.h"

enum EConstructTextureFlags : uint32;

UCLASS(abstract, MinimalAPI)
class UTextureRenderTarget : public UTexture
{
	GENERATED_UCLASS_BODY()	

	/** Will override FTextureRenderTarget2DResource::GetDisplayGamma if > 0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureRenderTarget)
	float TargetGamma;

	/** If true, there will be two copies in memory - one for the texture and one for the render target. If false, they will share memory if possible. This is useful for scene capture textures that are used in the scene. */
	uint32 bNeedsTwoCopies:1;

	/** If true, it will be possible to create a FUnorderedAccessViewRHIRef using RHICreateUnorderedAccessView and the internal FTextureRHIRef. */
	uint32 bCanCreateUAV : 1;

	/** If true, RTVs will be created for each slice (texture array and/or cube). */
	uint32 bTargetArraySlicesIndependently : 1;

	/**
	 * Render thread: Access the render target resource for this texture target object
	 * @return pointer to resource or NULL if not initialized
	 */
	ENGINE_API class FTextureRenderTargetResource* GetRenderTargetResource();

	/**
	 * Returns a pointer to the (game thread managed) render target resource.  Note that you're not allowed
	 * to deferenced this pointer on the game thread, you can only pass the pointer around and check for NULLness
	 * @return pointer to resource
	 */
	ENGINE_API class FTextureRenderTargetResource* GameThread_GetRenderTargetResource();


	//~ Begin UTexture Interface
	virtual ETextureClass GetTextureClass() const { return ETextureClass::RenderTarget; }
	ENGINE_API virtual class FTextureResource* CreateResource() override;
	ENGINE_API virtual EMaterialValueType GetMaterialType() const override;
	//~ End UTexture Interface

	/**
	 * Validates that the UTextureRenderTarget can be converted to a UTexture (e.g. supported format, valid size, etc.)
	 * @param OutPixelFormat - The format of the render target (only if conversion is possible)
	 * @param OutTextureSourceFormat - The format of the texture that would be used if the UTextureRenderTarget was converted to a UTexture (only if conversion is possible)
	 * @param OutErrorMessage - (Optional) If passed, will contain an error message in case conversion is impossible
	 * @return True if render target can be converted to a texture
	 */
	virtual bool CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage = nullptr) const PURE_VIRTUAL(UTextureRenderTarget, return false;);

	/**
	 * Validates that the UTextureRenderTarget can be converted to a UTexture (e.g. supported format, valid size, etc.)
	 * @param OutErrorMessage - (Optional) If passed, will contain an error message in case conversion is impossible
	 * @return True if render target can be converted to a texture
	 */
	ENGINE_API bool CanConvertToTexture(FText* OutErrorMessage = nullptr) const;

	/**
	 * Returns the UTexture class that corresponds to this render target (e.g. UTexture2D for UTextureRenderTarget2D)
	 */
	virtual TSubclassOf<UTexture> GetTextureUClass() const PURE_VIRTUAL(GetTextureUClass, return nullptr;)
	// GetTextureClass() will just return "RenderTarget" ; to get the sub-type (2d/cube), use GetRenderTargetTextureClass
	virtual ETextureClass GetRenderTargetTextureClass() const PURE_VIRTUAL(GetRenderTargetTextureClass, return ETextureClass::Invalid;)

	virtual EPixelFormat GetFormat() const PURE_VIRTUAL(GetFormat,return PF_Unknown;)
	virtual bool IsSRGB() const PURE_VIRTUAL(IsSRGB,return false;)
	virtual float GetDisplayGamma() const PURE_VIRTUAL(GetDisplayGamma,return 0.f;)

	// UTextureRenderTarget default display gamma if none is set
	//	returns hard-coded 2.2
	ENGINE_API static float GetDefaultDisplayGamma();

	// get the variant of ReadPixels call that should be used
	//	either BGRA8,RGBA32F,or RGBA16F
	//	for FColor,FLinearColor, or Float16
	static ERawImageFormat::Type GetReadPixelsFormat(EPixelFormat PF,bool bIsVolume);

#if WITH_EDITOR

	/**
	 * Utility for updating an existing UTexture from a TextureRenderTarget
	 * @param InTexture - Texture which will contain the content of this render target after the call.
	 * @param InFlags - Various control flags for operation (see EConstructTextureFlags)
	 * @param InAlphaOverride - (Optional) the values here will become the alpha values in the resulting texture
	 * @param InOnTextureChangingDelegate - (Optional) If the texture needs to be modified (as its properties or content will change), this delegate will be called beforehand. 
	 * @param OutErrorMessage - (Optional) Error message in case of failure
	 * @return true if conversion from render to texture was successful
	 */
	using FOnTextureChangingDelegate = TFunctionRef<void(UTexture* /*InTexture*/)>;
	constexpr static auto OnTextureChangingDefault = [](UTexture*){};
	ENGINE_API bool UpdateTexture(UTexture* InTexture, EConstructTextureFlags InFlags = CTF_Default, const TArray<uint8>* InAlphaOverride = nullptr, FOnTextureChangingDelegate InOnTextureChangingDelegate = OnTextureChangingDefault, FText* OutErrorMessage = nullptr);

	/**
	 * Utility for creating a new UTexture from a UTextureRenderTarget. The method can fail if conversion is impossible
	 * @param InOuter - Outer to use when constructing the new UTexture.
	 * @param InNewTextureName - Name of new UTexture object.
	 * @param InObjectFlags - Flags to apply to the new UTexture object
	 * @param InFlags - Various control flags for operation (see EConstructTextureFlags)
	 * @param InAlphaOverride - (Optional) the values here will become the alpha values in the resulting texture (8 bits alpha, regardless of the texture's format)
	 * @param OutErrorMessage - (Optional) Error message in case of failure
	 * @return New UTexture object.
	 */
	ENGINE_API UTexture* ConstructTexture(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags = RF_NoFlags, EConstructTextureFlags InFlags = CTF_Default, const TArray<uint8>* InAlphaOverride = nullptr, FText* OutErrorMessage = nullptr);
#endif // WITH_EDITOR

protected:
	ETextureSourceFormat ValidateTextureFormatForConversionToTextureInternal(EPixelFormat InFormat, const TArrayView<const EPixelFormat>& InCompatibleFormats, FText* OutErrorMessage) const;
};



