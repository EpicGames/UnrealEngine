// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RenderUtils.h"
#include "Engine/TextureRenderTarget.h"
#include "TextureRenderTarget2D.generated.h"

class FTextureResource;
class UTexture2D;
struct FPropertyChangedEvent;

extern ENGINE_API int32 GTextureRenderTarget2DMaxSizeX;
extern ENGINE_API int32 GTextureRenderTarget2DMaxSizeY;

/** Subset of EPixelFormat exposed to UTextureRenderTarget2D */
UENUM(BlueprintType)
enum ETextureRenderTargetFormat : int
{
	/** R channel, 8 bit per channel fixed point, range [0, 1]. */
	RTF_R8,
	/** RG channels, 8 bit per channel fixed point, range [0, 1]. */
	RTF_RG8,
	/** RGBA channels, 8 bit per channel fixed point, range [0, 1]. */
	RTF_RGBA8,
	/** RGBA channels, 8 bit per channel fixed point, range [0, 1]. RGB is encoded with sRGB gamma curve. A is always stored as linear. */
	RTF_RGBA8_SRGB,
	/** R channel, 16 bit per channel floating point, range [-65504, 65504] */
	RTF_R16f,
	/** RG channels, 16 bit per channel floating point, range [-65504, 65504] */
	RTF_RG16f,
	/** RGBA channels, 16 bit per channel floating point, range [-65504, 65504] */
	RTF_RGBA16f,
	/** R channel, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38] */
	RTF_R32f,
	/** RG channels, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38] */
	RTF_RG32f,
	/** RGBA channels, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38] */
	RTF_RGBA32f,
	/** RGBA channels, 10 bit per channel fixed point and 2 bit of alpha */
	RTF_RGB10A2
};

inline EPixelFormat GetPixelFormatFromRenderTargetFormat(ETextureRenderTargetFormat RTFormat)
{
	switch (RTFormat)
	{
	case RTF_R8: return PF_G8;
	case RTF_RG8: return PF_R8G8;
	case RTF_RGBA8: return PF_B8G8R8A8;
	case RTF_RGBA8_SRGB: return PF_B8G8R8A8;

	case RTF_R16f: return PF_R16F;
	case RTF_RG16f: return PF_G16R16F;
	case RTF_RGBA16f: return PF_FloatRGBA;

	case RTF_R32f: return PF_R32_FLOAT;
	case RTF_RG32f: return PF_G32R32F;
	case RTF_RGBA32f: return PF_A32B32G32R32F;
	case RTF_RGB10A2: return PF_A2B10G10R10;
	}

	ensureMsgf(false, TEXT("Unhandled ETextureRenderTargetFormat entry %u"), (uint32)RTFormat);
	return PF_Unknown;
}

UENUM(BlueprintType)
enum class ETextureRenderTargetSampleCount : uint8
{
	RTSC_1 UMETA(DisplayName = "MSAAx1"),
	RTSC_2 UMETA(DisplayName = "MSAAx2"),
	RTSC_4 UMETA(DisplayName = "MSAAx4"),
	RTSC_8 UMETA(DisplayName = "MSAAx8"),

	RTSC_MAX,
};

inline int32 GetNumFromRenderTargetSampleCount(ETextureRenderTargetSampleCount InSampleCount)
{
	switch (InSampleCount)
	{
	case ETextureRenderTargetSampleCount::RTSC_1: return 1 << 0;
	case ETextureRenderTargetSampleCount::RTSC_2: return 1 << 1;
	case ETextureRenderTargetSampleCount::RTSC_4: return 1 << 2;
	case ETextureRenderTargetSampleCount::RTSC_8: return 1 << 3;

	default:
		ensureMsgf(false, TEXT("Unhandled ETextureRenderTargetSampleCount entry %u"), (uint32)InSampleCount);
		return 1;
	}
}

/**
 * TextureRenderTarget2D
 *
 * 2D render target texture resource. This can be used as a target
 * for rendering as well as rendered as a regular 2D texture resource.
 *
 */
UCLASS(hidecategories=Object, hidecategories=Texture, hidecategories=Compression, hidecategories=Adjustments, hidecategories=Compositing, MinimalAPI)
class UTextureRenderTarget2D : public UTextureRenderTarget
{
	GENERATED_UCLASS_BODY()

	/** The width of the texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTarget2D, AssetRegistrySearchable)
	int32 SizeX;

	/** The height of the texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTarget2D, AssetRegistrySearchable)
	int32 SizeY;

	/** the color the texture is cleared to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D)
	FLinearColor ClearColor;

	/** The addressing mode to use for the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureRenderTarget2D, AssetRegistrySearchable)
	TEnumAsByte<enum TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureRenderTarget2D, AssetRegistrySearchable)
	TEnumAsByte<enum TextureAddress> AddressY;

	/** True to force linear gamma space for this render target */
	UPROPERTY()
	uint8 bForceLinearGamma:1;

	/** If true fast clear will be disabled on the rendertarget. */
	uint8 bNoFastClear:1;

	/** Whether to support storing HDR values, which requires more memory. */
	UPROPERTY()
	uint8 bHDR_DEPRECATED:1;

	/** Whether to support GPU sharing of the underlying native texture resource. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, meta=(DisplayName = "Shared"), AssetRegistrySearchable, AdvancedDisplay)
	uint8 bGPUSharedFlag : 1;

	/** 
	 * Format of the texture render target. 
	 * Data written to the render target will be quantized to this format, which can limit the range and precision.
	 * The largest format (RTF_RGBA32f) uses 16x more memory and bandwidth than the smallest (RTF_R8) and can greatly affect performance.  
	 * Use the smallest format that has enough precision and range for what you are doing.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTarget2D, AssetRegistrySearchable)
	TEnumAsByte<ETextureRenderTargetFormat> RenderTargetFormat;

	/** Whether this render target can be used as an unordered access view */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, AssetRegistrySearchable)
	uint8 bSupportsUAV : 1;

	/** Whether to support Mip maps for this render target texture */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, AssetRegistrySearchable)
	uint8 bAutoGenerateMips : 1;

	/** Sampler filter type for AutoGenerateMips. Defaults to match texture filter. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, AssetRegistrySearchable, meta = (editcondition = "bAutoGenerateMips"))
	TEnumAsByte<enum TextureFilter> MipsSamplerFilter;

	/**  AutoGenerateMips sampler address mode for U channel. Defaults to clamp. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, AssetRegistrySearchable, meta = (editcondition = "bAutoGenerateMips"))
	TEnumAsByte<enum TextureAddress> MipsAddressU;

	/**  AutoGenerateMips sampler address mode for V channel. Defaults to clamp. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TextureRenderTarget2D, AssetRegistrySearchable, meta = (editcondition = "bAutoGenerateMips"))
	TEnumAsByte<enum TextureAddress> MipsAddressV;

	/** Normally the format is derived from RenderTargetFormat, this allows code to set the format explicitly. */
	UPROPERTY()
	TEnumAsByte<enum EPixelFormat> OverrideFormat;

	/**
	 * Initialize the settings needed to create a render target texture
	 * and create its resource
	 * @param	InSizeX - width of the texture
	 * @param	InSizeY - height of the texture
	 * @param	InFormat - format of the texture
	 * @param	bInForceLinearGame - forces render target to use linear gamma space
	 */
	ENGINE_API void InitCustomFormat(uint32 InSizeX, uint32 InSizeY, EPixelFormat InOverrideFormat, bool bInForceLinearGamma);

	/** Initializes the render target, the format will be derived from the value of bHDR. */
	ENGINE_API void InitAutoFormat(uint32 InSizeX, uint32 InSizeY);

	/** Resizes the render target without recreating the FTextureResource.  Will not flush commands unless the render target resource doesnt exist */
	ENGINE_API void ResizeTarget(uint32 InSizeX, uint32 InSizeY);

	/**
	 * Utility for creating a new UTexture2D from a UTextureRenderTarget2D
	 * @param InOuter - Outer to use when constructing the new UTexture2D.
	 * @param InNewTextureName - Name of new UTexture2D object.
	 * @param InObjectFlags - Flags to apply to the new UTexture2D object
	 * @param InFlags - Various control flags for operation (see EConstructTextureFlags)
	 * @param InAlphaOverride - If specified, the values here will become the alpha values in the resulting texture
	 * @return New UTexture2D object.
	 */
	ENGINE_API UTexture2D* ConstructTexture2D(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags = CTF_Default, TArray<uint8>* InAlphaOverride = nullptr);
	
	UE_DEPRECATED(5.4, "Use CanConvertToTexture")
	ENGINE_API ETextureSourceFormat GetTextureFormatForConversionToTexture2D() const;

	//~ Begin UTextureRenderTarget Interface
	ENGINE_API virtual bool CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const override;
	ENGINE_API virtual TSubclassOf<UTexture> GetTextureUClass() const override;
	ENGINE_API virtual EPixelFormat GetFormat() const override;
	ENGINE_API virtual bool IsSRGB() const override;
	ENGINE_API virtual float GetDisplayGamma() const override;
	virtual ETextureClass GetRenderTargetTextureClass() const override { return ETextureClass::TwoD; }
	//~ End UTextureRenderTarget Interface

	/**
	 * Utility for updating an existing UTexture2D from a TextureRenderTarget2D
	 * @param InTexture2D					Texture which will contain the content of this render target after the call.
	 * @param InTextureFormat				Format in which the texture should be stored.
	 * @param Flags				Optional	Various control flags for operation (see EConstructTextureFlags)
	 * @param AlphaOverride		Optional	If non-null, the values here will become the alpha values in the resulting texture
	 */
	UE_DEPRECATED(5.4, "Use URenderTarget::UpdateTexture")
	ENGINE_API void UpdateTexture2D(UTexture2D* InTexture2D, ETextureSourceFormat InTextureFormat, uint32 Flags = CTF_Default, TArray<uint8>* AlphaOverride = nullptr);

	/**
	 * Utility for updating an existing UTexture2D from a TextureRenderTarget2D
	 * @param InTexture2D				Texture which will contain the content of this render target after the call.
	 * @param InTextureFormat			Format in which the texture should be stored.
	 * @param Flags						Various control flags for operation (see EConstructTextureFlags)
	 * @param AlphaOverride				If non-null, the values here will become the alpha values in the resulting texture
	 * @param TextureChangingDelegate	If the texture needs to be modified (as it's properties or content will change), this delegate will be called beforehand.
	 */
	UE_DEPRECATED(5.4, "Use URenderTarget::FOnTextureChangingDelegate")
	DECLARE_DELEGATE_OneParam(FTextureChangingDelegate, UTexture2D*);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.4, "Use URenderTarget::UpdateTexture")
	ENGINE_API void UpdateTexture2D(UTexture2D* InTexture2D, ETextureSourceFormat InTextureFormat, uint32 Flags, TArray<uint8>* AlphaOverride, FTextureChangingDelegate TextureChangingDelegate);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Updates (resolves) the render target texture immediately.
	 * Optionally clears the contents of the render target to green.
	 */
	ENGINE_API void UpdateResourceImmediate(bool bClearRenderTarget=true);

	//~ Begin UTexture Interface.
	virtual float GetSurfaceWidth() const override { return (float)SizeX; }
	virtual float GetSurfaceHeight() const override { return (float)SizeY; }
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	ENGINE_API virtual FTextureResource* CreateResource() override;
	ENGINE_API virtual EMaterialValueType GetMaterialType() const override;
	virtual uint32 CalcTextureMemorySizeEnum(ETextureMipCount Enum) const override;
	//~ End UTexture Interface.

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual FString GetDesc() override;
	//~ End UObject Interface

	inline int32 GetNumMips() const
	{
		return NumMips;
	}

	virtual ETextureRenderTargetSampleCount GetSampleCount() const;

private:
	int32	NumMips;
};



