// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TextureRenderTarget.h"
#include "TextureRenderTargetCube.generated.h"

class FTextureResource;
struct FPropertyChangedEvent;

/**
 * TextureRenderTargetCube
 *
 * Cube render target texture resource. This can be used as a target
 * for rendering as well as rendered as a regular cube texture resource.
 *
 */
UCLASS(hidecategories=Object, hidecategories=Texture, hidecategories=Compression, hidecategories=Adjustments, hidecategories=Compositing, MinimalAPI)
class UTextureRenderTargetCube : public UTextureRenderTarget
{
	GENERATED_UCLASS_BODY()

	/** The width of the texture.												*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureRenderTargetCube, AssetRegistrySearchable)
	int32 SizeX;

	/** the color the texture is cleared to */
	UPROPERTY()
	FLinearColor ClearColor;

	/** Specifies the format of the texture data.
	* When OverrideFormat is set to the default (PF_Unknown), the format is determined by bHDR.
	* Use OverrideFormat if you need to set the format explicitly from code instead. */
	UPROPERTY()
	TEnumAsByte<enum EPixelFormat> OverrideFormat;

	/** Determines the format of the render target.
	* When enabled, the format is 16-bit RGBA. When disabled, the format is 8-bit BGRA. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetCube, AssetRegistrySearchable)
	uint8 bHDR:1;

	/** Whether this render target can be used as an unordered access view */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetCube, AssetRegistrySearchable)
	uint8 bSupportsUAV : 1;

	/** True to force linear gamma space for this render target */
	UPROPERTY()
	uint8 bForceLinearGamma:1;

	/** Whether to support Mip maps for this render target texture */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetCube, AssetRegistrySearchable)
	uint8 bAutoGenerateMips : 1;

	/** Sampler filter type for AutoGenerateMips. Defaults to match texture filter. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetCube, AssetRegistrySearchable, meta = (editcondition = "bAutoGenerateMips"))
	TEnumAsByte<enum TextureFilter> MipsSamplerFilter;

	/** 
	* Initialize the settings needed to create a render target texture
	* and create its resource
	* @param	InSizeX - width of the texture
	* @param	InFormat - format of the texture
	*/
	ENGINE_API void Init(uint32 InSizeX, EPixelFormat InFormat);

	/** Initializes the render target, the format will be derived from the value of bHDR. */
	ENGINE_API void InitAutoFormat(uint32 InSizeX);

	ENGINE_API void UpdateResourceImmediate(bool bClearRenderTarget=true);

	/**
	 * Utility for creating a new UTextureCube from a UTextureRenderTargetCube
	 * @param InOuter - Outer to use when constructing the new UTextureCube.
	 * @param InNewTextureName - Name of new UTextureCube object.
	 * @param InObjectFlags - Flags to apply to the new UTextureCube object
	 * @param InFlags - Various control flags for operation (see EConstructTextureFlags)
	 * @param InAlphaOverride - If specified, the values here will become the alpha values in the resulting texture
	 * @return New UTextureCube object.
	 */
	ENGINE_API class UTextureCube* ConstructTextureCube(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags = CTF_Default, TArray<uint8>* InAlphaOverride = nullptr);

	//~ Begin UTexture Interface.
	virtual float GetSurfaceWidth() const  override { return static_cast<float>(SizeX); }
	// PVS-Studio notices that the implementation of GetSurfaceWidth is identical to this one
	// and warns us. In this case, it is intentional, so we disable the warning:
	virtual float GetSurfaceHeight()const  override { return static_cast<float>(SizeX); }
	virtual float GetSurfaceDepth() const override { return 0.0f; }
	virtual uint32 GetSurfaceArraySize() const override { return 6; }
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override;
	//~ End UTexture Interface.
	
	inline int32 GetNumMips() const
	{
		return NumMips;
	}
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual FString GetDesc() override;
	//~ End UObject Interface

	//~ Begin UTextureRenderTarget Interface
	virtual bool CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const override;
	virtual TSubclassOf<UTexture> GetTextureUClass() const override;
	virtual EPixelFormat GetFormat() const override;
	virtual bool IsSRGB() const override;
	virtual float GetDisplayGamma() const override;
	virtual ETextureClass GetRenderTargetTextureClass() const override { return ETextureClass::Cube; }
	//~ End UTextureRenderTarget Interface

private:
	int32	NumMips;
};



