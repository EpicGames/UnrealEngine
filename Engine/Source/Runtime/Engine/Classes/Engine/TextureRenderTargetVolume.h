// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TextureRenderTarget.h"
#include "TextureRenderTargetVolume.generated.h"

class FTextureResource;
struct FPropertyChangedEvent;

/**
 * TextureRenderTargetVolume
 *
 * Volume render target texture resource. This can be used as a target
 * for rendering as well as rendered as a regular Volume texture resource.
 *
 */
UCLASS(hidecategories=Object, hidecategories=Texture, hidecategories=Compression, hidecategories=Adjustments, hidecategories=Compositing, MinimalAPI)
class UTextureRenderTargetVolume : public UTextureRenderTarget
{
	GENERATED_UCLASS_BODY()

	/** The width of the texture.												*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureRenderTargetVolume, AssetRegistrySearchable)
	int32 SizeX;

	/** The height of the texture.												*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TextureRenderTargetVolume, AssetRegistrySearchable)
	int32 SizeY;

	/** The depth of the texture.												*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TextureRenderTargetVolume, AssetRegistrySearchable)
	int32 SizeZ;

	/** the color the texture is cleared to */
	UPROPERTY()
	FLinearColor ClearColor;
	
	/** Specifies the format of the texture data.											
	* When OverrideFormat is set to the default (PF_Unknown), the format is determined by bHDR.
	* Use OverrideFormat if you need to set the format explicitly from code instead. */
	UPROPERTY()
	TEnumAsByte<enum EPixelFormat> OverrideFormat;

	/** Determines the format of the render target. 
	* When enabled, the format is 16-bit RGBA. When disabled, the format is 8-bit BGRA. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetVolume, AssetRegistrySearchable)
	uint8 bHDR:1;

	/** Whether this render target can be used as an unordered access view */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureRenderTargetVolume, AssetRegistrySearchable)
	uint8 bSupportsUAV : 1;

	/** True to force linear gamma space for this render target */
	UPROPERTY()
	uint8 bForceLinearGamma:1;

	/** 
	* Initialize the settings needed to create a render target texture
	* and create its resource
	* @param	InSizeX - width of the texture
	* @param	InFormat - format of the texture
	*/
	ENGINE_API void Init(uint32 InSizeX, uint32 InSizeY, uint32 InSizeZ, EPixelFormat InFormat);

	/** Initializes the render target, the format will be derived from the value of bHDR. */
	ENGINE_API void InitAutoFormat(uint32 InSizeX, uint32 InSizeY, uint32 InSizeZ);

	ENGINE_API void UpdateResourceImmediate(bool bClearRenderTarget/*=true*/);

	/**
	 * Utility for creating a new UVolumeTexture from a UTextureRenderTargetVolume
	 * @param InOuter - Outer to use when constructing the new UVolumeTexture.
	 * @param InNewTextureName - Name of new UVolumeTexture object.
	 * @param InObjectFlags - Flags to apply to the new UVolumeTexture object
	 * @param InFlags - Various control flags for operation (see EConstructTextureFlags)
	 * @param InAlphaOverride - If specified, the values here will become the alpha values in the resulting texture
	 * @return New UVolumeTexture object.
	 */
	ENGINE_API class UVolumeTexture* ConstructTextureVolume(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags = CTF_Default, TArray<uint8>* InAlphaOverride = nullptr);

	//~ Begin UTexture Interface.
	virtual float GetSurfaceWidth() const  override { return static_cast<float>(SizeX); }
	virtual float GetSurfaceHeight()const  override { return static_cast<float>(SizeY); }
	virtual float GetSurfaceDepth()const  override { return static_cast<float>(SizeZ); }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override;
	//~ End UTexture Interface.

	inline int32 GetNumMips() const
	{
		return 1;
	}
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual FString GetDesc() override;
	//~ End UObject Interface

	//~ Begin UTextureRenderTarget Interface
	virtual bool CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const override;
	virtual TSubclassOf<UTexture> GetTextureUClass() const override;
	virtual EPixelFormat GetFormat() const override;
	virtual bool IsSRGB() const override;
	virtual float GetDisplayGamma() const override;
	virtual ETextureClass GetRenderTargetTextureClass() const override { return ETextureClass::Volume; }
	//~ End UTextureRenderTarget Interface
};



