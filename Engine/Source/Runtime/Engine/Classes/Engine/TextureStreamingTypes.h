// Copyright Epic Games, Inc. All Rights Reserved.

// Structs and defines used for texture streaming build

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "HAL/IConsoleManager.h"
#include "RHIDefinitions.h"
#include "SceneTypes.h"
#include "TextureStreamingTypes.generated.h"

class IPrimitiveComponent;
class ULevel;
class UMaterialInterface;
class UPrimitiveComponent;
class UStreamableRenderAsset;
class UTexture;
class UTexture;
struct FMaterialTextureInfo;
struct FMaterialTextureInfo;
struct FMeshUVChannelInfo;
struct FSlowTask;
struct FStreamingTextureBuildInfo;
struct FStreamingTextureBuildInfo;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(TextureStreamingBuild, Log, All);

// The PackedRelativeBox value that return the bound unaltered
inline constexpr uint32 PackedRelativeBox_Identity = 0xffff0000;

/** Information about a streaming texture/mesh that a primitive uses for rendering. */
USTRUCT()
struct FStreamingRenderAssetPrimitiveInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UStreamableRenderAsset> RenderAsset;

	/** 
	 * The streaming bounds of the texture/mesh, usually the component material bounds. 
	 * Usually only valid for registered component, as component bounds are only updated when the components are registered.
	 * otherwise only PackedRelativeBox can be used.Irrelevant when the component is not registered, as the component could be moved by ULevel::ApplyWorldOffset()
	 * In that case, only PackedRelativeBox is meaningful.
	 */
	UPROPERTY()
	FBoxSphereBounds Bounds;

	UPROPERTY()
	float TexelFactor;

	/** 
	 * When non zero, this represents the relative box used to compute Bounds, using the component bounds as reference.
	 * If available, this allows the renderable asset streamer to generate the level streaming data before the level gets visible.
	 * At that point, the component are not yet registered, and the bounds are unknown, but the precompiled build data is still available.
	 * Also allows to update the relative bounds after a level get moved around from ApplyWorldOffset.
	 */
	UPROPERTY()
	uint32 PackedRelativeBox;

	/** For mesh components, texel factors are their sphere bound diameters that are 0 when unregistered */
	UPROPERTY(Transient)
	uint32 bAllowInvalidTexelFactorWhenUnregistered : 1;

	/** Mesh texel factors aren't uv density and shouldn't be affected by component scales */
	UPROPERTY(Transient)
	uint32 bAffectedByComponentScale : 1;

	FStreamingRenderAssetPrimitiveInfo() : 
		RenderAsset(nullptr),
		Bounds(ForceInit), 
		TexelFactor(1.0f),
		PackedRelativeBox(0),
		bAllowInvalidTexelFactorWhenUnregistered(false),
		bAffectedByComponentScale(true)
	{
	}

	FStreamingRenderAssetPrimitiveInfo(
		UStreamableRenderAsset* InAsset,
		const FBoxSphereBounds& InBounds,
		float InTexelFactor,
		uint32 InPackedRelativeBox = 0,
		bool bInAllowInvalidTexelFactorWhenUnregistered = false,
		bool bInAffectedByComponentScale = true) :
		RenderAsset(InAsset),
		Bounds(InBounds), 
		TexelFactor(InTexelFactor),
		PackedRelativeBox(InPackedRelativeBox),
		bAllowInvalidTexelFactorWhenUnregistered(bInAllowInvalidTexelFactorWhenUnregistered),
		bAffectedByComponentScale(bInAffectedByComponentScale)
	{
	}

	bool CanBeStreamedByDistance(bool bOwningCompRegistered) const
	{
		return (TexelFactor > UE_SMALL_NUMBER || (!bOwningCompRegistered && bAllowInvalidTexelFactorWhenUnregistered))
			&& (Bounds.SphereRadius > UE_SMALL_NUMBER || !bOwningCompRegistered)
			&& ensure(FMath::IsFinite(TexelFactor));
	}
};

// Invalid streamable texture registration index
inline constexpr uint16 InvalidRegisteredStreamableTexture = (uint16)INDEX_NONE;

/**
 * Interface for texture streaming container
 */
struct ITextureStreamingContainer
{
#if WITH_EDITOR
	virtual void InitializeTextureStreamingContainer(uint32 InPackedTextureStreamingQualityLevelFeatureLevel) = 0;
	virtual uint16 RegisterStreamableTexture(UTexture* InTexture) = 0;
	virtual bool GetStreamableTexture(uint16 InTextureIndex, FString& OutTextureName, FGuid& OutTextureGuid) const { return false; }
#endif
};

/** 
 * This struct holds the result of TextureStreaming Build for each component texture, as referred by its used materials.
 * It is possible that the entry referred by this data is not actually relevant in a given quality / target.
 * It is also possible that some texture are not referred, and will then fall on fallbacks computation.
 * Because each component holds its precomputed data for each texture, this struct is designed to be as compact as possible.
 */
USTRUCT()
struct FStreamingTextureBuildInfo
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * The relative bounding box for this entry. The relative bounds is a bound equal or smaller than the component bounds and represent
	 * the merged LOD section bounds of all LOD section referencing the given texture. When the level transform is modified following
	 * a call to ApplyLevelTransform, this relative bound becomes deprecated as it was computed from the transform at build time.
	 */
	UPROPERTY()
	uint32 PackedRelativeBox;

	/** 
	 * The level scope identifier of the texture. When building the texture streaming data, each level holds a list of all referred texture Guids.
	 * This is required to prevent loading textures on platforms which would not require the texture to be loaded, and is a consequence of the texture
	 * streaming build not being platform specific (the same streaming data is build for every platform target). Could also apply to quality level.
	 */
	UPROPERTY()
	int32 TextureLevelIndex;

	/** 
	 * The texel factor for this texture. This represent the world size a texture square holding with unit UVs.
	 * This value is a combination of the TexelFactor from the mesh and also the material scale.
	 * It does not take into consideration StreamingDistanceMultiplier, or texture group scale.
	 */
	UPROPERTY()
	float TexelFactor;

	FStreamingTextureBuildInfo() : 
		PackedRelativeBox(0), 
		TextureLevelIndex(0), 
		TexelFactor(0) 
	{
	}

#if WITH_EDITOR
	/**
	 *	Set this struct to match the unpacked params.
	 *
	 *	@param	TextureStreamingContainer	[in,out]	Contains the list of registered streamable textures referred by components.
	 *	@param	RefBounds					[in]		The reference bounds used to compute the packed relative box.
	 *	@param	Info						[in]		The unpacked params.
	 */
	ENGINE_API void PackFrom(ITextureStreamingContainer* TextureStreamingContainer, const FBoxSphereBounds& RefBounds, const FStreamingRenderAssetPrimitiveInfo& Info);

	/** Returns hash of content. */
	ENGINE_API uint32 ComputeHash() const;
#endif
};

// The max number of uv channels processed in the texture streaming build.
#define TEXSTREAM_MAX_NUM_UVCHANNELS  4
// The initial texture scales (must be bigger than actual used values)
#define TEXSTREAM_INITIAL_GPU_SCALE 256
// The tile size when outputting the material texture scales.
#define TEXSTREAM_TILE_RESOLUTION 32
// The max number of textures processed in the material texture scales build.
#define TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL 256

struct FPrimitiveMaterialInfo
{
	FPrimitiveMaterialInfo() : Material(nullptr), UVChannelData(nullptr), PackedRelativeBox(0) {}

	// The material
	const UMaterialInterface* Material;

	// The mesh UV channel data
	const FMeshUVChannelInfo* UVChannelData;

	// The material bounds for the mesh.
	uint32 PackedRelativeBox;

	bool IsValid() const { return Material && UVChannelData && PackedRelativeBox != 0; }
};

enum ETextureStreamingBuildType
{
	TSB_MapBuild,
	TSB_ActorBuild,
	TSB_ValidationOnly,
	TSB_ViewMode
};

/** 
 * Context used to resolve FStreamingTextureBuildInfo to FStreamingRenderAssetPrimitiveInfo
 * The context make sure that build data and each texture is only processed once per component (with constant time).
 * It manage internally structures used to accelerate the binding between precomputed data and textures,
 * so that there is only one map lookup per texture per level. 
 * There is some complexity here because the build data does not reference directly texture objects to avoid hard references
 * which would load texture when the component is loaded, which could be wrong since the build data is built for a specific
 * feature level and quality level. The current feature and quality could reference more or less textures.
 * This requires the logic to not submit a streaming entry for precomputed data, as well as submit fallback data for 
 * texture that were referenced in the texture streaming build.
 */
class FStreamingTextureLevelContext
{
	/** Reversed lookup for ULevel::StreamingTextureGuids. */
	const TMap<FGuid, int32>* TextureGuidToLevelIndex;

	/** Whether the precomputed relative bounds should be used or not.  Will be false if the transform level was rotated since the last texture streaming build. */
	bool bUseRelativeBoxes;

	/** An id used to identify the component build data. */
	int32 BuildDataTimestamp;

	/** The last bound component texture streaming build data. */
	const TArray<FStreamingTextureBuildInfo>* ComponentBuildData;

	/** Level's StreamingTextures array resolved version of FName to UTexture*. */
	TArray<UTexture*> LevelStreamingTextures;

	struct FTextureBoundState
	{
		FTextureBoundState() {}

		FTextureBoundState(UTexture* InTexture) : BuildDataTimestamp(0), BuildDataIndex(0), Texture(InTexture) {}

		/** The timestamp of the build data to indentify whether BuildDataIndex is valid or not. */
		int32 BuildDataTimestamp;
		/** The ComponentBuildData Index referring this texture. */
		int32 BuildDataIndex;
		/**  The texture relative to this entry. */
		UTexture* Texture;
	};

	/*
	 * The component state of the each texture. Used to prevent processing each texture several time.
	 * Also used to find quickly the build data relating to each texture. 
	 */
	TArray<FTextureBoundState> BoundStates;

	EMaterialQualityLevel::Type QualityLevel;
	ERHIFeatureLevel::Type FeatureLevel;
	bool bIsBuiltDataValid;
	bool bForceNoUseBuiltData;

	ENGINE_API int32* GetBuildDataIndexRef(UTexture* Texture, bool bForceUpdate = false);
	ENGINE_API void UpdateQualityAndFeatureLevel(EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel, const ULevel* InLevel = nullptr);

public:

	// Needs InLevel to use precomputed data from 
	ENGINE_API FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const ULevel* InLevel = nullptr, const TMap<FGuid, int32>* InTextureGuidToLevelIndex = nullptr);
	ENGINE_API FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel, bool InUseRelativeBoxes);
	ENGINE_API FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const UPrimitiveComponent* Primitive);
	ENGINE_API FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const IPrimitiveComponent* Primitive);
	ENGINE_API ~FStreamingTextureLevelContext();

	void SetForceNoUseBuiltData(bool bValue) { bForceNoUseBuiltData = bValue; }

	ENGINE_API TArray<FStreamingRenderAssetPrimitiveInfo> GetStreamingRenderAssetInfoWithNULLRemoval(const IPrimitiveComponent* Component);

	ENGINE_API void UpdateContext(EMaterialQualityLevel::Type InQualityLevel, const ULevel* InLevel, const TMap<FGuid, int32>* InTextureGuidToLevelIndex);
	ENGINE_API void BindBuildData(const TArray<FStreamingTextureBuildInfo>* PreBuiltData);
	ENGINE_API void ProcessMaterial(const FBoxSphereBounds& ComponentBounds, const FPrimitiveMaterialInfo& MaterialData, float ComponentScaling, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures, bool bIsComponentBuildDataValid = false);
	UE_DEPRECATED(5.8, "The UPrimitiveComponent argument is no longer required")
	void ProcessMaterial(const FBoxSphereBounds& ComponentBounds, const FPrimitiveMaterialInfo& MaterialData, float ComponentScaling, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures, bool bIsComponentBuildDataValid, const UPrimitiveComponent* DebugComponent)
	{
		ProcessMaterial(ComponentBounds, MaterialData, ComponentScaling, OutStreamingTextures, bIsComponentBuildDataValid);
	}

	ENGINE_API bool CanUseTextureStreamingBuiltData() const;

	EMaterialQualityLevel::Type GetQualityLevel() { return QualityLevel; }
	ERHIFeatureLevel::Type GetFeatureLevel() { return FeatureLevel; }
};

enum class ELODStreamingCallbackResult
{
	Success,
	TimedOut,
	AssetRemoved,
	ComponentRemoved,
	StreamingDisabled,
	NotImplemented
};

typedef TFunction<void(IPrimitiveComponent*, UStreamableRenderAsset*, ELODStreamingCallbackResult)> FLODStreamingCallback;

/**
 * A Map that gives the (smallest) texture coordinate scale used when sampling each texture register of a shader.
 * The array index is the register index, and the value, is the coordinate scale.
 * Since a texture resource can be bound to several texture registers, it can related to different indices.
 * This is reflected in UMaterialInterface::GetUsedTexturesAndIndices where each texture is bound to 
 * an array of texture register indices.
 */
typedef TMap<UMaterialInterface*, TArray<FMaterialTextureInfo> > FTexCoordScaleMap;

/** A mapping between used material and levels for refering primitives. */
typedef TMap<UMaterialInterface*, TArray<ULevel*> > FMaterialToLevelsMap;

#if WITH_EDITOR
/** Build the texture streaming component data for an actor and save common data in a UActorTextureStreamingBuildDataComponent. */
ENGINE_API void BuildActorTextureStreamingData(AActor* InActor, EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel);

/** Build the texture streaming component data for a level based on actor's UActorTextureStreamingBuildDataComponent. */
ENGINE_API bool BuildLevelTextureStreamingComponentDataFromActors(ULevel* InLevel);

/** Resolves whether a texture is streamable independent of compression speed, and avoids unnecessary build settings generation work. */
ENGINE_API bool GetTextureIsStreamable(const UTexture& Texture);

/** Resolves whether a texture is streamable on the given platform, independent of compression speed, and avoids unnecessary build settings generation work. */
ENGINE_API bool GetTextureIsStreamableOnPlatform(const UTexture& Texture, const ITargetPlatform& TargetPlatform);
#endif

/** Build the shaders required for the texture streaming build. Returns whether or not the action was successful. */
ENGINE_API bool BuildTextureStreamingComponentData(UWorld* InWorld, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, bool bFullRebuild, FSlowTask& BuildTextureStreamingTask);

/** Check if the lighting build is dirty. Updates the needs rebuild status of the level and world. */
ENGINE_API void CheckTextureStreamingBuildValidity(UWorld* InWorld);

ENGINE_API uint32 PackRelativeBox(const FVector& RefOrigin, const FVector& RefExtent, const FVector& Origin, const FVector& Extent);
ENGINE_API uint32 PackRelativeBox(const FBox& RefBox, const FBox& Box);
ENGINE_API void UnpackRelativeBox(const FBoxSphereBounds& InRefBounds, uint32 InPackedRelBox, FBoxSphereBounds& OutBounds);

extern ENGINE_API TAutoConsoleVariable<int32> CVarStreamingUseNewMetrics;
extern ENGINE_API TAutoConsoleVariable<int32> CVarFramesForFullUpdate;

/** Reset the history of the value returned by GetAverageRequiredTexturePoolSize() */
ENGINE_API void ResetAverageRequiredTexturePoolSize();

/**
 * Returns the average value of the required texture pool "r.streaming.PoolSize" since engine start or since the last ResetAverageRequiredTexturePoolSize().
 * This value gives the perfect value for "r.streaming.PoolSize" so that the streamer would always have enough memory to stream in everything.
 * The requirements are different depending on whether GPoolSizeVRAMPercentage > 0 or not.
 * When GPoolSizeVRAMPercentage > 0, the non streaming mips are not accounted in the required pool size since StreamingPool = Min(TexturePool, GPoolSizeVRAMPercentage * VRAM - RenderTargets - NonStreamingTexture)
 * This means that the StreamingPool = TexturePool, unless there is not enough VRAM
 * Otherwise, when GPoolSizeVRAMPercentage == 0, StreamingPool = TexturePool - NonStreamingTexture. In which case "r.streaming.PoolSize" must account for the size of NonStreamingTexture
 */
ENGINE_API int64 GetAverageRequiredTexturePoolSize();
