// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Misc/QualifiedFrameTime.h"

#include "TimecodeProvider.generated.h"

/**
 * Possible states of TimecodeProvider.
 */
UENUM()
enum class ETimecodeProviderSynchronizationState
{
	/** TimecodeProvider has not been initialized or has been shutdown. */
	Closed,

	/** Unrecoverable error occurred during Synchronization. */
	Error,

	/** TimecodeProvider is currently synchronized with the source. */
	Synchronized,

	/** TimecodeProvider is initialized and being prepared for synchronization. */
	Synchronizing,
};

/**
 * A class responsible of fetching a timecode from a source.
 * Note, FApp::GetTimecode and FApp::GetTimecodeFramerate should be used to retrieve
 * the current system Timecode and Framerate.
 */
UCLASS(abstract, MinimalAPI)
class UTimecodeProvider : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Number of frames to subtract from the qualified frame time when GetDelayedQualifiedFrameTime or GetDelayedTimecode is called.
	 * @see GetDelayedQualifiedFrameTime, GetDelayedTimecode
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Settings")
	float FrameDelay = 0.f;

	/**
	 * Fetch current timecode from its source. e.g. From hardware/network/file/etc.
	 * It is recommended to cache the fetched timecode.
	*/
	UFUNCTION(BlueprintCallable, Category = "Provider")
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) { return false; };

	/**
	 * Update the state of the provider. Call it to ensure timecode and state are updated.
	 * It is suggested to fetch timecode from its source and cache it for the getters.
	*/
	UFUNCTION(BlueprintCallable, Category = "Provider")
	virtual void FetchAndUpdate() {}

	/**
	 * Return current frame time. 
	 * Since it may be called several times per frame, it is suggested to return a cached value.
	*/
	UFUNCTION(BlueprintCallable, Category = "Provider")
	virtual FQualifiedFrameTime GetQualifiedFrameTime() const PURE_VIRTUAL(UTimecodeProvider::GetQualifiedFrameTime, return FQualifiedFrameTime(););

	/**
	 * Return current frame time with FrameDelay applied.
	 * Only assume valid when GetSynchronizationState() returns Synchronized.
	*/
	UFUNCTION(BlueprintCallable, Category = "Provider")
	ENGINE_API FQualifiedFrameTime GetDelayedQualifiedFrameTime() const;

	/** Return the frame time converted into a timecode value. */
	UFUNCTION(BlueprintCallable, Category = "Provider")
	ENGINE_API FTimecode GetTimecode() const;

	/** Return the delayed frame time converted into a timecode value. */
	UFUNCTION(BlueprintCallable, Category = "Provider")
	ENGINE_API FTimecode GetDelayedTimecode() const;
	
	/** Return the frame rate of the frame time. */
	UFUNCTION(BlueprintCallable, Category = "Provider")
	FFrameRate GetFrameRate() const { return GetQualifiedFrameTime().Rate; }

	/** The state of the TimecodeProvider and if it's currently synchronized and the Timecode and FrameRate getters are valid. */
	UFUNCTION(BlueprintCallable, Category = "Provider")
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const PURE_VIRTUAL(UTimecodeProvider::IsSynchronized, return ETimecodeProviderSynchronizationState::Closed;);

public:
	/** This Provider became the Engine's Provider. */
	virtual bool Initialize(class UEngine* InEngine) PURE_VIRTUAL(UTimecodeProvider::Initialize, return false;);

	/** This Provider stopped being the Engine's Provider. */
	virtual void Shutdown(class UEngine* InEngine) PURE_VIRTUAL(UTimecodeProvider::Shutdown, );

	/** Whether this provider supports format autodetection. */
	virtual bool SupportsAutoDetected() const
	{
		return false;
	}
	
	/** Set the autodetected flag on this provider. */
	virtual void SetIsAutoDetected(bool bInIsAutoDetected)
	{
	}
	
	/** Get whether this provider is currently using autodetection. */
	virtual bool IsAutoDetected() const
	{
		return false;
	}
};
