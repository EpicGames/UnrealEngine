// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Components/TimelineComponent.h"
#include "Engine/Blueprint.h"
#include "TimelineTemplate.generated.h"

class UTimelineTemplate;

USTRUCT()
struct FTTTrackBase
{
	/** Enum to indicate whether this is an event track, a float interp track or a vector interp track */
	enum ETrackType
	{
		TT_Event,
		TT_FloatInterp,
		TT_VectorInterp,
		TT_LinearColorInterp,
	};

	GENERATED_USTRUCT_BODY()

private:
	/** Name of this track */
	UPROPERTY()
	FName TrackName;

public:
	/** Flag to identify internal/external curve*/
	UPROPERTY()
	bool bIsExternalCurve;
#if WITH_EDITORONLY_DATA
	/** Whether or not this track is expanded in the UI. */
	UPROPERTY()
	bool bIsExpanded;
	/** Whether or not this track has its curve's view synchronized with the other curve views. */
	UPROPERTY()
	bool bIsCurveViewSynchronized;
#endif

	/** Determine if Tracks are the same */
	ENGINE_API bool operator == (const FTTTrackBase& T2) const;

	FTTTrackBase()
		: TrackName(NAME_None)
		, bIsExternalCurve(false)
#if WITH_EDITORONLY_DATA
		, bIsExpanded(true)
		, bIsCurveViewSynchronized(true)
#endif
	{}

	virtual ~FTTTrackBase() = default;

	FName GetTrackName() const { return TrackName; }
	ENGINE_API virtual void SetTrackName(FName NewTrackName, UTimelineTemplate* OwningTimeline);
};

USTRUCT()
struct FTTTrackId
{
	GENERATED_USTRUCT_BODY()

	FTTTrackId (int32 InType, int32 InIndex)
		: TrackType(InType)
		, TrackIndex(InIndex)
	{}

	FTTTrackId()
		: FTTTrackId(0, 0)
	{}

	UPROPERTY()
	int32 TrackType;
	UPROPERTY()
	int32 TrackIndex;
};

/** Structure storing information about one event track */
USTRUCT()
struct FTTEventTrack : public FTTTrackBase
{
	GENERATED_USTRUCT_BODY()

private:
	UPROPERTY()
	FName FunctionName;

public:
	/** Curve object used to store keys */
	UPROPERTY()
	TObjectPtr<class UCurveFloat> CurveKeys;

	/** Determine if Tracks are the same */
	ENGINE_API bool operator == (const FTTEventTrack& T2) const;

	FName GetFunctionName() const { return FunctionName; }
	ENGINE_API virtual void SetTrackName(FName NewTrackName, UTimelineTemplate* OwningTimeline) override final;

	FTTEventTrack()
		: FTTTrackBase()
		, CurveKeys(nullptr)
	{}
};

USTRUCT()
struct FTTPropertyTrack : public FTTTrackBase
{
	GENERATED_USTRUCT_BODY()

	FName GetPropertyName() const { return PropertyName; }
	ENGINE_API virtual void SetTrackName(FName NewTrackName, UTimelineTemplate* OwningTimeline) override final;

private:
	UPROPERTY()
	FName PropertyName;
};

/** Structure storing information about one float interpolation track */
USTRUCT()
struct FTTFloatTrack : public FTTPropertyTrack
{
	GENERATED_USTRUCT_BODY()

	/** Curve object used to define float value over time */
	UPROPERTY()
	TObjectPtr<class UCurveFloat> CurveFloat;

	/** Determine if Tracks are the same */
	ENGINE_API bool operator == (const FTTFloatTrack& T2) const;

	FTTFloatTrack()
		: FTTPropertyTrack()
		, CurveFloat(nullptr)
	{}
	
};

/** Structure storing information about one vector interpolation track */
USTRUCT()
struct FTTVectorTrack : public FTTPropertyTrack
{
	GENERATED_USTRUCT_BODY()

	/** Curve object used to define vector value over time */
	UPROPERTY()
	TObjectPtr<class UCurveVector> CurveVector;

	/** Determine if Tracks are the same */
	ENGINE_API bool operator == (const FTTVectorTrack& T2) const;

	FTTVectorTrack()
		: FTTPropertyTrack()
		, CurveVector(nullptr)
	{}
	
};

/** Structure storing information about one color interpolation track */
USTRUCT()
struct FTTLinearColorTrack : public FTTPropertyTrack
{
	GENERATED_USTRUCT_BODY()

	/** Curve object used to define color value over time */
	UPROPERTY()
	TObjectPtr<class UCurveLinearColor> CurveLinearColor;

	/** Determine if Tracks are the same */
	ENGINE_API bool operator == (const FTTLinearColorTrack& T2) const;

	FTTLinearColorTrack()
		: FTTPropertyTrack()
		, CurveLinearColor(nullptr)
	{}
	
};

UCLASS(MinimalAPI)
class UTimelineTemplate : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Length of this timeline */
	UPROPERTY(EditAnywhere, Category=TimelineTemplate)
	float TimelineLength;

	/** How we want the timeline to determine its own length (e.g. specified length, last keyframe) */
	UPROPERTY(EditAnywhere, Category=TimelineTemplate)
	TEnumAsByte<ETimelineLengthMode> LengthMode;

	/** If we want the timeline to auto-play */
	UPROPERTY(EditAnywhere, Category=TimelineTemplate)
	uint8 bAutoPlay:1;

	/** If we want the timeline to loop */
	UPROPERTY(EditAnywhere, Category=TimelineTemplate)
	uint8 bLoop:1;

	/** If we want the timeline to loop */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TimelineTemplate)
	uint8 bReplicated:1;

	/** If we want the timeline to ignore global time dilation */
	UPROPERTY(EditAnywhere, Category = TimelineTemplate)
	uint8 bIgnoreTimeDilation : 1;

	/** Set of event tracks */
	UPROPERTY()
	TArray<FTTEventTrack> EventTracks;

	/** Set of float interpolation tracks */
	UPROPERTY()
	TArray<FTTFloatTrack> FloatTracks;

	/** Set of vector interpolation tracks */
	UPROPERTY()
	TArray<FTTVectorTrack> VectorTracks;

	/** Set of linear color interpolation tracks */
	UPROPERTY()
	TArray<FTTLinearColorTrack> LinearColorTracks;

	/** Metadata information for this timeline */
	UPROPERTY(EditAnywhere, Category=BPVariableDescription)
	TArray<FBPVariableMetaDataEntry> MetaDataArray;

	UPROPERTY(duplicatetransient)
	FGuid	TimelineGuid;

	/** Allow control of Timeline component TickGroup assignment via TimelineTemplates */
	UPROPERTY()
	TEnumAsByte<ETickingGroup> TimelineTickGroup;

	/** Find the index of a float track */
	int32 FindFloatTrackIndex(FName FloatTrackName) const;
	
	/** Find the index of a vector track */
	int32 FindVectorTrackIndex(FName VectorTrackName) const;
	
	/** Find the index of an event track */
	int32 FindEventTrackIndex(FName EventTrackName) const;

	/** Find the index of a linear color track */
	int32 FindLinearColorTrackIndex(FName ColorTrackName) const;

	/** @return true if a name is valid for a new track (it isn't already in use) */
	ENGINE_API bool IsNewTrackNameValid(FName NewTrackName) const;
	
	/** Get the name of the function we expect to find in the owning actor that we will bind the update event to */
	FName GetUpdateFunctionName() const { return UpdateFunctionName; }

	/** Get the name of the function we expect to find in the owning actor that we will bind the finished event to */
	FName GetFinishedFunctionName() const { return FinishedFunctionName; }

	/** Set a metadata value on the timeline */
	ENGINE_API void SetMetaData(FName Key, FString Value);

	/** Gets a metadata value on the timeline; asserts if the value isn't present.  Check for validiy using FindMetaDataEntryIndexForKey. */
	ENGINE_API const FString& GetMetaData(FName Key) const;

	/** Clear metadata value on the timeline */
	ENGINE_API void RemoveMetaData(FName Key);

	/** Find the index in the array of a timeline entry */
	ENGINE_API int32 FindMetaDataEntryIndexForKey(FName Key) const;

	/** Returns the variable name for the timeline */
	FName GetVariableName() const { return VariableName; }

	/** Returns the property name for the timeline's direction pin */
	FName GetDirectionPropertyName() const { return DirectionPropertyName; }

	/* Create a new unique name for a curve */
	ENGINE_API static FString MakeUniqueCurveName(UObject* Obj, UObject* InOuter);

	ENGINE_API static FString TimelineVariableNameToTemplateName(FName Name);

	ENGINE_API void GetAllCurves(TSet<class UCurveBase*>& InOutCurves) const;

	ENGINE_API FTTTrackId GetDisplayTrackId(int32 DisplayTrackIndex);
	ENGINE_API int32 GetNumDisplayTracks() const;
	ENGINE_API void RemoveDisplayTrack(int32 DisplayTrackIndex);
	ENGINE_API void MoveDisplayTrack(int32 DisplayTrackIndex, int32 DirectionDelta);
	ENGINE_API void AddDisplayTrack(FTTTrackId NewTrackId);


	//~ Begin UObject Interface
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostLoad() override;
	virtual bool Rename(const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostInitProperties() override;
	//~ End UObject Interface

private:
	/** Helper function to make sure all the cached FNames for the timeline template are updated relative to the current name of the template. */
	ENGINE_API void UpdateCachedNames();

	friend struct FUpdateTimelineCachedNames;

	UPROPERTY()
	FName VariableName;

	UPROPERTY()
	FName DirectionPropertyName;

	UPROPERTY()
	FName UpdateFunctionName;

	UPROPERTY()
	FName FinishedFunctionName;

#if WITH_EDITORONLY_DATA
	/** Whether or not this track is expanded in the UI. */
	UPROPERTY()
	TArray <FTTTrackId> TrackDisplayOrder;
#endif


public:
	static const FString TemplatePostfix;
};

/**
 *  Helper class that gives external implementations permission to update cached names.
 */
struct FUpdateTimelineCachedNames
{
private:
	static void Execute(UTimelineTemplate* TimelineTemplate)
	{
		TimelineTemplate->UpdateCachedNames();
	}

	/** Grant external permission to the compilation manager for backwards compatibility purposes as PostLoad will not have occurred yet for compile on load. */
	friend struct FBlueprintCompilationManagerImpl;
	friend struct FRigVMBlueprintCompilationManagerImpl;

	/** Grant external permission to Blueprint editor utility methods. For example, duplicating a Blueprint for conversion to C++ requires that cached names be updated. */
	friend class FBlueprintEditorUtils;
};
