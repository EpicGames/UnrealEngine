// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimerHandle.generated.h"

DECLARE_DYNAMIC_DELEGATE(FTimerDynamicDelegate);

/** Unique handle that can be used to distinguish timers that have identical delegates. */
USTRUCT(BlueprintType)
struct FTimerHandle
{
	GENERATED_BODY()

	friend class FTimerManager;
	friend struct FTimerHeapOrder;

	FTimerHandle()
	: Handle(0)
	{
	}

	/** True if this handle was ever initialized by the timer manager */
	bool IsValid() const
	{
		return Handle != 0;
	}

	/** Explicitly clear handle */
	void Invalidate()
	{
		Handle = 0;
	}

	bool operator==(const FTimerHandle& Other) const
	{
		return Handle == Other.Handle;
	}

	bool operator!=(const FTimerHandle& Other) const
	{
		return Handle != Other.Handle;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%llx"), Handle);
	}

private:
	static constexpr uint32 IndexBits        = 24;
	static constexpr uint32 SerialNumberBits = 40;

	static_assert(IndexBits + SerialNumberBits == 64, "The space for the timer index and serial number should total 64 bits");

	static constexpr int32  MaxIndex        = (int32)1 << IndexBits;
	static constexpr uint64 MaxSerialNumber = (uint64)1 << SerialNumberBits;

	void SetIndexAndSerialNumber(int32 Index, uint64 SerialNumber)
	{
		check(Index >= 0 && Index < MaxIndex);
		check(SerialNumber < MaxSerialNumber);
		Handle = (SerialNumber << IndexBits) | (uint64)(uint32)Index;
	}

	inline int32 GetIndex() const
	{
		return (int32)(Handle & (uint64)(MaxIndex - 1));
	}

	inline uint64 GetSerialNumber() const
	{
		return Handle >> IndexBits;
	}

	UPROPERTY(Transient)
	uint64 Handle;

	friend uint32 GetTypeHash(const FTimerHandle& InHandle)
	{
		return GetTypeHash(InHandle.Handle);
	}
};
