// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** 
 * Safezone related declarations. Used by GameViewportClient for rendering 'safe zones' on screen.
 *
 * @see UGameViewportClient 
 * @see ESafeZoneType
 * @see FTitleSafeZoneArea 
 */

/**
 * The 4 different kinds of safezones
 */
enum ESafeZoneType
{
	eSZ_TOP,
	eSZ_BOTTOM,
	eSZ_LEFT,
	eSZ_RIGHT,
	eSZ_MAX,
};

/** Max/Recommended screen viewable extents as a percentage */
struct FTitleSafeZoneArea
{
	float MaxPercentX;
	float MaxPercentY;
	float RecommendedPercentX;
	float RecommendedPercentY;

	FTitleSafeZoneArea()
		: MaxPercentX(0)
		, MaxPercentY(0)
		, RecommendedPercentX(0)
		, RecommendedPercentY(0)
	{
	}

};
