// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "RenderResource.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveFloat.h"
#include "ToonProfile.generated.h"

#define MAX_TOON_PROFILE_COUNT 256

class UTexture;
class UTexture2D;
class FRDGBuilder;
class FTextureReference;



// Struct with all the settings we want in UToonProfile, separate to make it easer to pass this data around in the engine.
USTRUCT(BlueprintType)
struct FToonProfileStruct
{
	GENERATED_USTRUCT_BODY()

	/**
	* Defines the diffuse ramp.
	*/
	UPROPERTY(Category = "Diffuse", EditAnywhere, meta = (AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers="false", ShowOutputGridNumbers="false"))
	FRuntimeCurveLinearColor DiffuseRamp;

	/**
	 * The texture containing ramp offset for diffuse. It is recommended for it to be grey scale level in [0, 1], internally remapped to [-1, 1] for positive and negative offset. A distance field is recommended.
	 */
	UPROPERTY(Category = "Diffuse", EditAnywhere)
	TObjectPtr<UTexture> DiffuseRampOffsetTexture;

	/**
	 * The strength of the offset applied by the ramp offset texture to the diffuse ramp.
	 */
	UPROPERTY(Category = "Diffuse", EditAnywhere, meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float DiffuseRampOffsetStrength;

	/**
	 * The multiplier applied on UVs before sampling.
	 */
	UPROPERTY(Category = "Diffuse", EditAnywhere, meta = (UIMin = 1.0, UIMax = 100.0, ClampMin = 1.0, SliderExponent = 2.0))
	float DiffuseRampOffsetSize;

	/**
	* Defines the specular ramp.
	*/
	UPROPERTY(Category = "Specular", EditAnywhere, meta = (AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers="false", ShowOutputGridNumbers="false"))
	FRuntimeCurveLinearColor SpecularRamp;

	/**
	 * The texture containing ramp offset for specular. It is recommended for it to be grey scale level in [0, 1], internally remapped to [-1, 1] for positive and negative offset. A distance field is recommended.
	 */
	UPROPERTY(Category = "Specular", EditAnywhere)
	TObjectPtr<UTexture> SpecularRampOffsetTexture;

	/**
	 * The strength of the offset applied by the ramp offset texture to the specular ramp.
	 */
	UPROPERTY(Category = "Specular", EditAnywhere, meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float SpecularRampOffsetStrength;

	/**
	 * The multiplier applied on UVs before sampling.
	 */
	UPROPERTY(Category = "Specular", EditAnywhere, meta = (UIMin = 1.0, UIMax = 100.0, ClampMin = 1.0, SliderExponent = 2.0))
	float SpecularRampOffsetSize;

	/**
	* Defines how deep light will penetrate inside the model for the shadow map occluder distance. 
	* This is effectively the extinction (unit=1/meter) of the virtual mesh medium. A value of 1 means that exp(-1)=36% of the energy will remain after 1 meter.
	*/
	UPROPERTY(Category = "Shadow", EditAnywhere, meta = (DisplayName = "Shadow Extinction", UIMin = 0.0, UIMax = 10.0, ClampMin = 0.0, ClampMax = 100.0, SliderExponent = 2.0))
	float ShadowExtinctionCoefficient;

	/**
	* Defines the curve distribution used to control how the four channels for the shadow hatching pattern texture are distributed along the evaluated shadow/transmittance.
	* Shadow/Transmittance => Shadow hatching ramp evaluation => Value is used to interpolate the shadow hatching pattern RGBA channels (1.0 maps to white, 0.75 to R, 0.5 to G, 0.25 to B, 0.0 to A)
	*/
	UPROPERTY(Category = "Shadow", EditAnywhere, meta = (AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers = "false", ShowOutputGridNumbers = "false"))
	FRuntimeFloatCurve ShadowHatchingPatternDistributionRamp;

	/**
	 * The shadow texture defining the hatching pattern applied to the grey scale shadow. 
	 * R applied first close onto the lit front surface and progressively G, B and A channels lerped when away from the lit front surface as the shadow/transmittance gets darker.
	 * Typically R, G, B and A would have light to bold hatching bHasDiffuseDitherPatternTexturepattern.
	 */
	UPROPERTY(Category = "Shadow", EditAnywhere)
	TObjectPtr<UTexture> ShadowHatchingPatternTexture;

	/**
	 * The multiplier applied on UVs before sampling.
	 */
	UPROPERTY(Category = "Shadow", EditAnywhere, meta = (UIMin = 1.0, UIMax = 100.0, ClampMin = 1.0, SliderExponent = 2.0))
	float ShadowHatchingPatternSize;

	/**
	 * The uniform strength of the shadow hatching pattern.
	 */
	UPROPERTY(Category = "Shadow", EditAnywhere, meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float ShadowHatchingPatternStrength;

	/**
	 * The scale the diffuse global illumination contribution.
	 */
	UPROPERTY(Category = "Global Illumination", EditAnywhere, meta = (DisplayName = "Diffuse Indirect Scale", UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float DiffuseIndirectScale;

	/**
	 * The scale the specular global illumination contribution.
	 */
	UPROPERTY(Category = "Global Illumination", EditAnywhere, meta = (DisplayName = "Specular Indirect Scale", UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float SpecularIndirectScale;

	/**
	* Defines the specular global illumination ramp that can be used to quantise the specular indirect reflections. Requires r.Substrate.Experimental.ToonReflectionQuantizationEnabled=1.
	*/
	UPROPERTY(Category = "Global Illumination", EditAnywhere, meta = (DisplayName = "Specular Indirect Ramp", AllowZoomOutput = "false", ShowZoomButtons = "false", ViewMinInput = "0", ViewMaxInput = "1", ViewMinOutput = "0", ViewMaxOutput = "1", TimelineLength = "1", ShowInputGridNumbers = "false", ShowOutputGridNumbers = "false"))
	FRuntimeFloatCurve SpecularIndirectRamp;

	/**
	 * The scale of the the specular indirect ramp application over the range of luminance. When 0, the ramp is not applied to specular indirect reflections. Requires r.Substrate.Experimental.ToonReflectionQuantizationEnabled=1.
	 */
	UPROPERTY(Category = "Global Illumination", EditAnywhere, meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0))
	float SpecularIndirectRampRepetition;

	/**
	 * Experimental. Include shadows as part of the unified diffuse ramp evaluation.
	 */
	UPROPERTY(Category = "Experimental", EditAnywhere, meta = (DisplayName = "Diffuse Ramp Includes Shadow"))
	uint8 bDiffuseRampIncludeShadow : 1;

	FToonProfileStruct();

	void Invalidate()
	{
		*this = FToonProfileStruct();
	}

	float GetShadowExtinctionCoefficientInInvCentimeter() const
	{
		// Convert from 1/m to 1/cm
		return ShadowExtinctionCoefficient * (1.0f / 100.0f);
	}
};

/**
 * Toon profile asset, can be specified at a material. 
 * Don't change at runtime. All properties in here are per material.
 */
UCLASS(autoexpandcategories = ToonProfile, MinimalAPI, meta = (DevelopmentStatus = "Experimental"))
class UToonProfile : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = UToonProfile, EditAnywhere, meta = (ShowOnlyInnerProperties))
	struct FToonProfileStruct Settings;

	UPROPERTY()
	FGuid Guid;

	//~ Begin UObject Interface
	virtual void BeginDestroy();
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode);
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	//~ End UObject Interface
};

namespace ToonProfile
{
// Atlas - Initializes or updates the contents of the toon profile texture.
ENGINE_API void UpdateToonProfileTextureAtlas(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);

// Atlas - Returns the toon profile texture if it exists, or null.
ENGINE_API FRHITexture* GetToonProfileTextureAtlas();

// Atlas - Returns the toon profile texture if it exists, or black.
ENGINE_API FRHITexture* GetToonProfileTextureAtlasWithFallback();

// Profile - Initializes or updates the contents of the toon profile texture.
ENGINE_API int32 AddOrUpdateProfile(const UToonProfile* InProfile, const FGuid& InGuid, const FToonProfileStruct InSettings);

// Profile - Returns the toon profile ID shader parameter name
ENGINE_API FName GetToonProfileParameterName(const UToonProfile* InProfile);

// Profile - Returns the toon profile ID for a given toon Profile object in [0, 255]
ENGINE_API float GetToonProfileId(const UToonProfile* In);

// Profile - Returns the shader parameter name for a toon profile.
ENGINE_API FName CreateToonProfileParameterName(UToonProfile* InProfile);
}