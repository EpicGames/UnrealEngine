// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "TriggerBase.generated.h"

class UBillboardComponent;
class UShapeComponent;

/** An actor used to generate collision events (begin/end overlap) in the level. */
UCLASS(ClassGroup=Common, abstract, ConversionRoot, MinimalAPI)
class ATriggerBase : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** Shape component used for collision */
	UPROPERTY(Category = TriggerBase, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UShapeComponent> CollisionComponent;

#if WITH_EDITORONLY_DATA
	/** Billboard used to see the trigger in the editor */
	UPROPERTY(Category = TriggerBase, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif

public:
	/** Returns CollisionComponent subobject **/
	UShapeComponent* GetCollisionComponent() const { return CollisionComponent; }

#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};



