// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TriggerBase.h"
#include "TriggerCapsule.generated.h"

/** A capsule shaped trigger, used to generate overlap events in the level */
UCLASS(MinimalAPI)
class ATriggerCapsule : public ATriggerBase
{
	GENERATED_UCLASS_BODY()


#if WITH_EDITOR
	//~ Begin AActor Interface.
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif
};



