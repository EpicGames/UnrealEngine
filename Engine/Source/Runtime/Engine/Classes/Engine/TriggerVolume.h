// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "TriggerVolume.generated.h"

UCLASS(MinimalAPI)
class ATriggerVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
#endif // WITH_EDITOR	
	//~ End UObject Interface.
};



