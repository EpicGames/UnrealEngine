// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This is the base class for Twitter integration (each platform has a subclass)
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Engine/PlatformInterfaceBase.h"
#include "TwitterIntegrationBase.generated.h"

/** The possible twitter request methods */

UENUM()
enum ETwitterRequestMethod : int
{
	TRM_Get,
	TRM_Post,
	TRM_Delete,
	TRM_MAX,
};

UENUM()
enum ETwitterIntegrationDelegate : int
{
	TID_AuthorizeComplete,
	TID_TweetUIComplete,
	TID_RequestComplete,
	TID_MAX,
};

UCLASS(config=Engine)
class UTwitterIntegrationBase : public UPlatformInterfaceBase
{
	GENERATED_UCLASS_BODY()

	/**
	 * Perform any needed initialization
	 */
	UFUNCTION()
	virtual void Init();

	/**
	 * @return true if the user is allowed to use the Tweet UI
	 */
	UFUNCTION()
	virtual bool CanShowTweetUI();

	/**
	 * Kicks off a tweet, using the platform to show the UI. If this returns false, or you are on a platform that doesn't support the UI,
	 * you can use the TwitterRequest method to perform a manual tweet using the Twitter API
	 *
	 * @param InitialMessage [optional] Initial message to show
	 * @param URL [optional] URL to attach to the tweet
	 * @param Picture [optional] Name of a picture (stored locally, platform subclass will do the searching for it) to add to the tweet
	 *
	 * @return true if a UI was displayed for the user to interact with, and a TID_TweetUIComplete will be sent
	 */
	UFUNCTION()
	virtual bool ShowTweetUI(const FString& InitialMessage = TEXT(""), const FString& URL = TEXT(""), const FString& Picture = TEXT(""));

	/**
	 * Starts the process of authorizing the local user(s). When TID_AuthorizeComplete is called, then GetNumAccounts() 
	 * will return a valid number of accounts
	 *
	 * @return true if the authorization process started, and TID_AuthorizeComplete delegates will be called
	 */
	UFUNCTION()
	virtual bool AuthorizeAccounts();

	/**
	 * @return The number of accounts that were authorized
	 */
	UFUNCTION()
	virtual int32 GetNumAccounts();

	/**
	 * @return the display name of the given Twitter account
	 */
	UFUNCTION()
	virtual FString GetAccountName(int32 AccountIndex);

	/**
	 * Kicks off a generic twitter request
	 *
	 * @param URL The URL for the twitter request
	 * @param KeysAndValues The extra parameters to pass to the request (request specific). Separate keys and values: < "key1", "value1", "key2", "value2" >
	 * @param Method The method for this request (get, post, delete)
	 * @param AccountIndex A user index if an account is needed, or -1 if an account isn't needed for the request
	 *
	 * @return true the request was sent off, and a TID_RequestComplete
	 */
	UFUNCTION()
	virtual bool TwitterRequest(const FString& URL, const TArray<FString>& ParamKeysAndValues, enum ETwitterRequestMethod RequestMethod, int32 AccountIndex);

};

