// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 *	User defined enumerations:
 *	- EnumType is always UEnum::ECppForm::Namespaced (to comfortable handle names collisions)
 *	- always have the last '_MAX' enumerator, that cannot be changed by user
 *	- Full enumerator name has form: '<enumeration path>::<short, user defined enumerator name>'
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "UserDefinedEnum.generated.h"

class UEnumCookedMetaData;

/** 
 *	An Enumeration is a list of named values.
 */
UCLASS(MinimalAPI)
class UUserDefinedEnum : public UEnum
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint32 UniqueNameIndex;

	/** Shows up in the content browser when the enum asset is hovered */
	UPROPERTY(EditAnywhere, Category=Description, meta=(MultiLine=true))
	FText EnumDescription;
#endif //WITH_EDITORONLY_DATA

	/**
	 * De-facto display names for enum entries mapped against their raw enum name (UEnum::GetNameByIndex).
	 * To sync DisplayNameMap use FEnumEditorUtils::EnsureAllDisplayNamesExist.
	 */
	UPROPERTY()
	TMap<FName, FText> DisplayNameMap;

public:
	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

	/**
	 * Generates full enum name give enum name.
	 * For UUserDefinedEnum full enumerator name has form: '<enumeration path>::<short, user defined enumerator name>'
	 *
	 * @param InEnumName Enum name.
	 * @return Full enum name.
	 */
	ENGINE_API virtual FString GenerateFullEnumName(const TCHAR* InEnumName) const override;

	/**
	 *	Try to update value in enum variable after an enum's change.
	 *
	 *	@param EnumeratorIndex	old index
	 *	@return	new index
	 */
	ENGINE_API virtual int64 ResolveEnumerator(FArchive& Ar, int64 EnumeratorValue) const override;

	/** Overridden to read DisplayNameMap*/
	ENGINE_API virtual FText GetDisplayNameTextByIndex(int32 InIndex) const override;
	ENGINE_API virtual FString GetAuthoredNameStringByIndex(int32 InIndex) const override;

	ENGINE_API virtual bool SetEnums(TArray<TPair<FName, int64>>& InNames, ECppForm InCppForm, UEnum::EUnderlyingType InUnderlyingType, EEnumFlags InFlags, EAddMaxKeyIfMissing bAddMaxKeyIfMissing) override;

#if WITH_EDITOR
	//~ Begin UObject Interface
	ENGINE_API virtual bool Rename(const TCHAR* NewName = NULL, UObject* NewOuter = NULL, ERenameFlags Flags = REN_None) override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	ENGINE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	//~ End UObject Interface

	ENGINE_API FString GenerateNewEnumeratorName();
#endif	// WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:
	ENGINE_API virtual TSubclassOf<UEnumCookedMetaData> GetCookedMetaDataClass() const;

private:
	UEnumCookedMetaData* NewCookedMetaData();
	const UEnumCookedMetaData* FindCookedMetaData();
	void PurgeCookedMetaData();

	UPROPERTY()
	TObjectPtr<UEnumCookedMetaData> CachedCookedMetaDataPtr;
#endif // WITH_EDITORONLY_DATA
};
