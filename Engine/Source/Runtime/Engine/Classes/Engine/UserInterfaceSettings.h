// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Curves/CurveFloat.h"
#include "Engine/DeveloperSettings.h"
#include "Widgets/SWidget.h"

#include "UserInterfaceSettings.generated.h"

class UDPICustomScalingRule;

/** When to render the Focus Brush for widgets that have user focus. Based on the EFocusCause. */
UENUM()
enum class ERenderFocusRule : uint8
{
	/** Focus Brush will always be rendered for widgets that have user focus. */
	Always,
	/** Focus Brush will be rendered for widgets that have user focus not set based on pointer causes. */
	NonPointer,
	/** Focus Brush will be rendered for widgets that have user focus only if the focus was set by navigation. */
	NavigationOnly,
	/** Focus Brush will not be rendered. */
	Never,
};

/** The Side to use when scaling the UI. */
UENUM()
enum class EUIScalingRule : uint8
{
	/** Evaluates the scale curve based on the shortest side of the viewport. */
	ShortestSide,
	/** Evaluates the scale curve based on the longest side of the viewport. */
	LongestSide,
	/** Evaluates the scale curve based on the X axis of the viewport. */
	Horizontal,
	/** Evaluates the scale curve based on the Y axis of the viewport. */
	Vertical,
	/** ScaleToFit - Does not use scale curve. Emulates behavior of scale box by using DesignScreenSize and scaling the content relatively to it. */
	ScaleToFit,
	/** Custom - Allows custom rule interpretation. */
	Custom
};

/** The most used DPI value. */
UENUM()
enum class EFontDPI : uint8
{
	/** Best for working with with web-based design tools like Figma. */
	Standard UMETA(DisplayName = "72 DPI (Default)"),
	/** Resolution used internally by Unreal Engine. */
	Unreal UMETA(DisplayName = "96 DPI (Unreal Engine)"),

	Custom UMETA(Hidden)
};

class UWidget;
class UDPICustomScalingRule;

/**
 * 
 */
USTRUCT()
struct FHardwareCursorReference
{
	GENERATED_BODY()

	/**
	 * Specify the partial game content path to the hardware cursor.  For example,
	 *   DO:   Slate/DefaultPointer
	 *   DONT: Slate/DefaultPointer.cur
	 *
	 * NOTE: Having a 'Slate' directory in your game content folder will always be cooked, if
	 *       you're trying to decide where to locate these cursor files.
	 * 
	 * The hardware cursor system will search for platform specific formats first if you want to 
	 * take advantage of those capabilities.
	 *
	 * Windows:
	 *   .ani -> .cur -> .png
	 *
	 * Mac:
	 *   .tiff -> .png
	 *
	 * Linux:
	 *   .png
	 *
	 * Multi-Resolution Png Fallback
	 *  Because there's not a universal multi-resolution format for cursors there's a pattern we look for
	 *  on all platforms where pngs are all that is found instead of cur/ani/tiff.
	 *
	 *    Pointer.png
	 *    Pointer@1.25x.png
	 *    Pointer@1.5x.png
	 *    Pointer@1.75x.png
	 *    Pointer@2x.png
	 *    ...etc
	 */
	UPROPERTY(EditAnywhere, Category="Hardware Cursor")
	FName CursorPath;

	/**
	 * HotSpot needs to be in normalized (0..1) coordinates since it may apply to different resolution images.
	 * NOTE: This HotSpot is only used on formats that do not provide their own hotspot, e.g. Tiff, PNG.
	 */
	UPROPERTY(EditAnywhere, Category="Hardware Cursor", meta=( ClampMin=0, ClampMax=1 ))
	FVector2D HotSpot = FVector2D::ZeroVector;
};

/**
 * User Interface settings that control Slate and UMG.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="User Interface"), MinimalAPI)
class UUserInterfaceSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Rule to determine if we should render the Focus Brush for widgets that have user focus.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Focus")
	ERenderFocusRule RenderFocusRule;

	UPROPERTY(config, EditAnywhere, Category = "Hardware Cursors")
	TMap<TEnumAsByte<EMouseCursor::Type>, FHardwareCursorReference> HardwareCursors;

	UPROPERTY(config, EditAnywhere, Category = "Software Cursors", meta = ( MetaClass = "/Script/UMG.UserWidget" ))
	TMap<TEnumAsByte<EMouseCursor::Type>, FSoftClassPath> SoftwareCursors;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath DefaultCursor_DEPRECATED;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath TextEditBeamCursor_DEPRECATED;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath CrosshairsCursor_DEPRECATED;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath HandCursor_DEPRECATED;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath GrabHandCursor_DEPRECATED;
	
	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath GrabHandClosedCursor_DEPRECATED;

	// DEPRECATED 4.16
	UPROPERTY(config)
	FSoftClassPath SlashedCircleCursor_DEPRECATED;

	/**
	 * An optional application scale to apply on top of the custom scaling rules.  So if you want to expose a 
	 * property in your game title, you can modify this underlying value to scale the entire UI.
	 */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling")
	float ApplicationScale;

	/**
	 * The rule used when trying to decide what scale to apply.
	 */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling", meta=( DisplayName="DPI Scale Rule" ))
	EUIScalingRule UIScaleRule;

	/**
	 * Set DPI Scale Rule to Custom, and this class will be used instead of any of the built-in rules.
	 */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling", meta=( MetaClass="/Script/Engine.DPICustomScalingRule" ))
	FSoftClassPath CustomScalingRuleClass;

	/**
	 * Controls how the UI is scaled at different resolutions based on the DPI Scale Rule
	 */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling", meta=(
		DisplayName="DPI Curve",
		XAxisName="Resolution",
		YAxisName="Scale"))
	FRuntimeFloatCurve UIScaleCurve;

	/**
	 * If true, game window on desktop platforms will be created with high-DPI awareness enabled.
	 * Recommended to be enabled only if the game's UI allows users to modify 3D resolution scaling.
	 */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling", meta=( DisplayName="Allow High DPI in Game Mode" ))
	bool bAllowHighDPIInGameMode;

	/** Used only with ScaleToFit scaling rule. Defines native resolution for which were source UI textures created. DPI scaling will be 1.0 at this screen resolution. */
	UPROPERTY(config, EditAnywhere, Category="DPI Scaling|Scale To Fit Rule", meta=( DisplayName="Design Screen Size", ClampMin="1", UIMin="1" ))
	FIntPoint DesignScreenSize = FIntPoint(1920, 1080);

	/**
	 * If false, widget references will be stripped during cook for server builds and not loaded at runtime.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Widgets")
	bool bLoadWidgetsOnDedicatedServer;

	/** 
	 * Setting to authorize or not automatic variable creation.
	 * If true, variables will be created automatically, if the type created allows it. Drawback: it's easier to have a bad data architecture because various blueprint graph will have access to many variables.
	 * If false, variables are never created automatically, and you have to create them manually on a case by case basis.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Widgets")
	bool bAuthorizeAutomaticWidgetVariableCreation;

public:

	ENGINE_API virtual void PostInitProperties() override;

	/** Loads assets, if bForceLoadEverything is true it will load despite environment */
	ENGINE_API void ForceLoadResources(bool bForceLoadEverything = false) const;

	/** Gets the current scale of the UI based on the size of a viewport */
	ENGINE_API float GetDPIScaleBasedOnSize(FIntPoint Size) const;

	/** Convert font size from the editor render DPI to the project FontResolution DPI */
	ENGINE_API float ConvertFontSizeFromNativeToDisplay(float FontSize) const;

	/** Convert font size from the project FontResolution DPI to the editor render DPI */
	ENGINE_API float ConvertFontSizeFromDisplayToNative(float FontSize) const;

	/** Gets the Font DPI that should be used for display */
	ENGINE_API uint32 GetFontDisplayDPI() const;

#if WITH_EDITOR
	/**
	 * Gets the string to use when we need detailed info about the DPI.
	 * It will take into account the defined presets to give more context.
	 */
	ENGINE_API FText GetFontDPIDisplayString() const;

	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	float CalculateScale(FIntPoint Size, bool& bError) const;

	/** Convert from an entry in the EFontDPI enum to an int */
	static constexpr uint32 ConvertToFontDPI(EFontDPI inFontDPIEntry);

#if WITH_EDITOR
	/** Convert from an int to an entry in the EFontDPI enum. Will return the 'Unset' entry if no match is found */
	static constexpr EFontDPI ConvertToEFontDPI(uint32 inFontDPI);
#endif

	/**
	 * Controls the relationship between UMG font size and pixel height.
	 */
	UPROPERTY(config, EditAnywhere, Category = "UMG Fonts", meta = (DisplayName = "Font Resolution", EditCondition = "bUseCustomFontDPI", EditConditionHides, ClampMin = "1", ClampMax = "1000"))
	uint32 CustomFontDPI;

	/**
	 * Controls the relationship between UMG font size and pixel height.
	 */
	UPROPERTY(config, EditAnywhere, Category = "UMG Fonts", meta = (DisplayName = "Font Resolution", EditCondition = "!bUseCustomFontDPI", EditConditionHides))
	EFontDPI FontDPIPreset;

	/**
	 * To set your own custom value, check this box, then enter the value in the text box.
	 */
	UPROPERTY(config, EditAnywhere, Category = "UMG Fonts", meta = (DisplayName = "Use Custom DPI"))
	bool bUseCustomFontDPI;

	UFUNCTION()
	static bool ShouldDisplayDistanceFieldFontSettings();

public:
	UPROPERTY(config, EditAnywhere, Category = "UMG Fonts", meta = (EditCondition = "ShouldDisplayDistanceFieldFontSettings()", EditConditionHides))
	bool bEnableDistanceFieldFontRasterization;

private:
	// This is just holding on to references to prevent GC in the Editor (tied to the lifetime of this object)
	UPROPERTY(Transient)
	mutable TArray<TObjectPtr<UObject>> CursorClasses;

	UPROPERTY(Transient)
	mutable TObjectPtr<UClass> CustomScalingRuleClassInstance;

	UPROPERTY(Transient)
	mutable TObjectPtr<UDPICustomScalingRule> CustomScalingRule;

	mutable TOptional<FIntPoint> LastViewportSize;
	mutable float CalculatedScale;
};
