// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "ViewportSplitScreen.generated.h"

/** 
 * Various data declarations relating to split screen on a GameViewportClient
 *
 * @see ESplitScreenType
 * @see FPerPlayerSplitscreenData
 * @see FSplitscreenData
 */
 
/**
 * Enum of the different splitscreen types
 */
UENUM(BlueprintType)
namespace ESplitScreenType
{
	enum Type : int
	{
		// No split
		None,
		// 2 player horizontal split
		TwoPlayer_Horizontal,
		// 2 player vertical split
		TwoPlayer_Vertical,
		// 3 Player split with 1 player on top and 2 on bottom
		ThreePlayer_FavorTop,
		// 3 Player split with 1 player on bottom and 2 on top
		ThreePlayer_FavorBottom,
		//3 Player vertical split
		ThreePlayer_Vertical,
		//3 Player horizontal split
		ThreePlayer_Horizontal,
		//3 Player grid split
		ThreePlayer_Grid,
		// 4 Player grid split
		FourPlayer_Grid,
		// 4 Player vertical split
		FourPlayer_Vertical,
		// 4 Player horizontal split
		FourPlayer_Horizontal,

		SplitTypeCount
	};
}

/** Structure to store splitscreen data. */
struct FPerPlayerSplitscreenData
{
	float SizeX;
	float SizeY;
	float OriginX;
	float OriginY;


	FPerPlayerSplitscreenData()
		: SizeX(0)
		, SizeY(0)
		, OriginX(0)
		, OriginY(0)
	{
	}

	FPerPlayerSplitscreenData(float NewSizeX, float NewSizeY, float NewOriginX, float NewOriginY)
		: SizeX(NewSizeX)
		, SizeY(NewSizeY)
		, OriginX(NewOriginX)
		, OriginY(NewOriginY)
	{
	}

};

/** Structure containing all the player splitscreen datas per splitscreen configuration. */
struct FSplitscreenData
{
	TArray<struct FPerPlayerSplitscreenData> PlayerData;
};

