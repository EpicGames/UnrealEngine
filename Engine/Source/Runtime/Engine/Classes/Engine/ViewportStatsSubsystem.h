// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Tickable.h"
#include "Stats/Stats.h"
#include "Engine/EngineTypes.h"

#include "ViewportStatsSubsystem.generated.h"

class UWorld;
class FViewport;
class UCanvas;
class FCanvas;

DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FViewportDisplayCallback, FText&, OutText, FLinearColor&, OutColor);

typedef TFunction<bool(FText& OutTest, FLinearColor& OutColor)> FShouldDisplayFunc;

/** Wrapper to allow multiple delegate types to be held in one container for the subsystem */
struct FViewportDisplayDelegate
{
	FViewportDisplayDelegate() = default;

	FViewportDisplayDelegate(FViewportDisplayCallback const& D) 
		: FuncDynDelegate(D)
	{};

	FViewportDisplayDelegate(FShouldDisplayFunc&& Callback)
		: FuncCallback(MoveTemp(Callback))
	{};

	FViewportDisplayCallback FuncDynDelegate;
	FShouldDisplayFunc FuncCallback;

	bool Execute(FText& OutText, FLinearColor& OutColor);

	inline bool IsBound() const { return FuncDynDelegate.IsBound() || FuncCallback; }

	void Unbind();

	// Movable only
	FViewportDisplayDelegate(FViewportDisplayDelegate&&) = default;
	FViewportDisplayDelegate(const FViewportDisplayDelegate&) = delete;
	FViewportDisplayDelegate& operator=(FViewportDisplayDelegate&&) = default;
	FViewportDisplayDelegate& operator=(const FViewportDisplayDelegate&) = delete;
};

/**
* The Viewport Stats Subsystem offers the ability to add messages to the current 
* viewport such as "LIGHTING NEEDS TO BE REBUILT" and "BLUEPRINT COMPILE ERROR".
* 
* Example usage:
*
*	if (UViewportStatsSubsystem* ViewportSubsystem = GetWorld()->GetSubsystem<UViewportStatsSubsystem>())
*	{
*		// Bind a member function delegate to the subsystem...
*		FViewportDisplayCallback Callback;
*		Callback.BindDynamic(this, &UCustomClass::DisplayViewportMessage);
*		ViewportSubsystem->AddDisplayDelegate(Callback);
*		
*		// ... or use inline lambda functions
*		ViewportSubsystem->AddDisplayDelegate([](FText& OutText, FLinearColor& OutColor)
*		{
*			// Some kind of state management
*			OutText = NSLOCTEXT("FooNamespace", "Blarg", "Display message here");
*			OutColor = FLinearColor::Red;
*			return bShouldDisplay;
*		});
*	}
*/
UCLASS(Category = "Viewport Stats Subsystem", MinimalAPI)
class UViewportStatsSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:

	/**
	* Draw the messages in this subsystem to the given viewport
	*/
	ENGINE_API void Draw(FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, float MessageStartY);

	/**
	* Add a message to be displayed on the viewport of this world
	* 
	* @param Text		The text to be displayed
	* @param Color		Color of the text to be displayed
	* @param Duration	How long the text will be on screen, if 0 then it will stay indefinitely
	* @param DisplayOffset	A position offset that the message should use when displayed. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Viewport Stats Subsystem", meta = (AutoCreateRefTerm = "DisplayOffset"))
	ENGINE_API void AddTimedDisplay(FText Text, FLinearColor Color = FLinearColor::White, float Duration = 0.0f, const FVector2D& DisplayOffset = FVector2D::ZeroVector);

	/**
	* Add a dynamic delegate to the display subsystem.
	*
	* @param Callback	The callback the subsystem will use to determine if a message should be displayed or not
	*					Signature of callbacks should be: bool(FText& OutTest, FLinearColor& OutColor)
	*/
	UFUNCTION(BlueprintCallable, Category = "Viewport Stats Subsystem")
	ENGINE_API int32 AddDisplayDelegate(FViewportDisplayCallback const& Delegate);

	/**
	* Add a callback function to the display subsystem.
	*
	* @param Callback	The callback the subsystem will use to determine if a message should be displayed or not
	*					Signature of callbacks should be: bool(FText& OutTest, FLinearColor& OutColor)
	*/
	ENGINE_API int32 AddDisplayDelegate(FShouldDisplayFunc&& Callback);

	/**
	* Remove a callback function from the display subsystem
	*
	* @param IndexToRemove	The index in the DisplayDelegates array to remove. 
	*						This is the value returned from AddDisplayDelegate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Viewport Stats Subsystem")
	ENGINE_API void RemoveDisplayDelegate(const int32 IndexToRemove);
	
protected:

	//~USubsystem interface
	ENGINE_API void Deinitialize() override;
	//~End of USubsystem interface

	struct FUniqueDisplayData
	{
		FUniqueDisplayData() = default;

		FUniqueDisplayData(const FText& Text, const FLinearColor& Col, const FVector2D& Offset = FVector2D::ZeroVector)
		: DisplayText(Text)
		, DisplayColor(Col)
		, DisplayOffset(Offset)
		{};

		FText DisplayText = FText::GetEmpty();
		FLinearColor DisplayColor = FLinearColor::White;
		FVector2D DisplayOffset = FVector2D::ZeroVector;
	};

	/** Array of delegates that will be displayed if they return true when evaluated */
	TArray<FViewportDisplayDelegate> DisplayDelegates;

	/** Array of unique one-off timed messages to display each frame no matter what */
	TArray<TSharedPtr<FUniqueDisplayData>> UniqueDisplayMessages;
};
