// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndirectVirtualTextureDefinitions.h"
#include "Engine/DataAsset.h"
#include "RenderResource.h"
#include "IndirectVirtualTextureDefinitions.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Texture2D.h"
#include "TextureCollection.h"
#include "VirtualTexturing.h"
#include "VT/VirtualTextureBuildSettings.h"
#include "VirtualTextureCollection.generated.h"

class UVirtualTextureAdapter;
class FTextureResource;
class UTexture;
class UTextureCollection;
struct FTextureCollectionProducerData;

struct FVirtualTextureCollectionResource : public FTextureCollectionResource
{
	FVirtualTextureCollectionResource(UVirtualTextureCollection* InParent);
	virtual ~FVirtualTextureCollectionResource() override = default;

public: /** FRenderResource */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) final;
	virtual void ReleaseRHI() override;

	struct FTextureEntry
	{
		// Optional, the virtual producer
		FVirtualTextureProducerHandle VirtualProducerHandle;

		// Optional, physical texture
		// Keep the GT object around since RT resources may be released
		UTexture* PhysicalTexture = nullptr;

		// If true, this entry will be converted internally to the physical format
		// This also includes compression
		bool bRequiresAdapter = false;

		// The format of this texture, before adaptation
		EPixelFormat Format = PF_Unknown;

		// Number of blocks in this entry
		FUintVector2 BlockCount;

		// From square allocation size
		uint8 MaxLevel = 0;
	};
	
	UE::HLSL::FIndirectVirtualTextureUniform GetVirtualPackedUniform() const;
	
	FRHIShaderResourceView* GetVirtualCollectionRHI() const
	{
		check(VirtualCollectionRHISRV);
		return VirtualCollectionRHISRV;
	}

	FRHITexture* GetVirtualPageTable() const
	{
		return PageTable;
	}
	
	FRHIShaderResourceView* GetVirtualPhysicalTextureSRV() const
	{
		return PhysicalTextureSRV;
	}

	FVirtualTextureProducerHandle GetProducerHandle() const
	{
		return ProducerHandle;
	}
	
private:
	/** Find the first applicable format */
	void FindFirstFormat();
	
	/** Find a conservative format for the collection */
	void FindConservativeFormat();

	/** Compute the texture block layout */
	void ComputeLayout(FTextureCollectionProducerData& Data);
	
	/** Create the host side index table */
	void CreateIndexTable(FTextureCollectionProducerData& Data);

	/** Helper for error formatting */
	void FormatCollectionError(const TCHAR* Reason, uint32 TextureIndex);

private:
	TArray<UTexture*> Textures;

	/** Virtual build settings */
	FVirtualTextureBuildSettings BuildSettings;

	/** Software (compared to the bindless collection) collection buffer */
	FBufferRHIRef             VirtualCollectionRHI;
	FRHIShaderResourceViewRef VirtualCollectionRHISRV;

	/** Virtual resources are currently limited to a single physical texture and associated page table */
	FRHITexture*            PageTable          = nullptr;
	FRHIShaderResourceView* PhysicalTextureSRV = nullptr;

	/** Actual virtual table */
	IAllocatedVirtualTexture*     AllocatedVT = nullptr;
	FVirtualTextureProducerHandle ProducerHandle;
	
	TResourceArray<UE::HLSL::FIndirectVirtualTextureEntry> VirtualUniforms;

	EPixelFormat Format         = PF_Unknown;
	bool bIsSRGB                = false;
	bool bAllowFormatConversion = false;
};

UCLASS(MinimalAPI, Experimental)
class UVirtualTextureCollection : public UTextureCollection
{
	GENERATED_BODY()

public:	
	/**
	 * Allow format conversions, including differing compression schemes.
	 * This has a potentially large runtime overhead.
	 **/
	UPROPERTY(EditAnywhere, Category=VirtualTextureCollection)
	bool bAllowFormatConversion = true;

	/**
	 * If this texture collection if in SRGB, requires format conversion.
	 * Textures not matching this will be converted at runtime.
	 */
	UPROPERTY(EditAnywhere, Category=VirtualTextureCollection, meta = (editcondition = "bAllowFormatConversion"))
	bool bIsSRGB = true;

	/**
	 * The pixel format chosen at runtime, should PixelFormat not be specified.
	 **/
	UPROPERTY(VisibleAnywhere, Category=VirtualTextureCollection, AdvancedDisplay)
	TEnumAsByte<EPixelFormat> RuntimePixelFormat = PF_Unknown;
	
public: /** UObject */
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;

	/** UTextureCollection */
	virtual FTextureCollectionResource* CreateResource() override;
	virtual bool IsVirtualCollection() const override { return true; }

protected:
	friend struct FVirtualTextureCollectionResource;
	
#if WITH_EDITOR
	void ValidateVirtualCollection();
	void FormatCollectionError(const TCHAR* Reason, uint32 TextureIndex);
#endif //  WITH_EDITOR
};
