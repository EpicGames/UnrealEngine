// Copyright Epic Games, Inc. All Rights Reserved.

//
// A channel for exchanging voice data.
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/Channel.h"
#include "Net/VoiceDataCommon.h"
#include "VoiceChannel.generated.h"

class FInBunch;

UCLASS(transient, customConstructor, MinimalAPI)
class UVoiceChannel : public UChannel
{
	GENERATED_UCLASS_BODY()

	// Variables.

	/** The set of outgoing voice packets for this channel */
	FVoicePacketList VoicePackets;
	
	/**
	 * Default constructor
	 */
	UVoiceChannel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: UChannel(ObjectInitializer)
	{
		ChName = NAME_Voice;
	}


// UChannel interface
protected:
	/** 
	 * Cleans up any voice data remaining in the queue 
	 */
	ENGINE_API virtual bool CleanUp( const bool bForDestroy, EChannelCloseReason CloseReason ) override;

	/**
	 * Processes the in bound bunch to extract the voice data
	 *
	 * @param Bunch the voice data to process
	 */
	ENGINE_API virtual void ReceivedBunch(FInBunch& Bunch) override;

	/**
	 * Performs any per tick update of the VoIP state
	 */
	ENGINE_API virtual void Tick() override;

	/** Always tick voice channels for now. */
	virtual bool CanStopTicking() const override { return false; }

	/** Human readable information about the channel */
	virtual FString Describe() override
	{
		return FString("VoIP: ") + UChannel::Describe();
	}

public:

	/**
	 * Adds the voice packet to the list to send for this channel
	 *
	 * @param VoicePacket the voice packet to send
	 */
	ENGINE_API virtual void AddVoicePacket(TSharedPtr<class FVoicePacket> VoicePacket);
};
