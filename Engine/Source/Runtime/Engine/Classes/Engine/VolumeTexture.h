// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture2D.h"
#include "VolumeTexture.generated.h"

extern bool GSupportsVolumeTextureStreaming;

class FTextureResource;

UCLASS(hidecategories=(Object, Compositing, ImportSettings), MinimalAPI)
class UVolumeTexture : public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The derived data for this texture on this platform. */
	FTexturePlatformData* PrivatePlatformData;

public:

	/** Set the derived data for this texture on this platform. */
	ENGINE_API void SetPlatformData(FTexturePlatformData* PlatformData);
	/** Get the derived data for this texture on this platform. */
	ENGINE_API FTexturePlatformData* GetPlatformData();
	/** Get the const derived data for this texture on this platform. */
	ENGINE_API const FTexturePlatformData* GetPlatformData() const;

#if WITH_EDITOR
	TMap<FString, FTexturePlatformData*> CookedPlatformData;
#endif

#if WITH_EDITORONLY_DATA
	/** A (optional) reference texture from which the volume texture was built */
	UPROPERTY(EditAnywhere, Category=Source2D, meta=(DisplayName="Source Texture"))
	TObjectPtr<UTexture2D> Source2DTexture;	

	UPROPERTY()
	FGuid SourceLightingGuid_DEPRECATED;

	UPROPERTY(EditAnywhere, Category=Source2D, meta=(DisplayName="Tile Size X"))
	/** The reference texture tile size X */
	int32 Source2DTileSizeX;
	/** The reference texture tile size Y */
	UPROPERTY(EditAnywhere, Category=Source2D, meta=(DisplayName="Tile Size Y"))
	int32 Source2DTileSizeY;
#endif

	/** The addressing mode to use for the X, Y and Z axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Texture, meta = (DisplayName = "Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressMode = TA_Wrap;

	ENGINE_API bool UpdateSourceFromSourceTexture();
	
	/**
	 * Updates a volume texture from a user function, which allows for arbitrary UVolumeTexture objects to be filled.
	 * The pointer passed to the user function is owned by UVolumeTexture and should be cast to the required type
	 * before being filled up with the new voxel data.
	 * @param	Func	A function taking the x, y, z position as input and returning the texture data for this position.
	 * @param	SizeX	The width of the volume texture.
	 * @param	SizeY	The height of the volume texture.
	 * @param	SizeZ	The depth of the volume texture.
	 * @param	Format	The input volume texture format.
	 */
	ENGINE_API bool UpdateSourceFromFunction(TFunction<void(int32 /*PosX*/, int32 /*PosY*/, int32 /*PosZ*/, void* /*OutValue*/)> Func, int32 SizeX, int32 SizeY, int32 SizeZ, ETextureSourceFormat Format);

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual FString GetDesc() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface.

	ENGINE_API int32 GetSizeX() const;
	ENGINE_API int32 GetSizeY() const;
	ENGINE_API int32 GetSizeZ() const;
	ENGINE_API int32 GetNumMips() const;
	ENGINE_API EPixelFormat GetPixelFormat() const;

	//~ Begin UTexture Interface
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::Volume; }
	virtual float GetSurfaceWidth() const override { return static_cast<float>(GetSizeX()); }
	virtual float GetSurfaceHeight() const override { return static_cast<float>(GetSizeY()); }
	virtual float GetSurfaceDepth() const override { return static_cast<float>(GetSizeZ()); }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual TextureAddress GetTextureAddressX() const override { return AddressMode; }
	virtual TextureAddress GetTextureAddressY() const override { return AddressMode; }
	virtual TextureAddress GetTextureAddressZ() const override { return AddressMode; }
	virtual FTextureResource* CreateResource() override;
#if WITH_EDITOR
	ENGINE_API void SetDefaultSource2DTileSize();
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_VolumeTexture; }
	virtual FTexturePlatformData** GetRunningPlatformData() override;
#if WITH_EDITOR
	virtual bool IsDefaultTexture() const override;
	virtual TMap<FString, FTexturePlatformData*>* GetCookedPlatformData() override { return &CookedPlatformData; }
#endif // WITH_EDITOR
	//~ End UTexture Interface

	/** Creates and initializes a new VolumeTexture with the requested settings */
	ENGINE_API static class UVolumeTexture* CreateTransient(int32 InSizeX, int32 InSizeY, int32 InSizeZ, EPixelFormat InFormat = PF_B8G8R8A8, const FName InName = NAME_None);

	/**
	 * Calculates the size of this texture in bytes if it had MipCount miplevels streamed in.
	 *
	 * @param	MipCount	Number of mips to calculate size for, counting from the smallest 1x1 mip-level and up.
	 * @return	Size of MipCount mips in bytes
	 */
	uint32 CalcTextureMemorySize(int32 MipCount) const;

	/**
	 * Calculates the size of this texture if it had MipCount miplevels streamed in.
	 *
	 * @param	Enum	Which mips to calculate size for.
	 * @return	Total size of all specified mips, in bytes
	 */
	virtual uint32 CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const override;

#if WITH_EDITOR
	/**
	* Return maximum dimension for this texture type.
	*/
	virtual uint32 GetMaximumDimension() const override;

#endif

	//~ Begin UStreamableRenderAsset Interface
	virtual int32 CalcCumulativeLODSize(int32 NumLODs) const final override { return CalcTextureMemorySize(NumLODs); }
	virtual bool StreamOut(int32 NewMipCount) final override;
	virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) final override;
	//~ End UStreamableRenderAsset Interface

protected:

#if WITH_EDITOR
	void UpdateMipGenSettings();
#endif
};



