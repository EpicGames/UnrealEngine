// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "WindDirectionalSource.generated.h"

/** Actor that provides a directional wind source. Only affects SpeedTree assets. */
UCLASS(ClassGroup=Wind, showcategories=(Rendering, Transformation, DataLayers), MinimalAPI)
class AWindDirectionalSource : public AInfo
{
	GENERATED_UCLASS_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

	UPROPERTY(Category = WindDirectionalSource, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UWindDirectionalSourceComponent> Component;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
#endif

public:
	/** Returns Component subobject **/
	class UWindDirectionalSourceComponent* GetComponent() const { return Component; }

#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif
};



