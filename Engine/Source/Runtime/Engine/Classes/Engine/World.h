// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "HAL/ThreadSafeCounter.h"
#include "Online/CoreOnlineFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "GameTime.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"
#include "EngineDefines.h"
#include "Engine/PendingNetGame.h"
#include "Engine/LatentActionManager.h"
#include "Physics/PhysicsQueryHandler.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "Particles/WorldPSCPool.h"
#include "Containers/SortedMap.h"
#include "AudioDeviceHandle.h"
#include "Subsystems/WorldSubsystem.h"
#include "Subsystems/SubsystemCollection.h"
#include "RHIShaderPlatform.h"
#include "CollisionProfile.h"
#include "RHIFeatureLevel.h"
#include "Engine/WorldInitializationValues.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Engine/ScopedMovementUpdate.h"
#include "PSOPrecacheFwd.h"
#include "World.generated.h"

#define UE_API ENGINE_API

class ABrush;
class ACameraActor;
class AController;
class AGameModeBase;
class AGameStateBase;
class APhysicsVolume;
class APlayerController;
class AServerStreamingLevelsVisibility;
class AWorldDataLayers;
class AWorldSettings;
class UWorldPartition;
class UDataLayerManager;
class Error;
class FConstPawnIterator;
class FRegisterComponentContext;
class FTimerManager;
class FWorldInGamePerformanceTrackers;
class IInterface_PostProcessVolume;
class IStateStreamManager;
class UAISystemBase;
class UChaosEventRelay;
class UCanvas;
class UDemoNetDriver;
class UGameViewportClient;
class ULevel;
class ULevelStreaming;
class ULocalPlayer;
class UMaterialParameterCollection;
class UMaterialParameterCollectionInstance;
class UModel;
class UNavigationSystemBase;
class UNetConnection;
class UNetDriver;
class UPrimitiveComponent;
class UTexture2D;
class FPhysScene_Chaos;
class FSceneView;
struct FSceneViewProjectionData;
struct FUniqueNetIdRepl;
struct FEncryptionKeyResponse;
struct FParticlePerfStats;
enum class ELevelPendingVisibilityState : uint8;

template<typename,typename> class TOctree2;

/**
 * Misc. Iterator types
 *
 */
template <typename ActorType> class TActorIterator;
typedef TArray<TWeakObjectPtr<AController> >::TConstIterator FConstControllerIterator;
typedef TArray<TWeakObjectPtr<APlayerController> >::TConstIterator FConstPlayerControllerIterator;
typedef TArray<TWeakObjectPtr<ACameraActor> >::TConstIterator FConstCameraActorIterator;
typedef TArray<ULevel*>::TConstIterator FConstLevelIterator;
typedef TArray<TWeakObjectPtr<APhysicsVolume> >::TConstIterator FConstPhysicsVolumeIterator;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSpawn, Warning, All);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorSpawned, AActor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorDestroyed, AActor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostRegisterAllActorComponents, AActor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreUnregisterAllActorComponents, AActor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorRemovedFromWorld, AActor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFeatureLevelChanged, ERHIFeatureLevel::Type);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieSceneSequenceTick, float);

/** Proxy class that allows verification on GWorld accesses. */
class UWorldProxy
{
public:

	UWorldProxy() :
		World(NULL)
	{}

	inline UWorld* operator->()
	{
		// GWorld is changed often on the game thread when in PIE, accessing on any other thread is going to be a race condition
		// In general, the rendering thread should not dereference UObjects, unless there is a mechanism in place to make it safe	
		checkSlow(IsInGameThread());							
		return World;
	}

	inline const UWorld* operator->() const
	{
		checkSlow(IsInGameThread());
		return World;
	}

	inline UWorld& operator*()
	{
		checkSlow(IsInGameThread());
		return *World;
	}

	inline const UWorld& operator*() const
	{
		checkSlow(IsInGameThread());
		return *World;
	}

	inline UWorldProxy& operator=(UWorld* InWorld)
	{
		World = InWorld;
		return *this;
	}

	inline UWorldProxy& operator=(const UWorldProxy& InProxy)
	{
		World = InProxy.World;
		return *this;
	}

	inline bool operator==(const UWorldProxy& Other) const
	{
		return World == Other.World;
	}

	inline operator UWorld*() const
	{
		checkSlow(IsInGameThread());
		return World;
	}

	inline UWorld* GetReference() 
	{
		checkSlow(IsInGameThread());
		return World;
	}

private:

	UWorld* World;
};

// List of delegates for the world being registered to an audio device.
class FAudioDeviceWorldDelegates
{
public:
	// Called whenever a world is registered to an audio device. UWorlds are not guaranteed to be registered to the same
	// audio device throughout their lifecycle, and there is no guarantee on the lifespan of both the UWorld and the Audio
	// Device registered in this callback.
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWorldRegisteredToAudioDevice, const UWorld* /*InWorld */, Audio::FDeviceId /* AudioDeviceId*/);
	static UE_API FOnWorldRegisteredToAudioDevice OnWorldRegisteredToAudioDevice;

	// Called whenever a world is unregistered from an audio device. UWorlds are not guaranteed to be registered to the same
	// audio device throughout their lifecycle, and there is no guarantee on the lifespan of both the UWorld and the Audio
	// Device registered in this callback.
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWorldUnregisteredWithAudioDevice, const UWorld* /*InWorld */, Audio::FDeviceId /* AudioDeviceId*/);
	static UE_API FOnWorldUnregisteredWithAudioDevice OnWorldUnregisteredWithAudioDevice;
};

/** class that encapsulates seamless world traveling */
class FSeamlessTravelHandler
{
private:
	/** URL we're traveling to */
	FURL PendingTravelURL;
	/** set to the loaded package once loading is complete. Transition to it is performed in the next tick where it's safe to perform the required operations */
	UObject* LoadedPackage;
	/** the world we are travelling from */
	UWorld* CurrentWorld;
	/** set to the loaded world object inside that package. This is added to the root set (so that if a GC gets in between it won't break loading) */
	UWorld* LoadedWorld;
	/** set when a transition is in progress */
	bool bTransitionInProgress;
	/** whether or not we've transitioned to the entry level and are now moving on to the specified map */
	bool bSwitchedToDefaultMap;
	/** while set, pause at midpoint (after loading transition level, before loading final destination) */
	bool bPauseAtMidpoint;
	/** set when we started a new travel in the middle of a previous one and still need to clean up that previous attempt */
	bool bNeedCancelCleanUp;
	/** The context we are running in. Can be used to get the FWorldContext from Engine*/
	FName WorldContextHandle;
	/** Real time which we started traveling at  */
	double SeamlessTravelStartTime = 0.0;

	/** copy data between the old world and the new world */
	void CopyWorldData();

	/** callback sent to async loading code to inform us when the level package is complete */
	void SeamlessTravelLoadCallback(const FName& PackageName, UPackage* LevelPackage, EAsyncLoadingResult::Type Result);

	void SetHandlerLoadedData(UObject* InLevelPackage, UWorld* InLoadedWorld);

	/** Wrapper for loading transition or destination map, returns false if not found */
	bool StartLoadingMap(FString MapPackageToLoadFrom);

	/** called to kick off async loading of the destination map and any other packages it requires */
	void StartLoadingDestination();

public:
	FSeamlessTravelHandler()
		: PendingTravelURL(NoInit)
		, LoadedPackage(NULL)
		, CurrentWorld(NULL)
		, LoadedWorld(NULL)
		, bTransitionInProgress(false)
		, bSwitchedToDefaultMap(false)
		, bPauseAtMidpoint(false)
		, bNeedCancelCleanUp(false)
	{}

	/** starts traveling to the given URL. The required packages will be loaded async and Tick() will perform the transition once we are ready
	 * @param InURL the URL to travel to
	 * @return whether or not we succeeded in starting the travel
	 */
	bool StartTravel(UWorld* InCurrentWorld, const FURL& InURL);

	/** @return whether a transition is already in progress */
	inline bool IsInTransition() const
	{
		return bTransitionInProgress;
	}
	/** @return if current transition has switched to the default map; returns false if no transition is in progress */
	inline bool HasSwitchedToDefaultMap() const
	{
		return IsInTransition() && bSwitchedToDefaultMap;
	}

	/** @return the destination map that is being travelled to via seamless travel */
	inline FString GetDestinationMapName() const
	{
		return (IsInTransition() ? PendingTravelURL.Map : TEXT(""));
	}

	/** @return the destination world that has been loaded asynchronously by the seamless travel handler. */
	inline const UWorld* GetLoadedWorld() const
	{
		return LoadedWorld;
	}

	/** cancels transition in progress */
	void CancelTravel();

	/** turns on/off pausing after loading the transition map
	 * only valid during travel, before we've started loading the final destination
	 * @param bNowPaused - whether the transition should now be paused
	 */
	void SetPauseAtMidpoint(bool bNowPaused);

	/** 
	 * Ticks the transition; handles performing the world switch once the required packages have been loaded 
	 *
	 * @returns	The new primary world if the world has changed, null if it has not
	 */
	ENGINE_API UWorld* Tick();
};


/** Saved editor viewport state information */
USTRUCT()
struct FLevelViewportInfo
{
	GENERATED_BODY()

	/** Where the camera is positioned within the viewport. */
	UPROPERTY()
	FVector CamPosition;

	/** The camera's position within the viewport. */
	UPROPERTY()
	FRotator CamRotation;

	/** The zoom value  for orthographic mode. */
	UPROPERTY()
	float CamOrthoZoom;

	/** Whether camera settings have been systematically changed since the last level viewport update. */
	UPROPERTY()
	bool CamUpdated;

	FLevelViewportInfo()
		: CamPosition(FVector::ZeroVector)
		, CamRotation(FRotator::ZeroRotator)
		, CamOrthoZoom(DEFAULT_ORTHOZOOM)
		, CamUpdated(false)
	{
	}

	FLevelViewportInfo(const FVector& InCamPosition, const FRotator& InCamRotation, float InCamOrthoZoom)
		: CamPosition(InCamPosition)
		, CamRotation(InCamRotation)
		, CamOrthoZoom(InCamOrthoZoom)
		, CamUpdated(false)
	{
	}
	
	// Needed for backwards compatibility for VER_UE4_ADD_EDITOR_VIEWS, can be removed along with it
	friend FArchive& operator<<( FArchive& Ar, FLevelViewportInfo& I )
	{
		Ar << I.CamPosition;
		Ar << I.CamRotation;
		Ar << I.CamOrthoZoom;

		if ( Ar.IsLoading() )
		{
			I.CamUpdated = true;

			if ( I.CamOrthoZoom < MIN_ORTHOZOOM || I.CamOrthoZoom > MAX_ORTHOZOOM )
			{
				I.CamOrthoZoom = DEFAULT_ORTHOZOOM;
			}
		}

		return Ar;
	}
};

/** 
* Tick function that starts the physics tick
**/
USTRUCT()
struct FStartPhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** World this tick function belongs to **/
	class UWorld*	Target;

	/** 
		* Abstract function actually execute the tick. 
		* @param DeltaTime - frame time to advance, in seconds
		* @param TickType - kind of tick for this frame
		* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
		* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	**/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FStartPhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FStartPhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/** 
* Tick function that ends the physics tick
**/
USTRUCT()
struct FEndPhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** World this tick function belongs to **/
	class UWorld*	Target;

	/** 
		* Abstract function actually execute the tick. 
		* @param DeltaTime - frame time to advance, in seconds
		* @param TickType - kind of tick for this frame
		* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
		* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	**/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FEndPhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FEndPhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/* Struct of optional parameters passed to SpawnActor function(s). */
struct FActorSpawnParameters
{
	UE_API FActorSpawnParameters();

	/* A name to assign as the Name of the Actor being spawned. If no value is specified, the name of the spawned Actor will be automatically generated using the form [Class]_[Number]. */
	FName Name;

	/* An Actor to use as a template when spawning the new Actor. The spawned Actor will be initialized using the property values of the template Actor. If left NULL the class default object (CDO) will be used to initialize the spawned Actor. */
	AActor* Template;

	/* The Actor that spawned this Actor. (Can be left as NULL). */
	AActor* Owner;

	/* The APawn that is responsible for damage done by the spawned Actor. (Can be left as NULL). */
	APawn*	Instigator;

	/* The ULevel to spawn the Actor in, i.e. the Outer of the Actor. If left as NULL the Outer of the Owner is used. If the Owner is NULL the persistent level is used. */
	class	ULevel* OverrideLevel;

#if WITH_EDITOR
	/* The UPackage to set the Actor in. If left as NULL the Package will not be set and the actor will be saved in the same package as the persistent level. */
	class	UPackage* OverridePackage;

	/** Optional initial editor label to apply to the actor */
	FStringView InitialActorLabel;

	/** The Guid to set to this actor. Should only be set when reinstancing blueprint actors. */
	FGuid	OverrideActorGuid;
#endif

	/* The parent component to set the Actor in. */
	class   UChildActorComponent* OverrideParentComponent;

	/** Method for resolving collisions at the spawn point. Undefined means no override, use the actor's setting. */
	ESpawnActorCollisionHandlingMethod SpawnCollisionHandlingOverride;

	/** Determines whether to multiply or override root component with provided spawn transform */
	ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::MultiplyWithRoot;

private:

	friend class UPackageMapClient;

	friend class UNetActorFactory;

	/* Is the actor remotely owned. This should only be set true by the package map when it is creating an actor on a client that was replicated from the server. */
	uint8	bRemoteOwned:1;
	
public:

	bool IsRemoteOwned() const { return bRemoteOwned; }

	/* Determines whether spawning will not fail if certain conditions are not met. If true, spawning will not fail because the class being spawned is `bStatic=true` or because the class of the template Actor is not the same as the class of the Actor being spawned. */
	uint8	bNoFail:1;

	/* Determines whether the construction script will be run. If true, the construction script will not be run on the spawned Actor. Only applicable if the Actor is being spawned from a Blueprint. */
	uint8	bDeferConstruction:1;
	
	/* Determines whether or not the actor may be spawned when running a construction script. If true spawning will fail if a construction script is being run. */
	uint8	bAllowDuringConstructionScript:1;

#if !WITH_EDITOR
	/* Force the spawned actor to use a globally unique name (provided name should be none). */
	uint8	bForceGloballyUniqueName:1;
#else
	/* Determines whether the begin play cycle will run on the spawned actor when in the editor. */
	uint8	bTemporaryEditorActor:1;

	/* Determines whether or not the actor should be hidden from the Scene Outliner */
	uint8	bHideFromSceneOutliner:1;

	/** Determines whether to create a new package for the actor or not, if the level supports it. */
	uint16	bCreateActorPackage:1;
#endif

	/* Modes that SpawnActor can use the supplied name when it is not None. */
	enum class ESpawnActorNameMode : uint8
	{
		/* Fatal if unavailable, application will assert */
		Required_Fatal,

		/* Report an error return null if unavailable */
		Required_ErrorAndReturnNull,

		/* Return null if unavailable */
		Required_ReturnNull,

		/* If the supplied Name is already in use the generate an unused one using the supplied version as a base */
		Requested
	};

	/* In which way should SpawnActor should treat the supplied Name if not none. */
	ESpawnActorNameMode NameMode;

	/* Flags used to describe the spawned actor/object instance. */
	EObjectFlags ObjectFlags;

	/* Custom function allowing the caller to specific a function to execute post actor construction but before other systems see this actor spawn. */
	TFunction<void(AActor*)> CustomPreSpawnInitialization;
};

/* World actors spawning helper functions */
struct FActorSpawnUtils
{
	/**
	 * Function to generate a locally or globally unique actor name. To generate a globally unique name, we store an epoch number
	 * in the name number (while maintaining compatibility with fast path name generation, see GFastPathUniqueNameGeneration) and
	 * also append a unique user id to the name.
	 *
	 * @param	Level			the new actor level
	 * @param	Class			the new actor class
	 * @param	BaseName		optional base name
	 * @param	bGloballyUnique	whether to create a globally unique name
	 * @return	generated actor name
	**/
	static UE_API FName MakeUniqueActorName(ULevel* Level, const UClass* Class, FName BaseName, bool bGloballyUnique);

	/**
	 * Determine if an actor name is globally unique or not.
	 *
	 * @param	Name			the name to check
	 * @return true if the provided name is globally unique
	**/
	static UE_API bool IsGloballyUniqueName(FName Name);

	/**
	 * Return the base name (without any number of globally unique identifier).
	**/
	static UE_API FName GetBaseName(FName Name);
};

struct FActorsInitializedParams
{
	FActorsInitializedParams(UWorld* InWorld, bool InResetTime) : World(InWorld), ResetTime(InResetTime) {}
	UWorld* World;
	bool ResetTime;
};

/**
 *  This encapsulate World's async trace functionality. This contains two buffers of trace data buffer and alternates it for each tick. 
 *  
 *	You can use async trace using following APIs : AsyncLineTrace, AsyncSweep, AsyncOverlap
 *	When you use those APIs, it will be saved to AsyncTraceData
 *	FWorldAsyncTraceState contains two buffers to rotate each frame as you might need the result in the next frame
 *	However, if you do not get the result by next frame, the result will be discarded. 
 *	Use Delegate if you would like to get the result right away when available. 
 */
struct FWorldAsyncTraceState
{
	UE_API FWorldAsyncTraceState();

	/** Get the Buffer for input Frame */
	inline AsyncTraceData& GetBufferForFrame        (int32 Frame) { return DataBuffer[ Frame             % 2]; }
	/** Get the Buffer for Current Frame */
	inline AsyncTraceData& GetBufferForCurrentFrame ()            { return DataBuffer[ CurrentFrame      % 2]; }
	/** Get the Buffer for Previous Frame */
	inline AsyncTraceData& GetBufferForPreviousFrame()            { return DataBuffer[(CurrentFrame + 1) % 2]; }

	/** Async Trace Data Buffer Array. For now we only saves 2 frames. **/
	AsyncTraceData DataBuffer[2];

	/** Used as counter for Buffer swap for DataBuffer. Right now it's only 2, but it can change. */
	int32 CurrentFrame;
};

#if WITH_EDITOR
struct FWorldRenameFromRootContext
{
	/** New name for PreRenameFromRoot, old name for PostRenameFromRoot. */
	FName Name;

	/** New outer for PreRenameFromRoot, old outer for PostRenameFromRoot. */
	UObject* Outer = nullptr;

	/** Rename flags. */
	ERenameFlags Flags = REN_None;
};

/* FAsyncPreRegisterDDCRequest - info about an async DDC request that we're going to wait on before registering components */
class FAsyncPreRegisterDDCRequest
{
	/* DDC Key used for the request */
	FString DDCKey;

	/* Handle for Async DDC request. 0 if no longer invalid. */
	uint32 Handle;
public:
	/** constructor */
	FAsyncPreRegisterDDCRequest(const FString& InKey, uint32 InHandle)
	: DDCKey(InKey)
	, Handle(InHandle)
	{}

	/** destructor */
	UE_API ~FAsyncPreRegisterDDCRequest();

	/** returns true if the request is complete */
	UE_API bool PollAsynchronousCompletion();

	/** waits until the request is complete */
	UE_API void WaitAsynchronousCompletion();

	/** returns true if the DDC returned the results requested. Must only be called once. */
	UE_API bool GetAsynchronousResults(TArray<uint8>& OutData);

	/** get the DDC key associated with this request */
	const FString& GetKey() const { return DDCKey; }
};
#endif

/**
 * Contains a group of levels of a particular ELevelCollectionType within a UWorld
 * and the context required to properly tick/update those levels. This object is move-only.
 */
USTRUCT()
struct FLevelCollection
{
	GENERATED_BODY()

	UE_API FLevelCollection();

	FLevelCollection(const FLevelCollection&) = delete;
	FLevelCollection& operator=(const FLevelCollection&) = delete;

	UE_API FLevelCollection(FLevelCollection&& Other);
	UE_API FLevelCollection& operator=(FLevelCollection&& Other);

	/** The destructor will clear the cached collection pointers in this collection's levels. */
	UE_API ~FLevelCollection();

	/** Gets the type of this collection. */
	ELevelCollectionType GetType() const { return CollectionType; }

	/** Sets the type of this collection. */
	void SetType(const ELevelCollectionType InType) { CollectionType = InType; }

	/** Gets the game state for this collection. */
	AGameStateBase* GetGameState() const { return GameState; }

	/** Sets the game state for this collection. */
	void SetGameState(AGameStateBase* const InGameState) { GameState = InGameState; }

	/** Gets the net driver for this collection. */
	UNetDriver* GetNetDriver() const { return NetDriver; }
	
	/** Sets the net driver for this collection. */
	void SetNetDriver(UNetDriver* const InNetDriver) { NetDriver = InNetDriver; }

	/** Gets the demo net driver for this collection. */
	UDemoNetDriver* GetDemoNetDriver() const { return DemoNetDriver; }

	/** Sets the demo net driver for this collection. */
	void SetDemoNetDriver(UDemoNetDriver* const InDemoNetDriver) { DemoNetDriver = InDemoNetDriver; }

	/** Returns the set of levels in this collection. */
	const TSet<TObjectPtr<ULevel>>& GetLevels() const { return Levels; }

	/** Adds a level to this collection and caches the collection pointer on the level for fast access. */
	UE_API void AddLevel(ULevel* const Level);

	/** Removes a level from this collection and clears the cached collection pointer on the level. */
	UE_API void RemoveLevel(ULevel* const Level);

	/** Sets this collection's PersistentLevel and adds it to the Levels set. */
	UE_API void SetPersistentLevel(ULevel* const Level);

	/** Returns this collection's PersistentLevel. */
	ULevel* GetPersistentLevel() const { return PersistentLevel; }

	/** Gets whether this collection is currently visible. */
	bool IsVisible() const { return bIsVisible; }

	/** Sets whether this collection is currently visible. */
	void SetIsVisible(const bool bInIsVisible) { bIsVisible = bInIsVisible; }

private:
	/** The type of this collection. */
	ELevelCollectionType CollectionType;

	/**
	* Whether or not this collection is currently visible. While invisible, actors in this collection's
	* levels will not be rendered and sounds originating from levels in this collection will not be played.
	*/
	bool bIsVisible;

	/**
	 * The GameState associated with this collection. This may be different than the UWorld's GameState
	 * since the source collection and the duplicated collection will have their own instances.
	 */
	UPROPERTY()
	TObjectPtr<class AGameStateBase> GameState;

	/**
	 * The network driver associated with this collection.
	 * The source collection and the duplicated collection will have their own instances.
	 */
	UPROPERTY()
	TObjectPtr<class UNetDriver> NetDriver;

	/**
	 * The demo network driver associated with this collection.
	 * The source collection and the duplicated collection will have their own instances.
	 */
	UPROPERTY()
	TObjectPtr<class UDemoNetDriver> DemoNetDriver;

	/**
	 * The persistent level associated with this collection.
	 * The source collection and the duplicated collection will have their own instances.
	 */
	UPROPERTY()
	TObjectPtr<class ULevel> PersistentLevel;

	/** All the levels in this collection. */
	UPROPERTY()
	TSet<TObjectPtr<ULevel>> Levels;
};

template<>
struct TStructOpsTypeTraits<FLevelCollection> : public TStructOpsTypeTraitsBase2<FLevelCollection>
{
	enum
	{
		WithCopy = false
	};
};

/**
 * A helper RAII class to set the relevant context on a UWorld for a particular FLevelCollection within a scope.
 * The constructor will set the PersistentLevel, GameState, NetDriver, and DemoNetDriver on the world and
 * the destructor will restore the original values.
 */
class FScopedLevelCollectionContextSwitch
{
public:
	/**
	 * Constructor that will save the current relevant values of InWorld
	 * and set the collection's context values for InWorld.
	 * The constructor that takes an index is preferred, but this one
	 * still exists for backwards compatibility.
	 *
	 * @param InLevelCollection The collection's context to use
	 * @param InWorld The world on which to set the context.
	 */
	UE_API FScopedLevelCollectionContextSwitch(const FLevelCollection* const InLevelCollection, UWorld* const InWorld);

	/**
	 * Constructor that will save the current relevant values of InWorld
	 * and set the collection's context values for InWorld.
	 *
	 * @param InLevelCollectionIndex The index of the collection to use
	 * @param InWorld The world on which to set the context.
	 */
	UE_API FScopedLevelCollectionContextSwitch(int32 InLevelCollectionIndex, UWorld* const InWorld);

	/** The destructor restores the context on the world that was saved in the constructor. */
	UE_API ~FScopedLevelCollectionContextSwitch();

private:
	class UWorld* World;
	int32 SavedTickingCollectionIndex;
};

USTRUCT()
struct FStreamingLevelsToConsider
{
	GENERATED_BODY()

	FStreamingLevelsToConsider()
		: StreamingLevelsBeingConsidered(0)
	{}

private:

	/** Priority sorted array of streaming levels actively being considered. */
	UPROPERTY()
	TArray<TObjectPtr<ULevelStreaming>> StreamingLevels;

	enum class EProcessReason : uint8
	{
		Add,
		Reevaluate
	};

	/** Streaming levels that had their priority changed or were added to the container while consideration was underway. */
	TSortedMap<TObjectPtr<ULevelStreaming>, EProcessReason> LevelsToProcess;

	/** Whether the streaming levels are under active consideration */
	int32 StreamingLevelsBeingConsidered;

	/** 
	 * Add an element to the container. 
	 * If bGuaranteedRemoved is true, Contains check to avoid duplicates will not be used. This should only be used immediately after calling Remove.
	*/
	UE_API void Add_Internal(ULevelStreaming* StreamingLevel, bool bGuaranteedNotInContainer);

public:

	const TArray<TObjectPtr<ULevelStreaming>>& GetStreamingLevels() const { return StreamingLevels; }

	UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	UE_API void BeginConsideration();
	UE_API void EndConsideration();
	bool AreStreamingLevelsBeingConsidered() { return StreamingLevelsBeingConsidered > 0; }

	/** Add an element to the container if not already in the container. */
	void Add(ULevelStreaming* StreamingLevel) { Add_Internal(StreamingLevel, false); }

	/* Remove an element from the container. */
	UE_API bool Remove(ULevelStreaming* StreamingLevel);

	/* Remove the element at a given index from the container. */
	UE_API void RemoveAt(int32 Index);

	/* Returns if an element is in the container. */
	UE_API bool Contains(ULevelStreaming* StreamingLevel) const;

	/* Resets the container to an empty state without freeing array memory. */
	UE_API void Reset();

	/* Instructs the container that state changed such that the position in the priority sorted array the level may no longer be correct. */
	UE_API void Reevaluate(ULevelStreaming* StreamingLevel);
};

struct FWorldCachedViewInfo
{
	FMatrix ViewMatrix;
	FMatrix ProjectionMatrix;
	FMatrix ViewProjectionMatrix;
	FMatrix ViewToWorld;
};

/**
 * Helper class allows UWorldPartition to broadcast UWorld events 
 */
struct FWorldPartitionEvents
{
	friend UWorldPartition;

private:
	static void BroadcastWorldPartitionInitialized(UWorld* InWorld, UWorldPartition* InWorldPartition);
	static void BroadcastWorldPartitionUninitialized(UWorld* InWorld, UWorldPartition* InWorldPartition);
};

/**
* Represents a single "move" of a component generated by a scoped movement update.
* Keeps track of movement data for scene components who have requested to processed
* as part of a Deferred Movement Group. This is populated by using ScopedMovementUpdates of type
* EScopedUpdate::DeferredGroupUpdates.
* 
* This is a snapshot of the state of a FScopedMovementUpdate and keep track of to use for later.
* Populated in UWorld::PushComponentGroupMove after a DeferredGroupMove finishes on a scene component.
*
* The world will process these deferred moves all at once to improve game thread performance,
* including propagating changes to the physics simulation state.
* 
* @see FScopedMovementUpdate
*/
struct FDeferredComponentMoveData
{
	FDeferredComponentMoveData() = default;
	
	/**
	 * The component which this move data should be applied to. 
	 */
	TWeakObjectPtr<USceneComponent> ComponentToMove = nullptr;
	
	FTransform InitialTransform;

	FScopedMovementUpdate::TScopedOverlapInfoArray PendingOverlaps;
	
	FScopedMovementUpdate::TScopedBlockingHitArray BlockingHits;
	
	bool bHasTransformMovedForGroupUpdate = false;

	bool bHasMoved = false;
	
	FScopedMovementUpdate::EOverlapState OverlapState = FScopedMovementUpdate::EOverlapState::eUnknown;
		
	ETeleportType TeleportType = ETeleportType::None;

	// If not INDEX_NONE, overlaps at this index and beyond in PendingOverlaps are at the final destination
	int32 FinalOverlapCandidatesIndex = INDEX_NONE;
};

class FWorldSubsystemCollection : public FObjectSubsystemCollection<UWorldSubsystem>
{
public:
	using Super = FObjectSubsystemCollection<UWorldSubsystem>;

	FWorldSubsystemCollection();
	virtual ~FWorldSubsystemCollection();
private:
	virtual void OnSubsystemAdded(TNonNullPtr<USubsystem> Subsystem) override;

};


/** 
 * The World is the top level object representing a map or a sandbox in which Actors and Components will exist and be rendered.  
 *
 * A World can be a single Persistent Level with an optional list of streaming levels that are loaded and unloaded via volumes and blueprint functions
 * or it can be a collection of levels organized with a World Composition.
 *
 * In a standalone game, generally only a single World exists except during seamless area transitions when both a destination and current world exists.
 * In the editor many Worlds exist: The level being edited, each PIE instance, each editor tool which has an interactive rendered viewport, and many more.
 *
 */

UCLASS(MinimalAPI, customConstructor, config=Engine)
class UWorld final : public UObject, public FNetworkNotify
{
	GENERATED_UCLASS_BODY()

	UE_API ~UWorld();

#if WITH_EDITORONLY_DATA
	/** List of all the layers referenced by the world's actors */
	UPROPERTY(Transient)
	TArray< TObjectPtr<class ULayer> > Layers; 

	// Group actors currently "active"
	UPROPERTY(Transient)
	TArray<TObjectPtr<AActor>> ActiveGroupActors;

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category=Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
#endif // WITH_EDITORONLY_DATA

	/** Persistent level containing the world info, default brush and actors spawned during gameplay among other things			*/
	UPROPERTY(Transient)
	TObjectPtr<class ULevel>								PersistentLevel;

	/** The NAME_GameNetDriver game connection(s) for client/server communication */
	UPROPERTY(Transient)
	TObjectPtr<class UNetDriver>							NetDriver;

	/** Line Batchers. All lines to be drawn in the world. */
	UE_DEPRECATED(5.6, "Please use GetLineBatcher(UWorld::ELineBatcherType::World")
	UPROPERTY(Transient)
	TObjectPtr<class ULineBatchComponent>					LineBatcher_DEPRECATED;

	/** Persistent Line Batchers. They don't get flushed every frame.  */
	UE_DEPRECATED(5.6, "Please use GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent")
	UPROPERTY(Transient)
	TObjectPtr<class ULineBatchComponent>					PersistentLineBatcher_DEPRECATED;

	/** Foreground Line Batchers. This can't be Persistent.  */
	UE_DEPRECATED(5.6, "Please use GetLineBatcher(UWorld::ELineBatcherType::Foreground")
	UPROPERTY(Transient)
	TObjectPtr<class ULineBatchComponent>					ForegroundLineBatcher_DEPRECATED;

	/** Instance of this world's game-specific networking management */
	UPROPERTY(Transient)
	TObjectPtr<class AGameNetworkManager>					NetworkManager;

	/** Instance of this world's game-specific physics collision handler */
	UPROPERTY(Transient)
	TObjectPtr<class UPhysicsCollisionHandler>				PhysicsCollisionHandler;

	/** Instance of this world's game-specific physics query handler */
	UPROPERTY(Transient)
	TObjectPtr<class UPhysicsQueryHandler>				    PhysicsQueryHandler;

	/** Array of any additional objects that need to be referenced by this world, to make sure they aren't GC'd */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>>							ExtraReferencedObjects;

	/**
	 * External modules can have additional data associated with this UWorld.
	 * This is a list of per module world data objects. These aren't
	 * loaded/saved by default.
	 */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>>							PerModuleDataObjects;

private:
	/** Level collection. ULevels are referenced by FName (Package name) to avoid serialized references. Also contains offsets in world units */
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULevelStreaming>> StreamingLevels;

	/** This is the list of streaming levels that are actively being considered for what their state should be. It will be a subset of StreamingLevels */
	UPROPERTY(Transient, DuplicateTransient)
	FStreamingLevelsToConsider StreamingLevelsToConsider;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<AServerStreamingLevelsVisibility> ServerStreamingLevelsVisibility;

public:
	enum class ELineBatcherType : uint8
	{
		World = 0,
		WorldPersistent,
		Foreground,
		ForegroundPersistent,
		// --- ---
		NUM,
		Invalid = NUM
	};

	/** Returns a pointer to the specified line batcher type */
	UE_API ULineBatchComponent* GetLineBatcher(ELineBatcherType Type) const;
	
	/** Flush the specified Line Batcher types */
	UE_API void FlushLineBatchers(const TArrayView<const UWorld::ELineBatcherType>& TypesToFlush);

	/** Returns whether the world supports for a client to use "making visible" transaction requests to the server. */
	UE_API bool SupportsMakingVisibleTransactionRequests() const;

	/** Returns whether the world supports for a client to use "making invisible" transaction requests to the server. */
	UE_API bool SupportsMakingInvisibleTransactionRequests() const;

	/** Used to allow changing of visiblity transaction information in situations where overrides are applied after the world is created */
	UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
	UE_API void InvalidateVisiblityTransactionRequestCachedVariables();

	/** Returns the object used to query server streaming level visibility. */
	UE_API const AServerStreamingLevelsVisibility* GetServerStreamingLevelsVisibility() const;

	/** Return a const version of the streaming levels array */
	const TArray<ULevelStreaming*>& GetStreamingLevels() const { return StreamingLevels; }

	uint16 GetNumStreamingLevelsBeingLoaded() const { return NumStreamingLevelsBeingLoaded; }

	/** Returns true if StreamingLevel is part of the levels being considered for update */
	bool IsStreamingLevelBeingConsidered(ULevelStreaming* StreamingLevel) const { return StreamingLevelsToConsider.Contains(StreamingLevel); }
	
	/** Returns true if there is at least one level being considered for update */
	bool HasStreamingLevelsToConsider() const { return StreamingLevelsToConsider.GetStreamingLevels().Num() > 0; }

	/** Returns the level, if any, in the process of being made visible */
	UE_DEPRECATED(5.6, "Use IsLevelMakingVisible instead.")
	ULevel* GetCurrentLevelPendingVisibility() const { return nullptr; }

	/** Returns true if the level is in the process of being made visible */
	UE_API bool IsLevelMakingVisible(const ULevel* InLevel) const;

	/** Returns true if any level is in the process of being made visible */
	UE_API bool HasAnyLevelMakingVisible() const;

	/** Returns the level, if any, in the process of being made invisible */
	UE_DEPRECATED(5.6, "Use IsLevelMakingInvisible instead.")
	ULevel* GetCurrentLevelPendingInvisibility() const { return nullptr; }

	/** Returns true if the level is in the process of being made invisible */
	UE_API bool IsLevelMakingInvisible(const ULevel* InLevel) const;

	/** Returns true if any level is in the process of being made invisible */
	UE_API bool HasAnyLevelMakingInvisible() const;

	/** Add a streaming level to the list of streamed levels to consider. */
	UE_API void AddStreamingLevel(ULevelStreaming* StreamingLevelToAdd);

	/** Add multiple streaming levels to the list of streamed levels to consider.  */
	UE_API void AddStreamingLevels(TArrayView<ULevelStreaming* const> StreamingLevelsToAdd);	

	/** Add a streaming level to the list of streamed levels to consider. If this streaming level is in the array already then it won't be added again. */
	UE_API void AddUniqueStreamingLevel(ULevelStreaming* StreamingLevelToAdd);

	/** Add multiple streaming levels to the list of streamed levels to consider.  If any of these streaming levels are in the array already then they won't be added again.  */
	UE_API void AddUniqueStreamingLevels(TArrayView<ULevelStreaming* const> StreamingLevelsToAdd);

	/** Replace the streaming levels array */
	UE_API void SetStreamingLevels(TArray<ULevelStreaming*>&& StreamingLevels);

	/** Replace the streaming levels array */
	UE_API void SetStreamingLevels(TArrayView<ULevelStreaming* const> StreamingLevels);

	/** Remove a streaming level to the list of streamed levels to consider.
	 *  Returns true if the specified level was in the streaming levels list.
	 */
	UE_API bool RemoveStreamingLevel(ULevelStreaming* StreamingLevelToRemove);

	/** Remove a streaming level to the list of streamed levels to consider.
	*  Returns true if the specified index was a valid index for removal.
	*/
	UE_API bool RemoveStreamingLevelAt(int32 IndexToRemove);

	/** Remove multiple streaming levels to the list of streamed levels to consider. 
	 * Returns a count of how many of the specified levels were in the streaming levels list
	 */
	UE_API int32 RemoveStreamingLevels(TArrayView<ULevelStreaming* const> StreamingLevelsToRemove);

	/** Reset the streaming levels array */
	UE_API void ClearStreamingLevels();

	/** Inform the world that a streaming level has had a potentially state changing modification made to it so that it needs to be in the StreamingLevelsToConsider list. */
	UE_API void UpdateStreamingLevelShouldBeConsidered(ULevelStreaming* StreamingLevelToConsider);

	/** Inform the world that the streaming level has had its priority change and may need to be resorted if under consideration. */
	UE_API void UpdateStreamingLevelPriority(ULevelStreaming* StreamingLevel);

	/** Examine all streaming levels and determine which ones should be considered. */
	UE_API void PopulateStreamingLevelsToConsider();

	/** Whether the world is currently in a BlockTillLevelStreamingCompleted() call */
	bool GetIsInBlockTillLevelStreamingCompleted() const { return IsInBlockTillLevelStreamingCompleted > 0; }

	/** Returns BlockTillLevelStreamingCompletedEpoch. */
	int32 GetBlockTillLevelStreamingCompletedEpoch() const { return BlockTillLevelStreamingCompletedEpoch; }

	/** Prefix we used to rename streaming levels, non empty in PIE and standalone preview */
	UPROPERTY()
	FString										StreamingLevelsPrefix;

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	/** 
	 * Enable/disable actor ticking and any user callbacks.
	 *
	 * Applies to the following callbacks:
	 *		BeginPlay()
	 *		EndPlay()
	 *		BeginComponentOverlap()
	 *		EndComponentOverlap()
	 * 		Actor::Destroyed() and related ones found in the flow of UWorld::DestroyActor().
	 */
	UE_API void EnableActorTickAndUserCallbacks(bool bEnable);
	bool IsActorTickAndUserCallbacksEnabled() const { return bEnableActorTickAndUserCallbacks; };
#endif

	/**
	 * Experimental.
	 * 
	 * Adds the given scene component and movement data to be applied later in the frame (ProcessPendingGroupMoves)
	 * 
	 * @param SceneComp The Scene component to move later 
	 * @param Scope The scoped movement update which represents the movement state to be applied later.
	 */
	UE_API void PushComponentGroupMove(const FScopedMovementUpdate* Scope);

	/**
	 * Experimental.
	 *
	 * Processes any currently pending grouped move updates.
	 * 
	 * @param bProcessAllMoves if true, always process all pending moves. False for incremental partial updates
	 */
	UE_API void ProcessPendingGroupMoves(bool bProcessAllMoves = true);

private:
	/**
	 * Experimental.
	 *
	 * Tracks data for movements which should be applied to components later on in the frame
	 */
	TArray<FDeferredComponentMoveData> DeferredComponentMoves;
	
	/** Line Batchers. Persistent types don't get flushed every frame. */
	UPROPERTY(Transient)
	TObjectPtr<class ULineBatchComponent> LineBatchers[(int32)ELineBatcherType::NUM];

	/** Sets level's PendingVisibilityState */
	UE_API void SetLevelPendingVisibilityState(ULevel* InLevel, ELevelPendingVisibilityState InState);

	/** Returns whether AddToWorld should be skipped on a given level */
	UE_API bool CanAddLoadedLevelToWorld(ULevel* Level) const;

	/** Spawn an instance of ANoPawnPlayerController and set it as the value of Connection::PlayerController. */
	void SpawnAndSetNoPawnPlayerController(UNetConnection* Connection, uint8 NetPlayerIndex, int32 LocalPlayerIdentifier);

	/** Stores whether the game world supports for a client to use "making visible" transaction requests to the server. */
	mutable TOptional<bool> bSupportsMakingVisibleTransactionRequests;

	/** Stores whether the game world supports for a client to use "making invisible" transaction requests to the server. */
	mutable TOptional<bool> bSupportsMakingInvisibleTransactionRequests;

	/** Used for MT access to MakingVisibleLevels and MakingInvisibleLevels */
	mutable FTransactionallySafeRWLock PendingVisibilityLock;

	/** Level(s) in the queue to be made visible. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<class ULevel>>						MakingVisibleLevels;

	/** Level(s) in the queue to be made invisible. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<class ULevel>>						MakingInvisibleLevels;

	/** NetDriver for capturing network traffic to record demos */
	UPROPERTY()
	TObjectPtr<class UDemoNetDriver>						DemoNetDriver;

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	bool bEnableActorTickAndUserCallbacks = true;
#endif

public:
	/** Gets the demo net driver for this world. */
	UDemoNetDriver* GetDemoNetDriver() const { return DemoNetDriver; }

	/** Sets the demo net driver for this world. */
	void SetDemoNetDriver(UDemoNetDriver* const InDemoNetDriver) { DemoNetDriver = InDemoNetDriver; }

	/** Particle event manager **/
	UPROPERTY()
	TObjectPtr<class AParticleEventManager>				MyParticleEventManager;

private:
	/** DefaultPhysicsVolume used for whole game **/
	UPROPERTY(Transient)
	TObjectPtr<APhysicsVolume>								DefaultPhysicsVolume;

	// Flag for allowing physics state creation deferall during load 
	bool bAllowDeferredPhysicsStateCreation;

public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline ERHIFeatureLevel::Type GetFeatureLevel() const{ return FeatureLevel; }
	inline void SetFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel) { FeatureLevel = InFeatureLevel; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** View locations rendered in the previous frame, if any. */
	TArray<FVector>								ViewLocationsRenderedLastFrame;

	/** Cached view information from the last rendered frame. */
	TArray<FWorldCachedViewInfo>				CachedViewInfoRenderedLastFrame;
	/** WorldTimeSeconds when this world was last rendered. */
	double										LastRenderTime = 0.0;

	/** The current renderer feature level of this world */
	UE_DEPRECATED(5.3, "FeatureLevel will be removed in the future from UWorld. Please use GetFeatureLevel or SetFeatureLevel instead")
	TEnumAsByte<ERHIFeatureLevel::Type> FeatureLevel;

	/** The current ticking group																								*/
	TEnumAsByte<ETickingGroup> TickGroup;

	/** The type of world this is. Describes the context in which it is being used (Editor, Game, Preview etc.) */
	TEnumAsByte<EWorldType::Type> WorldType;

	/** set for one tick after completely loading and initializing a new world
	 * (regardless of whether it's LoadMap() or seamless travel)
	 */
	uint8 bWorldWasLoadedThisTick:1;

	/**
	 * Triggers a call to PostLoadMap() the next Tick, turns off loading movie if LoadMap() has been called.
	 */
	uint8 bTriggerPostLoadMap:1;

	/** Whether we are in the middle of ticking actors/components or not														*/
	uint8 bInTick:1;

	/** Whether we have already built the collision tree or not                                                                 */
	uint8 bIsBuilt:1;

	/** We are in the middle of actor ticking, so add tasks for newly spawned actors											*/
	uint8 bTickNewlySpawned:1;

	/** 
	 * Indicates that during world ticking we are doing the final component update of dirty components 
	 * (after PostAsyncWork and effect physics scene has run. 
	 */
	uint8 bPostTickComponentUpdate:1;

	/** Whether world object has been initialized via Init and has not yet had CleanupWorld called								*/
	uint8 bIsWorldInitialized:1;
	
	/** Is level streaming currently frozen?																					*/
	uint8 bIsLevelStreamingFrozen:1;

	/** True we want to execute a call to UpdateCulledTriggerVolumes during Tick */
	uint8 bDoDelayedUpdateCullDistanceVolumes:1;

	/** If true this world is in the process of running the construction script for an actor */
	uint8 bIsRunningConstructionScript:1;

	/** If true this world will tick physics to simulate. This isn't same as having Physics Scene. 
	*  You need Physics Scene if you'd like to trace. This flag changed ticking */
	uint8 bShouldSimulatePhysics:1;

#if !UE_BUILD_SHIPPING || WITH_EDITOR
	/** If TRUE, 'hidden' components will still create render proxy, so can draw info (see USceneComponent::ShouldRender) */
	uint8 bCreateRenderStateForHiddenComponentsWithCollsion:1;
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITOR
	/** this is special flag to enable collision by default for components that are not Volume
	* currently only used by editor level viewport world, and do not use this for in-game scene
	*/
	uint8 bEnableTraceCollision:1;

	/** Special flag to enable movement component in non game worlds (see UMovementComponent::OnRegister) */
	uint8 bForceUseMovementComponentInNonGameWorld:1;

	/** If True, overloaded method IsNameStableForNetworking will always return true. */
	uint8 bIsNameStableForNetworking:1;
#endif

	/** That map is default map or not **/
	uint8 bIsDefaultLevel:1;
	
	/** Whether it was requested that the engine bring up a loading screen and block on async loading. */   
	uint8 bRequestedBlockOnAsyncLoading:1;

	/** Whether actors have been initialized for play */
	uint8 bActorsInitialized:1;

private:
	/** Whether BeginPlay has been called on actors. Use the public accessors GetBegunPlay() & SetBegunPlay() */
	uint8 bBegunPlay:1;

public:
	/** Sets whether BeginPlay has been called for actors in the world. Use BeginPlay and EndPlay below to start the process of changing this. */
	UE_API void SetBegunPlay(bool bHasBegunPlay);

	/** Returns true if BeginPlay has been called on actors in the world (and EndPlay has not) */
	UE_API bool GetBegunPlay() const;

	DECLARE_EVENT_OneParam(UWorld, FOnBeginPlay, bool);
	/** Return the event that is broadcast when bBegunPlay is changed */
	FOnBeginPlay& GetOnBeginPlayEvent() { return OnBeginPlay; }

	/** Whether the match has been started */
	uint8 bMatchStarted:1;

	/**  When ticking the world, only update players. */
	uint8 bPlayersOnly:1;

	/** Indicates that at the end the frame bPlayersOnly will be set to true. */
	uint8 bPlayersOnlyPending:1;

	/** Is the world in its actor initialization phase. */
	uint8 bStartup:1;

	/** Is the world being torn down */
	uint8 bIsTearingDown:1;

	/**
	 * This is a bool that indicates that one or more blueprints in the level (blueprint instances, level script, etc)
	 * have compile errors that could not be automatically resolved.
	 */
	uint8 bKismetScriptError:1;

	// Kismet debugging flags - they can be only editor only, but they're uint32, so it doesn't make much difference
	uint8 bDebugPauseExecution:1;

	/** When set, camera is potentially moveable even when paused */
	uint8 bIsCameraMoveableWhenPaused:1;

	/** Indicates this scene always allows audio playback. */
	uint8 bAllowAudioPlayback:1;

#if WITH_EDITOR
	/** When set, will tell us to pause simulation after one tick.  If a breakpoint is encountered before tick is complete we will stop there instead. */
	uint8 bDebugFrameStepExecution:1;

	/** Indicates that a single frame advance happened this frame. */
	uint8 bDebugFrameStepExecutedThisFrame : 1;

	/** Indicates that toggling between Play-in-Editor and Simulate-in-Editor happened this frame. */
	uint8 bToggledBetweenPIEandSIEThisFrame : 1;

	/** Indicates that the renderer scene for this editor world was purged while Play-in-Editor. */
	uint8 bPurgedScene : 1;
#endif

	/** Keeps track whether actors moved via PostEditMove and therefore constraint syncup should be performed. */
	UPROPERTY(transient)
	uint8 bAreConstraintsDirty:1;

private:
	/** Whether the render scene for this World should be created with HitProxies or not */
	uint8 bRequiresHitProxies:1;

	/** Whether to do any ticking at all for this world. */
	uint8 bShouldTick:1;

	/** Whether we have a pending call to BuildStreamingData(). */
	uint8 bStreamingDataDirty : 1;

	/** Is forcibly unloading streaming levels?																					*/
	uint8 bShouldForceUnloadStreamingLevels:1;

	/** Is forcibly making streaming levels visible?																			*/
	uint8 bShouldForceVisibleStreamingLevels:1;

	/** Is there at least one material parameter collection instance waiting for a deferred update?								*/
	uint8 bMaterialParameterCollectionInstanceNeedsDeferredUpdate : 1;

	/** Whether InitWorld was ever called on this world since its creation. Not cleared to false during CleanupWorld			*/
	uint8 bHasEverBeenInitialized: 1;

	/** Whether FLumenSceneData::bTrackAllPrimitives can be set even if the world is a preview world. Used to make asset editor preview viewports work with Lumen. */
	uint8 bAllowLumenPrimitiveTrackingInPreviewWorld : 1;

	/** Whether PostGC was registered. It can happen in PostLoad for loaded levels or InitWorld for Persistent levels that were loaded or created at runtime			*/
	uint8 bIsThePostGCDelegateRegistered : 1;
	
	/** Indicates that the world is in the process of being cleaned up */
	bool bIsBeingCleanedUp;
	
	/** Whether the world is currently in a BlockTillLevelStreamingCompleted() call */
	uint32 IsInBlockTillLevelStreamingCompleted;

	/** Epoch updated every time BlockTillLevelStreamingCompleted() is called. */
	int32 BlockTillLevelStreamingCompletedEpoch;

	/** The world's navigation data manager */
	UPROPERTY(Transient)
	TObjectPtr<class UNavigationSystemBase>				NavigationSystem;

	/** The current GameMode, valid only on the server */
	UPROPERTY(Transient)
	TObjectPtr<class AGameModeBase>						AuthorityGameMode;

	/** The replicated actor which contains game state information that can be accessible to clients. Direct access is not allowed, use GetGameState<>() */
	UPROPERTY(Transient)
	TObjectPtr<class AGameStateBase>						GameState;

	/** The AI System handles generating pathing information and AI behavior */
	UPROPERTY(Transient)
	TObjectPtr<class UAISystemBase>						AISystem;
	
	/** RVO avoidance manager used by game */
	UPROPERTY(Transient)
	TObjectPtr<class UAvoidanceManager>					AvoidanceManager;

	/** Array of levels currently in this world. Not serialized to disk to avoid hard references. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<class ULevel>>						Levels;

	/** Array of level collections currently in this world. */
	UPROPERTY(Transient, NonTransactional, Setter = None, Getter = None)
	TArray<FLevelCollection>					LevelCollections;

	/** Index of the level collection that's currently ticking. */
	int32										ActiveLevelCollectionIndex;

	/** Creates the dynamic source and static level collections if they don't already exist. */
	UE_API void ConditionallyCreateDefaultLevelCollections();
	
	FOnBeginPlay OnBeginPlay;
public:

	/** Handle to the active audio device for this world. */
	FAudioDeviceHandle AudioDeviceHandle;

#if WITH_EDITOR
	/** Hierarchical LOD System. */
	struct FHierarchicalLODBuilder*				HierarchicalLODBuilder;

	/** Original World Name before PostLoad rename. Used to get external actors on disk. */
	FName OriginalWorldName;
#endif // WITH_EDITOR

	/** Called from DemoNetDriver when playing back a replay and the timeline is successfully scrubbed */
	UFUNCTION()
	UE_API void HandleTimelineScrubbed();

private:

	/** Handle to delegate in case audio device is destroyed. */
	FDelegateHandle AudioDeviceDestroyedHandle;

#if WITH_EDITORONLY_DATA
	/** Pointer to the current level being edited. Level has to be in the Levels array and == PersistentLevel in the game. */
	UPROPERTY(Transient)
	TObjectPtr<class ULevel>								CurrentLevel;
#endif

	UPROPERTY(Transient)
	TObjectPtr<class UGameInstance>						OwningGameInstance;

	/** Parameter collection instances that hold parameter overrides for this world. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<class UMaterialParameterCollectionInstance>> ParameterCollectionInstances;

	/** 
	 * Canvas object used for drawing to render targets from blueprint functions eg DrawMaterialToRenderTarget.
	 * This is cached as UCanvas creation takes >100ms.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UCanvas> CanvasForRenderingToTarget;

	UPROPERTY(Transient)
	TObjectPtr<UCanvas> CanvasForDrawMaterialToRenderTarget;

public:
	/** Set the pointer to the Navigation System instance. */
	UE_API void SetNavigationSystem(UNavigationSystemBase* InNavigationSystem);

	/** The interface to the scene manager for this world. */
	class FSceneInterface*						Scene;

#if WITH_EDITORONLY_DATA
	/** Saved editor viewport states - one for each view type. Indexed using ELevelViewportType from UnrealEdTypes.h.	*/
	UPROPERTY(NonTransactional)
	TArray<FLevelViewportInfo>					EditorViews;
#endif

#if WITH_EDITORONLY_DATA
	/** 
	 * Set the CurrentLevel for this world. 
	 * @return true if the current level changed.
	 */
	UE_API bool SetCurrentLevel( class ULevel* InLevel );
#endif

	/** Get the CurrentLevel for this world. **/
	UE_API class ULevel* GetCurrentLevel() const;

	/** A static map that is populated before loading a world from a package. This is so UWorld can look up its WorldType in ::PostLoad */
	static UE_API TMap<FName, EWorldType::Type> WorldTypePreLoadMap;

#if WITH_EDITOR
	/** Map of blueprints that are being debugged and the object instance they are debugging. */
	typedef TMap<TWeakObjectPtr<class UBlueprint>, TWeakObjectPtr<UObject> > FBlueprintToDebuggedObjectMap;

	/** Return the array of objects currently being debugged. */
	const FBlueprintToDebuggedObjectMap& GetBlueprintObjectsBeingDebugged() const{ return BlueprintObjectsBeingDebugged; };
#endif

	/** Creates a new FX system for this world */
	UE_API void CreateFXSystem();

	/** Initialize all world subsystems */
	UE_API void InitializeSubsystems();

	/** Finalize initialization of all world subsystems */
	UE_API void PostInitializeSubsystems();

#if WITH_EDITOR

	/** Change the feature level that this world is current rendering with */
	UE_DEPRECATED(5.8, "ChangeFeatureLevel is deprecated, use ShaderPlatformChanged instead")
	UE_API void ChangeFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel, bool bShowSlowProgressDialog = true, bool bForceUpdate = false);

	UE_API void ShaderPlatformChanged(EShaderPlatform InShaderPlatform = EShaderPlatform::SP_NumPlatforms, bool bShowSlowProgressDialog = true);

	UE_API void RecreateScene(ERHIFeatureLevel::Type InFeatureLevel, bool bBroadcastChange = true);

	/** Recreate the editor world's FScene with a null scene interface to drop extra GPU memory during PIE */
	UE_API void PurgeScene();

	/** Restore the purged editor world FScene back to the proper GPU representation */
	UE_API void RestoreScene();

	static UE_API void OnAddExtraObjectsToDelete(const TArray<UObject*>& InObjectsToDelete, TSet<UObject*>& OutSecondaryObjects);

#endif // WITH_EDITOR

	/**
	 * Sets whether or not this world is ticked by the engine, but use it at your own risk! This could
	 * have unintended consequences if used carelessly. That said, for worlds that are not interactive
	 * and not rendering, it can save the cost of ticking them. This should probably never be used
	 * for a primary game world.
	 */
	void SetShouldTick(const bool bInShouldTick) { bShouldTick = bInShouldTick; }

	/** Returns whether or not this world is currently ticking. See SetShouldTick. */
	bool ShouldTick() const { return bShouldTick; }

private:
	/** List of all the controllers in the world. */
	TArray<TWeakObjectPtr<class AController> > ControllerList;

	/** List of all the player controllers in the world. */
	TArray<TWeakObjectPtr<class APlayerController> > PlayerControllerList;

	/** List of all the cameras in the world that auto-activate for players. */
	TArray<TWeakObjectPtr<ACameraActor> > AutoCameraActorList;

	/** List of all physics volumes in the world. Does not include the DefaultPhysicsVolume. */
	TArray<TWeakObjectPtr<APhysicsVolume> > NonDefaultPhysicsVolumeList;

	/** Physics scene for this world. */
	FPhysScene*									PhysicsScene;
	// Note that this should be merged with PhysScene going forward but is needed for now.

#if WITH_STATE_STREAM
	/** Manager for all existing state streams */
	IStateStreamManager* StateStreamManager = nullptr;
	uint32 LaneId;
#endif

public:

	/** Current global physics scene. */
	TSharedPtr<FPhysScene_Chaos> PhysicsScene_Chaos;

	/** Default global physics scene. */
	TSharedPtr<FPhysScene_Chaos> DefaultPhysicsScene_Chaos;

	/** Access to the ChaosEventRelay to access all registered events. */
	UE_API UChaosEventRelay* GetChaosEventRelay();

	/** Physics Field component. */
	UPROPERTY(Transient)
	TObjectPtr<class UPhysicsFieldComponent> PhysicsField;


#if WITH_STATE_STREAM
	/** Get statestream of type T */
	template<typename T> T& GetStateStream() { return *static_cast<T*>(InternalGetStateStream(T::Id)); }

	/** Get statestream manager */
	IStateStreamManager* GetStateStreamManager() { return StateStreamManager; }
	uint32 GetLaneId() { return LaneId; }

#endif

private:
	enum class EEndOfFrameUpdateLocation
	{
		Immediate,
		Async
	};

	inline TArray<TObjectPtr<UActorComponent>>& GetComponentsThatNeedEndOfFrameUpdate(UActorComponent* Component)
	{
		return Component->IsReadyForEarlyEndOfFrameUpdate() ? ComponentsThatNeedEndOfFrameUpdate_AsyncTick : ComponentsThatNeedEndOfFrameUpdate;
	}

	inline const TArray<TObjectPtr<UActorComponent>>& GetComponentsThatNeedEndOfFrameUpdate(UActorComponent* Component) const
	{
		return Component->IsReadyForEarlyEndOfFrameUpdate() ? ComponentsThatNeedEndOfFrameUpdate_AsyncTick : ComponentsThatNeedEndOfFrameUpdate;
	}

	inline TSet<TObjectPtr<UActorComponent>>& GetComponentsThatNeedEndOfFrameSync(UActorComponent* Component)
	{
		return Component->IsReadyForEarlyEndOfFrameUpdate() ? ComponentsThatNeedPreEndOfFrameSync_AsyncTick : ComponentsThatNeedPreEndOfFrameSync;
	}

	/** Array of components that need to wait on tasks before end of frame updates */
	UPROPERTY(Transient, NonTransactional)
	TSet<TObjectPtr<UActorComponent>> ComponentsThatNeedPreEndOfFrameSync;

	/** Array of components that need to wait on tasks before end of frame updates */
	UPROPERTY(Transient, NonTransactional)
	TSet<TObjectPtr<UActorComponent>> ComponentsThatNeedPreEndOfFrameSync_AsyncTick;

	/** Array of components that need updates at the end of the frame */
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UActorComponent>> ComponentsThatNeedEndOfFrameUpdate;

	/** Array of components that need updates at the end of the frame */
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UActorComponent>> ComponentsThatNeedEndOfFrameUpdate_AsyncTick;

	/** Array of components that need game thread updates at the end of the frame */
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UActorComponent>> ComponentsThatNeedEndOfFrameUpdate_OnGameThread;

	/** Array of components that need to be marked dirty *after* updating, because it wasn't safe to do so during an update. */
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UActorComponent>> ComponentsThatNeedEndOfFrameMarkRenderStateDirty;

	struct FEndOfFrameUpdateContext;
	TArray<FEndOfFrameUpdateContext*> EndOfFrameUpdateContexts;
	TArray<UE::Tasks::FTask> EndOfFrameUpdateTasks;
	UE::Tasks::FTask EndOfFrameUpdatePrerequisiteTask;

	void SendAllEndOfFrameUpdatesInternal(EEndOfFrameUpdateLocation Location);

	/** The state of async tracing - abstracted into its own object for easier reference */
	FWorldAsyncTraceState AsyncTraceState;

#if !UE_BUILD_SHIPPING
	double AsyncDelegateHitchLoggingLastTimestamp = 0.0;
#endif //!UE_BUILD_SHIPPING

#if WITH_EDITOR
	/**	Objects currently being debugged in Kismet	*/
	FBlueprintToDebuggedObjectMap BlueprintObjectsBeingDebugged;
#endif

	/**
	 * Broadcasts a notification whenever an actor is spawned.
	 * This event is only for newly created actors.
	 */
	mutable FOnActorSpawned OnActorSpawned;

	/**
	 * Broadcasts a notification before a newly spawned actor is initialized.
	 * This event is only for newly created actors.
	 */
	mutable FOnActorSpawned OnActorPreSpawnInitialization;

	/**
	 * Broadcasts a notification whenever an actor is destroyed.
	 * This event is not fired for unloaded actors.
	 */
	mutable FOnActorDestroyed OnActorDestroyed;

	/**
	 * Broadcasts after an actor has registered all its components.
	 * This is called for both spawned and loaded actors.
	 */
	mutable FOnPostRegisterAllActorComponents OnPostRegisterAllActorComponents;

	/**
	 * Broadcasts before an actor unregisters all its components.
	 * This is called for both spawned and loaded actors.
	 */
	mutable FOnPreUnregisterAllActorComponents OnPreUnregisterAllActorComponents;

	/**
	 * Broadcasts when an actor has been removed from the world.
	 * This event is the earliest point where an actor can be safely renamed without affecting replication.
	 */
	mutable FOnActorRemovedFromWorld OnActorRemovedFromWorld;

	/** Reset Async Trace Buffer **/
	UE_API void ResetAsyncTrace();

	/** Wait for all Async Trace Buffer to be done **/
	UE_API void WaitForAllAsyncTraceTasks();

	/** Finish Async Trace Buffer **/
	UE_API void FinishAsyncTrace();

	/** Utility function that is used to ensure that a World has the correct singleton actor of the provided class */
	UE_API void RepairSingletonActorOfClass(TSubclassOf<AActor> ActorClass);	
	template <class T> void RepairSingletonActorOfClass() { RepairSingletonActorOfClass(T::StaticClass()); }

	/** Utility function that is used to ensure that a World has the correct WorldSettings */
	UE_API void RepairWorldSettings();

	/** Utility function that is used to ensure that a World has the correct singleton actors*/
	UE_API void RepairSingletonActors();

	/** Utility function to cleanup streaming levels that point to invalid level packages */
	UE_API void RepairStreamingLevels();

	/** Utility function that is used to ensure that a World has the correct ChaosActor */
	UE_API void RepairChaosActors();

#if WITH_EDITOR
	/** Utility function to make sure there is a valid default builder brush */
	UE_API void RepairDefaultBrush();
#endif

	/** Gameplay timers. */
	class FTimerManager* TimerManager;

	/** Latent action manager. */
	struct FLatentActionManager LatentActionManager;

	/** Timestamp (in FPlatformTime::Seconds) when the next call to BuildStreamingData() should be made, if bDirtyStreamingData is true. */
	double BuildStreamingDataTimer;

	using FOnNetTickEvent UE_DEPRECATED(5.5, "The FOnNetTickEvent typedef has been deprecated - use TMulticastDelegateRegistration<void(float)> instead.") = TMulticastDelegate<void(float)>;
	using FOnTickFlushEvent UE_DEPRECATED(5.5, "The FOnTickFlushEvent typedef has been deprecated - use TMulticastDelegateRegistration<void()> instead.") = TMulticastDelegate<void()>;

	/** Event to gather up all net drivers and call TickDispatch at once */
	UE_DEPRECATED_FORGAME(5.5, "Public access to TickDispatchEvent has been deprecated - use OnTickDispatch() instead.")
	TMulticastDelegate<void(float)> TickDispatchEvent;

	/** Event to gather up all net drivers and call PostTickDispatch at once */
	UE_DEPRECATED_FORGAME(5.5, "Public access to PostTickDispatchEvent has been deprecated - use OnPostTickDispatch() instead.")
	TMulticastDelegate<void()> PostTickDispatchEvent;

	/** Event called prior to calling TickFlush */
	UE_DEPRECATED_FORGAME(5.5, "Public access to PreTickFlushEvent has been deprecated - use OnPreTickFlush() instead.")
	TMulticastDelegate<void(float)> PreTickFlushEvent;

	/** Event to gather up all net drivers and call TickFlush at once */
	UE_DEPRECATED_FORGAME(5.5, "Public access to TickFlushEvent has been deprecated - use OnTickFlush() instead.")
	TMulticastDelegate<void(float)> TickFlushEvent;
	
	/** Event to gather up all net drivers and call PostTickFlush at once */
	UE_DEPRECATED_FORGAME(5.5, "Public access to PostTickFlushEvent has been deprecated - use OnPostTickFlush() instead.")
	TMulticastDelegate<void()> PostTickFlushEvent;

	/** All registered net drivers TickDispatch() */
	UE_API void BroadcastTickDispatch(float DeltaTime);

	/** All registered net drivers PostTickDispatch() */
	UE_API void BroadcastPostTickDispatch();

	/** PreTickFlush */
	UE_API void BroadcastPreTickFlush(float DeltaTime);

	/** All registered net drivers TickFlush() */
	UE_API void BroadcastTickFlush(float DeltaTime);

	/** All registered net drivers PostTickFlush() */
	UE_API void BroadcastPostTickFlush(float DeltaTime);

	/** Called when the number of levels changes. */
	DECLARE_EVENT(UWorld, FOnLevelsChangedEvent);
	
	/** Broadcasts whenever the number of levels changes */
	FOnLevelsChangedEvent LevelsChangedEvent;

	/** Called when all the levels have changed. */
	DECLARE_EVENT(UWorld, FOnAllLevelsChangedEvent);

	/** Broadcasts whenever all the levels change */
	FOnAllLevelsChangedEvent AllLevelsChangedEvent;

	/** Broadcasted when WorldPartition gets initialized */
	DECLARE_EVENT_OneParam(UWorld, FWorldPartitionInitializedEvent, UWorldPartition*);
	
	FWorldPartitionInitializedEvent OnWorldPartitionInitializedEvent;

	void BroadcastWorldPartitionInitialized(UWorldPartition* InWorldPartition)
	{
		OnWorldPartitionInitializedEvent.Broadcast(InWorldPartition);
	}

	/** Broadcasted when WorldPartition gets uninitialized */
	DECLARE_EVENT_OneParam(UWorld, FWorldPartitionUninitializedEvent, UWorldPartition*);

	FWorldPartitionUninitializedEvent OnWorldPartitionUninitializedEvent;

	void BroadcastWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
	{
		OnWorldPartitionUninitializedEvent.Broadcast(InWorldPartition);
	}

	/** Broadcasted when Level is being added to the world (used to add custom logic).
      * Set bOutHasCompleted to false if custom execution has not completed.
	  */
	DECLARE_EVENT_ThreeParams(UWorld, FAddLevelToWorldExtensionEvent, ULevel* /*InLevel*/, const bool /*bWaitForCompletion*/, bool& /*bOutHasCompleted*/);
	FAddLevelToWorldExtensionEvent AddLevelToWorldExtensionEvent;

	/** Broadcasted when Level is being removed from the world (used to add custom logic).
      * Set bOutHasCompleted to false if custom execution has not completed.
	  */
	DECLARE_EVENT_ThreeParams(UWorld, FRemoveLevelFromWorldExtensionEvent, ULevel* /*InLevel*/, const bool /*bWaitForCompletion*/, bool& /*bOutHasCompleted*/);
	FRemoveLevelFromWorldExtensionEvent RemoveLevelFromWorldExtensionEvent;

	friend FWorldPartitionEvents;
	friend class FWorldPartitionLevelHelper;

#if WITH_EDITOR

	/** Broadcasts that selected levels have changed. */
	UE_API void BroadcastSelectedLevelsChanged();

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Called when selected level list changes. */
	DECLARE_TS_MULTICAST_DELEGATE(FOnSelectedLevelsChangedEvent);

	/** Broadcasts whenever selected level list changes. */
	FOnSelectedLevelsChangedEvent				SelectedLevelsChangedEvent;

	/** Array of selected levels currently in this world. Not serialized to disk to avoid hard references.	*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<class ULevel>>			SelectedLevels;

	/** Disables the broadcasting of level selection change. Internal use only. */
	uint32 bBroadcastSelectionChange:1;

	/** a delegate that broadcasts a notification whenever the current feature level is changed */
	FOnFeatureLevelChanged OnFeatureLevelChanged;
#endif //WITH_EDITORONLY_DATA

	FOnMovieSceneSequenceTick MovieSceneSequenceTick;

public:
	/** The URL that was used when loading this World.																			*/
	FURL										URL;

	/** Interface to the FX system managing particles and related effects for this world.										*/
	TSharedPtr<class FFXSystemInterface>		FXSystem;

	/** Tick function for starting physics																						*/
	FStartPhysicsTickFunction StartPhysicsTickFunction;
	/** Tick function for ending physics																						*/
	FEndPhysicsTickFunction EndPhysicsTickFunction;

	/** Counter for allocating game- unique controller player numbers															*/
	int32										PlayerNum;
	
	/** Number of frames to delay Streaming Volume updating, useful if you preload a bunch of levels but the camera hasn't caught up yet (INDEX_NONE for infinite) */
	int32										StreamingVolumeUpdateDelay;

	bool GetShouldForceUnloadStreamingLevels() const { return bShouldForceUnloadStreamingLevels; }
	UE_API void SetShouldForceUnloadStreamingLevels(bool bInShouldForceUnloadStreamingLevels);

	bool GetShouldForceVisibleStreamingLevels() const { return bShouldForceVisibleStreamingLevels; }
	UE_API void SetShouldForceVisibleStreamingLevels(bool bInShouldForceVisibleStreamingLevels);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** When non-'None', all line traces where the TraceTag match this will be drawn */
	FName    DebugDrawTraceTag;

	/** When set to true, all scene queries will be drawn */
	bool bDebugDrawAllTraceTags;

	bool DebugDrawSceneQueries(const FName& UsedTraceTag) const
	{
		return (bDebugDrawAllTraceTags || ((DebugDrawTraceTag != NAME_None) && (DebugDrawTraceTag == UsedTraceTag))) && IsInGameThread();
	}
#endif

	/** Called when the world computes how post process volumes contribute to the scene. */
	DECLARE_EVENT_TwoParams(UWorld, FOnBeginPostProcessSettings, FVector, FSceneView*);
	FOnBeginPostProcessSettings OnBeginPostProcessSettings;

	// Storing PPVs safely is somewhat awkward due to historical API decisions.
	// We want to store IInterface_PostProcessVolume*, but callers do not always properly call the Remove function,
	// which leads to dangling pointers.
	// Most PPVs are UObject-based, and so can use TWeakInterfacePtr to detect object expiration.
	// The API signature historically allows for passing a raw pointer though, so we also support those cases for now.
	// TLDR: Variant type for PPV storage: UObject-based PPVs use TWeakInterfacePtr (detects stale entries),
	// non-UObject raw pointer PPVs are assumed always valid.
	using FPostProcessVolumeEntry = TVariant<
		TWeakInterfacePtr<IInterface_PostProcessVolume>,
		IInterface_PostProcessVolume*>;

	/** Iterator range over post process volumes in ascending priority order, filtering out invalid/expired entries. */
	struct FPostProcessVolumeIterator
	{
		struct FSentinel {};

		struct FIterator
		{
			FIterator(const TConstArrayView<FPostProcessVolumeEntry> InEntries, int32 InIndex)
				: Entries(InEntries), Index(InIndex)
			{
				AdvanceToValid();
			}

			IInterface_PostProcessVolume& operator*() const { return *Current; }

			FIterator& operator++()
			{
				++Index;
				AdvanceToValid();
				return *this;
			}

			bool operator!=(FSentinel) const { return Current != nullptr; }

		private:
			UE_API void AdvanceToValid();

			const TConstArrayView<FPostProcessVolumeEntry> Entries;
			int32 Index;
			IInterface_PostProcessVolume* Current = nullptr;
		};

		explicit FPostProcessVolumeIterator(const TArray<FPostProcessVolumeEntry>& InEntries)
			: Entries(InEntries)
		{}

		FIterator begin() const { return FIterator(Entries, 0); }
		FSentinel end() const { return {}; }

	private:
		const TConstArrayView<FPostProcessVolumeEntry> Entries;
	};

	/** Inserts a UObject-based post process volume into the world in priority order */
	UE_API void AddPostProcessVolume(TWeakInterfacePtr<IInterface_PostProcessVolume> InVolume);

	UE_DEPRECATED(5.8, "Use AddPostProcessVolume(TWeakInterfacePtr<IInterface_PostProcessVolume>): for memory-safety, all IInterface_PostProcessVolume need to be UObject-based")
	UE_API void InsertPostProcessVolume(IInterface_PostProcessVolume* InVolume);

	/** Removes a post process volume from the world.  Returns true if it was successfully removed. 
	 *  Accepts both UObject and non-UObject based PPVs.
	 */
	UE_API bool RemovePostProcessVolume(IInterface_PostProcessVolume* InVolume);

	/** Called when a scene view for this world needs the worlds post process settings computed */
	UE_API void AddPostProcessingSettings(FVector ViewLocation, FSceneView* SceneView);

	/** Returns an iterator over post process volumes in ascending priority order, filtering out invalid entries. */
	FPostProcessVolumeIterator GetPostProcessVolumeIterator() const
	{
		return FPostProcessVolumeIterator(PostProcessVolumeEntries);
	}

	UE_DEPRECATED(5.8, "Use GetPostProcessVolumeIterator() instead")
	TArray<IInterface_PostProcessVolume*> PostProcessVolumes;


private:
	TArray<FPostProcessVolumeEntry> PostProcessVolumeEntries;
	void InsertPostProcessVolumeEntry(FPostProcessVolumeEntry InVolume);
	void RemovePostProcessVolume(int32 Index); // do not make public, indices are unstable
	/** Caches the size of each volume at the time of insertion, for sort determinism in case it later changes.  One for one with PostProcessVolumeEntries above. */
	TArray<double> PostProcessVolumeCachedSizes;
public:

	/** Set of AudioVolumes sorted by  */
	// TODO: Make this be property UPROPERTY(Transient)
	TArray<class AAudioVolume*> AudioVolumes;

	/** Time in FPlatformTime::Seconds unbuilt time was last encountered. 0 means not yet.							*/
	double LastTimeUnbuiltLightingWasEncountered;

	/**  Time in seconds since level began play, but IS paused when the game is paused, and IS dilated/clamped. */
	double TimeSeconds;

	/**  Time in seconds since level began play, but IS NOT paused when the game is paused, and IS dilated/clamped. */
	double UnpausedTimeSeconds;

	/** Time in seconds since level began play, but IS NOT paused when the game is paused, and IS NOT dilated/clamped. */
	double RealTimeSeconds;

	/** Time in seconds since level began play, but IS paused when the game is paused, and IS NOT dilated/clamped. */
	double AudioTimeSeconds;

	/** Frame delta time in seconds with no adjustment for time dilation. */
	float DeltaRealTimeSeconds;

	/** Frame delta time in seconds adjusted by e.g. time dilation. */
	float DeltaTimeSeconds;

	/** time at which to start pause **/
	double PauseDelay;

	/** Current location of this world origin */
	FIntVector OriginLocation;

	/** Requested new world origin location */
	FIntVector RequestedOriginLocation;

	/** World origin offset value. Non-zero only for a single frame when origin is rebased */
	FVector OriginOffsetThisFrame;

	/** Amount of time to wait before traveling to next map, gives clients time to receive final RPCs @see ServerTravelPause */
	float NextSwitchCountdown;

	/** All levels information from which our world is composed */
	UPROPERTY()
	TObjectPtr<class UWorldComposition> WorldComposition;

	UPROPERTY()
	TObjectPtr<class UContentBundleManager> ContentBundleManager;
	
	/** Whether we flushing level streaming state */ 
	EFlushLevelStreamingType FlushLevelStreamingType;

	/** The type of travel to perform next when doing a server travel */
	TEnumAsByte<ETravelType> NextTravelType;

private:
	/** An internal count of how many streaming levels are currently loading */
	uint16 NumStreamingLevelsBeingLoaded; // Move this somewhere better

	friend struct FWorldNotifyStreamingLevelLoading;

public:
	/** The URL to be used for the upcoming server travel */
	FString NextURL;

	/** array of levels that were loaded into this map via PrepareMapChange() / CommitMapChange() (to inform newly joining clients) */
	TArray<FName> PreparingLevelNames;

	/** Name of persistent level if we've loaded levels via CommitMapChange() that aren't normally in the StreamingLevels array (to inform newly joining clients) */
	FName CommittedPersistentLevelName;

#if !UE_BUILD_SHIPPING || WITH_EDITOR
	/**
	 * This is a int on the level which is set when a light that needs to have lighting rebuilt
	 * is moved.  This is then checked in CheckMap for errors to let you know that this level should
	 * have lighting rebuilt.
	 **/
	uint32 NumLightingUnbuiltObjects;
	
	uint32 NumUnbuiltReflectionCaptures;

	/** Num of components missing valid texture streaming data. Updated in map check. */
	int32 NumTextureStreamingUnbuiltComponents;

	/** Num of resources that have changed since the last texture streaming build. Updated in map check. */
	int32 NumTextureStreamingDirtyResources;
#endif

	/** Indicates that the world has marked contained objects as pending kill */
	bool HasMarkedObjectsPendingKill() const { return bMarkedObjectsPendingKill; }
private:
	uint32 bMarkedObjectsPendingKill:1;

#if WITH_EDITOR
	uint32 bWorldWasCleanedUp: 1;
#endif

	uint32 CleanupWorldTag;
	static UE_API uint32 CleanupWorldGlobalTag;

public:
#if WITH_EDITOR
	/** List of DDC async requests we need to wait on before we register components. Game thread only. */
	TArray<TSharedPtr<FAsyncPreRegisterDDCRequest>> AsyncPreRegisterDDCRequests;
#endif

	//Experimental: In game performance tracking.
	FWorldInGamePerformanceTrackers* PerfTrackers;

	//Tracking for VFX cost for this world.
	mutable FParticlePerfStats* ParticlePerfStats = nullptr;

	/**
	 * UWorld default constructor
	 */
	UE_API UWorld(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	// LINE TRACE

	/**
	 *  Trace a ray against the world using a specific channel and return if a blocking hit is found.
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool LineTraceTestByChannel(const FVector& Start,const FVector& End,ECollisionChannel TraceChannel, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;
	

	/**
	 *  Trace a ray against the world using object types and return if a blocking hit is found.
	 *  @param  Start           	Start location of the ray
	 *  @param  End             	End location of the ray
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param  Params          	Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool LineTraceTestByObjectType(const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Trace a ray against the world using a specific profile and return if a blocking hit is found.
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool LineTraceTestByProfile(const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Trace a ray against the world using a specific channel and return the first blocking hit
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool LineTraceSingleByChannel(struct FHitResult& OutHit,const FVector& Start,const FVector& End,ECollisionChannel TraceChannel,const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	 *  Deproject a 2D screen-space coordinate using the supplied projection data, then line-trace from the resulting world-space ray.
	 *
	 *  @param  OutHit          First blocking hit found
	 *  @param  ProjectionData  View origin / rotation / projection matrix and view rect used for deprojection
	 *  @param  ScreenPosition  Screen-space coordinate (pixels, relative to the view rect)
	 *  @param  TraceChannel    The 'channel' the ray is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 *  @param  TraceDistance   Maximum distance to trace along the deprojected direction
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool LineTraceFromScreenProjection(FHitResult& OutHit, const FSceneViewProjectionData& ProjectionData, const FVector2D& ScreenPosition, const ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const float TraceDistance) const;

	/**
	 *  Trace a ray against the world using object types and return the first blocking hit
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool LineTraceSingleByObjectType(struct FHitResult& OutHit,const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Trace a ray against the world using a specific profile and return the first blocking hit
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool LineTraceSingleByProfile(struct FHitResult& OutHit, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Trace a ray against the world using a specific channel and return overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool LineTraceMultiByChannel(TArray<struct FHitResult>& OutHits,const FVector& Start,const FVector& End,ECollisionChannel TraceChannel,const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;


	/**
	 *  Trace a ray against the world using object types and return overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool LineTraceMultiByObjectType(TArray<struct FHitResult>& OutHits,const FVector& Start,const FVector& End,const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;


	/**
	 *  Trace a ray against the world using a specific profile and return overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool LineTraceMultiByProfile(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Sweep a shape against the world using a specific channel and return if a blocking hit is found.
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  TraceChannel    The 'channel' that this trace uses, used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 	 
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool SweepTestByChannel(const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	 *  Sweep a shape against the world using object types and return if a blocking hit is found.
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool SweepTestByObjectType(const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;


	/**
	 *  Sweep a shape against the world using a specific profile and return if a blocking hit is found.
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if a blocking hit is found
	 */
	UE_API bool SweepTestByProfile(const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params) const;

	/**
	 *  Sweep a shape against the world and return the first blocking hit using a specific channel
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  TraceChannel    The 'channel' that this trace is in, used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool SweepSingleByChannel(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	 *  Sweep a shape against the world and return the first blocking hit using object types
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool SweepSingleByObjectType(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Sweep a shape against the world and return the first blocking hit using a specific profile
	 *  @param  OutHit          First blocking hit found
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool SweepSingleByProfile(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Sweep a shape against the world and return all initial overlaps using a specific channel (including blocking) if requested, then overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool SweepMultiByChannel(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	 *  Sweep a shape against the world and return all initial overlaps using object types (including blocking) if requested, then overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any hit is found
	 */
	UE_API bool SweepMultiByObjectType(TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Sweep a shape against the world and return all initial overlaps using a specific profile, then overlapping hits and then first blocking hit
	 *  Results are sorted, so a blocking hit (if found) will be the last element of the array
	 *  Only the single closest blocking result will be generated, no tests will be done after that
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool SweepMultiByProfile(TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	*  Test the collision of a shape at the supplied location using a specific channel, and return if any blocking overlap is found
	*  @param  Pos             Location of center of box to test against the world
	*  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	*  @param  CollisionShape	CollisionShape - supports Box, Sphere, Capsule, Convex
	*  @param  Params          Additional parameters used for the trace
	*  @param  ResponseParam	ResponseContainer to be used for this trace
	*  @return TRUE if any blocking results are found
	*/
	UE_API bool OverlapBlockingTestByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	*  Test the collision of a shape at the supplied location using a specific channel, and return if any blocking or overlapping shape is found
	*  @param  Pos             Location of center of box to test against the world
	*  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	*  @param  CollisionShape	CollisionShape - supports Box, Sphere, Capsule, Convex
	*  @param  Params          Additional parameters used for the trace
	*  @param  ResponseParam	ResponseContainer to be used for this trace
	*  @return TRUE if any blocking or overlapping results are found
	*/
	UE_API bool OverlapAnyTestByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;

	/**
	*  Test the collision of a shape at the supplied location using object types, and return if any overlap is found
	*  @param  Pos             Location of center of box to test against the world
	*  @param  ObjectQueryParams	List of object types it's looking for
	*  @param  CollisionShape	CollisionShape - supports Box, Sphere, Capsule, Convex
	*  @param  Params          Additional parameters used for the trace
	*  @return TRUE if any blocking results are found
	*/
	UE_API bool OverlapAnyTestByObjectType(const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	*  Test the collision of a shape at the supplied location using a specific profile, and return if any blocking overlap is found
	*  @param  Pos             Location of center of box to test against the world
	*  @param  ProfileName     The 'profile' used to determine which components to hit
	*  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	*  @param  Params          Additional parameters used for the trace
	*  @return TRUE if any blocking results are found
	*/
	UE_API bool OverlapBlockingTestByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	*  Test the collision of a shape at the supplied location using a specific profile, and return if any blocking or overlap is found
	*  @param  Pos             Location of center of box to test against the world
	*  @param  ProfileName     The 'profile' used to determine which components to hit
	*  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	*  @param  Params          Additional parameters used for the trace
	*  @return TRUE if any blocking or overlapping results are found
	*/
	UE_API bool OverlapAnyTestByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	
	/**
	 *  Test the collision of a shape at the supplied location using a specific channel, and determine the set of components that it overlaps
	 *  @param  OutOverlaps     Array of components found to overlap supplied box
	 *  @param  Pos             Location of center of shape to test against the world
	 *  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *  @return TRUE if OutOverlaps contains any blocking results
	 */
	UE_API bool OverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam) const;


	/**
	 *  Test the collision of a shape at the supplied location using object types, and determine the set of components that it overlaps
	 *  @param  OutOverlaps     Array of components found to overlap supplied box
	 *  @param  Pos             Location of center of shape to test against the world
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if any overlap is found
	 */
	UE_API bool OverlapMultiByObjectType(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	/**
	 *  Test the collision of a shape at the supplied location using a specific profile, and determine the set of components that it overlaps
	 *  @param  OutOverlaps     Array of components found to overlap supplied box
	 *  @param  Pos             Location of center of shape to test against the world
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutOverlaps contains any blocking results
	 */
	UE_API bool OverlapMultiByProfile(TArray<struct FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam) const;

	// COMPONENT SWEEP

	/**
	 *  Sweep the geometry of the supplied component, and determine the set of components that it hits.
	 *  @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  PrimComp        Component's geometry to test against the world. Transform of this component is ignored
	 *  @param  Start           Start location of the trace
	 *  @param  End             End location of the trace
	 *  @param  Rot             Rotation of PrimComp geometry for test against the world (rotation remains constant over sweep)
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool ComponentSweepMulti(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FQuat& Rot,    const FComponentQueryParams& Params) const;
	bool ComponentSweepMulti(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FRotator& Rot, const FComponentQueryParams& Params) const;

	/**
	 *  Sweep the geometry of the supplied component using a specific channel, and determine the set of components that it hits.
	 *  @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 *  @param  OutHits         Array of hits found between ray and the world
	 *  @param  PrimComp        Component's geometry to test against the world. Transform of this component is ignored
	 *  @param  Start           Start location of the trace
	 *  @param  End             End location of the trace
	 *  @param  Rot             Rotation of PrimComp geometry for test against the world (rotation remains constant over sweep)
	 *  @param  Params          Additional parameters used for the trace
	 *  @return TRUE if OutHits contains any blocking hit entries
	 */
	UE_API bool ComponentSweepMultiByChannel(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params) const;
	bool ComponentSweepMultiByChannel(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FRotator& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params) const;

	// COMPONENT OVERLAP

	/**
	 *  Test the collision of the supplied component at the supplied location/rotation using object types, and determine the set of components that it overlaps
	 *  @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 *  @param  OutOverlaps     Array of overlaps found between component in specified pose and the world
	 *  @param  PrimComp        Component's geometry to test against the world. Transform of this component is ignored
	 *  @param  Pos             Location of PrimComp geometry for test against the world
	 *  @param  Rot             Rotation of PrimComp geometry for test against the world
	 *	@param	ObjectQueryParams	List of object types it's looking for. When this enters, we do object query with component shape
	 *  @return TRUE if any hit is found
	 */
	UE_API bool ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FQuat& Rot,    const FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const FCollisionObjectQueryParams& ObjectQueryParams=FCollisionObjectQueryParams::DefaultObjectQueryParam) const;
	bool ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FRotator& Rot, const FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const FCollisionObjectQueryParams& ObjectQueryParams=FCollisionObjectQueryParams::DefaultObjectQueryParam) const;

	/**
	 *  Test the collision of the supplied component at the supplied location/rotation using a specific channel, and determine the set of components that it overlaps
	 *  @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 *  @param  OutOverlaps     Array of overlaps found between component in specified pose and the world
	 *  @param  PrimComp        Component's geometry to test against the world. Transform of this component is ignored
	 *  @param  Pos             Location of PrimComp geometry for test against the world
	 *  @param  Rot             Rotation of PrimComp geometry for test against the world
	 *  @param  TraceChannel	The 'channel' that this query is in, used to determine which components to hit
	 *  @return TRUE if OutOverlaps contains any blocking results
	 */
	UE_API bool ComponentOverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FQuat& Rot,    ECollisionChannel TraceChannel, const FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const FCollisionObjectQueryParams& ObjectQueryParams=FCollisionObjectQueryParams::DefaultObjectQueryParam) const;
	bool ComponentOverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FRotator& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params = FComponentQueryParams::DefaultComponentQueryParams, const FCollisionObjectQueryParams& ObjectQueryParams=FCollisionObjectQueryParams::DefaultObjectQueryParam) const;


	/**
	 * Interface for Async. Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param	UserData		UserData
	 */ 
	UE_API FTraceHandle	AsyncLineTraceByChannel(EAsyncTraceType InTraceType, const FVector& Start,const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0 );

	/**
	 * Interface for Async. Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single hit result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param	UserData		UserData
	 */ 
	UE_API FTraceHandle	AsyncLineTraceByObjectType(EAsyncTraceType InTraceType, const FVector& Start,const FVector& End, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0 );

	/**
	 * Interface for Async. Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the ray
	 *  @param  End             End location of the ray
	 *  @param  ProfileName		The 'profile' used to determine which components to hit
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function,
	 *
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 *
	 *	@param	UserData		UserData
	 */
	UE_API FTraceHandle	AsyncLineTraceByProfile(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, FName ProfileName, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single hit result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  TraceChannel    The 'channel' that this trace is in, used to determine which components to hit
	 *  @param	CollisionShape		CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace	 
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param	UserData		UserData
	 */ 
	UE_API FTraceHandle	AsyncSweepByChannel(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single hit result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape		CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param	UserData		UserData
	 */ 
	UE_API FTraceHandle	AsyncSweepByObjectType(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *	@param	InTraceType		Indicates if you want multiple results, single hit result, or just yes/no (no hit information)
	 *  @param  Start           Start location of the shape
	 *  @param  End             End location of the shape
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape	CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function,
	 *
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 *
	 *	@param	UserData		UserData
	 */
	UE_API FTraceHandle	AsyncSweepByProfile(EAsyncTraceType InTraceType, const FVector& Start, const FVector& End, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FTraceDelegate* InDelegate = nullptr, uint32 UserData = 0);

	// overlap functions

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *  @param  Pos             Location of center of shape to test against the world
	 *  @param  TraceChannel    The 'channel' that this query is in, used to determine which components to hit
	 *  @param	CollisionShape		CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 * 	@param 	ResponseParam	ResponseContainer to be used for this trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param UserData			UserData
	 */ 
	UE_API FTraceHandle	AsyncOverlapByChannel(const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam, const FOverlapDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *  @param  Pos             Location of center of shape to test against the world
	 *	@param	ObjectQueryParams	List of object types it's looking for
	 *  @param	CollisionShape		CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function, 
	 *						
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 * 
	 *	@param UserData			UserData
	 */ 
	UE_API FTraceHandle	AsyncOverlapByObjectType(const FVector& Pos, const FQuat& Rot, const FCollisionObjectQueryParams& ObjectQueryParams, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FOverlapDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Interface for Async trace
	 * Pretty much same parameter set except you can optional set delegate to be called when execution is completed and you can set UserData if you'd like
	 * if no delegate, you can query trace data using QueryTraceData or QueryOverlapData
	 * the data is available only in the next frame after request is made - in other words, if request is made in frame X, you can get the result in frame (X+1)
	 *
	 *  @param  Pos             Location of center of shape to test against the world
	 *  @param  ProfileName     The 'profile' used to determine which components to hit
	 *  @param	CollisionShape		CollisionShape - supports Box, Sphere, Capsule
	 *  @param  Params          Additional parameters used for the trace
	 *	@param	InDelegate		Delegate function to be called - to see example, search FTraceDelegate
	 *							Example can be void MyActor::TraceDone(const FTraceHandle& TraceHandle, FTraceDatum & TraceData)
	 *							Before sending to the function,
	 *
	 *							FTraceDelegate TraceDelegate;
	 *							TraceDelegate.BindRaw(this, &MyActor::TraceDone);
	 *
	 *	@param UserData			UserData
	 */
	UE_API FTraceHandle	AsyncOverlapByProfile(const FVector& Pos, const FQuat& Rot, FName ProfileName, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Params = FCollisionQueryParams::DefaultQueryParam, const FOverlapDelegate* InDelegate = nullptr, uint32 UserData = 0);

	/**
	 * Query function 
	 * return true if already done and returning valid result - can be hit or no hit
	 * return false if either expired or not yet evaluated or invalid
	 * Use IsTraceHandleValid to find out if valid and to be evaluated
	 */
	UE_API bool QueryTraceData(const FTraceHandle& Handle, FTraceDatum& OutData);

	/**
	 * Query function 
	 * return true if already done and returning valid result - can be hit or no hit
	 * return false if either expired or not yet evaluated or invalid
	 * Use IsTraceHandleValid to find out if valid and to be evaluated
	 */
	UE_API bool QueryOverlapData(const FTraceHandle& Handle, FOverlapDatum& OutData);
	/** 
	 * See if TraceHandle is still valid or not
	 *
	 * @param	Handle			TraceHandle that was returned when request Trace
	 * @param	bOverlapTrace	true if this is overlap test Handle, not trace test handle
	 * 
	 * return true if it will be evaluated OR it has valid result 
	 * return false if it already has expired Or not valid 
	 */
	UE_API bool IsTraceHandleValid(const FTraceHandle& Handle, bool bOverlapTrace);

private:
	friend class FPhysicsAsyncQueryReporter;
	UE_API void AddTraceData(const FTraceHandle& Handle, const TArray<FHitResult>& Results);
	UE_API void AddOverlapData(const FTraceHandle& Handle, const TArray<FOverlapResult>& Results);
	
	static UE_API void GetCollisionProfileChannelAndResponseParams(FName ProfileName, ECollisionChannel& CollisionChannel, FCollisionResponseParams& ResponseParams);

public:

	/** NavigationSystem getter */
	inline UNavigationSystemBase* GetNavigationSystem() { return NavigationSystem; }
	/** NavigationSystem const getter */
	inline const UNavigationSystemBase* GetNavigationSystem() const { return NavigationSystem; }

	/** AISystem getter. if AISystem is missing it tries to create one and returns the result.
	 *	@NOTE the result can be NULL, for example on client games or if no AI module or AISystem class have not been specified
	 *	@see UAISystemBase::AISystemClassName and UAISystemBase::AISystemModuleName*/
	UE_API UAISystemBase* CreateAISystem();

	/** AISystem getter */
	inline UAISystemBase* GetAISystem() { return AISystem; }
	/** AISystem const getter */
	inline const UAISystemBase* GetAISystem() const { return AISystem; }
	
	/** Avoidance manager getter */
	inline class UAvoidanceManager* GetAvoidanceManager() { return AvoidanceManager; }
	/** Avoidance manager getter */
	inline const class UAvoidanceManager* GetAvoidanceManager() const { return AvoidanceManager; }

	/** Returns an iterator for the controller list. */
	UE_API FConstControllerIterator GetControllerIterator() const;

	/** @return Returns the number of Controllers. */
	UE_API int32 GetNumControllers() const;
	
	/** @return Returns an iterator for the player controller list. */
	UE_API FConstPlayerControllerIterator GetPlayerControllerIterator() const;

	/** @return Returns the number of Player Controllers. */
	UE_API int32 GetNumPlayerControllers() const;
	
	/** 
	 * @return Returns the first player controller cast to the template type, or NULL if there is not one.
	 *
	 * May return NULL if the cast fails.
	 */
	template< class T >
	T* GetFirstPlayerController() const
	{
		return Cast<T>(GetFirstPlayerController());
	}
	
	/** @return Returns the first player controller, or NULL if there is not one. */	
	UE_API APlayerController* GetFirstPlayerController() const;
	
	/*
	 *	Get the first valid local player via the first player controller.
	 *
	 *  @return Pointer to the first valid ULocalPlayer cast to the template type, or NULL if there is not one.
	 *
	 *  May Return NULL if the cast fails.
	 */	
	template< class T >
	T* GetFirstLocalPlayerFromController() const
	{
		return Cast<T>(GetFirstLocalPlayerFromController());
	}

	/*
	 *	Get the first valid local player via the first player controller.
	 *
	 *  @return Pointer to the first valid ULocalPlayer, or NULL if there is not one.
	 */	
	UE_API ULocalPlayer* GetFirstLocalPlayerFromController() const;

	/** Register a CameraActor that auto-activates for a PlayerController. */
	UE_API void RegisterAutoActivateCamera(ACameraActor* CameraActor, int32 PlayerIndex);

	/** Get an iterator for the list of CameraActors that auto-activate for PlayerControllers. */
	UE_API FConstCameraActorIterator GetAutoActivateCameraIterator() const;
	
	/** Returns a reference to the game viewport displaying this world if one exists. */
	UE_API UGameViewportClient* GetGameViewport() const;

public:

	/** 
	 * Returns the default brush for the persistent level.
	 * This is usually the 'builder brush' for editor builds, undefined for non editor instances and may be NULL.
	 */
	UE_API ABrush* GetDefaultBrush() const;

	/** Returns true if the actors have been initialized and are ready to start play */
	UE_API bool AreActorsInitialized() const;

	/** For backwards compatibility */
	using FActorsInitializedParams = ::FActorsInitializedParams;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldInitializedActors, const FActorsInitializedParams&);
	FOnWorldInitializedActors OnActorsInitialized;

	DECLARE_MULTICAST_DELEGATE(FOnWorldBeginPlay);
	FOnWorldBeginPlay OnWorldBeginPlay;

	DECLARE_MULTICAST_DELEGATE(FOnMatchStarting);
	FOnMatchStarting OnWorldMatchStarting;

	/** Called by WorldSettings's NotifyBeginPlay before calling DispatchBeginPlay on all world actors when world's bBegunPlay switches to true. */
	DECLARE_MULTICAST_DELEGATE(FOnWorldPreBeginPlay);
	FOnWorldPreBeginPlay OnWorldPreBeginPlay;

	/** Returns true if gameplay has already started, false otherwise. */
	UE_API bool HasBegunPlay() const;

	/**
	 * Returns time in seconds since world was brought up for play, IS stopped when game pauses, IS dilated/clamped
	 *
	 * @return time in seconds since world was brought up for play
	 */
	double GetTimeSeconds() const;

	/**
	* Returns time in seconds since world was brought up for play, IS NOT stopped when game pauses, IS dilated/clamped
	*
	* @return time in seconds since world was brought up for play
	*/
	double GetUnpausedTimeSeconds() const;

	/**
	* Returns time in seconds since world was brought up for play, does NOT stop when game pauses, NOT dilated/clamped
	*
	* @return time in seconds since world was brought up for play
	*/
	double GetRealTimeSeconds() const;

	/**
	* Returns time in seconds since world was brought up for play, IS stopped when game pauses, NOT dilated/clamped
	*
	* @return time in seconds since world was brought up for play
	*/
	double GetAudioTimeSeconds() const;

	/**
	 * Returns the frame delta time in seconds adjusted by e.g. time dilation.
	 *
	 * @return frame delta time in seconds adjusted by e.g. time dilation
	 */
	float GetDeltaSeconds() const;
	
	/**
	 * Returns the dilatable time
	 *
	 * @return Returns the dilatable time
	 */
	FGameTime GetTime() const;

	/** Helper for getting the time since a certain time. */
	double TimeSince(double Time) const;

	/** Creates a new physics scene for this world. */
	UE_API void CreatePhysicsScene(const AWorldSettings* Settings = nullptr);

	/** Returns a pointer to the physics scene for this world. */
	FPhysScene* GetPhysicsScene() const { return PhysicsScene; }

	/** Set the physics scene to use by this world */
	UE_API void SetPhysicsScene(FPhysScene* InScene);

	/**
	 * Returns the default physics volume and creates it if necessary.
	 * 
	 * @return default physics volume
	 */
	APhysicsVolume* GetDefaultPhysicsVolume() const { return DefaultPhysicsVolume ? ToRawPtr(DefaultPhysicsVolume) : InternalGetDefaultPhysicsVolume(); }

	/** Returns true if a DefaultPhysicsVolume has been created. */
	bool HasDefaultPhysicsVolume() const { return DefaultPhysicsVolume != nullptr; }

	/** Add a physics volume to the list of those in the world. DefaultPhysicsVolume is not tracked. Used internally by APhysicsVolume. */
	UE_API void AddPhysicsVolume(APhysicsVolume* Volume);

	/** Removes a physics volume from the list of those in the world. */
	UE_API void RemovePhysicsVolume(APhysicsVolume* Volume);

	/** Get an iterator for all PhysicsVolumes in the world that are not a DefaultPhysicsVolume. */
	FConstPhysicsVolumeIterator GetNonDefaultPhysicsVolumeIterator() const;

	/** Get the count of all PhysicsVolumes in the world that are not a DefaultPhysicsVolume. */
	int32 GetNonDefaultPhysicsVolumeCount() const;

	UE_API void SetAllowDeferredPhysicsStateCreation(bool bAllow);
	UE_API bool GetAllowDeferredPhysicsStateCreation() const;

	/**
	 * Returns the current (or specified) level's level scripting actor
	 *
	 * @param	OwnerLevel	the level to get the level scripting actor for.  Must correspond to one of the levels in GWorld's Levels array;
	 *						Thus, only applicable when editing a multi-level map.  Defaults to the level currently being edited.
	 *
	 * @return	A pointer to the level scripting actor, if any, for the specified level, or NULL if no level scripting actor is available
	 */
	UE_API class ALevelScriptActor* GetLevelScriptActor( class ULevel* OwnerLevel=NULL ) const;

	/**
	 * Returns the AWorldSettings actor associated with this world.
	 *
	 * @return AWorldSettings actor associated with this world
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|World", meta=(DisplayName="GetWorldSettings", ScriptName="GetWorldSettings"))
	UE_API AWorldSettings* K2_GetWorldSettings();
	UE_API AWorldSettings* GetWorldSettings( bool bCheckStreamingPersistent = false, bool bChecked = true ) const;

	/**
	 * Returns the AWorldDataLayers actor associated with this world.
	 *
	 * @return AWorldDataLayers actor associated with this world
	 */
	UE_API AWorldDataLayers* GetWorldDataLayers() const;
	UE_API void SetWorldDataLayers(AWorldDataLayers* NewWorldDataLayers);

	/** Returns a human friendly display string for the current world (showing the kind of world when in multiplayer PIE) */
	UE_API FString GetDebugDisplayName() const;
	/**
	 * Returns the UWorldPartition associated with this world.
	 *
	 * @return UWorldPartition object associated with this world
	 */
	UE_API UWorldPartition* GetWorldPartition() const;

	/**
	 * Returns the UDataLayerManager associated with this world.
	 *
	 * @return UDataLayerManager object associated with this world
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|WorldPartition")
	UE_API UDataLayerManager* GetDataLayerManager() const;

	/**
	* Returns true if world contains an associated UWorldPartition object.
	*/
	bool IsPartitionedWorld() const { return GetWorldPartition() != nullptr; }

	/**
	* Returns true if world contains an associated UWorldPartition object.
	*/
	static bool IsPartitionedWorld(const UWorld* InWorld)
	{
		if (InWorld)
		{
			return InWorld->IsPartitionedWorld();
		}

		return false;
	}

	FWorldPartitionInitializedEvent& OnWorldPartitionInitialized() { return OnWorldPartitionInitializedEvent; }
	FWorldPartitionUninitializedEvent& OnWorldPartitionUninitialized() { return OnWorldPartitionUninitializedEvent; }
		
	/**
	 * Returns the current levels BSP model.
	 *
	 * @return BSP UModel
	 */
	UE_API UModel* GetModel() const;

	/**
	 * Returns the Z component of the current world gravity.
	 *
	 * @return Z component of current world gravity.
	 */
	UE_API float GetGravityZ() const;

	/**
	 * Returns the Z component of the default world gravity.
	 *
	 * @return Z component of the default world gravity.
	 */
	UE_API float GetDefaultGravityZ() const;

	/**
	 * Returns the name of the current map, taking into account using a dummy persistent world
	 * and loading levels into it via PrepareMapChange.
	 *
	 * @return	name of the current map
	 */
	UE_API const FString GetMapName() const;
	
	/** Accessor for bRequiresHitProxies. */
	bool RequiresHitProxies() const 
	{
		return bRequiresHitProxies;
	}

	/** Accessor for bAllowLumenPrimitiveTrackingInPreviewWorld. */
	bool AllowLumenPrimitiveTrackingInPreviewWorld() const
	{
		return bAllowLumenPrimitiveTrackingInPreviewWorld;
	}

	/**
	 * Inserts the passed in controller at the front of the linked list of controllers.
	 *
	 * @param	Controller	Controller to insert, use NULL to clear list
	 */
	UE_API void AddController( AController* Controller );
	
	/**
	 * Removes the passed in controller from the linked list of controllers.
	 *
	 * @param	Controller	Controller to remove
	 */
	UE_API void RemoveController( AController* Controller );
	
	/**
	 * Adds the passed in actor to the special network actor list
	 * This list is used to specifically single out actors that are relevant for networking without having to scan the much large list
	 * @param	Actor	Actor to add
	 */
	UE_API void AddNetworkActor( AActor* Actor );
	
	/**
	 * Removes the passed in actor to from special network actor list
	 * @param	Actor	Actor to remove
	 */
	UE_API void RemoveNetworkActor( AActor* Actor ) const;

	/** Add a listener for OnActorSpawned events */
	UE_API FDelegateHandle AddOnActorSpawnedHandler( const FOnActorSpawned::FDelegate& InHandler ) const;

	/** Remove a listener for OnActorSpawned events */
	UE_API void RemoveOnActorSpawnedHandler( FDelegateHandle InHandle ) const;

	/** Notify the World that an Actor has finished spawning (all OnActorSpawnedHandlers will run) */
	UE_API void OnActorFinishedSpawning(AActor* Actor);

	/** Add a listener for OnActorPreSpawnInitialization events */
	UE_API FDelegateHandle AddOnActorPreSpawnInitialization(const FOnActorSpawned::FDelegate& InHandler) const;

	/** Remove a listener for OnActorPreSpawnInitialization events */
	UE_API void RemoveOnActorPreSpawnInitialization(FDelegateHandle InHandle) const;

	/** Add a listener for OnActorDestroyed events */
	UE_API FDelegateHandle AddOnActorDestroyedHandler(const FOnActorDestroyed::FDelegate& InHandler) const;

	/** Remove a listener for OnActorDestroyed events */
	UE_DEPRECATED(5.5, "Use the typo corrected version RemoveOnActorDestroyedHandler")
	UE_API void RemoveOnActorDestroyededHandler(FDelegateHandle InHandle) const;
	UE_API void RemoveOnActorDestroyedHandler(FDelegateHandle InHandle) const;

	/** Add a listener for OnPostRegisterAllActorComponents events */
	UE_API FDelegateHandle AddOnPostRegisterAllActorComponentsHandler(const FOnPostRegisterAllActorComponents::FDelegate& InHandler) const;

	/** Remove a listener for OnPostRegisterAllActorComponents events */
	UE_API void RemoveOnPostRegisterAllActorComponentsHandler(FDelegateHandle InHandle) const;

	/**
	 * Broadcast an OnPostRegisterAllActorComponents event.
	 * This method should only be called from internal actor and level code and never on inactive worlds.
	 */
	UE_API void NotifyPostRegisterAllActorComponents(AActor* Actor);

	/** Add a listener for OnPreUnregisterAllActorComponents events */
	UE_API FDelegateHandle AddOnPreUnregisterAllActorComponentsHandler(const FOnPreUnregisterAllActorComponents::FDelegate& InHandler) const;

	/** Remove a listener for OnPreUnregisterAllActorComponents events */
	UE_API void RemoveOnPreUnregisterAllActorComponentsHandler(FDelegateHandle InHandle) const;

	/**
	 * Broadcast an OnPreUnregisterAllActorComponents event.
	 * This method should only be called from internal actor and level code. Calls on inactive or GCing worlds are
	 * ignored.
	 */
	UE_API void NotifyPreUnregisterAllActorComponents(AActor* Actor);

	/** Add a listener for OnActorRemovedFromWorld events */
	UE_API FDelegateHandle AddOnActorRemovedFromWorldHandler(const FOnActorRemovedFromWorld::FDelegate& InHandler) const;

	/** Remove a listener for OnActorRemovedFromWorld events */
	UE_API void RemoveOnActorRemovedFromWorldHandler(FDelegateHandle InHandle) const;

	/**
	 * Returns whether the passed in actor is part of any of the loaded levels actors array.
	 * Warning: Will return true for pending kill actors!
	 *
	 * @param	Actor	Actor to check whether it is contained by any level
	 *	
	 * @return	true if actor is contained by any of the loaded levels, false otherwise
	 */
	UE_API bool ContainsActor( AActor* Actor ) const;

	/**
	 * Returns whether audio playback is allowed for this scene.
	 *
	 * @return true if current world is GWorld, false otherwise
	 */
	UE_API bool AllowAudioPlayback() const;

	/** Adds a tick handler for sequences. These handlers get ticked before pre-physics */
	UE_API FDelegateHandle AddMovieSceneSequenceTickHandler(const FOnMovieSceneSequenceTick::FDelegate& InHandler);
	/** Removes a tick handler for sequences */
	UE_API void RemoveMovieSceneSequenceTickHandler(FDelegateHandle InHandle);
	/** Check if movie sequences tick handler is bound at all */
	UE_API bool IsMovieSceneSequenceTickHandlerBound() const;

	//~ Begin UObject Interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static UE_API void AppendToClassSchema(FAppendToClassSchemaContext& Context);
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	UE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	UE_API virtual UWorld* GetWorld() const override;
	UE_API virtual FPrimaryAssetId GetPrimaryAssetId() const override;
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
	UE_API virtual bool Rename(const TCHAR* NewName = NULL, UObject* NewOuter = NULL, ERenameFlags Flags = REN_None) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	UE_API virtual void GetExtendedAssetRegistryTagsForSave(const ITargetPlatform* TargetPlatform, TArray<FAssetRegistryTag>& OutTags) const override;
protected:
	UE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const override;
public:
	UE_API virtual bool IsNameStableForNetworking() const override;
#endif
	UE_API virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface
	
	/**
	 * Clears all level components and world components like e.g. line batcher.
	 */
	UE_API void ClearWorldComponents();

	/**
	 * Updates world components like e.g. line batcher and all level components.
	 *
	 * @param	bRerunConstructionScripts	If we should rerun construction scripts on actors
	 * @param	bCurrentLevelOnly			If true, affect only the current level.
	 */
	UE_API void UpdateWorldComponents(bool bRerunConstructionScripts, bool bCurrentLevelOnly, FRegisterComponentContext* Context = nullptr);

	/**
	 * Updates cull distance volumes for a specified component or a specified actor or all actors
         * @param ComponentToUpdate If specified just that Component will be updated
	 * @param ActorToUpdate If specified (and ComponentToUpdate is not specified), all Components owned by this Actor will be updated
	 * @return True if the passed in actors or components were within a volume
	 */
	UE_API bool UpdateCullDistanceVolumes(AActor* ActorToUpdate = nullptr, UPrimitiveComponent* ComponentToUpdate = nullptr);

	/**
	 * Cleans up components, streaming data and assorted other intermediate data.
	 * @param bSessionEnded whether to notify the viewport that the game session has ended.
	 * @param NewWorld Optional new world that will be loaded after this world is cleaned up. Specify a new world to prevent it and it's sublevels from being GCed during map transitions.
	 */
	UE_API void CleanupWorld(bool bSessionEnded = true, bool bCleanupResources = true, UWorld* NewWorld = nullptr);
	
	/**
	 * Invalidates the cached data used to render the levels' UModel.
	 *
	 * @param	InLevel		Level to invalidate. If this is NULL it will affect ALL levels
	 */
	UE_API void InvalidateModelGeometry( ULevel* InLevel );

	/**
	 * Discards the cached data used to render the levels' UModel.  Assumes that the
	 * faces and vertex positions haven't changed, only the applied materials.
	 *
	 * @param	bCurrentLevelOnly		If true, affect only the current level.
	 */
	UE_API void InvalidateModelSurface(bool bCurrentLevelOnly);

	/**
	 * Commits changes made to the surfaces of the UModels of all levels.
	 */
	UE_API void CommitModelSurfaces();

	/** Purges all sky capture cached derived data. */
	UE_API void InvalidateAllSkyCaptures();

	/** Purges all sky capture cached derived data and forces a re-render of captured scene data. */
	UE_API void UpdateAllSkyCaptures();

	/** Returns the active lighting scenario for this world or NULL if none. */
	UE_API ULevel* GetActiveLightingScenario() const;

	/** Propagates a change to the active lighting scenario. */
	UE_API void PropagateLightingScenarioChange();

	/**
	 * Associates the passed in level with the world. If using a time limit the work to make the level visible can be spread across several frames
	 * so this function has to be called multiple time for the level to be visible/ associated with the world and no longer be in a limbo state.
	 *
	 * @param Level				Level object we should add
	 * @param LevelTransform	Transformation to apply to each actor in the level
	 * @param bConsiderTimeLimit optional bool indicating if time-slicing will be used to limit the amount of work done, defaults to false
	 * @param ExternalTimeout optional timeout that will override the default behavior if bConsiderTimeLimit is true
	 * @param TransactionId optional parameter that carries the current transaction id associated with calls updating LevelVisibility used when communicating level visibility with server
	 * @param OwningLevelStreaming optional parameter, the ULevelStreaming object driving this level's presence in the world
	 */
	UE_API void AddToWorld(ULevel* Level, const FTransform& LevelTransform = FTransform::Identity, bool bConsiderTimeLimit = false, const TOptional<const UE::FTimeout>& ExternalTimeout = TOptional<const UE::FTimeout>(), FNetLevelVisibilityTransactionId TransactionId = FNetLevelVisibilityTransactionId(), ULevelStreaming* OwningLevelStreaming = nullptr);

	UE_DEPRECATED(5.6, "Use version that requires passing the ExternalTimout, bConsiderTimeLimit now defaults to false to match common usage")
	void AddToWorld(ULevel* Level, const FTransform& LevelTransform, bool bConsiderTimeLimit, FNetLevelVisibilityTransactionId TransactionId, ULevelStreaming* OwningLevelStreaming = nullptr)
	{
		AddToWorld(Level, LevelTransform, bConsiderTimeLimit, TOptional<const UE::FTimeout>(), TransactionId, OwningLevelStreaming);
	}

	/** 
	 * Dissociates the passed in level from the world. The removal is blocking unless bAllowIncrementalRemoval is set
	 *
	 * @param Level			Level object we should remove
	 * @param bAllowIncrementalRemoval optional bool indicating if time-slicing will be used to remove objects from scene, defaults to false
	 * @param ExternalTimeout optional timeout that will override the default behavior if bAllowIncrementalRemoval is true
	 * @param TransactionId optional parameter that carries the current transaction id associated with calls updating LevelVisibility used when communicating level visibility with server
	 * @param OwningLevelStreaming optional parameter, the ULevelStreaming object driving this level's presence in the world
	 */
	UE_API void RemoveFromWorld(ULevel* Level, bool bAllowIncrementalRemoval = false, const TOptional<const UE::FTimeout>& ExternalTimeout = TOptional<const UE::FTimeout>(), FNetLevelVisibilityTransactionId TransactionId = FNetLevelVisibilityTransactionId(), ULevelStreaming* OwningLevelStreaming = nullptr);

	UE_DEPRECATED(5.6, "Use version that requires passing the optional ExternalTimout")
	void RemoveFromWorld(ULevel* Level, bool bAllowIncrementalRemoval, FNetLevelVisibilityTransactionId TransactionId, ULevelStreaming* OwningLevelStreaming = nullptr)
	{
		RemoveFromWorld(Level, bAllowIncrementalRemoval, TOptional<const UE::FTimeout>(), TransactionId, OwningLevelStreaming);
	}

	/** Returns current timeout used by AddToWorld */
	UE_API UE::FTimeout GetAddToWorldTimeout() const;

	/** Returns current timeout used by RemoveFromWorld */
	UE_API UE::FTimeout GetRemoveFromWorldTimeout() const;

	/**
	 * Updates sub-levels (load/unload/show/hide) using streaming levels current state
	 *
	 * @param ExternalTimeout Optional soft limit to the time this function is allowed to take [time-slicing].
	 */
	UE_API void UpdateLevelStreaming(const TOptional<const UE::FTimeout>& ExternalTimeout = TOptional<const UE::FTimeout>());

	/** Releases PhysicsScene manually */
	UE_API void ReleasePhysicsScene();
public:
	/**
	 * Flushes level streaming in blocking fashion and returns when all levels are loaded/ visible/ hidden
	 * so further calls to UpdateLevelStreaming won't do any work unless state changes. Basically blocks
	 * on all async operation like updating components.
	 *
	 * @param FlushType					Whether to only flush level visibility operations (optional)
	 */
	UE_API void FlushLevelStreaming(EFlushLevelStreamingType FlushType = EFlushLevelStreamingType::Full);

	/**
	 * Triggers a call to ULevel::BuildStreamingData(this,NULL,NULL) within a few seconds.
	 */
	UE_API void TriggerStreamingDataRebuild();

	/**
	 * Calls ULevel::BuildStreamingData(this,NULL,NULL) if it has been triggered within the last few seconds.
	 */
	UE_API void ConditionallyBuildStreamingData();

	/** @return whether there is at least one level with a pending visibility request */
	UE_API bool IsVisibilityRequestPending() const;

	/** Returns whether all the 'always loaded' levels are loaded. */
	UE_API bool AreAlwaysLoadedLevelsLoaded() const;

	/** Requests async loading of any 'always loaded' level. Used in seamless travel to prevent blocking in the first UpdateLevelStreaming.  */
	UE_API void AsyncLoadAlwaysLoadedLevelsForSeamlessTravel();

	/**
	 * Returns whether the level streaming code is allowed to issue load requests.
	 *
	 * @return true if level load requests are allowed, false otherwise.
	 */
	UE_API bool AllowLevelLoadRequests() const;

	/** Creates instances for each parameter collection in memory.  Called when a world is created. */
	UE_API void SetupParameterCollectionInstances();

	/** Adds a new instance of the given collection, or overwrites an existing instance if there is one. */
	UE_API void AddParameterCollectionInstance(class UMaterialParameterCollection* Collection, bool bUpdateScene);

	/** Gets this world's instance for a given collection. */
	UE_API UMaterialParameterCollectionInstance* GetParameterCollectionInstance(const UMaterialParameterCollection* Collection) const;

	/** Updates this world's scene with the list of instances, and optionally updates each instance's uniform buffer. */
	UE_API void UpdateParameterCollectionInstances(bool bUpdateInstanceUniformBuffers, bool bRecreateUniformBuffer);

	/* clean up any material parameter collection instances which have had their collections destroyed */
	UE_API void OnPostGC(); 

private:
	UE_API UMaterialParameterCollectionInstance* CreateParameterCollectionInstance(int32 ExistingIndex, UMaterialParameterCollection* Collection, bool bUpdateScene);

	/** Returns true if streaming levels have not loaded and are waiting on outstanding async load requests */
	UE_API bool HasAsyncLevelRequests();

	/** Waits for all streaming levels with outstanding async load requests */
	UE_API void FlushAsyncLevelRequests();

public:

	/** Gets the canvas object for rendering to a render target.  Will allocate one if needed. */
	UE_API UCanvas* GetCanvasForRenderingToTarget();
	UE_API UCanvas* GetCanvasForDrawMaterialToRenderTarget();

	// Legacy for backwards compatibility
	using InitializationValues = FWorldInitializationValues;

	/**
	 * Initializes the world, associates the persistent level and sets the proper zones.
	 */
	UE_API void InitWorld(const FWorldInitializationValues IVS = FWorldInitializationValues());
#if WITH_EDITOR
	/**
	 * InitWorld usually has to be balanced with CleanupWorld. If the KeepInitializedDuringLoadTag LinkerInstancingContext tag is present,
	 * operations that need to call InitWorld during the Load of the World's package should break that rule and not call CleanupWorld.
	 */
	static UE_API const FName KeepInitializedDuringLoadTag;

	/** Returns whether InitWorld has ever been called since this World was created.  */
	bool HasEverBeenInitialized() const { return bHasEverBeenInitialized; }

	/**
	 * Calls CleanupWorld and InitWorld, while handling preservation of StreamingLevels and WorldInitialization values
	 * Gives a warning and does nothing if IsInitialized is not currently true.
	 */
	UE_API void ReInitWorld();
#endif

	/** Returns whether InitWorld has been called without yet calling CleanupWorld.  */
	bool IsInitialized() const { return bIsWorldInitialized; }

	/**
	 * Initializes a newly created world.
	 */
	UE_API void InitializeNewWorld(const InitializationValues IVS = InitializationValues(), bool bInSkipInitWorld = false);
	
	/**
	 * Static function that creates a new UWorld and returns a pointer to it
	 */
	static UE_API UWorld* CreateWorld( const EWorldType::Type InWorldType, bool bInformEngineOfWorld, FName WorldName = NAME_None, UPackage* InWorldPackage = NULL, bool bAddToRoot = true, ERHIFeatureLevel::Type InFeatureLevel = ERHIFeatureLevel::Num, const InitializationValues* InIVS = nullptr, bool bInSkipInitWorld = false);

	/** 
	 * Destroy this World instance. If destroying the world to load a different world, supply it here to prevent GC of the new world or it's sublevels.
	 */
	UE_API void DestroyWorld( bool bInformEngineOfWorld, UWorld* NewWorld = nullptr );

	/** Returns true if the world is in the process of being cleaned up. */
	bool IsBeingCleanedUp() const { return bIsBeingCleanedUp; }

	/** Returns true if the world has or is in the process of being cleaned up. */
	bool IsCleanedUp() const { return CleanupWorldTag != 0; }

#if WITH_EDITOR
	/** Force reset world to a state where it's not considered cleaned up. Use with caution this could disable important validations regarding component registration. */
	void ResetCleanedUpState() { CleanupWorldTag = 0; }
#endif


	/** 
	 * Marks this world and all objects within as pending kill
	 */
	UE_API void MarkObjectsPendingKill();

	/**
	 *	Remove NULL entries from actor list. Only does so for dynamic actors to avoid resorting. 
	 *	In theory static actors shouldn't be deleted during gameplay.
	 */
	UE_API void CleanupActors();	

public:

	/** Network Tick events */
	UE_API TMulticastDelegateRegistration<void(float)>& OnTickDispatch();
	UE_API TMulticastDelegateRegistration<void()>& OnPostTickDispatch();
	UE_API TMulticastDelegateRegistration<void(float)>& OnPreTickFlush();
	UE_API TMulticastDelegateRegistration<void(float)>& OnTickFlush();
	UE_API TMulticastDelegateRegistration<void()>& OnPostTickFlush();

	/**
	 * Update the level after a variable amount of time, DeltaSeconds, has passed.
	 * All child actors are ticked after their owners have been ticked.
	 */
	UE_API void Tick( ELevelTick TickType, float DeltaSeconds );

	/**
	 * Set up the physics tick function if they aren't already
	 */
	UE_API void SetupPhysicsTickFunctions(float DeltaSeconds);

	/**
	 * Run a tick group, ticking all actors and components
	 * @param Group - Ticking group to run
	 * @param bBlockTillComplete - if true, do not return until all ticks are complete
	 */
	UE_API void RunTickGroup(ETickingGroup Group, bool bBlockTillComplete);

#if UE_WITH_PSO_PRECACHING
	/**
	 * Informs the world that a primitive component has completed a PSO pre-cache operation.
	 */
	UE_API void AddPrimitiveComponentPSOPrecacheCompleted(UPrimitiveComponent* Component);
#endif

	/**
	 * Mark a component as needing an end of frame update
	 * @param Component - Component to update at the end of the frame
	 * @param bForceGameThread - if true, force this to happen on the game thread
	 */
	UE_API void MarkActorComponentForNeededEndOfFrameUpdate(UActorComponent* Component, bool bForceGameThread);

	/**
	* Clears the need for a component to have an end of frame update
	* @param Component - Component to update at the end of the frame
	*/
	UE_API void ClearActorComponentEndOfFrameUpdate(UActorComponent* Component);

#if WITH_EDITOR
	/**
	 * Updates an ActorComponent's cached state of whether it has been marked for end of frame update based on the current
	 * state of the World's NeedsEndOfFrameUpdate arrays
	 * @param Component - Component to update the cached state of
	 */
	UE_API void UpdateActorComponentEndOfFrameUpdateState(UActorComponent* Component) const;
#endif

	/** 
	 * Used to indicate a UMaterialParameterCollectionInstance needs a deferred update 
	 */
	UE_API void SetMaterialParameterCollectionInstanceNeedsUpdate();

	/** 
	 * Returns true if we have any updates that have been deferred to the end of the current frame.
	 */
	UE_API bool HasEndOfFrameUpdates() const;

	/**
	 * Send all render updates to the rendering thread.
	 */
	UE_API void SendAllEndOfFrameUpdates();

	/**
	 * Starts an async region of sending frame updates to the render thread.
	 */
	UE_API void StartAsyncSendAllEndOfFrameUpdates();

	/**
	 * Waits for an async region of sending frame updates to the render thread.
	 */
	UE_API void FinishAsyncSendAllEndOfFrameUpdates();

	/**
	 * Flush any pending parameter collection updates to the render thread.
	 */
	UE_API void FlushDeferredParameterCollectionInstanceUpdates();

	/** Do per frame tick behaviors related to the network driver */
	UE_API void TickNetClient( float DeltaSeconds );

	/**
	 * Issues level streaming load/unload requests based on whether
	 * local players are inside/outside level streaming volumes.
	 *
	 * @param OverrideViewLocation Optional position used to override the location used to calculate current streaming volumes
	 */
	UE_API void ProcessLevelStreamingVolumes(FVector* OverrideViewLocation=NULL);

	/*
	 * Updates world's level streaming state using active game players view and blocks until all sub - levels are loaded / visible / hidden
	 * so further calls to UpdateLevelStreaming won't do any work unless state changes.
	 */
	UE_API void BlockTillLevelStreamingCompleted();

	/**
	 * Transacts the specified level -- the correct way to modify a level
	 * as opposed to calling Level->Modify.
	 */
	UE_API void ModifyLevel(ULevel* Level) const;

	/**
	 * Ensures that the collision detection tree is fully built. This should be called after the full level reload to make sure
	 * the first traces are not abysmally slow.
	 */
	UE_API void EnsureCollisionTreeIsBuilt();

	/** Sent when Level is being added to the world (used to add custom logic). */
	FAddLevelToWorldExtensionEvent& OnAddLevelToWorldExtension()
	{
		return AddLevelToWorldExtensionEvent;
	}

	/** Sent when Level is being removed from the world (used to add custom logic). */
	FRemoveLevelFromWorldExtensionEvent& OnRemoveLevelFromWorldExtension()
	{
		return RemoveLevelFromWorldExtensionEvent;
	}

#if WITH_EDITOR	
	/** Returns the SelectedLevelsChangedEvent member. */
	FOnSelectedLevelsChangedEvent& OnSelectedLevelsChanged() { return SelectedLevelsChangedEvent; }

	/**
	 * Flag a level as selected.
	 */
	UE_API void SelectLevel( ULevel* InLevel );
	
	/**
	 * Flag a level as not selected.
	 */
	UE_API void DeSelectLevel( ULevel* InLevel );

	/**
	 * Query whether or not a level is selected.
	 */
	UE_API bool IsLevelSelected( ULevel* InLevel ) const;

	/**
	 * Set the selected levels from the given array (Clears existing selections)
	 */
	UE_API void SetSelectedLevels( const TArray<class ULevel*>& InLevels );

	/**
	 * Return the number of levels in this world.
	 */
	UE_API int32 GetNumSelectedLevels() const;
	
	/**
	 * Return the selected level with the given index.
	 */
	UE_API ULevel* GetSelectedLevel( int32 InLevelIndex ) const;

	/**
	 * Return the list of selected levels in this world.
	 */
	UE_API TArray<TObjectPtr<class ULevel>>& GetSelectedLevels();

	/** Shrink level elements to their minimum size. */
	UE_API void ShrinkLevel();

	/** Add a listener for OnFeatureLevelChanged events */
	UE_API FDelegateHandle AddOnFeatureLevelChangedHandler(const FOnFeatureLevelChanged::FDelegate& InHandler);

	/** Remove a listener for OnFeatureLevelChanged events */
	UE_API void RemoveOnFeatureLevelChangedHandler(FDelegateHandle InHandle);
#endif // WITH_EDITOR
	
	/**
	 * Returns an iterator for the level list.
	 */
	UE_API FConstLevelIterator		GetLevelIterator() const;

	/**
	 * Return the level with the given index.
	 */
	UE_API ULevel* GetLevel( int32 InLevelIndex ) const;

	/**
	 * Does the level list contain the given level.
	 */
	UE_API bool ContainsLevel( ULevel* InLevel ) const;

	/**
	 * Return the number of levels in this world.
	 */
	UE_API int32 GetNumLevels() const;

	/**
	 * Return the list of levels in this world.
	 */
	UE_API const TArray<class ULevel*>& GetLevels() const;

	/**
	 * Add a level to the level list.
	 */
	UE_API bool AddLevel( ULevel* InLevel );
	
	/**
	 * Remove a level from the level list.
	 */
	UE_API bool RemoveLevel( ULevel* InLevel );

	/** Returns the FLevelCollection for the passed in streaming level, which could depend on configuration */
	UE_API FLevelCollection& FindOrAddCollectionForLevelStreaming(const ULevelStreaming* Level);

	/** Returns the FLevelCollection for the given InType. If one does not exist, it is created. */
	UE_API FLevelCollection& FindOrAddCollectionByType(const ELevelCollectionType InType);

	/** Returns the index of the first FLevelCollection of the given InType. If one does not exist, it is created and its index returned. */
	UE_API int32 FindOrAddCollectionByType_Index(const ELevelCollectionType InType);

	/** Returns the FLevelCollection for the given InType, or null if a collection of that type hasn't been created yet. */
	UE_API FLevelCollection* FindCollectionByType(const ELevelCollectionType InType);

	/** Returns the FLevelCollection for the given InType, or null if a collection of that type hasn't been created yet. */
	UE_API const FLevelCollection* FindCollectionByType(const ELevelCollectionType InType) const;

	/** Returns the index of the FLevelCollection with the given InType, or INDEX_NONE if a collection of that type hasn't been created yet. */
	UE_API int32 FindCollectionIndexByType(const ELevelCollectionType InType) const;
	
	/**
	 * Returns the level collection which currently has its context set on this world. May be null.
	 * If non-null, this implies that execution is currently within the scope of an FScopedLevelCollectionContextSwitch for this world.
	 */
	UE_API const FLevelCollection* GetActiveLevelCollection() const;

	/**
	 * Returns the index of the level collection which currently has its context set on this world. May be INDEX_NONE.
	 * If not INDEX_NONE, this implies that execution is currently within the scope of an FScopedLevelCollectionContextSwitch for this world.
	 */
	int32 GetActiveLevelCollectionIndex() const { return ActiveLevelCollectionIndex; }

	/** Sets the level collection and its context on this world. Should only be called by FScopedLevelCollectionContextSwitch. */
	UE_API void SetActiveLevelCollection(int32 LevelCollectionIndex);

	/** Returns a read-only reference to the list of level collections in this world. */
	const TArray<FLevelCollection>& GetLevelCollections() const { return LevelCollections; }

	/**
	 * Creates a new level collection of type DynamicDuplicatedLevels by duplicating the levels in DynamicSourceLevels.
	 * Should only be called by engine.
	 *
	 * @param MapName The name of the soure map, used as a parameter to UEngine::Experimental_ShouldPreDuplicateMap
	 */
	UE_API void DuplicateRequestedLevels(const FName MapName);

	/** Handle Exec/Console Commands related to the World */
	UE_API bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog );

private:
	/** Internal helper to release the Scene and clear the pointer. */
	void ReleaseScene();

	/** Internal version of CleanupWorld. */
	UE_API void CleanupWorldInternal(bool bSessionEnded, bool bCleanupResources, bool bWorldChanged);

	/** Utility function to handle Exec/Console Commands related to the Trace Tags */
	UE_API bool HandleTraceTagCommand( const TCHAR* Cmd, FOutputDevice& Ar );

	/** Utility function to handle Exec/Console Commands related to persistent debug lines */
	UE_API bool HandleFlushPersistentDebugLinesCommand( const TCHAR* Cmd, FOutputDevice& Ar );

	/** Utility function to handle Exec/Console Commands related to logging actor counts */
	UE_API bool HandleLogActorCountsCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );

	/** Utility function to handle Exec/Console Commands related to demo recording */
	UE_API bool HandleDemoRecordCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );

	/** Utility function to handle Exec/Console Commands related to playing a demo recording*/
	UE_API bool HandleDemoPlayCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );

	/** Utility function to handle Exec/Console Commands related to stopping demo playback */
	UE_API bool HandleDemoStopCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );

	/** Utility function to handle Exec/Console Command for scrubbing to a specific time */
	UE_API bool HandleDemoScrubCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

	/** Utility function to handle Exec/Console Command for pausing and unpausing a replay */
	UE_API bool HandleDemoPauseCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

	/** Utility function to handle Exec/Console Command for setting the speed of a replay */
	UE_API bool HandleDemoSpeedCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

	/** Utility function to handle Exec/Console Command for requesting a replay checkpoint */
	UE_API bool HandleDemoCheckpointCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

public:

	// Destroys the current demo net driver
	UE_API void DestroyDemoNetDriver();

	// Remove internal references to pending demo net driver when starting a replay, but do not destroy it
	UE_API void ClearDemoNetDriver();

	// Remove all internal references to this net driver, but do not destroy it. Called by the engine when destroying the driver.
	UE_API void ClearNetDriver(UNetDriver* Driver);

	/** Returns true if we are currently playing a replay */
	UE_API bool IsPlayingReplay() const;

	/** Returns true if we are currently recording a replay */
	UE_API bool IsRecordingReplay() const;

	// Start listening for connections.
	UE_API bool Listen( FURL& InURL );

	/** @return true if the world is in the paused state */
	UE_API bool IsPaused() const;

	/** @return true if the camera is in a moveable state (taking pausedness into account) */
	UE_API bool IsCameraMoveable() const;

	/**
	 * Wrapper for DestroyActor() that should be called in the editor.
	 *
	 * @param	bShouldModifyLevel		If true, Modify() the level before removing the actor.
	 */
	UE_API bool EditorDestroyActor( AActor* Actor, bool bShouldModifyLevel );

	/**
	 * Removes the actor from its level's actor list and generally cleans up the engine's internal state.
	 * What this function does not do, but is handled via garbage collection instead, is remove references
	 * to this actor from all other actors, and kill the actor's resources.  This function is set up so that
	 * no problems occur even if the actor is being destroyed inside its recursion stack.
	 *
	 * @param	ThisActor				Actor to remove.
	 * @param	bNetForce				[opt] Ignored unless called during play.  Default is false.
	 * @param	bShouldModifyLevel		[opt] If true, Modify() the level before removing the actor.  Default is true.
	 * @return							true if destroyed or already marked for destruction, false if actor couldn't be destroyed.
	 */
	UE_API bool DestroyActor( AActor* Actor, bool bNetForce=false, bool bShouldModifyLevel=true );

	/**
	 * Removes the passed in actor from the actor lists. Please note that the code actually doesn't physically remove the
	 * index but rather clears it so other indices are still valid and the actors array size doesn't change.
	 *
	 * @param	Actor					Actor to remove.
	 * @param	bShouldModifyLevel		If true, Modify() the level before removing the actor if in the editor.
	 */
	UE_API void RemoveActor( AActor* Actor, bool bShouldModifyLevel ) const;

	/**
	 * Spawn Actors with given transform and SpawnParameters
	 * 
	 * @param	Class					Class to Spawn
	 * @param	Location				Location To Spawn
	 * @param	Rotation				Rotation To Spawn
	 * @param	SpawnParameters			Spawn Parameters
	 *
	 * @return	Actor that just spawned
	 */
	UE_API AActor* SpawnActor( UClass* InClass, FVector const* Location=NULL, FRotator const* Rotation=NULL, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters() );
	/**
	 * Spawn Actors with given transform and SpawnParameters
	 * 
	 * @param	Class					Class to Spawn
	 * @param	Transform				World Transform to spawn on
	 * @param	SpawnParameters			Spawn Parameters
	 *
	 * @return	Actor that just spawned
	 */
	UE_API AActor* SpawnActor( UClass* Class, FTransform const* Transform, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters());

	/**
	 * Spawn Actors with given absolute transform (override root component transform) and SpawnParameters
	 * 
	 * @param	Class					Class to Spawn
	 * @param	AbsoluteTransform		World Transform to spawn on - without considering CDO's relative transform, thus Absolute
	 * @param	SpawnParameters			Spawn Parameters
	 *
	 * @return	Actor that just spawned
	 */
	UE_API AActor* SpawnActorAbsolute( UClass* Class, FTransform const& AbsoluteTransform, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters());

	/** Templated version of SpawnActor that allows you to specify a class type via the template type */
	template< class T >
	T* SpawnActor( const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters() )
	{
		return CastChecked<T>(SpawnActor(T::StaticClass(), NULL, NULL, SpawnParameters),ECastCheckedType::NullAllowed);
	}

	/** Templated version of SpawnActor that allows you to specify location and rotation in addition to class type via the template type */
	template< class T >
	T* SpawnActor( FVector const& Location, FRotator const& Rotation, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters() )
	{
		return CastChecked<T>(SpawnActor(T::StaticClass(), &Location, &Rotation, SpawnParameters),ECastCheckedType::NullAllowed);
	}
	
	/** Templated version of SpawnActor that allows you to specify the class type via parameter while the return type is a parent class of that type */
	template< class T >
	T* SpawnActor( UClass* Class, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters() )
	{
		return CastChecked<T>(SpawnActor(Class, NULL, NULL, SpawnParameters),ECastCheckedType::NullAllowed);
	}

	/** 
	 *  Templated version of SpawnActor that allows you to specify the rotation and location in addition
	 *  class type via parameter while the return type is a parent class of that type 
	 */
	template< class T >
	T* SpawnActor( UClass* Class, FVector const& Location, FRotator const& Rotation, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters() )
	{
		return CastChecked<T>(SpawnActor(Class, &Location, &Rotation, SpawnParameters),ECastCheckedType::NullAllowed);
	}
	/** 
	 *  Templated version of SpawnActor that allows you to specify whole Transform
	 *  class type via parameter while the return type is a parent class of that type 
	 */
	template< class T >
	T* SpawnActor(UClass* Class, FTransform const& Transform,const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters())
	{
		return CastChecked<T>(SpawnActor(Class, &Transform, SpawnParameters), ECastCheckedType::NullAllowed);
	}

	/** Templated version of SpawnActorAbsolute that allows you to specify absolute location and rotation in addition to class type via the template type */
	template< class T >
	T* SpawnActorAbsolute(FVector const& AbsoluteLocation, FRotator const& AbsoluteRotation, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters())
	{
		return CastChecked<T>(SpawnActorAbsolute(T::StaticClass(), FTransform(AbsoluteRotation, AbsoluteLocation), SpawnParameters), ECastCheckedType::NullAllowed);
	}

	/** 
	 *  Templated version of SpawnActorAbsolute that allows you to specify whole absolute Transform
	 *  class type via parameter while the return type is a parent class of that type 
	 */
	template< class T >
	T* SpawnActorAbsolute(UClass* Class, FTransform const& Transform,const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters())
	{
		return CastChecked<T>(SpawnActorAbsolute(Class, Transform, SpawnParameters), ECastCheckedType::NullAllowed);
	}

	/**
	 * Spawns given class and returns class T pointer, forcibly sets world transform (note this allows scale as well). WILL NOT run Construction Script of Blueprints 
	 * to give caller an opportunity to set parameters beforehand.  Caller is responsible for invoking construction
	 * manually by calling UGameplayStatics::FinishSpawningActor (see AActor::OnConstruction).
	 */
	template< class T >
	T* SpawnActorDeferred(
		UClass* Class,
		FTransform const& Transform,
		AActor* Owner = nullptr,
		APawn* Instigator = nullptr,
		ESpawnActorCollisionHandlingMethod CollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::Undefined,
		ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::MultiplyWithRoot
		)
	{
		if( Owner )
		{
			check(this==Owner->GetWorld());
		}
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = CollisionHandlingOverride;
		SpawnInfo.TransformScaleMethod = TransformScaleMethod;
		SpawnInfo.Owner = Owner;
		SpawnInfo.Instigator = Instigator;
		SpawnInfo.bDeferConstruction = true;
		return (Class != nullptr) ? Cast<T>(SpawnActor(Class, &Transform, SpawnInfo)) : nullptr;
	}

	/** 
	 * Returns the current Game Mode instance cast to the template type.
	 * This can only return a valid pointer on the server and may be null if the cast fails. Will always return null on a client.
	 */
	template< class T >
	T* GetAuthGameMode() const
	{
		return Cast<T>(AuthorityGameMode);
	}

	/**
	 * Returns the current Game Mode instance, which is always valid during gameplay on the server.
	 * This will only return a valid pointer on the server. Will always return null on a client.
	 */
	AGameModeBase* GetAuthGameMode() const { return AuthorityGameMode; }
	
	/** Returns the current GameState instance cast to the template type. */
	template< class T >
	T* GetGameState() const
	{
		return Cast<T>(GameState);
	}

	/** Returns the current GameState instance cast to the template type, asserting that it is of the correct type. */
	template<class T>
	T* GetGameStateChecked() const
	{
		return CastChecked<T>(GameState);
	}

	/** Returns the current GameState instance. */
	AGameStateBase* GetGameState() const { return GameState; }

	/** Sets the current GameState instance on this world and the game state's level collection. */
	UE_API void SetGameState(AGameStateBase* NewGameState);

	/** Copies GameState properties from the GameMode. */
	UE_API void CopyGameState(AGameModeBase* FromGameMode, AGameStateBase* FromGameState);

	DECLARE_EVENT_OneParam(UWorld, FOnGameStateSetEvent, AGameStateBase*);
	/** Called whenever the gamestate is set on the world. */
	FOnGameStateSetEvent GameStateSetEvent;


	/** Spawns a Brush Actor in the World */
	UE_API ABrush*	SpawnBrush();

	/** 
	 * Spawns a PlayerController and binds it to the passed in Player with the specified RemoteRole and options
	 * 
	 * @param Player - the Player to set on the PlayerController
	 * @param RemoteRole - the RemoteRole to set on the PlayerController
	 * @param URL - URL containing player options (name, etc)
	 * @param UniqueId - unique net ID of the player (may be zeroed if no online subsystem or not logged in, e.g. a local game or LAN match)
	 * @param Error (out) - if set, indicates that there was an error - usually is set to a property from which the calling code can look up the actual message
	 * @param InNetPlayerIndex (optional) - the NetPlayerIndex to set on the PlayerController
	 * @param InLocalPlayerIdentifier (optional) - the identifier of the local player on the client associated with this player controller.
	 * @return the PlayerController that was spawned (may fail and return NULL)
	 */
	UE_API APlayerController* SpawnPlayActor(class UPlayer* Player, ENetRole RemoteRole, const FURL& InURL, const FUniqueNetIdRepl& UniqueId, FString& Error, uint8 InNetPlayerIndex = 0, int32 InLocalPlayerIdentifier = 0);
	
	/**
	 * Try to find an acceptable non-colliding location to place TestActor as close to possible to PlaceLocation. Expects PlaceLocation to be a valid location inside the level.
	 * Returns true if a location without blocking collision is found, in which case PlaceLocation is overwritten with the new clear location.
	 * Returns false if no suitable location could be found, in which case PlaceLocation is unmodified.
	 */
	UE_API bool FindTeleportSpot( const AActor* TestActor, FVector& PlaceLocation, FRotator PlaceRotation );

	/** @Return true if Actor would encroach at TestLocation on something that blocks it.  Returns a ProposedAdjustment that might result in an unblocked TestLocation. */
	UE_API bool EncroachingBlockingGeometry( const AActor* TestActor, FVector TestLocation, FRotator TestRotation, FVector* ProposedAdjustment = NULL );

	/** Begin physics simulation */ 
	UE_API void StartPhysicsSim();

	/** Waits for the physics scene to be done processing */
	UE_API void FinishPhysicsSim();

	/** Spawns GameMode for the level. */
	UE_API bool SetGameMode(const FURL& InURL);

	/** 
	 * Initializes all actors and prepares them to start gameplay
	 * @param InURL commandline URL
	 * @param bResetTime (optional) whether the WorldSettings's TimeSeconds should be reset to zero
	 */
	UE_API void InitializeActorsForPlay(const FURL& InURL, bool bResetTime = true, FRegisterComponentContext* Context = nullptr);

	/** 
	 * Mark a world that was initialized for play as starting to tear down in preparation for cleanup.
	 * This will block the creation of new gameplay objects.
	 */
	UE_API void BeginTearingDown();

	/**
	 * Start gameplay. This will cause the game mode to transition to the correct state and call BeginPlay on all actors.
	 * If this is called on a world with no game mode, it will execute world callbacks but will not set BegunPlay to true.
	 * On networked clients, actor BeginPlay and SetBegunPlay are called from the game state replication.
	 */
	UE_API void BeginPlay();

	/**
	 * Tries to stop gameplay by sending EndPlay to all actors in the world.
	 * This will call BeginTearingDown if it has not been called yet.
	 * @param EndPlayReason The reason that play is ending
	 * @return true if this actually stopped play (BegunPlay will return false)
	 */
	UE_API bool EndPlay(EEndPlayReason::Type EndPlayReason);

	/** 
	 * Looks for a PlayerController that was being swapped by the given NetConnection and, if found, destroys it
	 * (because the swap is complete or the connection was closed)
	 * @param Connection - the connection that performed the swap
	 * @return whether a PC waiting for a swap was found
	 */
	UE_API bool DestroySwappedPC(UNetConnection* Connection);

	//~ Begin FNetworkNotify Interface
	UE_API virtual EAcceptConnection::Type NotifyAcceptingConnection() override;
	UE_API virtual void NotifyAcceptedConnection( class UNetConnection* Connection ) override;
	UE_API virtual bool NotifyAcceptingChannel( class UChannel* Channel ) override;
	UE_API virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch) override;
	//~ End FNetworkNotify Interface

	/**
	 * Log error and close connection for prelogin failures.
	 */
	UE_API bool PreLoginCheckError(UNetConnection* Connection, const FString& ErrorMsg);

	/**
	 * Check GameMode PreLogin results and welcome player if needed.
	 */
	UE_API void PreLoginComplete(const FString& ErrorMsg, TWeakObjectPtr<UNetConnection> WeakConnection);

	/**
	 * Check GameMode PreLogin results for split screen player joins.
	 */
	UE_API void PreLoginCompleteSplit(const FString& ErrorMsg, TWeakObjectPtr<UNetConnection> WeakConnection, FUniqueNetIdRepl SplitRequestUniqueIdRepl, FString SplitRequestURL);

	/** Welcome a new player joining this server. */
	UE_API void WelcomePlayer(UNetConnection* Connection);

	/**
	 * Used to get a net driver object.
	 * @return a pointer to the net driver or NULL if no driver is available.
	 */
	inline UNetDriver* GetNetDriver() const
	{
		return NetDriver;
	}

	/**
	 * Returns the net mode this world is running under.
	 * @see IsNetMode()
	 */
	ENetMode GetNetMode() const;

	/**
	* Test whether net mode is the given mode.
	* In optimized non-editor builds this can be more efficient than GetNetMode()
	* because it can check the static build flags without considering PIE.
	*/
	bool IsNetMode(ENetMode Mode) const;

private:

	/** Private version without inlining that does *not* check Dedicated server build flags (which should already have been done). */
	UE_API ENetMode InternalGetNetMode() const;

	/** Attempts to derive the net mode from URL */
	UE_API ENetMode AttemptDeriveFromURL() const;

	UE_API APhysicsVolume* InternalGetDefaultPhysicsVolume() const;

	/** Updates world's required streaming levels */
	UE_API void InternalUpdateStreamingState();

#if WITH_STATE_STREAM
	/** Returns state stream based on Id */
	UE_API void* InternalGetStateStream(uint32 Id);
#endif

#if WITH_EDITOR
public:
	void SetPlayInEditorInitialNetMode(ENetMode InNetMode)
	{
		PlayInEditorNetMode = InNetMode;

		// Disable audio playback on PIE dedicated server
		bAllowAudioPlayback = bAllowAudioPlayback && PlayInEditorNetMode != NM_DedicatedServer;
	}

private:
	/** In PIE, what Net Mode was this world started in? Fallback for not having a NetDriver */
	ENetMode PlayInEditorNetMode;
#endif

public:

	/**
	 * Sets the net driver to use for this world
	 * @param NewDriver the new net driver to use
	 */
	void SetNetDriver(UNetDriver* NewDriver)
	{
		NetDriver = NewDriver;
	}

	/**
	 * Returns true if the game net driver exists and is a client and the demo net driver exists and is a server.
	 */
	UE_API bool IsRecordingClientReplay() const;

	/**
	* Returns true if the demo net driver exists and is playing a client recorded replay.
	*/
	UE_API bool IsPlayingClientReplay() const;

	/**
	 * Sets the number of frames to delay Streaming Volume updating, 
	 * useful if you preload a bunch of levels but the camera hasn't caught up yet 
	 */
	void DelayStreamingVolumeUpdates(int32 InFrameDelay)
	{
		StreamingVolumeUpdateDelay = InFrameDelay;
	}

	/**
	 * Transfers the set of Kismet / Blueprint objects being debugged to the new world that are not already present, and updates blueprints accordingly
	 * @param	NewWorld	The new world to find equivalent objects in
	 */
	UE_API void TransferBlueprintDebugReferences(UWorld* NewWorld);

	/**
	 * Notifies the world of a blueprint debugging reference
	 * @param	Blueprint	The blueprint the reference is for
	 * @param	DebugObject The associated debugging object (may be NULL)
	 */
	UE_API void NotifyOfBlueprintDebuggingAssociation(class UBlueprint* Blueprint, UObject* DebugObject);

	/** Broadcasts that the number of levels has changed. */
	UE_API void BroadcastLevelsChanged();

	/** Returns the LevelsChangedEvent member. */
	FOnLevelsChangedEvent& OnLevelsChanged() { return LevelsChangedEvent; }

	/** Returns the AllLevelsChangedEvent member. */
	FOnAllLevelsChangedEvent& OnAllLevelsChanged() { return AllLevelsChangedEvent; }

	/** Returns the actor count. */
	UE_API int32 GetProgressDenominator() const;
	
	/** Returns the actor count. */
	UE_API int32 GetActorCount() const;
	
public:

	/**
	 * Finds the audio settings to use for a given view location, taking into account the world's default
	 * settings and the audio volumes in the world.
	 *
	 * @param	ViewLocation			Current view location.
	 * @param	OutReverbSettings		[out] Upon return, the reverb settings for a camera at ViewLocation.
	 * @param	OutInteriorSettings		[out] Upon return, the interior settings for a camera at ViewLocation.
	 * @return							If the settings came from an audio volume, the audio volume object is returned.
	 */
	UE_API class AAudioVolume* GetAudioSettings( const FVector& ViewLocation, struct FReverbSettings* OutReverbSettings, struct FInteriorSettings* OutInteriorSettings ) const;

	UE_API void SetAudioDevice(const FAudioDeviceHandle& InHandle);

	/**
	 * Get the audio device used by this world.
	 */
	UE_API FAudioDeviceHandle GetAudioDevice() const;

	/**
	* Returns the audio device associated with this world.
	* Lifecycle of the audio device is not guaranteed unless you used GetAudioDevice().
	*
	* @return Audio device to use with this world.
	*/
	UE_API class FAudioDevice* GetAudioDeviceRaw() const;

	/** Return the URL of this level on the local machine. */
	UE_API FString GetLocalURL() const;

	/** Returns whether script is executing within the editor. */
	UE_API bool IsPlayInEditor() const;

	/** Returns whether script is executing within a preview window */
	UE_API bool IsPlayInPreview() const;

	/** Returns whether script is executing within a mobile preview window */
	UE_API bool IsPlayInMobilePreview() const;

	/** Returns whether script is executing within a vulkan preview window */
	UE_DEPRECATED(5.7, "PlayInVulkanPreview is no longer supported, please use PlayInMobilePreview instead")
	bool IsPlayInVulkanPreview() const { return false; };

	/** Returns true if this world is any kind of game world (including PIE worlds) */
	UE_API bool IsGameWorld() const;

	/** Returns true if this world is any kind of editor world (including editor preview worlds) */
	UE_API bool IsEditorWorld() const;

	/** Returns true if this world is a preview game world (editor or game) */
	UE_API bool IsPreviewWorld() const;

	/** Returns true if this world should look at game hidden flags instead of editor hidden flags for the purposes of rendering */
	UE_API bool UsesGameHiddenFlags() const;

	// Return the URL of this level, which may possibly
	// exist on a remote machine.
	UE_API FString GetAddressURL() const;

	/**
	 * Called after GWorld has been set. Used to load, but not associate, all
	 * levels in the world in the Editor and at least create linkers in the game.
	 * Should only be called against GWorld::PersistentLevel's WorldSettings.
	 *
	 * @param bForce	If true, load the levels even is a commandlet
	 */
	UE_API void LoadSecondaryLevels(bool bForce = false, TSet<FName>* PackageNamesToSkip = nullptr);

	/** Utility for returning the ULevelStreaming object for a particular sub-level, specified by package name */
	UE_API ULevelStreaming* GetLevelStreamingForPackageName(FName PackageName);

#if WITH_EDITOR
	/** 
	 * Called when level property has changed
	 * It refreshes any streaming stuff
	 */
	UE_API void RefreshStreamingLevels();

	/**
	 * Called when a specific set of streaming levels need to be refreshed
	 * @param LevelsToRefresh A TArray<ULevelStreaming*> containing pointers to the levels to refresh
	 */
	UE_API void RefreshStreamingLevels( const TArray<class ULevelStreaming*>& InLevelsToRefresh );
		
private:
	bool bIsRefreshingStreamingLevels;

public:

	bool IsRefreshingStreamingLevels() const { return bIsRefreshingStreamingLevels; }

	UE_API void IssueEditorLoadWarnings();

#endif

	/**
	 * Jumps the server to new level.  If bAbsolute is true and we are using seamless traveling, we
	 * will do an absolute travel (URL will be flushed).
	 *
	 * @param URL the URL that we are traveling to
	 * @param bAbsolute whether we are using relative or absolute travel
	 * @param bShouldSkipGameNotify whether to notify the clients/game or not
	 */
	UE_API bool ServerTravel(const FString& InURL, bool bAbsolute = false, bool bShouldSkipGameNotify = false);

	/** seamlessly travels to the given URL by first loading the entry level in the background,
	 * switching to it, and then loading the specified level. Does not disrupt network communication or disconnect clients.
	 * You may need to implement GameModeBase::GetSeamlessTravelActorList(), PlayerController::GetSeamlessTravelActorList(),
	 * GameModeBase::PostSeamlessTravel(), and/or GameModeBase::HandleSeamlessTravelPlayer() to handle preserving any information
	 * that should be maintained (player teams, etc)
	 * This codepath is designed for worlds that use little or no level streaming and GameModes where the game state
	 * is reset/reloaded when transitioning. (like UT)
	 * @param URL - the URL to travel to; must be on the same server as the current URL
	 * @param bAbsolute (opt) - if true, URL is absolute, otherwise relative
	 */
	UE_API void SeamlessTravel(const FString& InURL, bool bAbsolute = false);

	/** @return whether we're currently in a seamless transition */
	UE_API bool IsInSeamlessTravel() const;

	/** this function allows pausing the seamless travel in the middle,
	 * right before it starts loading the destination (i.e. while in the transition level)
	 * this gives the opportunity to perform any other loading tasks before the final transition
	 * this function has no effect if we have already started loading the destination (you will get a log warning if this is the case)
	 * @param bNowPaused - whether the transition should now be paused
	 */
	UE_API void SetSeamlessTravelMidpointPause(bool bNowPaused);

	/** @return the current detail mode, like EDetailMode but can be outside of the range */
	UE_API int32 GetDetailMode() const;

	/** asynchronously loads the given levels in preparation for a streaming map transition.
	 * This codepath is designed for worlds that heavily use level streaming and GameModes where the game state should
	 * be preserved through a transition.
	 * @param LevelNames the names of the level packages to load. LevelNames[0] will be the new persistent (primary) level
	 */
	UE_API void PrepareMapChange(const TArray<FName>& LevelNames);

	/** @return true if there's a map change currently in progress */
	UE_API bool IsPreparingMapChange() const;

	/** @return true if there is a map change being prepared, returns whether that change is ready to be committed, otherwise false */
	UE_API bool IsMapChangeReady() const;

	/** cancels pending map change (@note: we can't cancel pending async loads, so this won't immediately free the memory) */
	UE_API void CancelPendingMapChange();

	/** actually performs the map transition prepared by PrepareMapChange()
	 * it happens in the next tick to avoid GC issues
	 * if a map change is being prepared but isn't ready yet, the transition code will block until it is
	 * wait until IsMapChangeReady() returns true if this is undesired behavior
	 */
	UE_API void CommitMapChange();

	/**
	 * Sets NumLightingUnbuiltObjects to the specified value.  Marks the worldsettings package dirty if the value changed.
	 * @param	InNumLightingUnbuiltObjects			The new value.
	 */
	UE_API void SetMapNeedsLightingFullyRebuilt(int32 InNumLightingUnbuiltObjects, int32 InNumUnbuiltReflectionCaptures);

	/** Returns TimerManager instance for this world. */
	UE_API FTimerManager& GetTimerManager() const;

	/**
	 * Returns LatentActionManager instance, preferring the one allocated by the game instance if a game instance is associated with this.
	 *
	 * This pattern is a little bit of a kludge to allow UWorld clients (for instance, preview world in the Blueprint Editor
 	 * to not worry about replacing features from GameInstance. Alternatively we could mandate that they implement a game instance
	 * for their scene.
	 */
	UE_API FLatentActionManager& GetLatentActionManager();

	/**
	 * Get a Subsystem of specified type
	 */
	UWorldSubsystem* GetSubsystemBase(TSubclassOf<UWorldSubsystem> SubsystemClass) const
	{
		return SubsystemCollection.GetSubsystem<UWorldSubsystem>(SubsystemClass);
	}

	/**
	 * Get a Subsystem of specified type
	 */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem() const
	{
		return SubsystemCollection.GetSubsystem<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Get a Subsystem of specified type, checked
	 */
	template <typename TSubsystemClass>
	TNotNull<TSubsystemClass*> GetSubsystemChecked() const
	{
		return GetSubsystem<TSubsystemClass>();
	}

	/**
	 * Get a Subsystem of specified type from the provided GameInstance
	 * returns nullptr if the Subsystem cannot be found or the GameInstance is null
	 */
	template <typename TSubsystemClass>
	static inline TSubsystemClass* GetSubsystem(const UWorld* World)
	{
		if (World)
		{
			return World->GetSubsystem<TSubsystemClass>();
		}
		return nullptr;
	}

	/**
	 * Check if world has a subsystem of the specified type
	 */
	template <typename TSubsystemClass>
	bool HasSubsystem() const
	{
		return GetSubsystem<TSubsystemClass>() != nullptr;
	}

	/**
	 * Check if world has a subsystem of the specified type from the provided GameInstance
	 * returns false if the Subsystem cannot be found or the GameInstance is null
	 */
	template <typename TSubsystemClass>
	static inline bool HasSubsystem(const UWorld* World)
	{
		return GetSubsystem<TSubsystemClass>(World) != nullptr;
	}

	/**
	 * Get all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy() const
	{
		return SubsystemCollection.GetSubsystemArrayCopy<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Performs an operation on all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	void ForEachSubsystem(TFunctionRef<void(TSubsystemClass*)> Operation) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, UWorldSubsystem>::IsDerived, "TSubsystemClass must be derived from UWorldSubsystem");
		return SubsystemCollection.ForEachSubsystem([Operation=MoveTemp(Operation)](UWorldSubsystem* Subsystem){
			Operation(CastChecked<TSubsystemClass>(Subsystem));
		}, TSubsystemClass::StaticClass());
	}

	/** Sets the owning game instance for this world */
	inline void SetGameInstance(UGameInstance* NewGI)
	{
		OwningGameInstance = NewGI;
	}
	/** Returns the owning game instance for this world */
	inline UGameInstance* GetGameInstance() const
	{
		return OwningGameInstance;
	}

	/** Returns the OwningGameInstance cast to the template type. */
	template<class T>
	T* GetGameInstance() const
	{
		return Cast<T>(OwningGameInstance);
	}

	/** Returns the OwningGameInstance cast to the template type, asserting that it is of the correct type. */
	template<class T>
	T* GetGameInstanceChecked() const
	{
		return CastChecked<T>(OwningGameInstance);
	}

	/** Retrieves information whether all navigation with this world has been rebuilt */
	UE_API bool IsNavigationRebuilt() const;

	/** Request to translate world origin to specified position on next tick */
	UE_API void RequestNewWorldOrigin(FIntVector InNewOriginLocation);
	
	/** Translate world origin to specified position  */
	UE_API bool SetNewWorldOrigin(FIntVector InNewOriginLocation);

	/** Sets world origin at specified position and stream-in all relevant levels */
	UE_API void NavigateTo(FIntVector InLocation);

	/** Updates all physics constraint actor joint locations.  */
	UE_API void UpdateConstraintActors();

	/** Gets all LightMaps and ShadowMaps associated with this world. Specify the level or leave null for persistent */
	UE_API void GetLightMapsAndShadowMaps(ULevel* Level, TArray<UTexture2D*>& OutLightMapsAndShadowMaps, bool bForceLazyLoad = true);

public:
	/** Rename this world such that it has the prefix on names for the given PIE Instance ID */
	UE_API void RenameToPIEWorld(int32 PIEInstanceID);

	/** Given a level script actor, modify the string such that it points to the correct instance of the object. For replays. */
	UE_API bool RemapCompiledScriptActor(FString& Str) const;

	/** Returns true if world package is instanced. */
	UE_API bool IsInstanced() const;

	/** 
	 * If World Package is instanced return a mapping that can be used to fixup SoftObjectPaths for this world 
	 *
	 * returns true if world package is instanced and needs remapping.
	 */
	UE_API bool GetSoftObjectPathMapping(FString& OutSourceWorldPath, FString& OutRemappedWorldPath) const;

	/** Given a PackageName and a PIE Instance ID return the name of that Package when being run as a PIE world */
	static UE_API FString ConvertToPIEPackageName(const FString& PackageName, int32 PIEInstanceID);

	/** Given a PackageName and a prefix type, get back to the original package name (i.e. the saved map name) */
	static UE_API FString StripPIEPrefixFromPackageName(const FString& PackageName, const FString& Prefix);

	/** Return the prefix for PIE packages given a PIE Instance ID */
	static UE_API FString BuildPIEPackagePrefix(int32 PIEInstanceID);

	/** Duplicate the editor world to create the PIE world. */
	static UE_API UWorld* GetDuplicatedWorldForPIE(UWorld* InWorld, UPackage* InPIEackage, int32 PIEInstanceID);

	/** Given a loaded editor UWorld, duplicate it for play in editor purposes with OwningWorld as the world with the persistent level. */
	static UE_API UWorld* DuplicateWorldForPIE(const FString& PackageName, UWorld* OwningWorld);

	/** Given a string, return that string with any PIE prefix removed. Optionally returns the PIE Instance ID. */
	static UE_API FString RemovePIEPrefix(const FString &Source, int32* OutPIEInstanceID = nullptr);

	/** Given a package, locate the UWorld contained within if one exists */
	static UE_API UWorld* FindWorldInPackage(UPackage* Package);

	/** Given a package, return if package contains a UWorld or an external world object */
	static UE_API bool IsWorldOrWorldExternalPackage(UPackage* Package);

	/** If the specified package contains a redirector to a UWorld, that UWorld is returned. Otherwise, nullptr is returned. */
	static UE_API UWorld* FollowWorldRedirectorInPackage(UPackage* Package, UObjectRedirector** OptionalOutRedirector = nullptr);

	inline FWorldPSCPool& GetPSCPool() { return PSCPool; }

	private:

	UPROPERTY()
	FWorldPSCPool PSCPool;

	//PSC Pooling END
	FWorldSubsystemCollection SubsystemCollection;

	TOptional<UE::FTimeout> AddToWorldTimeout;
	TOptional<UE::FTimeout> RemoveFromWorldTimeout;
};

/** Global UWorld pointer. Use of this pointer should be avoided whenever possible. */
extern ENGINE_API class UWorldProxy GWorld;

/** World delegates */
class FWorldDelegates
{
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FWorldInitializationEvent, UWorld* /*World*/, const UWorld::InitializationValues /*IVS*/);
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FWorldCleanupEvent, UWorld* /*World*/, bool /*bSessionEnded*/, bool /*bCleanupResources*/);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FWorldEvent, UWorld* /*World*/);

	/**
	 * Post UWorld duplicate event.
	 *
	 * Sometimes there is a need to duplicate additional element after
	 * duplicating UWorld. If you do this using this event you need also fill
	 * ReplacementMap and ObjectsToFixReferences in order to properly fix
	 * duplicated objects references.
	 */
	typedef TMap<UObject*, UObject*> FReplacementMap; // Typedef needed so the macro below can properly digest comma in template parameters.
	DECLARE_MULTICAST_DELEGATE_FourParams(FWorldPostDuplicateEvent, UWorld* /*World*/, bool /*bDuplicateForPIE*/, FReplacementMap& /*ReplacementMap*/, TArray<UObject*>& /*ObjectsToFixReferences*/);

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_FiveParams(FWorldPreRenameEvent, UWorld* /*World*/, const TCHAR* /*InName*/, UObject* /*NewOuter*/, ERenameFlags /*Flags*/, bool& /*bShouldFailRename*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPostRenameEvent, UWorld*);
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FWorldCurrentLevelChangedEvent, ULevel* /*NewLevel*/, ULevel* /*OldLevel*/, UWorld* /*World*/);
#endif // WITH_EDITOR

	// Delegate type for level change events
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLevelChanged, ULevel*, UWorld*);

	// delegate for generating world asset registry tags so project/game scope can add additional tags for filtering levels in their UI, etc
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FWorldGetAssetTagsWithContext, const UWorld*, FAssetRegistryTagsContext);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FWorldGetAssetTags, const UWorld*, TArray<UObject::FAssetRegistryTag>&);

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWorldTickStart, UWorld*, ELevelTick, float);
	static UE_API FOnWorldTickStart OnWorldTickStart;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWorldTickEnd, UWorld*, ELevelTick, float);
	static UE_API FOnWorldTickEnd OnWorldTickEnd;

	// Delegate called before actors are ticked for each world. Delta seconds is already dilated and clamped.
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWorldPreActorTick, UWorld* /*World*/, ELevelTick/**Tick Type*/, float/**Delta Seconds*/);
	static UE_API FOnWorldPreActorTick OnWorldPreActorTick;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWorldPostActorTick, UWorld* /*World*/, ELevelTick/**Tick Type*/, float/**Delta Seconds*/);
	static UE_API FOnWorldPostActorTick OnWorldPostActorTick;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPreSendAllEndOfFrameUpdates, UWorld* /*World*/);
	static UE_API FOnWorldPreSendAllEndOfFrameUpdates OnWorldPreSendAllEndOfFrameUpdates;

	// Callback for world creation
	static UE_API FWorldEvent OnPostWorldCreation;
	
	// Callback for world initialization (pre)
	static UE_API FWorldInitializationEvent OnPreWorldInitialization;
	
	// Callback for world initialization (post)
	static UE_API FWorldInitializationEvent OnPostWorldInitialization;

#if WITH_EDITOR
	// Callback for world rename event (pre)
	static UE_API FWorldPreRenameEvent OnPreWorldRename;

	// Callback for world rename event (post)
	static UE_API FWorldPostRenameEvent OnPostWorldRename;

	static UE_API FWorldCurrentLevelChangedEvent OnCurrentLevelChanged;

	// PIE has started
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPIEStarted, UGameInstance*);
	static UE_API FOnWorldPIEStarted OnPIEStarted;

	// PIE is ready
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPIEReady, UGameInstance*);
	static UE_API FOnWorldPIEStarted OnPIEReady;

	// PIE map is created
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPIEMapCreated, UGameInstance*);
	static UE_API FOnWorldPIEMapCreated OnPIEMapCreated;

	// PIE map is created
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPIEMapReady, UGameInstance*);
	static UE_API FOnWorldPIEMapReady OnPIEMapReady;

	// PIE has ended
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldPIEEnded, UGameInstance*);
	static UE_API FOnWorldPIEEnded OnPIEEnded;

	// Callback to add references when Serialize is called during SavePackage
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FWorldCollectSaveReferencesEvent, UWorld*, FArchive&);
	static UE_API FWorldCollectSaveReferencesEvent OnCollectSaveReferences;

	static UE_API FWorldEvent OnPreRecreateScene;
	static UE_API FWorldEvent OnPostRecreateScene;
#endif // WITH_EDITOR

	// Post duplication event.
	static UE_API FWorldPostDuplicateEvent OnPostDuplicate;

	// Callback for world cleanup start
	static UE_API FWorldCleanupEvent OnWorldCleanup;

	// Callback for world cleanup end
	static UE_API FWorldCleanupEvent OnPostWorldCleanup;

	// Callback for world destruction (only called for initialized worlds)
	static UE_API FWorldEvent OnPreWorldFinishDestroy;

	// Sent when a ULevel is added to the world via UWorld::AddToWorld
	static UE_API FOnLevelChanged			LevelAddedToWorld;

	// Sent before a ULevel is removed from the world via UWorld::RemoveFromWorld or 
	// LoadMap (a NULL object means the LoadMap case, because all levels will be 
	// removed from the world without a RemoveFromWorld call for each)
	static UE_API FOnLevelChanged			PreLevelRemovedFromWorld;

	// Sent when a ULevel is removed from the world via UWorld::RemoveFromWorld or 
	// LoadMap (a NULL object means the LoadMap case, because all levels will be 
	// removed from the world without a RemoveFromWorld call for each)
	static UE_API FOnLevelChanged			LevelRemovedFromWorld;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FLevelComponentsEvent, UWorld*, ULevel*);

	/** Sent when level components are all unregistered (ULevel::ClearLevelComponents). */
	static UE_API FLevelComponentsEvent	LevelComponentsCleared;

	/** Sent when level components are all updated (ULevel::UpdateLevelComponents). */
	static UE_API FLevelComponentsEvent	LevelComponentsUpdated;

	// Called after offset was applied to a level
	DECLARE_MULTICAST_DELEGATE_FourParams(FLevelOffsetEvent, ULevel*,  UWorld*, const FVector&, bool);
	static UE_API FLevelOffsetEvent		PostApplyLevelOffset;

	// Called after transform is applied to level
	DECLARE_MULTICAST_DELEGATE_TwoParams(FLevelTransformEvent, ULevel*, const FTransform&);
	static UE_API FLevelTransformEvent		PostApplyLevelTransform;

	// called by UWorld::GetAssetRegistryTags()
	static UE_API FWorldGetAssetTagsWithContext GetAssetTagsWithContext;

#if WITH_EDITOR
	// Delegate called when levelscript actions need refreshing
	DECLARE_MULTICAST_DELEGATE_OneParam(FRefreshLevelScriptActionsEvent, UWorld*);

	// Called when changes in the levels require blueprint actions to be refreshed.
	static UE_API FRefreshLevelScriptActionsEvent RefreshLevelScriptActions;
#endif
	
	// Global Callback after actors have been initialized (on any world)
	static UE_API UWorld::FOnWorldInitializedActors OnWorldInitializedActors;

	// Global Callback when an initialized world begins to tear down before cleanup
	static UE_API FWorldEvent OnWorldBeginTearDown;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSeamlessTravelStart, UWorld*, const FString&);
	static UE_API FOnSeamlessTravelStart OnSeamlessTravelStart;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSeamlessTravelTransition, UWorld*);
	static UE_API FOnSeamlessTravelTransition OnSeamlessTravelTransition;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNetDriverCreated, UWorld*, UNetDriver*);
	static UE_API FOnNetDriverCreated OnNetDriverCreated;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCopyWorldData, UWorld*, UWorld*);
	static UE_API FOnCopyWorldData OnCopyWorldData;

	DECLARE_MULTICAST_DELEGATE_OneParam(FGameInstanceEvent, UGameInstance* /*GameInstance*/);
	static UE_API FGameInstanceEvent OnStartGameInstance;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FGameInstanceWorldChangedEvent, UGameInstance* /*GameInstance*/, UWorld* /*OldWorld*/, UWorld* /*NewWorld*/);
	static UE_API FGameInstanceWorldChangedEvent OnGameInstanceWorldChanged;

private:
	FWorldDelegates() {}
};

/** Helper struct to allow ULevelStreaming to update its World on how many streaming levels are being loaded */
struct FWorldNotifyStreamingLevelLoading
{
private:
	static void Started(UWorld* World)
	{
		++World->NumStreamingLevelsBeingLoaded;
	}

	static void Finished(UWorld* World)
	{
		if (ensure(World->NumStreamingLevelsBeingLoaded > 0))
		{
			--World->NumStreamingLevelsBeingLoaded;
		}
	}

	friend ULevelStreaming;
};

//////////////////////////////////////////////////////////////////////////
// UWorld inlines:

inline double UWorld::GetTimeSeconds() const
{
	return TimeSeconds;
}

inline double UWorld::GetUnpausedTimeSeconds() const
{
	return UnpausedTimeSeconds;
}

inline double UWorld::GetRealTimeSeconds() const
{
	checkSlow(!IsInActualRenderingThread());
	return RealTimeSeconds;
}

inline double UWorld::GetAudioTimeSeconds() const
{
	return AudioTimeSeconds;
}

inline float UWorld::GetDeltaSeconds() const
{
	return DeltaTimeSeconds;
}

inline FGameTime UWorld::GetTime() const
{
	return FGameTime::CreateDilated(
		RealTimeSeconds, DeltaRealTimeSeconds,
		TimeSeconds, DeltaTimeSeconds);
}

inline double UWorld::TimeSince(double Time) const
{
	return GetTimeSeconds() - Time;
}

inline FConstPhysicsVolumeIterator UWorld::GetNonDefaultPhysicsVolumeIterator() const
{
	auto Result = NonDefaultPhysicsVolumeList.CreateConstIterator();
	return (const FConstPhysicsVolumeIterator&)Result;
}

inline int32 UWorld::GetNonDefaultPhysicsVolumeCount() const
{
	return NonDefaultPhysicsVolumeList.Num();
}

inline bool UWorld::ComponentOverlapMulti(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FRotator& Rot, const FComponentQueryParams& Params, const FCollisionObjectQueryParams& ObjectQueryParams) const
{
	// Pass through to FQuat version.
	return ComponentOverlapMulti(OutOverlaps, PrimComp, Pos, Rot.Quaternion(), Params, ObjectQueryParams);
}

inline bool UWorld::ComponentOverlapMultiByChannel(TArray<struct FOverlapResult>& OutOverlaps, const class UPrimitiveComponent* PrimComp, const FVector& Pos, const FRotator& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params /* = FComponentQueryParams::DefaultComponentQueryParams */, const FCollisionObjectQueryParams& ObjectQueryParams/* =FCollisionObjectQueryParams::DefaultObjectQueryParam */) const
{
	// Pass through to FQuat version.
	return ComponentOverlapMultiByChannel(OutOverlaps, PrimComp, Pos, Rot.Quaternion(), TraceChannel, Params, ObjectQueryParams);
}

inline bool UWorld::ComponentSweepMulti(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FRotator& Rot, const FComponentQueryParams& Params) const
{
	// Pass through to FQuat version.
	return ComponentSweepMulti(OutHits, PrimComp, Start, End, Rot.Quaternion(), Params);
}

inline bool UWorld::ComponentSweepMultiByChannel(TArray<struct FHitResult>& OutHits, class UPrimitiveComponent* PrimComp, const FVector& Start, const FVector& End, const FRotator& Rot, ECollisionChannel TraceChannel, const FComponentQueryParams& Params) const
{
	// Pass through to FQuat version.
	return ComponentSweepMultiByChannel(OutHits, PrimComp, Start, End, Rot.Quaternion(), TraceChannel, Params);
}

inline ENetMode UWorld::GetNetMode() const
{
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (IsRunningDedicatedServer())
	{
		return NM_DedicatedServer;
	}

	return InternalGetNetMode();
}

inline bool UWorld::IsNetMode(ENetMode Mode) const
{
#if UE_EDITOR
	// Editor builds are special because of PIE, which can run a dedicated server without the app running with -server.
	return GetNetMode() == Mode;
#else
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (Mode == NM_DedicatedServer)
	{
		return IsRunningDedicatedServer();
	}
	else
	{
		return !IsRunningDedicatedServer() && (InternalGetNetMode() == Mode);
	}
#endif
}

FString ENGINE_API ToString(ENetMode NetMode);

#undef UE_API
