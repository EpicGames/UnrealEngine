// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/WorldCompositionUtility.h"
#include "WorldComposition.generated.h"

class ULevel;
class ULevelStreaming;

/**
 * Helper structure which holds information about level package which participates in world composition
 */
struct FWorldCompositionTile
{
	FWorldCompositionTile()
		: StreamingLevelStateChangeTime(0.0)
	{
	}
	
	// Long package name
	FName					PackageName;
	// Found LOD levels since last rescan
	TArray<FName>			LODPackageNames;
	// Tile information
	FWorldTileInfo			Info;
	// Timestamp when we have changed streaming level state
	double					StreamingLevelStateChangeTime;

	friend FArchive& operator<<( FArchive& Ar, FWorldCompositionTile& D )
	{
		Ar << D.PackageName << D.Info << D.LODPackageNames;
		return Ar;
	}
						
	/** Matcher */
	struct FPackageNameMatcher
	{
		FPackageNameMatcher( const FName& InPackageName )
			: PackageName( InPackageName )
		{
		}

		bool Matches( const FWorldCompositionTile& Candidate ) const
		{
			return Candidate.PackageName == PackageName;
		}

		const FName& PackageName;
	};
};

/**
 * Helper structure which holds results of distance queries to a world composition
 */
struct FDistanceVisibleLevel
{
	int32				TileIdx;	
	ULevelStreaming*	StreamingLevel;
	int32				LODIndex; 
};

/**
 * WorldComposition represents world structure:
 *	- Holds list of all level packages participating in this world and theirs base parameters (bounding boxes, offset from origin)
 *	- Holds list of streaming level objects to stream in and out based on distance from current view point
 *  - Handles properly levels repositioning during level loading and saving
 */
UCLASS(config=Engine, MinimalAPI)
class UWorldComposition : public UObject
{
	GENERATED_UCLASS_BODY()

	typedef TArray<FWorldCompositionTile> FTilesList;

	/** Adds or removes level streaming objects to world based on distance settings from players current view */
	ENGINE_API void UpdateStreamingState();
	
	/** Adds or removes level streaming objects to world based on distance settings from current view point */
	ENGINE_API void UpdateStreamingState(const FVector& InLocation);
	ENGINE_API void UpdateStreamingState(const FVector* InLocation, int32 Num);
	ENGINE_API void UpdateStreamingStateCinematic(const FVector* InLocation, int32 Num);

#if WITH_EDITOR
	/** Simulates streaming in editor world, only visibility, no loading/unloading, no LOD sub-levels 
	 *  @returns Whether streaming levels state was updated by this call
	 */
	ENGINE_API bool UpdateEditorStreamingState(const FVector& InLocation);

	ENGINE_API TArray<FWorldTileLayer> GetDistanceDependentLayers() const;
#endif// WITH_EDITOR

	/**
	 * Evaluates current world origin location against provided view location
	 * Issues request for world origin rebasing in case location is far enough from current origin
	 */
	ENGINE_API void EvaluateWorldOriginLocation(const FVector& ViewLocation);

	/** Returns currently visible and hidden levels based on distance based streaming */
	ENGINE_API void GetDistanceVisibleLevels(const FVector& InLocation, TArray<FDistanceVisibleLevel>& OutVisibleLevels, TArray<FDistanceVisibleLevel>& OutHiddenLevels) const;
	ENGINE_API void GetDistanceVisibleLevels(const FVector* InLocations, int32 Num, TArray<FDistanceVisibleLevel>& OutVisibleLevels, TArray<FDistanceVisibleLevel>& OutHiddenLevels) const;

	/** @returns Whether specified streaming level is distance dependent */
	ENGINE_API bool IsDistanceDependentLevel(FName PackageName) const;

	/** @returns Currently opened world composition root folder (long PackageName)*/
	ENGINE_API FString GetWorldRoot() const;
	
	/** @returns Currently managed world obejct */
	ENGINE_API UWorld* GetWorld() const override final;
	
	/** Handles level OnPostLoad event*/
	static ENGINE_API void OnLevelPostLoad(ULevel* InLevel);

	/** Handles level just before it going to be saved to disk */
	ENGINE_API void OnLevelPreSave(ULevel* InLevel);
	
	/** Handles level just after it was saved to disk */
	ENGINE_API void OnLevelPostSave(ULevel* InLevel);
	
	/** Handles level is being added to world */
	ENGINE_API void OnLevelAddedToWorld(ULevel* InLevel);

	/** Handles level is being removed from the world */
	ENGINE_API void OnLevelRemovedFromWorld(ULevel* InLevel);

	/** @returns Level offset from current origin, with respect to parent levels */
	ENGINE_API FIntVector GetLevelOffset(ULevel* InLevel) const;

	/** @returns Level bounding box in current shifted space */
	ENGINE_API FBox GetLevelBounds(ULevel* InLevel) const;

	/** Scans world root folder for relevant packages and initializes world composition structures */
	ENGINE_API void Rescan(bool bForce = false);

	/**  */
	ENGINE_API void ReinitializeForPIE();

	/** @returns Whether specified tile package name is managed by world composition */
	ENGINE_API bool DoesTileExists(const FName& TilePackageName) const;

	/** @returns Tiles list in a world composition */
	ENGINE_API FTilesList& GetTilesList();

#if WITH_EDITOR
	/** @returns FWorldTileInfo associated with specified package */
	ENGINE_API FWorldTileInfo GetTileInfo(const FName& InPackageName) const;
	
	/** Notification from World browser about changes in tile info structure */
	ENGINE_API void OnTileInfoUpdated(const FName& InPackageName, const FWorldTileInfo& InInfo);

	/** Restores dirty tiles information after world composition being rescanned */
	ENGINE_API void RestoreDirtyTilesInfo(const FTilesList& TilesPrevState);
	
	/** Collect tiles package names to cook  */
	ENGINE_API void CollectTilesToCook(TArray<FString>& PackageNames);

	// Event to enable/disable world composition in the world
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FEnableWorldCompositionEvent, UWorld*, bool);
	static ENGINE_API FEnableWorldCompositionEvent EnableWorldCompositionEvent;
	
	// Event when world composition was successfully enabled/disabled in the world
	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldCompositionChangedEvent, UWorld*);
	static ENGINE_API FWorldCompositionChangedEvent WorldCompositionChangedEvent;

#endif //WITH_EDITOR

private:
	// UObject interface
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostLoad() override;
	// UObject interface

	/** Populate streaming level objects using tiles information */
	void PopulateStreamingLevels();

	/** Calculates tiles absolute positions based on relative positions */
	void CaclulateTilesAbsolutePositions();

	/** Resets world composition structures */
	void Reset();
	
	/** @returns  Streaming level object for corresponding FWorldCompositionTile */
	ULevelStreaming* CreateStreamingLevel(const FWorldCompositionTile& Info) const;
		
	/** Fixups internal structures for PIE mode */
	void FixupForPIE(int32 PIEInstanceID);

	/**
	 * Finds tile by package name 
	 * @return Pointer to a found tile 
	 */
	int32 FindTileIndexByName(const FName& InPackageName) const;
	FWorldCompositionTile* FindTileByName(const FName& InPackageName) const;
	
	/** @returns Whether specified streaming level is distance dependent */
	bool IsDistanceDependentLevel(int32 TileIdx) const;

	/** Attempts to set new streaming state for a particular tile, could be rejected if state change on 'cooldown' */
	bool CommitTileStreamingState(UWorld* PersistentWorld, int32 TileIdx, bool bShouldBeLoaded, bool bShouldBeVisible, bool bShouldBlock, int32 LODIdx);

public:
#if WITH_EDITOR
	UE_DEPRECATED(4.26, "No longer used; use bTemporarilyDisableOriginTracking instead.")
	bool						bTemporallyDisableOriginTracking;

	// Hack for a World Browser to be able to temporarily show hidden levels 
	// regardless of current world origin and without offsetting them temporarily 
	bool						bTemporarilyDisableOriginTracking;
#endif //WITH_EDITOR

private:
#if WITH_EDITOR
	bool						bRescanned = false;
#endif // WITH_EDITOR

	// Path to current world composition (long PackageName)
	FString						WorldRoot;
	
	// List of all tiles participating in the world composition
	FTilesList					Tiles;

public:
	// Streaming level objects for each tile
	UPROPERTY(transient)
	TArray<TObjectPtr<ULevelStreaming>>	TilesStreaming;

	// Time threshold between tile streaming state changes
	UPROPERTY(config)
	double						TilesStreamingTimeThreshold;

	// Whether all distance dependent tiles should be loaded and visible during cinematic
	UPROPERTY(config)
	bool						bLoadAllTilesDuringCinematic;

	// Whether to rebase origin in 3D space, otherwise only on XY plane
	UPROPERTY(config)
	bool						bRebaseOriginIn3DSpace;

#if WITH_EDITORONLY_DATA
	// Whether all tiles locations are locked
	UPROPERTY()
	bool						bLockTilesLocation;
#endif

	// Maximum distance to current view point where we should initiate origin rebasing
	UPROPERTY(config)
	float						RebaseOriginDistance;
};
