// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"

/** Struct containing a collection of optional parameters for initialization of a World. */
struct FWorldInitializationValues
{
	FWorldInitializationValues()
		: bInitializeScenes(true)
		, bAllowAudioPlayback(true)
		, bRequiresHitProxies(true)
		, bCreatePhysicsScene(true)
		, bCreateNavigation(true)
		, bCreateAISystem(true)
		, bShouldSimulatePhysics(true)
		, bEnableTraceCollision(false)
		, bForceUseMovementComponentInNonGameWorld(false)
		, bTransactional(true)
		, bCreateFXSystem(true)
		, bCreateWorldPartition(false)
		, bEnableWorldPartitionStreaming(true)
		, bAllowLumenPrimitiveTrackingInPreviewWorld(false)
	{
	}

	/** Should the scenes (physics, rendering) be created. */
	uint32 bInitializeScenes:1;

	/** Are sounds allowed to be generated from this world. */
	uint32 bAllowAudioPlayback:1;

	/** Should the render scene create hit proxies. */
	uint32 bRequiresHitProxies:1;

	/** Should the physics scene be created. bInitializeScenes must be true for this to be considered. */
	uint32 bCreatePhysicsScene:1;

	/** Should the navigation system be created for this world. */
	uint32 bCreateNavigation:1;

	/** Should the AI system be created for this world. */
	uint32 bCreateAISystem:1;

	/** Should physics be simulated in this world. */
	uint32 bShouldSimulatePhysics:1;

	/** Are collision trace calls valid within this world. */
	uint32 bEnableTraceCollision:1;

	/** Special flag to enable movement component in non game worlds (see UMovementComponent::OnRegister) */
	uint32 bForceUseMovementComponentInNonGameWorld:1;

	/** Should actions performed to objects in this world be saved to the transaction buffer. */
	uint32 bTransactional:1;

	/** Should the FX system be created for this world. */
	uint32 bCreateFXSystem:1;

	/** Should the world be partitioned */
	uint32 bCreateWorldPartition:1;

	/** If bCreateWorldPartition is set to true, this flag will init the streaming mode for the WorldPartition object (default to true to preserve previous behavior) */
	uint32 bEnableWorldPartitionStreaming:1;

	/** Allows LumenSceneData to set bTrackAllPrimitives even if the world is a preview world. This makes it possible to populate the Lumen surface cache and use Lumen in preview worlds, e.g. asset editor viewports. */
	uint32 bAllowLumenPrimitiveTrackingInPreviewWorld : 1;

	/** The default game mode for this world (if any) */
	TSubclassOf<class AGameModeBase> DefaultGameMode;

	FWorldInitializationValues& InitializeScenes(const bool bInitialize) { bInitializeScenes = bInitialize; return *this; }
	FWorldInitializationValues& AllowAudioPlayback(const bool bAllow) { bAllowAudioPlayback = bAllow; return *this; }
	FWorldInitializationValues& RequiresHitProxies(const bool bRequires) { bRequiresHitProxies = bRequires; return *this; }
	FWorldInitializationValues& CreatePhysicsScene(const bool bCreate) { bCreatePhysicsScene = bCreate; return *this; }
	FWorldInitializationValues& CreateNavigation(const bool bCreate) { bCreateNavigation = bCreate; return *this; }
	FWorldInitializationValues& CreateAISystem(const bool bCreate) { bCreateAISystem = bCreate; return *this; }
	FWorldInitializationValues& ShouldSimulatePhysics(const bool bInShouldSimulatePhysics) { bShouldSimulatePhysics = bInShouldSimulatePhysics; return *this; }
	FWorldInitializationValues& EnableTraceCollision(const bool bInEnableTraceCollision) { bEnableTraceCollision = bInEnableTraceCollision; return *this; }
	FWorldInitializationValues& ForceUseMovementComponentInNonGameWorld(const bool bInForceUseMovementComponentInNonGameWorld) { bForceUseMovementComponentInNonGameWorld = bInForceUseMovementComponentInNonGameWorld; return *this; }
	FWorldInitializationValues& SetTransactional(const bool bInTransactional) { bTransactional = bInTransactional; return *this; }
	FWorldInitializationValues& CreateFXSystem(const bool bCreate) { bCreateFXSystem = bCreate; return *this; }
	FWorldInitializationValues& CreateWorldPartition(const bool bCreate) { bCreateWorldPartition = bCreate; return *this; }
	FWorldInitializationValues& EnableWorldPartitionStreaming(const bool bEnableStreaming) { bEnableWorldPartitionStreaming = bEnableStreaming; return *this; }
	FWorldInitializationValues& AllowLumenPrimitiveTrackingInPreviewWorld(const bool bAllowLumenPrimitiveTracking) { bAllowLumenPrimitiveTrackingInPreviewWorld = bAllowLumenPrimitiveTracking; return *this; }
	FWorldInitializationValues& SetDefaultGameMode(TSubclassOf<class AGameModeBase> GameMode) { DefaultGameMode = GameMode; return *this; }
};


