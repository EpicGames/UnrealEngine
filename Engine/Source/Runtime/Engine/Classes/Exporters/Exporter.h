// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for all Exporters
 * An object responsible for exporting other objects to archives (files).
 * 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Exporter.generated.h"

class FExportObjectInnerContext;
class UActorComponent;
class UAssetExportTask;

UCLASS(abstract, transient, MinimalAPI)
class UExporter : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Supported class of this exporter */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TSubclassOf<class UObject>  SupportedClass;

	/**
	 * The root scope of objects to be exported, only used if PPF_ExportsNotFullyQualfied is set
	 * Objects being exported that are contained within ExportRootScope will use just their name instead of a full path
	 */
	UPROPERTY(transient)
	TObjectPtr<class UObject> ExportRootScope;    // transient

	/** File extension to use for this exporter */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TArray<FString> FormatExtension;

	/** Descriptiong of the export format */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TArray<FString> FormatDescription;

	/** Index into FormatExtension/FormatDescription of the preferred export format. */
	UPROPERTY()
	int32 PreferredFormatIndex;

	/** Current indentation of spaces of the exported text */
	UPROPERTY()
	int32 TextIndent;

	/** If true, this will export the data as text */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	uint32 bText:1;

	/** If true, this will export only the selected objects */
	UPROPERTY()
	uint32 bSelectedOnly:1;

	/** If true, this will force the exporter code to create a file-based Ar (this can keep large output files from taking too much memory) */
	UPROPERTY()
	uint32 bForceFileOperations:1;

	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TObjectPtr<class UAssetExportTask> ExportTask;

	ENGINE_API static		FString	CurrentFilename;
	/** (debugging purposes only) */
	static	const		bool	bEnableDebugBrackets;

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	//~ End UObject Interface.

	// Returns whether this exporter supports the specific object
	ENGINE_API virtual bool SupportsObject(UObject* Object) const;

	/**
	 * Export object to text
	 * 
	 * @param Context - Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
	 * @param Object - the object to export
	 * @param Type - filetype
	 * @param Ar - the archive to output the subobject definitions to
	 * @param Warn - Modal warning messages
	 * @param PortFlags - Flags controlling export behavior
	 * @return False
	 */
	 virtual bool ExportText( const class FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) { return( false );}

	/**
	 * Export parameters
	 * 
	 * @param RootMapPackageName - Package Name
	 * @param Context - Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
	 * @param InObject - the object to export
	 * @param Type - filetype
	 * @param Ar - the archive to output the subobject definitions to
	 * @param Warn - Modal warning messages
	 * @param PortFlags - Flags controlling export behavior
	 */
	 struct FExportPackageParams
	{
		FString RootMapPackageName;
		const class FExportObjectInnerContext* Context;
		UPackage* InPackage;
		UObject* InObject;
		const TCHAR* Type;
		FOutputDevice* Ar;
		FFeedbackContext* Warn;
		uint32 PortFlags;
	};

	/** Export Package Object */
	virtual void ExportPackageObject(FExportPackageParams& ExpPackageParams) {};

	/** Export Package Inner */
	virtual void ExportPackageInners(FExportPackageParams& ExpPackageParams) {};

	/**
	 * Export object to binary
	 * 
	 * @param Object - the object to export
	 * @param Type - filetype
	 * @param Ar - the archive to output the subobject definitions to
	 * @param Warn - Modal warning messages
	 * @param FileIndex - Index of files being exported
	 * @param PortFlags - Flags controlling export behavior
	 * @return False
	 */
	virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) { return( false );}

	/** 
	 * Number of binary files to export for this object. Should be 1 in the vast majority of cases. Noted exception would be multichannel sounds
	 * which have upto 8 raw waves stored within them.
	 * @todo document - this will only ever return 1 is that correct.
	 */
	virtual int32 GetFileCount( UObject* Object ) const { return( 1 ); }

	/** 
	 * Differentiates the filename for objects with multiple files to export. Only needs to be overridden if GetFileCount() returns > 1.
	 */
	virtual FString GetUniqueFilename( UObject* Object, const TCHAR* Filename, int32 FileIndex, int32 FileCount ) const
	{
		check( FileIndex == 0 && FileCount == 1 );
		return FString( Filename );
	}

	/**
	* Return true if the exporter is in batch mode.
	*/
	ENGINE_API virtual bool GetBatchMode() const;

	/**
	* Set the exporter batch mode.
	*/
	ENGINE_API virtual void SetBatchMode(bool InBatchExportMode);

	/**
	* Return true if the exporter batch was cancel.
	*/
	ENGINE_API virtual bool GetCancelBatch() const;

	/**
	* Set the exporter batch cancel state.
	*/
	ENGINE_API virtual void SetCancelBatch(bool InCancelBatch);

	/**
	* Return true if the exporter must show the export options.
	*/
	ENGINE_API virtual bool GetShowExportOption() const;

	/**
	* Set if the exporter should show or hide the export options.
	*/
	ENGINE_API virtual void SetShowExportOption(bool InShowExportOption);

	/**
	 * Find an exporter for the object and filetype.
	 *
	 * @param	Object - the object to export
	 * @param	Filetype - type of file
	 */
	ENGINE_API static UExporter* FindExporter( UObject* Object, const TCHAR* Filetype );

	/**
	 * Export this object to a file.  Child classes do not override this, but they do provide an Export() function
	 * to do the resource-specific export work.
	 * 
	 * @param	Object				the object to export
	 * @param	InExporter			exporter to use for exporting this object.  If NULL, attempts to create a valid exporter.
	 * @param	Filename			the name of the file to export this object to
	 * @param	InSelectedOnly		@todo
	 * @param	NoReplaceIdentical	false if we always want to overwrite any existing files, even if they're identical
	 * @param	Prompt				true if the user should be prompted to checkout/overwrite the output file if it already exists and is read-only
	 *
	 * @return	1 if the the object was successfully exported, 0 if a fatal error was encountered during export
	 */
	ENGINE_API static int32 ExportToFile( UObject* Object, UExporter* Exporter, const TCHAR* Filename, bool InSelectedOnly, bool NoReplaceIdentical=false, bool Prompt=false );

	/**
	 * Export object to an archive
	 * 
	 * @param Object - the object to export
	 * @param Exporter - the exporter class
	 * @param Ar - the archive to output the subobject definitions to
	 * @param FileType - Type of file to export
	 * @param FileIndex - Index of files being exported
	 * @return - true if the the object was successfully exported
	 */
	ENGINE_API static bool ExportToArchive(UObject* Object, UExporter* Exporter, FArchive& Ar, const TCHAR* FileType, int32 FileIndex);


	/**
	 * Export object to a device
	 * 
	 * @param Context - Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
	 * @param Object - the object to export
	 * @param InExporter - the exporter class
	 * @param Out - output device
	 * @param FileType - Type of file to export
	 * @param FileIndex - Index of files being exported
	 * @param Indent - Indent level
	 * @param PortFlags - Flags controlling export behavior
	 * @param bInSelectedOnly - true to export selected only
	 * @param	ExportRootScope	The scope to create relative paths from, if the PPF_ExportsNotFullyQualified flag is passed in.  If NULL, the package containing the object will be used instead.
	 * @return true if the export went successfully.
	 */
	ENGINE_API static bool ExportToOutputDevice( const class FExportObjectInnerContext* Context, UObject* Object, UExporter* InExporter, FOutputDevice& Out, const TCHAR* FileType, int32 Indent, uint32 PortFlags=0, bool bInSelectedOnly=false, UObject* ExportRootScope = NULL);

	struct FExportToFileParams
	{
		UObject* Object;
		UExporter* Exporter;
		const TCHAR* Filename;
		bool InSelectedOnly;
		bool NoReplaceIdentical/**=false*/;
		bool Prompt/**=false*/;
		bool bUseFileArchive;
		TArray<UObject*> IgnoreObjectList;
		bool WriteEmptyFiles;
	};

	/**
	* Export the given object to file.  Child classes do not override this, but they do provide an Export() function
	* to do the resource-specific export work.
	*
	* @param	Task		The task to export.
	*
	* @return	true if the the object was successfully exported
	*/
	UFUNCTION(BlueprintCallable, Category = "Miscellaneous")
	static ENGINE_API bool RunAssetExportTask( class UAssetExportTask* Task );

	/**
	* Export the given object to file.  Overridden by script based exporters.
	*
	* @param	Task		The task to export.
	*
	* @return	true if overridden by script exporter.
	*/
	UFUNCTION(BlueprintImplementableEvent, Category = "Miscellaneous")
	bool ScriptRunAssetExportTask( class UAssetExportTask* Task );

	/**
	 * Export the given objects to files.  Child classes do not override this, but they do provide an Export() function
	 * to do the resource-specific export work.
	 * 
	 * @param	ExportTasks		The array of tasks to export.
	 *
	 * @return	true if all tasks ran without error
	 */
	UFUNCTION(BlueprintCallable, Category = "Miscellaneous")
	static ENGINE_API bool RunAssetExportTasks( const TArray<UAssetExportTask*>& ExportTasks );
	
	/**
	 * Single entry point to export an object's subobjects, its components, and its properties
	 *
	 * @param Context			Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
	 * @param Object			The object to export 
	 * @param Ar				OutputDevice to print to
	 * @param PortFlags			Flags controlling export behavior
	 */
	ENGINE_API void ExportObjectInner(const class FExportObjectInnerContext* Context, UObject* Object, FOutputDevice& Ar, uint32 PortFlags);

	/**
	 * Should the given object be considered selected by the current export?
	 */
	static ENGINE_API bool IsObjectSelectedForExport(const FExportObjectInnerContext* Context, const UObject* Object);

protected:
	/**
	 * Allows the Exporter to export any extra information it would like about each instanced object.
	 * This occurs immediately after the component is exported.
	 *
	 * @param	Component		the instanced object being exported.
	 * @param	Ar				the archive to output the subobject definitions to.
	 * @param	PortFlags		the flags that were passed into the call to ExportText
	 */
	virtual void ExportComponentExtra(const FExportObjectInnerContext* Context, const TArray<UActorComponent*>& Components, FOutputDevice& Ar, uint32 PortFlags) {}

	/**
	 * Emits the starting line for a subobject definition.
	 *
	 * @param	Ar					the archive to output the text to
	 * @param	Obj					the object to emit the subobject block for
	 * @param	PortFlags			the flags that were passed into the call to ExportText
	 */
	ENGINE_API void EmitBeginObject( FOutputDevice& Ar, UObject* Obj, uint32 PortFlags );

	/**
	 * Emits the ending line for a subobject definition.
	 *
	 * @param	Ar					the archive to output the text to
	 * @param	bIncludeBrackets	(debugging purposes only)
	 */
	ENGINE_API void EmitEndObject( FOutputDevice& Ar );

protected:
	/** The set of registered exporters. */
	static TSet< TWeakObjectPtr<UExporter> > RegisteredExporters;

	/** The exporter is in batch mode. When the exporter is in batch mode we can offer a export all button in the options */
	bool BatchExportMode;

	/** The exporter will show the export options if this field is true */
	bool ShowExportOption;

	/** If set to true and BatchExportMode is also true, all other export from this bulk export should be cancel */
	bool CancelBatch;
};
