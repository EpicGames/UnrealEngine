// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Templates/Requires.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "PropertyPairsMap.h"
#include "Components/ChildActorComponent.h"
#include "RenderCommandFence.h"
#include "Net/Core/Misc/NetSubObjectRegistry.h"
#include "Engine/ReplicatedState.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteObjectTypes.h"
#endif

#include <atomic>

#if WITH_EDITOR
#include "Folder.h"
#endif

#include "WorldPartition/WorldPartitionActorDescType.h"

#include "Actor.generated.h"

class AActor;
class AController;
class APawn;
class APlayerController;
class UActorChannel;
class UChildActorComponent;
class UNetDriver;
class UPrimitiveComponent;
struct FAttachedActorInfo;
struct FNetViewer;
struct FNetworkObjectInfo;
class FActorTransactionAnnotation;
class FComponentInstanceDataCache;
struct FWorldRenameFromRootContext;
class UDataLayerAsset;
class UExternalDataLayerAsset;
class UDataLayerInstance;
class AWorldDataLayers;
class IWorldPartitionCell;
struct FActorReplicationParams;
struct FStopReplicatingActorParams;
class UActorFolder;
struct FActorDataLayer;
struct FHitResult;

namespace UE::Net
{
	class FTearOffSetter;
	struct FReplicationSystemUtil;
}

// By default, debug and development builds (even cooked) will keep actor labels. Manually define this if you want to make a local build
// that keep actor labels for Test or Shipping builds.
#ifndef ACTOR_HAS_LABELS
#define ACTOR_HAS_LABELS (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || WITH_PROFILEGPU)
#endif

/** Chooses a method for actors to update overlap state (objects it is touching) on initialization, currently only used during level streaming. */
UENUM(BlueprintType)
enum class EActorUpdateOverlapsMethod : uint8
{
	// Use the default value specified by the native class or config value.
	UseConfigDefault,
	// Always update overlap state on initialization.
	AlwaysUpdate,
	// Only update if root component has Movable mobility.
	OnlyUpdateMovable,
	// Never update overlap state on initialization.
	NeverUpdate
};

/** Determines how the transform being passed into actor spawning methods interact with the actor's default root component */
UENUM(BlueprintType)
enum class ESpawnActorScaleMethod : uint8
{
	/** Ignore the default scale in the actor's root component and hard-set it to the value of SpawnTransform Parameter */
	OverrideRootScale						UMETA(DisplayName = "Override Root Component Scale"),
	/** Multiply value of the SpawnTransform Parameter with the default scale in the actor's root component */
	MultiplyWithRoot						UMETA(DisplayName = "Multiply Scale With Root Component Scale"),
	SelectDefaultAtRuntime					UMETA(Hidden),
};

/** Determines in what type of LevelInstance the actor is in, if any. */
UENUM()
enum class ELevelInstanceType : uint8
{
	None,
	LevelInstance,
	LevelInstanceEdit,
	LevelInstancePropertyOverride
};

UENUM()
enum class ELevelInstanceFlags : uint8
{
	None = 0,
	/** Actor is inside an Level Instance editing hierarchy (used to show post process effect) */
	IsInEditHierarchy = 1,
	/** Actor has property overrides applied */
	HasPropertyOverrides = 2,
	/** Actor has property overrides from a top level editable level instance */
	HasEditablePropertyOverrides = 4 
};

ENUM_CLASS_FLAGS(ELevelInstanceFlags)

#if WITH_EDITORONLY_DATA
/** Enum defining how actor will be placed in the partition */
UENUM()
enum class UE_DEPRECATED(5.0, "EActorGridPlacement is deprecated.") EActorGridPlacement : uint8
{
	// Actor uses its bounds to determine in which runtime cells it's going to be placed.
	Bounds,
	// Actor uses its location to determine in which runtime cells it's going to be placed.
	Location,
	// Actor is always loaded (not placed in the grid), also affects editor.
	AlwaysLoaded,
	None UMETA(Hidden)
};
#endif

/** Helper struct that allows UPrimitiveComponent and FPrimitiveSceneInfo write to the Actor's LastRenderTime member */
struct FActorLastRenderTime
{
	UE_DEPRECATED(5.6, "Use GetLastRenderTime/SetLastRenderTime instead.")
	std::atomic<float> LastRenderTime;

	void SetLastRenderTime(float InLastRenderTime)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LastRenderTime.store(InLastRenderTime, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	float GetLastRenderTime() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LastRenderTime.load(std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "Use HasAlwaysVisibleComponents/IncrementAlwaysVisibleComponents/DecrementAlwaysVisibleComponents instead.")
	std::atomic_int32_t NumAlwaysVisibleComponents = 0;

	bool HasAlwaysVisibleComponents() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NumAlwaysVisibleComponents.load(std::memory_order_relaxed) > 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void IncrementAlwaysVisibleComponents()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumAlwaysVisibleComponents.fetch_add(1, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Returns the previous count (before decrement). */
	int32 DecrementAlwaysVisibleComponents()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return NumAlwaysVisibleComponents.fetch_sub(1, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	static void Set(AActor* InActor, float LastRenderTime);
	static FActorLastRenderTime* GetPtr(AActor* InActor);

	friend class UPrimitiveComponent;
	friend struct FPrimitiveSceneInfoAdapter;
};

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogActor, Log, All);

// Delegate signatures
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FiveParams( FTakeAnyDamageSignature, AActor, OnTakeAnyDamage, AActor*, DamagedActor, float, Damage, const class UDamageType*, DamageType, class AController*, InstigatedBy, AActor*, DamageCauser );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_NineParams( FTakePointDamageSignature, AActor, OnTakePointDamage, AActor*, DamagedActor, float, Damage, class AController*, InstigatedBy, FVector, HitLocation, class UPrimitiveComponent*, FHitComponent, FName, BoneName, FVector, ShotFromDirection, const class UDamageType*, DamageType, AActor*, DamageCauser );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_SevenParams( FTakeRadialDamageSignature, AActor, OnTakeRadialDamage, AActor*, DamagedActor, float, Damage, const class UDamageType*, DamageType, FVector, Origin, const FHitResult&, HitInfo, class AController*, InstigatedBy, AActor*, DamageCauser );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorBeginOverlapSignature, AActor, OnActorBeginOverlap, AActor*, OverlappedActor, AActor*, OtherActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorEndOverlapSignature, AActor, OnActorEndOverlap, AActor*, OverlappedActor, AActor*, OtherActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams( FActorHitSignature, AActor, OnActorHit, AActor*, SelfActor, AActor*, OtherActor, FVector, NormalImpulse, const FHitResult&, Hit );

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam( FActorBeginCursorOverSignature, AActor, OnBeginCursorOver, AActor*, TouchedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam( FActorEndCursorOverSignature, AActor, OnEndCursorOver, AActor*, TouchedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorOnClickedSignature, AActor, OnClicked, AActor*, TouchedActor , FKey, ButtonPressed );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorOnReleasedSignature, AActor, OnReleased, AActor*, TouchedActor , FKey, ButtonReleased );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorOnInputTouchBeginSignature, AActor, OnInputTouchBegin, ETouchIndex::Type, FingerIndex, AActor*, TouchedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorOnInputTouchEndSignature, AActor, OnInputTouchEnd, ETouchIndex::Type, FingerIndex, AActor*, TouchedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorBeginTouchOverSignature, AActor, OnInputTouchEnter, ETouchIndex::Type, FingerIndex, AActor*, TouchedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams( FActorEndTouchOverSignature, AActor, OnInputTouchLeave, ETouchIndex::Type, FingerIndex, AActor*, TouchedActor );

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FActorDestroyedSignature, AActor, OnDestroyed, AActor*, DestroyedActor );
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FActorEndPlaySignature, AActor, OnEndPlay, AActor*, Actor , EEndPlayReason::Type, EndPlayReason);

DECLARE_DELEGATE_SixParams(FMakeNoiseDelegate, AActor*, float /*Loudness*/, class APawn*, const FVector&, float /*MaxRange*/, FName /*Tag*/);

#if WITH_EDITOR
DECLARE_EVENT_TwoParams(AActor, FActorOnPackagingModeChanged, AActor*, bool /* bExternal */);

using FReplacementComponentMap = TMap<UObject*, UObject*>;
DECLARE_MULTICAST_DELEGATE_OneParam(FActorOnComponentsReplaced, const FReplacementComponentMap&);
#endif

#if !UE_BUILD_SHIPPING
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnProcessEvent, AActor*, UFunction*, void*);
#endif

/**
 * TInlineComponentArray is simply a TArray that reserves a fixed amount of space on the stack
 * to try to avoid heap allocation when there are fewer than a specified number of elements expected in the result.
 */
template<class T, uint32 NumElements = NumInlinedActorComponents>
class TInlineComponentArray : public TArray<T, TInlineAllocator<NumElements>>
{
	typedef TArray<T, TInlineAllocator<NumElements>> Super;

public:
	TInlineComponentArray() : Super() { }
	TInlineComponentArray(const AActor* Actor, bool bIncludeFromChildActors = false);
};

template<class T, uint32 NumElements>
struct TIsContiguousContainer<TInlineComponentArray<T, NumElements>>
{
	enum { Value = true };
};

/**
 * Actor is the base class for an Object that can be placed or spawned in a level.
 * Actors may contain a collection of ActorComponents, which can be used to control how actors move, how they are rendered, etc.
 * The other main function of an Actor is the replication of properties and function calls across the network during play.
 * 
 * 
 * Actor initialization has multiple steps, here's the order of important virtual functions that get called:
 * - UObject::PostLoad: For actors statically placed in a level, the normal UObject PostLoad gets called both in the editor and during gameplay.
 *                      This is not called for newly spawned actors.
 * - UActorComponent::OnComponentCreated: When an actor is spawned in the editor or during gameplay, this gets called for any native components.
 *                                        For blueprint-created components, this gets called during construction for that component.
 *                                        This is not called for components loaded from a level.
 * - AActor::PreRegisterAllComponents: For statically placed actors and spawned actors that have native root components, this gets called now.
 *                                     For blueprint actors without a native root component, these registration functions get called later during construction.
 * - UActorComponent::RegisterComponent: All components are registered in editor and at runtime, this creates their physical/visual representation.
 *                                       These calls may be distributed over multiple frames, but are always after PreRegisterAllComponents.
 *                                       This may also get called later on after an UnregisterComponent call removes it from the world.
 * - AActor::PostRegisterAllComponents: Called for all actors both in the editor and in gameplay, this is the last function that is called in all cases.
 * - AActor::PostActorCreated: When an actor is created in the editor or during gameplay, this gets called right before construction.
 *                             This is not called for components loaded from a level.
 * - AActor::UserConstructionScript: Called for blueprints that implement a construction script.
 * - AActor::OnConstruction: Called at the end of ExecuteConstruction, which calls the blueprint construction script.
 *                           This is called after all blueprint-created components are fully created and registered.
 *                           This is only called during gameplay for spawned actors, and may get rerun in the editor when changing blueprints.
 * - AActor::PreInitializeComponents: Called before InitializeComponent is called on the actor's components.
 *                                    This is only called during gameplay and in certain editor preview windows.
 * - UActorComponent::Activate: This will be called only if the component has bAutoActivate set.
 *                              It will also got called later on if a component is manually activated.
 * - UActorComponent::InitializeComponent: This will be called only if the component has bWantsInitializeComponentSet.
 *                                         This only happens once per gameplay session.
 * - AActor::PostInitializeComponents: Called after the actor's components have been initialized, only during gameplay and some editor previews.
 * - AActor::BeginPlay: Called when the level starts ticking, only during actual gameplay.
 *                      This normally happens right after PostInitializeComponents but can be delayed for networked or child actors.
 *
 * @see https://docs.unrealengine.com/Programming/UnrealArchitecture/Actors
 * @see https://docs.unrealengine.com/Programming/UnrealArchitecture/Actors/ActorLifecycle
 * @see UActorComponent
 */
UCLASS(BlueprintType, Blueprintable, config=Engine, meta=(ShortTooltip="An Actor is an object that can be placed or spawned in the world."), MinimalAPI)
class AActor : public UObject
{
	GENERATED_BODY()

public:
	/** Default constructor for AActor */
	ENGINE_API AActor();

	/** Constructor for AActor that takes an ObjectInitializer for backward compatibility */
	ENGINE_API AActor(const FObjectInitializer& ObjectInitializer);

private:
	/** Called from the constructor to initialize the class to its default settings */
	ENGINE_API void InitializeDefaults();

	enum class EAddComponentForReplicationBehavior : uint8
	{
		Immediate,
		Deferred,
		NeverAdd
	};

public:
	/** Returns the properties used for network replication, this needs to be overridden by all actor classes with native replicated properties */
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	/** Called when this actor begins replicating to initialize the state of custom property conditions */
	ENGINE_API virtual void GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const override;

	/**
	 * Primary Actor tick function, which calls TickActor().
	 * Tick functions can be configured to control whether ticking is enabled, at what time during a frame the update occurs, and to set up tick dependencies.
	 * @see https://docs.unrealengine.com/API/Runtime/Engine/Engine/FTickFunction
	 * @see AddTickPrerequisiteActor(), AddTickPrerequisiteComponent()
	 */
	UPROPERTY(EditDefaultsOnly, Category=Tick)
	struct FActorTickFunction PrimaryActorTick;

	/** If true, when the actor is spawned it will be sent to the client but receive no further replication updates from the server afterwards. */
	UPROPERTY()
	uint8 bNetTemporary:1;

	/** If true, this actor was loaded directly from the map, and for networking purposes can be addressed by its full path name */
	uint8 bNetStartup:1;

	/** If true, this actor is only relevant to its owner. If this flag is changed during play, all non-owner channels would need to be explicitly closed. */
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadOnly)
	uint8 bOnlyRelevantToOwner:1;

	/** Always relevant for network (overrides bOnlyRelevantToOwner). */
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite)
	uint8 bAlwaysRelevant:1;    

	/** Called on client when updated bReplicateMovement value is received for this actor. */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicateMovement();

#if WITH_EDITOR
	virtual bool ShouldDuplicateInPIE() const { return !bIgnoreInPIE; }
	virtual bool ShouldForceDuplicateInPIE() const { return bForceDuplicateInPIE; }
#endif

private:
	/**
	 * If true, replicate movement/location related properties.
	 * Actor must also be set to replicate.
	 * @see SetReplicates()
	 * @see https://docs.unrealengine.com/InteractiveExperiences/Networking/Actors
	 */
	UPROPERTY(ReplicatedUsing=OnRep_ReplicateMovement, Category=Replication, EditDefaultsOnly)
	uint8 bReplicateMovement:1;    

	UPROPERTY(EditDefaultsOnly, Category = Replication, AdvancedDisplay)
	uint8 bCallPreReplication:1;

	UPROPERTY(EditDefaultsOnly, Category = Replication, AdvancedDisplay)
	uint8 bCallPreReplicationForReplay:1;

	/**
	 * Allows us to only see this Actor in the Editor, and not in the actual game.
	 * @see SetActorHiddenInGame()
	 */
	UPROPERTY(Interp, EditAnywhere, Category=Rendering, BlueprintReadOnly, Replicated, meta=(AllowPrivateAccess="true", DisplayName="Actor Hidden In Game", SequencerTrackClass="/Script/MovieSceneTracks.MovieSceneVisibilityTrack"))
	uint8 bHidden:1;

	UPROPERTY(Replicated)
	uint8 bTearOff:1;

	friend class UE::Net::FTearOffSetter;

	/** When set, indicates that external guarantees ensure that this actor's name is deterministic between server and client, and as such can be addressed by its full path */
	UPROPERTY()
	uint8 bForceNetAddressable:1;

#if WITH_EDITORONLY_DATA
	/** Whether this actor belongs to a level instance or not and what type of level instance. */
	UPROPERTY(Transient, NonTransactional)
	ELevelInstanceType LevelInstanceType;

	/** Flags related to level instances for this actor. */
	UPROPERTY(Transient, NonTransactional)
	ELevelInstanceFlags LevelInstanceFlags;

	friend struct FSetActorIsInLevelInstance;
	friend struct FAddActorLevelInstanceFlags;
	friend struct FRemoveActorLevelInstanceFlags;

public:
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = LevelInstance, meta = (Tooltip = "If checked, this Actor will only get loaded in a main world (persistent level), it will not be loaded through Level Instances."))
	uint8 bIsMainWorldOnly : 1;

	/** Experimental: this will force this actor to be ignored in PIE, will not work if the actor has references to other actors without the flag set, use with care. */
	UPROPERTY(Transient)
	uint8 bIgnoreInPIE : 1;

	UPROPERTY(Transient)
	uint8 bForceDuplicateInPIE : 1;

private:
#endif

	/** If true, PreReplication will be called on this actor before each potential replication. */
	ENGINE_API bool ShouldCallPreReplication() const;

	/** If true, PreReplicationForReplay will be called on this actor before each potential replication. */
	ENGINE_API bool ShouldCallPreReplicationForReplay() const;

public:
	/** Set whether or not we should make calls to PreReplication. */
	ENGINE_API void SetCallPreReplication(bool bCall);

	/** Set whether or not we should make calls to PreReplicationForReplay. */
	ENGINE_API void SetCallPreReplicationForReplay(bool bCall);

	/** If true, this actor is no longer replicated to new clients, and is "torn off" (becomes a ROLE_Authority) on clients to which it was being replicated. */
	bool GetTearOff() const
	{
		return bTearOff;
	}

#if WITH_EDITOR

	/** If true, the actor belongs to a level instance which is currently in an edit mode: edit or property override edit */
	bool IsInAnyEditLevelInstance() const
	{
		return IsInEditLevelInstance() || IsInPropertyOverrideLevelInstance();
	}

	/** If true, the actor belongs to a level instance which is currently being edited */
	bool IsInEditLevelInstance() const
	{
		return GetLevelInstanceType() == ELevelInstanceType::LevelInstanceEdit;
	}

	/** If true, the actor belongs to a level instance which is currently in property override edit */
	bool IsInPropertyOverrideLevelInstance() const
	{
		return GetLevelInstanceType() == ELevelInstanceType::LevelInstancePropertyOverride;
	}

	/** If true, the actor belongs to a level instance which is currently being edited or a parent level instance being edited. */
	bool IsInEditLevelInstanceHierarchy() const
	{
		return EnumHasAnyFlags(GetLevelInstanceFlags(), ELevelInstanceFlags::IsInEditHierarchy);
	}

	/** If true, this actor or one of its components has property overrides applied. */
	bool HasLevelInstancePropertyOverrides() const
	{
		return EnumHasAnyFlags(GetLevelInstanceFlags(), ELevelInstanceFlags::HasPropertyOverrides);
	}

	/** If true, this actor or one of its components has property overrides applied from an editable level instance (level instance is in a non readonly level) */
	bool HasEditableLevelLevelInstancePropertyOverrides() const
	{
		return EnumHasAnyFlags(GetLevelInstanceFlags(), ELevelInstanceFlags::HasEditablePropertyOverrides);
	}

	/** If true, the actor belongs to a level instance. */
	bool IsInLevelInstance() const
	{
		return GetLevelInstanceType() != ELevelInstanceType::None;
	}

	/** Whether this actor belongs to a level instance or not and what type of level instance. */
	ENGINE_API ELevelInstanceType GetLevelInstanceType() const;

	/** Flags related to level instances for this actor. */
	ENGINE_API ELevelInstanceFlags GetLevelInstanceFlags() const;
#endif

	/** Networking - Server - TearOff this actor to stop replication to clients. Will set bTearOff to true. */
	UFUNCTION(BlueprintCallable, Category=Networking)
	ENGINE_API virtual void TearOff();

	/**
	 * Whether we have already exchanged Role/RemoteRole on the client, as when removing then re-adding a streaming level.
	 * Causes all initialization to be performed again even though the actor may not have actually been reloaded.
	 */
	UPROPERTY(Transient)
	uint8 bExchangedRoles:1;

	/** This actor will be loaded on network clients during map load */
	UPROPERTY(Category=Replication, EditAnywhere)
	uint8 bNetLoadOnClient:1;

	/** If actor has valid Owner, call Owner's IsNetRelevantFor and GetNetPriority */
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite)
	uint8 bNetUseOwnerRelevancy:1;

	/** If true, this actor will be replicated to network replays (default is true) */
	UPROPERTY()
	uint8 bRelevantForNetworkReplays:1;

	/** 
	 * If true, this actor's component's bounds will be included in the level's
	 * bounding box unless the Actor's class has overridden IsLevelBoundsRelevant 
	 */
	UPROPERTY(EditAnywhere, Category=Collision, AdvancedDisplay)
	uint8 bRelevantForLevelBounds:1;

	/**
	 * If true, this actor will only be destroyed during scrubbing if the replay is set to a time before the actor existed.
	 * Otherwise, RewindForReplay will be called if we detect the actor needs to be reset.
	 * Note, this Actor must not be destroyed by gamecode, and RollbackViaDeletion may not be used. 
	 */
	UPROPERTY(Category=Replication, EditDefaultsOnly, AdvancedDisplay)
	uint8 bReplayRewindable:1;

	/**
	 * Whether we allow this Actor to tick before it receives the BeginPlay event.
	 * Normally we don't tick actors until after BeginPlay; this setting allows this behavior to be overridden.
	 * This Actor must be able to tick for this setting to be relevant.
	 */
	UPROPERTY(EditDefaultsOnly, Category=Tick)
	uint8 bAllowTickBeforeBeginPlay:1;

private:
	/** If true then destroy self when "finished", meaning all relevant components report that they are done and no timelines or timers are in flight. */
	UPROPERTY(BlueprintSetter=SetAutoDestroyWhenFinished, Category=Actor)
	uint8 bAutoDestroyWhenFinished:1;

	/**
	 * Whether this actor can take damage. Must be true for damage events (e.g. ReceiveDamage()) to be called.
	 * @see https://www.unrealengine.com/blog/damage-in-ue4
	 * @see TakeDamage(), ReceiveDamage()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, SaveGame, Replicated, Category=Actor, meta=(AllowPrivateAccess="true"))
	uint8 bCanBeDamaged:1;

public:
	/** If true, all input on the stack below this actor will not be considered */
	UPROPERTY(EditDefaultsOnly, Category=Input)
	uint8 bBlockInput:1;

	/** This actor collides with the world when placing in the editor, even if RootComponent collision is disabled. Does not affect spawning, @see SpawnCollisionHandlingMethod */
	UPROPERTY()
	uint8 bCollideWhenPlacing:1;

	/** If true, this actor should search for an owned camera component to view through when used as a view target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Actor, AdvancedDisplay)
	uint8 bFindCameraComponentWhenViewTarget:1;
	
    /**
	 * If true, this actor will generate overlap Begin/End events when spawned as part of level streaming, which includes initial level load.
	 * You might enable this is in the case where a streaming level loads around an actor and you want Begin/End overlap events to trigger.
	 * @see UpdateOverlapsMethodDuringLevelStreaming
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Collision)
	uint8 bGenerateOverlapEventsDuringLevelStreaming:1;

	/** Whether this actor should not be affected by world origin shifting. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Actor)
	uint8 bIgnoresOriginShifting:1;

	/** Whether this actor should be considered or not during HLOD generation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=HLOD, meta=(DisplayName="Include Actor in HLOD"))
	uint8 bEnableAutoLODGeneration:1;

	/** Whether this actor is editor-only. Use with care, as if this actor is referenced by anything else that reference will be NULL in cooked builds */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Cooking)
	uint8 bIsEditorOnlyActor:1;

	/** Indicates the actor was pulled through a seamless travel.  */
	uint8 bActorSeamlessTraveled:1;

	/**
	 * Does this actor have an owner responsible for replication? (APlayerController typically)
	 *
	 * @return true if this actor can call RPCs or false if no such owner chain exists
	 */
	ENGINE_API virtual bool HasNetOwner() const;

	/**
	 * Does this actor have a locally controlled owner responsible for replication? (APlayerController typically)
	 *
	 * @return true if this actor can call RPCs or false if no such owner chain exists
	 */
	ENGINE_API virtual bool HasLocalNetOwner() const;

	bool GetAutoDestroyWhenFinished() const { return bAutoDestroyWhenFinished; }

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetAutoDestroyWhenFinished(bool bVal);

protected:
	/**
	 * If true, this actor will replicate to remote machines
	 * @see SetReplicates()
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category=Replication)
	uint8 bReplicates:1;

	/** This function should only be used in the constructor of classes that need to set the RemoteRole for backwards compatibility purposes */
	void SetRemoteRoleForBackwardsCompat(const ENetRole InRemoteRole) { RemoteRole = InRemoteRole; }

	/** Called when owner changes, does nothing by default but can be overridden */
	UFUNCTION()
	ENGINE_API virtual void OnRep_Owner();

	/** If true, this actor can be put inside of a GC Cluster to improve Garbage Collection performance */
	UPROPERTY(Category=Actor, EditAnywhere, AdvancedDisplay)
	uint8 bCanBeInCluster:1;

#if WITH_EDITORONLY_DATA
	/** If true, this actor can be migrated to another server even if it's been loaded from disk */
	UPROPERTY(Category = Actor, EditAnywhere, AdvancedDisplay)
	uint8 bMigratingAsset : 1;
#endif

	/**
	 * If false, the Blueprint ReceiveTick() event will be disabled on dedicated servers.
	 * @see AllowReceiveTickEventOnDedicatedServer()
	 */
	UPROPERTY()
	uint8 bAllowReceiveTickEventOnDedicatedServer:1;

	/** Flag indicating we have checked initial simulating physics state to sync networked proxies to the server. */
	uint8 bNetCheckedInitialPhysicsState : 1;

	/**
	* When true the replication system will only replicate the registered subobjects and the replicated actor components list
	* When false the replication system will instead call the virtual ReplicateSubobjects() function where the subobjects and actor components need to be manually replicated.
	*/
	UPROPERTY(Config, EditDefaultsOnly, BlueprintReadOnly, Category=Replication, AdvancedDisplay)
	uint8 bReplicateUsingRegisteredSubObjectList : 1;

private:
	friend class FActorDeferredScriptManager;

	/** Whether FinishSpawning has been called for this Actor.  If it has not, the Actor is in a malformed state */
	uint8 bHasFinishedSpawning:1;

	/** 
	 *	Indicates that PreInitializeComponents/PostInitializeComponents have been called on this Actor 
	 *	Prevents re-initializing of actors spawned during level startup
	 */
	uint8 bActorInitialized:1;

	/** Set when DispatchBeginPlay() triggers from level streaming, and cleared afterwards. @see IsActorBeginningPlayFromLevelStreaming(). */
	uint8 bActorBeginningPlayFromLevelStreaming:1;

	/** Whether we've tried to register tick functions. Reset when they are unregistered. */
	uint8 bTickFunctionsRegistered:1;

	/** Whether we've deferred the RegisterAllComponents() call at spawn time. Reset when RegisterAllComponents() is called. */
	uint8 bHasDeferredComponentRegistration:1;

	/** True if this actor is currently running user construction script (used to defer component registration) */
	uint8 bRunningUserConstructionScript:1;

	/** True if PreRegisterAllComponents was called on this actor. */
	uint8 bHasPreRegisteredAllComponents : 1;

	/** Set true just before PostRegisterAllComponents() is called and false just before PostUnregisterAllComponents() is called */
	uint8 bHasRegisteredAllComponents:1;

	/**
	 * Enables any collision on this actor.
	 * @see SetActorEnableCollision(), GetActorEnableCollision()
	 */
	UPROPERTY()
	uint8 bActorEnableCollision:1;

	/** Set when actor is about to be deleted. Needs to be a FProperty so it is included in transactions. */
	UPROPERTY(Transient, DuplicateTransient)
	uint8 bActorIsBeingDestroyed:1;

	/** Set if an Actor tries to be destroyed while it is beginning play so that once BeginPlay ends we can issue the destroy call. */
	uint8 bActorWantsDestroyDuringBeginPlay : 1;

	/** Enum defining if BeginPlay has started or finished */
	enum class EActorBeginPlayState : uint8
	{
		HasNotBegunPlay,
		BeginningPlay,
		HasBegunPlay,
	};

	/** 
	 *	Indicates that BeginPlay has been called for this Actor.
	 *  Set back to HasNotBegunPlay once EndPlay has been called.
	 */
	EActorBeginPlayState ActorHasBegunPlay:2;

	/** Set while actor is being constructed. Used to ensure that construction is not re-entrant. */
	uint8 bActorIsBeingConstructed : 1;

	/** Set for actors created from networking that are not yet fully initialized */
	uint8 bActorIsPendingPostNetInit : 1;	

	static ENGINE_API uint32 BeginPlayCallDepth;

protected:
		
	/** Whether to use use the async physics tick with this actor. */
	UPROPERTY(EditAnywhere, Category=Physics)
	uint8 bAsyncPhysicsTickEnabled : 1;
	
	/**
	 * Condition for calling UpdateOverlaps() to initialize overlap state when loaded in during level streaming.
	 * If set to 'UseConfigDefault', the default specified in ini (displayed in 'DefaultUpdateOverlapsMethodDuringLevelStreaming') will be used.
	 * If overlaps are not initialized, this actor and attached components will not have an initial state of what objects are touching it,
	 * and overlap events may only come in once one of those objects update overlaps themselves (for example when moving).
	 * However if an object touching it *does* initialize state, both objects will know about their touching state with each other.
	 * This can be a potentially large performance savings during level loading and streaming, and is safe if the object and others initially
	 * overlapping it do not need the overlap state because they will not trigger overlap notifications.
	 * 
	 * Note that if 'bGenerateOverlapEventsDuringLevelStreaming' is true, overlaps are always updated in this case, but that flag
	 * determines whether the Begin/End overlap events are triggered.
	 * 
	 * @see bGenerateOverlapEventsDuringLevelStreaming, DefaultUpdateOverlapsMethodDuringLevelStreaming, GetUpdateOverlapsMethodDuringLevelStreaming()
	 */
	UPROPERTY(Category=Collision, EditAnywhere)
	EActorUpdateOverlapsMethod UpdateOverlapsMethodDuringLevelStreaming;

public:

	/** Get the method used to UpdateOverlaps() when loaded via level streaming. Resolves the 'UseConfigDefault' option to the class default specified in config. */
	ENGINE_API EActorUpdateOverlapsMethod GetUpdateOverlapsMethodDuringLevelStreaming() const;

private:

	/**
	 * Default value taken from config file for this class when 'UseConfigDefault' is chosen for
	 * 'UpdateOverlapsMethodDuringLevelStreaming'. This allows a default to be chosen per class in the matching config.
	 * For example, for Actor it could be specified in DefaultEngine.ini as:
	 * 
	 * [/Script/Engine.Actor]
	 * DefaultUpdateOverlapsMethodDuringLevelStreaming = OnlyUpdateMovable
	 *
	 * Another subclass could set their default to something different, such as:
	 *
	 * [/Script/Engine.BlockingVolume]
	 * DefaultUpdateOverlapsMethodDuringLevelStreaming = NeverUpdate
	 * 
	 * @see UpdateOverlapsMethodDuringLevelStreaming
	 */
	UPROPERTY(Config, Category = Collision, VisibleAnywhere)
	EActorUpdateOverlapsMethod DefaultUpdateOverlapsMethodDuringLevelStreaming;

	/** Internal helper to update Overlaps during Actor initialization/BeginPlay correctly based on the UpdateOverlapsMethodDuringLevelStreaming and bGenerateOverlapEventsDuringLevelStreaming settings. */
	ENGINE_API void UpdateInitialOverlaps(bool bFromLevelStreaming);
	
	/** Describes how much control the remote machine has over the actor. */
	UPROPERTY(Replicated, Transient, VisibleInstanceOnly, Category=Networking)
	TEnumAsByte<enum ENetRole> RemoteRole;

public:
	/**
	 * Set whether this actor replicates to network clients. When this actor is spawned on the server it will be sent to clients as well.
	 * Properties flagged for replication will update on clients if they change on the server.
	 * Internally changes the RemoteRole property and handles the cases where the actor needs to be added to the network actor list.
	 * @param bInReplicates Whether this Actor replicates to network clients.
	 * @see https://docs.unrealengine.com/InteractiveExperiences/Networking/Actors
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Networking)
	ENGINE_API void SetReplicates(bool bInReplicates);

	/**
	 * Set whether this actor's movement replicates to network clients.
	 * @param bInReplicateMovement Whether this Actor's movement replicates to clients.
	 */
	UFUNCTION(BlueprintCallable, Category=Networking)
	ENGINE_API virtual void SetReplicateMovement(bool bInReplicateMovement);

	/** Sets whether or not this Actor is an autonomous proxy, which is an actor on a network client that is controlled by a user on that client. */
	ENGINE_API void SetAutonomousProxy(const bool bInAutonomousProxy, const bool bAllowForcePropertyCompare=true);
	
	/** Copies RemoteRole from another Actor and adds this actor to the list of network actors if necessary. */
	ENGINE_API void CopyRemoteRoleFrom(const AActor* CopyFromActor);

	/** Returns how much control the local machine has over this actor. */
	UFUNCTION(BlueprintCallable, Category=Networking)
	ENetRole GetLocalRole() const { return Role; }

	/** Returns how much control the remote machine has over this actor. */
	UFUNCTION(BlueprintCallable, Category=Networking)
	ENetRole GetRemoteRole() const;

	/**
	 * Allows this actor to be net-addressable by full path name, even if the actor was spawned after map load.
	 * @note: The caller is required to ensure that this actor's name is stable between server/client. Must be called before FinishSpawning
	 */
	ENGINE_API void SetNetAddressable();

public:
	/** Project-specific field that help to categorize actors for reporting purposes*/
	uint8 ActorCategory = 0;

	/** How long this Actor lives before dying, 0=forever. Note this is the INITIAL value and should not be modified once play has begun. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Actor)
	float InitialLifeSpan;

	/** Allow each actor to run at a different time speed. The DeltaTime for a frame is multiplied by the global TimeDilation (in WorldSettings) and this CustomTimeDilation for this actor's tick.  */
	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, Category=Actor)
	float CustomTimeDilation;
	
private:
	/** The RayTracingGroupId this actor and its components belong to. (For components that did not specify any) */
	UPROPERTY()
	int32 RayTracingGroupId;

protected:
#if WITH_EDITORONLY_DATA
	/** @deprecated Use bIsSpatiallyLoaded instead */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	EActorGridPlacement GridPlacement_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** 
	 * Determine in which partition grid this actor will be placed in the partition (if the world is partitioned).
	 * If None, the decision will be left to the partition.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=WorldPartition)
	FName RuntimeGrid;
#endif

	/**
	 * Used for replicating attachment of this actor's RootComponent to another actor.
	 * This is filled in via GatherCurrentMovement() when the RootComponent has an AttachParent.
	 */
	UPROPERTY(Transient, ReplicatedUsing=OnRep_AttachmentReplication)
	struct FRepAttachment AttachmentReplication;

private:
	/** Used for replication of our RootComponent's position and velocity */
	UPROPERTY(EditDefaultsOnly, ReplicatedUsing=OnRep_ReplicatedMovement, Category=Replication, AdvancedDisplay)
	struct FRepMovement ReplicatedMovement;

public:
	/**
	 * Owner of this Actor, used primarily for replication (bNetUseOwnerRelevancy & bOnlyRelevantToOwner) and visibility (PrimitiveComponent bOwnerNoSee and bOnlyOwnerSee)
	 * @see SetOwner(), GetOwner()
	 */
	UPROPERTY(ReplicatedUsing=OnRep_Owner)
	TObjectPtr<AActor> Owner;

#if WITH_EDITOR
	/**
	 * Used to track changes to Owner during Undo events.
	 */
	TWeakObjectPtr<AActor> IntermediateOwner = nullptr;
#endif

protected:
	/** Used to specify the net driver to replicate on (NAME_None || NAME_GameNetDriver is the default net driver) */
	UPROPERTY()
	FName NetDriverName;

public:
	/** Get read-only access to current AttachmentReplication. */
	const struct FRepAttachment& GetAttachmentReplication() const { return AttachmentReplication; }

	/** Called on client when updated AttachmentReplication value is received for this actor. */
	UFUNCTION()
	ENGINE_API virtual void OnRep_AttachmentReplication();

private:
	/** Describes how much control the local machine has over the actor. */
	UPROPERTY(Replicated, VisibleInstanceOnly, Category=Networking)
	TEnumAsByte<enum ENetRole> Role;

public:
	/** Dormancy setting for actor to take itself off of the replication list without being destroyed on clients. */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category=Replication)
	TEnumAsByte<enum ENetDormancy> NetDormancy;

	/** Controls how to handle spawning this actor in a situation where it's colliding with something else. "Default" means AlwaysSpawn here. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Actor)
	ESpawnActorCollisionHandlingMethod SpawnCollisionHandlingMethod;

	/** Automatically registers this actor to receive input from a player. */
	UPROPERTY(EditAnywhere, Category=Input)
	TEnumAsByte<EAutoReceiveInput::Type> AutoReceiveInput;

	/** The priority of this input component when pushed in to the stack. */
	UPROPERTY(EditAnywhere, Category=Input)
	int32 InputPriority;
	
	/**
	 * The time this actor was created, relative to World->GetTimeSeconds().
	 * @see UWorld::GetTimeSeconds()
	 */
	float CreationTime;

	/** Component that handles input for this actor, if input is enabled. */
	UPROPERTY(DuplicateTransient)
	TObjectPtr<class UInputComponent> InputComponent;

	/** Internal - used by UNetDriver */
	UPROPERTY(Transient)
	int32 NetTag;

	/** Square of the max distance from the client's viewpoint that this actor is relevant and will be replicated. */
	UE_DEPRECATED(5.5, "Public access to NetCullDistanceSquared has been deprecated. Use SetNetCullDistanceSquared() and GetNetCullDistanceSquared() instead.")
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite, meta=(AllowPrivateAccess=true), BlueprintGetter=GetNetCullDistanceSquared, BlueprintSetter=SetNetCullDistanceSquared)
	float NetCullDistanceSquared;

	/** How often (per second) this actor will be considered for replication, used to determine NetUpdateTime */
	UE_DEPRECATED(5.5, "Public access to NetUpdateFrequency has been deprecated. Use SetNetUpdateFrequency() and GetNetUpdateFrequency() instead.")
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite, meta=(AllowPrivateAccess=true), BlueprintGetter=GetNetUpdateFrequency, BlueprintSetter=SetNetUpdateFrequency)
	float NetUpdateFrequency;

	/** Used to determine what rate to throttle down to when replicated properties are changing infrequently */
	UE_DEPRECATED(5.5, "Public access MinNetUpdateFrequency has been deprecated. Use SetMinNetUpdateFrequency() and GetMinNetUpdateFrequency() instead.")
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite, meta=(AllowPrivateAccess=true), BlueprintGetter=GetMinNetUpdateFrequency, BlueprintSetter=SetMinNetUpdateFrequency)
	float MinNetUpdateFrequency;

	/** Priority for this actor when checking for replication in a low bandwidth or saturated situation, higher priority means it is more likely to replicate */
	UPROPERTY(Category=Replication, EditDefaultsOnly, BlueprintReadWrite)
	float NetPriority;

private:
	/** Which mode to replicate physics through for this actor. Only relevant if the actor replicates movement and has a component that simulate physics.*/
	UPROPERTY(EditDefaultsOnly, Category = Replication)
	EPhysicsReplicationMode PhysicsReplicationMode;
	
public:
	/** Set the physics replication mode of this body, via EPhysicsReplicationMode */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API void SetPhysicsReplicationMode(const EPhysicsReplicationMode ReplicationMode);

	/** Get the physics replication mode of this body, via EPhysicsReplicationMode */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API EPhysicsReplicationMode GetPhysicsReplicationMode();

	/** Register this actors root components physics object as a focal particle in Physics Repliocation LOD */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API void RegisterAsFocalPointInPhysicsReplicationLOD() const;

	/** Unregister this actors root components physics object from being a focal particle in Physics Repliocation LOD */
	UFUNCTION(BlueprintCallable, Category = Replication)
	ENGINE_API void UnregisterAsFocalPointInPhysicsReplicationLOD() const;

	/** Can this body trigger a resimulation when Physics Prediction is enabled */
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API bool CanTriggerResimulation() const;

	/** Get the error threshold in centimeters before this object should enforce a resimulation to trigger. */
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API float GetResimulationThreshold() const;

private:
	/**
	 * The value of WorldSettings->TimeSeconds for the frame when one of this actor's components was last rendered.  This is written
	 * from the render thread, which is up to a frame behind the game thread, so you should allow this time to
	 * be at least a frame behind the game thread's world time before you consider the actor non-visible.
	 */
	FActorLastRenderTime LastRenderTime;

	friend struct FActorLastRenderTime;

public:
	/**
	 * Set the name of the net driver associated with this actor.  Will move the actor out of the list of network actors from the old net driver and add it to the new list
	 * @param NewNetDriverName name of the new net driver association
	 */
	ENGINE_API void SetNetDriverName(FName NewNetDriverName);

	/** Returns name of the net driver associated with this actor (all RPCs will go out via this connection) */
	FName GetNetDriverName() const { return NetDriverName; }

	/** 
    * Returns true if this actor is replicating SubObjects & ActorComponents via the registration list.
    * Returns false when it replicates them via the virtual ReplicateSubobjects method. 
    */
	bool IsUsingRegisteredSubObjectList() const { return bReplicateUsingRegisteredSubObjectList; }

	/** 
	* Method that allows an actor to replicate subobjects on its actor channel. 
	* Must return true if any data was serialized into the bunch.
	* This method is used only when bReplicateUsingRegisteredSubObjectList is false.
	* Otherwise this function is not called and only the ReplicatedSubObjects list is used.
	*/
	ENGINE_API virtual bool ReplicateSubobjects(class UActorChannel *Channel, class FOutBunch *Bunch, FReplicationFlags *RepFlags);

	/** Called on the actor when a new subobject is dynamically created via replication */
	ENGINE_API virtual void OnSubobjectCreatedFromReplication(UObject *NewSubobject);

	/** Called on the actor when a subobject is dynamically destroyed via replication */
	ENGINE_API virtual void OnSubobjectDestroyFromReplication(UObject *Subobject);

	/**
	 * Called on the actor right before replication occurs. 
	 * Only called on Server, and for autonomous proxies if recording a Client Replay.
	 */
	ENGINE_API virtual void PreReplication(IRepChangedPropertyTracker & ChangedPropertyTracker);

	/**
	 * Called on the actor right before replication occurs.
	 * Called for everyone when recording a Client Replay, including Simulated Proxies.
	 */
	ENGINE_API virtual void PreReplicationForReplay(IRepChangedPropertyTracker & ChangedPropertyTracker);

	/**
	 * Called on the actor before checkpoint data is applied during a replay.
	 * Only called if bReplayRewindable is set.
	 */
	ENGINE_API virtual void RewindForReplay();

	/** Called by the networking system to call PreReplication on this actor and its components using the given NetDriver to find or create RepChangedPropertyTrackers. */
	ENGINE_API void CallPreReplication(UNetDriver* NetDriver);	

private:
	/** Pawn responsible for damage and other gameplay events caused by this actor. */
	UPROPERTY(BlueprintReadWrite, ReplicatedUsing=OnRep_Instigator, meta=(ExposeOnSpawn=true, AllowPrivateAccess=true), Category=Actor)
	TObjectPtr<class APawn> Instigator;

public:
	/** Called on clients when Instigator is replicated. */
	UFUNCTION()
	ENGINE_API virtual void OnRep_Instigator();

	/** Array of all Actors whose Owner is this actor, these are not necessarily spawned by UChildActorComponent */
	UPROPERTY(Transient)
	TArray<TObjectPtr<AActor>> Children;

protected:
	/** The component that defines the transform (location, rotation, scale) of this Actor in the world, all other components must be attached to this one somehow */
	UPROPERTY(BlueprintGetter=K2_GetRootComponent, Category="Transformation")
	TObjectPtr<USceneComponent> RootComponent;

#if WITH_EDITORONLY_DATA
	/** Local space pivot offset for the actor, only used in the editor */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=Actor)
	FVector PivotOffset;
#endif

	/** Handle for efficient management of LifeSpanExpired timer */
	FTimerHandle TimerHandle_LifeSpanExpired;

public:
#if WITH_EDITOR
	/** Return the HLOD layer that should include this actor. */
	ENGINE_API class UHLODLayer* GetHLODLayer() const;

	/** Specify in which HLOD layer this actor should be included. */
	ENGINE_API void SetHLODLayer(class UHLODLayer* InHLODLayer);

	/** Gets the property name for HLODLayer. */
	static const FName GetHLODLayerPropertyName() { return GET_MEMBER_NAME_CHECKED(AActor, HLODLayer); }
#endif

	/** Specify a RayTracingGroupId for this actors. Components with invalid RayTracingGroupId will inherit the actors. */
	UFUNCTION(BlueprintCallable, Category = RayTracing)
	ENGINE_API void SetRayTracingGroupId(int32 InRaytracingGroupId);
	
	/** Return the RayTracingGroupId for this actor. */
	UFUNCTION(BlueprintCallable, Category = RayTracing)
	ENGINE_API int32 GetRayTracingGroupId() const;

private:
	/** The UHLODLayer in which this actor should be included. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Layer"))
	TObjectPtr<class UHLODLayer> HLODLayer;

#if WITH_EDITORONLY_DATA
	/** Temporary list of objects that were outered to this actor's external package. Used when detaching/attaching external actor packages during cook. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> ObjectsDetachedFromExternalPackage;
#endif

public:
	/** Return the value of bAllowReceiveTickEventOnDedicatedServer, indicating whether the Blueprint ReceiveTick() event will occur on dedicated servers. */
	inline bool AllowReceiveTickEventOnDedicatedServer() const { return bAllowReceiveTickEventOnDedicatedServer; }

	/** Returns if this actor is currently running the User Construction Script */
	inline bool IsRunningUserConstructionScript() const { return bRunningUserConstructionScript; }

	/** Layers the actor belongs to.  This is outside of the editoronly data to allow hiding of LD-specified layers at runtime for profiling. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Actor)
	TArray< FName > Layers;

private:
#if WITH_EDITORONLY_DATA
	/** @deprecated Use ParentComponent instead */
	UPROPERTY()
	TWeakObjectPtr<AActor> ParentComponentActor_DEPRECATED;	
#endif

	/** The UChildActorComponent that owns this Actor. */
	UPROPERTY()
	TWeakObjectPtr<UChildActorComponent> ParentComponent;	

#if WITH_EDITORONLY_DATA
protected:
	/**
	 * The GUID for this actor; this guid will be the same for actors from instanced streaming levels.
	 * @see		ActorInstanceGuid, FActorInstanceGuidMapper
	 * @note	Don't use VisibleAnywhere here to avoid getting the CPF_Edit flag and get this property reset when resetting to defaults.
	 *			See FActorDetails::AddActorCategory and EditorUtilities::CopySingleProperty for details.
	 */
	UPROPERTY(BluePrintReadOnly, AdvancedDisplay, Category=Actor, NonPIEDuplicateTransient, TextExportTransient, NonTransactional)
	FGuid ActorGuid;

	/**
	 * The instance GUID for this actor; this guid will be unique for actors from instanced streaming levels.
	 * @see		ActorGuid
	 * @note	This is not guaranteed to be valid during PostLoad, but safe to access from RegisterAllComponents.
	 */
	UPROPERTY(BluePrintReadOnly, AdvancedDisplay, Category=Actor, Transient, NonPIEDuplicateTransient, TextExportTransient, NonTransactional)
	FGuid ActorInstanceGuid;

	/**
	 * The GUID for this actor's content bundle.
	 */
	UPROPERTY(BluePrintReadOnly, AdvancedDisplay, Category=Actor, TextExportTransient, NonTransactional)
	FGuid ContentBundleGuid;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use DataLayerAssets instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use DataLayerAssets instead."))
	TArray<FActorDataLayer> DataLayers;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	/** DataLayers assets the actor belongs to.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = DataLayers, meta=(DisableLevelInstancePropertyOverride))
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssets;

	TArray<TSoftObjectPtr<UDataLayerAsset>> PreEditChangeDataLayers;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = DataLayers, TextExportTransient)
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;

public:
	/** The copy/paste id used to remap actors during copy operations */
	uint32 CopyPasteId;

	/** The editor-only group this actor is a part of. */
	UPROPERTY(Transient)
	TObjectPtr<AActor> GroupActor;

	/** The scale to apply to any billboard components in editor builds (happens in any WITH_EDITOR build, including non-cooked games). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rendering, meta=(DisplayName="Editor Billboard Scale"))
	float SpriteScale;

	/** Bitflag to represent which views this actor is hidden in, via per-view layer visibility. */
	UPROPERTY(Transient)
	uint64 HiddenEditorViews;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/**
	 * Set the actor packaging mode.
	 * @param bExternal will set the actor packaging mode to external if true, to internal otherwise
	 * @param bShouldDirty should dirty or not the level package
	 * @param ActorExternalPackage if non-null and bExternal is true, will use the provided package instead of creating one
	 */
	ENGINE_API void SetPackageExternal(bool bExternal, bool bShouldDirty = true, UPackage* ActorExternalPackage = nullptr);

	/**
	 * Determine how this actor should be referenced by the level when external (saved in its own package).
	 / @return true if the level should keep a reference to the actor even if it's saved in its own package.
	 */
	virtual bool ShouldLevelKeepRefIfExternal() const { return false; }

	/**
	 * Whether this actor should be ignored when it is not loaded as part of the Main World (Persistent Level).
	 * eg. Will not be loaded through Level Instances.
	 */
	ENGINE_API bool IsMainWorldOnly() const;

	FActorOnPackagingModeChanged OnPackagingModeChanged;

	/** Returns this actor's current target runtime grid. */
	virtual FName GetRuntimeGrid() const { return RuntimeGrid; }

	/** Sets this actor's current target runtime grid. */
	void SetRuntimeGrid(FName InRuntimeGrid) { RuntimeGrid = InRuntimeGrid; }

	/** Gets the property name for RuntimeGrid. */
	static const FName GetRuntimeGridPropertyName()	{ return GET_MEMBER_NAME_CHECKED(AActor, RuntimeGrid); }

	/** Returns this actor's Guid. Actor Guids are only available in editor builds. */
	inline const FGuid& GetActorGuid() const { return ActorGuid; }

	/** Returns this actor's instance Guid. Actor Guids are only available in editor builds. */
	inline const FGuid& GetActorInstanceGuid() const { return ActorInstanceGuid.IsValid() ? ActorInstanceGuid : ActorGuid; }

	/** Returns this actor's content bundle Guid. */
	inline const FGuid& GetContentBundleGuid() const { return ContentBundleGuid; }

	/** Returns true if actor location should be locked. */
	ENGINE_API virtual bool IsLockLocation() const;

	/** Set the bLockLocation flag */
	void SetLockLocation(bool bInLockLocation) { bLockLocation = bInLockLocation; }

	/** Called on actor which initiated the PIE session */
	ENGINE_API virtual void OnPlayFromHere();

	bool CanPlayFromHere() const { return bCanPlayFromHere; }

	/**
	 * Creates an uninitialized actor descriptor from this actor. Meant to be called on the class CDO.
	 */
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const;

public:
	/**
	 * Creates an initialized actor descriptor from this actor.
	 */
	ENGINE_API TUniquePtr<class FWorldPartitionActorDesc> CreateActorDesc() const;

	/**
	 * Add properties to the actor desc.
	 */
	ENGINE_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const;

	/**
	 * Creates an uninitialized actor descriptor from a specific class.
	 */
	static ENGINE_API TUniquePtr<class FWorldPartitionActorDesc> StaticCreateClassActorDesc(const TSubclassOf<AActor>& ActorClass);

	/** Called when this actor gets added to the level in the editor by world partition loading. */
	virtual void OnLoadedActorAddedToLevel() {}

	/** Called when this actor gets removed from the level in the editor by world partition unloading. */
	virtual void OnLoadedActorRemovedFromLevel() {}
#endif // WITH_EDITOR

private:
#if WITH_EDITORONLY_DATA
	/**
	 * The friendly name for this actor, displayed in the editor.  You should always use AActor::GetActorLabel() to access the actual label to display,
	 * and call AActor::SetActorLabel() or FActorLabelUtilities::SetActorLabelUnique() to change the label.  Never set the label directly.
	 */
	UPROPERTY()
	FString ActorLabel;
#endif

#if !WITH_EDITORONLY_DATA && ACTOR_HAS_LABELS
	FString ActorLabel;
#endif

public:
	FString GetActorNameOrLabel() const
	{
#if WITH_EDITORONLY_DATA || ACTOR_HAS_LABELS
		if (!ActorLabel.IsEmpty())
		{
			return ActorLabel;
		}
#endif
		return GetName();
	}

	FStringView GetActorLabelView() const
	{
#if WITH_EDITORONLY_DATA || ACTOR_HAS_LABELS
		return ActorLabel;
#else
		return FStringView();
#endif
	}

#if WITH_EDITORONLY_DATA
private:
	/** 
	 * The folder path of this actor in the world.
	 * If the actor's level uses the actor folder objects feature, the path is computed using FolderGuid.
	 * If not, it contains the actual path (empty=root, / separated).
	 */
	UPROPERTY()
	FName FolderPath;

	/** If the actor's level uses the actor folder objects feature, contains the actor folder unique identifier (invalid=root). */
	UPROPERTY(TextExportTransient)
	FGuid FolderGuid;

public:
	/** Whether this actor is hidden within the editor viewport. */
	UPROPERTY()
	uint8 bHiddenEd:1;

	/** True if this actor is the preview actor dragged out of the content browser */
	UPROPERTY(Transient)
	uint8 bIsEditorPreviewActor:1;

	/** Whether this actor is hidden by the layer browser. */
	UPROPERTY(Transient)
	uint8 bHiddenEdLayer:1;

	/** Whether this actor is hidden by the level browser. */
	UPROPERTY(Transient)
	uint8 bHiddenEdLevel:1;

	/** If true during PostEditMove the construction script will be run every time. If false it will only run when the drag finishes. */
	uint8 bRunConstructionScriptOnDrag:1;

	/** Default expansion state for this actor. Some actors have attached children that we may not want to automatically expand by default */
	uint8 bDefaultOutlinerExpansionState : 1;

protected:
	/** If true, prevents the actor from being moved in the editor viewport. */
	UPROPERTY()
	uint8 bLockLocation:1;

	/** Is the actor label editable by the user? */
	UPROPERTY()
	uint8 bActorLabelEditable:1; 

	/** Whether the actor can be manipulated by editor operations. */
	UPROPERTY()
	uint8 bEditable:1;

	/** Whether this actor should be listed in the scene outliner. */
	UPROPERTY()
	uint8 bListedInSceneOutliner:1;

	/** Whether to cook additional data to speed up spawn events at runtime for any Blueprint classes based on this Actor. This option may slightly increase memory usage in a cooked build. */
	UPROPERTY(EditDefaultsOnly, AdvancedDisplay, Category=Cooking, meta=(DisplayName="Generate Optimized Blueprint Component Data"))
	uint8 bOptimizeBPComponentData:1;

	/** Whether the actor can be used as a PlayFromHere origin (OnPlayFromHere() will be called on that actor) */
	UPROPERTY()
	uint8 bCanPlayFromHere : 1;

	/** 
	 * Determine if this actor is spatially loaded when placed in a partitioned world.
	 *	If true, this actor will be loaded when in the range of any streaming sources and if (1) in no data layers, or (2) one or more of its data layers are enabled.
	 *	If false, this actor will be loaded if (1) in no data layers, or (2) one or more of its data layers are enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=WorldPartition)
	uint8 bIsSpatiallyLoaded : 1;

private:
	/** Whether this actor is temporarily hidden within the editor; used for show/hide/etc functionality w/o dirtying the actor. */
	UPROPERTY(Transient)
	uint8 bHiddenEdTemporary:1;

	UPROPERTY(Transient)
	uint8 bForceExternalActorLevelReferenceForPIE : 1;
#endif // WITH_EDITORONLY_DATA

public:
	/** Array of tags that can be used for grouping and categorizing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Actor)
	TArray<FName> Tags;


	//~==============================================================================================
	// Delegates
	
	/** Called when the actor is damaged in any way. */
	UPROPERTY(BlueprintAssignable, Category="Game|Damage")
	FTakeAnyDamageSignature OnTakeAnyDamage;

	/** Called when the actor is damaged by point damage. */
	UPROPERTY(BlueprintAssignable, Category="Game|Damage")
	FTakePointDamageSignature OnTakePointDamage;

	/** Called when the actor is damaged by radial damage. */
	UPROPERTY(BlueprintAssignable, Category="Game|Damage")
	FTakeRadialDamageSignature OnTakeRadialDamage;
	
	/** 
	 * Called when another actor begins to overlap this actor, for example a player walking into a trigger.
	 * For events when objects have a blocking collision, for example a player hitting a wall, see 'Hit' events.
	 * @note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FActorBeginOverlapSignature OnActorBeginOverlap;

	/** 
	 * Called when another actor stops overlapping this actor. 
	 * @note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FActorEndOverlapSignature OnActorEndOverlap;

	/** Called when the mouse cursor is moved over this actor if mouse over events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FActorBeginCursorOverSignature OnBeginCursorOver;

	/** Called when the mouse cursor is moved off this actor if mouse over events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FActorEndCursorOverSignature OnEndCursorOver;

	/** Called when the left mouse button is clicked while the mouse is over this actor and click events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FActorOnClickedSignature OnClicked;

	/** Called when the left mouse button is released while the mouse is over this actor and click events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Mouse Input")
	FActorOnReleasedSignature OnReleased;

	/** Called when a touch input is received over this actor when touch events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FActorOnInputTouchBeginSignature OnInputTouchBegin;
		
	/** Called when a touch input is received over this component when touch events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FActorOnInputTouchEndSignature OnInputTouchEnd;

	/** Called when a finger is moved over this actor when touch over events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FActorBeginTouchOverSignature OnInputTouchEnter;

	/** Called when a finger is moved off this actor when touch over events are enabled in the player controller. */
	UPROPERTY(BlueprintAssignable, Category="Input|Touch Input")
	FActorEndTouchOverSignature OnInputTouchLeave;

	/** 
	 *	Called when this Actor hits (or is hit by) something solid. This could happen due to things like Character movement, using Set Location with 'sweep' enabled, or physics simulation.
	 *	For events when objects overlap (e.g. walking into a trigger) see the 'Overlap' event.
	 *	@note For collisions during physics simulation to generate hit events, 'Simulation Generates Hit Events' must be enabled.
	 */
	UPROPERTY(BlueprintAssignable, Category="Collision")
	FActorHitSignature OnActorHit;

	/** 
	 * Pushes this actor on to the stack of input being handled by a PlayerController.
	 * @param PlayerController The PlayerController whose input events we want to receive.
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	ENGINE_API virtual void EnableInput(class APlayerController* PlayerController);

	/** 
	 * Creates an input component from the input component passed in
	 * @param InputComponentToCreate The UInputComponent to create.
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	ENGINE_API virtual void CreateInputComponent(TSubclassOf<UInputComponent> InputComponentToCreate);

	/** 
	 * Removes this actor from the stack of input being handled by a PlayerController.
	 * @param PlayerController The PlayerController whose input events we no longer want to receive. If null, this actor will stop receiving input from all PlayerControllers.
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	ENGINE_API virtual void DisableInput(class APlayerController* PlayerController);

	/** Gets the value of the input axis if input is enabled for this actor. */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", HideSelfPin="true", HidePin="InputAxisName"))
	ENGINE_API float GetInputAxisValue(const FName InputAxisName) const;

	/** Gets the value of the input axis key if input is enabled for this actor. */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", HideSelfPin="true", HidePin="InputAxisKey"))
	ENGINE_API float GetInputAxisKeyValue(const FKey InputAxisKey) const;

	/** Gets the value of the input axis key if input is enabled for this actor. */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", HideSelfPin="true", HidePin="InputAxisKey"))
	ENGINE_API FVector GetInputVectorAxisValue(const FKey InputAxisKey) const;

	/** Returns the instigator for this actor, or nullptr if there is none. */
	UFUNCTION(BlueprintCallable, meta=(BlueprintProtected = "true"), Category="Game")
	ENGINE_API APawn* GetInstigator() const;

	/**
	 * Get the instigator, cast as a specific class.
	 * @return The instigator for this actor if it is the specified type, nullptr otherwise.
	 */
	template <class T>
	T* GetInstigator() const
	{
		return Cast<T>(GetInstigator());
	}

	/** Returns the instigator's controller for this actor, or nullptr if there is none. */
	UFUNCTION(BlueprintCallable, meta=(BlueprintProtected = "true"), Category="Game")
	ENGINE_API AController* GetInstigatorController() const;

	/** 
	 * Returns the instigator's controller, cast as a specific class.
	 * @return The instigator's controller for this actor if it is the specified type, nullptr otherwise.
	 * */
	template<class T>
	T* GetInstigatorController() const
	{
		return Cast<T>(GetInstigatorController());
	}

	//~=============================================================================
	// DataLayers functions.
#if WITH_EDITOR
private:
	virtual bool ActorTypeIsMainWorldOnly() const { return false; }
	ENGINE_API TArray<const UDataLayerAsset*> ResolveDataLayerAssets(const TArray<TSoftObjectPtr<UDataLayerAsset>>& InDataLayerAssets) const;
public:
	ENGINE_API bool AddDataLayer(const UDataLayerInstance* DataLayerInstance);
	ENGINE_API bool RemoveDataLayer(const UDataLayerInstance* DataLayerInstance);
	ENGINE_API bool CanAddDataLayer(const UDataLayerInstance* InDataLayerInstance, FText* OutReason = nullptr) const;
	ENGINE_API void ForceLoadDataLayerAssets() const;

	ENGINE_API TArray<const UDataLayerInstance*> RemoveAllDataLayers();
	ENGINE_API bool SupportsDataLayerType(TSubclassOf<UDataLayerInstance> DataLayerType) const;
	
	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstancesForLevel() const;
	ENGINE_API TArray<FName> GetDataLayerInstanceNames() const;
	ENGINE_API bool IsPropertyChangedAffectingDataLayers(FPropertyChangedEvent& PropertyChangedEvent) const;
	ENGINE_API void FixupDataLayers(bool bRevertChangesOnLockedDataLayer = false);
	static const FName GetDataLayerAssetsPropertyName() { return GET_MEMBER_NAME_CHECKED(AActor, DataLayerAssets); }
	static const FName GetExternalDataLayerAssetPropertyName() { return GET_MEMBER_NAME_CHECKED(AActor, ExternalDataLayerAsset); }

	UE_DEPRECATED(5.8, "FActorDataLayer is no longer in use.")
	static const FName GetDataLayerPropertyName() { return FName(); }

	ENGINE_API TArray<const UDataLayerAsset*> GetDataLayerAssets(bool bIncludeExternalDataLayerAsset = true) const;
	ENGINE_API bool HasExternalContent() const;

	virtual bool ActorTypeSupportsDataLayer() const { return true; }
	virtual bool ActorTypeSupportsExternalDataLayer() const { return true; }

	// This getter is kept solely to enable UDataLayerToAssetCommandlet for conversions of old data
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "Convert DataLayer using UDataLayerToAssetCommandlet and use GetDataLayerAssets() instead")
	TArray<FActorDataLayer> const& GetActorDataLayers() const { return DataLayers; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif

	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstances() const;

	ENGINE_API bool ContainsDataLayer(const UDataLayerAsset* DataLayerAsset) const;
	ENGINE_API bool ContainsDataLayer(const UDataLayerInstance* DataLayerInstance) const;
	ENGINE_API bool HasDataLayers() const;
	ENGINE_API bool HasContentBundle() const;
	ENGINE_API const UExternalDataLayerAsset* GetExternalDataLayerAsset() const;

private:
	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstancesInternal(bool bUseLevelContext, bool bIncludeParentDataLayers = true) const;
	ENGINE_API const IWorldPartitionCell* GetWorldPartitionRuntimeCell() const;

	/** Whether actor supports incremental pre-register of components. */
	virtual bool SupportsIncrementalPreRegisterComponents() const
	{
		return true;
	}

	/** Whether actor supports incremental pre-unregister of components. */
	ENGINE_API virtual bool SupportsIncrementalPreUnregisterComponents() const;
	
	/** Incrementally pre-registers components associated with this actor (used during level streaming). */
	ENGINE_API void IncrementalPreRegisterComponents();

	/** Returns bHasPreRegisteredAllComponents which indicates whether this actor has pre-registered all its components. */
	bool HasPreRegisteredAllComponents() const { return bHasPreRegisteredAllComponents; }

	/** Incrementally pre-unregisters components associated with this actor (used during level streaming). */
	ENGINE_API void IncrementalPreUnregisterComponents();

	//~=============================================================================
	// General functions.
public:
	/**
	 * Get the actor-to-world transform.
	 * @return The transform that transforms from actor space to world space.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Actor Transform", ScriptName = "GetActorTransform"), Category="Transformation")
	const FTransform& GetTransform() const
	{
		return ActorToWorld();
	}

	/** Get the local-to-world transform of the RootComponent. Identical to GetTransform(). */
	inline const FTransform& ActorToWorld() const
	{
		return (RootComponent ? RootComponent->GetComponentTransform() : FTransform::Identity);
	}

	/** Returns the location of the RootComponent of this Actor */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Actor Location", ScriptName = "GetActorLocation", Keywords="position"), Category="Transformation")
	FVector K2_GetActorLocation() const;

	/** 
	 * Move the Actor to the specified location.
	 * @param NewLocation	The new location to move the Actor to.
	 * @param bSweep		Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *						Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport		Whether we teleport the physics state (if physics collision is enabled for this object).
	 *						If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *						If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *						If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                      Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *                      Setting the location without teleporting will not update the location of simulated child/attached components.
	 * @param SweepHitResult	The hit result from the move if swept.
	 * @return	Whether the location was successfully set (if not swept), or whether movement occurred at all (if swept).
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Set Actor Location", ScriptName = "SetActorLocation", Keywords="position"), Category="Transformation")
	ENGINE_API bool K2_SetActorLocation(FVector NewLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);

	/** Returns rotation of the RootComponent of this Actor. */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Actor Rotation", ScriptName = "GetActorRotation"), Category="Transformation")
	FRotator K2_GetActorRotation() const;

	/** Get the forward (X) vector (length 1.0) from this Actor, in world space.  */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	FVector GetActorForwardVector() const;

	/** Get the up (Z) vector (length 1.0) from this Actor, in world space.  */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	FVector GetActorUpVector() const;

	/** Get the right (Y) vector (length 1.0) from this Actor, in world space.  */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	FVector GetActorRightVector() const;

	/**
	 * Returns the bounding box of all components that make up this Actor (excluding ChildActorComponents).
	 * @param	bOnlyCollidingComponents	If true, will only return the bounding box for components with collision enabled.
	 * @param	Origin						Set to the center of the actor in world space
	 * @param	BoxExtent					Set to half the actor's size in 3d space
	 * @param	bIncludeFromChildActors		If true then recurse in to ChildActor components 
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(DisplayName = "Get Actor Bounds"))
	ENGINE_API virtual void GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors = false) const;

	/** Returns the RootComponent of this Actor */
	UFUNCTION(BlueprintGetter)
	USceneComponent* K2_GetRootComponent() const;

	/** Returns velocity (in cm/s (Unreal Units/second) of the rootcomponent if it is either using physics or has an associated MovementComponent */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API virtual FVector GetVelocity() const;

	/** 
	 * Move the actor instantly to the specified location. 
	 * 
	 * @param NewLocation	The new location to teleport the Actor to.
	 * @param bSweep		Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *						Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param Teleport		How we teleport the physics state (if physics collision is enabled for this object).
	 *						If equal to ETeleportType::TeleportPhysics, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *						If equal to ETeleportType::None, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *						If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                      Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *                      Setting the location without teleporting will not update the location of simulated child/attached components.
	 * @param OutSweepHitResult The hit result from the move if swept.
	 * @return	Whether the location was successfully set if not swept, or whether movement occurred if swept.
	 */
	ENGINE_API bool SetActorLocation(const FVector& NewLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** 
	 * Set the Actor's rotation instantly to the specified rotation.
	 * 
	 * @param	NewRotation	The new rotation for the Actor.
	 * @param	bTeleportPhysics Whether we teleport the physics state (if physics collision is enabled for this object).
	 *			If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *			If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *          Setting the rotation without teleporting will not update the rotation of simulated child/attached components.
	 * @return	Whether the rotation was successfully set.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Set Actor Rotation", ScriptName = "SetActorRotation"), Category="Transformation")
	ENGINE_API bool K2_SetActorRotation(FRotator NewRotation, bool bTeleportPhysics);
	
	/**
	 * Set the Actor's rotation instantly to the specified rotation.
	 *
	 * @param	NewRotation	The new rotation for the Actor.
	 * @param	Teleport	How we teleport the physics state (if physics collision is enabled for this object).
	 *						If equal to ETeleportType::TeleportPhysics, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *						If equal to ETeleportType::None, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *                      Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                      Setting the rotation without teleporting will not update the rotation of simulated child/attached components.
	 * @return	Whether the rotation was successfully set.
	 */
	ENGINE_API bool SetActorRotation(FRotator NewRotation, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API bool SetActorRotation(const FQuat& NewRotation, ETeleportType Teleport = ETeleportType::None);

	/** 
	 * Move the actor instantly to the specified location and rotation.
	 * 
	 * @param NewLocation		The new location to teleport the Actor to.
	 * @param NewRotation		The new rotation for the Actor.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *                          Setting the location without teleporting will not update the location of simulated child/attached components.
	 * @param SweepHitResult	The hit result from the move if swept.
	 * @return	Whether the rotation was successfully set.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Actor Location And Rotation", ScriptName="SetActorLocationAndRotation"))
	ENGINE_API bool K2_SetActorLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	
	/** 
	 * Move the actor instantly to the specified location and rotation.
	 * 
	 * @param NewLocation		The new location to teleport the Actor to.
	 * @param NewRotation		The new rotation for the Actor.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param Teleport			How we teleport the physics state (if physics collision is enabled for this object).
	 *							If equal to ETeleportType::TeleportPhysics, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If equal to ETeleportType::None, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the location without teleporting will not update the location of simulated child/attached components.
	 * @param OutSweepHitResult	The hit result from the move if swept.
	 * @return	Whether the rotation was successfully set.
	 */
	ENGINE_API bool SetActorLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API bool SetActorLocationAndRotation(FVector NewLocation, const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Set the Actor's world-space scale. */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void SetActorScale3D(FVector NewScale3D);

	/** Returns the Actor's world-space scale. */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FVector GetActorScale3D() const;

	/** Returns the distance from this Actor to OtherActor. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetDistanceTo(const AActor* OtherActor) const;

	/** Returns the squared distance from this Actor to OtherActor. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetSquaredDistanceTo(const AActor* OtherActor) const;

	/** Returns the distance from this Actor to OtherActor, ignoring Z. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetHorizontalDistanceTo(const AActor* OtherActor) const;

	/** Returns the squared distance from this Actor to OtherActor, ignoring Z. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetSquaredHorizontalDistanceTo(const AActor* OtherActor) const;

	/** Returns the distance from this Actor to OtherActor, ignoring XY. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetVerticalDistanceTo(const AActor* OtherActor) const;

	/** Returns the dot product from this Actor to OtherActor. Returns -2.0 on failure. Returns 0.0 for coincidental actors. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetDotProductTo(const AActor* OtherActor) const;

	/** Returns the dot product from this Actor to OtherActor, ignoring Z. Returns -2.0 on failure. Returns 0.0 for coincidental actors. */
	UFUNCTION(BlueprintCallable, Category = "Transformation")
	ENGINE_API float GetHorizontalDotProductTo(const AActor* OtherActor) const;

	/**
	 * Adds a delta to the location of this actor in world space.
	 * 
	 * @param DeltaLocation		The change in location.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the location without teleporting will not update the location of simulated child/attached components.
	 * @param SweepHitResult	The hit result from the move if swept.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor World Offset", ScriptName="AddActorWorldOffset", Keywords="location position"))
	ENGINE_API void K2_AddActorWorldOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorWorldOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the rotation of this actor in world space.
	 * 
	 * @param DeltaRotation		The change in rotation.
	 * @param bSweep			Whether to sweep to the target rotation (not currently supported for rotation).
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the rotation without teleporting will not update the rotation of simulated child/attached components.
	 * @param SweepHitResult	The hit result from the move if swept.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor World Rotation", ScriptName="AddActorWorldRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_AddActorWorldRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorWorldRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void AddActorWorldRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Adds a delta to the transform of this actor in world space. Ignores scale and sets it to (1,1,1). */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor World Transform", ScriptName="AddActorWorldTransform"))
	ENGINE_API void K2_AddActorWorldTransform(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorWorldTransform(const FTransform& DeltaTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** Adds a delta to the transform of this actor in world space. Scale is unchanged. */
	UFUNCTION(BlueprintCallable, Category = "Transformation", meta = (DisplayName = "Add Actor World Transform Keep Scale", ScriptName = "AddActorWorldTransformKeepScale"))
	ENGINE_API void K2_AddActorWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorWorldTransformKeepScale(const FTransform& DeltaTransform, bool bSweep = false, FHitResult* OutSweepHitResult = nullptr, ETeleportType Teleport = ETeleportType::None);

	/** 
	 * Set the Actors transform to the specified one.
	 * @param NewTransform		The new transform.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the transform without teleporting will not update the transform of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Actor Transform", ScriptName="SetActorTransform"))
	ENGINE_API bool K2_SetActorTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API bool SetActorTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/** 
	 * Adds a delta to the location of this component in its local reference frame.
	 * @param DelatLocation		The change in location in local space.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the location without teleporting will not update the location of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor Local Offset", ScriptName="AddActorLocalOffset", Keywords="location position"))
	ENGINE_API void K2_AddActorLocalOffset(FVector DeltaLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorLocalOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the rotation of this component in its local reference frame
	 * @param DeltaRotation		The change in rotation in local space.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the rotation without teleporting will not update the rotation of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor Local Rotation", ScriptName="AddActorLocalRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_AddActorLocalRotation(FRotator DeltaRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorLocalRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void AddActorLocalRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Adds a delta to the transform of this component in its local reference frame
	 * @param NewTransform		The change in transform in local space.
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                          Setting the transform without teleporting will not update the transform of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Add Actor Local Transform", ScriptName="AddActorLocalTransform"))
	ENGINE_API void K2_AddActorLocalTransform(const FTransform& NewTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void AddActorLocalTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the actor's RootComponent to the specified relative location.
	 * @param NewRelativeLocation	New relative location of the actor's root component
	 * @param bSweep				Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *								Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport				Whether we teleport the physics state (if physics collision is enabled for this object).
	 *								If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *								If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *								If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                              Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *                              Setting the location without teleporting will not update the location of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Actor Relative Location", ScriptName="SetActorRelativeLocation"))
	ENGINE_API void K2_SetActorRelativeLocation(FVector NewRelativeLocation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetActorRelativeLocation(FVector NewRelativeLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the actor's RootComponent to the specified relative rotation
	 * @param NewRelativeRotation	New relative rotation of the actor's root component
	 * @param bSweep				Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *								Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport				Whether we teleport the physics state (if physics collision is enabled for this object).
	 *								If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *								If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *								If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                              Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated.
	 *                              Setting the rotation without teleporting will not update the rotation of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Actor Relative Rotation", ScriptName="SetActorRelativeRotation", AdvancedDisplay="bSweep,SweepHitResult,bTeleport"))
	ENGINE_API void K2_SetActorRelativeRotation(FRotator NewRelativeRotation, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetActorRelativeRotation(FRotator NewRelativeRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);
	ENGINE_API void SetActorRelativeRotation(const FQuat& NewRelativeRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the actor's RootComponent to the specified relative transform
	 * @param NewRelativeTransform		New relative transform of the actor's root component
	 * @param bSweep			Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 *							Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	 * @param bTeleport			Whether we teleport the physics state (if physics collision is enabled for this object).
	 *							If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 *							If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 *							If CCD is on and not teleporting, this will affect objects along the entire swept volume.
	 *                          Note that when teleporting, any child/attached components will be teleported too, maintaining their current offset even if they are being simulated. 
	 *                          Setting the transform without teleporting will not update the transform of simulated child/attached components.
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation", meta=(DisplayName="Set Actor Relative Transform", ScriptName="SetActorRelativeTransform"))
	ENGINE_API void K2_SetActorRelativeTransform(const FTransform& NewRelativeTransform, bool bSweep, FHitResult& SweepHitResult, bool bTeleport);
	ENGINE_API void SetActorRelativeTransform(const FTransform& NewRelativeTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Set the actor's RootComponent to the specified relative scale 3d
	 * @param NewRelativeScale	New scale to set the actor's RootComponent to
	 */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API void SetActorRelativeScale3D(FVector NewRelativeScale);

	/** Return the actor's relative scale 3d */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	ENGINE_API FVector GetActorRelativeScale3D() const;

	/**
	 * Marks the bounds of all SceneComponents attached to this actor which have `bComputeBoundsOnceForGame` as needing to be recomputed the next time UpdateBounds is called. 
	 * This might be necessary if the bounds that were cached on cook no longer reflect the actor's transform (ex. if level transform is applied).
	 */
	ENGINE_API void MarkNeedsRecomputeBoundsOnceForGame();

	/**
	 *	Sets the actor to be hidden in the game
	 *	@param	bNewHidden	Whether or not to hide the actor and all its components
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=( DisplayName = "Set Actor Hidden In Game", Keywords = "Visible Hidden Show Hide" ))
	ENGINE_API virtual void SetActorHiddenInGame(bool bNewHidden);

	/** Allows enabling/disabling collision for the whole actor */
	UFUNCTION(BlueprintCallable, Category="Collision")
	ENGINE_API void SetActorEnableCollision(bool bNewActorEnableCollision);

	/** Get current state of collision for the whole actor */
	UFUNCTION(BlueprintPure, Category="Collision")
	bool GetActorEnableCollision() const;

	/** Destroy the actor */
	UFUNCTION(BlueprintCallable, Category="Actor", meta=(Keywords = "Delete", DisplayName = "Destroy Actor", ScriptName = "DestroyActor"))
	ENGINE_API virtual void K2_DestroyActor();

	/** Returns whether this actor has network authority */
	UFUNCTION(BlueprintCallable, Category="Networking")
	bool HasAuthority() const;

	/** 
	 * Creates a new component and assigns ownership to the Actor this is 
	 * called for. Automatic attachment causes the first component created to 
	 * become the root, and all subsequent components to be attached under that 
	 * root. When bManualAttachment is set, automatic attachment is 
	 * skipped and it is up to the user to attach the resulting component (or 
	 * set it up as the root) themselves.
	 *
	 * @see UK2Node_AddComponent	DO NOT CALL MANUALLY - BLUEPRINT INTERNAL USE ONLY (for Add Component nodes)
	 *
	 * @param TemplateName					The name of the Component Template to use.
	 * @param bManualAttachment				Whether manual or automatic attachment is to be used
	 * @param RelativeTransform				The relative transform between the new component and its attach parent (automatic only)
	 * @param ComponentTemplateContext		Optional UBlueprintGeneratedClass reference to use to find the template in. If null (or not a BPGC), component is sought in this Actor's class
	 * @param bDeferredFinish				Whether or not to immediately complete the creation and registration process for this component. Will be false if there are expose on spawn properties being set
	 */
	UFUNCTION(BlueprintCallable, meta=(ScriptNoExport, BlueprintInternalUseOnly = "true", DefaultToSelf="ComponentTemplateContext", InternalUseParam="ComponentTemplateContext,bDeferredFinish"))
	ENGINE_API UActorComponent* AddComponent(FName TemplateName, bool bManualAttachment, const FTransform& RelativeTransform, const UObject* ComponentTemplateContext, bool bDeferredFinish = false);

	/**
	 * Creates a new component and assigns ownership to the Actor this is
	 * called for. Automatic attachment causes the first component created to
	 * become the root, and all subsequent components to be attached under that
	 * root. When bManualAttachment is set, automatic attachment is
	 * skipped and it is up to the user to attach the resulting component (or
	 * set it up as the root) themselves.
	 *
	 * @see UK2Node_AddComponentByClass		DO NOT CALL MANUALLY - BLUEPRINT INTERNAL USE ONLY (for Add Component nodes)
	 *
	 * @param Class						The class of component to create
	 * @param bManualAttachment				Whether manual or automatic attachment is to be used
	 * @param RelativeTransform				The relative transform between the new component and its attach parent (automatic only)
	 * @param bDeferredFinish				Whether or not to immediately complete the creation and registration process for this component. Will be false if there are expose on spawn properties being set
	 */
	UFUNCTION(BlueprintCallable, meta=(ScriptNoExport, BlueprintInternalUseOnly = "true", InternalUseParam="bDeferredFinish"))
	ENGINE_API UActorComponent* AddComponentByClass(UPARAM(meta = (AllowAbstract = "false")) TSubclassOf<UActorComponent> Class, bool bManualAttachment, const FTransform& RelativeTransform, bool bDeferredFinish);

	/** 
	 * Completes the creation of a new actor component. Called either from blueprint after
	 * expose on spawn properties are set, or directly from AddComponent
	 *
	 * @see UK2Node_AddComponent	DO NOT CALL MANUALLY - BLUEPRINT INTERNAL USE ONLY (for Add Component nodes)
	 *
	 * @param Component						The component created in AddComponent to finish creation of
	 * @param bManualAttachment				Whether manual or automatic attachment is to be used
	 * @param RelativeTransform				The relative transform between the new component and its attach parent (automatic only)
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true"))
	ENGINE_API void FinishAddComponent(UActorComponent* Component, bool bManualAttachment, const FTransform& RelativeTransform);

	/** DEPRECATED - Use AttachToComponent() instead */
	UE_DEPRECATED(4.17, "Use AttachToComponent() instead.")
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "AttachRootComponentTo (Deprecated)", ScriptNoExport, AttachLocationType = "KeepRelativeOffset"), Category = "Transformation")
	ENGINE_API void K2_AttachRootComponentTo(USceneComponent* InParent, FName InSocketName = NAME_None, EAttachLocation::Type AttachLocationType = EAttachLocation::KeepRelativeOffset, bool bWeldSimulatedBodies = true);

	/**
	 * Attaches the RootComponent of this Actor to the supplied component, optionally at a named socket. It is not valid to call this on components that are not Registered.
	 * @param Parent					Parent to attach to.
	 * @param SocketName				Optional socket to attach to on the parent.
	 * @param LocationRule				How to handle translation when attaching.
	 * @param RotationRule				How to handle rotation when attaching.
	 * @param ScaleRule					How to handle scale when attaching.
	 * @param bWeldSimulatedBodies		Whether to weld together simulated physics bodies. This transfers the shapes in the welded object into the parent (if simulated), which can result in permanent changes that persist even after subsequently detaching.
	 * @return							Whether the attachment was successful or not
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Attach Actor To Component", ScriptName = "AttachToComponent", bWeldSimulatedBodies = true), Category = "Transformation")
	ENGINE_API bool K2_AttachToComponent(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule, bool bWeldSimulatedBodies);

	/**
	 * Attaches the RootComponent of this Actor to the supplied component, optionally at a named socket. It is not valid to call this on components that are not Registered.
	 * @param  Parent					Parent to attach to.
	 * @param  AttachmentRules			How to handle transforms and welding when attaching.
	 * @param  SocketName				Optional socket to attach to on the parent.
	 * @return							Whether the attachment was successful or not
	 */
	ENGINE_API bool AttachToComponent(USceneComponent* Parent, const FAttachmentTransformRules& AttachmentRules, FName SocketName = NAME_None);

	/** DEPRECATED - Use AttachToActor() instead */
	UE_DEPRECATED(4.17, "Use AttachToActor() instead.")
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "AttachRootComponentToActor (Deprecated)", ScriptNoExport, AttachLocationType = "KeepRelativeOffset"), Category = "Transformation")
	ENGINE_API void K2_AttachRootComponentToActor(AActor* InParentActor, FName InSocketName = NAME_None, EAttachLocation::Type AttachLocationType = EAttachLocation::KeepRelativeOffset, bool bWeldSimulatedBodies = true);

	/**
	 * Attaches the RootComponent of this Actor to the RootComponent of the supplied actor, optionally at a named socket.
	 * @param ParentActor				Actor to attach this actor's RootComponent to
	 * @param AttachmentRules			How to handle transforms and modification when attaching.
	 * @param SocketName				Socket name to attach to, if any
	 * @return							Whether the attachment was successful or not
	 */
	ENGINE_API bool AttachToActor(AActor* ParentActor, const FAttachmentTransformRules& AttachmentRules, FName SocketName = NAME_None);

	/**
	 * Attaches the RootComponent of this Actor to the supplied actor, optionally at a named socket.
	 * @param ParentActor				Actor to attach this actor's RootComponent to
	 * @param SocketName				Socket name to attach to, if any
	 * @param LocationRule				How to handle translation when attaching.
	 * @param RotationRule				How to handle rotation when attaching.
	 * @param ScaleRule					How to handle scale when attaching.
	 * @param bWeldSimulatedBodies		Whether to weld together simulated physics bodies.This transfers the shapes in the welded object into the parent (if simulated), which can result in permanent changes that persist even after subsequently detaching.
	 * @return							Whether the attachment was successful or not
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Attach Actor To Actor", ScriptName = "AttachToActor", bWeldSimulatedBodies=true), Category = "Transformation")
	ENGINE_API bool K2_AttachToActor(AActor* ParentActor, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule, bool bWeldSimulatedBodies);

	/** DEPRECATED - Use DetachFromActor() instead */
	UE_DEPRECATED(4.17, "Use DetachFromActor() instead")
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "DetachActorFromActor (Deprecated)", ScriptNoExport), Category="Transformation")
	ENGINE_API void DetachRootComponentFromParent(bool bMaintainWorldPosition = true);

	/** 
	 * Detaches the RootComponent of this Actor from any SceneComponent it is currently attached to. 
	 * @param  LocationRule				How to handle translation when detaching.
	 * @param  RotationRule				How to handle rotation when detaching.
	 * @param  ScaleRule				How to handle scale when detaching.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Detach From Actor", ScriptName = "DetachFromActor"), Category="Transformation")
	ENGINE_API void K2_DetachFromActor(EDetachmentRule LocationRule = EDetachmentRule::KeepRelative, EDetachmentRule RotationRule = EDetachmentRule::KeepRelative, EDetachmentRule ScaleRule = EDetachmentRule::KeepRelative);

	/** 
	 * Detaches the RootComponent of this Actor from any SceneComponent it is currently attached to. 
	 * @param  DetachmentRules			How to handle transforms when detaching.
	 */
	ENGINE_API void DetachFromActor(const FDetachmentTransformRules& DetachmentRules);

	/**
	 *	Detaches all SceneComponents in this Actor from the supplied parent SceneComponent.
	 *	@param InParentComponent		SceneComponent to detach this actor's components from
	 *	@param DetachmentRules			Rules to apply when detaching components
	 */
	ENGINE_API void DetachAllSceneComponents(class USceneComponent* InParentComponent, const FDetachmentTransformRules& DetachmentRules);

	/** See if this actor's Tags array contains the supplied name tag */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API bool ActorHasTag(FName Tag) const;


	//~==============================================================================
	// Misc Blueprint support

	/** 
	 * Get ActorTimeDilation - this can be used for input control or speed control for slomo.
	 * We don't want to scale input globally because input can be used for UI, which do not care for TimeDilation.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API float GetActorTimeDilation() const;

	/**
	 * More efficient version that takes the Actor's current world.
	 */
	ENGINE_API float GetActorTimeDilation(const UWorld& ActorWorld) const;

	/** Make this actor tick after PrerequisiteActor. This only applies to this actor's tick function; dependencies for owned components must be set up separately if desired. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void AddTickPrerequisiteActor(AActor* PrerequisiteActor);

	/** Make this actor tick after PrerequisiteComponent. This only applies to this actor's tick function; dependencies for owned components must be set up separately if desired. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void AddTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent);

	/** Remove tick dependency on PrerequisiteActor. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void RemoveTickPrerequisiteActor(AActor* PrerequisiteActor);

	/** Remove tick dependency on PrerequisiteComponent. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick", meta=(Keywords = "dependency"))
	ENGINE_API virtual void RemoveTickPrerequisiteComponent(UActorComponent* PrerequisiteComponent);
	
	/** Gets whether this actor can tick when paused. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API bool GetTickableWhenPaused();

	/** Sets whether this actor can tick when paused. */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API void SetTickableWhenPaused(bool bTickableWhenPaused);

	/** The number of seconds (in game time) since this Actor was created, relative to Get Game Time In Seconds. */
	UFUNCTION(BlueprintPure, Category=Actor)
	ENGINE_API float GetGameTimeSinceCreation() const;

protected:
	/** Event when play begins for this actor. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "BeginPlay"))
	ENGINE_API void ReceiveBeginPlay();

	/** Overridable native event for when play begins for this actor. */
	ENGINE_API virtual void BeginPlay();

	/** Event to notify blueprints this actor is being deleted or removed from a level. */
	UFUNCTION(BlueprintImplementableEvent, meta=(Keywords = "delete", DisplayName = "End Play"))
	ENGINE_API void ReceiveEndPlay(EEndPlayReason::Type EndPlayReason);

	/** Overridable function called whenever this actor is being removed from a level */
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason);

public:
	/** Initiate a begin play call on this Actor, will handle calling in the correct order. */
	ENGINE_API void DispatchBeginPlay(bool bFromLevelStreaming = false);

	/** Returns whether an actor has been initialized for gameplay */
	bool IsActorInitialized() const { return bActorInitialized; }

	/** Returns whether an actor is in the process of beginning play */
	bool IsActorBeginningPlay() const { return ActorHasBegunPlay == EActorBeginPlayState::BeginningPlay; }

	/** Returns whether an actor has had BeginPlay called on it (and not subsequently had EndPlay called) */
	bool HasActorBegunPlay() const { return ActorHasBegunPlay == EActorBeginPlayState::HasBegunPlay; }

	/** Returns whether an actor is beginning play in DispatchBeginPlay() during level streaming (which includes initial level load). */
	bool IsActorBeginningPlayFromLevelStreaming() const { return bActorBeginningPlayFromLevelStreaming; }

	/** Returns true if this actor is currently being destroyed, some gameplay events may be unsafe */
	UFUNCTION(BlueprintCallable, Category="Game")
	bool IsActorBeingDestroyed() const 
	{
		return bActorIsBeingDestroyed;
	}

	/** Returns bHasRegisteredAllComponents which indicates whether this actor has registered all their components without 
	 *	unregisatering all of them them.
	 *	bHasRegisteredAllComponents is set true just before PostRegisterAllComponents() is called and false just before PostUnregisterAllComponents() is called.
	 */ 
	bool HasActorRegisteredAllComponents() const { return bHasRegisteredAllComponents; }

	/** Sets bHasRegisteredAllComponents true. bHasRegisteredAllComponents must be set true just prior to calling PostRegisterAllComponents(). */
	void SetHasActorRegisteredAllComponents() { bHasRegisteredAllComponents = true; }

	/** Event when this actor takes ANY damage */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly, meta=(DisplayName = "AnyDamage"), Category="Game|Damage")
	ENGINE_API void ReceiveAnyDamage(float Damage, const class UDamageType* DamageType, class AController* InstigatedBy, AActor* DamageCauser);
	
	/** Event when this actor takes RADIAL damage */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly, meta=(DisplayName = "RadialDamage"), Category="Game|Damage")
	ENGINE_API void ReceiveRadialDamage(float DamageReceived, const class UDamageType* DamageType, FVector Origin, const struct FHitResult& HitInfo, class AController* InstigatedBy, AActor* DamageCauser);

	/** Event when this actor takes POINT damage */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly, meta=(DisplayName = "PointDamage"), Category="Game|Damage")
	ENGINE_API void ReceivePointDamage(float Damage, const class UDamageType* DamageType, FVector HitLocation, FVector HitNormal, class UPrimitiveComponent* HitComponent, FName BoneName, FVector ShotFromDirection, class AController* InstigatedBy, AActor* DamageCauser, const FHitResult& HitInfo);

	/** Event called every frame, if ticking is enabled */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Tick"))
	ENGINE_API void ReceiveTick(float DeltaSeconds);

	/** Event called every physics tick if bAsyncPhysicsTickEnabled is true */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Async Physics Tick"))
	ENGINE_API void ReceiveAsyncPhysicsTick(float DeltaSeconds, float SimSeconds);

	/** 
	 *	Event when this actor overlaps another actor, for example a player walking into a trigger.
	 *	For events when objects have a blocking collision, for example a player hitting a wall, see 'Hit' events.
	 *	@note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	ENGINE_API virtual void NotifyActorBeginOverlap(AActor* OtherActor);
	/** 
	 *	Event when this actor overlaps another actor, for example a player walking into a trigger.
	 *	For events when objects have a blocking collision, for example a player hitting a wall, see 'Hit' events.
	 *	@note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorBeginOverlap"), Category="Collision")
	ENGINE_API void ReceiveActorBeginOverlap(AActor* OtherActor);

	/** 
	 *	Event when an actor no longer overlaps another actor, and they have separated. 
	 *	@note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	ENGINE_API virtual void NotifyActorEndOverlap(AActor* OtherActor);
	/** 
	 *	Event when an actor no longer overlaps another actor, and they have separated. 
	 *	@note Components on both this and the other Actor must have bGenerateOverlapEvents set to true to generate overlap events.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorEndOverlap"), Category="Collision")
	ENGINE_API void ReceiveActorEndOverlap(AActor* OtherActor);

	/** Event when this actor has the mouse moved over it with the clickable interface. */
	ENGINE_API virtual void NotifyActorBeginCursorOver();
	/** Event when this actor has the mouse moved over it with the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorBeginCursorOver"), Category="Mouse Input")
	ENGINE_API void ReceiveActorBeginCursorOver();

	/** Event when this actor has the mouse moved off of it with the clickable interface. */
	ENGINE_API virtual void NotifyActorEndCursorOver();
	/** Event when this actor has the mouse moved off of it with the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorEndCursorOver"), Category="Mouse Input")
	ENGINE_API void ReceiveActorEndCursorOver();

	/** Event when this actor is clicked by the mouse when using the clickable interface. */
	ENGINE_API virtual void NotifyActorOnClicked(FKey ButtonPressed = EKeys::LeftMouseButton);
	/** Event when this actor is clicked by the mouse when using the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorOnClicked"), Category="Mouse Input")
	ENGINE_API void ReceiveActorOnClicked(FKey ButtonPressed = EKeys::LeftMouseButton);

	/** Event when this actor is under the mouse when left mouse button is released while using the clickable interface. */
	ENGINE_API virtual void NotifyActorOnReleased(FKey ButtonReleased = EKeys::LeftMouseButton);
	/** Event when this actor is under the mouse when left mouse button is released while using the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "ActorOnReleased"), Category="Mouse Input")
	ENGINE_API void ReceiveActorOnReleased(FKey ButtonReleased = EKeys::LeftMouseButton);

	/** Event when this actor is touched when click events are enabled. */
	ENGINE_API virtual void NotifyActorOnInputTouchBegin(const ETouchIndex::Type FingerIndex);
	/** Event when this actor is touched when click events are enabled. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "BeginInputTouch"), Category="Touch Input")
	ENGINE_API void ReceiveActorOnInputTouchBegin(const ETouchIndex::Type FingerIndex);

	/** Event when this actor is under the finger when untouched when click events are enabled. */
	ENGINE_API virtual void NotifyActorOnInputTouchEnd(const ETouchIndex::Type FingerIndex);
	/** Event when this actor is under the finger when untouched when click events are enabled. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "EndInputTouch"), Category="Touch Input")
	ENGINE_API void ReceiveActorOnInputTouchEnd(const ETouchIndex::Type FingerIndex);

	/** Event when this actor has a finger moved over it with the clickable interface. */
	ENGINE_API virtual void NotifyActorOnInputTouchEnter(const ETouchIndex::Type FingerIndex);
	/** Event when this actor has a finger moved over it with the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "TouchEnter"), Category="Touch Input")
	ENGINE_API void ReceiveActorOnInputTouchEnter(const ETouchIndex::Type FingerIndex);

	/** Event when this actor has a finger moved off of it with the clickable interface. */
	ENGINE_API virtual void NotifyActorOnInputTouchLeave(const ETouchIndex::Type FingerIndex);
	/** Event when this actor has a finger moved off of it with the clickable interface. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "TouchLeave"), Category="Touch Input")
	ENGINE_API void ReceiveActorOnInputTouchLeave(const ETouchIndex::Type FingerIndex);

	/** 
	 * Returns list of actors this actor is overlapping (any component overlapping any component). Does not return itself.
	 * @param OverlappingActors		[out] Returned list of overlapping actors
	 * @param ClassFilter			[optional] If set, only returns actors of this class or subclasses
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(UnsafeDuringActorConstruction="true", DeterminesOutputType="ClassFilter",  DynamicOutputParam="OverlappingActors"))
	ENGINE_API void GetOverlappingActors(TArray<AActor*>& OverlappingActors, TSubclassOf<AActor> ClassFilter=nullptr) const;

	/** 
	 * Returns set of actors this actor is overlapping (any component overlapping any component). Does not return itself.
	 * @param OverlappingActors		[out] Returned list of overlapping actors
	 * @param ClassFilter			[optional] If set, only returns actors of this class or subclasses
	 */
	ENGINE_API void GetOverlappingActors(TSet<AActor*>& OverlappingActors, TSubclassOf<AActor> ClassFilter=nullptr) const;

	/** Returns list of components this actor is overlapping. */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API void GetOverlappingComponents(TArray<UPrimitiveComponent*>& OverlappingComponents) const;

	/** Returns set of components this actor is overlapping. */
	ENGINE_API void GetOverlappingComponents(TSet<UPrimitiveComponent*>& OverlappingComponents) const;

	/** 
	 * Event when this actor bumps into a blocking object, or blocks another actor that bumps into it.
	 * This could happen due to things like Character movement, using Set Location with 'sweep' enabled, or physics simulation.
	 * For events when objects overlap (e.g. walking into a trigger) see the 'Overlap' event.
	 *
	 * @note For collisions during physics simulation to generate hit events, 'Simulation Generates Hit Events' must be enabled.
	 * @note When receiving a hit from another object's movement (bSelfMoved is false), the directions of 'Hit.Normal' and 'Hit.ImpactNormal'
	 * will be adjusted to indicate force from the other object against this object.
	 */
	ENGINE_API virtual void NotifyHit(class UPrimitiveComponent* MyComp, AActor* Other, class UPrimitiveComponent* OtherComp, bool bSelfMoved, FVector HitLocation, FVector HitNormal, FVector NormalImpulse, const FHitResult& Hit);
	/** 
	 * Event when this actor bumps into a blocking object, or blocks another actor that bumps into it.
	 * This could happen due to things like Character movement, using Set Location with 'sweep' enabled, or physics simulation.
	 * For events when objects overlap (e.g. walking into a trigger) see the 'Overlap' event.
	 *
	 * @note For collisions during physics simulation to generate hit events, 'Simulation Generates Hit Events' must be enabled.
	 * @note When receiving a hit from another object's movement (bSelfMoved is false), the directions of 'Hit.Normal' and 'Hit.ImpactNormal'
	 * will be adjusted to indicate force from the other object against this object.
	 * @note NormalImpulse will be filled in for physics-simulating bodies, but will be zero for swept-component blocking collisions.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Hit"), Category="Collision")
	ENGINE_API void ReceiveHit(class UPrimitiveComponent* MyComp, AActor* Other, class UPrimitiveComponent* OtherComp, bool bSelfMoved, FVector HitLocation, FVector HitNormal, FVector NormalImpulse, const FHitResult& Hit);

	/** Set the lifespan of this actor. When it expires the object will be destroyed. If requested lifespan is 0, the timer is cleared and the actor will not be destroyed. */
	UFUNCTION(BlueprintCallable, Category="Actor", meta=(Keywords = "delete destroy"))
	ENGINE_API virtual void SetLifeSpan( float InLifespan );

	/** Get the remaining lifespan of this actor. If zero is returned the actor lives forever. */
	UFUNCTION(BlueprintCallable, Category="Actor", meta=(Keywords = "delete destroy"))
	ENGINE_API virtual float GetLifeSpan() const;

	/**
	 * Construction script, the place to spawn components and do other setup.
	 * @note Name used in CreateBlueprint function
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(BlueprintInternalUseOnly = "true", DisplayName = "Construction Script"))
	ENGINE_API void UserConstructionScript();

	/**
	 * Destroy this actor. Returns true if the actor is destroyed or already marked for destruction, false if indestructible.
	 * Destruction is latent. It occurs at the end of the tick.
	 * @param	bNetForce				[opt] Ignored unless called during play.  Default is false.
	 * @param	bShouldModifyLevel		[opt] If true, Modify() the level before removing the actor.  Default is true.	
	 * @returns	true if destroyed or already marked for destruction, false if indestructible.
	 */
	ENGINE_API bool Destroy(bool bNetForce = false, bool bShouldModifyLevel = true );

	/** Called when the actor has been explicitly destroyed. */
	UFUNCTION(BlueprintImplementableEvent, meta = (Keywords = "delete", DisplayName = "Destroyed"))
	ENGINE_API void ReceiveDestroyed();

	/** Event triggered when the actor has been explicitly destroyed. */
	UPROPERTY(BlueprintAssignable, Category="Game")
	FActorDestroyedSignature OnDestroyed;

	/** Event triggered when the actor is being deleted or removed from a level. */
	UPROPERTY(BlueprintAssignable, Category="Game")
	FActorEndPlaySignature OnEndPlay;
	
	//~ Begin UObject Interface
	ENGINE_API virtual bool CheckDefaultSubobjectsInternal() const override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void ProcessEvent( UFunction* Function, void* Parameters ) override;
	ENGINE_API virtual int32 GetFunctionCallspace( UFunction* Function, FFrame* Stack ) override;
	ENGINE_API virtual bool CallRemoteFunction( UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack ) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostLoadSubobjects( FObjectInstancingGraph* OuterInstanceGraph ) override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
	ENGINE_API virtual bool IsMigrationRoot() const override;

	/**
	 * Returns the server this actor should live on for push migration (ownership change).
	 * Override for class-specific behavior (e.g., Controller follows Pawn).
	 * Default:
	 *  - If the Actor is local and movable, use GetPreferredServerDelegate for game-specific implementation.
	 *  - Otherwise, report the current owner server (no change).
	 */
	ENGINE_API virtual FRemoteServerId GetPreferredRemoteServer() const;

	/**
	 * Static delegate for plugins or games to provide an implementation (typically a spatial-partition-based server lookup).
	 * Called by the default GetPreferredRemoteServer() implementation.
	 */
	static ENGINE_API TDelegate<FRemoteServerId(const AActor*)> GetPreferredServerDelegate;
#endif
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool Rename( const TCHAR* NewName=nullptr, UObject* NewOuter=nullptr, ERenameFlags Flags=REN_None ) override;
	ENGINE_API virtual void PostRename( UObject* OldOuter, const FName OldName ) override;
	ENGINE_API virtual bool CanBeInCluster() const override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual bool IsEditorOnly() const override;
	virtual bool IsEditorOnlyLoadedInPIE() const { return false; }
	virtual bool IsRuntimeOnly() const { return false; }
	ENGINE_API virtual bool IsAsset() const override;

	ENGINE_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	ENGINE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	/** Used to check if Actor is the main actor of a package (currently Child Actors are not) */
	ENGINE_API bool IsMainPackageActor() const;

	static ENGINE_API AActor* FindActorInPackage(UPackage* InPackage, bool bEvenIfPendingKill = true);

#if WITH_EDITOR
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	ENGINE_API virtual bool IsSelectedInEditor() const override;

	/** DuplicationSeedInterface prevents exposing the entire DupSeed map to every actor and provides a method to do so instead. */
	struct UE_DEPRECATED(5.6, "Use PreDuplicateFromRoot instead.") FDuplicationSeedInterface
	{
	public:
		FDuplicationSeedInterface(TMap<UObject*, UObject*>& InDuplicationSeed) {}

		/** Add a new entry to the duplication seed. */
		ENGINE_API void AddEntry(UObject* Source, UObject* Dest) {}
	};

	/** Populate the duplication seed when duplicating the actor for PIE. This allows objects to be remapped to existing objects rather than duplicating. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use PreDuplicateFromRoot instead.")
	virtual void PopulatePIEDuplicationSeed(FDuplicationSeedInterface& DupSeed) {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Defines if preview should be shown when you select an actor, but none of its children */
	virtual bool IsDefaultPreviewEnabled() const
	{
		return true;
	}

	/** Used to know if actor supports some editor operations. (Delete, Replace) */
	virtual bool IsUserManaged() const { return true; }

	/** When selected can this actor be deleted? */
	ENGINE_API virtual bool CanDeleteSelectedActor(FText& OutReason) const;

	/** When selected can this actor be replaced */
	ENGINE_API virtual bool CanReplaceSelectedActor(FText& OutReason) const;

	/** Does this actor supports external packaging? */
	ENGINE_API virtual bool SupportsExternalPackaging() const;
#endif

#if WITH_EDITOR
	/** Cached pointer to the transaction annotation data from PostEditUndo to be used in the next RerunConstructionScript */
	TSharedPtr<FActorTransactionAnnotation> CurrentTransactionAnnotation;

	ENGINE_API virtual TSharedPtr<ITransactionObjectAnnotation> FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const override;
	ENGINE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;

	/** Returns true if the component is allowed to re-register its components when modified. False for CDOs or PIE instances. */
	ENGINE_API bool ReregisterComponentsWhenModified() const;

	/** Called after an actor has been moved in the editor */
	ENGINE_API virtual void PostEditMove(bool bFinished);
#endif // WITH_EDITOR
	//~ End UObject Interface

#if WITH_EDITOR
	virtual void PreRenameFromRoot(const FWorldRenameFromRootContext& RenameContext) {}
	virtual void PostRenameFromRoot(const FWorldRenameFromRootContext& RenameContext) {}
#endif
	virtual void PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams) {}
	ENGINE_API virtual void PreSaveFromRoot(FObjectPreSaveRootContext ObjectSaveContext);
	ENGINE_API virtual void PostSaveFromRoot(FObjectPostSaveRootContext ObjectSaveContext);

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const;
#endif

	//~=============================================================================
	// Property Replication

	/** Fills ReplicatedMovement property */
	ENGINE_API virtual void GatherCurrentMovement();

	/** See if this actor is owned by TestOwner. */
	inline bool IsOwnedBy( const AActor* TestOwner ) const
	{
		for( const AActor* Arg=this; Arg; Arg=Arg->Owner )
		{
			if( Arg == TestOwner )
				return true;
		}
		return false;
	}

	/** Returns this actor's root component. */
	inline USceneComponent* GetRootComponent() const { return RootComponent; }

	/**
	 * Returns this actor's default attachment component for attaching children to
	 * @return The scene component to be used as parent
	 */
	virtual USceneComponent* GetDefaultAttachComponent() const { return GetRootComponent(); }

	/**
	 * Sets root component to be the specified component.  NewRootComponent's owner should be this actor.
	 * @return true if successful
	 */
	ENGINE_API bool SetRootComponent(USceneComponent* NewRootComponent);

	/** Returns the transform of the RootComponent of this Actor*/ 
	inline const FTransform& GetActorTransform() const
	{
		return TemplateGetActorTransform(ToRawPtr(RootComponent));
	}

	/** Returns the location of the RootComponent of this Actor*/ 
	inline FVector GetActorLocation() const
	{
		return TemplateGetActorLocation(ToRawPtr(RootComponent));
	}

	/** Returns the rotation of the RootComponent of this Actor */
	inline FRotator GetActorRotation() const
	{
		return TemplateGetActorRotation(ToRawPtr(RootComponent));
	}

	/** Returns the scale of the RootComponent of this Actor */
	inline FVector GetActorScale() const
	{
		return TemplateGetActorScale(ToRawPtr(RootComponent));
	}

	/** Returns the quaternion of the RootComponent of this Actor */
	inline FQuat GetActorQuat() const
	{
		return TemplateGetActorQuat(ToRawPtr(RootComponent));
	}

#if WITH_EDITOR
	/** Sets the local space offset added to the actor's pivot as used by the editor */
	inline void SetPivotOffset(const FVector& InPivotOffset)
	{
		PivotOffset = InPivotOffset;
	}

	/** Gets the local space offset added to the actor's pivot as used by the editor */
	inline FVector GetPivotOffset() const
	{
		return PivotOffset;
	}

	/**
	 * Returns the bounding boxes of all components that make up this Actor for loading at runtime and editor.
	 *
	 * This function differs from GetActorBounds because it will return a valid origin and an empty extent if this actor
	 * doesn't have primitive components.
	 *
	 * @see GetActorBounds()
	 */
	ENGINE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const;

	UE_DEPRECATED(5.5, "Use the override that takes both Runtime and Editor boxes")
	ENGINE_API FBox GetStreamingBounds() const;
#endif


	//~=============================================================================
	// Relations

	/** 
	 * Called by owning level to shift an actor location and all relevant data structures by specified delta
	 *  
	 * @param InOffset		Offset vector to shift actor location
	 * @param bWorldShift	Whether this call is part of whole world shifting
	 */
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift);

	/** Indicates whether this actor should participate in level bounds calculations */
	virtual bool IsLevelBoundsRelevant() const { return bRelevantForLevelBounds; }

	/** Indicates whether this actor contributes to the HLOD generation. */
	ENGINE_API virtual bool IsHLODRelevant() const;

	/** Indicates whether this actor can provide HLOD relevant components. */
	ENGINE_API virtual bool HasHLODRelevantComponents() const;

	/** Return the list of components to use when building the HLOD representation of this actor. */
	ENGINE_API virtual TArray<UActorComponent*> GetHLODRelevantComponents() const;

	/** 
	 * Set LOD Parent component for all of our components, normally associated with an ALODActor. 
	 * @param InLODParent			This component used to compute visibility when hierarchical LOD is enabled. 
	 * @param InParentDrawDistance	Updates the MinDrawDistances of the LODParent
	 */
	ENGINE_API void SetLODParent(class UPrimitiveComponent* InLODParent, float InParentDrawDistance);

#if WITH_EDITOR
	/** @todo: Remove this flag once it is decided that additive interactive scaling is what we want */
	static ENGINE_API bool bUsePercentageBasedScaling;

	/**
	 * Called by ApplyDeltaToActor to perform an actor class-specific operation based on widget manipulation.
	 * The default implementation is simply to translate the actor's location.
	 */
	ENGINE_API virtual void EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown);

	/**
	 * Called by ApplyDeltaToActor to perform an actor class-specific operation based on widget manipulation.
	 * The default implementation is simply to modify the actor's rotation.
	 */
	ENGINE_API virtual void EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown);

	/**
	 * Called by ApplyDeltaToActor to perform an actor class-specific operation based on widget manipulation.
	 * The default implementation is simply to modify the actor's draw scale.
	 */
	ENGINE_API virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown);

	/** Called by MirrorActors to perform a mirroring operation on the actor */
	ENGINE_API virtual void EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation);	

	/** Get underlying actors */
	ENGINE_API virtual void EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const;

	/** Returns true if the actor is hidden upon editor startup/by default, false if it is not */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	bool IsHiddenEdAtStartup() const
	{
		return bHiddenEd;
	}

	/** Returns true if this actor is hidden in the editor viewports, also checking temporary flags. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API virtual bool IsHiddenEd() const;

	/**
	 * Explicitly sets whether or not this actor is hidden in the editor for the duration of the current editor session
	 * @param bIsHidden	True if the actor is hidden
	 */
	UFUNCTION(BlueprintCallable, Category="Editor Scripting | Actor Editing")
	ENGINE_API virtual void SetIsTemporarilyHiddenInEditor( bool bIsHidden );

	/** Changes bHiddenEdLayer flag and returns true if flag changed. */
	ENGINE_API virtual bool SetIsHiddenEdLayer(bool bIsHiddenEdLayer);

	/** Returns true if the actor supports modifications to its Layers property */
	ENGINE_API virtual bool SupportsLayers() const;

	/** Returns if level should keep a reference to the external actor for PIE (used for always loaded actors). */
	ENGINE_API bool IsForceExternalActorLevelReferenceForPIE() const;

	void SetForceExternalActorLevelReferenceForPIE(bool bValue)
	{
		if (IsPackageExternal())
		{
			bForceExternalActorLevelReferenceForPIE = bValue;
		}
	}

	/** Returns true if this actor is spatially loaded. */
	ENGINE_API bool GetIsSpatiallyLoaded() const;
	
	/** Set if this actor should be spatially loaded or not. */
	void SetIsSpatiallyLoaded(bool bInIsSpatiallyLoaded)
	{
		if (bIsSpatiallyLoaded != bInIsSpatiallyLoaded)
		{
			check(CanChangeIsSpatiallyLoadedFlag());
			bIsSpatiallyLoaded = bInIsSpatiallyLoaded;
		}
	}

	/** Returns true if this actor allows changing the spatially loaded flag.  */
	ENGINE_API virtual bool CanChangeIsSpatiallyLoadedFlag() const;

	/** Gets the property name for bIsSpatiallyLoaded. */
	static const FName GetIsSpatiallyLoadedPropertyName() { return GET_MEMBER_NAME_CHECKED(AActor, bIsSpatiallyLoaded); }

	/**
	 * Returns whether or not this actor was explicitly hidden in the editor for the duration of the current editor session
	 * @param bIncludeParent - Whether to recurse up child actor hierarchy or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API bool IsTemporarilyHiddenInEditor(bool bIncludeParent = false) const;

	/** Returns true if this actor is allowed to be displayed, selected and manipulated by the editor. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API bool IsEditable() const;

	/** Returns true if this actor can EVER be selected in a level in the editor.  Can be overridden by specific actors to make them unselectable. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API virtual bool IsSelectable() const;

	/** Returns true if this actor should be shown in the scene outliner */
	ENGINE_API virtual bool IsListedInSceneOutliner() const;

	/** Returns true if this actor is allowed to be attached to the given actor */
	ENGINE_API virtual bool EditorCanAttachTo(const AActor* InParent, FText& OutReason) const;

	/** Returns true if this actor is allowed to be attached from the given actor */
	ENGINE_API virtual bool EditorCanAttachFrom(const AActor* InChild, FText& OutReason) const;

	/** Returns true if this actor is allowed to be detached from the given actor */
	virtual bool EditorCanDetachFrom(const AActor* InParent, FText& OutReason) const { return true; }

	/** Returns the actor attachement parent that should be used in the scene outliner */
	ENGINE_API virtual AActor* GetSceneOutlinerParent() const;

	/** Returns the actor package that should be used in the scene outliner */
#if !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.6, "Please use GetSceneOutlinerItemPackage or GetSceneOutlinerTopParentPackage instead.")
	ENGINE_API virtual UPackage* GetSceneOutlinerPackage() const;
#endif

	/** Returns the actor item package that should be used in the scene outliner */
	ENGINE_API virtual UPackage* GetSceneOutlinerItemPackage() const;

	/** Returns the actor top parent package that should be used in the scene outliner */
	ENGINE_API virtual UPackage* GetSceneOutlinerTopParentPackage() const;

	/** Called before editor copy, true allow export */
	virtual bool ShouldExport() { return true; }

	/** Called before editor paste, true allow import */
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) { return true; }

	/** Called by ReplaceSelectedActors to allow a new actor to copy properties from an old actor when it is replaced */
	ENGINE_API virtual void EditorReplacedActor(AActor* OldActor);

	/**
	 * Function that gets called from within Map_Check to allow this actor to check itself
	 * for any potential errors and register them with map check dialog.
	 */
	ENGINE_API virtual void CheckForErrors();

	/**
	 * Function that gets called from within Map_Check to allow this actor to check itself
	 * for any potential errors and register them with map check dialog.
	 */
	ENGINE_API virtual void CheckForDeprecated();

	/** Returns this actor's default label (does not include any numeric suffix).  Actor labels are only available in development builds. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing", meta = (KeyWords = "Display Name"))
	ENGINE_API virtual FString GetDefaultActorLabel() const;

	/** Returns this actor's current label.  Actor labels are only available in development builds. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing", meta = (KeyWords = "Display Name"))
	ENGINE_API const FString& GetActorLabel(bool bCreateIfNone = true) const;

	/**
	 * Assigns a new label to this actor.  Actor labels are only available in development builds.
	 * @param	NewActorLabel	The new label string to assign to the actor.  If empty, the actor will have a default label.
	 * @param	bMarkDirty		If true the actor's package will be marked dirty for saving.  Otherwise it will not be.  You should pass false for this parameter if dirtying is not allowed (like during loads)
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing", meta = (KeyWords = "Display Name"))
	ENGINE_API void SetActorLabel( const FString& NewActorLabel, bool bMarkDirty = true );

	/** Advanced - clear the actor label. */
	ENGINE_API void ClearActorLabel();

	/** Advanced - only to be called by UWorld::SpawnActor to set an initial label for this actor */
	void SetInitialActorLabel(FStringView InitialLabel);

	/**
	 * Returns if this actor's current label is editable.  Actor labels are only available in development builds.
	 * @return	The editable status of the actor's label
	 */
	ENGINE_API virtual bool IsActorLabelEditable() const;

	/** Returns this actor's folder path. Actor folder paths are only available in development builds. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API FName GetFolderPath() const;

	/*
	 * Returns actor folder guid. If level is not using actor folder objects, returns an invalid guid. 
	 * @param bDirectAccess If true, returns the raw value without testing if level uses Actor Folders.
	 */
	ENGINE_API FGuid GetFolderGuid(bool bDirectAccess = false) const;

	/** Returns the actor's folder root object. Null, is interpreted as the actor's world. */
	ENGINE_API FFolder::FRootObject GetFolderRootObject() const;
	
	/** Detects and fixes invalid actor folder */
	ENGINE_API void FixupActorFolder();

	/** Creates or updates actor's folder object if necessary. Returns false if an error occured while creating folder. */
	ENGINE_API bool CreateOrUpdateActorFolder();

	/** Returns a FFolder that contains the actor  folder path and its folder root object. */
	ENGINE_API FFolder GetFolder() const;

	/**
	 * Assigns a new folder to this actor. Actor folder paths are only available in development builds.
	 * @param	NewFolderPath		The new folder to assign to the actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Editing")
	ENGINE_API void SetFolderPath(const FName& NewFolderPath);

	/**
	 * Assigns a new folder to this actor and any attached children. Actor folder paths are only available in development builds.
	 * @param	NewFolderPath		The new folder to assign to the actors.
	 */
	ENGINE_API void SetFolderPath_Recursively(const FName& NewFolderPath);

	/**
	 * Used by the "Sync to Content Browser" right-click menu option in the editor.
	 * @param	Objects	Array to add content object references to.
	 * @return	Whether the object references content (all overrides of this function should return true)
	 */
	ENGINE_API virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const;

	/** Similar to GetReferencedContentObjects, but for soft referenced objects */
	ENGINE_API virtual bool GetSoftReferencedContentObjects(TArray<FSoftObjectPath>& SoftObjects) const;

	/** Used to allow actor classes to open an actor specific editor */
	virtual bool OpenAssetEditor() { return false; }

	/** Returns NumUncachedStaticLightingInteractions for this actor */
	ENGINE_API const int32 GetNumUncachedStaticLightingInteractions() const;

	/** Returns a custom brush icon name to use in place of the automatic class icon where actors are represented via 2d icons in the editor (e.g scene outliner and menus) */
	virtual FName GetCustomIconName() const { return NAME_None; }

	/** Returns whether or not to cook optimized Blueprint component data for this actor */
	inline bool ShouldCookOptimizedBPComponentData() const
	{
		return bOptimizeBPComponentData;
	}

	/** Sets metadata on the Actor that allows it to point to a different asset than its source when browsing in the Content Browser */
	UE_DEPRECATED(5.7, "SetBrowseToAssetOverride is deprecated as it was unreliable. Register a callback with UBrowseToAssetOverrideSubsystem instead, and store the override as a property on your actor.")
	ENGINE_API void SetBrowseToAssetOverride(const FString& PackageName);
	/** Gets metadata on the Actor that says which package to show in the Content Browser when browsing to it, or an empty string if there is no override */
	UE_DEPRECATED(5.7, "GetBrowseToAssetOverride is deprecated. Use UBrowseToAssetOverrideSubsystem::GetBrowseToAssetOverride instead.")
	ENGINE_API const FString& GetBrowseToAssetOverride() const;
#endif		// WITH_EDITOR

	/**
	 * Function used to prioritize actors when deciding which to replicate
	 * @param ViewPos		Position of the viewer
	 * @param ViewDir		Vector direction of viewer
	 * @param Viewer		"net object" owned by the client for whom net priority is being determined (typically player controller)
	 * @param ViewTarget	The actor that is currently being viewed/controlled by Viewer, usually a pawn
	 * @param InChannel		Channel on which this actor is being replicated.
	 * @param Time			Time since actor was last replicated
	 * @param bLowBandwidth True if low bandwidth of viewer
	 * @return				Priority of this actor for replication, higher is more important
	 */
	ENGINE_API virtual float GetNetPriority(const FVector& ViewPos, const FVector& ViewDir, class AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth);

	/**
	 * Similar to GetNetPriority, but will only be used for prioritizing actors while recording a replay.
	 * @param ViewPos		Position of the viewer
	 * @param ViewDir		Vector direction of viewer
	 * @param Viewer		"net object" owned by the client for whom net priority is being determined (typically player controller)
	 * @param ViewTarget	The actor that is currently being viewed/controlled by Viewer, usually a pawn
	 * @param InChannel		Channel on which this actor is being replicated.
	 * @param Time			Time since actor was last replicated
	 * @return				Priority of this actor for replays, higher is more important
	 */
	ENGINE_API virtual float GetReplayPriority(const FVector& ViewPos, const FVector& ViewDir, class AActor* Viewer, AActor* ViewTarget, UActorChannel* const InChannel, float Time);

	/** Returns true if the actor should be dormant for a specific net connection. Only checked for DORM_DormantPartial */
	ENGINE_API virtual bool GetNetDormancy(const FVector& ViewPos, const FVector& ViewDir, class AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth);

	/** 
	 * Allows for a specific response from the actor when the actor channel is opened (client side)
	 * @param InBunch Bunch received at time of open
	 * @param Connection the connection associated with this actor
	 */
	virtual void OnActorChannelOpen(class FInBunch& InBunch, class UNetConnection* Connection) {};

	/**
	 * Used by the net connection to determine if a net owning actor should switch to using the shortened timeout value
	 * 
	 * @return true to switch from InitialConnectTimeout to ConnectionTimeout values on the net driver
	 */
	virtual bool UseShortConnectTimeout() const { return false; }

	/**
	 * SerializeNewActor has just been called on the actor before network replication (server side)
	 * @param OutBunch Bunch containing serialized contents of actor prior to replication
	 */
	virtual void OnSerializeNewActor(class FOutBunch& OutBunch) {};

	/** 
	 * Handles cleaning up the associated Actor when killing the connection 
	 * @param Connection the connection associated with this actor
	 */
	virtual void OnNetCleanup(class UNetConnection* Connection) {};

	/** Swaps Role and RemoteRole if client */
	ENGINE_API void ExchangeNetRoles(bool bRemoteOwner);

	/** Calls this to swap the Role and RemoteRole.  Only call this if you know what you're doing! */
	ENGINE_API void SwapRoles();

	/**
	 * When called, will call the virtual call chain to register all of the tick functions for both the actor and optionally all components
	 * Do not override this function or make it virtual
	 * @param bRegister - true to register, false, to unregister
	 * @param bDoComponents - true to also apply the change to all components
	 */
	ENGINE_API void RegisterAllActorTickFunctions(bool bRegister, bool bDoComponents);

	/** 
	 * Set this actor's tick functions to be enabled or disabled. Only has an effect if the function is registered
	 * This only modifies the tick function on actor itself
	 * @param	bEnabled	Whether it should be enabled or not
	 */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API virtual void SetActorTickEnabled(bool bEnabled);

	/**  Returns whether this actor has tick enabled or not	 */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API bool IsActorTickEnabled() const;

	/** 
	 * Sets the tick interval of this actor's primary tick function. Will not enable a disabled tick function. Takes effect on next tick. 
	 * @param TickInterval	The rate at which this actor should be ticking
	 */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API void SetActorTickInterval(float TickInterval);

	/**  Returns the tick interval of this actor's primary tick function */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick")
	ENGINE_API float GetActorTickInterval() const;

	/**
	 * Dispatches the once-per frame Tick() function for this actor
	 * @param	DeltaTime			The time slice of this tick
	 * @param	TickType			The type of tick that is happening
	 * @param	ThisTickFunction	The tick function that is firing, useful for getting the completion handle
	 */
	ENGINE_API virtual void TickActor( float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction );

	/**
	 * Override this function to implement custom logic to be executed every physics step.
	 * bAsyncPhysicsTick must be set to true.
	 *	
	 * @param DeltaTime - The physics step delta time
	 * @param SimTime - This is the total sim time since the sim began.
	 */
	virtual void AsyncPhysicsTickActor(float DeltaTime, float SimTime) { ReceiveAsyncPhysicsTick(DeltaTime, SimTime); }

	/**
	 * Called when an actor is done spawning into the world (from UWorld::SpawnActor), both in the editor and during gameplay
	 * For actors with a root component, the location and rotation will have already been set.
	 * This is called before calling construction scripts, but after native components have been created
	 */
	ENGINE_API virtual void PostActorCreated();

	/** Called when the lifespan of an actor expires (if it has one). */
	ENGINE_API virtual void LifeSpanExpired();

	/** Always called immediately before properties are received from the remote. */
	ENGINE_API virtual void PreNetReceive() override;
	
	/** Always called immediately after properties are received from the remote. */
	ENGINE_API virtual void PostNetReceive() override;

	/** Always called immediately after a new Role is received from the remote. */
	ENGINE_API virtual void PostNetReceiveRole();

	/** IsNameStableForNetworking means an object can be referred to its path name (relative to outer) over the network */
	ENGINE_API virtual bool IsNameStableForNetworking() const override;

	/** IsSupportedForNetworking means an object can be referenced over the network */
	ENGINE_API virtual bool IsSupportedForNetworking() const override;

	/** Always called immediately after spawning and reading in replicated properties */
	ENGINE_API virtual void PostNetInit();

	/** ReplicatedMovement struct replication event */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedMovement();

	/** Update location and rotation from ReplicatedMovement. Not called for simulated physics! */
	ENGINE_API virtual void PostNetReceiveLocationAndRotation();

	/** Update velocity - typically from ReplicatedMovement, not called for simulated physics! */
	ENGINE_API virtual void PostNetReceiveVelocity(const FVector& NewVelocity);

	/** Update and smooth simulated physic state, replaces PostNetReceiveLocation() and PostNetReceiveVelocity() */
	ENGINE_API virtual void PostNetReceivePhysicState();

	/** Set the current state as a faked networked physics state for physics replication
	* Limited for use with actors using EPhysicsReplicationMode::PredictiveInterpolation only.
	* @param bShouldSleep  Should the replication force the object to sleep */
	void SetFakeNetPhysicsState(bool bShouldSleep);

protected:
	/** Sync IsSimulatingPhysics() with ReplicatedMovement.bRepPhysics */
	ENGINE_API void SyncReplicatedPhysicsSimulation();

public:
	/** 
	 * Set the owner of this Actor, used primarily for network replication. 
	 * @param NewOwner	The Actor who takes over ownership of this Actor
	 */
	UFUNCTION(BlueprintCallable, Category=Actor)
	ENGINE_API virtual void SetOwner( AActor* NewOwner );

	/** Get the owner of this Actor, used primarily for network replication. */
	UFUNCTION(BlueprintCallable, Category=Actor)
	AActor* GetOwner() const;

	/** Templated version of GetOwner(), will return nullptr if cast fails */
	template< class T >
	T* GetOwner() const
	{
		return Cast<T>(GetOwner());
	}

	/**
	 * This will check to see if the Actor is still in the world.  It will check things like
	 * the KillZ, outside world bounds, etc. and handle the situation.
	 */
	ENGINE_API virtual bool CheckStillInWorld();

#if WITH_EDITOR
	/** Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object. */
	ENGINE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR


	//~=============================================================================
	// Actor overlap tracking
	
	/**
	 * Dispatch all EndOverlap for all of the Actor's PrimitiveComponents. 
	 * Generally used when removing the Actor from the world.
	 */
	ENGINE_API void ClearComponentOverlaps();

	/** 
	 * Queries world and updates overlap detection state for this actor.
	 * @param bDoNotifies		True to dispatch being/end overlap notifications when these events occur.
	 */
	ENGINE_API void UpdateOverlaps(bool bDoNotifies=true);
	
	/** 
	 * Check whether any component of this Actor is overlapping any component of another Actor.
	 * @param Other The other Actor to test against
	 * @return Whether any component of this Actor is overlapping any component of another Actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(UnsafeDuringActorConstruction="true"))
	ENGINE_API bool IsOverlappingActor(const AActor* Other) const;

	/** See if the root component has ModifyFrequency of MF_Static */
	ENGINE_API bool IsRootComponentStatic() const;

	/** See if the root component has Mobility of EComponentMobility::Stationary */
	ENGINE_API bool IsRootComponentStationary() const;

	/** See if the root component has Mobility of EComponentMobility::Movable */
	ENGINE_API bool IsRootComponentMovable() const;

	/** Get the physics volume that is currently applied to this Actor (there can only ever be one) */
	ENGINE_API virtual APhysicsVolume* GetPhysicsVolume() const;

	//~=============================================================================
	// Actor ticking

	/** Accessor for the value of bCanEverTick */
	inline bool CanEverTick() const { return PrimaryActorTick.bCanEverTick; }

	/** 
	 *	Function called every frame on this Actor. Override this function to implement custom logic to be executed every frame.
	 *	Note that Tick is disabled by default, and you will need to check PrimaryActorTick.bCanEverTick is set to true to enable it.
	 *
	 *	@param	DeltaSeconds	Game time elapsed during last frame modified by the time dilation
	 */
	ENGINE_API virtual void Tick( float DeltaSeconds );

	/** If true, actor is ticked even if TickType==LEVELTICK_ViewportsOnly	 */
	ENGINE_API virtual bool ShouldTickIfViewportsOnly() const;


	//~=============================================================================
	// Actor relevancy determination

protected:
	/**
	 * Determines whether or not the distance between the given SrcLocation and the Actor's location
	 * is within the net relevancy distance. Actors outside relevancy distance may not be replicated.
	 *
	 * @param SrcLocation	Location to test against.
	 * @return True if the actor is within net relevancy distance, false otherwise.
	 */
	ENGINE_API bool IsWithinNetRelevancyDistance(const FVector& SrcLocation) const;
	
public:	
	/** 
	 * Checks to see if this actor is relevant for a specific network connection
	 *
	 * @param RealViewer - is the "controlling net object" associated with the client for which network relevancy is being checked (typically player controller)
	 * @param ViewTarget - is the Actor being used as the point of view for the RealViewer
	 * @param SrcLocation - is the viewing location
	 *
	 * @return bool - true if this actor is network relevant to the client associated with RealViewer 
	 */
	ENGINE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const;

	/**
	 * Checks to see if this actor is relevant for a recorded replay
	 *
	 * @param RealViewer - is the "controlling net object" associated with the client for which network relevancy is being checked (typically player controller)
	 * @param ViewTarget - is the Actor being used as the point of view for the RealViewer
	 * @param SrcLocation - is the viewing location
	 *
	 * @return bool - true if this actor is replay relevant to the client associated with RealViewer
	 */
	ENGINE_API virtual bool IsReplayRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation, const float CullDistanceSquared) const;

	/**
	 * Check if this actor is the owner when doing relevancy checks for actors marked bOnlyRelevantToOwner
	 *
	 * @param ReplicatedActor - the actor we're doing a relevancy test on
	 * @param ActorOwner - the owner of ReplicatedActor
	 * @param ConnectionActor - the controller of the connection that we're doing relevancy checks for
	 *
	 * @return bool - true if this actor should be considered the owner
	 */
	ENGINE_API virtual bool IsRelevancyOwnerFor(const AActor* ReplicatedActor, const AActor* ActorOwner, const AActor* ConnectionActor) const;

	/** Called after the actor is spawned in the world.  Responsible for setting up actor for play. */
	ENGINE_API void PostSpawnInitialize(FTransform const& SpawnTransform, AActor* InOwner, APawn* InInstigator, bool bRemoteOwned, bool bNoFail, bool bDeferConstruction, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::MultiplyWithRoot);

	/** Called to finish the spawning process, generally in the case of deferred spawning */
	ENGINE_API void FinishSpawning(const FTransform& Transform, bool bIsDefaultTransform = false, const FComponentInstanceDataCache* InstanceDataCache = nullptr, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::OverrideRootScale);

	/** Called after the actor has run its construction. Responsible for finishing the actor spawn process. */
	ENGINE_API void PostActorConstruction();

public:
	/** Called right before components are initialized, only called during gameplay */
	ENGINE_API virtual void PreInitializeComponents();

	/** Allow actors to initialize themselves on the C++ side after all of their components have been initialized, only called during gameplay */
	ENGINE_API virtual void PostInitializeComponents();


	/** Dispatches ReceiveHit virtual and OnComponentHit delegate */
	ENGINE_API virtual void DispatchPhysicsCollisionHit(const struct FRigidBodyCollisionInfo& MyInfo, const struct FRigidBodyCollisionInfo& OtherInfo, const FCollisionImpactData& RigidCollisionData);
	
	/** Return the actor responsible for replication, if any. Typically the player controller */
	virtual const AActor* GetNetOwner() const;

	/** Return the owning UPlayer (if any) of this actor if it's role is ROLE_Authority. This will be a local player, a net connection, or nullptr. */
	ENGINE_API virtual class UPlayer* GetNetOwningPlayer();

	/** Return the owning UPlayer (if any) of this actor, regardless of it's current role. This will be a local player, a net connection, or nullptr. */
	ENGINE_API virtual class UPlayer* GetNetOwningPlayerAnyRole();

	/**
	 * Get the owning connection used for communicating between client/server 
	 * @return NetConnection to the client or server for this actor
	 */
	ENGINE_API virtual class UNetConnection* GetNetConnection() const;

	/**
	 * Called by DestroyActor(), gives actors a chance to op out of actor destruction
	 * Used by network code to have the net connection timeout/cleanup first
	 *
	 * @return true if DestroyActor() should not continue with actor destruction, false otherwise
	 */
	ENGINE_API virtual bool DestroyNetworkActorHandled();

	/**
	 * Get the network mode (dedicated server, client, standalone, etc) for this actor.
	 * @see IsNetMode()
	 */
	ENetMode GetNetMode() const;

	/**
	 * Test whether net mode is the given mode.
	 * In optimized non-editor builds this can be more efficient than GetNetMode()
	 * because it can check the static build flags without considering PIE.
	 */
	bool IsNetMode(ENetMode Mode) const;

	/** Returns the net driver that this actor is bound to, may be null */
	ENGINE_API class UNetDriver* GetNetDriver() const;

	/** Puts actor in dormant networking state */
	UFUNCTION(BlueprintAuthorityOnly, BlueprintCallable, Category = "Networking")
	ENGINE_API void SetNetDormancy(ENetDormancy NewDormancy);

	/** Forces dormant actor to replicate but doesn't change NetDormancy state (i.e., they will go dormant again if left dormant) */
	UFUNCTION(BlueprintAuthorityOnly, BlueprintCallable, Category="Networking")
	ENGINE_API void FlushNetDormancy();

	/** Forces properties on this actor to do a compare for one frame (rather than share shadow state) */
	ENGINE_API void ForcePropertyCompare();

	/** Returns whether this Actor was spawned by a child actor component */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API bool IsChildActor() const;

	/** Returns whether this actor can select its attached actors */
	ENGINE_API virtual bool IsSelectionParentOfAttachedActors() const;

	/** Returns whether this Actor is part of another's actor selection */
	ENGINE_API virtual bool IsSelectionChild() const;

	/** Returns immediate selection parent */
	ENGINE_API virtual AActor* GetSelectionParent() const;

	/** Returns top most selection parent */
	ENGINE_API virtual AActor* GetRootSelectionParent() const;

	/** Returns true if actor can be selected as a sub selection of its root selection parent */
	ENGINE_API virtual bool SupportsSubRootSelection() const;

	/** Returns if actor or selection parent is selected */
	ENGINE_API bool IsActorOrSelectionParentSelected() const;

	/** Push Selection to actor */
	ENGINE_API virtual void PushSelectionToProxies();

#if WITH_EDITOR
	/** Push Foundation Editing state to primitive scene proxy */
	ENGINE_API virtual void PushLevelInstanceEditingStateToProxies(bool bInEditingState);
#endif

	/** 
	 * Returns a list of all actors spawned by our Child Actor Components, including children of children. 
	 * This does not return the contents of the Children array
	 */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API void GetAllChildActors(TArray<AActor*>& ChildActors, bool bIncludeDescendants = true) const;

	/** If this Actor was created by a Child Actor Component returns that Child Actor Component  */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API UChildActorComponent* GetParentComponent() const;

	/** If this Actor was created by a Child Actor Component returns the Actor that owns that Child Actor Component  */
	UFUNCTION(BlueprintCallable, Category="Actor")
	ENGINE_API AActor* GetParentActor() const;

	/** Ensure that all the components in the Components array are registered */
	ENGINE_API virtual void RegisterAllComponents();

	/** Called before all the components in the Components array are registered, called both in editor and during gameplay */
	ENGINE_API virtual void PreRegisterAllComponents();

	/** 
	 * Called after all the components in the Components array are registered, called both in editor and during gameplay.
	 * bHasRegisteredAllComponents must be set true prior to calling this function.
	 */
	ENGINE_API virtual void PostRegisterAllComponents();

	/** Returns true if Actor has deferred the RegisterAllComponents() call at spawn time (e.g. pending Blueprint SCS execution to set up a scene root component). */
	inline bool HasDeferredComponentRegistration() const { return bHasDeferredComponentRegistration; }

	/** Returns true if Actor has a registered root component */
	ENGINE_API bool HasValidRootComponent() const;

	/** 
	 * Unregister all currently registered components
	 * @param bForReregister If true, RegisterAllComponents will be called immediately after this so some slow operations can be avoided
	 */
	ENGINE_API virtual void UnregisterAllComponents(bool bForReregister = false);

	/** Called after all currently registered components are cleared */
	ENGINE_API virtual void PostUnregisterAllComponents();

	/** Will reregister all components on this actor. Does a lot of work - should only really be used in editor, generally use UpdateComponentTransforms or MarkComponentsRenderStateDirty. */
	ENGINE_API virtual void ReregisterAllComponents();

	/** Finish initializing the component and register tick functions and beginplay if it's the proper time to do so. */
	ENGINE_API void HandleRegisterComponentWithWorld(UActorComponent* Component);

	/**
	 * Incrementally registers components associated with this actor, used during level streaming
	 *
	 * @param NumComponentsToRegister  Number of components to register in this run, 0 for all
	 * @return true when all components were registered for this actor
	 */
	ENGINE_API bool IncrementalRegisterComponents(int32 NumComponentsToRegister, FRegisterComponentContext* Context = nullptr);

	/** Flags all component's render state as dirty	 */
	ENGINE_API void MarkComponentsRenderStateDirty();

	/** Update all components transforms */
	ENGINE_API void UpdateComponentTransforms();

	/** Update all components visibility state */
	ENGINE_API void UpdateComponentVisibility();

	/** Iterate over components array and call InitializeComponent, which happens once per actor */
	ENGINE_API void InitializeComponents();

	/** Iterate over components array and call UninitializeComponent, called when the actor is ending play */
	ENGINE_API void UninitializeComponents();

	/** Debug rendering to visualize the component tree for this actor. */
	ENGINE_API void DrawDebugComponents(FColor const& BaseColor=FColor::White) const;

	/** Called to mark all components as garbage when the actor is being destroyed
	 *
	 *  @param bModify if True, Modify will be called on actor before marking components
	 */
	ENGINE_API virtual void MarkComponentsAsGarbage(bool bModify = true);
	
	/**
	 * Returns true if this actor has begun the destruction process.
	 * This is set to true in UWorld::DestroyActor, after the network connection has been closed but before any other shutdown has been performed.
	 * @return true if this actor has begun destruction, or if this actor has been destroyed already.
	 */
	inline bool IsPendingKillPending() const
	{
		return bActorIsBeingDestroyed || !IsValidChecked(this);
	}

	/** Invalidate lighting cache with default options. */
	void InvalidateLightingCache()
	{
		if (GIsEditor && !GIsDemoMode)
		{
			InvalidateLightingCacheDetailed(false);
		}
	}

	/** Invalidates anything produced by the last lighting build. */
	ENGINE_API virtual void InvalidateLightingCacheDetailed(bool bTranslationOnly);

	/**
	 * Used for adding actors to levels or teleporting them to a new location.
	 * The result of this function is independent of the actor's current location and rotation.
	 * If the actor doesn't fit exactly at the location specified, tries to slightly move it out of walls and such if bNoCheck is false.
	 *
	 * @param DestLocation The target destination point
	 * @param DestRotation The target rotation at the destination
	 * @param bIsATest is true if this is a test movement, which shouldn't cause any notifications (used by AI pathfinding, for example)
	 * @param bNoCheck is true if we should skip checking for encroachment in the world or other actors
	 * @return true if the actor has been successfully moved, or false if it couldn't fit.
	 */
	ENGINE_API virtual bool TeleportTo( const FVector& DestLocation, const FRotator& DestRotation, bool bIsATest=false, bool bNoCheck=false );

	/**
	 * Teleport this actor to a new location. If the actor doesn't fit exactly at the location specified, tries to slightly move it out of walls and such.
	 *
	 * @param DestLocation The target destination point
	 * @param DestRotation The target rotation at the destination
	 * @return true if the actor has been successfully moved, or false if it couldn't fit.
	 */
	UFUNCTION(BlueprintCallable, meta=( DisplayName="Teleport", ScriptName="Teleport", Keywords = "Move Position" ), Category="Transformation")
	ENGINE_API bool K2_TeleportTo( FVector DestLocation, FRotator DestRotation );

	/** Called from TeleportTo() when teleport succeeds */
	virtual void TeleportSucceeded(bool bIsATest) {}

	/**
	 * Trace a ray against the Components of this Actor and return the first blocking hit
	 * @param  OutHit          First blocking hit found
	 * @param  Start           Start location of the ray
	 * @param  End             End location of the ray
	 * @param  TraceChannel    The 'channel' that this ray is in, used to determine which components to hit
	 * @param  Params          Additional parameters used for the trace
	 * @return TRUE if a blocking hit is found
	 */
	ENGINE_API bool ActorLineTraceSingle(struct FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params) const;

	/** 
	 * returns Distance to closest Body Instance surface. 
	 * Checks against all components of this Actor having valid collision and blocking TraceChannel.
	 *
	 * @param Point						World 3D vector
	 * @param TraceChannel				The 'channel' used to determine which components to consider.
	 * @param ClosestPointOnCollision	Point on the surface of collision closest to Point
	 * @param OutPrimitiveComponent		PrimitiveComponent ClosestPointOnCollision is on.
	 * 
	 * @return		Success if returns > 0.f, if returns 0.f, it is either not convex or inside of the point
	 *				If returns < 0.f, this Actor does not have any primitive with collision
	 */
	ENGINE_API float ActorGetDistanceToCollision(const FVector& Point, ECollisionChannel TraceChannel, FVector& ClosestPointOnCollision, UPrimitiveComponent** OutPrimitiveComponent = nullptr) const;

	/** Returns true if this actor is contained by TestLevel. */
	ENGINE_API bool IsInLevel(const class ULevel *TestLevel) const;

	/** Return the ULevel that this Actor is part of. */
	UFUNCTION(BlueprintCallable, Category=Level)
	ENGINE_API ULevel* GetLevel() const;

	/** Return the FTransform of the level this actor is a part of. */
	UFUNCTION(BlueprintCallable, Category=Level)
	ENGINE_API FTransform GetLevelTransform() const;

	/**	Do anything needed to clear out cross level references; Called from ULevel::PreSave	 */
	ENGINE_API virtual void ClearCrossLevelReferences();
	
	/** Non-virtual function to evaluate which portions of the EndPlay process should be dispatched for each actor */
	ENGINE_API void RouteEndPlay(const EEndPlayReason::Type EndPlayReason);

	/**
	 * Iterates up the movement base chain to see whether or not this Actor is based on the given Actor, defaults to checking attachment
	 * @param Other the Actor to test for
	 * @return true if this Actor is based on Other Actor
	 */
	ENGINE_API virtual bool IsBasedOnActor(const AActor* Other) const;
	
	/**
	 * Iterates up the attachment chain to see whether or not this Actor is attached to the given Actor
	 * @param Other the Actor to test for
	 * @return true if this Actor is attached on Other Actor
	 */
	ENGINE_API virtual bool IsAttachedTo( const AActor* Other ) const;

	/** Get the extent used when placing this actor in the editor, used for 'pulling back' hit. */
	ENGINE_API FVector GetPlacementExtent() const;


	//~=============================================================================
	// Blueprint

#if WITH_EDITOR
	/** Find all FRandomStream structs in this Actor and generate new random seeds for them. */
	ENGINE_API void SeedAllRandomStreams();
#endif // WITH_EDITOR

	/** Reset private properties to defaults, and all FRandomStream structs in this Actor, so they will start their sequence of random numbers again. */
	ENGINE_API void ResetPropertiesForConstruction();

	/** Returns true if the actor's class has a non trivial user construction script. */
	ENGINE_API bool HasNonTrivialUserConstructionScript() const;

#if WITH_EDITOR
	/** Rerun construction scripts, destroying all autogenerated components; will attempt to preserve the root component location. */
	ENGINE_API virtual void RerunConstructionScripts();
#endif

	/** 
	 * Debug helper to show the component hierarchy of this actor.
	 * @param Info			Optional String to display at top of info
	 * @param bShowPosition	If true, will display component's position in world space
	 */
	ENGINE_API void DebugShowComponentHierarchy( const TCHAR* Info, bool bShowPosition  = true);
	
	/** Debug helper for showing the component hierarchy of one component */
	ENGINE_API void DebugShowOneComponentHierarchy( USceneComponent* SceneComp, int32& NestLevel, bool bShowPosition );

	/**
	 * Run any construction script for this Actor. Will call OnConstruction.
	 * @param	Transform			The transform to construct the actor at.
	 * @param   TransformRotationCache Optional rotation cache to use when applying the transform.
	 * @param	InstanceDataCache	Optional cache of state to apply to newly created components (e.g. precomputed lighting)
	 * @param	bIsDefaultTransform	Whether or not the given transform is a "default" transform, in which case it can be overridden by template defaults
	 *
	 * @return Returns false if the hierarchy was not error free and we've put the Actor is disaster recovery mode
	 */
	ENGINE_API bool ExecuteConstruction(const FTransform& Transform, const struct FRotationConversionCache* TransformRotationCache, const class FComponentInstanceDataCache* InstanceDataCache, bool bIsDefaultTransform = false, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::OverrideRootScale);

	/**
	 * Called when an instance of this class is placed (in editor) or spawned.
	 * @param	Transform			The transform the actor was constructed at.
	 */
	virtual void OnConstruction(const FTransform& Transform) {}

	/**
	 * Helper function to register the specified component, and add it to the serialized components array
	 * @param	Component	Component to be finalized
	 */
	ENGINE_API void FinishAndRegisterComponent(UActorComponent* Component);

	/**  Util to create a component based on a template	 */
	ENGINE_API UActorComponent* CreateComponentFromTemplate(UActorComponent* Template, const FName InName = NAME_None );
	ENGINE_API UActorComponent* CreateComponentFromTemplateData(const struct FBlueprintCookedComponentInstancingData* TemplateData, const FName InName = NAME_None);

	/** Destroys the constructed components. */
	ENGINE_API void DestroyConstructedComponents();

	ENGINE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	struct FFillReplicationParamsContext
	{
	};
	/**
	 * Called if using iris replication right before an actor start replicating, override to provide custom actor replication parameters 
	 */
	ENGINE_API virtual void FillReplicationParams(const FFillReplicationParamsContext& Context, FActorReplicationParams& OutParams);

protected:

	struct FOnReplicationStartedParams
	{
	};
	/**
	 * Called if using iris replication after an actor has started to replicate with iris replication, Normally this will occur right before BeginPlay, but might will also be called later if SetReplicates(true) is called and the object is not already replicating
	 */
	ENGINE_API virtual void OnReplicationStartedForIris(const FOnReplicationStartedParams&);

	struct FOnStopReplicationParams
	{
		const FStopReplicatingActorParams& StopReplicatingParams;
	};
	/**
	 * Called if using iris replication when we are about to stop replicating an actor, typically called from EndPlay() for actors that should be replicated during their lifespan
	 */
	ENGINE_API virtual void OnStopReplicationForIris(const FOnStopReplicationParams&);

#if !UE_STRIP_DEPRECATED_PROPERTIES 
	/**
	* Called for all Actors set to replicate during BeginPlay, It will also be called if SetReplicates(true) is called and the object is not already replicating
	*/
	UE_DEPRECATED(5.7, "This is being deprecated as part of iris beta. Use OnReplicationStartedForIris or FillReplicationParams overrides instead.")
	virtual void BeginReplication() {}

	/**
	* Called when we want to end replication for this actor, typically called from EndPlay() for actors that should be replicated during their lifespan
	*/
	UE_DEPRECATED(5.7, "This is being deprecated as part of iris beta. Override OnStopReplicationForIris instead.")
	virtual void EndReplication(EEndPlayReason::Type EndPlayReason) {}

	/**
	* Helper to BeginReplication passing on additional parameters to the ReplicationSystem, typically called from code overriding normal BeginReplication()
	* @param Params Additional parameters we want to pass on
	*/
	UE_DEPRECATED(5.7, "This is being deprecated as part of iris beta. Do not call directly override FillReplicationParams instead to provide custom parameters.")
	void BeginReplication(const FActorReplicationParams& Params) {}
	
#endif

	/**
	 * Pushes the owning NetConnection for the actor and all of its children to the replication system.
	 * This information decides whether properties with owner conditionals are replicated or not.
	 */
	ENGINE_API void UpdateOwningNetConnection();

	// Required by both these classes to call OnReplicationStartedForIris().
	friend struct UE::Net::FReplicationSystemUtil;
	friend class UEngineReplicationBridge;

	/**
	 * Updates the ReplicatePhysics condition. That information needs to be pushed to the ReplicationSystem.
	 */
	ENGINE_API void UpdateReplicatePhysicsCondition();
	
protected:
	/**
	 * Virtual call chain to register all tick functions for the actor class hierarchy
	 * @param bRegister - true to register, false, to unregister
	 */
	ENGINE_API virtual void RegisterActorTickFunctions(bool bRegister);

	/** Runs UserConstructionScript, delays component registration until it's complete. */
	ENGINE_API void ProcessUserConstructionScript();

	/** Checks components for validity, implemented in AActor */
	ENGINE_API bool CheckActorComponents() const;

	/** Called after instancing a new Blueprint Component from either a template or cooked data. */
	ENGINE_API void PostCreateBlueprintComponent(UActorComponent* NewActorComp);

public:
	/** Checks for and resolve any name conflicts prior to instancing a new Blueprint Component. */
	ENGINE_API void CheckComponentInstanceName(const FName InName);

	/** Walk up the attachment chain from RootComponent until we encounter a different actor, and return it. If we are not attached to a component in a different actor, returns nullptr */
	UFUNCTION(BlueprintPure, Category = "Actor")
	ENGINE_API AActor* GetAttachParentActor() const;

	/** Walk up the attachment chain from RootComponent until we encounter a different actor, and return the socket name in the component. If we are not attached to a component in a different actor, returns NAME_None */
	UFUNCTION(BlueprintPure, Category = "Actor")
	ENGINE_API FName GetAttachParentSocketName() const;

	/** Call a functor for Actors which are attached directly to a component in this actor. Functor should return true to carry on, false to abort. */
	ENGINE_API void ForEachAttachedActors(TFunctionRef<bool(class AActor*)> Functor) const;
	
	/** Find all Actors which are attached directly to a component in this actor */
	UFUNCTION(BlueprintPure, Category = "Actor")
	ENGINE_API void GetAttachedActors(TArray<AActor*>& OutActors, bool bResetArray = true, bool bRecursivelyIncludeAttachedActors = false) const;

	/**
	 * Sets the ticking group for this actor.
	 * @param NewTickGroup the new value to assign
	 */
	UFUNCTION(BlueprintCallable, Category="Actor|Tick", meta=(Keywords = "dependency"))
	ENGINE_API void SetTickGroup(ETickingGroup NewTickGroup);

	/** Called when this actor is explicitly being destroyed during gameplay or in the editor, not called during level streaming or gameplay ending */
	ENGINE_API virtual void Destroyed();

	/** Call ReceiveHit, as well as delegates on Actor and Component */
	ENGINE_API void DispatchBlockingHit(UPrimitiveComponent* MyComp, UPrimitiveComponent* OtherComp, bool bSelfMoved, FHitResult const& Hit);

	/** Called when the actor falls out of the world 'safely' (below KillZ and such) */
	ENGINE_API virtual void FellOutOfWorld(const class UDamageType& dmgType);

	/** Called when the Actor is outside the hard limit on world bounds */
	ENGINE_API virtual void OutsideWorldBounds();

	/** 
	 * Returns the world space bounding box of all components in this Actor.
	 * @param bNonColliding Indicates that you want to include non-colliding components in the bounding box
	 * @param bIncludeFromChildActors If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	ENGINE_API virtual FBox GetComponentsBoundingBox(bool bNonColliding = false, bool bIncludeFromChildActors = false) const;

	/** 
	 * Calculates the actor space bounding box of all components in this Actor.  This is slower than GetComponentsBoundingBox(), because the local bounds of the components are not cached -- they are recalculated every time this function is called.
	 * @param bNonColliding Indicates that you want to include non-colliding components in the bounding box
	 * @param bIncludeFromChildActors If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	ENGINE_API virtual FBox CalculateComponentsBoundingBoxInLocalSpace(bool bNonColliding = false, bool bIncludeFromChildActors = false) const;

	/** 
	 * Get half-height/radius of a big axis-aligned cylinder around this actors registered colliding components, or all registered components if bNonColliding is false. 
	 * @param bNonColliding Indicates that you want to include non-colliding components in the bounding cylinder
	 * @param bIncludeFromChildActors If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	*/
	ENGINE_API virtual void GetComponentsBoundingCylinder(float& CollisionRadius, float& CollisionHalfHeight, bool bNonColliding = false, bool bIncludeFromChildActors = false) const;

	/**
	 * Get axis-aligned cylinder around this actor, used for simple collision checks (ie Pawns reaching a destination).
	 * If IsRootComponentCollisionRegistered() returns true, just returns its bounding cylinder, otherwise falls back to GetComponentsBoundingCylinder.
	 */
	ENGINE_API virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const;

	/** Returns the radius of the collision cylinder from GetSimpleCollisionCylinder(). */
	float GetSimpleCollisionRadius() const;

	/** Returns the half height of the collision cylinder from GetSimpleCollisionCylinder(). */
	float GetSimpleCollisionHalfHeight() const;

	/** Returns collision extents vector for this Actor, based on GetSimpleCollisionCylinder(). */
	FVector GetSimpleCollisionCylinderExtent() const;

	/** Returns true if the root component is registered and has collision enabled.  */
	ENGINE_API virtual bool IsRootComponentCollisionRegistered() const;

	/**
	 * Networking - called on client when actor is torn off (bTearOff==true), meaning it's no longer replicated to clients.
	 * @see bTearOff
	 */
	ENGINE_API virtual void TornOff();


	//~=============================================================================
	// Collision/Physics functions.
 
	/** 
	 * Get Collision Response to the passed in Channel for all components
	 * It returns Max of state - i.e. if Component A overlaps, but if Component B blocks, it will return block as response
	 * if Component A ignores, but if Component B overlaps, it will return overlap
	 */
	ENGINE_API virtual ECollisionResponse GetComponentsCollisionResponseToChannel(ECollisionChannel Channel) const;

	/** Stop all simulation from all components in this actor */
	ENGINE_API void DisableComponentsSimulatePhysics();

	/**
	 * Returns the WorldSettings for the World the actor is in
	 * If you'd like to know what UWorld this placed actor (not dynamic spawned actor) belong to, use GetTypedOuter<UWorld>()
	 */
	ENGINE_API class AWorldSettings* GetWorldSettings() const;

	/**
	 * Return true if the given Pawn can be "based" on this actor (ie walk on it).
	 * @param Pawn - The pawn that wants to be based on this actor
	 */
	ENGINE_API virtual bool CanBeBaseForCharacter(class APawn* Pawn) const;

	/**
	 * Apply damage to this actor.
	 * @see https://www.unrealengine.com/blog/damage-in-ue4
	 * @param DamageAmount		How much damage to apply
	 * @param DamageEvent		Data package that fully describes the damage received.
	 * @param EventInstigator	The Controller responsible for the damage.
	 * @param DamageCauser		The Actor that directly caused the damage (e.g. the projectile that exploded, the rock that landed on you)
	 * @return					The amount of damage actually applied.
	 */
	ENGINE_API virtual float TakeDamage(float DamageAmount, struct FDamageEvent const& DamageEvent, class AController* EventInstigator, AActor* DamageCauser);

protected:
	ENGINE_API virtual float InternalTakeRadialDamage(float Damage, struct FRadialDamageEvent const& RadialDamageEvent, class AController* EventInstigator, AActor* DamageCauser);
	ENGINE_API virtual float InternalTakePointDamage(float Damage, struct FPointDamageEvent const& PointDamageEvent, class AController* EventInstigator, AActor* DamageCauser);

public:
	/** Called when this actor becomes the given PlayerController's ViewTarget. Triggers the Blueprint event K2_OnBecomeViewTarget. */
	ENGINE_API virtual void BecomeViewTarget( class APlayerController* PC );

	/** Called when this actor is no longer the given PlayerController's ViewTarget. Also triggers the Blueprint event K2_OnEndViewTarget. */
	ENGINE_API virtual void EndViewTarget( class APlayerController* PC );

	/** Event called when this Actor becomes the view target for the given PlayerController. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="OnBecomeViewTarget", ScriptName="OnBecomeViewTarget", Keywords="Activate Camera"), Category=Actor)
	ENGINE_API void K2_OnBecomeViewTarget( class APlayerController* PC );

	/** Event called when this Actor is no longer the view target for the given PlayerController. */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="OnEndViewTarget", ScriptName="OnEndViewTarget", Keywords="Deactivate Camera"), Category=Actor)
	ENGINE_API void K2_OnEndViewTarget( class APlayerController* PC );

	/**
	 * Calculate camera view point, when viewing this actor.
	 *
	 * @param	DeltaTime	Delta time seconds since last update
	 * @param	OutResult	Camera configuration
	 */
	ENGINE_API virtual void CalcCamera(float DeltaTime, struct FMinimalViewInfo& OutResult);

	/** Returns true if the actor contains an active camera component */
	ENGINE_API virtual bool HasActiveCameraComponent(bool bForceFindCamera = false) const;

	/** Returns true if the actor contains an active locked to HMD camera component */
	ENGINE_API virtual bool HasActivePawnControlCameraComponent() const;

	/** Returns the human readable string representation of an object. */
	ENGINE_API virtual FString GetHumanReadableName() const;

	/** Reset actor to initial state - used when restarting level without reloading. */
	ENGINE_API virtual void Reset();

	/** Event called when this Actor is reset to its initial state - used when restarting level without reloading. */
	UFUNCTION(BlueprintImplementableEvent, Category=Actor, meta=(DisplayName="OnReset", ScriptName="OnReset"))
	ENGINE_API void K2_OnReset();

	/**
	 * Returns true if this actor has been rendered "recently", with a tolerance in seconds to define what "recent" means. 
	 * e.g.: If a tolerance of 0.1 is used, this function will return true only if the actor was rendered in the last 0.1 seconds of game time. 
	 *
	 * @param Tolerance  How many seconds ago the actor last render time can be and still count as having been "recently" rendered.
	 * @return Whether this actor was recently rendered.
	 */
	UFUNCTION(Category="Rendering", BlueprintCallable, meta=(DisplayName="Was Actor Recently Rendered", Keywords="scene visible"))
	ENGINE_API bool WasRecentlyRendered(float Tolerance = 0.2f) const;

	/** Returns the most recent time any of this actor's components were rendered */
	ENGINE_API virtual float GetLastRenderTime() const;

	/** Forces this actor to be net relevant if it is not already by default	 */
	ENGINE_API virtual void ForceNetRelevant();

	/** Force actor to be updated to clients/demo net drivers */
	UFUNCTION( BlueprintCallable, Category="Networking")
	ENGINE_API virtual void ForceNetUpdate();

	/**
	 * Calls PrestreamTextures() for all the actor's meshcomponents.
	 * @param Seconds - Number of seconds to force all mip-levels to be resident
	 * @param bEnableStreaming	- Whether to start (true) or stop (false) streaming
	 * @param CinematicTextureGroups - Bitfield indicating which texture groups that use extra high-resolution mips
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering")
	ENGINE_API virtual void PrestreamTextures( float Seconds, bool bEnableStreaming, int32 CinematicTextureGroups = 0 );

	/**
	 * Returns the point of view of the actor.
	 * Note that this doesn't mean the camera, but the 'eyes' of the actor.
	 * For example, for a Pawn, this would define the eye height location,
	 * and view rotation (which is different from the pawn rotation which has a zeroed pitch component).
	 * A camera first person view will typically use this view point. Most traces (weapon, AI) will be done from this view point.
	 *
	 * @param	OutLocation - location of view point
	 * @param	OutRotation - view rotation of actor.
	 */
	UFUNCTION(BlueprintCallable, Category = Actor)
	ENGINE_API virtual void GetActorEyesViewPoint( FVector& OutLocation, FRotator& OutRotation ) const;

	/**
	 * Returns the optimal location to fire weapons at this actor
	 * @param RequestedBy - the Actor requesting the target location
	 */
	ENGINE_API virtual FVector GetTargetLocation(AActor* RequestedBy = nullptr) const;

	/**
	 * Hook to allow actors to render HUD overlays for themselves.  Called from AHUD::DrawActorOverlays(). 
	 * @param PC is the PlayerController on whose view this overlay is rendered
	 * @param Canvas is the Canvas on which to draw the overlay
	 * @param CameraPosition Position of Camera
	 * @param CameraDir direction camera is pointing in.
	 */
	ENGINE_API virtual void PostRenderFor(class APlayerController* PC, class UCanvas* Canvas, FVector CameraPosition, FVector CameraDir);

	/** Returns whether this Actor is in the persistent level, i.e. not a sublevel */
	ENGINE_API bool IsInPersistentLevel(bool bIncludeLevelStreamingPersistent = false) const;

	/** Getter for the cached world pointer, will return null if the actor is not actually spawned in a level */
	ENGINE_API virtual UWorld* GetWorld() const override final;

	/** Get the timer instance from the actors world */
	ENGINE_API class FTimerManager& GetWorldTimerManager() const;

	/** Gets the GameInstance that ultimately contains this actor. */
	ENGINE_API class UGameInstance* GetGameInstance() const;
	
	/** 
	 * Gets the GameInstance that ultimately contains this actor cast to the template type.
	 * May return NULL if the cast fails. 
	 */
	template< class T >
	T* GetGameInstance() const 
	{ 
		return Cast<T>(GetGameInstance()); 
	}

	/** Returns true if this is a replicated actor that was placed in the map */
	ENGINE_API bool IsNetStartupActor() const;

	/** Searches components array and returns first encountered component of the specified class, native version of GetComponentByClass */
	ENGINE_API virtual UActorComponent* FindComponentByClass(const TSubclassOf<UActorComponent> ComponentClass) const;
	
	/** Searches components array and returns first encountered component of the specified class */
	UFUNCTION(BlueprintCallable, Category = "Actor", meta = (ComponentClass = "/Script/Engine.ActorComponent"), meta = (DeterminesOutputType = "ComponentClass"))
	ENGINE_API UActorComponent* GetComponentByClass(TSubclassOf<UActorComponent> ComponentClass) const;

	/** Templated version of GetComponentByClass */
	template<class T>
	T* GetComponentByClass() const
	{
		return FindComponentByClass<T>();
	}

	/**
	 * Gets all the components that inherit from the given class.
	 * Currently returns an array of UActorComponent which must be cast to the correct type.
	 * This intended to only be used by blueprints. Use GetComponents() in C++.
	 */
	UFUNCTION(BlueprintCallable, Category = "Actor", meta = (ComponentClass = "/Script/Engine.ActorComponent", DisplayName = "Get Components By Class", ScriptName = "GetComponentsByClass", DeterminesOutputType = "ComponentClass"))
	ENGINE_API TArray<UActorComponent*> K2_GetComponentsByClass(TSubclassOf<UActorComponent> ComponentClass) const;

	/** Searches components array and returns first encountered component with a given tag. */
	UFUNCTION(BlueprintCallable, Category = "Actor", meta = (ComponentClass = "/Script/Engine.ActorComponent"), meta = (DeterminesOutputType = "ComponentClass"))
	ENGINE_API UActorComponent* FindComponentByTag(TSubclassOf<UActorComponent> ComponentClass, FName Tag) const;

	/** Gets all the components that inherit from the given class with a given tag. */
	UFUNCTION(BlueprintCallable, Category = "Actor", meta = (ComponentClass = "/Script/Engine.ActorComponent"), meta = (DeterminesOutputType = "ComponentClass"))
	ENGINE_API TArray<UActorComponent*> GetComponentsByTag(TSubclassOf<UActorComponent> ComponentClass, FName Tag) const;

	/** Searches components array and returns first encountered component that implements the given interface. */
	ENGINE_API virtual UActorComponent* FindComponentByInterface(const TSubclassOf<UInterface> Interface) const;
	
	/** Gets all the components that implements the given interface. */
	UFUNCTION(BlueprintCallable, Category = "Actor")
	ENGINE_API TArray<UActorComponent*> GetComponentsByInterface(TSubclassOf<UInterface> Interface) const;

	/** Templatized version of FindComponentByClass that handles casting for you */
	template<class T>
	T* FindComponentByClass() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, const UActorComponent>::Value, "'T' template parameter to FindComponentByClass must be derived from UActorComponent");

		return (T*)FindComponentByClass(T::StaticClass());
	}
	
	/** Templatized version of FindComponentByTag that handles casting for you */
	template<class T>
	T* FindComponentByTag(const FName& Tag) const
	{
		static_assert(TPointerIsConvertibleFromTo<T, const UActorComponent>::Value, "'T' template parameter to FindComponentByTag must be derived from UActorComponent");

		return (T*)FindComponentByTag(T::StaticClass(), Tag);
	}

	/** Templatized version of FindComponentByInterface that handles casting for you */
	template<class T UE_REQUIRES(TPointerIsConvertibleFromTo<T, UInterface>::Value)>
	UE_DEPRECATED(5.5, "This version incorrectly casts to the UInterface type used for reflection. Use FindComponentByInterface<IMyInterface>() instead")
	T* FindComponentByInterface() const
	{
		return (T*)FindComponentByInterface(T::StaticClass());
	}

	/** Templatized version of FindComponentByInterface that handles casting for you */
	template<class T UE_REQUIRES(TIsIInterface<T>::Value)>
	T* FindComponentByInterface() const
	{
		return Cast<T>(FindComponentByInterface(T::UClassType::StaticClass()));
	}

private:

	/**
	 * Internal helper function to call a compile-time lambda on all components of a given type
	 * Use template parameter bClassIsActorComponent to avoid doing unnecessary IsA checks when the ComponentClass is exactly UActorComponent
	 * Use template parameter bIncludeFromChildActors to recurse in to ChildActor components and find components of the appropriate type in those actors as well
	 */
	template<class ComponentType, bool bClassIsActorComponent, bool bIncludeFromChildActors, typename Func>
	void ForEachComponent_Internal(TSubclassOf<UActorComponent> InComponentClass, Func InFunc) const
	{
		if (const UClass* ComponentClass = InComponentClass)
		{
			check(bClassIsActorComponent == false || ComponentClass == UActorComponent::StaticClass());
			check(ComponentClass->IsChildOf(ComponentType::StaticClass()));

			// static check, so that the most common case (bIncludeFromChildActors) doesn't need to allocate an additional array : 
			if (bIncludeFromChildActors)
			{
				TArray<AActor*, TInlineAllocator<NumInlinedActorComponents>> ChildActors;
				for (UActorComponent* OwnedComponent : OwnedComponents)
				{
					if (OwnedComponent)
					{
						if (bClassIsActorComponent || OwnedComponent->IsA(ComponentClass))
						{
							InFunc(static_cast<ComponentType *>(OwnedComponent));
						}
						if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(OwnedComponent))
						{
							if (AActor* ChildActor = ChildActorComponent->GetChildActor())
							{
								ChildActors.Add(ChildActor);
							}
						}
					}
				}

				for (AActor* ChildActor : ChildActors)
				{
					ChildActor->ForEachComponent_Internal<ComponentType, bClassIsActorComponent, bIncludeFromChildActors>(InComponentClass, InFunc);
				}
			}
			else
			{
				for (UActorComponent* OwnedComponent : OwnedComponents)
				{
					if (OwnedComponent)
					{
						if (bClassIsActorComponent || OwnedComponent->IsA(ComponentClass))
						{
							InFunc(static_cast<ComponentType *>(OwnedComponent));
						}
					}
				}
			}
		}
	}

	/**
	 * Internal helper function to call a compile-time lambda on all components of a given type
	 * Redirects the call to the proper template function specialization for bClassIsActorComponent and bIncludeFromChildActors :
	 */
	template<class ComponentType, typename Func>
	void ForEachComponent_Internal(TSubclassOf<UActorComponent> ComponentClass, bool bIncludeFromChildActors, Func InFunc) const
	{
		static_assert(TPointerIsConvertibleFromTo<ComponentType, const UActorComponent>::Value, "'ComponentType' template parameter to ForEachComponent must be derived from UActorComponent");
		if (ComponentClass == UActorComponent::StaticClass())
		{
			if (bIncludeFromChildActors)
			{
				ForEachComponent_Internal<ComponentType, true /*bClassIsActorComponent*/, true /*bIncludeFromChildActors*/>(ComponentClass, InFunc);
			}
			else
			{
				ForEachComponent_Internal<ComponentType, true /*bClassIsActorComponent*/, false /*bIncludeFromChildActors*/>(ComponentClass, InFunc);
			}
		}
		else
		{
			if (bIncludeFromChildActors)
			{
				ForEachComponent_Internal<ComponentType, false /*bClassIsActorComponent*/, true /*bIncludeFromChildActors*/>(ComponentClass, InFunc);
			}
			else
			{
				ForEachComponent_Internal<ComponentType, false /*bClassIsActorComponent*/, false /*bIncludeFromChildActors*/>(ComponentClass, InFunc);
			}
		}
	}

public:

	/**
	 * Calls the compile-time lambda on each component of the specified type
	 * @param ComponentType             The component class to find all components of a class derived from
	 * @param bIncludeFromChildActors   If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<class ComponentType, typename Func>
	void ForEachComponent(bool bIncludeFromChildActors, Func InFunc) const
	{		
		ForEachComponent_Internal<ComponentType>(ComponentType::StaticClass(), bIncludeFromChildActors, InFunc);
	}

	/**
	 * Calls the compile-time lambda on each valid component 
	 * @param bIncludeFromChildActors   If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<typename Func>
	void ForEachComponent(bool bIncludeFromChildActors, Func InFunc) const
	{
		ForEachComponent_Internal<UActorComponent>(UActorComponent::StaticClass(), bIncludeFromChildActors, InFunc);
	}

	/**
	 * Get all components derived from specified ComponentClass and fill in the OutComponents array with the result.
	 * It's recommended to use TArrays with a TInlineAllocator to potentially avoid memory allocation costs.
	 * TInlineComponentArray is defined to make this easier, for example:
	 * {
	 * 	   TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor);
	 * }
	 *
	 * @param ComponentClass            The component class to find all components of a class derived from
	 * @param bIncludeFromChildActors   If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	*/
	template<class AllocatorType, class ComponentType>
	void GetComponents(TSubclassOf<UActorComponent> ComponentClass, TArray<ComponentType*, AllocatorType>& OutComponents, bool bIncludeFromChildActors = false) const
	{
		OutComponents.Reset();
		ForEachComponent_Internal<ComponentType>(ComponentClass, bIncludeFromChildActors, [&](ComponentType* InComp)
		{
			OutComponents.Add(InComp);
		});
	}

	/**
	 * Get all components derived from class 'ComponentType' and fill in the OutComponents array with the result.
	 * It's recommended to use TArrays with a TInlineAllocator to potentially avoid memory allocation costs.
	 * TInlineComponentArray is defined to make this easier, for example:
	 * {
	 * 	   TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor);
	 * }
	 *
	 * @param bIncludeFromChildActors	If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<class ComponentType, class AllocatorType>
	void GetComponents(TArray<ComponentType, AllocatorType>& OutComponents, bool bIncludeFromChildActors = false) const
	{
		typedef TPointedToType<ComponentType> T;

		OutComponents.Reset();
		ForEachComponent_Internal<T>(T::StaticClass(), bIncludeFromChildActors, [&](T* InComp)
		{
			OutComponents.Add(InComp);
		});
	}

	/**
	 * Get all components derived from class 'ComponentType' and fill in the OutComponents array with the result.
	 * It's recommended to use TArrays with a TInlineAllocator to potentially avoid memory allocation costs.
	 * TInlineComponentArray is defined to make this easier, for example:
	 * {
	 * 	   TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor);
	 * }
	 *
	 * @param bIncludeFromChildActors	If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<class T, class AllocatorType>
	void GetComponents(TArray<T*, AllocatorType>& OutComponents, bool bIncludeFromChildActors = false) const
	{
		// We should consider removing this function.  It's not really hurting anything by existing but the one above it was added so that
		// we weren't assuming T*, preventing TObjectPtrs from working for this function.  The only downside is all the people who force the
		// template argument with GetComponents's code suddenly not compiling with no clear error message.

		OutComponents.Reset();
		ForEachComponent_Internal<T>(T::StaticClass(), bIncludeFromChildActors, [&](T* InComp)
		{
			OutComponents.Add(InComp);
		});
	}

	/**
	 * Get all components derived from class 'T' and fill in the OutComponents array with the result.
	 * It's recommended to use TArrays with a TInlineAllocator to potentially avoid memory allocation costs.
	 * TInlineComponentArray is defined to make this easier, for example:
	 * {
	 * 	   TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor);
	 * }
	 *
	 * @param bIncludeFromChildActors	If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<class T, class AllocatorType>
	void GetComponents(TArray<TObjectPtr<T>, AllocatorType>& OutComponents, bool bIncludeFromChildActors = false) const
	{
		OutComponents.Reset();
		ForEachComponent_Internal<T>(T::StaticClass(), bIncludeFromChildActors, [&](T* InComp)
			{
				OutComponents.Add(InComp);
			});
	}

	/**
	 * UActorComponent specialization of GetComponents() to avoid unnecessary casts.
	 * It's recommended to use TArrays with a TInlineAllocator to potentially avoid memory allocation costs.
	 * TInlineComponentArray is defined to make this easier, for example:
	 * {
	 * 	   TInlineComponentArray<UActorComponent*> PrimComponents;
	 *     Actor->GetComponents(PrimComponents);
	 * }
	 *
	 * @param bIncludeFromChildActors	If true then recurse in to ChildActor components and find components of the appropriate type in those Actors as well
	 */
	template<class AllocatorType>
	void GetComponents(TArray<UActorComponent*, AllocatorType>& OutComponents, bool bIncludeFromChildActors = false) const
	{
		OutComponents.Reset();
		ForEachComponent_Internal<UActorComponent>(UActorComponent::StaticClass(), bIncludeFromChildActors, [&](UActorComponent* InComp)
		{
			OutComponents.Add(InComp);
		});
	}

	/**
	 * Get a direct reference to the Components set rather than a copy with the null pointers removed.
	 * WARNING: anything that could cause the component to change ownership or be destroyed will invalidate
	 * this array, so use caution when iterating this set!
	 */
	const TSet<UActorComponent*>& GetComponents() const
	{
		return ObjectPtrDecay(OwnedComponents);
	}

	/**
	 * Puts a component in to the OwnedComponents array of the Actor.
	 * The Component must be owned by the Actor or else it will assert
	 * In general this should not need to be called directly by anything other than UActorComponent functions
	 */
	ENGINE_API void AddOwnedComponent(UActorComponent* Component);

	/**
	 * Removes a component from the OwnedComponents array of the Actor.
	 * In general this should not need to be called directly by anything other than UActorComponent functions
	 */
	ENGINE_API void RemoveOwnedComponent(UActorComponent* Component);

#if DO_CHECK || USING_CODE_ANALYSIS
	/** Utility function for validating that a component is correctly in its Owner's OwnedComponents array */
	ENGINE_API bool OwnsComponent(UActorComponent* Component) const;
#endif

	/**
	 * Force the Actor to clear and rebuild its OwnedComponents array by evaluating all children (recursively) and locating components
	 * In general this should not need to be called directly, but can sometimes be necessary as part of undo/redo code paths.
	 */
	ENGINE_API void ResetOwnedComponents();

private:
	
	ENGINE_API void ResetOwnedComponents(EAddComponentForReplicationBehavior Type);

public:

	/** Called when the replicated state of a component changes to update the Actor's cached ReplicatedComponents array */
	ENGINE_API void UpdateReplicatedComponent(UActorComponent* Component);

	/** Completely synchronizes the replicated components array so that it contains exactly the number of replicated components currently owned */
	ENGINE_API void UpdateAllReplicatedComponents();

	/** 
	* Allows classes to control if a replicated component can actually be replicated or not in a specific actor class. 
	* You can also choose a netcondition to filter to whom the component is replicated to.
	* Called on existing replicated component right before BeginPlay() and after that on every new replicated component added to the OwnedComponent list
    *
	* @param ComponentToReplicate The replicated component added to the actor.
	* @return Return COND_None if this component should be replicated to everyone, COND_Never if it should not be replicated at all or any other conditions for specific filtering.
	*/
	ENGINE_API virtual ELifetimeCondition AllowActorComponentToReplicate(const UActorComponent* ComponentToReplicate) const;

	/** 
	* Change the network condition of a replicated component but only after BeginPlay. 
	* Using a network condition can allow you to filter to which client the component gets replicated to. 
	*/
	ENGINE_API void SetReplicatedComponentNetCondition(const UActorComponent* ReplicatedComponent, ELifetimeCondition NetCondition);

	/** Returns whether replication is enabled or not. */
	inline bool GetIsReplicated() const
	{
		return bReplicates;
	}

	/** Returns a constant reference to the replicated components set */
	const TArray<UActorComponent*>& GetReplicatedComponents() const
	{ 
		return ReplicatedComponents; 
	}

	/**
	* Register a SubObject that will get replicated along with the actor.
	* The subobject needs to be manually removed from the list before it gets deleted.
	* @param SubObject The SubObject to replicate
	* @param NetCondition Optional condition to select which type of connection we will replicate the object to.
	*/
	ENGINE_API void AddReplicatedSubObject(UObject* SubObject, ELifetimeCondition NetCondition = COND_None);

	/**
	* Unregister a SubObject to stop replicating it's properties to clients.
	* This does not remove or delete it from connections where it was already replicated.
	* By default a replicated subobject gets deleted on clients when the original pointer on the authority becomes invalid.
	* If you want to immediately remove it from client use the DestroyReplicatedSubObjectOnRemotePeers or TearOffReplicatedSubObject functions instead of this one.
	* @param SubObject The SubObject to remove
	*/
	ENGINE_API void RemoveReplicatedSubObject(UObject* SubObject);

	/**
	* Stop replicating a subobject and tell actor channels to delete the replica of this subobject next time the Actor gets replicated.
	* Note it is up to the caller to delete the local object on the authority.
	* If you are using the legacy subobject replication method (ReplicateSubObjects() aka bReplicateUsingRegisteredSubObjectList=false) make sure the
	* subobject doesn't get replicated there either.
	* @param SubObject THe SubObject to delete
	*/
	ENGINE_API void DestroyReplicatedSubObjectOnRemotePeers(UObject* SubObject);

	/** Similar to the other destroy function but for subobjects owned by an ActorComponent */
	ENGINE_API void DestroyReplicatedSubObjectOnRemotePeers(UActorComponent* OwnerComponent, UObject* SubObject);

	/**
	* Stop replicating a subobject and tell actor channels who spawned a replica of this subobject to release ownership over it.
	* This means that on the remote connection the network engine will stop holding a reference to the subobject and it's up to other systems 
	* to keep that reference active or the subobject will get garbage collected.
    * Note that the subobject won't be replicated anymore, so it's final state on the client will be the one from the last replication update sent.
	* If you are using the legacy subobject replication method (ReplicateSubObjects() aka bReplicateUsingRegisteredSubObjectList=false) make sure the
	* subobject doesn't get replicated there either.
	* @param SubObject The SubObject to tear off
	*/
	ENGINE_API void TearOffReplicatedSubObjectOnRemotePeers(UObject* SubObject);

	/** Similar to the other tear off function but for subobjects owned by an ActorComponent */
	ENGINE_API void TearOffReplicatedSubObjectOnRemotePeers(UActorComponent* OwnerComponent, UObject* SubObject);

	/**
	* Register a SubObject that will get replicated along with the actor component owning it.
	* The subobject needs to be manually removed from the list before it gets deleted.
	* @param SubObject The SubObject to replicate
	* @param NetCondition Optional condition to select which type of connection we will replicate the object to.
	*/
	ENGINE_API void AddActorComponentReplicatedSubObject(UActorComponent* OwnerComponent, UObject* SubObject, ELifetimeCondition NetCondition = COND_None);

	/**
	* Unregister a SubObject owned by an ActorComponent so it stops being replicated.
	* @param SubObject The SubObject to remove
	*/
	ENGINE_API void RemoveActorComponentReplicatedSubObject(UActorComponent* OwnerComponent, UObject* SubObject);

	/** Tells if the object has been registered as a replicated subobject of this actor */
	ENGINE_API bool IsReplicatedSubObjectRegistered(const UObject* SubObject) const;

	/** Tells if the component has been registered as a replicated component */
	ENGINE_API bool IsReplicatedActorComponentRegistered(const UActorComponent* ReplicatedComponent) const;

	/** Tells if an object owned by a component has been registered as a replicated subobject of the component */
	ENGINE_API bool IsActorComponentReplicatedSubObjectRegistered(const UActorComponent* OwnerComponent, const UObject* SubObject) const;


	/**
	* Fetches all the components of ActorClass's CDO, including the ones added via the BP editor (which AActor.GetComponents fails to do for CDOs).
	* @param InActorClass		Class of AActor for which we will retrieve all components.
	* @param InComponentClass	Only retrieve components of this type.
	* @param OutComponents this is where the found components will end up. Note that the preexisting contents of OutComponents will get overridden.
	*/
	static ENGINE_API void GetActorClassDefaultComponents(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass, TArray<const UActorComponent*>& OutComponents);

	/**
	* Fetches the first component of ActorClass's CDO which match the requested component class. Will include the components added via the BP editor (which AActor.GetComponents fails to do for CDOs).
	* @param InActorClass		Class of AActor for which we will retrieve all components.
	* @param InComponentClass	Only retrieve components of this type.
	* @param OutComponents this is where the found components will end up. Note that the preexisting contents of OutComponents will get overridden.
	*/
	static ENGINE_API const UActorComponent* GetActorClassDefaultComponent(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass);

	/**
	* Get the component of ActorClass's CDO that matches the given object name. Will consider all components, including the ones added via the BP editor (which AActor.GetComponents fails to do for CDOs).
	* @param InActorClass		Class of AActor for which we will search all components.
	* @param InComponentClass	Only consider components of this type.
	* @param OutComponents this is where the found components will end up. Note that the preexisting contents of OutComponents will get overridden.
	*/
	static ENGINE_API const UActorComponent* GetActorClassDefaultComponentByName(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass, FName InComponentName);

	/**
	* Iterate over the components of ActorClass's CDO, including the ones added via the BP editor (which AActor.GetComponents fails to return).
	* @param InActorClass		Class of AActor for which we will retrieve all components.
	* @param InComponentClass	Only consider components of this type.
	* @param InFunc				Code that will be executed for each component. Must return true to continue iteration, or false to stop.
	*/
	static ENGINE_API void ForEachComponentOfActorClassDefault(const TSubclassOf<AActor>& InActorClass, const TSubclassOf<UActorComponent>& InComponentClass, TFunctionRef<bool(const UActorComponent*)> InFunc);

	/**
	* Templated version of GetActorClassDefaultComponents()
	* @see GetActorClassDefaultComponents
	*/
	template <typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	static void GetActorClassDefaultComponents(const TSubclassOf<AActor>& InActorClass, TArray<const TComponentClass*>& OutComponents)
	{
		ForEachComponentOfActorClassDefault(InActorClass, TComponentClass::StaticClass(), [&](const UActorComponent* TemplateComponent)
		{
			OutComponents.Add(CastChecked<TComponentClass>(TemplateComponent));
			return true;
		});
	}

	/**
	* Templated version of GetActorClassDefaultComponent()
	* @see GetActorClassDefaultComponent
	*/
	template <typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	static const TComponentClass* GetActorClassDefaultComponent(const TSubclassOf<AActor>& InActorClass)
	{
		return Cast<TComponentClass>(GetActorClassDefaultComponent(InActorClass, TComponentClass::StaticClass()));
	}

	/**
	* Templated version of GetActorClassDefaultComponentByName()
	* @see GetActorClassDefaultComponentByName
	*/
	template <typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	static const TComponentClass* GetActorClassDefaultComponentByName(const TSubclassOf<AActor>& InActorClass, FName InComponentName)
	{
		return Cast<TComponentClass>(GetActorClassDefaultComponentByName(InActorClass, TComponentClass::StaticClass(), InComponentName));
	}

	/**
	* Templated version of ForEachComponentOfActorClassDefault()
	* @see ForEachComponentOfActorClassDefault
	*/
	template <typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	static void ForEachComponentOfActorClassDefault(const TSubclassOf<AActor>& InActorClass, TFunctionRef<bool(const TComponentClass*)> InFunc)
	{
		ForEachComponentOfActorClassDefault(InActorClass, TComponentClass::StaticClass(), [&](const UActorComponent* TemplateComponent)
		{
			return InFunc(CastChecked<TComponentClass>(TemplateComponent));
		});
	}

private:
	/** Collection of SubObjects that get replicated when this actor gets replicated. */
	UE::Net::FSubObjectRegistry ReplicatedSubObjects;
	friend class UE::Net::FSubObjectRegistryGetter;

	/** Array of replicated components and the list of replicated subobjects they own. Replaces the deprecated ReplicatedCompoments array. */
	TArray<UE::Net::FReplicatedComponentInfo> ReplicatedComponentsInfo;

	/** Remove the subobject from the registry list. Returns true if the object was found and removed from the list. */
	ENGINE_API bool RemoveReplicatedSubObjectFromList(UObject* SubObject);

	/** Remove the subobject from a component's registry list. Returns true if the object was found and removed from the list. */
	ENGINE_API bool RemoveActorComponentReplicatedSubObjectFromList(UActorComponent* OwnerComponent, UObject* SubObject);

	/** Check if a new component is replicated by the actor and must be registered in the list */
	ENGINE_API void AddComponentForReplication(UActorComponent* Component);
	void AddComponentForReplication(UActorComponent* Component, EAddComponentForReplicationBehavior Type);

	/** Remove a component from the replicated list */
	ENGINE_API void RemoveReplicatedComponent(UActorComponent* Component);

	/** Constructs the list of replicated components and their netcondition */
	ENGINE_API void BuildReplicatedComponentsInfo();

protected:
	/** Set of replicated components, stored as an array to save space as this is generally not very large */
	TArray<UActorComponent*> ReplicatedComponents;

private:
	/**
	 * All ActorComponents owned by this Actor. Stored as a Set as actors may have a large number of components
	 * @see GetComponents()
	 */
	TSet<TObjectPtr<UActorComponent>> OwnedComponents;

#if WITH_EDITOR
	/** Maps natively-constructed components to properties that reference them. */
	TMultiMap<FName, FObjectProperty*> NativeConstructedComponentToPropertyMap;
#endif

	/** Array of ActorComponents that have been added by the user on a per-instance basis. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UActorComponent>> InstanceComponents;

public:
	/** Array of ActorComponents that are created by blueprints and serialized per-instance. */
	UPROPERTY(TextExportTransient, NonTransactional)
	TArray<TObjectPtr<UActorComponent>> BlueprintCreatedComponents;

	/** Adds a component to the instance components array */
	ENGINE_API void AddInstanceComponent(UActorComponent* Component);

	/** Removes a component from the instance components array */
	ENGINE_API void RemoveInstanceComponent(UActorComponent* Component);

	/** Clears the instance components array */
	ENGINE_API void ClearInstanceComponents(bool bDestroyComponents);

	/** Returns the instance components array */
	ENGINE_API const TArray<UActorComponent*>& GetInstanceComponents() const;

#if WITH_EDITOR
	FActorOnComponentsReplaced OnComponentsReplaced;
#endif

	//~=============================================================================
	// Navigation/AI related functions
	
	/**
	 * Trigger a noise caused by a given Pawn, at a given location.
	 * Note that the NoiseInstigator Pawn MUST have a PawnNoiseEmitterComponent for the noise to be detected by a PawnSensingComponent.
	 * Senders of MakeNoise should have an Instigator if they are not pawns, or pass a NoiseInstigator.
	 *
	 * @param Loudness The relative loudness of this noise. Usual range is 0 (no noise) to 1 (full volume). If MaxRange is used, this scales the max range, otherwise it affects the hearing range specified by the sensor.
	 * @param NoiseInstigator Pawn responsible for this noise.  Uses the actor's Instigator if NoiseInstigator is null
	 * @param NoiseLocation Position of noise source.  If zero vector, use the actor's location.
	 * @param MaxRange Max range at which the sound may be heard. A value of 0 indicates no max range (though perception may have its own range). Loudness scales the range. (Note: not supported for legacy PawnSensingComponent, only for AIPerception)
	 * @param Tag Identifier for the noise.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="AI", meta=(BlueprintProtected = "true"))
	ENGINE_API void MakeNoise(float Loudness=1.f, APawn* NoiseInstigator=nullptr, FVector NoiseLocation=FVector::ZeroVector, float MaxRange = 0.f, FName Tag = NAME_None);

	/** Default Implementation of MakeNoise */
	static ENGINE_API void MakeNoiseImpl(AActor* NoiseMaker, float Loudness, APawn* NoiseInstigator, const FVector& NoiseLocation, float MaxRange, FName Tag);

	/** Modifies the global delegate used for handling MakeNoise */
	static ENGINE_API void SetMakeNoiseDelegate(const FMakeNoiseDelegate& NewDelegate);

	/**
	 * Check if owned component should be relevant for navigation
	 * Allows implementing master switch to disable e.g. collision export in projectiles
	 */
	virtual bool IsComponentRelevantForNavigation(UActorComponent* Component) const { return true; }

private:
	static ENGINE_API FMakeNoiseDelegate MakeNoiseDelegate;


public:
	//~=============================================================================
	// Debugging functions

	/**
	 * Draw important Actor variables on canvas.  HUD will call DisplayDebug() on the current ViewTarget when the ShowDebug exec is used
	 *
	 * @param Canvas			Canvas to draw on
	 *
	 * @param DebugDisplay		Contains information about what debug data to display
	 *
	 * @param YL				[in]	Height of the previously drawn line.
	 *							[out]	Height of the last line drawn by this function.
	 *
	 * @param YPos				[in]	Y position on Canvas for the previously drawn line. YPos += YL, gives position to draw text for next debug line.
	 *							[out]	Y position on Canvas for the last line drawn by this function.
	 */
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const class FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);

	/** Retrieves actor's name used for logging, or string "NULL" if Actor is null */
	static FString GetDebugName(const AActor* Actor) { return Actor ? Actor->GetName() : TEXT("NULL"); }

#if !UE_BUILD_SHIPPING
	/** Delegate for globally hooking ProccessEvent calls - used by a non-public testing plugin */
	static ENGINE_API FOnProcessEvent ProcessEventDelegate;
#endif

private:
	/** Helper that already assumes the Hit info is reversed, and avoids creating a temp FHitResult if possible. */
	ENGINE_API void InternalDispatchBlockingHit(UPrimitiveComponent* MyComp, UPrimitiveComponent* OtherComp, bool bSelfMoved, FHitResult const& Hit);

	/** Private version without inlining that does *not* check Dedicated server build flags (which should already have been done). */
	ENGINE_API ENetMode InternalGetNetMode() const;

	/** Unified implementation function to be called from the two implementations of PostEditUndo for the AActor specific elements that need to happen. */
	ENGINE_API bool InternalPostEditUndo();

	friend struct FMarkActorIsBeingDestroyed;
	friend struct FActorParentComponentSetter;
	friend struct FSetActorWantsDestroyDuringBeginPlay;
	friend struct FActorPreRegistrationAccessor;
#if WITH_EDITOR
	ENGINE_API bool IsActorFolderValid() const;
	ENGINE_API void SetFolderPathInternal(const FName& InNewFolderPath, bool bInBroadcastChange = true);
	ENGINE_API void SetFolderGuidInternal(const FGuid& InFolderGuid, bool bInBroadcastChange = true);
	ENGINE_API UActorFolder* GetActorFolder(bool bSkipDeleted = true) const;

	friend struct FSetActorHiddenInSceneOutliner;
	friend struct FSetActorGuid;
	friend struct FSetActorReplicates;
	friend struct FSetActorInstanceGuid;
	friend struct FSetActorContentBundleGuid;
	friend struct FAssignActorDataLayer;
	friend struct FSetActorSelectable;
	friend struct FSetActorFolderPath;
#endif

	// Static helpers for accessing functions on SceneComponent.
	// These are templates for no other reason than to delay compilation until USceneComponent is defined.

	template<class T>
	static inline const FTransform& TemplateGetActorTransform(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetComponentTransform() : FTransform::Identity;
	}

	template<class T>
	static inline FVector TemplateGetActorLocation(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetComponentLocation() : FVector::ZeroVector;
	}

	template<class T>
	static inline FRotator TemplateGetActorRotation(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetComponentRotation() : FRotator::ZeroRotator;
	}

	template<class T>
	static inline FVector TemplateGetActorScale(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetComponentScale() : FVector(1.f,1.f,1.f);
	}

	template<class T>
	static inline FQuat TemplateGetActorQuat(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetComponentQuat() : FQuat(ForceInit);
	}

	template<class T>
	static inline FVector TemplateGetActorForwardVector(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetForwardVector() : FVector::ForwardVector;
	}

	template<class T>
	static inline FVector TemplateGetActorUpVector(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetUpVector() : FVector::UpVector;
	}

	template<class T>
	static inline FVector TemplateGetActorRightVector(const T* RootComponent)
	{
		return (RootComponent != nullptr) ? RootComponent->GetRightVector() : FVector::RightVector;
	}
	
	//~ Begin Methods for Replicated Members.
public:

	/**
	 * Gets the property name for bHidden.
	 * This exists so subclasses don't need to have direct access to the bHidden property so it
	 * can be made private later.
	 */
	static const FName GetHiddenPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(AActor, bHidden);
	}

	/**
	 * Gets the literal value of bHidden.
	 *
	 * This exists so subclasses don't need to have direct access to the bHidden property so it
	 * can be made private later.
	 */
	bool IsHidden() const
	{
		return bHidden;
	}

	/**
	 * Sets the value of bHidden without causing other side effects to this instance.
	 *
	 * SetActorHiddenInGame is preferred preferred in most cases because it respects virtual behavior.
	 */
	ENGINE_API void SetHidden(const bool bInHidden);

	/**
	 * Gets the property name for bReplicateMovement.
	 * This exists so subclasses don't need to have direct access to the bReplicateMovement property so it
	 * can be made private later.
	 */
	static const FName GetReplicateMovementPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(AActor, bReplicateMovement);
	}

	/**
	 * Gets the literal value of bReplicateMovement.
	 *
	 * This exists so subclasses don't need to have direct access to the bReplicateMovement property so it
	 * can be made private later.
	 */
	bool IsReplicatingMovement() const
	{
		return bReplicateMovement;
	}

	/** Sets the value of bReplicateMovement without causing other side effects to this instance. */
	ENGINE_API void SetReplicatingMovement(bool bInReplicateMovement);

	/**
	 * Gets the property name for bCanBeDamaged.
	 * This exists so subclasses don't need to have direct access to the bCanBeDamaged property so it
	 * can be made private later.
	 */
	static const FName GetCanBeDamagedPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(AActor, bCanBeDamaged);
	}

	/**
	 * Gets the literal value of bCanBeDamaged.
	 *
	 * This exists so subclasses don't need to have direct access to the bCanBeDamaged property so it
	 * can be made private later.
	 */
	bool CanBeDamaged() const
	{
		return bCanBeDamaged;
	}

	/** Sets the value of bCanBeDamaged without causing other side effects to this instance. */
	ENGINE_API void SetCanBeDamaged(bool bInCanBeDamaged);

	/**
	 * Gets the property name for Role.
	 * This exists so subclasses don't need to have direct access to the Role property so it
	 * can be made private later.
	 */
	static const FName GetRolePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(AActor, Role);
	}

	/**
	 * Sets the value of Role without causing other side effects to this instance.
	 */
	ENGINE_API void SetRole(ENetRole InRole);
	
	/**
	 * Gets the literal value of ReplicatedMovement.
	 *
	 * This exists so subclasses don't need to have direct access to the Role property so it
	 * can be made private later.
	 */
	const FRepMovement& GetReplicatedMovement() const
	{
		return ReplicatedMovement;
	}

	/**
	 * Gets a reference to ReplicatedMovement with the expectation that it will be modified.
	 *
	 * This exists so subclasses don't need to have direct access to the ReplicatedMovement property
	 * so it can be made private later.
	 */
	ENGINE_API FRepMovement& GetReplicatedMovement_Mutable();

	/** Sets the value of ReplicatedMovement without causing other side effects to this instance. */
	ENGINE_API void SetReplicatedMovement(const FRepMovement& InReplicatedMovement);

	/**
	 * Set the frequency at which this object will be considered for replication.
	 */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetNetUpdateFrequency(float Frequency);

	/** 
	 * Get the current frequency at which this object will be considered for replication.
	 */
	UFUNCTION(BlueprintGetter)
	ENGINE_API float GetNetUpdateFrequency() const;

	/**
	 * Set the frequency to throttle down to when replicated properties are changing infrequently. 
	 */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetMinNetUpdateFrequency(float MinFrequency);

	/**
	 * Get the frequency to throttle down to when replicated properties are changing infrequently. 
	 */
	UFUNCTION(BlueprintGetter)
	ENGINE_API float GetMinNetUpdateFrequency() const;

	/** 
	 * Set the square of the max distance from the client's viewpoint that this actor is relevant and will be replicated.
	 */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetNetCullDistanceSquared(float DistanceSq);

	/** 
	 * Get the square of the max distance from the client's viewpoint that this actor is relevant and will be replicated.
	 */
	UFUNCTION(BlueprintGetter)
	ENGINE_API float GetNetCullDistanceSquared() const;

	/**
	 * Gets the property name for Instigator.
	 * This exists so subclasses don't need to have direct access to the Instigator property so it
	 * can be made private later.
	 */
	static const FName GetInstigatorPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(AActor, Instigator);
	}

	/** Sets the value of Instigator without causing other side effects to this instance. */
	ENGINE_API void SetInstigator(APawn* InInstigator);

	/** This should only be used by UNetActorFactory */
	void SetActorIsPendingPostNetInit(bool bInIsPendingPostNetInit)
	{
		bActorIsPendingPostNetInit = bInIsPendingPostNetInit;
	}
	
	//~ End Methods for Replicated Members.
};

/** Internal struct used by level code to mark actors as destroyed */
struct FMarkActorIsBeingDestroyed
{
private:
	FMarkActorIsBeingDestroyed(AActor* InActor)
	{
		InActor->bActorIsBeingDestroyed = true;
	}

	friend UWorld;
};

/** This should only be used by UWorld::DestroyActor when the actor is in the process of beginning play so it can't be destroyed yet */
struct FSetActorWantsDestroyDuringBeginPlay
{
private:
	FSetActorWantsDestroyDuringBeginPlay(AActor* InActor)
	{
		ensure(InActor->IsActorBeginningPlay()); // Doesn't make sense to call this under any other circumstances
		InActor->bActorWantsDestroyDuringBeginPlay = true;
	}

	friend UWorld;
};


inline void FActorLastRenderTime::Set(AActor* InActor, float LastRenderTime)
{
	InActor->LastRenderTime.SetLastRenderTime(LastRenderTime);
}

inline FActorLastRenderTime* FActorLastRenderTime::GetPtr(AActor* InActor)
{
	return (InActor ? &InActor->LastRenderTime : nullptr);
}

/** Accessor use by Level for Actor pre-pregistration. */
struct FActorPreRegistrationAccessor
{
private:
	static void IncrementalPreRegisterComponents(AActor* InActor)
	{
		InActor->IncrementalPreRegisterComponents();
	}

	static bool HasPreRegisteredAllComponents(const AActor* InActor)
	{
		return InActor->HasPreRegisteredAllComponents();
	}

	static void IncrementalPreUnregisterComponents(AActor* InActor)
	{
		InActor->IncrementalPreUnregisterComponents();
	}
	friend class ULevel;
};

#if WITH_EDITOR
struct FSetActorHiddenInSceneOutliner
{
private:
	FSetActorHiddenInSceneOutliner(AActor* InActor, bool bHidden = true)
	{
		InActor->bListedInSceneOutliner = !bHidden;
	}

	friend UWorld;
	friend class FFoliageHelper;
	friend class ULevelInstanceSubsystem;
	friend class UExternalDataLayerInstance;
	friend class FModelingToolsSetActorHiddenInSceneOutliner;
};

struct FSetActorGuid
{
private:
	FSetActorGuid(AActor* InActor, const FGuid& InActorGuid)
	{
		InActor->ActorGuid = InActorGuid;
	}
	friend class UWorld;
	friend class UEngine;
	friend class UExternalActorsCommandlet;
	friend class UWorldPartitionConvertCommandlet;
};

struct FSetActorReplicates
{
private:
	FSetActorReplicates(AActor* InActor, bool bInReplicates)
	{
		if (InActor->bReplicates != bInReplicates)
		{
			check(!InActor->bActorInitialized);
			InActor->bReplicates = bInReplicates;
			InActor->RemoteRole = (bInReplicates ? ROLE_SimulatedProxy : ROLE_None);
		}
	}
	friend class FWorldPartitionLevelHelper;
};

struct FSetActorInstanceGuid
{
private:
	FSetActorInstanceGuid(AActor* InActor, const FGuid& InActorInstanceGuid)
	{
		InActor->ActorInstanceGuid = InActorInstanceGuid;
		if (InActorInstanceGuid == InActor->ActorGuid)
		{
			InActor->ActorInstanceGuid.Invalidate();
		}
	}
	friend class UEngine;
	friend class ULevelStreamingLevelInstance;
	friend class FWorldPartitionLevelHelper;
	friend class FReplaceActorHelperSetActorInstanceGuid;
};

struct FSetActorFolderPath
{
private:
	FSetActorFolderPath(AActor* InActor, const FName InFolderPath, bool bInBroadcastChange = true)
	{
		InActor->SetFolderPathInternal(InFolderPath, bInBroadcastChange);
	}
	friend class UWorldPartitionRuntimeCell;
};

struct FSetActorContentBundleGuid
{
private:
	FSetActorContentBundleGuid(AActor* InActor, const FGuid& InContentBundleGuid)
	{
		InActor->ContentBundleGuid = InContentBundleGuid;
	}
	friend class FContentBundleEditor;
	friend class FExternalDataLayerHelper;
	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
};

struct FAssignActorDataLayer
{
private:
	ENGINE_API static bool AddDataLayerAsset(AActor* InActor, const UDataLayerAsset* InDataLayerAsset);
	ENGINE_API static bool RemoveDataLayerAsset(AActor* InActor, const UDataLayerAsset* InDataLayerAsset);

	friend class UEngine;
	friend class FContentBundleEditor;
	friend class UDataLayerEditorSubsystem;
	friend class UDataLayerInstanceWithAsset;
	friend class UDataLayerInstancePrivate;
	friend class UExternalDataLayerInstance;
	friend class ULevelInstanceSubsystem;
	friend class FExternalDataLayerHelper;
	friend class UActorPartitionSubsystem;
};

struct FSetActorIsInLevelInstance
{
private:
	FSetActorIsInLevelInstance(AActor* InActor, ELevelInstanceType InLevelInstanceType)
	{
		InActor->LevelInstanceType = InLevelInstanceType;
	}

	friend class ULevelStreamingLevelInstance;
	friend class ULevelStreamingLevelInstanceEditor;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
};

struct FAddActorLevelInstanceFlags
{
private:
	FAddActorLevelInstanceFlags(AActor* InActor, ELevelInstanceFlags InFlagsToAdd)
	{
		EnumAddFlags(InActor->LevelInstanceFlags, InFlagsToAdd);
	}

	friend class ULevelStreamingLevelInstance;
	friend class ULevelStreamingLevelInstanceEditor;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
};

struct FRemoveActorLevelInstanceFlags
{
private:
	FRemoveActorLevelInstanceFlags(AActor* InActor, ELevelInstanceFlags InFlagsToRemove)
	{
		EnumRemoveFlags(InActor->LevelInstanceFlags, InFlagsToRemove);
	}

	friend class ULevelStreamingLevelInstance;
	friend class ULevelStreamingLevelInstanceEditor;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
};
#endif

/** Helper function for executing component tick functions using the same conditions as FActorTickFunction */
template <typename ExecuteTickLambda>
void FActorComponentTickFunction::ExecuteTickHelper(UActorComponent* Target, bool bTickInEditor, float DeltaTime, ELevelTick TickType, const ExecuteTickLambda& ExecuteTickFunc)
{
	if (IsValid(Target))
	{
		FScopeCycleCounterUObject ComponentScope(Target);
		FScopeCycleCounterUObject AdditionalScope(Target->AdditionalStatObject());

		if (Target->bRegistered)
		{
			AActor* MyOwner = Target->GetOwner();
			if (TickType != LEVELTICK_ViewportsOnly || bTickInEditor ||
				(MyOwner && MyOwner->ShouldTickIfViewportsOnly()))
			{
				const float TimeDilation = (MyOwner ? MyOwner->CustomTimeDilation : 1.f);
				ExecuteTickFunc(DeltaTime * TimeDilation);
			}
		}
	}
}

template<class T, uint32 NumElements>
TInlineComponentArray<T, NumElements>::TInlineComponentArray(const AActor* Actor, bool bIncludeFromChildActors) 
	: Super()
{
	if (Actor)
	{
		Actor->GetComponents(*this, bIncludeFromChildActors);
	}
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline FVector AActor::K2_GetActorLocation() const
{
	return GetActorLocation();
}

inline FRotator AActor::K2_GetActorRotation() const
{
	return GetActorRotation();
}

inline USceneComponent* AActor::K2_GetRootComponent() const
{
	return GetRootComponent();
}

inline FVector AActor::GetActorForwardVector() const
{
	return TemplateGetActorForwardVector(ToRawPtr(RootComponent));
}

inline FVector AActor::GetActorUpVector() const
{
	return TemplateGetActorUpVector(ToRawPtr(RootComponent));
}

inline FVector AActor::GetActorRightVector() const
{
	return TemplateGetActorRightVector(ToRawPtr(RootComponent));
}


inline float AActor::GetSimpleCollisionRadius() const
{
	float Radius, HalfHeight;
	GetSimpleCollisionCylinder(Radius, HalfHeight);
	return Radius;
}

inline float AActor::GetSimpleCollisionHalfHeight() const
{
	float Radius, HalfHeight;
	GetSimpleCollisionCylinder(Radius, HalfHeight);
	return HalfHeight;
}

inline FVector AActor::GetSimpleCollisionCylinderExtent() const
{
	float Radius, HalfHeight;
	GetSimpleCollisionCylinder(Radius, HalfHeight);
	return FVector(Radius, Radius, HalfHeight);
}

inline bool AActor::GetActorEnableCollision() const
{
	return bActorEnableCollision;
}

inline bool AActor::HasAuthority() const
{
	return (GetLocalRole() == ROLE_Authority);
}

inline AActor* AActor::GetOwner() const
{ 
	return Owner; 
}

inline const AActor* AActor::GetNetOwner() const
{
	// NetOwner is the Actor Owner unless otherwise overridden (see PlayerController/Pawn/Beacon)
	// Used in ServerReplicateActors
	return Owner;
}

inline ENetRole AActor::GetRemoteRole() const
{
	return RemoteRole;
}

inline ENetMode AActor::GetNetMode() const
{
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (IsRunningDedicatedServer() && (NetDriverName == NAME_None || NetDriverName == NAME_GameNetDriver))
	{
		// Only normal net driver actors can have this optimization
		return NM_DedicatedServer;
	}

	return InternalGetNetMode();
}

inline bool AActor::IsNetMode(ENetMode Mode) const
{
#if UE_EDITOR
	// Editor builds are special because of PIE, which can run a dedicated server without the app running with -server.
	return GetNetMode() == Mode;
#else
	// IsRunningDedicatedServer() is a compile-time check in optimized non-editor builds.
	if (Mode == NM_DedicatedServer)
	{
		return IsRunningDedicatedServer();
	}
	else if (NetDriverName == NAME_None || NetDriverName == NAME_GameNetDriver)
	{
		// Only normal net driver actors can have this optimization
		return !IsRunningDedicatedServer() && (InternalGetNetMode() == Mode);
	}
	else
	{
		return (InternalGetNetMode() == Mode);
	}
#endif
}

#if WITH_EDITOR
/** Callback for editor actor selection. This must be in engine instead of editor for AActor::IsSelectedInEditor to work */
extern ENGINE_API TFunction<bool(const AActor*)> GIsActorSelectedInEditor;
#endif

DEFINE_ACTORDESC_TYPE(AActor, FWorldPartitionActorDesc);

//////////////////////////////////////////////////////////////////////////
// Macro to hide common Transform functions in native code for classes where they don't make sense.
// Note that this doesn't prevent access through function calls from parent classes (ie an AActor*), but
// does prevent use in the class that hides them and any derived child classes.

#define HIDE_ACTOR_TRANSFORM_FUNCTIONS() private: \
	FTransform GetTransform() const { return Super::GetTransform(); } \
	FTransform GetActorTransform() const { return Super::GetActorTransform(); } \
	FVector GetActorLocation() const { return Super::GetActorLocation(); } \
	FRotator GetActorRotation() const { return Super::GetActorRotation(); } \
	FQuat GetActorQuat() const { return Super::GetActorQuat(); } \
	FVector GetActorScale() const { return Super::GetActorScale(); } \
	bool SetActorTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorTransform(NewTransform, bSweep, OutSweepHitResult, Teleport); } \
	bool SetActorLocation(const FVector& NewLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorLocation(NewLocation, bSweep, OutSweepHitResult, Teleport); } \
	bool SetActorRotation(FRotator NewRotation, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorRotation(NewRotation, Teleport); } \
	bool SetActorRotation(const FQuat& NewRotation, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorRotation(NewRotation, Teleport); } \
	bool SetActorLocationAndRotation(FVector NewLocation, FRotator NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorLocationAndRotation(NewLocation, NewRotation, bSweep, OutSweepHitResult, Teleport); } \
	bool SetActorLocationAndRotation(FVector NewLocation, const FQuat& NewRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { return Super::SetActorLocationAndRotation(NewLocation, NewRotation, bSweep, OutSweepHitResult, Teleport); } \
	virtual bool TeleportTo( const FVector& DestLocation, const FRotator& DestRotation, bool bIsATest, bool bNoCheck ) override { return Super::TeleportTo(DestLocation, DestRotation, bIsATest, bNoCheck); } \
	virtual FVector GetVelocity() const override { return Super::GetVelocity(); } \
	float GetHorizontalDistanceTo(AActor* OtherActor)  { return Super::GetHorizontalDistanceTo(OtherActor); } \
	float GetVerticalDistanceTo(AActor* OtherActor)  { return Super::GetVerticalDistanceTo(OtherActor); } \
	float GetDotProductTo(AActor* OtherActor) { return Super::GetDotProductTo(OtherActor); } \
	float GetHorizontalDotProductTo(AActor* OtherActor) { return Super::GetHorizontalDotProductTo(OtherActor); } \
	float GetDistanceTo(AActor* OtherActor) { return Super::GetDistanceTo(OtherActor); } \
	float GetSquaredDistanceTo(const AActor* OtherActor) { return Super::GetSquaredDistanceTo(OtherActor); } \
	FVector GetActorForwardVector() const { return Super::GetActorForwardVector(); } \
	FVector GetActorUpVector() const { return Super::GetActorUpVector(); } \
	FVector GetActorRightVector() const { return Super::GetActorRightVector(); } \
	void GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors = false) const { return Super::GetActorBounds(bOnlyCollidingComponents, Origin, BoxExtent, bIncludeFromChildActors); } \
	void SetActorScale3D(FVector NewScale3D) { Super::SetActorScale3D(NewScale3D); } \
	FVector GetActorScale3D() const { return Super::GetActorScale3D(); } \
	void SetActorRelativeScale3D(FVector NewRelativeScale) { Super::SetActorRelativeScale3D(NewRelativeScale); } \
	FVector GetActorRelativeScale3D() const { return Super::GetActorRelativeScale3D(); } \
	FTransform ActorToWorld() const { return Super::ActorToWorld(); } \
	void AddActorWorldOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorWorldOffset(DeltaLocation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorWorldRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorWorldRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorWorldRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorWorldRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorWorldTransform(const FTransform& DeltaTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorWorldTransform(DeltaTransform, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorLocalOffset(FVector DeltaLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorLocalOffset(DeltaLocation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorLocalRotation(FRotator DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorLocalRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorLocalRotation(const FQuat& DeltaRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorLocalRotation(DeltaRotation, bSweep, OutSweepHitResult, Teleport); } \
	void AddActorLocalTransform(const FTransform& NewTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::AddActorLocalTransform(NewTransform, bSweep, OutSweepHitResult, Teleport); } \
	void SetActorRelativeLocation(FVector NewRelativeLocation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::SetActorRelativeLocation(NewRelativeLocation, bSweep, OutSweepHitResult, Teleport); } \
	void SetActorRelativeRotation(FRotator NewRelativeRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::SetActorRelativeRotation(NewRelativeRotation, bSweep, OutSweepHitResult, Teleport); } \
	void SetActorRelativeRotation(const FQuat& NewRelativeRotation, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::SetActorRelativeRotation(NewRelativeRotation, bSweep, OutSweepHitResult, Teleport); } \
	void SetActorRelativeTransform(const FTransform& NewRelativeTransform, bool bSweep=false, FHitResult* OutSweepHitResult=nullptr, ETeleportType Teleport = ETeleportType::None) { Super::SetActorRelativeTransform(NewRelativeTransform, bSweep, OutSweepHitResult, Teleport); }
