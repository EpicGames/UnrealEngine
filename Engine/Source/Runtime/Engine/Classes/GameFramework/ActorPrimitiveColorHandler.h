// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"

#define UE_API ENGINE_API

class UWorld;
class UPrimitiveComponent;

#define ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

/**
 * FActorPrimitiveColorHandler is a simple mechanism for custom actor coloration registration. Once an actor color
 * handler is registered, it can automatically be activated with the SHOW ACTORCOLORATION <HANDLERNAME> command.
 */
class FActorPrimitiveColorHandler
{
	using FGetColorFunc = TFunction<FLinearColor(const UPrimitiveComponent*)>;
	using FActivateFunc = TFunction<void(void)>;
	using FDeactivateFunc = TFunction<void(void)>;
	constexpr static auto DefaultFunc = []() {};

public:
	struct FPrimitiveColorHandler
	{
		FName HandlerName;
		FText HandlerText;
		FText HandlerToolTipText;
		bool bAvailalbleInEditor = true;
		FGetColorFunc GetColorFunc;
		FActivateFunc ActivateFunc;
		FDeactivateFunc DeactivateFunc;
	};	

	UE_API FActorPrimitiveColorHandler();
	static UE_API FActorPrimitiveColorHandler& Get();

	UE_API void RegisterPrimitiveColorHandler(FPrimitiveColorHandler& PrimitiveColorHandler);
	UE_API void RegisterPrimitiveColorHandler(FName InHandlerName, const FText& InHandlerText, const FGetColorFunc& InHandlerFunc, const FActivateFunc& InActivateFunc = DefaultFunc, const FText& InHandlerToolTipText = FText());
	UE_API void RegisterPrimitiveColorHandler(FName InHandlerName, const FText& InHandlerText, bool bInAvailalbleInEditor, const FGetColorFunc& InHandlerFunc, const FActivateFunc& InActivateFunc = DefaultFunc, const FText& InHandlerToolTipText = FText());
	UE_API void UnregisterPrimitiveColorHandler(FName InHandlerName);
	UE_API void GetRegisteredPrimitiveColorHandlers(TArray<FPrimitiveColorHandler>& OutPrimitiveColorHandlers) const;

	UE_API FName GetActivePrimitiveColorHandler() const;
	UE_API bool SetActivePrimitiveColorHandler(FName InHandlerName, UWorld* InWorld);

	UE_API FText GetActivePrimitiveColorHandlerDisplayName() const;

	UE_API void RefreshPrimitiveColorHandler(FName InHandlerName, UWorld* InWorld);
	UE_API void RefreshPrimitiveColorHandler(FName InHandlerName, const TArray<AActor*>& InActors);
	UE_API void RefreshPrimitiveColorHandler(FName InHandlerName, const TArray<UPrimitiveComponent*>& InPrimitiveComponents);

	UE_API FLinearColor GetPrimitiveColor(const UPrimitiveComponent* InPrimitiveComponent) const;

private:
	void InitActivePrimitiveColorHandler();

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	FName ActivePrimitiveColorHandlerName;
	FPrimitiveColorHandler* ActivePrimitiveColorHandler;
	TMap<FName, FPrimitiveColorHandler> Handlers;
#endif
};

#undef UE_API
