// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/StreamableManager.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Templates/SubclassOf.h"

#include "AsyncActionHandleSaveGame.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAsyncHandleSaveGame, USaveGame*, SaveGame, bool, bSuccess);

/** Async action to handle async load/save of a USaveGame. This can be subclassed by a specific game */
UCLASS(MinimalAPI)
class UAsyncActionHandleSaveGame : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()
public:

	/**
	 * Schedule an async save to a specific slot. UGameplayStatics::AsyncSaveGameToSlot is the native version of this.
	 * When the save has succeeded or failed, the completed pin is activated with success/failure and the save game object.
	 * Keep in mind that some platforms may not support trying to load and save at the same time.
	 *
	 * @param SaveGameObject	Object that contains data about the save game that we want to write out.
	 * @param SlotName			Name of the save game slot to load from.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", Category = "SaveGame",  WorldContext = "WorldContextObject"))
	static ENGINE_API UAsyncActionHandleSaveGame* AsyncSaveGameToSlot(UObject* WorldContextObject, USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex);

	/**
	 * Schedule an async load of a specific slot. UGameplayStatics::AsyncLoadGameFromSlot is the native version of this.
	 * When the load has succeeded or failed, the completed pin is activated with success/failure and the newly loaded save game object if valid.
	 * Keep in mind that some platforms may not support trying to load and save at the same time.
	 *
	 * @param SlotName			Name of the save game slot to load from.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "SaveGame", WorldContext = "WorldContextObject"))
	static ENGINE_API UAsyncActionHandleSaveGame* AsyncLoadGameFromSlot(UObject* WorldContextObject, const FString& SlotName, const int32 UserIndex);

	/** Delegate called when the save/load completes */
	UPROPERTY(BlueprintAssignable)
	FOnAsyncHandleSaveGame Completed;

	/** Execute the actual operation */
	ENGINE_API virtual void Activate() override;

protected:
	enum class ESaveGameOperation : uint8
	{
		Save,
		Load,
	};

	/** Which operation is being run */
	ESaveGameOperation Operation;
	
	/** Slot/user to use */
	FString SlotName;
	int32 UserIndex;

	/** The object that was either saved or loaded */
	UPROPERTY()
	TObjectPtr<USaveGame> SaveGameObject;
	
	/** Function callbacks for load/save */
	ENGINE_API virtual void HandleAsyncSave(const FString& SlotName, const int32 UserIndex, bool bSuccess);
	ENGINE_API virtual void HandleAsyncLoad(const FString& SlotName, const int32 UserIndex, USaveGame* LoadedSave);
	
	/** Called at completion of save/load to execute delegate */
	ENGINE_API virtual void ExecuteCompleted(bool bSuccess);
};
