// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "CameraBlockingVolume.generated.h"

/**
 * A volume which blocks the Camera channel by default.
 */
UCLASS(MinimalAPI)
class ACameraBlockingVolume : public AVolume
{
	GENERATED_UCLASS_BODY()
};



