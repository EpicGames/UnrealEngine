// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "UObject/CoreNet.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/CharacterMovementReplication.h"
#include "Animation/AnimationAsset.h"
#include "GameFramework/RootMotionSource.h"
#include "Interfaces/MovementBaseInterface.h"
#include "Character.generated.h"

class ACharacter;
class AController;
class FDebugDisplayInfo;
class UAnimMontage;
class UArrowComponent;
class UCapsuleComponent;
class UCharacterMovementComponent;
class UPawnMovementComponent;
class UPrimitiveComponent;
class USkeletalMeshComponent;
struct FAnimMontageInstance;
struct FCharacterAsyncInput;
struct FCharacterAsyncOutput;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMovementModeChangedSignature, class ACharacter*, Character, EMovementMode, PrevMovementMode, uint8, PreviousCustomMode);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FCharacterMovementUpdatedSignature, float, DeltaSeconds, FVector, OldLocation, FVector, OldVelocity);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FCharacterReachedApexSignature);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FLandedSignature, const FHitResult&, Hit);

// CVars
namespace CharacterCVars
{	
	extern ENGINE_API int32 EnableCharacterAccelerationReplication;
}

/** Replicated data when playing a root motion montage. */
USTRUCT()
struct FRepRootMotionMontage
{
	GENERATED_USTRUCT_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRepRootMotionMontage() = default;
	~FRepRootMotionMontage() = default;
	FRepRootMotionMontage(const FRepRootMotionMontage&) = default;
	FRepRootMotionMontage(FRepRootMotionMontage&&) = default;
	FRepRootMotionMontage& operator=(const FRepRootMotionMontage&) = default;
	FRepRootMotionMontage& operator=(FRepRootMotionMontage&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Animation providing Root Motion */
	UPROPERTY()
	TObjectPtr<UAnimSequenceBase> Animation = nullptr;

	/** Whether this has useful/active data. */
	UPROPERTY()
	bool bIsActive = false;

	/** Additional replicated flag, if MovementBase can't be resolved on the client. So we don't use wrong data. */
	UPROPERTY()
	bool bRelativePosition = false;

	/** Whether rotation is relative or absolute. */
	UPROPERTY()
	bool bRelativeRotation = false;

	/** Track position of Montage */
	UPROPERTY()
	float Position = 0.f;

	/** Location */
	UPROPERTY()
	FVector_NetQuantize100 Location;

	/** Rotation */
	UPROPERTY()
	FRotator Rotation = FRotator(0.f);

	/** Movement Relative to Base */
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use PhysicsObjectOwner instead and use that to create a FMovementBaseInterfaceData. See ACharacter::OnRep_ReplicatedBasedMovement() for an example of this.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage="Please use PhysicsObjectOwner instead and use that to create a FMovementBaseInterfaceData."))
	TObjectPtr<UPrimitiveComponent> MovementBase = nullptr;

	/** PhysicsObjectOwner for BodyInstance used as movement base */
	UPROPERTY()
	TObjectPtr<UObject> PhysicsObjectOwner = nullptr;
	
	/** Bone on the MovementBase, if a skeletal mesh. */
	UPROPERTY()
	FName MovementBaseBoneName;

	/** State of Root Motion Sources on Authority */
	UPROPERTY()
	FRootMotionSourceGroup AuthoritativeRootMotion;

	/** Acceleration */
	UPROPERTY()
	FVector_NetQuantize10 Acceleration;

	/** Velocity */
	UPROPERTY()
	FVector_NetQuantize10 LinearVelocity;

	/** Clear root motion sources and root motion montage */
	void Clear()
	{
		bIsActive = false;
		Animation = nullptr;
		AuthoritativeRootMotion.Clear();
	}

	/** Is Valid - animation root motion only */
	bool HasRootMotion() const
	{
		return (Animation != nullptr);
	}

	UAnimMontage* GetAnimMontage() const;
};

USTRUCT()
struct FSimulatedRootMotionReplicatedMove
{
	GENERATED_USTRUCT_BODY()

	/** Local time when move was received on client and saved. */
	UPROPERTY()
	float Time = 0.f;

	/** Root Motion information */
	UPROPERTY()
	FRepRootMotionMontage RootMotion;
};


/** MovementBaseUtility provides utilities for working with movement bases, for which we may need relative positioning info. */
namespace MovementBaseUtility
{
	// Start deprecated movement base utility functions
	/** Determine whether MovementBase can possibly move. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool IsDynamicBase(const UPrimitiveComponent* MovementBase);

	/** Determine whether MovementBase is simulating or attached to a simulating object. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool IsSimulatedBase(const UPrimitiveComponent* MovementBase);

	/** Determine if we should use relative positioning when based on a component (because it may move). */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	inline bool UseRelativeLocation(const UPrimitiveComponent* MovementBase)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IsDynamicBase(MovementBase);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	/** Ensure that BasedObjectTick ticks after NewBase */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API void AddTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* NewBase);

	/** Remove tick dependency of BasedObjectTick on OldBase */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API void RemoveTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* OldBase);

	/** Get the velocity of the given component, first checking the ComponentVelocity and falling back to the physics velocity if necessary. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API FVector GetMovementBaseVelocity(const UPrimitiveComponent* MovementBase, const FName BoneName);

	/** Get the tangential velocity at WorldLocation for the given component. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API FVector GetMovementBaseTangentialVelocity(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldLocation);

	/** Get the transform (local-to-world) for the given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool GetMovementBaseTransform(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat);

	/** Convert a local location to a world location for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool TransformLocationToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& LocalLocation, FVector& OutLocationWorldSpace);

	/** Convert a world location to a local location for a given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool TransformLocationToLocal(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldSpaceLocation, FVector& OutLocalLocation);

	/** Convert a local direction to a world direction for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool TransformDirectionToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& LocalDirection, FVector& OutDirectionWorldSpace);

	/** Convert a world direction to a local direction for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UE_DEPRECATED(5.8, "MovementBaseUtility functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API bool TransformDirectionToLocal(const UPrimitiveComponent* MovementBase, const FName BoneName, const FVector& WorldSpaceDirection, FVector& OutLocalDirection);
	// End deprecated movement base utility functions


	/** Determine whether MovementBase can possibly move. */
	ENGINE_API bool IsDynamicBase(const FMovementBaseInterfaceData* MovementBase);

	/** Determine whether MovementBase is simulating or attached to a simulating object. */
	ENGINE_API bool IsSimulatedBase(const FMovementBaseInterfaceData* MovementBase);

	/** Ensure that BasedObjectTick ticks after NewBase */
	ENGINE_API void AddTickDependency(FTickFunction& BasedObjectTick, FMovementBaseInterfaceData* MovementBase);

	/** Remove tick dependency of BasedObjectTick on OldBase */
	ENGINE_API void RemoveTickDependency(FTickFunction& BasedObjectTick, FMovementBaseInterfaceData* MovementBase);
	
	/** Determine if we should use relative positioning when based on a component (because it may move). */
	inline bool UseRelativeLocation(const FMovementBaseInterfaceData* MovementBase)
	{
		return IsDynamicBase(MovementBase);
	}

	/** Get the velocity of the MovementBase, first checking the ComponentVelocity and falling back to the physics velocity if necessary. */
	ENGINE_API FVector GetMovementBaseVelocity(const FMovementBaseInterfaceData* MovementBase, const FName BoneName);

	/** Get the tangential velocity at WorldLocation for the MovementBase. */
	ENGINE_API FVector GetMovementBaseTangentialVelocity(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldLocation);
	
	/** Get the transform (local-to-world) for the given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. */
	ENGINE_API bool GetMovementBaseTransform(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat);

	/** Convert a local location to a world location for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	ENGINE_API bool TransformLocationToWorld(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& LocalLocation, FVector& OutLocationWorldSpace);

	/** Convert a world location to a local location for a given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	ENGINE_API bool TransformLocationToLocal(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldSpaceLocation, FVector& OutLocalLocation);

	/** Convert a local direction to a world direction for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	ENGINE_API bool TransformDirectionToWorld(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& LocalDirection, FVector& OutDirectionWorldSpace);

	/** Convert a world direction to a local direction for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	ENGINE_API bool TransformDirectionToLocal(const FMovementBaseInterfaceData* MovementBase, const FName BoneName, const FVector& WorldSpaceDirection, FVector& OutLocalDirection);

	/** Returns true if movement base data passed in matches. Even if both are invalid. */
	ENGINE_API bool DoesMovementBaseDataMatch(const FMovementBaseInterfaceData* MovementBaseDataA, const FMovementBaseInterfaceData* MovementBaseDataB);
	
	/** Returns true if MovementBaseData is valid (both not null and underlying FMovementBaseInterfaceData::IsValid check) */
	ENGINE_API bool IsMovementBaseDataValid(const FMovementBaseInterfaceData* MovementBaseData);
	
	/** Returns FMovementBaseInterfaceData with movement base data extracted from the PhysicsObjectOwner passed in. */
	ENGINE_API FMovementBaseInterfaceData GetMovementBaseDataFromPhysicsOwner(UObject* PhysicsObjectOwner);

	/** Returns FMovementBaseInterfaceData with movement base data extracted from the HitResult passed in. */
	ENGINE_API FMovementBaseInterfaceData GetMovementBaseDataFromHitResult(const FHitResult* HitResult);
}

/** Struct to hold information about the "base" object the character is standing on. */
USTRUCT()
struct FBasedMovementInfo
{
	GENERATED_USTRUCT_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FBasedMovementInfo() = default;
	~FBasedMovementInfo() = default;
	FBasedMovementInfo(const FBasedMovementInfo&) = default;
	FBasedMovementInfo(FBasedMovementInfo&&) = default;
	FBasedMovementInfo& operator=(const FBasedMovementInfo&) = default;
	FBasedMovementInfo& operator=(FBasedMovementInfo&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Unique (within a reasonable timespan) ID of the base component. Can be used to detect changes in the base when the pointer can't replicate, eg during fast shared replication. */
	UPROPERTY()
	uint16 BaseID = 0;

	/** Whether the server says that there is a base. On clients, the component may not have resolved yet. */
	UPROPERTY()
	uint8 bServerHasBaseComponent:1 = false;

	/** Whether rotation is relative to the base or absolute. It can only be relative if location is also relative. */
	UPROPERTY()
	uint8 bRelativeRotation:1 = false;

	/** Whether there is a velocity on the server. Used for forcing replication when velocity goes to zero. */
	UPROPERTY()
	uint8 bServerHasVelocity:1 = false;

	/** Bone name on component, for skeletal meshes. NAME_None if not a skeletal mesh or if bone is invalid. */
	UPROPERTY()
	FName BoneName;

	/** Component we are based on */
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use MovementBaseInterfaceData instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage="Use MovementBaseInterfaceData instead."))
	TObjectPtr<UPrimitiveComponent> MovementBase = nullptr;

	/** Location relative to MovementBase. Only valid if HasRelativeLocation() is true. */
	UPROPERTY()
	FVector_NetQuantize100 Location;

	/** Rotation: relative to MovementBase if HasRelativeRotation() is true, absolute otherwise. */
	UPROPERTY()
	FRotator Rotation = FRotator(0.f);

	/** Struct containing Movement base functionality and relevant data for the UObject we are based on */
	UPROPERTY()
	FMovementBaseInterfaceData MovementBaseInterfaceData;
	
	/** Is location relative? */
	inline bool HasRelativeLocation() const
	{
		return MovementBaseUtility::UseRelativeLocation(&MovementBaseInterfaceData);
	}

	/** Is rotation relative or absolute? It can only be relative if location is also relative. */
	inline bool HasRelativeRotation() const
	{
		return bRelativeRotation && HasRelativeLocation();
	}

	/** Return true if the client should have MovementBase, but it hasn't replicated (possibly component has not streamed in). */
	inline bool IsBaseUnresolved() const
	{
		return (!MovementBaseInterfaceData.IsValid()) && bServerHasBaseComponent;
	}
};


/**
 * Characters are Pawns that have a mesh, collision, and built-in movement logic.
 * They are responsible for all physical interaction between the player or AI and the world, and also implement basic networking and input models.
 * They are designed for a vertically-oriented player representation that can walk, jump, fly, and swim through the world using CharacterMovementComponent.
 *
 * @see APawn, UCharacterMovementComponent
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Pawn/Character/
 */ 
UCLASS(config=Game, BlueprintType, meta=(ShortTooltip="A character is a type of Pawn that includes the ability to walk around."), MinimalAPI)
class ACharacter : public APawn
{
	GENERATED_BODY()
public:
	/** Default UObject constructor. */
	ENGINE_API ACharacter(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	ENGINE_API virtual void GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const override;

private:
	/** The main skeletal mesh associated with this Character (optional sub-object). */
	UPROPERTY(Category=Character, VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"))
	TObjectPtr<USkeletalMeshComponent> Mesh;

	/** Movement component used for movement logic in various movement modes (walking, falling, etc), containing relevant settings and functions to control movement. */
	UPROPERTY(Category=Character, VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"))
	TObjectPtr<UCharacterMovementComponent> CharacterMovement;

	/** The CapsuleComponent being used for movement collision (by CharacterMovement). Always treated as being vertically aligned in simple collision check functions. */
	UPROPERTY(Category=Character, VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"))
	TObjectPtr<UCapsuleComponent> CapsuleComponent;

#if WITH_EDITORONLY_DATA
	/** Component shown in the editor only to indicate character facing */
	UPROPERTY()
	TObjectPtr<UArrowComponent> ArrowComponent;
#endif

public:

	//////////////////////////////////////////////////////////////////////////
	// Server RPC that passes through to CharacterMovement (avoids RPC overhead for components).
	// The base RPC function (eg 'ServerMove') is auto-generated for clients to trigger the call to the server function,
	// eventually going to the _Implementation function (which we just pass to the CharacterMovementComponent).
	//////////////////////////////////////////////////////////////////////////

	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMovePacked(const FCharacterServerMovePackedBits& PackedBits);
	ENGINE_API void ServerMovePacked_Implementation(const FCharacterServerMovePackedBits& PackedBits);
	ENGINE_API bool ServerMovePacked_Validate(const FCharacterServerMovePackedBits& PackedBits);

	//////////////////////////////////////////////////////////////////////////
	// Client RPC that passes through to CharacterMovement (avoids RPC overhead for components).
	//////////////////////////////////////////////////////////////////////////

	UFUNCTION(unreliable, client, WithValidation)
	ENGINE_API void ClientMoveResponsePacked(const FCharacterMoveResponsePackedBits& PackedBits);
	ENGINE_API void ClientMoveResponsePacked_Implementation(const FCharacterMoveResponsePackedBits& PackedBits);
	ENGINE_API bool ClientMoveResponsePacked_Validate(const FCharacterMoveResponsePackedBits& PackedBits);


	//////////////////////////////////////////////////////////////////////////
	// BEGIN DEPRECATED RPCs that don't use variable sized payloads. Use ServerMovePacked and ClientMoveResponsePacked instead.
	//////////////////////////////////////////////////////////////////////////

	/** Replicated function sent by client to server - contains client movement and view info. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMove, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMove(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMove_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMove_Implementation(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API bool ServerMove_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/**
	 * Replicated function sent by client to server. Saves bandwidth over ServerMove() by implying that ClientMovementBase and ClientBaseBoneName are null.
	 * Passes through to CharacterMovement->ServerMove_Implementation() with null base params.
	 */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveNoBase, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMoveNoBase(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveNoBase_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMoveNoBase_Implementation(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);
	ENGINE_API bool ServerMoveNoBase_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);

	/** Replicated function sent by client to server - contains client movement and view info for two moves. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDual, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMoveDual(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDual_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMoveDual_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API bool ServerMoveDual_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/** Replicated function sent by client to server - contains client movement and view info for two moves. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualNoBase, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMoveDualNoBase(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualNoBase_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMoveDualNoBase_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);
	ENGINE_API bool ServerMoveDualNoBase_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, uint8 ClientMovementMode);

	/** Replicated function sent by client to server - contains client movement and view info for two moves. First move is non root motion, second is root motion. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualHybridRootMotion, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMoveDualHybridRootMotion(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualHybridRootMotion_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMoveDualHybridRootMotion_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API bool ServerMoveDualHybridRootMotion_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/* Resending an (important) old move. Process it if not already processed. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveOld, ServerMovePacked)
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerMoveOld(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveOld_Implementation, ServerMovePacked_Implementation)
	ENGINE_API void ServerMoveOld_Implementation(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);
	ENGINE_API bool ServerMoveOld_Validate(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);

	/** If no client adjustment is needed after processing received ServerMove(), ack the good move so client can remove it from SavedMoves */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAckGoodMove, ClientMoveResponsePacked)
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientAckGoodMove(float TimeStamp);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAckGoodMove_Implementation, ClientMoveResponsePacked_Implementation)
	ENGINE_API void ClientAckGoodMove_Implementation(float TimeStamp);

	/** Replicate position correction to client, associated with a timestamped servermove.  Client will replay subsequent moves after applying adjustment.  */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustPosition, ClientMoveResponsePacked)
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientAdjustPosition(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustPosition_Implementation, ClientMoveResponsePacked_Implementation)
	ENGINE_API void ClientAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientVeryShortAdjustPosition, ClientMoveResponsePacked)
	/* Bandwidth saving version, when velocity is zeroed */
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientVeryShortAdjustPosition(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientVeryShortAdjustPosition_Implementation, ClientMoveResponsePacked_Implementation)
	ENGINE_API void ClientVeryShortAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/** Replicate position correction to client when using root motion for movement. (animation root motion specific) */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionPosition, ClientMoveResponsePacked)
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientAdjustRootMotionPosition(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionPosition_Implementation, ClientMoveResponsePacked_Implementation)
	ENGINE_API void ClientAdjustRootMotionPosition_Implementation(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/** Replicate root motion source correction to client when using root motion for movement. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionSourcePosition, ClientMoveResponsePacked)
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientAdjustRootMotionSourcePosition(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionSourcePosition_Implementation, ClientMoveResponsePacked_Implementation)
	ENGINE_API void ClientAdjustRootMotionSourcePosition_Implementation(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	//////////////////////////////////////////////////////////////////////////
	// END DEPRECATED RPCs
	//////////////////////////////////////////////////////////////////////////

public:
	/** Returns Mesh subobject **/
	inline class USkeletalMeshComponent* GetMesh() const { return Mesh; }

	/** Name of the MeshComponent. Use this name if you want to prevent creation of the component (with ObjectInitializer.DoNotCreateDefaultSubobject). */
	static ENGINE_API FName MeshComponentName;

	/** Returns CharacterMovement subobject **/
	template <class T>
	inline T* GetCharacterMovement() const
	{
		return CastChecked<T>(CharacterMovement, ECastCheckedType::NullAllowed);
	}
	inline UCharacterMovementComponent* GetCharacterMovement() const { return CharacterMovement; }

	/** Name of the CharacterMovement component. Use this name if you want to use a different class (with ObjectInitializer.SetDefaultSubobjectClass). */
	static ENGINE_API FName CharacterMovementComponentName;

	/** Returns CapsuleComponent subobject **/
	inline class UCapsuleComponent* GetCapsuleComponent() const { return CapsuleComponent; }

	/** Name of the CapsuleComponent. */
	static ENGINE_API FName CapsuleComponentName;

#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif

	/** Sets the component the Character is walking on, used by CharacterMovement walking movement to be able to follow dynamic objects. */
	UE_DEPRECATED(5.8, "Movement base functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void SetBase(UPrimitiveComponent* NewBase, const FName BoneName = NAME_None, bool bNotifyActor=true);

	/** Sets the component the Character is walking on, used by CharacterMovement walking movement to be able to follow dynamic objects. */
	ENGINE_API virtual void SetBase(FMovementBaseInterfaceData* MovementBaseInterfaceData, const FName BoneName = NAME_None, bool bNotifyActor=true);
	
	/**
	 * Cache mesh offset from capsule. This is used as the target for network smoothing interpolation, when the mesh is offset with lagged smoothing.
	 * This is automatically called during initialization; call this at runtime if you intend to change the default mesh offset from the capsule.
	 * @see GetBaseTranslationOffset(), GetBaseRotationOffset()
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual void CacheInitialMeshOffset(FVector MeshRelativeLocation, FRotator MeshRelativeRotation);

protected:
	/** Info about our current movement base (object we are standing on). */
	UPROPERTY()
	struct FBasedMovementInfo BasedMovement;

	/** Replicated version of relative movement. Read-only on simulated proxies! */
	UPROPERTY(ReplicatedUsing=OnRep_ReplicatedBasedMovement)
	struct FBasedMovementInfo ReplicatedBasedMovement;

	/** This is the Replicated version of relative movement. Do not modify on simulated proxies! Returns non-const ReplicatedBasedMovement. */
	ENGINE_API FBasedMovementInfo& GetReplicatedBasedMovement_Mutable();
	/** This is the Replicated version of relative movement. Do not modify on simulated proxies! Sets ReplicatedBasedMovement*/
	ENGINE_API void SetReplicatedBasedMovement(const FBasedMovementInfo& InReplicatedBasedMovement);

public:
	/** Rep notify for ReplicatedBasedMovement */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedBasedMovement();

	/** Set whether this actor's movement replicates to network clients. */
	ENGINE_API virtual void SetReplicateMovement(bool bInReplicateMovement) override;

protected:
	/** Whether this Character should include acceleration data in its replicated movement */
	virtual bool ShouldReplicateAcceleration() const { return CharacterCVars::EnableCharacterAccelerationReplication != 0; }

	/** CharacterMovement ServerLastTransformUpdateTimeStamp value, replicated to simulated proxies. */
	UPROPERTY(Replicated)
	float ReplicatedServerLastTransformUpdateTimeStamp;

	UPROPERTY(ReplicatedUsing=OnRep_ReplayLastTransformUpdateTimeStamp)
	float ReplayLastTransformUpdateTimeStamp;

	/** Saved rotation offset of mesh. */
	UPROPERTY()
	FQuat BaseRotationOffset;

	/** Saved translation offset of mesh. */
	UPROPERTY()
	FVector BaseTranslationOffset;

	/** Event called after actor's base changes (if SetBase was requested to notify us with bNotifyPawn). */
	ENGINE_API virtual void BaseChange();

	/** CharacterMovement Custom gravity direction replicated for simulated proxies. */
	UPROPERTY(Replicated)
	FVector_NetQuantizeNormal ReplicatedGravityDirection;

	/** Cached version of the replicated gravity direction before replication. Used to compare if the value was changed as a result of replication. */
	FVector_NetQuantizeNormal PreNetReceivedGravityDirection;

	/** Scale to apply to root motion translation on this Character */
	UPROPERTY(Replicated)
	float AnimRootMotionTranslationScale;

	/** This is the value of the CharacterMovement timestamp, and is replicated down to SimulatedProxies to assist with visual smoothing of the character. May not be updated every frame to reduce bandwidth. This function sets ReplicatedServerLastTransformUpdateTimeStamp. */
	ENGINE_API void SetReplicatedServerLastTransformUpdateTimeStamp(const float InReplicatedServerLastTransformUpdateTimeStamp);

	/** This is updated every frame to the latest timestamp, but only replicated to Replays, this is because we want to save positions to the replay replication connection each frame to ensure smooth movement. Sets ReplayLastTransformUpdateTimeStamp.*/
	ENGINE_API void SetReplayLastTransformUpdateTimeStamp(const float InReplayLastTransformUpdateTimeStamp);

	/** CharacterMovement Custom gravity direction replicated for simulated proxies. This function sets ReplicatedGravityDirection and also marks property as dirty. */
	ENGINE_API void SetReplicatedGravityDirection(const FVector& InReplicatedGravityDirection);
public:
	UFUNCTION()
	ENGINE_API void OnRep_ReplayLastTransformUpdateTimeStamp();

	/** This is the value of the CharacterMovement timestamp, and is replicated down to SimulatedProxies to assist with visual smoothing of the character. May not be updated every frame to reduce bandwidth. This is the Accessor for ReplicatedServerLastTransformUpdateTimeStamp. */
	inline float GetReplicatedServerLastTransformUpdateTimeStamp() const { return ReplicatedServerLastTransformUpdateTimeStamp; }

	/** This is updated every frame to the latest timestamp, but only replicated to Replays, this is because we want to save positions to the replay replication connection each frame to ensure smooth movement. This is the Accessor for ReplayLastTransformUpdateTimeStamp. */
	inline float GetReplayLastTransformUpdateTimeStamp() const { return ReplayLastTransformUpdateTimeStamp; }

	/** Accessor for BasedMovement */
	inline const FBasedMovementInfo& GetBasedMovement() const { return BasedMovement; }
	
	/** Accessor for ReplicatedBasedMovement */
	inline const FBasedMovementInfo& GetReplicatedBasedMovement() const { return ReplicatedBasedMovement; }

	/** Save a new relative location in BasedMovement and a new rotation with is either relative or absolute. */
	ENGINE_API void SaveRelativeBasedMovement(const FVector& NewRelativeLocation, const FRotator& NewRotation, bool bRelativeRotation);

	/** CharacterMovement MovementMode (and custom mode) replicated for simulated proxies. Use CharacterMovementComponent::UnpackNetworkMovementMode() to translate it. Returns ReplicatedMovementMode */
	inline uint8 GetReplicatedMovementMode() const { return ReplicatedMovementMode; }

	/** Get the saved translation offset of mesh. This is how much extra offset is applied from the center of the capsule. */
	UFUNCTION(BlueprintCallable, Category=Character)
	FVector GetBaseTranslationOffset() const { return BaseTranslationOffset; }

	/** Get the saved rotation offset of mesh. This is how much extra rotation is applied from the capsule rotation. */
	virtual FQuat GetBaseRotationOffset() const { return BaseRotationOffset; }

	/** Get the saved rotation offset of mesh. This is how much extra rotation is applied from the capsule rotation. */
	UFUNCTION(BlueprintCallable, Category=Character, meta=(DisplayName="Get Base Rotation Offset", ScriptName="GetBaseRotationOffset"))
	FRotator GetBaseRotationOffsetRotator() const { return GetBaseRotationOffset().Rotator(); }

	/** Returns vector direction of gravity */
	ENGINE_API virtual FVector GetGravityDirection() const override;

	/** Returns a quaternion transforming from world space into gravity relative space */
	ENGINE_API virtual FQuat GetGravityTransform() const override;

	/** Returns replicated gravity direction for simulated proxies */
	ENGINE_API virtual FVector GetReplicatedGravityDirection() const;

	/** Default crouched eye height */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float CrouchedEyeHeight;

protected:
	/** Flag that we are receiving replication of the based movement. */
	UPROPERTY()
	uint8 bInBaseReplication:1;

public:
	/** Set by character movement to specify that this Character is currently crouched. */
	UPROPERTY(BlueprintReadOnly, replicatedUsing=OnRep_IsCrouched, Category=Character)
	uint8 bIsCrouched:1;

	/** Set to indicate that this Character is currently under the force of a jump (if JumpMaxHoldTime is non-zero). IsJumpProvidingForce() handles this as well. */
	UPROPERTY(Transient, Replicated)
	uint8 bProxyIsJumpForceApplied : 1;

	/** Handle Crouching replicated from server */
	UFUNCTION()
	ENGINE_API virtual void OnRep_IsCrouched();

	/** When true, player wants to jump */
	UPROPERTY(BlueprintReadOnly, Category=Character)
	uint8 bPressedJump:1;

	/** When true, applying updates to network client (replaying saved moves for a locally controlled character) */
	UPROPERTY(Transient)
	uint8 bClientUpdating:1;

	/** True if Pawn was initially falling when started to replay network moves. */
	UPROPERTY(Transient)
	uint8 bClientWasFalling:1; 

	/** If server disagrees with root motion track position, client has to resimulate root motion from last AckedMove. */
	UPROPERTY(Transient)
	uint8 bClientResimulateRootMotion:1;

	/** If server disagrees with root motion state, client has to resimulate root motion from last AckedMove. */
	UPROPERTY(Transient)
	uint8 bClientResimulateRootMotionSources:1;

	/** Disable simulated gravity (set when character encroaches geometry on client, to keep it from falling through floors) */
	UPROPERTY()
	uint8 bSimGravityDisabled:1;

	UPROPERTY(Transient)
	uint8 bClientCheckEncroachmentOnNetUpdate:1;

	/** Disable root motion on the server. When receiving a DualServerMove, where the first move is not root motion and the second is. */
	UPROPERTY(Transient)
	uint8 bServerMoveIgnoreRootMotion:1;

	/** Tracks whether or not the character was already jumping last frame. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category=Character)
	uint8 bWasJumping : 1;

protected:
	/** CharacterMovement bIsActive replicated for simulated proxies. */
	UPROPERTY(Transient, ReplicatedUsing=OnRep_IsCharacterMovementActive)
	uint8 bProxyIsCharacterMovementActive : 1;

	ENGINE_API void SetProxyCharacterMovementActive(bool bInActive);

	UFUNCTION()
	ENGINE_API virtual void OnRep_IsCharacterMovementActive();

	/** CharacterMovement MovementMode (and custom mode) replicated for simulated proxies. Use CharacterMovementComponent::UnpackNetworkMovementMode() to translate it. */
	UPROPERTY(Replicated)
	uint8 ReplicatedMovementMode;

	/** Sets CharacterMovement MovementMode (and custom mode) replicated for simulated proxies. Use CharacterMovementComponent::UnpackNetworkMovementMode() to translate it. This function sets the compressed value of ReplicatedMovementMode.*/
	ENGINE_API void SetReplicatedMovementMode(const uint8 InReplicatedMovementMode);

public:
	/** 
	 * Jump key Held Time.
	 * This is the time that the player has held the jump key, in seconds.
	 */
	UPROPERTY(Transient, BlueprintReadOnly, VisibleInstanceOnly, Category=Character)
	float JumpKeyHoldTime;

	/** Amount of jump force time remaining, if JumpMaxHoldTime > 0. */
	UPROPERTY(Transient, BlueprintReadOnly, VisibleInstanceOnly, Category=Character)
	float JumpForceTimeRemaining;

	/** Track last time a jump force started for a proxy. */
	UPROPERTY(Transient, BlueprintReadOnly, VisibleInstanceOnly, Category=Character)
	float ProxyJumpForceStartedTime;

	/** 
	 * The max time the jump key can be held.
	 * Note that if StopJumping() is not called before the max jump hold time is reached,
	 * then the character will carry on receiving vertical velocity. Therefore it is usually 
	 * best to call StopJumping() when jump input has ceased (such as a button up event).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category=Character, Meta=(ClampMin=0.0, UIMin=0.0))
	float JumpMaxHoldTime;

    /**
     * The max number of jumps the character can perform.
     * Note that if JumpMaxHoldTime is non zero and StopJumping is not called, the player
     * may be able to perform and unlimited number of jumps. Therefore it is usually
     * best to call StopJumping() when jump input has ceased (such as a button up event).
     */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Replicated, Category=Character)
    int32 JumpMaxCount;

    /**
     * Tracks the current number of jumps performed.
     * This is incremented in CheckJumpInput, used in CanJump_Implementation, and reset in OnMovementModeChanged.
     * When providing overrides for these methods, it's recommended to either manually
     * increment / reset this value, or call the Super:: method.
     */
    UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category=Character)
    int32 JumpCurrentCount;

	/**
	 * Represents the current number of jumps performed before CheckJumpInput modifies JumpCurrentCount.
	 * This is set in CheckJumpInput and is used in SetMoveFor and PrepMoveFor instead of JumpCurrentCount
	 * since CheckJumpInput can modify JumpCurrentCount.
	 * When providing overrides for these methods, it's recommended to either manually
	 * set this value, or call the Super:: method.
	*/
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = Character)
	int32 JumpCurrentCountPreJump;

	/** Incremented every time there is an Actor overlap event (start or stop) on this actor. */
	uint32 NumActorOverlapEventsCounter;

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin AActor Interface.
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void ClearCrossLevelReferences() override;
	ENGINE_API virtual void PreNetReceive() override;
	ENGINE_API virtual void PostNetReceive() override;
	ENGINE_API virtual void GatherCurrentMovement() override;
	ENGINE_API virtual void OnRep_ReplicatedMovement() override;
	ENGINE_API virtual void PostNetReceiveLocationAndRotation() override;
	ENGINE_API virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const override;
	ENGINE_API virtual UActorComponent* FindComponentByClass(const TSubclassOf<UActorComponent> ComponentClass) const override;
	ENGINE_API virtual void TornOff() override;
	ENGINE_API virtual void NotifyActorBeginOverlap(AActor* OtherActor);
	ENGINE_API virtual void NotifyActorEndOverlap(AActor* OtherActor);
	//~ End AActor Interface

	template<class T>
	T* FindComponentByClass() const
	{
		return AActor::FindComponentByClass<T>();
	}

	//~ Begin INavAgentInterface Interface
	ENGINE_API virtual FVector GetNavAgentLocation() const override;
	//~ End INavAgentInterface Interface

	//~ Begin APawn Interface.
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual UPawnMovementComponent* GetMovementComponent() const override;
	
	UE_DEPRECATED(5.8, "GetMovementBase is deprecated. Use GetMovementBaseObject or GetMovementBaseInterfaceData instead.")
	virtual UPrimitiveComponent* GetMovementBase() const override final { return Cast<UPrimitiveComponent>(GetMovementBaseObject()); }
	
	virtual UObject* GetMovementBaseObject() const override final { return BasedMovement.MovementBaseInterfaceData.GetMovementBaseObject(); }
	ENGINE_API virtual const FMovementBaseInterfaceData* GetMovementBaseInterfaceData() const override final;
	ENGINE_API virtual FMovementBaseInterfaceData* GetMovementBaseInterfaceData_Mutable() override final;
	
	ENGINE_API virtual float GetDefaultHalfHeight() const override;
	ENGINE_API virtual void TurnOff() override;
	ENGINE_API virtual void Restart() override;
	ENGINE_API virtual void PawnClientRestart() override;
	ENGINE_API virtual void PossessedBy(AController* NewController) override;
	ENGINE_API virtual void UnPossessed() override;
	ENGINE_API virtual void SetupPlayerInputComponent(class UInputComponent* PlayerInputComponent) override;
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	ENGINE_API virtual void RecalculateBaseEyeHeight() override;
	ENGINE_API virtual void UpdateNavigationRelevance() override;
	//~ End APawn Interface

	/** Apply momentum caused by damage. */
	ENGINE_API virtual void ApplyDamageMomentum(float DamageTaken, FDamageEvent const& DamageEvent, APawn* PawnInstigator, AActor* DamageCauser);

	/** 
	 * Make the character jump on the next update.	 
	 * If you want your character to jump according to the time that the jump key is held,
	 * then you can set JumpMaxHoldTime to some non-zero value. Make sure in this case to
	 * call StopJumping() when you want the jump's z-velocity to stop being applied (such 
	 * as on a button up event), otherwise the character will carry on receiving the 
	 * velocity until JumpKeyHoldTime reaches JumpMaxHoldTime.
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual void Jump();

	/** 
	 * Stop the character from jumping on the next update. 
	 * Call this from an input event (such as a button 'up' event) to cease applying
	 * jump Z-velocity. If this is not called, then jump z-velocity will be applied
	 * until JumpMaxHoldTime is reached.
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual void StopJumping();

	/**
	 * Check if the character can jump in the current state.
	 *
	 * The default implementation may be overridden or extended by implementing the custom CanJump event in Blueprints. 
	 * 
	 * @Return Whether the character can jump in the current state. 
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API bool CanJump() const;

	/** Indicates that this Character is currently under the force of a jump (if JumpMaxHoldTime is non-zero). IsJumpProvidingForce() handles this as well. This function returns bProxyIsJumpForceApplied */
	inline bool GetProxyIsJumpForceApplied() const { return bProxyIsJumpForceApplied; }
	/** Set to indicate that this Character is currently under the force of a jump (if JumpMaxHoldTime is non-zero). IsJumpProvidingForce() handles this as well. This function sets bProxyIsJumpForceApplied*/
	ENGINE_API void SetProxyIsJumpForceApplied(const bool bInProxyIsJumpForceApplied);

	/** Set by character movement to specify that this Character is currently crouched. This function returns bIsCrouched.*/
	ENGINE_API bool IsCrouched() const;
	/** Specifies whether this Character is currently crouched or not. This function sets bIsCrouched and marks property as dirty.*/
	ENGINE_API void SetIsCrouched(const bool bInIsCrouched);

protected:
	/**
	 * Customizable event to check if the character can jump in the current state.
	 * Default implementation returns true if the character is on the ground and not crouching,
	 * has a valid CharacterMovementComponent and CanEverJump() returns true.
	 * Default implementation also allows for 'hold to jump higher' functionality: 
	 * As well as returning true when on the ground, it also returns true when GetMaxJumpTime is more
	 * than zero and IsJumping returns true.
	 * 
	 *
	 * @Return Whether the character can jump in the current state. 
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Character, meta=(DisplayName="CanJump"))
	ENGINE_API bool CanJumpInternal() const;
	ENGINE_API virtual bool CanJumpInternal_Implementation() const;
	ENGINE_API bool JumpIsAllowedInternal() const;

public:

	/** Marks character as not trying to jump */
	ENGINE_API virtual void ResetJumpState();

	/**
	 * True if jump is actively providing a force, such as when the jump key is held and the time it has been held is less than JumpMaxHoldTime.
	 * @see CharacterMovement->IsFalling
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual bool IsJumpProvidingForce() const;

	/** Play Animation Montage on the character mesh. Returns the length of the animation montage in seconds, or 0.f if failed to play. **/
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API virtual float PlayAnimMontage(class UAnimMontage* AnimMontage, float InPlayRate = 1.f, FName StartSectionName = NAME_None);

	/** Stop Animation Montage. If nullptr, it will stop what's currently active. The Blend Out Time is taken from the montage asset that is being stopped. **/
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API virtual void StopAnimMontage(class UAnimMontage* AnimMontage = nullptr);

	/** Return current playing Montage **/
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API class UAnimMontage* GetCurrentMontage() const;

	/**
	 * Set a pending launch velocity on the Character. This velocity will be processed on the next CharacterMovementComponent tick,
	 * and will set it to the "falling" state. Triggers the OnLaunched event.
	 * @PARAM LaunchVelocity is the velocity to impart to the Character
	 * @PARAM bXYOverride if true replace the XY part of the Character's velocity instead of adding to it.
	 * @PARAM bZOverride if true replace the Z component of the Character's velocity instead of adding to it.
	 */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual void LaunchCharacter(FVector LaunchVelocity, bool bXYOverride, bool bZOverride);

	/** Let blueprint know that we were launched */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void OnLaunched(FVector LaunchVelocity, bool bXYOverride, bool bZOverride);

	/** Event fired when the character has just started jumping */
	UFUNCTION(BlueprintNativeEvent, Category=Character)
	ENGINE_API void OnJumped();
	ENGINE_API virtual void OnJumped_Implementation();

	/** Called when the character's movement enters falling */
	virtual void Falling() {}

	/** Called when character's jump reaches Apex. Needs CharacterMovement->bNotifyApex = true */
	ENGINE_API virtual void NotifyJumpApex();

	/** Broadcast when Character's jump reaches its apex. Needs CharacterMovement->bNotifyApex = true */
	UPROPERTY(BlueprintAssignable, Category=Character)
	FCharacterReachedApexSignature OnReachedJumpApex;

	/**
	 * Called upon landing when falling, to perform actions based on the Hit result. Triggers the OnLanded event.
	 * Note that movement mode is still "Falling" during this event. Current Velocity value is the velocity at the time of landing.
	 * Consider OnMovementModeChanged() as well, as that can be used once the movement mode changes to the new mode (most likely Walking).
	 *
	 * @param Hit Result describing the landing that resulted in a valid landing spot.
	 * @see OnMovementModeChanged()
	 */
	ENGINE_API virtual void Landed(const FHitResult& Hit);

	/**
	 * Called upon landing when falling, to perform actions based on the Hit result.
	 * Note that movement mode is still "Falling" during this event. Current Velocity value is the velocity at the time of landing.
	 * Consider OnMovementModeChanged() as well, as that can be used once the movement mode changes to the new mode (most likely Walking).
	 *
	 * @param Hit Result describing the landing that resulted in a valid landing spot.
	 * @see OnMovementModeChanged()
	 */
	UPROPERTY(BlueprintAssignable, Category=Character)
	FLandedSignature LandedDelegate;

	/**
	 * Called upon landing when falling, to perform actions based on the Hit result.
	 * Note that movement mode is still "Falling" during this event. Current Velocity value is the velocity at the time of landing.
	 * Consider OnMovementModeChanged() as well, as that can be used once the movement mode changes to the new mode (most likely Walking).
	 *
	 * @param Hit Result describing the landing that resulted in a valid landing spot.
	 * @see OnMovementModeChanged()
	 */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void OnLanded(const FHitResult& Hit);

	/**
	 * Event fired when the Character is walking off a surface and is about to fall because CharacterMovement->CurrentFloor became unwalkable.
	 * If CharacterMovement->MovementMode does not change during this event then the character will automatically start falling afterwards.
	 * @note Z velocity is zero during walking movement, and will be here as well. Another velocity can be computed here if desired and will be used when starting to fall.
	 *
	 * @param  PreviousFloorImpactNormal Normal of the previous walkable floor.
	 * @param  PreviousFloorContactNormal Normal of the contact with the previous walkable floor.
	 * @param  PreviousLocation	Previous character location before movement off the ledge.
	 * @param  TimeTick	Time delta of movement update resulting in moving off the ledge.
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Character)
	ENGINE_API void OnWalkingOffLedge(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta);
	ENGINE_API virtual void OnWalkingOffLedge_Implementation(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta);

	/**
	 * Called when pawn's movement is blocked
	 * @param Impact describes the blocking hit.
	 */
	virtual void MoveBlockedBy(const FHitResult& Impact) {};

	/**
	 * Request the character to start crouching. The request is processed on the next update of the CharacterMovementComponent.
	 * @see OnStartCrouch
	 * @see IsCrouched
	 * @see CharacterMovement->WantsToCrouch
	 */
	UFUNCTION(BlueprintCallable, Category=Character, meta=(HidePin="bClientSimulation"))
	ENGINE_API virtual void Crouch(bool bClientSimulation = false);

	/**
	 * Request the character to stop crouching. The request is processed on the next update of the CharacterMovementComponent.
	 * @see OnEndCrouch
	 * @see IsCrouched
	 * @see CharacterMovement->WantsToCrouch
	 */
	UFUNCTION(BlueprintCallable, Category=Character, meta=(HidePin="bClientSimulation"))
	ENGINE_API virtual void UnCrouch(bool bClientSimulation = false);

	/** @return true if this character is currently able to crouch (and is not currently crouched) */
	UFUNCTION(BlueprintCallable, Category=Character)
	ENGINE_API virtual bool CanCrouch() const;

	/** 
	 * Called when Character stops crouching. Called on non-owned Characters through bIsCrouched replication.
	 * @param	HalfHeightAdjust		difference between default collision half-height, and actual crouched capsule half-height.
	 * @param	ScaledHalfHeightAdjust	difference after component scale is taken in to account.
	 */
	ENGINE_API virtual void OnEndCrouch(float HalfHeightAdjust, float ScaledHalfHeightAdjust);

	/** 
	 * Event when Character stops crouching.
	 * @param	HalfHeightAdjust		difference between default collision half-height, and actual crouched capsule half-height.
	 * @param	ScaledHalfHeightAdjust	difference after component scale is taken in to account.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="OnEndCrouch", ScriptName="OnEndCrouch"))
	ENGINE_API void K2_OnEndCrouch(float HalfHeightAdjust, float ScaledHalfHeightAdjust);

	/**
	 * Called when Character crouches. Called on non-owned Characters through bIsCrouched replication.
	 * @param	HalfHeightAdjust		difference between default collision half-height, and actual crouched capsule half-height.
	 * @param	ScaledHalfHeightAdjust	difference after component scale is taken in to account.
	 */
	ENGINE_API virtual void OnStartCrouch(float HalfHeightAdjust, float ScaledHalfHeightAdjust);

	/**
	 * Event when Character crouches.
	 * @param	HalfHeightAdjust		difference between default collision half-height, and actual crouched capsule half-height.
	 * @param	ScaledHalfHeightAdjust	difference after component scale is taken in to account.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="OnStartCrouch", ScriptName="OnStartCrouch"))
	ENGINE_API void K2_OnStartCrouch(float HalfHeightAdjust, float ScaledHalfHeightAdjust);

	/** Calculates the crouched eye height based on movement component settings */
	ENGINE_API void RecalculateCrouchedEyeHeight();

	/**
	 * Called from CharacterMovementComponent to notify the character that the movement mode has changed.
	 * @param	PrevMovementMode	Movement mode before the change
	 * @param	PrevCustomMode		Custom mode before the change (applicable if PrevMovementMode is Custom)
	 */
	ENGINE_API virtual void OnMovementModeChanged(EMovementMode PrevMovementMode, uint8 PreviousCustomMode = 0);

	/** Multicast delegate for MovementMode changing. */
	UPROPERTY(BlueprintAssignable, Category=Character)
	FMovementModeChangedSignature MovementModeChangedDelegate;

	/**
	 * Called from CharacterMovementComponent to notify the character that the movement mode has changed.
	 * @param	PrevMovementMode	Movement mode before the change
	 * @param	NewMovementMode		New movement mode
	 * @param	PrevCustomMode		Custom mode before the change (applicable if PrevMovementMode is Custom)
	 * @param	NewCustomMode		New custom mode (applicable if NewMovementMode is Custom)
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="OnMovementModeChanged", ScriptName="OnMovementModeChanged"))
	ENGINE_API void K2_OnMovementModeChanged(EMovementMode PrevMovementMode, EMovementMode NewMovementMode, uint8 PrevCustomMode, uint8 NewCustomMode);

	/**
	 * Event for implementing custom character movement mode. Called by CharacterMovement if MovementMode is set to Custom.
	 * @note C++ code should override UCharacterMovementComponent::PhysCustom() instead.
	 * @see UCharacterMovementComponent::PhysCustom()
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="UpdateCustomMovement", ScriptName="UpdateCustomMovement"))
	ENGINE_API void K2_UpdateCustomMovement(float DeltaTime);

	/**
	 * Event triggered at the end of a CharacterMovementComponent movement update.
	 * This is the preferred event to use rather than the Tick event when performing custom updates to CharacterMovement properties based on the current state.
	 * This is mainly due to the nature of network updates, where client corrections in position from the server can cause multiple iterations of a movement update,
	 * which allows this event to update as well, while a Tick event would not.
	 *
	 * @param	DeltaSeconds		Delta time in seconds for this update
	 * @param	InitialLocation		Location at the start of the update. May be different than the current location if movement occurred.
	 * @param	InitialVelocity		Velocity at the start of the update. May be different than the current velocity.
	 */
	UPROPERTY(BlueprintAssignable, Category=Character)
	FCharacterMovementUpdatedSignature OnCharacterMovementUpdated;

	/** Returns true if the Landed() event should be called. Used by CharacterMovement to prevent notifications while playing back network moves. */
	ENGINE_API virtual bool ShouldNotifyLanded(const struct FHitResult& Hit);

	/** Trigger jump if jump button has been pressed. */
	ENGINE_API virtual void CheckJumpInput(float DeltaTime);

	/** Update jump input state after having checked input. */
	ENGINE_API virtual void ClearJumpInput(float DeltaTime);

	/**
	 * Get the maximum jump time for the character.
	 * Note that if StopJumping() is not called before the max jump hold time is reached,
	 * then the character will carry on receiving vertical velocity. Therefore it is usually 
	 * best to call StopJumping() when jump input has ceased (such as a button up event).
	 * 
	 * @return Maximum jump time for the character
	 */
	ENGINE_API virtual float GetJumpMaxHoldTime() const;

	/** 
	 * By default this function returns false, which prevents jumping while in falling state (in many cases, we use this to consume the first jump), 
     * but subclasses can override this behavior to allow jump to happen while falling under specific conditions they may care about 
	 */
	ENGINE_API virtual bool CanJumpWhileFalling() const;

	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientCheatWalk();
	ENGINE_API virtual void ClientCheatWalk_Implementation();

	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientCheatFly();
	ENGINE_API virtual void ClientCheatFly_Implementation();

	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientCheatGhost();
	ENGINE_API virtual void ClientCheatGhost_Implementation();

	UFUNCTION(Reliable, Client)
	ENGINE_API void RootMotionDebugClientPrintOnScreen(const FString& InString);
	ENGINE_API virtual void RootMotionDebugClientPrintOnScreen_Implementation(const FString& InString);

	// Root Motion

	/** 
	 * For LocallyControlled Autonomous clients. 
	 * During a PerformMovement() after root motion is prepared, we save it off into this and
	 * then record it into our SavedMoves.
	 * During SavedMove playback we use it as our "Previous Move" SavedRootMotion which includes
	 * last received root motion from the Server
	 */
	UPROPERTY(Transient)
	FRootMotionSourceGroup SavedRootMotion;

	/** For LocallyControlled Autonomous clients. Saved root motion data to be used by SavedMoves. */
	UPROPERTY(Transient)
	FRootMotionMovementParams ClientRootMotionParams;

	/** Array of previously received root motion moves from the server. */
	UPROPERTY(Transient)
	TArray<FSimulatedRootMotionReplicatedMove> RootMotionRepMoves;
	
	/** Find usable root motion replicated move from our buffer.
	 * Goes through the buffer back in time, to find the first move that clears 'CanUseRootMotionRepMove' below.
	 * Returns index of that move or INDEX_NONE otherwise.
	 */
	ENGINE_API int32 FindRootMotionRepMove(const FAnimMontageInstance& ClientMontageInstance) const;

	/** True if buffered move is usable to teleport client back to. */
	ENGINE_API bool CanUseRootMotionRepMove(const FSimulatedRootMotionReplicatedMove& RootMotionRepMove, const FAnimMontageInstance& ClientMontageInstance) const;

	/** Restore actor to an old buffered move. */
	ENGINE_API bool RestoreReplicatedMove(const FSimulatedRootMotionReplicatedMove& RootMotionRepMove);
	
	/**
	 * Called on client after position update is received to respond to the new location and rotation.
	 * Actual change in location is expected to occur in CharacterMovement->SmoothCorrection(), after which this occurs.
	 * Default behavior is to check for penetration in a blocking object if bClientCheckEncroachmentOnNetUpdate is enabled, and set bSimGravityDisabled=true if so.
	 */
	ENGINE_API virtual void OnUpdateSimulatedPosition(const FVector& OldLocation, const FQuat& OldRotation);

	/** Replicated Root Motion montage */
	UPROPERTY(ReplicatedUsing=OnRep_RootMotion)
	struct FRepRootMotionMontage RepRootMotion;

	/** Replicated Root Motion montage. Returns RepRootMotion */
	inline const FRepRootMotionMontage& GetRepRootMotion() const { return RepRootMotion; }

	/** Replicated Root Motion montage. Returns non-const RepRootMotion*/
	ENGINE_API FRepRootMotionMontage& GetRepRootMotion_Mutable();
	/** Replicated Root Motion montage. Sets RepRootMotion*/
	ENGINE_API void SetRepRootMotion(const FRepRootMotionMontage& InRepRootMotion);
	
	/** Handles replicated root motion properties on simulated proxies and position correction. */
	UFUNCTION()
	ENGINE_API void OnRep_RootMotion();

	/** Position fix up for Simulated Proxies playing Root Motion */
	ENGINE_API void SimulatedRootMotionPositionFixup(float DeltaSeconds);

	/** Get FAnimMontageInstance playing RootMotion */
	ENGINE_API FAnimMontageInstance * GetRootMotionAnimMontageInstance() const;

	/** True if we are playing Anim root motion right now */
	UFUNCTION(BlueprintCallable, Category=Animation, meta=(DisplayName="Is Playing Anim Root Motion"))
	ENGINE_API bool IsPlayingRootMotion() const;

	/** True if we are playing root motion from any source right now (anim root motion, root motion source) */
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API bool HasAnyRootMotion() const;

	/**
	 * True if we are playing Root Motion right now, through a Montage with RootMotionMode == ERootMotionMode::RootMotionFromMontagesOnly.
	 * This means code path for networked root motion is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API bool IsPlayingNetworkedRootMotionMontage() const;

	/** Sets scale to apply to root motion translation on this Character */
	ENGINE_API void SetAnimRootMotionTranslationScale(float InAnimRootMotionTranslationScale = 1.f);

	/** Scale to apply to root motion translation on this Character. Returns current value of AnimRootMotionScale. */
	UFUNCTION(BlueprintCallable, Category=Animation)
	ENGINE_API float GetAnimRootMotionTranslationScale() const;

	/**
	 * Called on the actor right before replication occurs.
	 * Only called on Server, and for autonomous proxies if recording a Client Replay.
	 */
	ENGINE_API virtual void PreReplication(IRepChangedPropertyTracker & ChangedPropertyTracker) override;

	/**
	 * Called on the actor right before replication occurs.
	 * Called for everyone when recording a Client Replay, including Simulated Proxies.
	 */
	ENGINE_API virtual void PreReplicationForReplay(IRepChangedPropertyTracker& ChangedPropertyTracker) override;

	/** Async simulation API */
	ENGINE_API void FillAsyncInput(FCharacterAsyncInput& Input) const;
	ENGINE_API void InitializeAsyncOutput(FCharacterAsyncOutput& Output) const;
	ENGINE_API void ApplyAsyncOutput(const FCharacterAsyncOutput& Output);

	/** Sets bOnlyAllowAutonomousTickPose based on the character's current state */
	ENGINE_API void UpdateMeshAllowAutonomousTickPose();
};
