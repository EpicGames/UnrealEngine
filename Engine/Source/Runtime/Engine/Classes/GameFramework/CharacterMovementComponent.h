// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/RandomStream.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "WorldCollision.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimMontage.h"
#include "GameFramework/RootMotionSource.h"
#include "AI/Navigation/NavigationAvoidanceTypes.h"
#include "AI/RVOAvoidanceInterface.h"
#include "GameFramework/PawnMovementComponent.h"
#include "GameFramework/CharacterMovementReplication.h"
#include "Interfaces/NetworkPredictionInterface.h"
#include "CharacterMovementComponentAsync.h"
#include "Interfaces/MovementBaseInterface.h"
#include "CharacterMovementComponent.generated.h"

class ACharacter;
class FDebugDisplayInfo;
class FNetworkPredictionData_Server_Character;
class FSavedMove_Character;
class UPrimitiveComponent;
class INavigationData;
class UCharacterMovementComponent;

DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FOnProcessRootMotion, const FTransform&, UCharacterMovementComponent*, float)

namespace CharacterMovementConstants
{
	extern const float MAX_STEP_SIDE_Z;
	extern const float VERTICAL_SLOPE_NORMAL_Z;
}

namespace CharacterMovementCVars
{
	// Is Async Character Movement enabled?
	extern ENGINE_API int32 AsyncCharacterMovement;
	extern int32 ForceJumpPeakSubstep;
}

/** 
 * Tick function that calls UCharacterMovementComponent::PostPhysicsTickComponent
 **/
USTRUCT()
struct FCharacterMovementComponentPostPhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** CharacterMovementComponent that is the target of this tick **/
	class UCharacterMovementComponent* Target;

	/** 
	 * Abstract function actually execute the tick. 
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completion of this task until certain child tasks are complete.
	 **/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;

	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FCharacterMovementComponentPostPhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FCharacterMovementComponentPostPhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

USTRUCT()
struct FCharacterMovementComponentPrePhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** CharacterMovementComponent that is the target of this tick **/
	class UCharacterMovementComponent* Target;

	/**
	 * Abstract function actually execute the tick.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completion of this task until certain child tasks are complete.
	 **/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;

	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FCharacterMovementComponentPrePhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FCharacterMovementComponentPrePhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/** Shared pointer for easy memory management of FSavedMove_Character, for accumulating and replaying network moves. */
typedef TSharedPtr<class FSavedMove_Character> FSavedMovePtr;


//=============================================================================
/**
 * CharacterMovementComponent handles movement logic for the associated Character owner.
 * It supports various movement modes including: walking, falling, swimming, flying, custom.
 *
 * Movement is affected primarily by current Velocity and Acceleration. Acceleration is updated each frame
 * based on the input vector accumulated thus far (see UPawnMovementComponent::GetPendingInputVector()).
 *
 * Networking is fully implemented, with server-client correction and prediction included.
 *
 * @see ACharacter, UPawnMovementComponent
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Pawn/Character/
 */

UCLASS(MinimalAPI)
class UCharacterMovementComponent : public UPawnMovementComponent, public IRVOAvoidanceInterface, public INetworkPredictionInterface
{
	GENERATED_BODY()
public:

	/**
	 * Default UObject constructor.
	 */
	ENGINE_API UCharacterMovementComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:

	/** Character movement component belongs to */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<ACharacter> CharacterOwner;

public:

	/** Custom gravity scale. Gravity is multiplied by this amount for the character. */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite)
	float GravityScale;

	/** Maximum height character can step up */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm"))
	float MaxStepHeight;

	/** Initial velocity (instantaneous vertical acceleration) when jumping. */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(DisplayName="Jump Z Velocity", ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float JumpZVelocity;

	/** Fraction of JumpZVelocity to use when automatically "jumping off" of a base actor that's not allowed to be a base for a character. (For example, if you're not allowed to stand on other players.) */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0"))
	float JumpOffJumpZFactor;


private:
	FCharacterMovementComponentAsyncCallback* AsyncCallback;

protected:
	// This is the most recent async state from simulated. Only safe for access on physics thread.
	TSharedPtr<FCharacterMovementComponentAsyncOutput, ESPMode::ThreadSafe> AsyncSimState;
	bool bMovementModeDirty = false; // Gamethread changed movement mode, need to update sim.
private:

	/**
	 * Max angle in degrees of a walkable surface. Any greater than this and it is too steep to be walkable.
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, meta=(ClampMin="0.0", ClampMax="90.0", UIMin = "0.0", UIMax = "90.0", ForceUnits="degrees"))
	float WalkableFloorAngle;

	/**
	 * Minimum Z value for floor normal. If less, not a walkable surface. Computed from WalkableFloorAngle.
	 */
	UPROPERTY(Category="Character Movement: Walking", VisibleAnywhere)
	float WalkableFloorZ;

	/**
	 * A normalized vector representing the direction of gravity for gravity relative movement modes: walking, falling,
	 * and custom movement modes. Gravity direction remaps player input as being within the plane defined by the gravity
	 * direction. Movement simulation values like velocity and acceleration are maintained in their existing world coordinate
	 * space but are transformed internally as gravity relative (for instance moving forward up a vertical wall that gravity is
	 * defined to be perpendicular to and jump "up" from that wall). If ShouldRemainVertical() is true the character's capsule
	 * will be oriented to align with the gravity direction.
	 */
	UPROPERTY(Category="Character Movement: Gravity", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = true))
	FVector GravityDirection;

	/**
	 * A cached quaternion representing the rotation from world space to gravity relative space defined by GravityDirection.
	 */
	UPROPERTY(Category="Character Movement: Gravity", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = true))
	FQuat WorldToGravityTransform;

	/**
	 * A cached quaternion representing the inverse rotation from world space to gravity relative space defined by GravityDirection.
	 */
	UPROPERTY(Category="Character Movement: Gravity", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = true))
	FQuat GravityToWorldTransform;

	/** Whether the character has custom local gravity set. Cached in SetGravityDirection(). */
	bool bHasCustomGravity;
public:
	
	/**
	 * Actor's current movement mode (walking, falling, etc).
	 *    - walking:  Walking on a surface, under the effects of friction, and able to "step up" barriers. Vertical velocity is zero.
	 *    - falling:  Falling under the effects of gravity, after jumping or walking off the edge of a surface.
	 *    - flying:   Flying, ignoring the effects of gravity.
	 *    - swimming: Swimming through a fluid volume, under the effects of gravity and buoyancy.
	 *    - custom:   User-defined custom movement mode, including many possible sub-modes.
	 * This is automatically replicated through the Character owner and for client-server movement functions.
	 * @see SetMovementMode(), CustomMovementMode
	 */
	UPROPERTY(Category="Character Movement: MovementMode", BlueprintReadOnly)
	TEnumAsByte<enum EMovementMode> MovementMode;

	/**
	 * Current custom sub-mode if MovementMode is set to Custom.
	 * This is automatically replicated through the Character owner and for client-server movement functions.
	 * @see SetMovementMode()
	 */
	UPROPERTY(Category="Character Movement: MovementMode", BlueprintReadOnly)
	uint8 CustomMovementMode;

	/** The default direction that gravity points for movement simulation.  */
	static ENGINE_API const FVector DefaultGravityDirection;

	/** Smoothing mode for simulated proxies in network game. */
	UPROPERTY(Category="Character Movement (Networking)", EditAnywhere, BlueprintReadOnly)
	ENetworkSmoothingMode NetworkSmoothingMode;

	/**
	 * Setting that affects movement control. Higher values allow faster changes in direction.
	 * If bUseSeparateBrakingFriction is false, also affects the ability to stop more quickly when braking (whenever Acceleration is zero), where it is multiplied by BrakingFrictionFactor.
	 * When braking, this property allows you to control how much friction is applied when moving across the ground, applying an opposing force that scales with current velocity.
	 * This can be used to simulate slippery surfaces such as ice or oil by changing the value (possibly based on the material pawn is standing on).
	 * @see BrakingDecelerationWalking, BrakingFriction, bUseSeparateBrakingFriction, BrakingFrictionFactor
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float GroundFriction;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/**
	* An extra scaling factor on the impact physics force applied to pushing object when the component is doing substepping 
	*/
	float PhysicsForceSubsteppingFactor = 1.0f;
#endif
	

	/** Saved location of object we are standing on, for UpdateBasedMovement() to determine if base moved in the last frame, and therefore pawn needs an update. */
	UPROPERTY(Transient)
	FQuat OldBaseQuat;

	/** Saved location of object we are standing on, for UpdateBasedMovement() to determine if base moved in the last frame, and therefore pawn needs an update. */
	UPROPERTY(Transient)
	FVector OldBaseLocation;

	/** The maximum ground speed when walking. Also determines maximum lateral speed when falling. */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float MaxWalkSpeed;

	/** The maximum ground speed when walking and crouched. */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float MaxWalkSpeedCrouched;

	/** The maximum swimming speed. */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float MaxSwimSpeed;

	/** The maximum flying speed. */
	UPROPERTY(Category="Character Movement: Flying", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float MaxFlySpeed;

	/** The maximum speed when using Custom movement mode. */
	UPROPERTY(Category="Character Movement: Custom Movement", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", ForceUnits="cm/s"))
	float MaxCustomMovementSpeed;

	/** Max Acceleration (rate of change of velocity) */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float MaxAcceleration;

	/** The ground speed that we should accelerate up to when walking at minimum analog stick tilt */
	UPROPERTY(Category = "Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", UIMin = "0", ForceUnits="cm/s"))
	float MinAnalogWalkSpeed;

	/**
	 * Factor used to multiply actual value of friction used when braking.
	 * This applies to any friction value that is currently used, which may depend on bUseSeparateBrakingFriction.
	 * @note This is 2 by default for historical reasons, a value of 1 gives the true drag equation.
	 * @see bUseSeparateBrakingFriction, GroundFriction, BrakingFriction
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float BrakingFrictionFactor;

	/**
	 * Friction (drag) coefficient applied when braking (whenever Acceleration = 0, or if character is exceeding max speed); actual value used is this multiplied by BrakingFrictionFactor.
	 * When braking, this property allows you to control how much friction is applied when moving across the ground, applying an opposing force that scales with current velocity.
	 * Braking is composed of friction (velocity-dependent drag) and constant deceleration.
	 * This is the current value, used in all movement modes; if this is not desired, override it or bUseSeparateBrakingFriction when movement mode changes.
	 * @note Only used if bUseSeparateBrakingFriction setting is true, otherwise current friction such as GroundFriction is used.
	 * @see bUseSeparateBrakingFriction, BrakingFrictionFactor, GroundFriction, BrakingDecelerationWalking
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", EditCondition="bUseSeparateBrakingFriction"))
	float BrakingFriction;

	/**
	 * Time substepping when applying braking friction. Smaller time steps increase accuracy at the slight cost of performance, especially if there are large frame times.
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0.0166", ClampMax="0.05", UIMin="0.0166", UIMax="0.05"))
	float BrakingSubStepTime;

	/**
	 * Deceleration when walking and not applying acceleration. This is a constant opposing force that directly lowers velocity by a constant value.
	 * @see GroundFriction, MaxAcceleration
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float BrakingDecelerationWalking;

	/**
	 * Lateral deceleration when falling and not applying acceleration.
	 * @see MaxAcceleration
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float BrakingDecelerationFalling;

	/**
	 * Deceleration when swimming and not applying acceleration.
	 * @see MaxAcceleration
	 */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float BrakingDecelerationSwimming;

	/**
	 * Deceleration when flying and not applying acceleration.
	 * @see MaxAcceleration
	 */
	UPROPERTY(Category="Character Movement: Flying", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float BrakingDecelerationFlying;

	/**
	 * When falling, amount of lateral movement control available to the character.
	 * 0 = no control, 1 = full control at max speed of MaxWalkSpeed.
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float AirControl;

	/**
	 * When falling, multiplier applied to AirControl when lateral velocity is less than AirControlBoostVelocityThreshold.
	 * Setting this to zero will disable air control boosting. Final result is clamped at 1.
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float AirControlBoostMultiplier;

	/**
	 * When falling, if lateral velocity magnitude is less than this value, AirControl is multiplied by AirControlBoostMultiplier.
	 * Setting this to zero will disable air control boosting.
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float AirControlBoostVelocityThreshold;

	/**
	 * Friction to apply to lateral air movement when falling.
	 * If bUseSeparateBrakingFriction is false, also affects the ability to stop more quickly when braking (whenever Acceleration is zero).
	 * @see BrakingFriction, bUseSeparateBrakingFriction
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float FallingLateralFriction;

	/** Collision half-height when crouching (component scale is applied separately) */
	UE_DEPRECATED_FORGAME(5.0, "Public access to this property is deprecated, and it will become private in a future release. Please use SetCrouchedHalfHeight and GetCrouchedHalfHeight instead.")
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetCrouchedHalfHeight, BlueprintGetter=GetCrouchedHalfHeight, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float CrouchedHalfHeight;

	/** Water buoyancy. A ratio (1.0 = neutral buoyancy, 0.0 = no buoyancy) */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite)
	float Buoyancy;

	/**
	 * Don't allow the character to perch on the edge of a surface if the contact is this close to the edge of the capsule.
	 * Note that characters will not fall off if they are within MaxStepHeight of a walkable surface below.
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float PerchRadiusThreshold;

	/**
	 * When perching on a ledge, add this additional distance to MaxStepHeight when determining how high above a walkable floor we can perch.
	 * Note that we still enforce MaxStepHeight to start the step up; this just allows the character to hang off the edge or step slightly higher off the floor.
	 * (@see PerchRadiusThreshold)
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float PerchAdditionalHeight;

	/** Change in rotation per second, used when UseControllerDesiredRotation or OrientRotationToMovement are true. Set a negative value for infinite rotation rate and instant turns. */
	UPROPERTY(Category="Character Movement (Rotation Settings)", EditAnywhere, BlueprintReadWrite)
	FRotator RotationRate;

	/**
	 * If true, BrakingFriction will be used to slow the character to a stop (when there is no Acceleration).
	 * If false, braking uses the same friction passed to CalcVelocity() (ie GroundFriction when walking), multiplied by BrakingFrictionFactor.
	 * This setting applies to all movement modes; if only desired in certain modes, consider toggling it when movement modes change.
	 * @see BrakingFriction
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditDefaultsOnly, BlueprintReadWrite)
	uint8 bUseSeparateBrakingFriction:1;

	/** 
	 * True means while the jump key is held, we will not allow the vertical speed to fall below the JumpZVelocity tuning value 
	 * even if a stronger force, such as gravity, is opposing the jump. 
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bDontFallBelowJumpZVelocityDuringJump:1;

	/**
	 *	Apply gravity while the character is actively jumping (e.g. holding the jump key).
	 *	Helps remove frame-rate dependent jump height, but may alter base jump height.
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bApplyGravityWhileJumping:1;
	/**
	 * If true, smoothly rotate the Character toward the Controller's desired rotation (typically Controller->ControlRotation), using RotationRate as the rate of rotation change. Overridden by OrientRotationToMovement.
	 * Normally you will want to make sure that other settings are cleared, such as bUseControllerRotationYaw on the Character.
	 */
	UPROPERTY(Category="Character Movement (Rotation Settings)", EditAnywhere, BlueprintReadWrite)
	uint8 bUseControllerDesiredRotation:1;

	/**
	 * If true, rotate the Character toward the direction of acceleration, using RotationRate as the rate of rotation change. Overrides UseControllerDesiredRotation.
	 * Normally you will want to make sure that other settings are cleared, such as bUseControllerRotationYaw on the Character.
	 */
	UPROPERTY(Category="Character Movement (Rotation Settings)", EditAnywhere, BlueprintReadWrite)
	uint8 bOrientRotationToMovement:1;

	/**
	 * Whether or not the character should sweep for collision geometry while walking.
	 * @see USceneComponent::MoveComponent.
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite)
	uint8 bSweepWhileNavWalking:1;

private:

	// Tracks whether or not we need to update the bSweepWhileNavWalking flag do to an upgrade.
	uint8 bNeedsSweepWhileWalkingUpdate:1;

protected:

	/**
	 * True during movement update.
	 * Used internally so that attempts to change CharacterOwner and UpdatedComponent are deferred until after an update.
	 * @see IsMovementInProgress()
	 */
	UPROPERTY()
	uint8 bMovementInProgress:1;

public:

	/**
	 * If true, high-level movement updates will be wrapped in a movement scope that accumulates updates and defers a bulk of the work until the end.
	 * When enabled, touch and hit events will not be triggered until the end of multiple moves within an update, which can improve performance.
	 *
	 * @see FScopedMovementUpdate
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, AdvancedDisplay)
	uint8 bEnableScopedMovementUpdates:1;

	/**
	 * Optional scoped movement update to combine moves for cheaper performance on the server when the client sends two moves in one packet.
	 * Be warned that since this wraps a larger scope than is normally done with bEnableScopedMovementUpdates, this can result in subtle changes in behavior
	 * in regards to when overlap events are handled, when attached components are moved, etc.
	 *
	 * @see bEnableScopedMovementUpdates
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, AdvancedDisplay)
	uint8 bEnableServerDualMoveScopedMovementUpdates : 1;

	/** Ignores size of acceleration component, and forces max acceleration to drive character at full velocity. */
	UPROPERTY()
	uint8 bForceMaxAccel:1;    

	/**
	 * If true, movement will be performed even if there is no Controller for the Character owner.
	 * Normally without a Controller, movement will be aborted and velocity and acceleration are zeroed if the character is walking.
	 * Characters that are spawned without a Controller but with this flag enabled will initialize the movement mode to DefaultLandMovementMode or DefaultWaterMovementMode appropriately.
	 * @see DefaultLandMovementMode, DefaultWaterMovementMode
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bRunPhysicsWithNoController:1;

	/**
	 * Force the Character in MOVE_Walking to do a check for a valid floor even if it hasn't moved. Cleared after next floor check.
	 * Normally if bAlwaysCheckFloor is false we try to avoid the floor check unless some conditions are met, but this can be used to force the next check to always run.
	 */
	UPROPERTY(Category="Character Movement: Walking", VisibleInstanceOnly, BlueprintReadWrite, AdvancedDisplay)
	uint8 bForceNextFloorCheck:1;

	/** If true, the capsule needs to be shrunk on this simulated proxy, to avoid replication rounding putting us in geometry.
	  * Whenever this is set to true, this will cause the capsule to be shrunk again on the next update, and then set to false. */
	UPROPERTY()
	uint8 bShrinkProxyCapsule:1;

	/** If true, Character can walk off a ledge. */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite)
	uint8 bCanWalkOffLedges:1;

	/** If true, Character can walk off a ledge when crouching. */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite)
	uint8 bCanWalkOffLedgesWhenCrouching:1;

	/**
	 * Signals that smoothed position/rotation has reached target, and no more smoothing is necessary until a future update.
	 * This is used as an optimization to skip calls to SmoothClientPosition() when true. SmoothCorrection() sets it false when a new network update is received.
	 * SmoothClientPosition_Interpolate() sets this to true when the interpolation reaches the target, before one last call to SmoothClientPosition_UpdateVisuals().
	 * If this is not desired, override SmoothClientPosition() to always set this to false to avoid this feature.
	 */
	uint8 bNetworkSmoothingComplete:1;

	/** Flag indicating the client correction was larger than NetworkLargeClientCorrectionThreshold. */
	uint8 bNetworkLargeClientCorrection:1;

	/**
	 * Whether we skip prediction on frames where a proxy receives a network update. This can avoid expensive prediction on those frames,
	 * with the side-effect of predicting with a frame of additional latency.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly)
	uint8 bNetworkSkipProxyPredictionOnNetUpdate:1;

	/**
	 * Flag used on the server to determine whether to always replicate ReplicatedServerLastTransformUpdateTimeStamp to clients.
	 * Normally this is only sent when the network smoothing mode on character movement is set to Linear smoothing (on the server), to save bandwidth.
	 * Setting this to true will force the timestamp to replicate regardless, in case the server doesn't know about the smoothing mode, or if the timestamp is used for another purpose.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay)
	uint8 bNetworkAlwaysReplicateTransformUpdateTimestamp:1;

public:

	/** true to update CharacterOwner and UpdatedComponent after movement ends */
	UPROPERTY()
	uint8 bDeferUpdateMoveComponent:1;

	/** If enabled, the player will interact with physics objects when walking into them. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite)
	uint8 bEnablePhysicsInteraction:1;

	/** If enabled, the TouchForceFactor is applied per kg mass of the affected object. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	uint8 bTouchForceScaledToMass:1;

	/** If enabled, the PushForceFactor is applied per kg mass of the affected object. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	uint8 bPushForceScaledToMass:1;

	/** If enabled, the PushForce location is moved using PushForcePointZOffsetFactor. Otherwise simply use the impact point. */
	UPROPERTY(Category = "Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bEnablePhysicsInteraction"))
	uint8 bPushForceUsingZOffset:1;

	/** If enabled, the applied push force will try to get the physics object to the same velocity than the player, not faster. This will only
		scale the force down, it will never apply more force than defined by PushForceFactor. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	uint8 bScalePushForceToVelocity:1;

	/** What to update CharacterOwner and UpdatedComponent after movement ends */
	UPROPERTY()
	TObjectPtr<USceneComponent> DeferredUpdatedMoveComponent;

	/** Maximum step height for getting out of water */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float MaxOutOfWaterStepHeight;

	/** Z velocity applied when pawn tries to get out of water */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ForceUnits="cm/s"))
	float OutofWaterZ;

	/** Mass of pawn (for when momentum is imparted to it). */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0"))
	float Mass;

	/** Force applied to objects we stand on (due to Mass and Gravity) is scaled by this amount. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float StandingDownwardForceScale;

	/** Initial impulse force to apply when the player bounces into a blocking physics object. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float InitialPushForceFactor;

	/** Force to apply when the player collides with a blocking physics object. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float PushForceFactor;

	/** Z-Offset for the position the force is applied to. 0.0f is the center of the physics object, 1.0f is the top and -1.0f is the bottom of the object. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(UIMin = "-1.0", UIMax = "1.0"), meta=(editcondition = "bEnablePhysicsInteraction"))
	float PushForcePointZOffsetFactor;

	/** Force to apply to physics objects that are touched by the player. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float TouchForceFactor;

	/** Minimum Force applied to touched physics objects. If < 0.0f, there is no minimum. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float MinTouchForce;

	/** Maximum force applied to touched physics objects. If < 0.0f, there is no maximum. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float MaxTouchForce;

	/** Force per kg applied constantly to all overlapping components. */
	UPROPERTY(Category="Character Movement: Physics Interaction", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bEnablePhysicsInteraction"))
	float RepulsionForce;


public:
#if WITH_EDITORONLY_DATA
	// Deprecated properties
	UPROPERTY()
	uint32 bForceBraking_DEPRECATED:1;

	/** Multiplier to max ground speed to use when crouched */
	UPROPERTY()
	float CrouchedSpeedMultiplier_DEPRECATED;

	UPROPERTY()
	float UpperImpactNormalScale_DEPRECATED;
#endif

protected:

	/**
	 * Current acceleration vector (with magnitude).
	 * This is calculated each update based on the input vector and the constraints of MaxAcceleration and the current movement mode.
	 */
	UPROPERTY()
	FVector Acceleration;

	/**
	 * Rotation after last PerformMovement or SimulateMovement update.
	 */
	UPROPERTY()
	FQuat LastUpdateRotation;

	/**
	 * Location after last PerformMovement or SimulateMovement update. Used internally to detect changes in position from outside character movement to try to validate the current floor.
	 */
	UPROPERTY()
	FVector LastUpdateLocation;

	/**
	 * Velocity after last PerformMovement or SimulateMovement update. Used internally to detect changes in velocity from external sources.
	 */
	UPROPERTY()
	FVector LastUpdateVelocity;

	/** Timestamp when location or rotation last changed during an update. Only valid on the server. */
	UPROPERTY(Transient)
	float ServerLastTransformUpdateTimeStamp;

	/** Timestamp of last client adjustment sent. See NetworkMinTimeBetweenClientAdjustments. */
	UPROPERTY(Transient)
	float ServerLastClientGoodMoveAckTime;

	/** Timestamp of last client adjustment sent. See NetworkMinTimeBetweenClientAdjustments. */
	UPROPERTY(Transient)
	float ServerLastClientAdjustmentTime;

	/** Accumulated impulse to be added next tick. */
	UPROPERTY()
	FVector PendingImpulseToApply;

	/** Accumulated force to be added next tick. */
	UPROPERTY()
	FVector PendingForceToApply;

	/**
	 * Modifier to applied to values such as acceleration and max speed due to analog input.
	 */
	UPROPERTY()
	float AnalogInputModifier;

	/** Computes the analog input modifier based on current input vector and/or acceleration. */
	ENGINE_API virtual float ComputeAnalogInputModifier() const;

	/** Used for throttling "stuck in geometry" logging. */
	float LastStuckWarningTime;

	/** Used when throttling "stuck in geometry" logging, to output the number of events we skipped if throttling. */
	uint32 StuckWarningCountSinceNotify;

	/**
	 * Used to limit number of jump apex attempts per tick.
	 * @see MaxJumpApexAttemptsPerSimulation
	 */
	int32 NumJumpApexAttempts;

public:

	/** Returns the location at the end of the last tick. */
	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	FVector GetLastUpdateLocation() const { return LastUpdateLocation; }

	/** Returns the rotation at the end of the last tick. */
	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	FRotator GetLastUpdateRotation() const { return LastUpdateRotation.Rotator(); }

	/** Returns the rotation Quat at the end of the last tick. */
	FQuat GetLastUpdateQuat() const { return LastUpdateRotation; }

	/** Returns the velocity at the end of the last tick. */
	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	FVector GetLastUpdateVelocity() const { return LastUpdateVelocity; }

	/** Get the value of ServerLastTransformUpdateTimeStamp. */
	inline float GetServerLastTransformUpdateTimeStamp() const { return ServerLastTransformUpdateTimeStamp;  }

	/**
	 * Compute remaining time step given remaining time and current iterations.
	 * The last iteration (limited by MaxSimulationIterations) always returns the remaining time, which may violate MaxSimulationTimeStep.
	 *
	 * @param RemainingTime		Remaining time in the tick.
	 * @param Iterations		Current iteration of the tick (starting at 1).
	 * @return The remaining time step to use for the next sub-step of iteration.
	 * @see MaxSimulationTimeStep, MaxSimulationIterations
	 */
	ENGINE_API float GetSimulationTimeStep(float RemainingTime, int32 Iterations) const;

	/**
	 * Max time delta for each discrete simulation step.
	 * Used primarily in the the more advanced movement modes that break up larger time steps (usually those applying gravity such as falling and walking).
	 * Lowering this value can address issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationIterations
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0.0166", ClampMax="0.50", UIMin="0.0166", UIMax="0.50"))
	float MaxSimulationTimeStep;

	/**
	 * Max number of iterations used for each discrete simulation step.
	 * Used primarily in the the more advanced movement modes that break up larger time steps (usually those applying gravity such as falling and walking).
	 * Increasing this value can address issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationTimeStep
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="1", ClampMax="25", UIMin="1", UIMax="25"))
	int32 MaxSimulationIterations;

	/**
	 * Max number of attempts per simulation to attempt to exactly reach the jump apex when falling movement reaches the top of the arc.
	 * Limiting this prevents deep recursion when special cases cause collision or other conditions which reactivate the apex condition.
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="1", ClampMax="4", UIMin="1", UIMax="4"))
	int32 MaxJumpApexAttemptsPerSimulation;

	/**
	* Max distance we allow simulated proxies to depenetrate when moving out of anything but Pawns.
	* This is generally more tolerant than with Pawns, because other geometry is either not moving, or is moving predictably with a bit of delay compared to on the server.
	* @see MaxDepenetrationWithGeometryAsProxy, MaxDepenetrationWithPawn, MaxDepenetrationWithPawnAsProxy
	*/
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float MaxDepenetrationWithGeometry;

	/**
	* Max distance we allow simulated proxies to depenetrate when moving out of anything but Pawns.
	* This is generally more tolerant than with Pawns, because other geometry is either not moving, or is moving predictably with a bit of delay compared to on the server.
	* @see MaxDepenetrationWithGeometry, MaxDepenetrationWithPawn, MaxDepenetrationWithPawnAsProxy
	*/
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float MaxDepenetrationWithGeometryAsProxy;

	/**
	* Max distance we are allowed to depenetrate when moving out of other Pawns.
	* @see MaxDepenetrationWithGeometry, MaxDepenetrationWithGeometryAsProxy, MaxDepenetrationWithPawnAsProxy
	*/
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float MaxDepenetrationWithPawn;

	/**
	 * Max distance we allow simulated proxies to depenetrate when moving out of other Pawns.
	 * Typically we don't want a large value, because we receive a server authoritative position that we should not then ignore by pushing them out of the local player.
	 * @see MaxDepenetrationWithGeometry, MaxDepenetrationWithGeometryAsProxy, MaxDepenetrationWithPawn
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ForceUnits=cm))
	float MaxDepenetrationWithPawnAsProxy;

	/**
	 * How long to take to smoothly interpolate from the old pawn position on the client to the corrected one sent by the server. Not used by Linear smoothing.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", ClampMax="1.0", UIMin="0.0", UIMax="1.0", ForceUnits=s))
	float NetworkSimulatedSmoothLocationTime;

	/**
	 * How long to take to smoothly interpolate from the old pawn rotation on the client to the corrected one sent by the server. Not used by Linear smoothing.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", ClampMax="1.0", UIMin="0.0", UIMax="1.0", ForceUnits=s))
	float NetworkSimulatedSmoothRotationTime;

	/**
	* Similar setting as NetworkSimulatedSmoothLocationTime but only used on Listen servers.
	*/
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", ClampMax="1.0", UIMin="0.0", UIMax="1.0", ForceUnits=s))
	float ListenServerNetworkSimulatedSmoothLocationTime;

	/**
	* Similar setting as NetworkSimulatedSmoothRotationTime but only used on Listen servers.
	*/
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", ClampMax="1.0", UIMin="0.0", UIMax="1.0", ForceUnits=s))
	float ListenServerNetworkSimulatedSmoothRotationTime;

	/**
	 * Shrink simulated proxy capsule radius by this amount, to account for network rounding that may cause encroachment. Changing during gameplay is not supported.
	 * @see AdjustProxyCapsuleSize()
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=cm))
	float NetProxyShrinkRadius;

	/**
	 * Shrink simulated proxy capsule half height by this amount, to account for network rounding that may cause encroachment. Changing during gameplay is not supported.
	 * @see AdjustProxyCapsuleSize()
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, AdvancedDisplay, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=cm))
	float NetProxyShrinkHalfHeight;

	/** Maximum distance character is allowed to lag behind server location when interpolating between updates. */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=cm))
	float NetworkMaxSmoothUpdateDistance;

	/**
	 * Maximum distance beyond which character is teleported to the new server location without any smoothing.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=cm))
	float NetworkNoSmoothUpdateDistance;

	/**
	 * Minimum time on the server between acknowledging good client moves. This can save on bandwidth. Set to 0 to disable throttling.
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=s))
	float NetworkMinTimeBetweenClientAckGoodMoves;

	/**
	 * Minimum time on the server between sending client adjustments when client has exceeded allowable position error.
	 * Should be >= NetworkMinTimeBetweenClientAdjustmentsLargeCorrection (the larger value is used regardless).
  	 * This can save on bandwidth. Set to 0 to disable throttling.
	 * @see ServerLastClientAdjustmentTime
	 */
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=s))
	float NetworkMinTimeBetweenClientAdjustments;

	/**
	* Minimum time on the server between sending client adjustments when client has exceeded allowable position error by a large amount (NetworkLargeClientCorrectionDistance).
	* Should be <= NetworkMinTimeBetweenClientAdjustments (the smaller value is used regardless).
	* @see NetworkMinTimeBetweenClientAdjustments
	*/
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=s))
	float NetworkMinTimeBetweenClientAdjustmentsLargeCorrection;

	/**
	* If client error is larger than this, sets bNetworkLargeClientCorrection to reduce delay between client adjustments.
	* @see NetworkMinTimeBetweenClientAdjustments, NetworkMinTimeBetweenClientAdjustmentsLargeCorrection
	*/
	UPROPERTY(Category="Character Movement (Networking)", EditDefaultsOnly, meta=(ClampMin="0.0", UIMin="0.0", ForceUnits=cm))
	float NetworkLargeClientCorrectionDistance;

	/** Used in determining if pawn is going off ledge.  If the ledge is "shorter" than this value then the pawn will be able to walk off it. **/
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(ForceUnits=cm))
	float LedgeCheckThreshold;

	/** When exiting water, jump if control pitch angle is this high or above. */
	UPROPERTY(Category="Character Movement: Swimming", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	float JumpOutOfWaterPitch;

	/** Information about the floor the Character is standing on (updated only during walking movement). */
	UPROPERTY(Category="Character Movement: Walking", VisibleInstanceOnly, BlueprintReadOnly)
	FFindFloorResult CurrentFloor;

	/**
	 * Default movement mode when not in water. Used at player startup or when teleported.
	 * @see DefaultWaterMovementMode
	 * @see bRunPhysicsWithNoController
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite)
	TEnumAsByte<enum EMovementMode> DefaultLandMovementMode;

	/**
	 * Default movement mode when in water. Used at player startup or when teleported.
	 * @see DefaultLandMovementMode
	 * @see bRunPhysicsWithNoController
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite)
	TEnumAsByte<enum EMovementMode> DefaultWaterMovementMode;

	/** Accessor to the Last Server Movement Base, only relevant during server update*/
	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated (GetLastServerMovementBase included). Use GetLastServerMovementBaseInterfaceData instead.")
	UPrimitiveComponent* GetLastServerMovementBase() const { return Cast<UPrimitiveComponent>(LastMovementBaseInterfaceData.PhysicsObjectOwner.Get()); }

	/** Accessor to the Last Server Movement Base Interface Data, only relevant during server update*/
	const FMovementBaseInterfaceData* GetLastServerMovementBaseInterfaceData() const { return &LastMovementBaseInterfaceData; }

private:
	/**
	 * Ground movement mode to switch to after falling and resuming ground movement.
	 * Only allowed values are: MOVE_Walking, MOVE_NavWalking.
	 * @see SetGroundMovementMode(), GetGroundMovementMode()
	 */
	UPROPERTY(Transient)
	TEnumAsByte<enum EMovementMode> GroundMovementMode;

	/** Remember last server movement base so we can detect mounts/dismounts and respond accordingly. */
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use LastMovementBaseInterfaceData instead.")
	TWeakObjectPtr<UPrimitiveComponent> LastServerMovementBase = nullptr;
	
	/** Remember last server movement base interface data so we can detect mounts/dismounts and respond accordingly. */
	FMovementBaseInterfaceData LastMovementBaseInterfaceData;

public:
	/**
	 * If true, walking movement always maintains horizontal velocity when moving up ramps, which causes movement up ramps to be faster parallel to the ramp surface.
	 * If false, then walking movement maintains velocity magnitude parallel to the ramp surface.
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite)
	uint8 bMaintainHorizontalGroundVelocity:1;

	/** If true, impart the base actor's X velocity when falling off it (which includes jumping) */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite)
	uint8 bImpartBaseVelocityX:1;

	/** If true, impart the base actor's Y velocity when falling off it (which includes jumping) */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite)
	uint8 bImpartBaseVelocityY:1;

	/** If true, impart the base actor's Z velocity when falling off it (which includes jumping) */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite)
	uint8 bImpartBaseVelocityZ:1;

	/**
	 * If true, impart the base component's tangential components of angular velocity when jumping or falling off it.
	 * Only those components of the velocity allowed by the separate component settings (bImpartBaseVelocityX etc) will be applied.
	 * @see bImpartBaseVelocityX, bImpartBaseVelocityY, bImpartBaseVelocityZ
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite)
	uint8 bImpartBaseAngularVelocity:1;

	/** Used by movement code to determine if a change in position is based on normal movement or a teleport. If not a teleport, velocity can be recomputed based on the change in position. */
	UPROPERTY(Category="Character Movement (General Settings)", Transient, VisibleInstanceOnly, BlueprintReadWrite)
	uint8 bJustTeleported:1;

	/** True when a network replication update is received for simulated proxies. */
	UPROPERTY(Transient)
	uint8 bNetworkUpdateReceived:1;

	/** True when the networked movement mode has been replicated. */
	UPROPERTY(Transient)
	uint8 bNetworkMovementModeChanged:1;

	/** True when the networked gravity direction has been replicated. */
	UPROPERTY(Transient)
	uint8 bNetworkGravityDirectionChanged:1;

	/**
	 * If true, we should ignore server location difference checks for client error on this movement component.
	 * This can be useful when character is moving at extreme speeds for a duration and you need it to look
	 * smooth on clients without the server correcting the client. Make sure to disable when done, as this would
	 * break this character's server-client movement correction.
	 * Note: This only prevents the server from generating corrections. To also prevent the client from applying
	 * in-flight corrections, use bClientIgnoreMovementCorrections alongside this flag.
	 * @see bClientIgnoreMovementCorrections, bServerAcceptClientAuthoritativePosition, ServerCheckClientError()
	 */
	UPROPERTY(Category="Character Movement", EditAnywhere, BlueprintReadWrite)
	uint8 bIgnoreClientMovementErrorChecksAndCorrection:1;

	/**
	 * When true, the client will not apply server movement corrections (skips ClientAdjustPosition).
	 * Typically used together with bIgnoreClientMovementErrorChecksAndCorrection, which prevents the
	 * server from generating corrections in the first place.
	 */
	UPROPERTY(Category="Character Movement", EditAnywhere, BlueprintReadWrite)
	uint8 bClientIgnoreMovementCorrections:1;

	/**
	 * If true, and server does not detect client position error, server will copy the client movement location/velocity/etc after simulating the move.
	 * This can be useful for short bursts of movement that are difficult to sync over the network.
	 * Note that if bIgnoreClientMovementErrorChecksAndCorrection is used, this means the server will not detect an error.
	 * Also see GameNetworkManager->ClientAuthorativePosition which permanently enables this behavior.
	 * @see bIgnoreClientMovementErrorChecksAndCorrection, ServerShouldUseAuthoritativePosition()
	 */
	UPROPERTY(Category="Character Movement", EditAnywhere, BlueprintReadWrite)
	uint8 bServerAcceptClientAuthoritativePosition : 1;

	/**
	 * If true, event NotifyJumpApex() to CharacterOwner's controller when at apex of jump. Is cleared when event is triggered.
	 * By default this is off, and if you want the event to fire you typically set it to true when movement mode changes to "Falling" from another mode (see OnMovementModeChanged).
	 */
	UPROPERTY(Category="Character Movement: Jumping / Falling", VisibleAnywhere, BlueprintReadWrite)
	uint8 bNotifyApex:1;

	/** Instantly stop when in flying mode and no acceleration is being applied. */
	UPROPERTY()
	uint8 bCheatFlying:1;

	/** If true, try to crouch (or keep crouching) on next update. If false, try to stop crouching on next update. */
	UPROPERTY(Category="Character Movement (General Settings)", VisibleInstanceOnly, BlueprintReadOnly)
	uint8 bWantsToCrouch:1;

	/**
	 * If true, crouching should keep the base of the capsule in place by lowering the center of the shrunken capsule. If false, the base of the capsule moves up and the center stays in place.
	 * The same behavior applies when the character uncrouches: if true, the base is kept in the same location and the center moves up. If false, the capsule grows and only moves up if the base impacts something.
	 * By default this variable is set when the movement mode changes: set to true when walking and false otherwise. Feel free to override the behavior when the movement mode changes.
	 */
	UPROPERTY(Category="Character Movement (General Settings)", VisibleInstanceOnly, BlueprintReadWrite, AdvancedDisplay)
	uint8 bCrouchMaintainsBaseLocation:1;

	/**
	 * Whether the character ignores changes in rotation of the base it is standing on.
	 * If true, the character maintains current world rotation.
	 * If false, the character rotates with the moving base.
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite)
	uint8 bIgnoreBaseRotation:1;

	/** 
	 * Set this to true if riding on a moving base that you know is clear from non-moving world obstructions.
	 * Optimization to avoid sweeps during based movement, use with care.
	 */
	UPROPERTY()
	uint8 bFastAttachedMove:1;

	/**
	 * Whether we always force floor checks for stationary Characters while walking.
	 * Normally floor checks are avoided if possible when not moving, but this can be used to force them if there are use-cases where they are being skipped erroneously
	 * (such as objects moving up into the character from below).
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bAlwaysCheckFloor:1;

	/**
	 * Performs floor checks as if the character is using a shape with a flat base.
	 * This avoids the situation where characters slowly lower off the side of a ledge (as their capsule 'balances' on the edge).
	 */
	UPROPERTY(Category="Character Movement: Walking", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bUseFlatBaseForFloorChecks:1;

	/** Used to prevent reentry of JumpOff() */
	UPROPERTY()
	uint8 bPerformingJumpOff:1;

	/** Used to safely leave NavWalking movement mode */
	UPROPERTY()
	uint8 bWantsToLeaveNavWalking:1;

	/** If set, component will use RVO avoidance. This only runs on the server. */
	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly)
	uint8 bUseRVOAvoidance:1;

	/**
	 * Should use acceleration for path following?
	 * If true, acceleration is applied when path following to reach the target velocity.
	 * If false, path following velocity is set directly, disregarding acceleration.
	 */
	UPROPERTY(Category="Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	uint8 bRequestedMoveUseAcceleration:1;

	/** Set on clients when server's movement mode is NavWalking */
	uint8 bIsNavWalkingOnServer : 1;

	/** True when SimulatedProxies are simulating RootMotion */
	UPROPERTY(Transient)
	uint8 bWasSimulatingRootMotion:1;

	UPROPERTY(Category = "RootMotion", EditAnywhere, BlueprintReadWrite)
	uint8 bAllowPhysicsRotationDuringAnimRootMotion : 1;

	/**
	 * When applying a root motion override while falling off a moving object, this controls how long it takes to lose half the former base's velocity (in seconds).
	 * Set to 0 to ignore former bases (default).
	 * Set to -1 for no decay.
	 * Any other positive value sets the half-life for exponential decay.
	 */
	UPROPERTY(Category = "RootMotion", EditAnywhere, BlueprintReadWrite)
	float FormerBaseVelocityDecayHalfLife = 0.f;

protected:

	// AI PATH FOLLOWING

	/** Was velocity requested by path following? */
	UPROPERTY(Transient)
	uint8 bHasRequestedVelocity:1;

	/** Was acceleration requested to be always max speed? */
	UPROPERTY(Transient)
	uint8 bRequestedMoveWithMaxSpeed:1;

	/** Was avoidance updated in this frame? */
	UPROPERTY(Transient)
	uint8 bWasAvoidanceUpdated : 1;

	/** if set, PostProcessAvoidanceVelocity will be called */
	uint8 bUseRVOPostProcess : 1;

	/** Flag set in pre-physics update to indicate that based movement should be updated post-physics */
	uint8 bDeferUpdateBasedMovement : 1;

	/** Whether to raycast to underlying geometry to better conform navmesh-walking characters */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadOnly)
	uint8 bProjectNavMeshWalking : 1 = false;

	/** Should we attempt to slide along a navmesh instead of projecting a point on the navmesh and moving to that */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	uint8 bSlideAlongNavMeshEdge : 1 = false;

	/** Use both WorldStatic and WorldDynamic channels for NavWalking geometry conforming */
	UPROPERTY(Category = "Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	uint8 bProjectNavMeshOnBothWorldChannels : 1;

	/** forced avoidance velocity, used when AvoidanceLockTimer is > 0 */
	FVector AvoidanceLockVelocity;

	/** remaining time of avoidance velocity lock */
	float AvoidanceLockTimer;

public:

	/** Returns if the character rotation should be corrected on clients when sending a server move response correction. */
	virtual bool ShouldCorrectRotation() const { return false; }

	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly, meta=(ForceUnits=cm))
	float AvoidanceConsiderationRadius;

	/**
	 * Velocity requested by path following.
	 * @see RequestDirectMove()
	 */
	UPROPERTY(Transient)
	FVector RequestedVelocity;

	/**
	 * Velocity requested by path following during last Update
	 * Updated when we consume the value
	 */
	UPROPERTY(Transient)
	FVector LastUpdateRequestedVelocity;

	/** Returns velocity requested by path following */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(Keywords="Velocity RequestedVelocity"))
	ENGINE_API FVector GetLastUpdateRequestedVelocity() const;

	/** No default value, for now it's assumed to be valid if GetAvoidanceManager() returns non-NULL. */
	UPROPERTY(Category="Character Movement: Avoidance", VisibleAnywhere, BlueprintReadOnly, AdvancedDisplay)
	int32 AvoidanceUID;

	/** Moving actor's group mask */
	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask AvoidanceGroup;

	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DeprecatedFunction, DeprecationMessage="Please use SetAvoidanceGroupMask function instead."))
	ENGINE_API void SetAvoidanceGroup(int32 GroupFlags);

	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	ENGINE_API void SetAvoidanceGroupMask(const FNavAvoidanceMask& GroupMask);

	/** Will avoid other agents if they are in one of specified groups */
	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask GroupsToAvoid;

	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta = (DeprecatedFunction, DeprecationMessage = "Please use SetGroupsToAvoidMask function instead."))
	ENGINE_API void SetGroupsToAvoid(int32 GroupFlags);

	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	ENGINE_API void SetGroupsToAvoidMask(const FNavAvoidanceMask& GroupMask);

	/** Will NOT avoid other agents if they are in one of specified groups, higher priority than GroupsToAvoid */
	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask GroupsToIgnore;

	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta = (DeprecatedFunction, DeprecationMessage = "Please use SetGroupsToIgnoreMask function instead."))
	ENGINE_API void SetGroupsToIgnore(int32 GroupFlags);

	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	ENGINE_API void SetGroupsToIgnoreMask(const FNavAvoidanceMask& GroupMask);

	/** De facto default value 0.5 (due to that being the default in the avoidance registration function), indicates RVO behavior. */
	UPROPERTY(Category="Character Movement: Avoidance", EditAnywhere, BlueprintReadOnly)
	float AvoidanceWeight;

	/** Temporarily holds launch velocity when pawn is to be launched so it happens at end of movement. */
	UPROPERTY()
	FVector PendingLaunchVelocity;

	/** last known location projected on navmesh, used by NavWalking mode */
	FNavLocation CachedNavLocation;

	/** Last valid projected hit result from raycast to geometry from navmesh */
	FHitResult CachedProjectedNavMeshHitResult;

	/** Remember last server movement base bone so we can detect mounts/dismounts and respond accordingly. */
	FName LastServerMovementBaseBoneName = NAME_None;

	/** Remember if the client was previously falling so we can tell when they've just landed. */
	bool bLastClientIsFalling = false;

	/** Remember if the server was previously falling so we can tell when they've just landed. */
	bool bLastServerIsFalling = false;

	/** Whether we were just walking on something, used to help with transitions off moving objects. */
	bool bLastServerIsWalking = false;

	/** True if the UpdatedComponent was moved outside of this CharacterMovementComponent since the last move -- its starting location for this update doesn't match its ending position for the previous update. */
	bool bTeleportedSinceLastUpdate = false;

	/** Whether we're stepping off a moving platform (and should trust the client somewhat when landing). */
	bool bCanTrustClientOnLanding = false;

	/** How loosely the client can follow the server location during this fall. */
	float MaxServerClientErrorWhileFalling = 0.f;

	/** Left over velocity when leaving a moving base. Helps with airborne root motion. */
	FVector DecayingFormerBaseVelocity = FVector::ZeroVector;

	/** How often we should raycast to project from navmesh to underlying geometry */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking"))
	float NavMeshProjectionInterval;

	UPROPERTY(Transient)
	float NavMeshProjectionTimer;

	/** Speed at which to interpolate agent navmesh offset between traces. 0: Instant (no interp) > 0: Interp speed") */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionInterpSpeed;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the upward direction.
	 * In other words, start the trace at [CapsuleHeight * NavMeshProjectionHeightScaleUp] above nav mesh.
	 */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionHeightScaleUp;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the downward direction.
	 * In other words, trace down to [CapsuleHeight * NavMeshProjectionHeightScaleDown] below nav mesh.
	 */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionHeightScaleDown;

	/** Ignore small differences in ground height between server and client data during NavWalking mode */
	UPROPERTY(Category="Character Movement: NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	float NavWalkingFloorDistTolerance;

	/** Change avoidance state and registers in RVO manager if needed */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta = (UnsafeDuringActorConstruction = "true"))
	ENGINE_API void SetAvoidanceEnabled(bool bEnable);

	/** Get the Character that owns UpdatedComponent. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ACharacter* GetCharacterOwner() const;

	/**
	 * Change movement mode.
	 *
	 * @param NewMovementMode	The new movement mode
	 * @param NewCustomMode		The new custom sub-mode, only applicable if NewMovementMode is Custom.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void SetMovementMode(EMovementMode NewMovementMode, uint8 NewCustomMode = 0);

	/**
	 * Set movement mode to use when returning to walking movement (either MOVE_Walking or MOVE_NavWalking).
	 * If movement mode is currently one of Walking or NavWalking, this will also change the current movement mode (via SetMovementMode())
	 * if the new mode is not the current ground mode.
	 * 
	 * @param  NewGroundMovementMode New ground movement mode. Must be either MOVE_Walking or MOVE_NavWalking, other values are ignored.
	 * @see GroundMovementMode
	 */
	 ENGINE_API void SetGroundMovementMode(EMovementMode NewGroundMovementMode);

	/**
	 * Get current GroundMovementMode value.
	 * @return current GroundMovementMode
	 * @see GroundMovementMode, SetGroundMovementMode()
	 */
	 EMovementMode GetGroundMovementMode() const { return GroundMovementMode; }

protected:

	/** Called after MovementMode has changed. Base implementation does special handling for starting certain modes, then notifies the CharacterOwner. */
	ENGINE_API virtual void OnMovementModeChanged(EMovementMode PreviousMovementMode, uint8 PreviousCustomMode);

public:

	ENGINE_API virtual uint8 PackNetworkMovementMode() const;
	ENGINE_API virtual void UnpackNetworkMovementMode(const uint8 ReceivedMode, TEnumAsByte<EMovementMode>& OutMode, uint8& OutCustomMode, TEnumAsByte<EMovementMode>& OutGroundMode) const;
	ENGINE_API virtual void ApplyNetworkMovementMode(const uint8 ReceivedMode);

	// Begin UObject Interface
	ENGINE_API virtual void Serialize(FArchive& Archive) override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// End UObject Interface

	//Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Activate(bool bReset = false) override;
	ENGINE_API virtual void Deactivate() override;
	ENGINE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//End UActorComponent Interface

	//BEGIN UMovementComponent Interface
	ENGINE_API virtual float GetMaxSpeed() const override;
	ENGINE_API virtual void StopActiveMovement() override;
	ENGINE_API virtual bool IsCrouching() const override;
	ENGINE_API virtual bool IsFalling() const override;
	ENGINE_API virtual bool IsMovingOnGround() const override;
	ENGINE_API virtual bool IsSwimming() const override;
	ENGINE_API virtual bool IsFlying() const override;
	ENGINE_API virtual float GetGravityZ() const override;
	ENGINE_API virtual void AddRadialForce(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff) override;
	ENGINE_API virtual void AddRadialImpulse(const FVector& Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange) override;
	//END UMovementComponent Interface

	/** Returns true if the character is in the 'Walking' movement mode. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	bool IsWalking() const;

	/**
	 * Returns true if currently performing a movement update.
	 * @see bMovementInProgress
	 */
	bool IsMovementInProgress() const { return bMovementInProgress; }

	//BEGIN UNavMovementComponent Interface
	ENGINE_API virtual FVector GetActorFeetLocation() const override;
	ENGINE_API virtual void RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed) override;
	ENGINE_API virtual void RequestPathMove(const FVector& MoveInput) override;
	ENGINE_API virtual bool CanStartPathFollowing() const override;
	ENGINE_API virtual bool CanStopPathFollowing() const override;
	ENGINE_API virtual float GetPathFollowingBrakingDistance(float MaxSpeed) const override;
	//END UNaVMovementComponent Interface

	//Begin UPawnMovementComponent Interface
	ENGINE_API virtual void NotifyBumpedPawn(APawn* BumpedPawn) override;
	//End UPawnMovementComponent Interface

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Make movement impossible (sets movement mode to MOVE_None). */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void DisableMovement();

	/** Return true if we have a valid CharacterOwner and UpdatedComponent. */
	ENGINE_API virtual bool HasValidData() const;

	/**
	 * If ShouldPerformAirControlForPathFollowing() returns true, it will update Velocity and Acceleration to air control in the desired Direction for character using path following.
	 * @param Direction is the desired direction of movement
	 * @param ZDiff is the height difference between the destination and the Pawn's current position
	 * @see RequestDirectMove()
	*/
	ENGINE_API virtual void PerformAirControlForPathFollowing(FVector Direction, float ZDiff);

	/**
	 * Whether Character should perform air control via PerformAirControlForPathFollowing when falling and following a path at the same time
	 * Default implementation always returns true during MOVE_Falling.
	 */
	ENGINE_API virtual bool ShouldPerformAirControlForPathFollowing() const;

	/** Transition from walking to falling */
	ENGINE_API virtual void StartFalling(int32 Iterations, float remainingTime, float timeTick, const FVector& Delta, const FVector& subLoc);

	/**
	 * Whether Character should go into falling mode when walking and changing position, based on an old and new floor result (both of which are considered walkable).
	 * Default implementation always returns false.
	 * @return true if Character should start falling
	 */
	ENGINE_API virtual bool ShouldCatchAir(const FFindFloorResult& OldFloor, const FFindFloorResult& NewFloor);

	/**
	 * Trigger OnWalkingOffLedge event on CharacterOwner.
	 */
	ENGINE_API virtual void HandleWalkingOffLedge(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta);

	/** Adjust distance from floor, trying to maintain a slight offset from the floor when walking (based on CurrentFloor). */
	ENGINE_API virtual void AdjustFloorHeight();

	/** Return PrimitiveComponent we are based on (standing and walking on). */
	UE_DEPRECATED(5.8, "GetMovementBase has been deprecated. Use GetMovementBaseObject cast if necessary instead.")
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DeprecatedFunction="Note", DeprecationMessage="GetMovementBase has been deprecated. Use GetMovementBaseObject cast if necessary instead."))
	ENGINE_API UPrimitiveComponent* GetMovementBase() const;

	/** Return Uobject we are based on (standing and walking on). */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API UObject* GetMovementBaseObject() const;
	
	/** Returns struct containing references and functionality for UObject we're using as a MovementBase. Uses CharacterOwner to get relevant movement base data */
	ENGINE_API const FMovementBaseInterfaceData* GetMovementBaseInterfaceData() const;
	
	/** Returns mutable struct containing references and functionality for UObject we're using as a MovementBase. Uses CharacterOwner to get relevant movement base data */
	ENGINE_API FMovementBaseInterfaceData* GetMovementBaseInterfaceData_Mutable();
	
	/** Update or defer updating of position based on Base movement */
	ENGINE_API virtual void MaybeUpdateBasedMovement(float DeltaSeconds);

	/** Update position based on Base movement */
	ENGINE_API virtual void UpdateBasedMovement(float DeltaSeconds);

	/** Update controller's view rotation as pawn's base rotates */
	ENGINE_API virtual void UpdateBasedRotation(FRotator& FinalRotation, const FRotator& ReducedRotation);

	/** Call SaveBaseLocation() if not deferring updates (bDeferUpdateBasedMovement is false). */
	ENGINE_API virtual void MaybeSaveBaseLocation();

	/** Update OldBaseLocation and OldBaseQuat if there is a valid movement base, and store the relative location/rotation if necessary. Ignores bDeferUpdateBasedMovement and forces the update. */
	ENGINE_API virtual void SaveBaseLocation();

	/** Apply inherited velocity when leaving base, for example from jumping off it */
	ENGINE_API virtual void ApplyImpartedMovementBaseVelocity();

	/** Property to set if UpdateBasedMovement should ignore collision with actors part of the current MovementBase, if the base is simulated by physics */
	UPROPERTY(Category = "Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite)
	bool bBasedMovementIgnorePhysicsBase = false;

	/** Property to set if characters should stay based on objects attachment root instead of the traced object */
	UPROPERTY(Category = "Character Movement (General Settings)", EditAnywhere, BlueprintReadWrite)
	bool bBaseOnAttachmentRoot = false;

	/** Property to set if characters should stay based on objects while jumping */
	UPROPERTY(Category = "Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite)
	bool bStayBasedInAir = false;

	/** Property used to set how high above base characters should stay based on objects while jumping if bStayBasedInAir is set */
	UPROPERTY(Category = "Character Movement: Jumping / Falling", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bStayBasedInAir"))
	float StayBasedInAirHeight = 1000.0f;


	/** changes physics based on MovementMode */
	ENGINE_API virtual void StartNewPhysics(float deltaTime, int32 Iterations);
	
	/**
	 * NOTE: THIS FUNCTION IS DEPRECATED, PLEASE CALL DoJump(bool bReplayingMoves, float DeltaTime) ... 
	 */
	UE_DEPRECATED_FORGAME(5.5, "This function has been deprecated. Please call DoJump(bool bReplayingMoves, float DeltaTime)")
	ENGINE_API virtual bool DoJump(bool bReplayingMoves);

	/**
	 * Perform jump. Called by Character when a jump has been detected because Character->bPressedJump was true. Checks Character->CanJump().
	 * Note that you should usually trigger a jump through Character::Jump() instead.
	 * @param	bReplayingMoves: true if this is being done as part of replaying moves on a locally controlled client after a server correction.
	 * @param	DeltaTime: time slice for this jump move
	 * @return	True if the jump was triggered successfully.
	 */
	ENGINE_API virtual bool DoJump(bool bReplayingMoves, float DeltaTime);

	/**
	 * Returns true if current movement state allows an attempt at jumping. Used by Character::CanJump().
	 */
	ENGINE_API virtual bool CanAttemptJump() const;

	/** Queue a pending launch with velocity LaunchVel. */
	ENGINE_API virtual void Launch(FVector const& LaunchVel);

	/** Handle a pending launch during an update. Returns true if the launch was triggered. */
	ENGINE_API virtual bool HandlePendingLaunch();

	/**
	 * If we have a movement base, get the velocity that should be imparted by that base, usually when jumping off of it.
	 * Only applies the components of the velocity enabled by bImpartBaseVelocityX, bImpartBaseVelocityY, bImpartBaseVelocityZ.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual FVector GetImpartedMovementBaseVelocity() const;

	/** Force this pawn to bounce off its current base, which isn't an acceptable base for it. */
	ENGINE_API virtual void JumpOff(AActor* MovementBaseActor);

	/** Can be overridden to choose to jump based on character velocity, base actor dimensions, etc. */
	ENGINE_API virtual FVector GetBestDirectionOffActor(AActor* BaseActor) const; // Calculates the best direction to go to "jump off" an actor.

	/** 
	 * Determine whether the Character should jump when exiting water.
	 * @param	JumpDir is the desired direction to jump out of water
	 * @return	true if Pawn should jump out of water
	 */
	ENGINE_API virtual bool ShouldJumpOutOfWater(FVector& JumpDir);

	/** Jump onto shore from water */
	ENGINE_API virtual void JumpOutOfWater(FVector WallNormal);

	/** Returns how far to rotate character during the time interval DeltaTime. */
	ENGINE_API virtual FRotator GetDeltaRotation(float DeltaTime) const;

	/**
	  * Compute a target rotation based on current movement. Used by PhysicsRotation() when bOrientRotationToMovement is true.
	  * Default implementation targets a rotation based on Acceleration.
	  *
	  * @param CurrentRotation	- Current rotation of the Character
	  * @param DeltaTime		- Time slice for this movement
	  * @param DeltaRotation	- Proposed rotation change based simply on DeltaTime * RotationRate
	  *
	  * @return The target rotation given current movement.
	  */
	ENGINE_API virtual FRotator ComputeOrientToMovementRotation(const FRotator& CurrentRotation, float DeltaTime, FRotator& DeltaRotation) const;

	/**
	 * Use velocity requested by path following to compute a requested acceleration and speed.
	 * This does not affect the Acceleration member variable, as that is used to indicate input acceleration.
	 * This may directly affect current Velocity.
	 *
	 * @param DeltaTime				Time slice for this operation
	 * @param MaxAccel				Max acceleration allowed in OutAcceleration result.
	 * @param MaxSpeed				Max speed allowed when computing OutRequestedSpeed.
	 * @param Friction				Current friction.
	 * @param BrakingDeceleration	Current braking deceleration.
	 * @param OutAcceleration		Acceleration computed based on requested velocity.
	 * @param OutRequestedSpeed		Speed of resulting velocity request, which can affect the max speed allowed by movement.
	 * @return Whether there is a requested velocity and acceleration, resulting in valid OutAcceleration and OutRequestedSpeed values.
	 */
	ENGINE_API virtual bool ApplyRequestedMove(float DeltaTime, float MaxAccel, float MaxSpeed, float Friction, float BrakingDeceleration, FVector& OutAcceleration, float& OutRequestedSpeed);

	/** Called if bNotifyApex is true and character has just passed the apex of its jump. */
	ENGINE_API virtual void NotifyJumpApex();

	/**
	 * Compute new falling velocity from given velocity and gravity. Applies the limits of the current Physics Volume's TerminalVelocity.
	 */
	ENGINE_API virtual FVector NewFallVelocity(const FVector& InitialVelocity, const FVector& Gravity, float DeltaTime) const;

	/* Determine how deep in water the character is immersed.
	 * @return float in range 0.0 = not in water, 1.0 = fully immersed
	 */
	ENGINE_API virtual float ImmersionDepth() const;

	/** 
	 * Updates Velocity and Acceleration based on the current state, applying the effects of friction and acceleration or deceleration. Does not apply gravity.
	 * This is used internally during movement updates. Normally you don't need to call this from outside code, but you might want to use it for custom movement modes.
	 *
	 * @param	DeltaTime						time elapsed since last frame.
	 * @param	Friction						coefficient of friction when not accelerating, or in the direction opposite acceleration.
	 * @param	bFluid							true if moving through a fluid, causing Friction to always be applied regardless of acceleration.
	 * @param	BrakingDeceleration				deceleration applied when not accelerating, or when exceeding max velocity.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void CalcVelocity(float DeltaTime, float Friction, bool bFluid, float BrakingDeceleration);
	
	/**
	 *	Compute the max jump height based on the JumpZVelocity velocity and gravity.
	 *	This does not take into account the CharacterOwner's MaxJumpHoldTime.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual float GetMaxJumpHeight() const;

	/**
	 *	Compute the max jump height based on the JumpZVelocity velocity and gravity.
	 *	This does take into account the CharacterOwner's MaxJumpHoldTime.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual float GetMaxJumpHeightWithJumpTime() const;

	/** Returns maximum acceleration for the current state. */
	UFUNCTION(BlueprintCallable, Category = "Pawn|Components|CharacterMovement")
	ENGINE_API virtual float GetMinAnalogSpeed() const;
	
	/** Returns maximum acceleration for the current state. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual float GetMaxAcceleration() const;

	/** Returns maximum deceleration for the current state when braking (ie when there is no acceleration). */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual float GetMaxBrakingDeceleration() const;

	/** Returns current acceleration, computed from input vector each update. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(Keywords="Acceleration GetAcceleration"))
	ENGINE_API FVector GetCurrentAcceleration() const;

	/** Returns modifier [0..1] based on the magnitude of the last input vector, which is used to modify the acceleration and max speed during movement. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API float GetAnalogInputModifier() const;
	
	/** Returns true if we can step up on the actor in the given FHitResult. */
	ENGINE_API virtual bool CanStepUp(const FHitResult& Hit) const;

	/** 
	 * Move up steps or slope. Does nothing and returns false if CanStepUp(Hit) returns false.
	 *
	 * @param GravDir			Gravity vector direction (assumed normalized or zero)
	 * @param Delta				Requested move
	 * @param Hit				[In] The hit before the step up.
	 * @param OutStepDownResult	[Out] If non-null, a floor check will be performed if possible as part of the final step down, and it will be updated to reflect this result.
	 * @return true if the step up was successful.
	 */
	ENGINE_API virtual bool StepUp(const FVector& GravDir, const FVector& Delta, const FHitResult &Hit, FStepDownResult* OutStepDownResult = NULL);

	/** Update the base of the character, which is the PrimitiveComponent we are standing on. */
	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void SetBase(UPrimitiveComponent* NewBase, const FName BoneName = NAME_None, bool bNotifyActor=true);
	
	// TODO NS: Should this be a pointer?
	/** Update the base of the character, which is the Object we are standing on. */
	ENGINE_API virtual void SetBase(FMovementBaseInterfaceData* MovementBaseInterfaceData, const FName BoneName = NAME_None, bool bNotifyActor=true);

	/**
	 * Update the base of the character, using the given floor result if it is walkable, or null if not. Calls SetBase().
	 */
	ENGINE_API void SetBaseFromFloor(const FFindFloorResult& FloorResult);

	/**
	 * Applies downward force when walking on top of physics objects.
	 * @param DeltaSeconds Time elapsed since last frame.
	 */
	ENGINE_API virtual void ApplyDownwardForce(float DeltaSeconds);

	/** Applies repulsion force to all touched components. */
	ENGINE_API virtual void ApplyRepulsionForce(float DeltaSeconds);
	
	/** Applies momentum accumulated through AddImpulse() and AddForce(), then clears those forces. Does *not* use ClearAccumulatedForces() since that would clear pending launch velocity as well. */
	ENGINE_API virtual void ApplyAccumulatedForces(float DeltaSeconds);

	/** Clears forces accumulated through AddImpulse() and AddForce(), and also pending launch velocity. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void ClearAccumulatedForces();

	/** Update the character state in PerformMovement right before doing the actual position change */
	ENGINE_API virtual void UpdateCharacterStateBeforeMovement(float DeltaSeconds);

	/** Update the character state in PerformMovement after the position change. Some rotation updates happen after this. */
	ENGINE_API virtual void UpdateCharacterStateAfterMovement(float DeltaSeconds);

	/** Give one final chance to update the velocity right before doing the actual position change. */
	virtual void UpdateVelocityBeforeMovement(float DeltaSeconds) {}

	/** 
	 * Handle start swimming functionality
	 * @param OldLocation - Location on last tick
	 * @param OldVelocity - velocity at last tick
	 * @param timeTick - time since at OldLocation
	 * @param remainingTime - DeltaTime to complete transition to swimming
	 * @param Iterations - physics iteration count
	 */
	ENGINE_API virtual void StartSwimming(FVector OldLocation, FVector OldVelocity, float timeTick, float remainingTime, int32 Iterations);

	/* Swimming uses gravity - but scaled by (1.f - buoyancy) */
	ENGINE_API float Swim(FVector Delta, FHitResult& Hit);

	/** Get as close to waterline as possible, staying on same side as currently. */
	ENGINE_API FVector FindWaterLine(FVector Start, FVector End);

	/** Handle falling movement. */
	ENGINE_API virtual void PhysFalling(float deltaTime, int32 Iterations);

	// Helpers for PhysFalling

	/**
	 * Get the lateral acceleration to use during falling movement. The Z component of the result is ignored.
	 * Default implementation returns current Acceleration value modified by GetAirControl(), with Z component removed,
	 * with magnitude clamped to GetMaxAcceleration().
	 * This function is used internally by PhysFalling().
	 *
	 * @param DeltaTime Time step for the current update.
	 * @return Acceleration to use during falling movement.
	 */
	ENGINE_API virtual FVector GetFallingLateralAcceleration(float DeltaTime);
	
	/**
	 * Returns true if falling movement should limit air control. Limiting air control prevents input acceleration during falling movement
	 * from allowing velocity to redirect forces upwards while falling, which could result in slower falling or even upward boosting.
	 *
	 * @see GetFallingLateralAcceleration(), BoostAirControl(), GetAirControl(), LimitAirControl()
	 */
	ENGINE_API virtual bool ShouldLimitAirControl(float DeltaTime, const FVector& FallAcceleration) const;

	/**
	 * Get the air control to use during falling movement.
	 * Given an initial air control (TickAirControl), applies the result of BoostAirControl().
	 * This function is used internally by GetFallingLateralAcceleration().
	 *
	 * @param DeltaTime			Time step for the current update.
	 * @param TickAirControl	Current air control value.
	 * @param FallAcceleration	Acceleration used during movement.
	 * @return Air control to use during falling movement.
	 * @see AirControl, BoostAirControl(), LimitAirControl(), GetFallingLateralAcceleration()
	 */
	ENGINE_API virtual FVector GetAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration);

	// Gravity Direction

	/**
	 * Set a custom, local gravity direction to use during movement simulation.
	 * The gravity direction must be synchronized by external systems between the autonomous
	 * and authority processes. The gravity direction will be corrected as part of movement
	 * corrections should the movement state diverge.
	 * SetGravityDirection is responsible for initializing cached values used to tranform to
	 * from gravity relative space.
	 * @param GravityDir		A non-zero vector representing the new gravity direction. The vector will be normalized.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void SetGravityDirection(const FVector& GravityDir);

	/** Whether the gravity direction is different from UCharacterMovementComponent::DefaultGravityDirection. */
	UFUNCTION(BlueprintPure, Category="Pawn|Components|CharacterMovement")
	bool HasCustomGravity() const { return bHasCustomGravity; }

	/** Returns the current gravity direction. */
	UFUNCTION(BlueprintPure, Category="Pawn|Components|CharacterMovement")
	FVector GetGravityDirection() const { return GravityDirection; }

	/** Returns a quaternion transforming from world to gravity space. */
	FQuat GetWorldToGravityTransform() const { return WorldToGravityTransform; }

	/** Returns a quaternion transforming from gravity to world space. */
	FQuat GetGravityToWorldTransform() const { return GravityToWorldTransform; }

	/** Rotate a vector from world to gravity space. */
	FVector RotateGravityToWorld(const FVector& World) const { return WorldToGravityTransform.RotateVector(World); }

	/** Rotate a vector from gravity to world space. */
	FVector RotateWorldToGravity(const FVector& Gravity) const { return GravityToWorldTransform.RotateVector(Gravity); }

	/** Project a vector onto the floor defined by the gravity direction. */
	FVector ProjectToGravityFloor(const FVector& Vector) const { return FVector::VectorPlaneProject(Vector, GetGravityDirection()); }

	/** Returns the size of a vector in the gravity-space vertical direction. */
	FVector::FReal GetGravitySpaceZ(const FVector& Vector) const { return Vector.Dot(-GetGravityDirection()); }

	/** Returns the component of the vector in the gravity-space vertical direction.  */
	FVector GetGravitySpaceComponentZ(const FVector& Vector) const { return Vector.Dot(GetGravityDirection()) * GetGravityDirection(); }

	/** Set the vertical component of the vector to the given value in the gravity-space vertical direction. */
	void SetGravitySpaceZ(FVector& Vector, const FVector::FReal Z) const { Vector = ProjectToGravityFloor(Vector) - Z * GetGravityDirection(); }
	
protected:

	/**
	 * Increase air control if conditions of AirControlBoostMultiplier and AirControlBoostVelocityThreshold are met.
	 * This function is used internally by GetAirControl().
	 *
	 * @param DeltaTime			Time step for the current update.
	 * @param TickAirControl	Current air control value.
	 * @param FallAcceleration	Acceleration used during movement.
	 * @return Modified air control to use during falling movement
	 * @see GetAirControl()
	 */
	ENGINE_API virtual float BoostAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration);

	/**
	 * Limits the air control to use during falling movement, given an impact while falling.
	 * This function is used internally by PhysFalling().
	 *
	 * @param DeltaTime			Time step for the current update.
	 * @param FallAcceleration	Acceleration used during movement.
	 * @param HitResult			Result of impact.
	 * @param bCheckForValidLandingSpot If true, will use IsValidLandingSpot() to determine if HitResult is a walkable surface. If false, this check is skipped.
	 * @return Modified air control acceleration to use during falling movement.
	 * @see PhysFalling()
	 */
	ENGINE_API virtual FVector LimitAirControl(float DeltaTime, const FVector& FallAcceleration, const FHitResult& HitResult, bool bCheckForValidLandingSpot);
	

	/** Handle landing against Hit surface over remaingTime and iterations, calling SetPostLandedPhysics() and starting the new movement mode. */
	ENGINE_API virtual void ProcessLanded(const FHitResult& Hit, float remainingTime, int32 Iterations);

	/** Use new physics after landing. Defaults to swimming if in water, walking otherwise. */
	ENGINE_API virtual void SetPostLandedPhysics(const FHitResult& Hit);

	/** Updates acceleration and perform movement, called from the TickComponent on the authoritative side for controlled characters, 
	 *	or on the client for characters without a controller when either playing root motion or bRunPhysicsWithNoController is true.
	 */
	ENGINE_API virtual void ControlledCharacterMove(const FVector& InputVector, float DeltaSeconds);

	/** Switch collision settings for NavWalking mode (ignore world collisions) */
	ENGINE_API virtual void SetNavWalkingPhysics(bool bEnable);

	/** Get Navigation data for the Character. Returns null if there is no associated nav data. */
	ENGINE_API const class INavigationDataInterface* GetNavData() const;

	/** 
	 * Checks to see if the current location is not encroaching blocking geometry so the character can leave NavWalking.
	 * Restores collision settings and adjusts character location to avoid getting stuck in geometry.
	 * If it's not possible, MovementMode change will be delayed until character reach collision free spot.
	 * @return True if movement mode was successfully changed
	 */
	ENGINE_API virtual bool TryToLeaveNavWalking();

	/** 
	 * Attempts to better align navmesh walking characters with underlying geometry (sometimes 
	 * navmesh can differ quite significantly from geometry).
	 * Updates CachedProjectedNavMeshHitResult, access this for more info about hits.
	 */
	ENGINE_API virtual FVector ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset);

	/** Performs trace for ProjectLocationFromNavMesh */
	ENGINE_API virtual void FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const;

	/** 
	 * When a character requests a velocity (like when following a path), this method returns true if when we should compute the 
	 * acceleration toward requested velocity (including friction). If it returns false, it will snap instantly to requested velocity.
	 */
	ENGINE_API virtual bool ShouldComputeAccelerationToReachRequestedVelocity(const float RequestedSpeed) const;
public:

	/** Called by owning Character upon successful teleport from AActor::TeleportTo(). */
	ENGINE_API virtual void OnTeleported() override;

	/**
	 * Checks if new capsule size fits (no encroachment), and call CharacterOwner->OnStartCrouch() if successful.
	 * In general you should set bWantsToCrouch instead to have the crouch persist during movement, or just use the crouch functions on the owning Character.
	 * @param	bClientSimulation	true when called when bIsCrouched is replicated to non owned clients, to update collision cylinder and offset.
	 */
	ENGINE_API virtual void Crouch(bool bClientSimulation = false);
	
	/**
	 * Checks if default capsule size fits (no encroachment), and trigger OnEndCrouch() on the owner if successful.
	 * @param	bClientSimulation	true when called when bIsCrouched is replicated to non owned clients, to update collision cylinder and offset.
	 */
	ENGINE_API virtual void UnCrouch(bool bClientSimulation = false);

	/** Returns true if the character is allowed to crouch in the current state. By default it is allowed when walking or falling, if CanEverCrouch() is true. */
	ENGINE_API virtual bool CanCrouchInCurrentState() const;

	/** Sets collision half-height when crouching and updates dependent computations */
	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetCrouchedHalfHeight(const float NewValue);

	/** Returns the collision half-height when crouching (component scale is applied separately) */
	UFUNCTION(BlueprintGetter)
	ENGINE_API float GetCrouchedHalfHeight() const;

	/** Returns true if there is a suitable floor SideStep from current position. */
	ENGINE_API virtual bool CheckLedgeDirection(const FVector& OldLocation, const FVector& SideStep, const FFindFloorResult& OldFloor) const;

	/** 
	 * @param Delta is the current move delta (which ended up going over a ledge).
	 * @return new delta which moves along the ledge
	 */
	ENGINE_API virtual FVector GetLedgeMove(const FVector& OldLocation, const FVector& Delta, const FFindFloorResult& OldFloor) const;

	/** Check if pawn is falling */
	ENGINE_API virtual bool CheckFall(const FFindFloorResult& OldFloor, const FHitResult& Hit, const FVector& Delta, const FVector& OldLocation, float remainingTime, float timeTick, int32 Iterations, bool bMustJump);
	
	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API void RevertMove(const FVector& OldLocation, UPrimitiveComponent* OldBase, const FVector& InOldBaseLocation, const FFindFloorResult& OldFloor, bool bFailMove);

	/** 
	 *  Revert to previous position OldLocation, return to being based on OldBase.
	 *  if bFailMove, stop movement and notify controller
	 */	
	ENGINE_API void RevertMove(const FVector& OldLocation, FMovementBaseInterfaceData* OldMovementBaseInterfaceData, const FVector& InOldBaseLocation, const FFindFloorResult& OldFloor, bool bFailMove);

	/** Perform rotation over deltaTime */
	ENGINE_API virtual void PhysicsRotation(float DeltaTime);

	/** if true, DesiredRotation will be restricted to only Yaw component in PhysicsRotation() */
	ENGINE_API virtual bool ShouldRemainVertical() const;

	/** Delegate when PhysicsVolume of UpdatedComponent has been changed **/
	ENGINE_API virtual void PhysicsVolumeChanged(class APhysicsVolume* NewVolume) override;

	/** Set movement mode to the default based on the current physics volume. */
	ENGINE_API virtual void SetDefaultMovementMode();

	/**
	 * Moves along the given movement direction using simple movement rules based on the current movement mode (usually used by simulated proxies).
	 *
	 * @param InVelocity:			Velocity of movement
	 * @param DeltaSeconds:			Time over which movement occurs
	 * @param OutStepDownResult:	[Out] If non-null, and a floor check is performed, this will be updated to reflect that result.
	 */
	ENGINE_API virtual void MoveSmooth(const FVector& InVelocity, const float DeltaSeconds, FStepDownResult* OutStepDownResult = NULL );

	/**
	 * Used during SimulateMovement for proxies, this computes a new value for Acceleration before running proxy simulation.
	 * The base implementation simply derives a value from the normalized Velocity value, which may help animations that want some indication of the direction of movement.
	 * Proxies don't implement predictive acceleration by default so this value is not used for the actual simulation.
	 */
	ENGINE_API virtual void UpdateProxyAcceleration();
	
	ENGINE_API virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent) override;
	
	/** Returns MovementMode string */
	ENGINE_API virtual FString GetMovementName() const;

	/** 
	 * Add impulse to character. Impulses are accumulated each tick and applied together
	 * so multiple calls to this function will accumulate.
	 * An impulse is an instantaneous force, usually applied once. If you want to continually apply
	 * forces each frame, use AddForce().
	 * Note that changing the momentum of characters like this can change the movement mode.
	 * 
	 * @param	Impulse				Impulse to apply.
	 * @param	bVelocityChange		Whether or not the impulse is relative to mass.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void AddImpulse( FVector Impulse, bool bVelocityChange = false );

	/** 
	 * Add force to character. Forces are accumulated each tick and applied together
	 * so multiple calls to this function will accumulate.
	 * Forces are scaled depending on timestep, so they can be applied each frame. If you want an
	 * instantaneous force, use AddImpulse.
	 * Adding a force always takes the actor's mass into account.
	 * Note that changing the momentum of characters like this can change the movement mode.
	 * 
	 * @param	Force			Force to apply.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual void AddForce( FVector Force );

	/**
	 * Draw important variables on canvas.  Character will call DisplayDebug() on the current ViewTarget when the ShowDebug exec is used
	 *
	 * @param Canvas - Canvas to draw on
	 * @param DebugDisplay - Contains information about what debug data to display
	 * @param YL - Height of the current font
	 * @param YPos - Y position on Canvas. YPos += YL, gives position to draw text for next debug line.
	 */
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);

	/**
	 * Draw in-world debug information for character movement (called with p.VisualizeMovement > 0).
	 */
	ENGINE_API virtual float VisualizeMovement() const;

	/** Check if swimming pawn just ran into edge of the pool and should jump out. */
	ENGINE_API virtual bool CheckWaterJump(FVector CheckPoint, FVector& WallNormal);

	/** Returns whether this pawn is currently allowed to walk off ledges */
	ENGINE_API virtual bool CanWalkOffLedges() const;

	/** Returns The distance from the edge of the capsule within which we don't allow the character to perch on the edge of a surface. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API float GetPerchRadiusThreshold() const;

	/**
	 * Returns the radius within which we can stand on the edge of a surface without falling (if this is a walkable surface).
	 * Simply computed as the capsule radius minus the result of GetPerchRadiusThreshold().
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API float GetValidPerchRadius() const;

	/** Return true if the hit result should be considered a walkable surface for the character. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API virtual bool IsWalkable(const FHitResult& Hit) const;

	/** Get the max angle in degrees of a walkable surface for the character. */
	inline float GetWalkableFloorAngle() const { return WalkableFloorAngle; }

	/** Get the max angle in degrees of a walkable surface for the character. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DisplayName = "GetWalkableFloorAngle", ScriptName = "GetWalkableFloorAngle"))
	ENGINE_API float K2_GetWalkableFloorAngle() const;

	/** Set the max angle in degrees of a walkable surface for the character. Also computes WalkableFloorZ. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API void SetWalkableFloorAngle(float InWalkableFloorAngle);

	/** Get the Z component of the normal of the steepest walkable surface for the character. Any lower than this and it is not walkable. */
	inline float GetWalkableFloorZ() const { return WalkableFloorZ; }

	/** Get the Z component of the normal of the steepest walkable surface for the character. Any lower than this and it is not walkable. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DisplayName = "GetWalkableFloorZ", ScriptName = "GetWalkableFloorZ"))
	ENGINE_API float K2_GetWalkableFloorZ() const;

	/** Set the Z component of the normal of the steepest walkable surface for the character. Also computes WalkableFloorAngle. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement")
	ENGINE_API void SetWalkableFloorZ(float InWalkableFloorZ);
	
	/** Pre-physics tick function for this character */
	struct FCharacterMovementComponentPrePhysicsTickFunction PrePhysicsTickFunction;

	/** Tick function called before physics */
	ENGINE_API virtual void PrePhysicsTickComponent(float DeltaTime, FCharacterMovementComponentPrePhysicsTickFunction& ThisTickFunction);

	/** Post-physics tick function for this character */
	UPROPERTY()
	struct FCharacterMovementComponentPostPhysicsTickFunction PostPhysicsTickFunction;

	/** Tick function called after physics (sync scene) has finished simulation, before cloth */
	ENGINE_API virtual void PostPhysicsTickComponent(float DeltaTime, FCharacterMovementComponentPostPhysicsTickFunction& ThisTickFunction);

protected:
	/** @note Movement update functions should only be called through StartNewPhysics()*/
	ENGINE_API virtual void PhysWalking(float deltaTime, int32 Iterations);

	/** @note Movement update functions should only be called through StartNewPhysics()*/
	ENGINE_API virtual void PhysNavWalking(float deltaTime, int32 Iterations);

	/** @note Movement update functions should only be called through StartNewPhysics()*/
	ENGINE_API virtual void PhysFlying(float deltaTime, int32 Iterations);

	/** @note Movement update functions should only be called through StartNewPhysics()*/
	ENGINE_API virtual void PhysSwimming(float deltaTime, int32 Iterations);

	/** @note Movement update functions should only be called through StartNewPhysics()*/
	ENGINE_API virtual void PhysCustom(float deltaTime, int32 Iterations);

	/* Allow custom handling when character hits a wall while swimming. */
	ENGINE_API virtual void HandleSwimmingWallHit(const FHitResult& Hit, float DeltaTime);

	/**
	 * Compute a vector of movement, given a delta and a hit result of the surface we are on.
	 *
	 * @param Delta:				Attempted movement direction
	 * @param RampHit:				Hit result of sweep that found the ramp below the capsule
	 * @param bHitFromLineTrace:	Whether the floor trace came from a line trace
	 *
	 * @return If on a walkable surface, this returns a vector that moves parallel to the surface. The magnitude may be scaled if bMaintainHorizontalGroundVelocity is true.
	 * If a ramp vector can't be computed, this will just return Delta.
	 */
	ENGINE_API virtual FVector ComputeGroundMovementDelta(const FVector& Delta, const FHitResult& RampHit, const bool bHitFromLineTrace) const;

	/**
	 * Move along the floor, using CurrentFloor and ComputeGroundMovementDelta() to get a movement direction.
	 * If a second walkable surface is hit, it will also be moved along using the same approach.
	 *
	 * @param InVelocity:			Velocity of movement
	 * @param DeltaSeconds:			Time over which movement occurs
	 * @param OutStepDownResult:	[Out] If non-null, and a floor check is performed, this will be updated to reflect that result.
	 */
	ENGINE_API virtual void MoveAlongFloor(const FVector& InVelocity, float DeltaSeconds, FStepDownResult* OutStepDownResult = NULL);

	/** Notification that the character is stuck in geometry.  Only called during walking movement. */
	ENGINE_API virtual void OnCharacterStuckInGeometry(const FHitResult* Hit);

	/**
	 * Adjusts velocity when walking so that Z velocity is zero.
	 * When bMaintainHorizontalGroundVelocity is false, also rescales the velocity vector to maintain the original magnitude, but in the horizontal direction.
	 */
	ENGINE_API virtual void MaintainHorizontalGroundVelocity();

	/** Overridden to enforce max distances based on hit geometry. */
	ENGINE_API virtual FVector GetPenetrationAdjustment(const FHitResult& Hit) const override;

	/** Overridden to set bJustTeleported to true, so we don't make incorrect velocity calculations based on adjusted movement. */
	ENGINE_API virtual bool ResolvePenetrationImpl(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation) override;

	/** Handle a blocking impact. Calls ApplyImpactPhysicsForces for the hit, if bEnablePhysicsInteraction is true. */
	ENGINE_API virtual void HandleImpact(const FHitResult& Hit, float TimeSlice=0.f, const FVector& MoveDelta = FVector::ZeroVector) override;

	/**
	 * Apply physics forces to the impacted component, if bEnablePhysicsInteraction is true.
	 * @param Impact				HitResult that resulted in the impact
	 * @param ImpactAcceleration	Acceleration of the character at the time of impact
	 * @param ImpactVelocity		Velocity of the character at the time of impact
	 */
	ENGINE_API virtual void ApplyImpactPhysicsForces(const FHitResult& Impact, const FVector& ImpactAcceleration, const FVector& ImpactVelocity);

	/** Custom version of SlideAlongSurface that handles different movement modes separately; namely during walking physics we might not want to slide up slopes. */
	ENGINE_API virtual float SlideAlongSurface(const FVector& Delta, float Time, const FVector& Normal, FHitResult& Hit, bool bHandleImpact) override;

	/** Custom version that allows upwards slides when walking if the surface is walkable. */
	ENGINE_API virtual void TwoWallAdjust(FVector& WorldSpaceDelta, const FHitResult& Hit, const FVector& OldHitNormal) const override;

	/**
	 * Calculate slide vector along a surface.
	 * Has special treatment when falling, to avoid boosting up slopes (calling HandleSlopeBoosting() in this case).
	 *
	 * @param Delta:	Attempted move.
	 * @param Time:		Amount of move to apply (between 0 and 1).
	 * @param Normal:	Normal opposed to movement. Not necessarily equal to Hit.Normal (but usually is).
	 * @param Hit:		HitResult of the move that resulted in the slide.
	 * @return			New deflected vector of movement.
	 */
	ENGINE_API virtual FVector ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const override;

	/** 
	 * Limit the slide vector when falling if the resulting slide might boost the character faster upwards.
	 * @param SlideResult:	Vector of movement for the slide (usually the result of ComputeSlideVector)
	 * @param Delta:		Original attempted move
	 * @param Time:			Amount of move to apply (between 0 and 1).
	 * @param Normal:		Normal opposed to movement. Not necessarily equal to Hit.Normal (but usually is).
	 * @param Hit:			HitResult of the move that resulted in the slide.
	 * @return:				New slide result.
	 */
	ENGINE_API virtual FVector HandleSlopeBoosting(const FVector& SlideResult, const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const;

	/** Slows towards stop. */
	ENGINE_API virtual void ApplyVelocityBraking(float DeltaTime, float Friction, float BrakingDeceleration);


public:

	/**
	 * Return true if the 2D distance to the impact point is inside the edge tolerance (CapsuleRadius minus a small rejection threshold).
	 * Useful for rejecting adjacent hits when finding a floor or landing spot.
	 */
	ENGINE_API virtual bool IsWithinEdgeTolerance(const FVector& CapsuleLocation, const FVector& TestImpactPoint, const float CapsuleRadius) const;

	/**
	 * Sweeps a vertical trace to find the floor for the capsule at the given location. Will attempt to perch if ShouldComputePerchResult() returns true for the downward sweep result.
	 * No floor will be found if collision is disabled on the capsule!
	 *
	 * @param CapsuleLocation		Location where the capsule sweep should originate
	 * @param OutFloorResult		[Out] Contains the result of the floor check. The HitResult will contain the valid sweep or line test upon success, or the result of the sweep upon failure.
	 * @param bCanUseCachedLocation If true, may use a cached value (can be used to avoid unnecessary floor tests, if for example the capsule was not moving since the last test).
	 * @param DownwardSweepResult	If non-null and it contains valid blocking hit info, this will be used as the result of a downward sweep test instead of doing it as part of the update.
	 */
	ENGINE_API virtual void FindFloor(const FVector& CapsuleLocation, FFindFloorResult& OutFloorResult, bool bCanUseCachedLocation, const FHitResult* DownwardSweepResult = NULL) const;

	/**
	* Sweeps a vertical trace to find the floor for the capsule at the given location. Will attempt to perch if ShouldComputePerchResult() returns true for the downward sweep result.
	* No floor will be found if collision is disabled on the capsule!
	*
	* @param CapsuleLocation		Location where the capsule sweep should originate
	* @param FloorResult			Result of the floor check
	*/
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DisplayName="FindFloor", ScriptName="FindFloor"))
	ENGINE_API virtual void K2_FindFloor(FVector CapsuleLocation, FFindFloorResult& FloorResult) const;

	/**
	 * Compute distance to the floor from bottom sphere of capsule and store the result in OutFloorResult.
	 * This distance is the swept distance of the capsule to the first point impacted by the lower hemisphere, or distance from the bottom of the capsule in the case of a line trace.
	 * This function does not care if collision is disabled on the capsule (unlike FindFloor).
	 * @see FindFloor
	 *
	 * @param CapsuleLocation:	Location of the capsule used for the query
	 * @param LineDistance:		If non-zero, max distance to test for a simple line check from the capsule base. Used only if the sweep test fails to find a walkable floor, and only returns a valid result if the impact normal is a walkable normal.
	 * @param SweepDistance:	If non-zero, max distance to use when sweeping a capsule downwards for the test. MUST be greater than or equal to the line distance.
	 * @param OutFloorResult:	Result of the floor check. The HitResult will contain the valid sweep or line test upon success, or the result of the sweep upon failure.
	 * @param SweepRadius:		The radius to use for sweep tests. Should be <= capsule radius.
	 * @param DownwardSweepResult:	If non-null and it contains valid blocking hit info, this will be used as the result of a downward sweep test instead of doing it as part of the update.
	 */
	ENGINE_API virtual void ComputeFloorDist(const FVector& CapsuleLocation, float LineDistance, float SweepDistance, FFindFloorResult& OutFloorResult, float SweepRadius, const FHitResult* DownwardSweepResult = NULL) const;

	/**
	* Compute distance to the floor from bottom sphere of capsule and store the result in FloorResult.
	* This distance is the swept distance of the capsule to the first point impacted by the lower hemisphere, or distance from the bottom of the capsule in the case of a line trace.
	* This function does not care if collision is disabled on the capsule (unlike FindFloor).
	*
	* @param CapsuleLocation		Location where the capsule sweep should originate
	* @param LineDistance			If non-zero, max distance to test for a simple line check from the capsule base. Used only if the sweep test fails to find a walkable floor, and only returns a valid result if the impact normal is a walkable normal.
	* @param SweepDistance			If non-zero, max distance to use when sweeping a capsule downwards for the test. MUST be greater than or equal to the line distance.
	* @param SweepRadius			The radius to use for sweep tests. Should be <= capsule radius.
	* @param FloorResult			Result of the floor check
	*/
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|CharacterMovement", meta=(DisplayName="ComputeFloorDistance", ScriptName="ComputeFloorDistance"))
	ENGINE_API virtual void K2_ComputeFloorDist(FVector CapsuleLocation, float LineDistance, float SweepDistance, float SweepRadius, FFindFloorResult& FloorResult) const;

	/**
	 * Sweep against the world and return the first blocking hit.
	 * Intended for tests against the floor, because it may change the result of impacts on the lower area of the test (especially if bUseFlatBaseForFloorChecks is true).
	 *
	 * @param OutHit			First blocking hit found.
	 * @param Start				Start location of the capsule.
	 * @param End				End location of the capsule.
	 * @param TraceChannel		The 'channel' that this trace is in, used to determine which components to hit.
	 * @param CollisionShape	Capsule collision shape.
	 * @param Params			Additional parameters used for the trace.
	 * @param ResponseParam		ResponseContainer to be used for this trace.
	 * @return True if OutHit contains a blocking hit entry.
	 */
	ENGINE_API virtual bool FloorSweepTest(
		struct FHitResult& OutHit,
		const FVector& Start,
		const FVector& End,
		ECollisionChannel TraceChannel,
		const struct FCollisionShape& CollisionShape,
		const struct FCollisionQueryParams& Params,
		const struct FCollisionResponseParams& ResponseParam
		) const;

	/** Verify that the supplied hit result is a valid landing spot when falling. */
	ENGINE_API virtual bool IsValidLandingSpot(const FVector& CapsuleLocation, const FHitResult& Hit) const;

	/**
	 * Determine whether we should try to find a valid landing spot after an impact with an invalid one (based on the Hit result).
	 * For example, landing on the lower portion of the capsule on the edge of geometry may be a walkable surface, but could have reported an unwalkable impact normal.
	 */
	ENGINE_API virtual bool ShouldCheckForValidLandingSpot(float DeltaTime, const FVector& Delta, const FHitResult& Hit) const;

	/**
	 * Check if the result of a sweep test (passed in InHit) might be a valid location to perch, in which case we should use ComputePerchResult to validate the location.
	 * @see ComputePerchResult
	 * @param InHit:			Result of the last sweep test before this query.
	 * @param bCheckRadius:		If true, only allow the perch test if the impact point is outside the radius returned by GetValidPerchRadius().
	 * @return Whether perching may be possible, such that ComputePerchResult can return a valid result.
	 */
	ENGINE_API virtual bool ShouldComputePerchResult(const FHitResult& InHit, bool bCheckRadius = true) const;

	/**
	 * Compute the sweep result of the smaller capsule with radius specified by GetValidPerchRadius(),
	 * and return true if the sweep contacts a valid walkable normal within InMaxFloorDist of InHit.ImpactPoint.
	 * This may be used to determine if the capsule can or cannot stay at the current location if perched on the edge of a small ledge or unwalkable surface.
	 * Note: Only returns a valid result if ShouldComputePerchResult returned true for the supplied hit value.
	 *
	 * @param TestRadius:			Radius to use for the sweep, usually GetValidPerchRadius().
	 * @param InHit:				Result of the last sweep test before the query.
	 * @param InMaxFloorDist:		Max distance to floor allowed by perching, from the supplied contact point (InHit.ImpactPoint).
	 * @param OutPerchFloorResult:	Contains the result of the perch floor test.
	 * @return True if the current location is a valid spot at which to perch.
	 */
	ENGINE_API virtual bool ComputePerchResult(const float TestRadius, const FHitResult& InHit, const float InMaxFloorDist, FFindFloorResult& OutPerchFloorResult) const;


protected:

	/** Called when the collision capsule touches another primitive component */
	UFUNCTION()
	ENGINE_API virtual void CapsuleTouched(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult);


	/** Get the capsule extent for the Pawn owner, possibly reduced in size depending on ShrinkMode.
	 * @param ShrinkMode			Controls the way the capsule is resized.
	 * @param CustomShrinkAmount	The amount to shrink the capsule, used only for ShrinkModes that specify custom.
	 * @return The capsule extent of the Pawn owner, possibly reduced in size depending on ShrinkMode.
	 */
	ENGINE_API FVector GetPawnCapsuleExtent(const EShrinkCapsuleExtent ShrinkMode, const float CustomShrinkAmount = 0.f) const;
	
	/** Get the collision shape for the Pawn owner, possibly reduced in size depending on ShrinkMode.
	 * @param ShrinkMode			Controls the way the capsule is resized.
	 * @param CustomShrinkAmount	The amount to shrink the capsule, used only for ShrinkModes that specify custom.
	 * @return The capsule extent of the Pawn owner, possibly reduced in size depending on ShrinkMode.
	 */
	ENGINE_API FCollisionShape GetPawnCapsuleCollisionShape(const EShrinkCapsuleExtent ShrinkMode, const float CustomShrinkAmount = 0.f) const;

	/** Adjust the size of the capsule on simulated proxies, to avoid overlaps due to replication rounding.
	  * Changes to the capsule size on the proxy should set bShrinkProxyCapsule=true and possibly call AdjustProxyCapsuleSize() immediately if applicable.
	  */
	ENGINE_API virtual void AdjustProxyCapsuleSize();

	/** Enforce constraints on input given current state. For instance, don't move upwards if walking and looking up. */
	ENGINE_API virtual FVector ConstrainInputAcceleration(const FVector& InputAcceleration) const;

	/** Scale input acceleration, based on movement acceleration rate. */
	ENGINE_API virtual FVector ScaleInputAcceleration(const FVector& InputAcceleration) const;

	/**
	 * Event triggered at the end of a movement update. If scoped movement updates are enabled (bEnableScopedMovementUpdates), this is within such a scope.
	 * If that is not desired, bind to the CharacterOwner's OnMovementUpdated event instead, as that is triggered after the scoped movement update.
	 */
	ENGINE_API virtual void OnMovementUpdated(float DeltaSeconds, const FVector& OldLocation, const FVector& OldVelocity);

	/** Internal function to call OnMovementUpdated delegate on CharacterOwner. */
	ENGINE_API virtual void CallMovementUpdateDelegate(float DeltaSeconds, const FVector& OldLocation, const FVector& OldVelocity);

	/**
	 * Event triggered when we are moving on a base but we are not able to move the full DeltaPosition because something has blocked us.
	 * Note: MoveComponentFlags includes the flag to ignore the movement base while this event is fired.
	 * @param DeltaPosition		How far we tried to move with the base.
	 * @param OldLocation		Location before we tried to move with the base.
	 * @param MoveOnBaseHit		Hit result for the object we hit when trying to move with the base.
	 */
	ENGINE_API virtual void OnUnableToFollowBaseMove(const FVector& DeltaPosition, const FVector& OldLocation, const FHitResult& MoveOnBaseHit);


protected:
	/* Prepare root motion to be passed on to physics thread */
	ENGINE_API virtual void AccumulateRootMotionForAsync(float DeltaSeconds, FRootMotionAsyncData& RootMotion);
	/* Prepare inputs for asynchronous simulation on physics thread */ 
	ENGINE_API virtual void FillAsyncInput(const FVector& InputVector, FCharacterMovementComponentAsyncInput& AsyncInput);
	ENGINE_API virtual void BuildAsyncInput();
	ENGINE_API virtual void PostBuildAsyncInput();
	/* Apply outputs from async sim. */
	ENGINE_API virtual void ApplyAsyncOutput(FCharacterMovementComponentAsyncOutput& Output);
	ENGINE_API virtual void ProcessAsyncOutput();
	
	/* Register async callback with physics system. */
	ENGINE_API virtual void RegisterAsyncCallback();
	ENGINE_API virtual bool IsAsyncCallbackRegistered() const;
	
public:

	/**
	 * Project a location to navmesh to find adjusted height.
	 * @param TestLocation		Location to project
	 * @param NavFloorLocation	Location on navmesh
	 * @return True if projection was performed (successfully or not)
	 */
	ENGINE_API virtual bool FindNavFloor(const FVector& TestLocation, FNavLocation& NavFloorLocation) const;

protected:

	// Movement functions broken out based on owner's network Role.
	// TickComponent calls the correct version based on the Role.
	// These may be called during move playback and correction during network updates.
	//

	/** Perform movement on an autonomous client */
	ENGINE_API virtual void PerformMovement(float DeltaTime);

	/** Special Tick for Simulated Proxies */
	ENGINE_API virtual void SimulatedTick(float DeltaSeconds);

	/** Simulate movement on a non-owning client. Called by SimulatedTick(). */
	ENGINE_API virtual void SimulateMovement(float DeltaTime);

	/** Special Tick to allow custom server-side functionality on Autonomous Proxies. 
	 * Called for all remote APs, including APs controlled on Listen Servers such as the hosting player's Character.
	 * If full server-side control is desired, you may need to override ControlledCharacterMove as well.
	 */
	virtual void ServerAutonomousProxyTick(float DeltaSeconds) { }

public:

	/** Force a client update by making it appear on the server that the client hasn't updated in a long time. */
	ENGINE_API virtual void ForceReplicationUpdate();

	/** Force a client adjustment. Resets ServerLastClientAdjustmentTime. */
	ENGINE_API void ForceClientAdjustment();
	
	/**
	 * Generate a random angle in degrees that is approximately equal between client and server.
	 * Note that in networked games this result changes with low frequency and has a low period,
	 * so should not be used for frequent randomization.
	 */
	ENGINE_API virtual float GetNetworkSafeRandomAngleDegrees() const;

	/** Round acceleration, for better consistency and lower bandwidth in networked games. */
	ENGINE_API virtual FVector RoundAcceleration(FVector InAccel) const;

	//--------------------------------
	// INetworkPredictionInterface implementation

	//--------------------------------
	// Server hook
	//--------------------------------
	ENGINE_API virtual void SendClientAdjustment() override;
	ENGINE_API virtual bool ForcePositionUpdate(float DeltaTime) override;

	//--------------------------------
	// Client hook
	//--------------------------------

	/**
	 * React to new transform from network update. Sets bNetworkSmoothingComplete to false to ensure future smoothing updates.
	 * IMPORTANT: It is expected that this function triggers any movement/transform updates to match the network update if desired.
	 */
	ENGINE_API virtual void SmoothCorrection(const FVector& OldLocation, const FQuat& OldRotation, const FVector& NewLocation, const FQuat& NewRotation) override;

	/** Get prediction data for a client game. Should not be used if not running as a client. Allocates the data on demand and can be overridden to allocate a custom override if desired. Result must be a FNetworkPredictionData_Client_Character. */
	ENGINE_API virtual class FNetworkPredictionData_Client* GetPredictionData_Client() const override;
	/** Get prediction data for a server game. Should not be used if not running as a server. Allocates the data on demand and can be overridden to allocate a custom override if desired. Result must be a FNetworkPredictionData_Server_Character. */
	ENGINE_API virtual class FNetworkPredictionData_Server* GetPredictionData_Server() const override;

	ENGINE_API class FNetworkPredictionData_Client_Character* GetPredictionData_Client_Character() const;
	ENGINE_API class FNetworkPredictionData_Server_Character* GetPredictionData_Server_Character() const;

	ENGINE_API virtual bool HasPredictionData_Client() const override;
	ENGINE_API virtual bool HasPredictionData_Server() const override;

	ENGINE_API virtual void ResetPredictionData_Client() override;
	ENGINE_API virtual void ResetPredictionData_Server() override;

	static uint32 PackYawAndPitchTo32(const float Yaw, const float Pitch);

protected:
	class FNetworkPredictionData_Client_Character* ClientPredictionData;
	class FNetworkPredictionData_Server_Character* ServerPredictionData;

	FRandomStream RandomStream;

	/**
	 * Smooth mesh location for network interpolation, based on values set up by SmoothCorrection.
	 * Internally this simply calls SmoothClientPosition_Interpolate() then SmoothClientPosition_UpdateVisuals().
	 * This function is not called when bNetworkSmoothingComplete is true.
	 * @param DeltaSeconds Time since last update.
	 */
	ENGINE_API virtual void SmoothClientPosition(float DeltaSeconds);

	/**
	 * Update interpolation values for client smoothing. Does not change actual mesh location.
	 * Sets bNetworkSmoothingComplete to true when the interpolation reaches the target.
	 */
	ENGINE_API void SmoothClientPosition_Interpolate(float DeltaSeconds);

	/** Update mesh location based on interpolated values. */
	ENGINE_API void SmoothClientPosition_UpdateVisuals();

	/*
	========================================================================
	Here's how player movement prediction, replication and correction works in network games:
	
	Every tick, the TickComponent() function is called.  It figures out the acceleration and rotation change for the frame,
	and then calls PerformMovement() (for locally controlled Characters), or ReplicateMoveToServer() (if it's a network client).
	
	ReplicateMoveToServer() saves the move (in the PendingMove list), calls PerformMovement(), and then replicates the move
	to the server by calling the replicated function ServerMove() - passing the movement parameters, the client's
	resultant position, and a timestamp.
	
	ServerMove() is executed on the server.  It decodes the movement parameters and causes the appropriate movement
	to occur.  It then looks at the resulting position and if enough time has passed since the last response, or the
	position error is significant enough, the server calls ClientAdjustPosition(), a replicated function.
	
	ClientAdjustPosition() is executed on the client.  The client sets its position to the servers version of position,
	and sets the bUpdatePosition flag to true.
	
	When TickComponent() is called on the client again, if bUpdatePosition is true, the client will call
	ClientUpdatePosition() before calling PerformMovement().  ClientUpdatePosition() replays all the moves in the pending
	move list which occurred after the timestamp of the move the server was adjusting.
	*/

	/** Perform local movement and send the move to the server. */
	ENGINE_API virtual void ReplicateMoveToServer(float DeltaTime, const FVector& NewAcceleration);

	/** If bUpdatePosition is true, then replay any unacked moves. Returns whether any moves were actually replayed. */
	ENGINE_API virtual bool ClientUpdatePositionAfterServerUpdate();

	/** Call the appropriate replicated ServerMove() function to send a client player move to the server. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(CallServerMove, CallServerMovePacked)
	ENGINE_API virtual void CallServerMove(const FSavedMove_Character* NewMove, const FSavedMove_Character* OldMove);

	/**
	 * On the client, calls the ServerMovePacked_ClientSend() function with packed movement data.
	 * First the FCharacterNetworkMoveDataContainer from GetNetworkMoveDataContainer() is updated with ClientFillNetworkMoveData(), then serialized into a data stream to send client player moves to the server.
	 */
	ENGINE_API virtual void CallServerMovePacked(const FSavedMove_Character* NewMove, const FSavedMove_Character* PendingMove, const FSavedMove_Character* OldMove);

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void ServerMoveHandleClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	
	/**
	 * Have the server check if the client is outside an error tolerance, and queue a client adjustment if so.
	 * If either GetPredictionData_Server_Character()->bForceClientUpdate or ServerCheckClientError() are true, the client adjustment will be sent.
	 * RelativeClientLocation will be a relative location if MovementBaseUtility::UseRelativePosition(ClientMovementBase) is true, or a world location if false.
	 * @see ServerCheckClientError()
	 */
	ENGINE_API virtual void ServerMoveHandleClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode);


	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual bool ServerCheckClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/**
	 * Check for Server-Client disagreement in position or other movement state important enough to trigger a client correction.
	 * @see ServerMoveHandleClientError()
	 */
	ENGINE_API virtual bool ServerCheckClientError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode);

	
	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual bool ServerExceedsAllowablePositionError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/**
	 * Check position error within ServerCheckClientError(). Set bNetworkLargeClientCorrection to true if the correction should be prioritized (delayed less in SendClientAdjustment).
	 */
	ENGINE_API virtual bool ServerExceedsAllowablePositionError(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode);

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual bool ServerShouldUseAuthoritativePosition(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/**
	 * If ServerCheckClientError() does not find an error, this determines if the server should also copy the client's movement params rather than keep the server sim result.
	 */
	ENGINE_API virtual bool ServerShouldUseAuthoritativePosition(float ClientTimeStamp, float DeltaTime, const FVector& Accel, const FVector& ClientWorldLocation, const FVector& RelativeClientLocation, FMovementBaseInterfaceData* ClientMovementBaseInterfaceData, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/* Process a move at the given time stamp, given the compressed flags representing various events that occurred (ie jump). */
	ENGINE_API virtual void MoveAutonomous( float ClientTimeStamp, float DeltaTime, uint8 CompressedFlags, const FVector& NewAccel);

	/** Unpack compressed flags from a saved move and set state accordingly. See FSavedMove_Character. */
	ENGINE_API virtual void UpdateFromCompressedFlags(uint8 Flags);

	/** Return true if it is OK to delay sending this player movement to the server, in order to conserve bandwidth. */
	ENGINE_API virtual bool CanDelaySendingMove(const FSavedMovePtr& NewMove);

	/** Determine minimum delay between sending client updates to the server. If updates occur more frequently this than this time, moves may be combined delayed. */
	ENGINE_API virtual float GetClientNetSendDeltaTime(const APlayerController* PC, const FNetworkPredictionData_Client_Character* ClientData, const FSavedMovePtr& NewMove) const;

	/** Ticks the characters pose and accumulates root motion */
	ENGINE_API virtual void TickCharacterPose(float DeltaTime);

	/** On the server if we know we are having our replication rate throttled, this method checks if important replicated properties have changed that should cause us to return to the normal replication rate. */
	ENGINE_API virtual bool ShouldCancelAdaptiveReplication() const;

public:

	/** React to instantaneous change in position. Invalidates cached floor recomputes it if possible if there is a current movement base. */
	ENGINE_API virtual void UpdateFloorFromAdjustment();

	/** Minimum time between client TimeStamp resets.
	 !! This has to be large enough so that we don't confuse the server if the client can stall or timeout.
	 We do this as we use floats for TimeStamps, and server derives DeltaTime from two TimeStamps. 
	 As time goes on, accuracy decreases from those floating point numbers.
	 So we trigger a TimeStamp reset at regular intervals to maintain a high level of accuracy. */
	UPROPERTY()
	float MinTimeBetweenTimeStampResets;

	/** On the Server, verify that an incoming client TimeStamp is valid and has not yet expired.
		It will also handle TimeStamp resets if it detects a gap larger than MinTimeBetweenTimeStampResets / 2.f
		!! ServerData.CurrentClientTimeStamp can be reset !!
		@returns true if TimeStamp is valid, or false if it has expired. */
	ENGINE_API virtual bool VerifyClientTimeStamp(float TimeStamp, FNetworkPredictionData_Server_Character & ServerData);

protected:

	/** Clock time on the server of the last timestamp reset. */
	float LastTimeStampResetServerTime;

	/** Internal const check for client timestamp validity without side-effects. 
	  * @see VerifyClientTimeStamp */
	ENGINE_API bool IsClientTimeStampValid(float TimeStamp, const FNetworkPredictionData_Server_Character& ServerData, bool& bTimeStampResetDetected) const;

	/** Called by UCharacterMovementComponent::VerifyClientTimeStamp() when a client timestamp reset has been detected and is valid. */
	ENGINE_API virtual void OnClientTimeStampResetDetected();

	/** 
	 * Processes client timestamps from ServerMoves, detects and protects against time discrepancy between client-reported times and server time
	 * Called by UCharacterMovementComponent::VerifyClientTimeStamp() for valid timestamps.
	 */
	ENGINE_API virtual void ProcessClientTimeStampForTimeDiscrepancy(float ClientTimeStamp, FNetworkPredictionData_Server_Character& ServerData);

	/** 
	 * Called by UCharacterMovementComponent::ProcessClientTimeStampForTimeDiscrepancy() (on server) when the time from client moves 
	 * significantly differs from the server time, indicating potential time manipulation by clients (speed hacks, significant network 
	 * issues, client performance problems) 
	 * @param CurrentTimeDiscrepancy		Accumulated time difference between client ServerMove and server time - this is bounded
	 *										by MovementTimeDiscrepancy config variables in AGameNetworkManager, and is the value with which
	 *										we test against to trigger this function. This is reset when MovementTimeDiscrepancy resolution
	 *										is enabled
	 * @param LifetimeRawTimeDiscrepancy	Accumulated time difference between client ServerMove and server time - this is unbounded
	 *										and does NOT get affected by MovementTimeDiscrepancy resolution, and is useful as a longer-term
	 *										view of how the given client is performing. High magnitude unbounded error points to
	 *										intentional tampering by a client vs. occasional "naturally caused" spikes in error due to
	 *										burst packet loss/performance hitches
	 * @param Lifetime						Game time over which LifetimeRawTimeDiscrepancy has accrued (useful for determining severity
	 *										of LifetimeUnboundedError)
	 * @param CurrentMoveError				Time difference between client ServerMove and how much time has passed on the server for the
	 *										current move that has caused TimeDiscrepancy to accumulate enough to trigger detection.
	 */
	ENGINE_API virtual void OnTimeDiscrepancyDetected(float CurrentTimeDiscrepancy, float LifetimeRawTimeDiscrepancy, float Lifetime, float CurrentMoveError);

public:

	/**
	 * The actual network RPCs for character movement are passed to ACharacter, which wrap to the _Implementation call here, to avoid Component RPC overhead.
	 * For example:
	 *		Client: UCharacterMovementComponent::ServerMovePacked_ClientSend() => Calls CharacterOwner->ServerMove() triggering RPC on the server.
	 *		Server: ACharacter::ServerMovePacked_Implementation() from the RPC => Calls CharacterMovement->ServerMove_ServerReceive(), unpacked and sent to ServerMove_ServerHandleMoveData().
	 *
	 *	ServerMove_ClientSend() and ServerMove_ServerReceive() use a bitstream created from the current FCharacterNetworkMoveData data container that contains the client move.
	 *	See GetNetworkMoveDataContainer()/SetNetworkMoveDataContainer() for details on setting a custom container with custom unpacking through FCharacterNetworkMoveData::Serialize().
	 * 
	*/

	/**
	 * Wrapper to send packed move data to the server, through the Character.
	 * @see CallServerMovePacked()
	 */
	ENGINE_API void ServerMovePacked_ClientSend(const FCharacterServerMovePackedBits& PackedBits);

	/**
	 * On the server, receives packed move data from the Character RPC, unpacks them into the FCharacterNetworkMoveDataContainer returned from GetNetworkMoveDataContainer(),
	 * and passes the data container to ServerMove_HandleMoveData().
	 */
	ENGINE_API void ServerMovePacked_ServerReceive(const FCharacterServerMovePackedBits& PackedBits);

	/**
	 * Determines whether to use packed movement RPCs with variable length payloads, or legacy code which has multiple functions required for different situations.
	 * The default implementation checks the console variable "p.NetUsePackedMovementRPCs" and returns true if it is non-zero.
	 */
	ENGINE_API virtual bool ShouldUsePackedMovementRPCs() const;

	/* Sends a move response from the server to the client (through character to avoid component RPC overhead), eventually calling MoveResponsePacked_ClientReceive() on the client. */
	ENGINE_API void MoveResponsePacked_ServerSend(const FCharacterMoveResponsePackedBits& PackedBits);

	/* On the client, receives a packed move response from the server, unpacks it by serializing into the MoveResponseContainer from GetMoveResponseDataContainer(), and passes the data container to ClientHandleMoveResponse(). */
	ENGINE_API void MoveResponsePacked_ClientReceive(const FCharacterMoveResponsePackedBits& PackedBits);

	/* If no client adjustment is needed after processing received ServerMove(), ack the good move so client can remove it from SavedMoves */
	ENGINE_API virtual void ClientAckGoodMove_Implementation(float TimeStamp);

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void ClientAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, TOptional<FRotator> OptionalRotation = TOptional<FRotator>());
	
	/* Replicate position correction to client, associated with a timestamped servermove.  Client will replay subsequent moves after applying adjustment. */
	ENGINE_API virtual void ClientAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FVector NewVel, FMovementBaseInterfaceData* NewMovementBaseInterfaceData, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, TOptional<FRotator> OptionalRotation = TOptional<FRotator>());

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void ClientVeryShortAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/* Bandwidth saving version, when velocity is zeroed */
	ENGINE_API virtual void ClientVeryShortAdjustPosition_Implementation(float TimeStamp, FVector NewLoc, FMovementBaseInterfaceData* NewMovementBaseInterfaceData, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void ClientAdjustRootMotionPosition_Implementation(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/* Replicate position correction to client when using root motion for movement. (animation root motion specific) */
	ENGINE_API virtual void ClientAdjustRootMotionPosition_Implementation(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, FMovementBaseInterfaceData* ServerMovementBaseInterfaceData, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void ClientAdjustRootMotionSourcePosition_Implementation(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	
	/* Replicate root motion source correction to client when using root motion for movement. */
	ENGINE_API virtual void ClientAdjustRootMotionSourcePosition_Implementation(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, FMovementBaseInterfaceData* ServerMovementBaseInterfaceData, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/**
	 * Handle movement data after it's unpacked from the ServerMovePacked_ServerReceive() call.
	 * Default implementation passes through to ServerMove_PerformMovement(), which may be called twice in the case of a "dual move", and one additional time for an "old important move".
	 */
	ENGINE_API virtual void ServerMove_HandleMoveData(const FCharacterNetworkMoveDataContainer& MoveDataContainer);

	/**
	 * Check timestamps, generate a delta time, and pass through movement params to MoveAutonomous. Error checking is optionally done on the final location, compared to 'ClientLoc'.
	 * The FCharacterNetworkMoveData parameter to this function is also the same returned by GetCurrentNetworkMoveData(), to assist in migration of code that may want to access the data without changing function signatures.
	 * (Note: this is similar to "ServerMove_Implementation" in legacy versions).
	 */
	ENGINE_API virtual void ServerMove_PerformMovement(const FCharacterNetworkMoveData& MoveData);

	/**
	 * On the server, sends a packed move response to the client. First the FCharacterMoveResponseDataContainer from GetMoveResponseDataContainer() is filled in with ServerFillResponseData().
	 * Then this data is serialized to a bit stream that is sent to the client via MoveResponsePacked_ServerSend().
	 */
	ENGINE_API void ServerSendMoveResponse(const FClientAdjustment& PendingAdjustment);

	/**
	 * On the client, handles the move response from the server after it has been received and unpacked in MoveResponsePacked_ClientReceive.
	 * Based on the data in the response, dispatches a call to ClientAckGoodMove_Implementation if there was no error from the server.
	 * Otherwise dispatches a call to one of ClientAdjustRootMotionSourcePosition_Implementation, ClientAdjustRootMotionPosition_Implementation,
	 * or ClientAdjustPosition_Implementation depending on the payload.
	 */
	ENGINE_API virtual void ClientHandleMoveResponse(const FCharacterMoveResponseDataContainer& MoveResponse);


public:

	/////////////////////////////////////////////////////////////////////////////////////
	// BEGIN DEPRECATED movement RPCs. Use the Packed versions above instead. 
	/////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Replicated function sent by client to server - contains client movement and view info.
	 * Calls either CharacterOwner->ServerMove() or CharacterOwner->ServerMoveNoBase() depending on whehter ClientMovementBase is null.
	 */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMove, ServerMovePacked_ClientSend)
	ENGINE_API virtual void ServerMove(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMove_Implementation, ServerMove_PerformMovement)
	ENGINE_API virtual void ServerMove_Implementation(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API virtual bool ServerMove_Validate(float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 CompressedMoveFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/**
	 * Replicated function sent by client to server - contains client movement and view info for two moves.
	 * Calls either CharacterOwner->ServerMoveDual() or CharacterOwner->ServerMoveDualNoBase() depending on whehter ClientMovementBase is null.
	 */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDual, ServerMovePacked_ClientSend)
	ENGINE_API virtual void ServerMoveDual(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDual_Implementation, ServerMove_PerformMovement)
	ENGINE_API virtual void ServerMoveDual_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API virtual bool ServerMoveDual_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/** Replicated function sent by client to server - contains client movement and view info for two moves. First move is non root motion, second is root motion. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualHybridRootMotion, ServerMovePacked_ClientSend)
	ENGINE_API virtual void ServerMoveDualHybridRootMotion(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveDualHybridRootMotion_Implementation, ServerMove_PerformMovement)
	ENGINE_API virtual void ServerMoveDualHybridRootMotion_Implementation(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);
	ENGINE_API virtual bool ServerMoveDualHybridRootMotion_Validate(float TimeStamp0, FVector_NetQuantize10 InAccel0, uint8 PendingFlags, uint32 View0, float TimeStamp, FVector_NetQuantize10 InAccel, FVector_NetQuantize100 ClientLoc, uint8 NewFlags, uint8 ClientRoll, uint32 View, UPrimitiveComponent* ClientMovementBase, FName ClientBaseBoneName, uint8 ClientMovementMode);

	/* Resending an (important) old move. Process it if not already processed. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveOld, ServerMovePacked_ClientSend)
	ENGINE_API virtual void ServerMoveOld(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ServerMoveOld_Implementation, ServerMove_PerformMovement)
	ENGINE_API virtual void ServerMoveOld_Implementation(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);
	ENGINE_API virtual bool ServerMoveOld_Validate(float OldTimeStamp, FVector_NetQuantize10 OldAccel, uint8 OldMoveFlags);
	
	/** If no client adjustment is needed after processing received ServerMove(), ack the good move so client can remove it from SavedMoves */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAckGoodMove, ClientHandleMoveResponse)
	ENGINE_API virtual void ClientAckGoodMove(float TimeStamp);

	/** Replicate position correction to client, associated with a timestamped servermove.  Client will replay subsequent moves after applying adjustment.  */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustPosition, ClientHandleMoveResponse)
	ENGINE_API virtual void ClientAdjustPosition(float TimeStamp, FVector NewLoc, FVector NewVel, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/* Bandwidth saving version, when velocity is zeroed */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientVeryShortAdjustPosition, ClientHandleMoveResponse)
	ENGINE_API virtual void ClientVeryShortAdjustPosition(float TimeStamp, FVector NewLoc, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);
	
	/** Replicate position correction to client when using root motion for movement. (animation root motion specific) */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionPosition, ClientHandleMoveResponse)
	ENGINE_API virtual void ClientAdjustRootMotionPosition(float TimeStamp, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/** Replicate root motion source correction to client when using root motion for movement. */
	DEPRECATED_CHARACTER_MOVEMENT_RPC(ClientAdjustRootMotionSourcePosition, ClientHandleMoveResponse)
	ENGINE_API virtual void ClientAdjustRootMotionSourcePosition(float TimeStamp, FRootMotionSourceGroup ServerRootMotion, bool bHasAnimRootMotion, float ServerMontageTrackPosition, FVector ServerLoc, FVector_NetQuantizeNormal ServerRotation, float ServerVelZ, UPrimitiveComponent* ServerBase, FName ServerBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode);

	/////////////////////////////////////////////////////////////////////////////////////
	// END DEPRECATED movement RPCs
	/////////////////////////////////////////////////////////////////////////////////////

protected:

	UE_DEPRECATED(5.8, "Movement functions using UPrimitiveComponents have been deprecated. Use functions using the FMovementBaseInterfaceData struct instead.")
	ENGINE_API virtual void OnClientCorrectionReceived(class FNetworkPredictionData_Client_Character& ClientData, float TimeStamp, FVector NewLocation, FVector NewVelocity, UPrimitiveComponent* NewBase, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, FVector ServerGravityDirection);

	/** Event notification when client receives correction data from the server, before applying the data. Base implementation logs relevant data and draws debug info if "p.NetShowCorrections" is not equal to 0. */
	ENGINE_API virtual void OnClientCorrectionReceived(class FNetworkPredictionData_Client_Character& ClientData, float TimeStamp, FVector NewLocation, FVector NewVelocity, FMovementBaseInterfaceData* NewMovementBaseInterfaceData, FName NewBaseBoneName, bool bHasBase, bool bBaseRelativePosition, uint8 ServerMovementMode, FVector ServerGravityDirection);

	/**
	 * Set custom struct used for client to server move RPC serialization.
	 * This is typically set in the constructor for this component and should persist for the lifetime of the component.
	 * @see GetNetworkMoveDataContainer(), ServerMovePacked_ServerReceive(), ServerMove_HandleMoveData()
	 */
	void SetNetworkMoveDataContainer(FCharacterNetworkMoveDataContainer& PersistentDataStorage) { NetworkMoveDataContainerPtr = &PersistentDataStorage; }

	/**
	 * Get the struct used for client to server move RPC serialization.
	 * @see SetNetworkMoveDataContainer()
	 */
	FCharacterNetworkMoveDataContainer& GetNetworkMoveDataContainer() const { return *NetworkMoveDataContainerPtr; }

	/**
	 * Current move data being processed or handled.
	 * This is set before MoveAutonomous (for replayed moves and server moves), and cleared thereafter.
	 * Useful for being able to access custom movement data during internal movement functions such as MoveAutonomous() or UpdateFromCompressedFlags() to be able to maintain backwards API compatibility.
	 */
	FCharacterNetworkMoveData* GetCurrentNetworkMoveData() const { return CurrentNetworkMoveData; }

	/**
	 * Used internally to set the FCharacterNetworkMoveData currently being processed, either being serialized or replayed on the client, or being received and processed on the server.
	 * @see GetCurrentNetworkMoveData()
	 */
	void SetCurrentNetworkMoveData(FCharacterNetworkMoveData* CurrentData) { CurrentNetworkMoveData = CurrentData; }

	/**
	 * Set custom struct used for server response RPC serialization.
	 * This is typically set in the constructor for this component and should persist for the lifetime of the component.
	 * @see GetMoveResponseDataContainer()
	 */
	void SetMoveResponseDataContainer(FCharacterMoveResponseDataContainer& PersistentDataStorage) { MoveResponseDataContainerPtr = &PersistentDataStorage; }

	/**
	 * Get the struct used for server response RPC serialization.
	 * @see SetMoveResponseDataContainer(), ClientHandleMoveResponse(), ServerSendMoveResponse().
	 */
	FCharacterMoveResponseDataContainer& GetMoveResponseDataContainer() const { return *MoveResponseDataContainerPtr; }

	/**
	 * Used internally to save the SavedMove currently being replayed on the client so it is accessible to any functions that might need it.
	 * @see: ClientUpdatePositionAfterServerUpdate 
	 */
	void SetCurrentReplayedSavedMove(FSavedMove_Character* SavedMove) { CurrentReplayedSavedMove = SavedMove; }
	
public:

	/**
	 * Gets the SavedMove being replayed on the client after a correction is received.
	 * @see: ClientUpdatePositionAfterServerUpdate
	 */
	const FSavedMove_Character* GetCurrentReplayedSavedMove() const { return CurrentReplayedSavedMove; }

private:

	/** Current SavedMove being replayed on the client after a correction is received */
	FSavedMove_Character* CurrentReplayedSavedMove = nullptr;

	//////////////////////////////////////////////////////////////////////////
	// Server move data

	/** Default client to server move RPC data container. Can be bypassed via SetNetworkMoveDataContainer(). */
	FCharacterNetworkMoveDataContainer DefaultNetworkMoveDataContainer;

	/** Pointer to server move RPC data container. */
	FCharacterNetworkMoveDataContainer* NetworkMoveDataContainerPtr;

	/** Used for writing server move RPC bits. */
	FNetBitWriter ServerMoveBitWriter;

	/** Used for reading server move RPC bits. */
	FNetBitReader ServerMoveBitReader;

	/** Current network move data being processed or handled within the NetworkMoveDataContainer. */
	FCharacterNetworkMoveData* CurrentNetworkMoveData;

	//////////////////////////////////////////////////////////////////////////
	// Server response data

	/** Default server response RPC data container. Can be bypassed via SetMoveResponseDataContainer(). */
	FCharacterMoveResponseDataContainer DefaultMoveResponseDataContainer;

	/** Pointer to server response RPC data container. */
	FCharacterMoveResponseDataContainer* MoveResponseDataContainerPtr;

	/** Used for writing server response RPC bits. */
	FNetBitWriter MoveResponseBitWriter;

	/** Used for reading server response RPC bits. */
	FNetBitReader MoveResponseBitReader;

public:

	//////////////////////////////////////////////////////////////////////////
	// Root Motion

	/** Root Motion Group containing active root motion sources being applied to movement */
	UPROPERTY(Transient)
	FRootMotionSourceGroup CurrentRootMotion;

	UPROPERTY(Transient)
	FRootMotionSourceGroup ServerCorrectionRootMotion;

	FRootMotionAsyncData AsyncRootMotion;

	/** Returns true if we have Root Motion from any source to use in PerformMovement() physics. */
	ENGINE_API bool HasRootMotionSources() const;

	/** Apply a RootMotionSource to current root motion 
	 *  @return LocalID for this Root Motion Source */
	ENGINE_API uint16 ApplyRootMotionSource(TSharedPtr<FRootMotionSource> SourcePtr);

	/** Called during ApplyRootMotionSource call, useful for project-specific alerts for "something is about to be altering our movement" */
	ENGINE_API virtual void OnRootMotionSourceBeingApplied(const FRootMotionSource* Source);

	/** Get a RootMotionSource from current root motion by name */
	ENGINE_API TSharedPtr<FRootMotionSource> GetRootMotionSource(FName InstanceName);

	/** Get a RootMotionSource from current root motion by ID */
	ENGINE_API TSharedPtr<FRootMotionSource> GetRootMotionSourceByID(uint16 RootMotionSourceID);

	/** Remove a RootMotionSource from current root motion by name */
	ENGINE_API void RemoveRootMotionSource(FName InstanceName);

	/** Remove a RootMotionSource from current root motion by ID */
	ENGINE_API void RemoveRootMotionSourceByID(uint16 RootMotionSourceID);

	/** Converts received server IDs in a root motion group to local IDs  */
	ENGINE_API void ConvertRootMotionServerIDsToLocalIDs(const FRootMotionSourceGroup& LocalRootMotionToMatchWith, FRootMotionSourceGroup& InOutServerRootMotion, float TimeStamp);

	/** Collection of the most recent ID mappings */
	enum class ERootMotionMapping : uint32 { MapSize = 16 };
	TArray<FRootMotionServerToLocalIDMapping, TInlineAllocator<(uint32)ERootMotionMapping::MapSize> > RootMotionIDMappings;

protected:
	/** Restores Velocity to LastPreAdditiveVelocity during Root Motion Phys*() function calls */
	ENGINE_API virtual void RestorePreAdditiveRootMotionVelocity();

	/** Applies root motion from root motion sources to velocity (override and additive) */
	ENGINE_API virtual void ApplyRootMotionToVelocity(float deltaTime);

	/** Reduces former base velocity according to FormerBaseVelocityDecayHalfLife */
	ENGINE_API void DecayFormerBaseVelocity(float deltaTime);

public:

	/**
	*	Animation root motion (special case for now)
	*/

	/** Root Motion movement params. Holds result of anim montage root motion during PerformMovement(), and is overridden
	*   during autonomous move playback to force historical root motion for MoveAutonomous() calls */
	UPROPERTY(Transient)
	FRootMotionMovementParams RootMotionParams;

	/** Velocity extracted from RootMotionParams when there is anim root motion active. Invalid to use when HasAnimRootMotion() returns false. */
	UPROPERTY(Transient)
	FVector AnimRootMotionVelocity;

	/** Returns true if we have Root Motion from animation to use in PerformMovement() physics. 
		Not valid outside of the scope of that function. Since RootMotion is extracted and used in it. */
	bool HasAnimRootMotion() const
	{
		return RootMotionParams.bHasRootMotion;
	}

	// Takes component space root motion and converts it to world space
	ENGINE_API FTransform ConvertLocalRootMotionToWorld(const FTransform& InLocalRootMotion, float DeltaSeconds);

	// Delegate for modifying root motion pre conversion from component space to world space.
	FOnProcessRootMotion ProcessRootMotionPreConvertToWorld;
	
	// Delegate for modifying root motion post conversion from component space to world space.
	FOnProcessRootMotion ProcessRootMotionPostConvertToWorld;

	/** Simulate Root Motion physics on Simulated Proxies */
	ENGINE_API void SimulateRootMotion(float DeltaSeconds, const FTransform& LocalRootMotionTransform);

	/**
	 * Calculate velocity from anim root motion.
	 * @param RootMotionDeltaMove	Change in location from root motion.
	 * @param DeltaSeconds			Elapsed time
	 * @param CurrentVelocity		Non-root motion velocity at current time, used for components of result that may ignore root motion.
	 * @see ConstrainAnimRootMotionVelocity
	 */
	ENGINE_API virtual FVector CalcAnimRootMotionVelocity(const FVector& RootMotionDeltaMove, float DeltaSeconds, const FVector& CurrentVelocity) const;

	/**
	 * Constrain components of root motion velocity that may not be appropriate given the current movement mode (e.g. when falling Z may be ignored).
	 */
	ENGINE_API virtual FVector ConstrainAnimRootMotionVelocity(const FVector& RootMotionVelocity, const FVector& CurrentVelocity) const;

	// RVO Avoidance

	/** calculate RVO avoidance and apply it to current velocity */
	ENGINE_API virtual void CalcAvoidanceVelocity(float DeltaTime);

	/** allows modifing avoidance velocity, called when bUseRVOPostProcess is set */
	ENGINE_API virtual void PostProcessAvoidanceVelocity(FVector& NewVelocity);

	ENGINE_API virtual void FlushServerMoves();

	/** 
	 * When moving the character, we should inform physics as to whether we are teleporting.
	 * This allows physics to avoid injecting forces into simulations from client corrections (etc.)
	 */
	ENGINE_API ETeleportType GetTeleportType() const;

protected:

	/** called in Tick to update data in RVO avoidance manager */
	ENGINE_API void UpdateDefaultAvoidance();

public:
	/** lock avoidance velocity */
	ENGINE_API void SetAvoidanceVelocityLock(class UAvoidanceManager* Avoidance, float Duration);

	/** BEGIN IRVOAvoidanceInterface */
	ENGINE_API virtual void SetRVOAvoidanceUID(int32 UID) override;
	ENGINE_API virtual int32 GetRVOAvoidanceUID() override;
	ENGINE_API virtual void SetRVOAvoidanceWeight(float Weight) override;
	ENGINE_API virtual float GetRVOAvoidanceWeight() override;
	ENGINE_API virtual FVector GetRVOAvoidanceOrigin() override;
	ENGINE_API virtual float GetRVOAvoidanceRadius() override;
	ENGINE_API virtual float GetRVOAvoidanceHeight() override;
	ENGINE_API virtual float GetRVOAvoidanceConsiderationRadius() override;
	ENGINE_API virtual FVector GetVelocityForRVOConsideration() override;
	ENGINE_API virtual void SetAvoidanceGroupMask(int32 GroupFlags) override;
	ENGINE_API virtual int32 GetAvoidanceGroupMask() override;
	ENGINE_API virtual void SetGroupsToAvoidMask(int32 GroupFlags) override;
	ENGINE_API virtual int32 GetGroupsToAvoidMask() override;
	ENGINE_API virtual void SetGroupsToIgnoreMask(int32 GroupFlags) override;
	ENGINE_API virtual int32 GetGroupsToIgnoreMask() override;
	/** END IRVOAvoidanceInterface */

	/** a shortcut function to be called instead of GetRVOAvoidanceUID when
	*	callee knows it's dealing with a char movement comp */
	int32 GetRVOAvoidanceUIDFast() const { return AvoidanceUID; }

public:

	/** Minimum delta time considered when ticking. Delta times below this are not considered. This is a very small non-zero value to avoid potential divide-by-zero in simulation code. */
	static ENGINE_API const float MIN_TICK_TIME;

	/** Minimum acceptable distance for Character capsule to float above floor when walking. */
	static ENGINE_API const float MIN_FLOOR_DIST;

	/** Maximum acceptable distance for Character capsule to float above floor when walking. */
	static ENGINE_API const float MAX_FLOOR_DIST;

	/** Reject sweep impacts that are this close to the edge of the vertical portion of the capsule when performing vertical sweeps, and try again with a smaller capsule. */
	static ENGINE_API const float SWEEP_EDGE_REJECT_DISTANCE;

	/** Stop completely when braking and velocity magnitude is lower than this. */
	static ENGINE_API const float BRAKE_TO_STOP_VELOCITY;
};


inline ACharacter* UCharacterMovementComponent::GetCharacterOwner() const
{
	return CharacterOwner;
}

inline bool UCharacterMovementComponent::IsWalking() const
{
	return IsMovingOnGround();
}

inline uint32 UCharacterMovementComponent::PackYawAndPitchTo32(const float Yaw, const float Pitch)
{
	const uint32 YawShort = FRotator::CompressAxisToShort(Yaw);
	const uint32 PitchShort = FRotator::CompressAxisToShort(Pitch);
	const uint32 Rotation32 = (YawShort << 16) | PitchShort;
	return Rotation32;
}

class FSavedMove_Character_FixLayout
{
public:
	virtual ~FSavedMove_Character_FixLayout() = default;
};

/** FSavedMove_Character represents a saved move on the client that has been sent to the server and might need to be played back. */
class FSavedMove_Character : public FSavedMove_Character_FixLayout
{
public:
	ENGINE_API FSavedMove_Character();
	ENGINE_API virtual ~FSavedMove_Character();

	// UE_DEPRECATED_FORGAME(4.20)
	FSavedMove_Character(const FSavedMove_Character&);
	FSavedMove_Character(FSavedMove_Character&&);
	FSavedMove_Character& operator=(const FSavedMove_Character&);
	FSavedMove_Character& operator=(FSavedMove_Character&&);

	ACharacter* CharacterOwner;

	uint8 bPressedJump:1;
	uint8 bWantsToCrouch:1;
	uint8 bForceMaxAccel:1;

	/** If true, can't combine this move with another move. */
	uint8 bForceNoCombine:1;

	/** If true this move is using an old TimeStamp, before a reset occurred. */
	uint8 bOldTimeStampBeforeReset:1;

	uint8 bWasJumping:1;

	UE_DEPRECATED_FORGAME(4.20, "This property is deprecated, use StartPackedMovementMode or EndPackedMovementMode instead.")
	uint8 MovementMode;

	// Information at the start of the move
	uint8 StartPackedMovementMode;
	uint8 EndPackedMovementMode;

	float TimeStamp;    // Time of this move.
	float DeltaTime;    // amount of time for this move
	float CustomTimeDilation;
	float JumpKeyHoldTime;
	float JumpForceTimeRemaining;
	int32 JumpMaxCount;
	int32 JumpCurrentCount;
	
	FVector StartLocation;
	FVector StartRelativeLocation;
	FVector StartVelocity;
	FFindFloorResult StartFloor;
	FRotator StartRotation;
	FRotator StartControlRotation;
	FQuat StartBaseRotation;	// rotation of the base component (or bone), only saved if it can move.
	float StartCapsuleRadius;
	float StartCapsuleHalfHeight;
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use StartMovementBaseInterfaceData instead.")
	TWeakObjectPtr<UPrimitiveComponent> StartBase;
	FMovementBaseInterfaceData StartMovementBaseInterfaceData;
	FName StartBoneName;
	uint32 StartActorOverlapCounter;
	uint32 StartComponentOverlapCounter;
	TWeakObjectPtr<USceneComponent> StartAttachParent;
	FName StartAttachSocketName;
	FVector StartAttachRelativeLocation;
	FRotator StartAttachRelativeRotation;

	// Information after the move has been performed
	FVector SavedLocation;
	FRotator SavedRotation;
	FVector SavedVelocity;
	FVector SavedRelativeLocation;
	FVector SavedRelativeAcceleration;
	FRotator SavedControlRotation;
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use EndMovementBaseInterfaceData instead.")
	TWeakObjectPtr<UPrimitiveComponent> EndBase;
	FMovementBaseInterfaceData EndMovementBaseInterfaceData;
	FName EndBoneName;
	uint32 EndActorOverlapCounter;
	uint32 EndComponentOverlapCounter;
	TWeakObjectPtr<USceneComponent> EndAttachParent;
	FName EndAttachSocketName;
	FVector EndAttachRelativeLocation;
	FRotator EndAttachRelativeRotation;

	FVector Acceleration;
	float MaxSpeed;

	// Cached to speed up iteration over IsImportantMove().
	FVector AccelNormal;
	float AccelMag;

	mutable TWeakObjectPtr<class UAnimMontage> RootMotionMontage;
	float RootMotionTrackPosition;
	float RootMotionPreviousTrackPosition;
	float RootMotionPlayRateWithScale;
	FRootMotionMovementParams RootMotionMovement;

	FRootMotionSourceGroup SavedRootMotion;

	/** Threshold for deciding this is an "important" move based on DP with last acked acceleration. */
	float AccelDotThreshold;    
	/** Threshold for deciding is this is an important move because acceleration magnitude has changed too much */
	float AccelMagThreshold;	
	/** Threshold for deciding if we can combine two moves, true if cosine of angle between them is <= this. */
	float AccelDotThresholdCombine;
	/** Client saved moves will not combine if the result of GetMaxSpeed() differs by this much between moves. */
	float MaxSpeedThresholdCombine;
	
	/** Clear saved move properties, so it can be re-used. */
	ENGINE_API virtual void Clear();

	/** Called to set up this saved move (when initially created) to make a predictive correction. */
	ENGINE_API virtual void SetMoveFor(ACharacter* C, float InDeltaTime, FVector const& NewAccel, class FNetworkPredictionData_Client_Character & ClientData);

	/** Set the properties describing the position, etc. of the moved pawn at the start of the move. */
	ENGINE_API virtual void SetInitialPosition(ACharacter* C);

	/** Returns true if this move is an "important" move that should be sent again if not acked by the server */
	ENGINE_API virtual bool IsImportantMove(const FSavedMovePtr& LastAckedMove) const;
	
	/** Returns starting position if we were to revert the move, either absolute StartLocation, or StartRelativeLocation offset from MovementBase's current location (since we want to try to move forward at this time). */
	ENGINE_API virtual FVector GetRevertedLocation() const;

	enum EPostUpdateMode
	{
		PostUpdate_Record,		// Record a move after having run the simulation
		PostUpdate_Replay,		// Update after replaying a move for a client correction
	};

	/** Set the properties describing the final position, etc. of the moved pawn. */
	ENGINE_API virtual void PostUpdate(ACharacter* C, EPostUpdateMode PostUpdateMode);
	
	/** Returns true if this move can be combined with NewMove for replication without changing any behavior */
	ENGINE_API virtual bool CanCombineWith(const FSavedMovePtr& NewMove, ACharacter* InCharacter, float MaxDelta) const;

	/** Combine this move with an older move and update relevant state. */
	ENGINE_API virtual void CombineWith(const FSavedMove_Character* OldMove, ACharacter* InCharacter, APlayerController* PC, const FVector& OldStartLocation);
	
	/** Called before ClientUpdatePosition uses this SavedMove to make a predictive correction	 */
	ENGINE_API virtual void PrepMoveFor(ACharacter* C);

	/** Returns a byte containing encoded special movement information (jumping, crouching, etc.)	 */
	ENGINE_API virtual uint8 GetCompressedFlags() const;

	/** Compare current control rotation with stored starting data */
	ENGINE_API virtual bool IsMatchingStartControlRotation(const APlayerController* PC) const;

	/** Packs control rotation for network transport */
	ENGINE_API virtual void GetPackedAngles(uint32& YawAndPitchPack, uint8& RollPack) const;

	/** Allows references to be considered during GC */
	ENGINE_API virtual void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	// Bit masks used by GetCompressedFlags() to encode movement information.
	enum CompressedFlags
	{
		FLAG_JumpPressed	= 0x01,	// Jump pressed
		FLAG_WantsToCrouch	= 0x02,	// Wants to crouch
		FLAG_Reserved_1		= 0x04,	// Reserved for future use
		FLAG_Reserved_2		= 0x08,	// Reserved for future use
		// Remaining bit masks are available for custom flags.
		FLAG_Custom_0		= 0x10,
		FLAG_Custom_1		= 0x20,
		FLAG_Custom_2		= 0x40,
		FLAG_Custom_3		= 0x80,
	};
};

//UE_DEPRECATED_FORGAME(4.20)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
inline FSavedMove_Character::FSavedMove_Character(const FSavedMove_Character&) = default;
inline FSavedMove_Character::FSavedMove_Character(FSavedMove_Character&&) = default;
inline FSavedMove_Character& FSavedMove_Character::operator=(const FSavedMove_Character&) = default;
inline FSavedMove_Character& FSavedMove_Character::operator=(FSavedMove_Character&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS


class FNetworkPredictionData_Client_Character : public FNetworkPredictionData_Client, protected FNoncopyable
{
	using Super = FNetworkPredictionData_Client;

public:

	ENGINE_API FNetworkPredictionData_Client_Character(const UCharacterMovementComponent& ClientMovement);
	ENGINE_API virtual ~FNetworkPredictionData_Client_Character();

	/** Allows references to be considered during GC */
	ENGINE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const override;

	/** Timestamp of last time it sent a servermove() to the server. This is an increasing timestamp from the owning UWorld (undilated, real time seconds). Used for holding off on sending movement updates to save bandwidth. */
	float ClientUpdateRealTime;

	/** Current TimeStamp for sending new Moves to the Server. This time resets to zero at a frequency of MinTimeBetweenTimeStampResets and increases with this component's tick delta time. */
	float CurrentTimeStamp;

	/** Last World timestamp (undilated, real time) at which we received a server ack for a move. This could be either a good move or a correction from the server. */
	float LastReceivedAckRealTime;

	TArray<FSavedMovePtr> SavedMoves;		// Buffered moves pending position updates, orderd oldest to newest. Moves that have been acked by the server are removed.
	TArray<FSavedMovePtr> FreeMoves;		// freed moves, available for buffering
	FSavedMovePtr PendingMove;				// PendingMove already processed on client - waiting to combine with next movement to reduce client to server bandwidth
	FSavedMovePtr LastAckedMove;			// Last acknowledged sent move.

	int32 MaxFreeMoveCount;					// Limit on size of free list
	int32 MaxSavedMoveCount;				// Limit on the size of the saved move buffer

	uint32 bUpdatePosition:1; // when true, update the position (via ClientUpdatePosition)

	// Mesh smoothing variables (for network smoothing)
	//
	
	/** Used for position smoothing in net games */
	FVector OriginalMeshTranslationOffset;

	/** World space offset of the mesh. Target value is zero offset. Used for position smoothing in net games. */
	FVector MeshTranslationOffset;

	/** Used for rotation smoothing in net games (only used by linear smoothing). */
	FQuat OriginalMeshRotationOffset;

	/** Component space offset of the mesh. Used for rotation smoothing in net games. */
	FQuat MeshRotationOffset;

	/** Target for mesh rotation interpolation. */
	FQuat MeshRotationTarget;

	/** Used for remembering how much time has passed between server corrections */
	float LastCorrectionDelta;

	/** Used to track time of last correction */
	float LastCorrectionTime;

	/** Max time delta between server updates over which client smoothing is allowed to interpolate. */
	float MaxClientSmoothingDeltaTime;

	/** Used to track the timestamp of the last server move. */
	double SmoothingServerTimeStamp;

	/** Used to track the client time as we try to match the server.*/
	double SmoothingClientTimeStamp;

	/**
	 * Copied value from UCharacterMovementComponent::NetworkMaxSmoothUpdateDistance.
	 * @see UCharacterMovementComponent::NetworkMaxSmoothUpdateDistance
	 */
	float MaxSmoothNetUpdateDist;

	/**
	 * Copied value from UCharacterMovementComponent::NetworkNoSmoothUpdateDistance.
	 * @see UCharacterMovementComponent::NetworkNoSmoothUpdateDistance
	 */
	float NoSmoothNetUpdateDist;

	/** How long to take to smoothly interpolate from the old pawn position on the client to the corrected one sent by the server.  Must be >= 0. Not used for linear smoothing. */
	float SmoothNetUpdateTime;

	/** How long to take to smoothly interpolate from the old pawn rotation on the client to the corrected one sent by the server.  Must be >= 0. Not used for linear smoothing. */
	float SmoothNetUpdateRotationTime;
	
	/** 
	 * Max delta time for a given move, in real seconds
	 * Based off of AGameNetworkManager::MaxMoveDeltaTime config setting, but can be modified per actor
	 * if needed.
	 * This value is mirrored in FNetworkPredictionData_Server, which is what server logic runs off of.
	 * Client needs to know this in order to not send move deltas that are going to get clamped anyway (meaning
	 * they'll be rejected/corrected).
	 * Note: This was previously named MaxResponseTime, but has been renamed to reflect what it does more accurately
	 */
	float MaxMoveDeltaTime;

	/** Values used for visualization and debugging of simulated net corrections */
	FVector LastSmoothLocation;
	FVector LastServerLocation;
	float	SimulatedDebugDrawTime;

	/** Finds SavedMove index for given TimeStamp. Returns INDEX_NONE if not found (move has been already Acked or cleared). */
	ENGINE_API int32 GetSavedMoveIndex(float TimeStamp) const;

	/** Ack a given move. This move will become LastAckedMove, SavedMoves will be adjusted to only contain unAcked moves. */
	ENGINE_API void AckMove(int32 AckedMoveIndex, UCharacterMovementComponent& CharacterMovementComponent);

	/** Allocate a new saved move. Subclasses should override this if they want to use a custom move class. */
	ENGINE_API virtual FSavedMovePtr AllocateNewMove();

	/** Return a move to the free move pool. Assumes that 'Move' will no longer be referenced by anything but possibly the FreeMoves list. Clears PendingMove if 'Move' is PendingMove. */
	ENGINE_API virtual void FreeMove(const FSavedMovePtr& Move);

	/** Tries to pull a pooled move off the free move list, otherwise allocates a new move. Returns NULL if the limit on saves moves is hit. */
	ENGINE_API virtual FSavedMovePtr CreateSavedMove();

	/** Update CurentTimeStamp from passed in DeltaTime.
		It will measure the accuracy between passed in DeltaTime and how Server will calculate its DeltaTime.
		If inaccuracy is too high, it will reset CurrentTimeStamp to maintain a high level of accuracy.
		@return DeltaTime to use for Client's physics simulation prior to replicate move to server. */
	ENGINE_API float UpdateTimeStampAndDeltaTime(float DeltaTime, ACharacter & CharacterOwner, class UCharacterMovementComponent & CharacterMovementComponent);

	/** Used for simulated packet loss in development builds. */
	float DebugForcedPacketLossTimerStart;
};


class FNetworkPredictionData_Server_Character : public FNetworkPredictionData_Server, protected FNoncopyable
{
public:

	ENGINE_API FNetworkPredictionData_Server_Character(const UCharacterMovementComponent& ServerMovement);
	ENGINE_API virtual ~FNetworkPredictionData_Server_Character();

	FClientAdjustment PendingAdjustment;

	/** Timestamp from the client of most recent ServerMove() processed for this player. Reset occasionally for timestamp resets (to maintain accuracy). */
	float CurrentClientTimeStamp;

	/** Timestamp from the client of most recent ServerMove() received for this player, including rejected requests. */
	float LastReceivedClientTimeStamp;

	/** Timestamp of total elapsed client time. Similar to CurrentClientTimestamp but this is accumulated with the calculated DeltaTime for each move on the server. */
	double ServerAccumulatedClientTimeStamp;

	/** Last time server updated client with a move correction */
	float LastUpdateTime;

	/** Server clock time when last server move was received from client (does NOT include forced moves on server) */
	float ServerTimeStampLastServerMove;
	
	/** 
	 * Max delta time for a given move, in real seconds
	 * Based off of AGameNetworkManager::MaxMoveDeltaTime config setting, but can be modified per actor
	 * if needed.
	 * Note: This was previously named MaxResponseTime, but has been renamed to reflect what it does more accurately
	 */
	float MaxMoveDeltaTime;

	/** Force client update on the next ServerMoveHandleClientError() call. */
	uint32 bForceClientUpdate:1;

	/** Accumulated timestamp difference between autonomous client and server for tracking long-term trends */
	float LifetimeRawTimeDiscrepancy;

	/** 
	 * Current time discrepancy between client-reported moves and time passed
	 * on the server. Time discrepancy resolution's goal is to keep this near zero.
	 */
	float TimeDiscrepancy;

	/** True if currently in the process of resolving time discrepancy */
	bool bResolvingTimeDiscrepancy;

	/** 
	 * When bResolvingTimeDiscrepancy is true, we are in time discrepancy resolution mode whose output is
	 * this value (to be used as the DeltaTime for current ServerMove)
	 */
	float TimeDiscrepancyResolutionMoveDeltaOverride;

	/** 
	 * When bResolvingTimeDiscrepancy is true, we are in time discrepancy resolution mode where we bound
	 * move deltas by Server Deltas. In cases where there are multiple ServerMove RPCs handled within one
	 * server frame tick, we need to accumulate the client deltas of the "no tick" Moves so that the next
	 * Move processed that the server server has ticked for takes into account those previous deltas. 
	 * If we did not use this, the higher the framerate of a client vs the server results in more 
	 * punishment/payback time.
	 */
	float TimeDiscrepancyAccumulatedClientDeltasSinceLastServerTick;

	/** Creation time of this prediction data, used to contextualize LifetimeRawTimeDiscrepancy */
	float WorldCreationTime;

	/** Returns time delta to use for the current ServerMove(). Takes into account time discrepancy resolution if active. */
	ENGINE_API float GetServerMoveDeltaTime(float ClientTimeStamp, float ActorTimeDilation) const;

	/** Returns base time delta to use for a ServerMove, default calculation (no time discrepancy resolution) */
	ENGINE_API float GetBaseServerMoveDeltaTime(float ClientTimeStamp, float ActorTimeDilation) const;

	ENGINE_API virtual void Serialize(FArchive& Archive);

};

