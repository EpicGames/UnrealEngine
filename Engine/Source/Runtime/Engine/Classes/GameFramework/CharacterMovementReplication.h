// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Engine/NetSerialization.h"
#include "Serialization/BitWriter.h"
#include "Containers/BitArray.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Components/PrimitiveComponent.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"

#include "CharacterMovementReplication.generated.h"

class UPackageMap;
class FSavedMove_Character;
class UCharacterMovementComponent;
struct FRootMotionSourceGroup;

// If defined and not zero, deprecated RPCs on UCharacterMovementComponent will not be marked deprecated at compile time, to aid in migration of older projects. New projects should prefer the new API.
#ifndef SUPPORT_DEPRECATED_CHARACTER_MOVEMENT_RPCS
#define SUPPORT_DEPRECATED_CHARACTER_MOVEMENT_RPCS 0
#endif

#if SUPPORT_DEPRECATED_CHARACTER_MOVEMENT_RPCS
#define DEPRECATED_CHARACTER_MOVEMENT_RPC(...)
#else
#define DEPRECATED_CHARACTER_MOVEMENT_RPC(DeprecatedFunction, NewFunction) UE_DEPRECATED_FORGAME(4.26, #DeprecatedFunction "() is deprecated, use " #NewFunction "() instead, or define SUPPORT_DEPRECATED_CHARACTER_MOVEMENT_RPCS=1 in the project and set CVar p.NetUsePackedMovementRPCs=0 to use the old code path.")
#endif

// Number of bits to reserve in serialization container. Make this large enough to try to avoid re-allocation during the worst case RPC calls (dual move + unacknowledged "old important" move).
#ifndef CHARACTER_SERIALIZATION_PACKEDBITS_RESERVED_SIZE
#define CHARACTER_SERIALIZATION_PACKEDBITS_RESERVED_SIZE 1024
#endif

//////////////////////////////////////////////////////////////////////////
/**
 * Intermediate data stream used for network serialization of Character RPC data.
 * This is basically an array of bits that is packed/unpacked via NetSerialize into custom data structs on the sending and receiving ends.
 */
USTRUCT()
struct FCharacterNetworkSerializationPackedBits
{
	GENERATED_USTRUCT_BODY()

	FCharacterNetworkSerializationPackedBits()
		: SavedPackageMap(nullptr)
	{
	}

	ENGINE_API bool NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess);
	UPackageMap* GetPackageMap() const { return SavedPackageMap; }

	//------------------------------------------------------------------------
	// Data

	// TInlineAllocator used with TBitArray takes the number of 32-bit dwords, but the define is in number of bits, so convert here by dividing by 32.
	TBitArray<TInlineAllocator<CHARACTER_SERIALIZATION_PACKEDBITS_RESERVED_SIZE / NumBitsPerDWORD>> DataBits;

	// Since this struct uses custom serialization path we need to explicitly capture exports in order to forward them to iris
	// This is managed by the use of a custom packagemap.
	UE::Net::FIrisPackageMapExports PackageMapExports;

	// Since we capturing data outside of the normal serialization path we also need to store exports to inject when actually sending the data.
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

private:
	UPackageMap* SavedPackageMap;
};

template<>
struct TStructOpsTypeTraits<FCharacterNetworkSerializationPackedBits> : public TStructOpsTypeTraitsBase2<FCharacterNetworkSerializationPackedBits>
{
	enum
	{
		WithNetSerializer = true,
	};
};


//////////////////////////////////////////////////////////////////////////
// Client to Server movement data
//////////////////////////////////////////////////////////////////////////

/**
 * FCharacterNetworkMoveData encapsulates a client move that is sent to the server for UCharacterMovementComponent networking.
 *
 * Adding custom data to the network move is accomplished by deriving from this struct, adding new data members, implementing ClientFillNetworkMoveData(), implementing Serialize(), 
 * and setting up the UCharacterMovementComponent to use an instance of a custom FCharacterNetworkMoveDataContainer (see that struct for more details).
 * 
 * @see FCharacterNetworkMoveDataContainer
 */

struct FCharacterNetworkMoveData
{
public:

	enum class ENetworkMoveType
	{
		NewMove,
		PendingMove,
		OldMove
	};

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCharacterNetworkMoveData()
		: NetworkMoveType(ENetworkMoveType::NewMove)
		, TimeStamp(0.f)
		, Acceleration(ForceInitToZero)
		, Location(ForceInitToZero)
		, ControlRotation(ForceInitToZero)
		, CompressedMoveFlags(0)
		, MovementMode(0)
		, MovementBase(nullptr)
		, MovementBasePhysicsObjectOwner(nullptr)
		, MovementBaseBoneName(NAME_None)
	{
	}
	
	virtual ~FCharacterNetworkMoveData() = default;
	FCharacterNetworkMoveData(const FCharacterNetworkMoveData&) = default;
	FCharacterNetworkMoveData(FCharacterNetworkMoveData&&) = default;
	FCharacterNetworkMoveData& operator=(const FCharacterNetworkMoveData&) = default;
	FCharacterNetworkMoveData& operator=(FCharacterNetworkMoveData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * Given a FSavedMove_Character from UCharacterMovementComponent, fill in data in this struct with relevant movement data.
	 * Note that the instance of the FSavedMove_Character is likely a custom struct of a derived struct of your own, if you have added your own saved move data.
	 * @see UCharacterMovementComponent::AllocateNewMove()
	 */
	ENGINE_API virtual void ClientFillNetworkMoveData(const FSavedMove_Character& ClientMove, ENetworkMoveType MoveType);

	/**
	 * Serialize the data in this struct to or from the given FArchive. This packs or unpacks the data in to a variable-sized data stream that is sent over the
	 * network from client to server.
	 * @see UCharacterMovementComponent::CallServerMovePacked
	 */
	ENGINE_API virtual bool Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap, ENetworkMoveType MoveType);

	// Indicates whether this was the latest new move, a pending/dual move, or old important move.
	ENetworkMoveType NetworkMoveType;

	//------------------------------------------------------------------------
	// Basic movement data.

	float TimeStamp;
	FVector_NetQuantize10 Acceleration;
	FVector_NetQuantize100 Location;		// Either world location or relative to MovementBase if that is set.
	FRotator ControlRotation;
	uint8 CompressedMoveFlags;

	uint8 MovementMode;

	UE_DEPRECATED(5.8, "This property has been deprecated. Please use MovementBasePhysicsObjectOwner instead and use that to create a FMovementBaseInterfaceData. See UCharacterMovementComponent::ServerMove_PerformMovement for an example of this.")
	class UPrimitiveComponent* MovementBase;

	TObjectPtr<UObject> MovementBasePhysicsObjectOwner;
	FName MovementBaseBoneName;
};


//////////////////////////////////////////////////////////////////////////
/**
 * Struct used for network RPC parameters between client/server by ACharacter and UCharacterMovementComponent.
 * To extend network move data and add custom parameters, you typically override this struct with a custom derived struct and set the CharacterMovementComponent
 * to use your container with UCharacterMovementComponent::SetNetworkMoveDataContainer(). Your derived struct would then typically (in the constructor) replace the
 * NewMoveData, PendingMoveData, and OldMoveData pointers to use your own instances of a struct derived from FCharacterNetworkMoveData, where you add custom fields
 * and implement custom serialization to be able to pack and unpack your own additional data.
 * 
 * @see UCharacterMovementComponent::SetNetworkMoveDataContainer()
 */
struct FCharacterNetworkMoveDataContainer
{
public:

	/**
	 * Default constructor. Sets data storage (NewMoveData, PendingMoveData, OldMoveData) to point to default data members. Override those pointers to instead point to custom data if you want to use derived classes.
	 */
	FCharacterNetworkMoveDataContainer()
		: bHasPendingMove(false)
		, bIsDualHybridRootMotionMove(false)
		, bHasOldMove(false)
		, bDisableCombinedScopedMove(false)
	{
		NewMoveData		= &BaseDefaultMoveData[0];
		PendingMoveData	= &BaseDefaultMoveData[1];
		OldMoveData		= &BaseDefaultMoveData[2];
	}

	virtual ~FCharacterNetworkMoveDataContainer() = default;

	/**
	 * Passes through calls to ClientFillNetworkMoveData on each FCharacterNetworkMoveData matching the client moves. Note that ClientNewMove will never be null, but others may be.
	 */
	ENGINE_API virtual void ClientFillNetworkMoveData(const FSavedMove_Character* ClientNewMove, const FSavedMove_Character* ClientPendingMove, const FSavedMove_Character* ClientOldMove);

	/**
	 * Serialize movement data. Passes Serialize calls to each FCharacterNetworkMoveData as applicable, based on bHasPendingMove and bHasOldMove.
	 */
	ENGINE_API virtual bool Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap);

	//------------------------------------------------------------------------
	// Basic movement data. NewMoveData is the most recent move, PendingMoveData is a move right before it (dual move). OldMoveData is an "important" move not yet acknowledged.

	inline FCharacterNetworkMoveData* GetNewMoveData() const		{ return NewMoveData; }
	inline FCharacterNetworkMoveData* GetPendingMoveData() const	{ return PendingMoveData; }
	inline FCharacterNetworkMoveData* GetOldMoveData() const		{ return OldMoveData; }

	//------------------------------------------------------------------------
	// Optional pending data used in "dual moves".
	bool bHasPendingMove;
	bool bIsDualHybridRootMotionMove;
	
	// Optional "old move" data, for redundant important old moves not yet ack'd.
	bool bHasOldMove;

	// True if we want to disable a scoped move around both dual moves (optional from bEnableServerDualMoveScopedMovementUpdates), typically set if bForceNoCombine was true which can indicate an important change in moves.
	bool bDisableCombinedScopedMove;
	
protected:

	FCharacterNetworkMoveData* NewMoveData;
	FCharacterNetworkMoveData* PendingMoveData;	// Only valid if bHasPendingMove is true
	FCharacterNetworkMoveData* OldMoveData;		// Only valid if bHasOldMove is true

private:

	FCharacterNetworkMoveData BaseDefaultMoveData[3];
};


//////////////////////////////////////////////////////////////////////////
/**
 * Structure used internally to handle serialization of FCharacterNetworkMoveDataContainer over the network.
 */
USTRUCT()
struct FCharacterServerMovePackedBits : public FCharacterNetworkSerializationPackedBits
{
	GENERATED_USTRUCT_BODY()
	FCharacterServerMovePackedBits() {}
};

template<>
struct TStructOpsTypeTraits<FCharacterServerMovePackedBits> : public TStructOpsTypeTraitsBase2<FCharacterServerMovePackedBits>
{
	enum
	{
		WithNetSerializer = true,
	};
};


//////////////////////////////////////////////////////////////////////////
// Server to Client response
//////////////////////////////////////////////////////////////////////////

// ClientAdjustPosition replication (event called at end of frame by server)
struct FClientAdjustment
{
public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FClientAdjustment()
		: TimeStamp(0.f)
		, DeltaTime(0.f)
		, NewLoc(ForceInitToZero)
		, NewVel(ForceInitToZero)
		, NewRot(ForceInitToZero)
		, GravityDirection(FVector::DownVector)
		, NewBase(NULL)
		, NewMovementBasePhysicsObjectOwner(nullptr)
		, NewBaseBoneName(NAME_None)
		, bAckGoodMove(false)
		, bBaseRelativePosition(false)
		, bBaseRelativeVelocity(false)
		, MovementMode(0)
	{
	}

	~FClientAdjustment() = default;
	FClientAdjustment(const FClientAdjustment&) = default;
	FClientAdjustment(FClientAdjustment&&) = default;
	FClientAdjustment& operator=(const FClientAdjustment&) = default;
	FClientAdjustment& operator=(FClientAdjustment&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	
	float TimeStamp;
	float DeltaTime;
	FVector NewLoc; // Note: if bBaseRelativePosition is set, this is a relative location to the Movement base.
	FVector NewVel; // Note: if bBaseRelativeVelocity is set, this is a relative velocity to the Movement base.
	FRotator NewRot;
	FVector GravityDirection;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use NewMovementBasePhysicsObjectOwner instead and use that to create a FMovementBaseInterfaceData. See UCharacterMovementComponent::ClientHandleMoveResponse for an example of this.")
	TObjectPtr<UPrimitiveComponent> NewBase;
#else
	UE_DEPRECATED(5.8, "This property has been deprecated. Please use NewMovementBasePhysicsObjectOwner instead and use that to create a FMovementBaseInterfaceData. See UCharacterMovementComponent::ClientHandleMoveResponse for an example of this.")
	UPrimitiveComponent* NewBase;
#endif
	
	TObjectPtr<UObject> NewMovementBasePhysicsObjectOwner;
	FName NewBaseBoneName;
	bool bAckGoodMove;
	bool bBaseRelativePosition;
	bool bBaseRelativeVelocity;
	uint8 MovementMode;

	void Serialize(FArchive& Archive)
	{
		if (Archive.IsMigratingRemoteObjects())
		{
			Archive << TimeStamp;
			Archive << DeltaTime;
			Archive << NewLoc;
			Archive << NewVel;
			Archive << NewRot;
			Archive << GravityDirection;
			Archive << NewMovementBasePhysicsObjectOwner;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NewBase = Cast<UPrimitiveComponent>(NewMovementBasePhysicsObjectOwner.Get());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			Archive << NewBaseBoneName;
			Archive.SerializeBits(&bAckGoodMove, 1);
			Archive.SerializeBits(&bBaseRelativePosition, 1);
			Archive.SerializeBits(&bBaseRelativeVelocity, 1);
			Archive << MovementMode;
		}
	}
};


//////////////////////////////////////////////////////////////////////////
/**
 * Response from the server to the client about a move that is being acknowledged.
 * Internally it mainly copies the FClientAdjustment from the UCharacterMovementComponent indicating the response, as well as
 * setting a few relevant flags about the response and serializing the response to and from an FArchive for handling the variable-size
 * payload over the network.
 */
struct FCharacterMoveResponseDataContainer
{
public:

	FCharacterMoveResponseDataContainer()
		: bHasBase(false)
		, bHasRotation(false)
		, bRootMotionMontageCorrection(false)
		, bRootMotionSourceCorrection(false)
		, RootMotionTrackPosition(-1.0f)
		, RootMotionRotation(ForceInitToZero)
	{
	}

	virtual ~FCharacterMoveResponseDataContainer() = default;

	/**
	 * Copy the FClientAdjustment and set a few flags relevant to that data.
	 */
	ENGINE_API virtual void ServerFillResponseData(const UCharacterMovementComponent& CharacterMovement, const FClientAdjustment& PendingAdjustment);

	/**
	 * Serialize the FClientAdjustment data and other internal flags.
	 */
	ENGINE_API virtual bool Serialize(UCharacterMovementComponent& CharacterMovement, FArchive& Ar, UPackageMap* PackageMap);

	bool IsGoodMove() const		{ return ClientAdjustment.bAckGoodMove;}
	bool IsCorrection() const	{ return !IsGoodMove(); }

	ENGINE_API FRootMotionSourceGroup* GetRootMotionSourceGroup(UCharacterMovementComponent& CharacterMovement) const;

	bool bHasBase;
	bool bHasRotation; // By default ClientAdjustment.NewRot is not serialized. Set this to true after base ServerFillResponseData if you want Rotation to be serialized.
	bool bRootMotionMontageCorrection;
	bool bRootMotionSourceCorrection;

	// Client adjustment. All data other than bAckGoodMove and TimeStamp is only valid if this is a correction (not an ack).
	FClientAdjustment ClientAdjustment;

	float RootMotionTrackPosition;
	FVector_NetQuantizeNormal RootMotionRotation;
};

//////////////////////////////////////////////////////////////////////////
/**
 * Structure used internally to handle serialization of FCharacterMoveResponseDataContainer over the network.
 */
USTRUCT()
struct FCharacterMoveResponsePackedBits : public FCharacterNetworkSerializationPackedBits
{
	GENERATED_USTRUCT_BODY()
	FCharacterMoveResponsePackedBits() {}
};

template<>
struct TStructOpsTypeTraits<FCharacterMoveResponsePackedBits> : public TStructOpsTypeTraitsBase2<FCharacterMoveResponsePackedBits>
{
	enum
	{
		WithNetSerializer = true,
	};
};
