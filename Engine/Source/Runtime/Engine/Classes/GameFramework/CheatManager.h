// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CheatManager
// Object within playercontroller that manages development "cheat"
// commands only spawned in single player mode
// No cheat manager is created in shipping builds.
//=============================================================================

#pragma once

#include "CheatManagerDefines.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"

#include "CheatManager.generated.h"

/** Debug Trace info for capturing **/
struct FDebugTraceInfo
{
	// Line Trace Start
	FVector LineTraceStart;

	// Line Trace End
	FVector LineTraceEnd;

	// Hit Normal Start
	FVector HitNormalStart;

	// Hit Normal End
	FVector HitNormalEnd;

	// Hit ImpactNormal End
	FVector HitImpactNormalEnd;

	// Hit Location
	FVector HitLocation;

	// Half collision capsule height
	float CapsuleHalfHeight;

	// Half collision capsule radius
	float CapsuleRadius;

	// this is when blocked and penetrating
	uint32 bInsideOfObject:1;

	FDebugTraceInfo()
		: LineTraceStart(ForceInit)
		, LineTraceEnd(ForceInit)
		, HitNormalStart(ForceInit)
		, HitNormalEnd(ForceInit)
		, HitLocation(ForceInit)
		, CapsuleHalfHeight(0)
		, CapsuleRadius(0)
		, bInsideOfObject(false)
	{
	}
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCheatManagerCreated, UCheatManager*);


/** A cheat manager extension can extend the main cheat manager in a modular way, being enabled or disabled when the system associated with the cheats is enabled or disabled */
UCLASS(Blueprintable, Within=CheatManager, MinimalAPI)
class UCheatManagerExtension : public UObject
{
	GENERATED_BODY()

public:
	/** Use the outer cheat manager to get a World. */
	ENGINE_API virtual UWorld* GetWorld() const override;

	UFUNCTION(BlueprintPure, Category = "Cheat Manager")
	ENGINE_API APlayerController* GetPlayerController() const;

	UFUNCTION(BlueprintNativeEvent, meta=(DisplayName = "Added To Cheat Manager", Keywords="Begin Play"))
	ENGINE_API void AddedToCheatManager();
	ENGINE_API virtual void AddedToCheatManager_Implementation();

	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Removed From Cheat Manager", Keywords = "End Play"))
	ENGINE_API void RemovedFromCheatManager();
	ENGINE_API virtual void RemovedFromCheatManager_Implementation();

	ENGINE_API virtual void DoExtensionSpecificBugItLog(FOutputDevice& OutputFile);
};

/** 
	Cheat Manager is a central blueprint to implement test and debug code and actions that are not to ship with the game.
	As the Cheat Manager is not instanced in shipping builds, it is for debugging purposes only
*/
UCLASS(Blueprintable, Within=PlayerController, MinimalAPI)
class UCheatManager : public UObject
{
	GENERATED_UCLASS_BODY()

	//~UObject interface
	ENGINE_API virtual bool ProcessConsoleExec(const TCHAR* Cmd, FOutputDevice& Ar, UObject* Executor) override;
	//~End of UObject interface

	/** Debug camera - used to have independent camera without stopping gameplay */
	UE_DEPRECATED(5.8, "Use the FDebugCameraManager class instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the FDebugCameraManager class instead"))
	TObjectPtr<class ADebugCameraController> DebugCameraControllerRef;

	/** Debug camera - used to have independent camera without stopping gameplay */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Debug Camera")
	TSubclassOf<class ADebugCameraController>  DebugCameraControllerClass;

	// Trace/Sweep debug start
	/** If we should should perform a debug capsule trace and draw results. Toggled with DebugCapsuleSweep() */
	uint32 bDebugCapsuleSweep:1;

	/** If we should trace complex collision in debug capsule sweeps. Set with DebugCapsuleSweepComplex() */
	uint32 bDebugCapsuleTraceComplex:1;

	/** Holds information if we used ToggleAILogging cheat to activate AI logging */
	uint32 bToggleAILogging : 1;

	/** If we should should perform a debug capsule trace for pawns and draw results. Toggled with DebugCapsuleSweepPawn() */
	static ENGINE_API bool bDebugCapsuleSweepPawn;

	/** Return true if debug sweeps are enabled for pawns. */ 
	static inline bool IsDebugCapsuleSweepPawnEnabled() { return bDebugCapsuleSweepPawn; }

	/** How far debug trace should go out from player viewpoint */
	float DebugTraceDistance;

	/** Half distance between debug capsule sphere ends. Total heigh of capsule is 2*(this + DebugCapsuleRadius). */
	float DebugCapsuleHalfHeight;

	/** Radius of debug capsule */
	float DebugCapsuleRadius;

	/** How long to draw the normal result */
	float DebugTraceDrawNormalLength;

	/** what channel are we tracing **/
	TEnumAsByte<enum ECollisionChannel> DebugTraceChannel;

	/** array of information for capturing **/
	TArray<struct FDebugTraceInfo> DebugTraceInfoList;

	/** array of information for capturing **/
	TArray<struct FDebugTraceInfo> DebugTracePawnInfoList;

	/** Index of the array for current trace to overwrite.  Whenever you capture, this index will be increased **/
	int32 CurrentTraceIndex;

	/** Index of the array for current trace to overwrite.  Whenever you capture, this index will be increased **/
	int32 CurrentTracePawnIndex;

	/** Pause the game for Delay seconds. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void FreezeFrame(float Delay);

	/* Teleport to surface player is looking at. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void Teleport();

	/* Scale the player's size to be F * default size. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void ChangeSize(float F);

	/** Pawn can fly. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void Fly();

	/** Return to walking movement mode from Fly or Ghost cheat. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void Walk();

	/** Pawn no longer collides with the world, and can fly */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void Ghost();

	/** Invulnerability cheat. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void God();

	/** Modify time dilation to change apparent speed of passage of time. e.g. "Slomo 0.1" makes everything move very slowly, while "Slomo 10" makes everything move very fast. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void Slomo(float NewTimeDilation);

	/** Damage the actor you're looking at (sourced from the player). */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void DamageTarget(float DamageAmount);

	/** Destroy the actor you're looking at. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void DestroyTarget();
	
	/** Destroy all actors of class aClass */
	UFUNCTION(exec)
	ENGINE_API virtual void DestroyAll(TSubclassOf<class AActor>  aClass);

	/** Destroy all pawns except for the (pawn) target.  If no (pawn) target is found we don't destroy anything. */
	UFUNCTION(exec)
	ENGINE_API virtual void DestroyAllPawnsExceptTarget();

	/** Destroys (by calling destroy directly) all non-player pawns of class aClass in the level */
	UFUNCTION(exec)
	ENGINE_API virtual void DestroyPawns(TSubclassOf<class APawn> aClass);

	/** Load Classname and spawn an actor of that class */
	UFUNCTION(exec)
	ENGINE_API virtual void Summon(const FString& ClassName);

	/** Freeze everything in the level except for players. */
	UFUNCTION(exec,BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void PlayersOnly();

	/** Make controlled pawn the viewtarget again. */
	UFUNCTION(exec)
	ENGINE_API virtual void ViewSelf();

	/** View from the point of view of player with PlayerName S. */
	UFUNCTION(exec)
	ENGINE_API virtual void ViewPlayer(const FString& S);

	/** View from the point of view of AActor with Name ActorName. */
	UFUNCTION(exec)
	ENGINE_API virtual void ViewActor(FName ActorName);

	/** View from the point of view of an AActor of class DesiredClass.  Each subsequent ViewClass cycles through the list of actors of that class. */
	UFUNCTION(exec)
	ENGINE_API virtual void ViewClass(TSubclassOf<class AActor> DesiredClass);

	/** Stream in the given level. */
	UFUNCTION(exec)
	ENGINE_API virtual void StreamLevelIn(FName PackageName);

	/** Load the given level. */
	UFUNCTION(exec)
	ENGINE_API virtual void OnlyLoadLevel(FName PackageName);

	/** Stream out the given level. */
	UFUNCTION(exec)
	ENGINE_API virtual void StreamLevelOut(FName PackageName);

	/** Toggle between debug camera/player camera without locking gameplay and with locking local player controller input. */
	UFUNCTION(exec)
	ENGINE_API virtual void ToggleDebugCamera();

	/** Returns true if the player currently has a debug camera */
	ENGINE_API virtual bool IsDebugCameraActive() const;

	/** toggles AI logging */
	UFUNCTION(exec)
	ENGINE_API virtual void ToggleAILogging();

	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API virtual void ServerToggleAILogging();

	/** Toggle capsule trace debugging. Will trace a capsule from current view point and show where it hits the world */
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweep();

	/** Change Trace capsule size **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepSize(float HalfHeight, float Radius);

	/** Change Trace Channel **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepChannel(enum ECollisionChannel Channel);

	/** Change Trace Complex setting **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepComplex(bool bTraceComplex);

	/** Capture current trace and add to persistent list **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepCapture();

	/** Capture current local PC's pawn's location and add to persistent list **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepPawn();

	/** Clear persistent list for trace capture **/
	UFUNCTION(exec)
	ENGINE_API virtual void DebugCapsuleSweepClear();

	/** Test all volumes in the world to the player controller's view location**/
	UFUNCTION(exec)
	ENGINE_API virtual void TestCollisionDistance();

	/** Dump online session information */
	UFUNCTION(exec)
	ENGINE_API virtual void DumpOnlineSessionState();

	/** Dump known party information */
	UFUNCTION(exec)
	ENGINE_API virtual void DumpPartyState();

	/** Dump known chat information */
	UFUNCTION(exec)
	ENGINE_API virtual void DumpChatState();

	/** Dump current state of voice chat */
	UFUNCTION(exec)
	ENGINE_API virtual void DumpVoiceMutingState();

	/**
	 * This will move the player and set their rotation to the passed in values.
	 * We have this version of the BugIt family as it is easier to type in just raw numbers in the console.
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void BugItGo(float X, float Y, float Z, float Pitch, float Yaw, float Roll);

	/**
	 * This will move the player and set their rotation to the passed in values.
	 * We have this version of the BugIt family strings can be passed in from the game ?options easily
	 */
	ENGINE_API virtual void BugItGoString(const FString& TheLocation, const FString& TheRotation);

	/**
	* This function is used to print out the BugIt location.  It prints out copy and paste versions for both IMing someone to type in
	* and also a gameinfo ?options version so that you can append it to your launching url and be taken to the correct place.
	* Additionally, it will take a screen shot so reporting bugs is a one command action!
	*
	**/
	UFUNCTION(exec)
	ENGINE_API virtual void BugIt(const FString& ScreenShotDescription = TEXT(""));

	/** This will create a BugItGo string for us.  Nice for calling form c++ where you just want the string and no Screenshots **/
	UFUNCTION(exec)
	ENGINE_API virtual void BugItStringCreator(FVector ViewLocation, FRotator ViewRotation, FString& GoString, FString& LocString);

	/** This will force a flush of the output log to file*/
	UFUNCTION(exec)
	ENGINE_API virtual void FlushLog();

	/** Logs the current location in bugit format without taking screenshot and further routing. */
	UFUNCTION(exec)
	ENGINE_API virtual void LogLoc();

	/** Translate world origin to this player position */
	UFUNCTION(exec)
	ENGINE_API void SetWorldOrigin();

	/** Exec function to return the mouse sensitivity to its default value */
	UFUNCTION(exec)
	ENGINE_API virtual void SetMouseSensitivityToDefault();

	/** Backwards compatibility exec function for people used to it instead of using InvertAxisKey */
	UFUNCTION(exec)
	ENGINE_API virtual void InvertMouse();

	/** Executes commands listed in CheatScript.ScriptName ini section of DefaultGame.ini */
	UFUNCTION(exec)
	ENGINE_API void CheatScript(FString ScriptName);

	UFUNCTION(Exec)
	ENGINE_API void SpawnServerStatReplicator();

	UFUNCTION(Exec)
	ENGINE_API void DestroyServerStatReplicator();

	UFUNCTION(Exec)
	ENGINE_API void ToggleServerStatReplicatorClientOverwrite();

	UFUNCTION(Exec)
	ENGINE_API void ToggleServerStatReplicatorUpdateStatNet();

	UFUNCTION(exec)
	ENGINE_API void UpdateSafeArea();

	UFUNCTION()
	ENGINE_API void OnPlayerEndPlayed(AActor* Player, EEndPlayReason::Type EndPlayReason);

	/**
	 * This will move the player and set their rotation to the passed in values.
	 * This actually does the location / rotation setting.  Additionally it will set you as ghost as the level may have
	 * changed since the last time you were here.  And the bug may actually be inside of something.
	 */
	ENGINE_API virtual void BugItWorker( FVector TheLocation, FRotator TheRotation );

	/** Bug it log to file */
	ENGINE_API virtual void LogOutBugItGoToLogFile( const FString& InScreenShotDesc, const FString& InScreenShotPath, const FString& InGoString, const FString& InLocString );

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Do any trace debugging that is currently enabled */
	ENGINE_API void TickCollisionDebug();

	/** Add Debug Trace info into current index - used when DebugCapsuleSweepPawn is on**/
	ENGINE_API void AddCapsuleSweepDebugInfo(
		const FVector& LineTraceStart, 
		const FVector& LineTraceEnd, 
		const FVector& HitImpactLocation, 
		const FVector& HitNormal, 
		const FVector& HitImpactNormal, 
		const FVector& HitLocation, 
		float CapsuleHalfheight, 
		float CapsuleRadius, 
		bool bTracePawn, 
		bool bInsideOfObject );	

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	/** streaming level debugging */
	ENGINE_API virtual void SetLevelStreamingStatus(FName PackageName, bool bShouldBeLoaded, bool bShouldBeVisible);

	/** 
	* BP implementable event for when CheatManager is created to allow any needed initialization.  
	*/
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Init Cheat Manager", Keywords="Begin Play"))
	ENGINE_API void ReceiveInitCheatManager();

	/** 
	 * Called when CheatManager is created to allow any needed initialization.  
	 */
	ENGINE_API virtual void InitCheatManager();

	/**
	* This is the End Play event for the CheatManager
	*/
	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName = "Shutdown", keywords="endplay"))
	ENGINE_API void ReceiveEndPlay();
	
	/**
	* Called before destroying the object.  This is called immediately upon deciding to destroy the object, to allow the object to begin an
	* asynchronous cleanup process.
	*/
	ENGINE_API virtual void BeginDestroy() override;

	/** Use the Outer Player Controller to get a World.  */
	ENGINE_API virtual UWorld* GetWorld() const override;

public:
	/** Registers a cheat manager extension with this cheat manager */
	ENGINE_API void AddCheatManagerExtension(UCheatManagerExtension* CheatObject);

	/** Removes a cheat manager extension from this cheat manager */
	ENGINE_API void RemoveCheatManagerExtension(UCheatManagerExtension* CheatObject);
	
	/** Finds a previously registered cheat manager extension of the specified class */
	ENGINE_API UCheatManagerExtension* FindCheatManagerExtension(const UClass* InClass) const;

	/** Finds a previously registered cheat manager extension of the specified class */
	template<typename T>
	T* FindCheatManagerExtension() const
	{
		return CastChecked<T>(FindCheatManagerExtension(T::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Finds a previously registered cheat manager extension that implements the specified interface */
	ENGINE_API UCheatManagerExtension* FindCheatManagerExtensionInterface(const UClass* InClass) const;

	/** Finds a previously registered cheat manager extension that implements the specified interface */
	template<class T = UInterface>
	T* FindCheatManagerExtensionInterface() const
	{
		return CastChecked<T>(FindCheatManagerExtensionInterface(T::UClassType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Register a delegate to call whenever a cheat manager is spawned; it will also be called immediately for cheat managers that already exist at this point */
	static ENGINE_API FDelegateHandle RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate&& Delegate);

	/** Unregister a delegate previously registered with CallOrRegister_OnCheatManagerCreated */
	static ENGINE_API void UnregisterFromOnCheatManagerCreated(FDelegateHandle DelegateHandle);

	/** Do game specific bugIt */
	ENGINE_API virtual bool DoGameSpecificBugItLog(FOutputDevice& OutputFile);

protected:
	/** List of registered cheat manager extensions */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UCheatManagerExtension>> CheatManagerExtensions;

	/** Delegate called when the asset manager singleton is created */
	static ENGINE_API FOnCheatManagerCreated OnCheatManagerCreatedDelegate;

protected:

	/** Switch controller to debug camera without locking gameplay and with locking local player controller input */
	UFUNCTION(BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void EnableDebugCamera();
	
	/** Switch controller from debug camera back to normal controller */
	UFUNCTION(BlueprintCallable,Category="Cheat Manager")
	ENGINE_API virtual void DisableDebugCamera();
    
    /** Retrieve the given PlayerContoller's current "target" AActor. */
    ENGINE_API virtual AActor* GetTarget(APlayerController* PlayerController, struct FHitResult& OutHit);

public:
	UFUNCTION(BlueprintPure, Category = "Cheat Manager")
	ENGINE_API APlayerController* GetPlayerController() const;
};



