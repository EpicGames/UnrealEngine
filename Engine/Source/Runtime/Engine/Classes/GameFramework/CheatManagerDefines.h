// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

/** If set to 0, cheat manager functionality will be entirely disabled. If cheats are desired in shipping mode this can be overridden in the target definitions */
#ifndef UE_WITH_CHEAT_MANAGER
#define UE_WITH_CHEAT_MANAGER (1 && !UE_BUILD_SHIPPING)
#endif
