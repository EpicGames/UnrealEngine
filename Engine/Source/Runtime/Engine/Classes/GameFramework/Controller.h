// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/CoreNet.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "Controller.generated.h"

class ACharacter;
class APawn;
class APlayerState;
class FDebugDisplayInfo;
class UDamageType;

UDELEGATE(BlueprintAuthorityOnly)
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams( FInstigatedAnyDamageSignature, float, Damage, const UDamageType*, DamageType, AActor*, DamagedActor, AActor*, DamageCauser );
DECLARE_MULTICAST_DELEGATE_OneParam(FPawnChangedSignature, APawn* /*NewPawn*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPossessedPawnChanged, APawn*, OldPawn, APawn*, NewPawn);

/**
 * Controllers are non-physical actors that can possess a Pawn to control
 * its actions.  PlayerControllers are used by human players to control pawns, while
 * AIControllers implement the artificial intelligence for the pawns they control.
 * Controllers take control of a pawn using their Possess() method, and relinquish
 * control of the pawn by calling UnPossess().
 *
 * Controllers receive notifications for many of the events occurring for the Pawn they
 * are controlling.  This gives the controller the opportunity to implement the behavior
 * in response to this event, intercepting the event and superseding the Pawn's default
 * behavior.
 *
 * ControlRotation (accessed via GetControlRotation()), determines the viewing/aiming
 * direction of the controlled Pawn and is affected by input such as from a mouse or gamepad.
 * 
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Controller/
 */
UCLASS(abstract, notplaceable, NotBlueprintable, HideCategories=(Collision,Rendering,Transformation), MinimalAPI) 
class AController : public AActor, public INavAgentInterface
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	ENGINE_API AController(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** PlayerState containing replicated information about the player using this controller (only exists for players, not NPCs). */
	UPROPERTY(replicatedUsing = OnRep_PlayerState, BlueprintReadOnly, Category=Controller)
	TObjectPtr<APlayerState> PlayerState;
	ENGINE_API void SetPlayerState(APlayerState* InPlayerState);

	/** Actor marking where this controller spawned in. */
	TWeakObjectPtr<class AActor> StartSpot;

	/** Called when the controller has instigated damage in any way */
	UPROPERTY(BlueprintAssignable)
	FInstigatedAnyDamageSignature OnInstigatedAnyDamage;

	/** Called on both authorities and clients when the possessed pawn changes (either OldPawn or NewPawn might be nullptr) */
	UPROPERTY(BlueprintAssignable, Category=Pawn)
	FOnPossessedPawnChanged OnPossessedPawnChanged;

	/** Current gameplay state this controller is in */
	UPROPERTY()
	FName StateName;

private:
	/** Pawn currently being controlled by this controller.  Use Pawn.Possess() to take control of a pawn */
	UPROPERTY(replicatedUsing=OnRep_Pawn)
	TObjectPtr<APawn> Pawn;
	void SetPawn_Direct(APawn* InPawn);

	/**
	 * Used to track when pawn changes during OnRep_Pawn. 
	 * It's possible to use a OnRep parameter here, but I'm not sure what happens with pointers yet so playing it safe.
	 */
	TWeakObjectPtr< APawn > OldPawn;

	/** Character currently being controlled by this controller.  Value is same as Pawn if the controlled pawn is a character, otherwise nullptr */
	UPROPERTY()
	TObjectPtr<ACharacter> Character;

	/** Component to give controllers a transform and enable attachment if desired. */
	UPROPERTY()
	TObjectPtr<USceneComponent> TransformComponent;

protected:
	/** Delegate broadcast on authorities when possessing a new pawn or unpossessing one */
	FPawnChangedSignature OnNewPawn;

	/** The control rotation of the Controller. See GetControlRotation. */
	UPROPERTY()
	FRotator ControlRotation;

	/** Return false if rotation contains NaN or extremely large values (usually resulting from uninitialized values). */
	ENGINE_API bool IsValidControlRotation(FRotator CheckRotation) const;

	/**
	 * If true, the controller location will match the possessed Pawn's location. If false, it will not be updated. Rotation will match ControlRotation in either case.
	 * Since a Controller's location is normally inaccessible, this is intended mainly for purposes of being able to attach
	 * an Actor that follows the possessed Pawn location, but that still has the full aim rotation (since a Pawn might
	 * update only some components of the rotation).
	 */
	UPROPERTY(EditDefaultsOnly, Category="Controller|Transform")
	uint8 bAttachToPawn:1;

	/** Whether this controller is a PlayerController. */
	uint8 bIsPlayerController:1;

	/** Whether the controller must have authority to be able to call possess on a Pawn */
	uint8 bCanPossessWithoutAuthority:1;

	/** Ignores movement input. Stacked state storage, Use accessor function IgnoreMoveInput() */
	uint8 IgnoreMoveInput;

	/** Ignores look input. Stacked state storage, use accessor function IgnoreLookInput(). */
	uint8 IgnoreLookInput;

	/**
	 * Physically attach the Controller to the specified Pawn, so that our position reflects theirs.
	 * The attachment persists during possession of the pawn. The Controller's rotation continues to match the ControlRotation.
	 * Attempting to attach to a nullptr Pawn will call DetachFromPawn() instead.
	 */
	ENGINE_API virtual void AttachToPawn(APawn* InPawn);

	/** Detach the RootComponent from its parent, but only if bAttachToPawn is true and it was attached to a Pawn.	 */
	ENGINE_API virtual void DetachFromPawn();

	/** Add dependency that makes us tick before the given Pawn. This minimizes latency between input processing and Pawn movement.	 */
	ENGINE_API virtual void AddPawnTickDependency(APawn* NewPawn);

	/** Remove dependency that makes us tick before the given Pawn.	 */
	ENGINE_API virtual void RemovePawnTickDependency(APawn* InOldPawn);

	/** Returns TransformComponent subobject **/
	class USceneComponent* GetTransformComponent() const { return TransformComponent; }

public:

	/** Change the current state to named state */
	ENGINE_API virtual void ChangeState(FName NewState);

	/** 
	 * States (uses FNames for replication, correlated to state flags) 
	 * @param StateName the name of the state to test against
	 * @return true if current state is StateName
	 */
	ENGINE_API bool IsInState(FName InStateName) const;
	
	/** @return the name of the current state */
	ENGINE_API FName GetStateName() const;

	/**
	 * Get the control rotation. This is the full aim rotation, which may be different than a camera orientation (for example in a third person view),
	 * and may differ from the rotation of the controlled Pawn (which may choose not to visually pitch or roll, for example).
	 */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual FRotator GetControlRotation() const;

	/** Set the control rotation. The RootComponent's rotation will also be updated to match it if RootComponent->bAbsoluteRotation is true. */
	UFUNCTION(BlueprintCallable, Category=Pawn, meta=(Tooltip="Set the control rotation."))
	ENGINE_API virtual void SetControlRotation(const FRotator& NewRotation);

	/** Set the initial location and rotation of the controller, as well as the control rotation. Typically used when the controller is first created. */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual void SetInitialLocationAndRotation(const FVector& NewLocation, const FRotator& NewRotation);


	/**
	 * Checks line to center and top of other actor
	 * @param Other is the actor whose visibility is being checked.
	 * @param ViewPoint is eye position visibility is being checked from.  If vect(0,0,0) passed in, uses current viewtarget's eye position.
	 * @param bAlternateChecks used only in AIController implementation
	 * @return true if controller's pawn can see Other actor.
	 */
	UFUNCTION(BlueprintCallable, Category=Controller)
	ENGINE_API virtual bool LineOfSightTo(const class AActor* Other, FVector ViewPoint = FVector(ForceInit), bool bAlternateChecks = false) const;

	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_Pawn();

	UFUNCTION()
	ENGINE_API virtual void OnRep_PlayerState();
	
	/**
	 * @return this controller's PlayerState cast to the template type, or NULL if there is not one.
	 * May return null if the cast fails.
	 */
	template < class T >
	T* GetPlayerState() const
	{
		return Cast<T>(PlayerState);
	}

	/** Replicated function to set the pawn location and rotation, allowing server to force (ex. teleports). */
	UFUNCTION(Reliable, Client, WithValidation)
	ENGINE_API void ClientSetLocation(FVector NewLocation, FRotator NewRotation);

	/** Replicated function to set the pawn rotation, allowing the server to force. */
	UFUNCTION(Reliable, Client, WithValidation)
	ENGINE_API void ClientSetRotation(FRotator NewRotation, bool bResetCamera = false);

	/** Return the Pawn that is currently 'controlled' by this PlayerController */
	UFUNCTION(BlueprintCallable, Category=Pawn, meta=(DisplayName="Get Controlled Pawn", ScriptName="GetControlledPawn"))
	ENGINE_API APawn* K2_GetPawn() const;

	FPawnChangedSignature& GetOnNewPawnNotifier() { return OnNewPawn; }

public:

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	//~ Begin AActor Interface
	ENGINE_API virtual void TickActor( float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction ) override;
	ENGINE_API virtual void K2_DestroyActor() override;
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	ENGINE_API virtual void GetActorEyesViewPoint( FVector& out_Location, FRotator& out_Rotation ) const override;
	ENGINE_API virtual FString GetHumanReadableName() const override;

	/** Overridden to create the player replication info and perform other mundane initialization tasks. */
	ENGINE_API virtual void PostInitializeComponents() override;

	ENGINE_API virtual void Reset() override;
	ENGINE_API virtual void Destroyed() override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual FRemoteServerId GetPreferredRemoteServer() const override;
#endif
	//~ End AActor Interface

	/** Getter for Pawn */
	inline TObjectPtr<APawn> GetPawn() const { return Pawn; }

	/** Templated version of GetPawn, will return nullptr if cast fails */
	template<class T>
	T* GetPawn() const
	{
		return Cast<T>(Pawn);
	}

	/** Getter for Character */
	inline ACharacter* GetCharacter() const { return Character; }

	/** Setter for Pawn. Normally should only be used internally when possessing/unpossessing a Pawn. */
	ENGINE_API virtual void SetPawn(APawn* InPawn);

	/** Calls SetPawn and RepNotify locally */
	ENGINE_API void SetPawnFromRep(APawn* InPawn);

	/** Get the actor the controller is looking at */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual AActor* GetViewTarget() const;

	/** Get the desired pawn target rotation */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual FRotator GetDesiredRotation() const;

	/** Returns whether this Controller is a PlayerController.  */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	bool IsPlayerController() const;

	/** Returns whether this Controller is a locally controlled PlayerController.  */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	bool IsLocalPlayerController() const;

	/** Returns whether this Controller is a local controller.	 */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual bool IsLocalController() const;

	/** Called from Destroyed().  Cleans up PlayerState. */
	ENGINE_API virtual void CleanupPlayerState();

	/**
	 * Handles attaching this controller to the specified pawn.
	 * Only runs on the network authority (where HasAuthority() returns true).
	 * Derived native classes can override OnPossess to filter the specified pawn.
	 * When possessed pawn changed, blueprint class gets notified by ReceivePossess
	 * and OnNewPawn delegate is broadcasted.
	 * @param InPawn The Pawn to be possessed.
	 * @see HasAuthority, OnPossess, ReceivePossess
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Pawn, meta=(Keywords="set controller"))
	ENGINE_API virtual void Possess(APawn* InPawn) final; // DEPRECATED(4.22, "Possess is marked virtual final as you should now be overriding OnPossess instead")

	/** Called to unpossess our pawn for any reason that is not the pawn being destroyed (destruction handled by PawnDestroyed()). */
	UFUNCTION(BlueprintCallable, Category=Pawn, meta=(Keywords="set controller"))
	ENGINE_API virtual void UnPossess() final; // DEPRECATED(4.22, "Possess is marked virtual final as you should now be overriding OnUnPossess instead")

protected:
	/** Blueprint implementable event to react to the controller possessing a pawn */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On Possess"))
	ENGINE_API void ReceivePossess(APawn* PossessedPawn);

	/**
	 * Overridable native function for when this controller is asked to possess a pawn.
	 * @param InPawn The Pawn to be possessed
	 */
	ENGINE_API virtual void OnPossess(APawn* InPawn);

	/** Blueprint implementable event to react to the controller unpossessing a pawn */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On UnPossess"))
	ENGINE_API void ReceiveUnPossess(APawn* UnpossessedPawn);

	/** Overridable native function for when this controller unpossesses its pawn. */
	ENGINE_API virtual void OnUnPossess();

public:
	/**
	 * Called to unpossess our pawn because it is going to be destroyed.
	 * (other unpossession handled by UnPossess())
	 */
	ENGINE_API virtual void PawnPendingDestroy(APawn* inPawn);

	/** Called when this controller instigates ANY damage */
	ENGINE_API virtual void InstigatedAnyDamage(float Damage, const class UDamageType* DamageType, class AActor* DamagedActor, class AActor* DamageCauser);

	/** spawns and initializes the PlayerState for this Controller */
	ENGINE_API virtual void InitPlayerState();

	/**
	 * Called from game mode upon end of the game, used to transition to proper state. 
	 * @param EndGameFocus Actor to set as the view target on end game
	 * @param bIsWinner true if this controller is on winning team
	 */
	ENGINE_API virtual void GameHasEnded(class AActor* EndGameFocus = nullptr, bool bIsWinner = false);

	/**
	 * Returns Player's Point of View
	 * For the AI this means the Pawn's 'Eyes' ViewPoint
	 * For a Human player, this means the Camera's ViewPoint
	 *
	 * @output	out_Location, view location of player
	 * @output	out_rotation, view rotation of player
	 */
	UFUNCTION(BlueprintCallable, Category = Pawn)
	ENGINE_API virtual void GetPlayerViewPoint( FVector& Location, FRotator& Rotation ) const;

	/** GameMode failed to spawn pawn for me. */
	ENGINE_API virtual void FailedToSpawnPawn();

	//~ Begin INavAgentInterface Interface
	ENGINE_API virtual const struct FNavAgentProperties& GetNavAgentPropertiesRef() const override;
	ENGINE_API virtual FVector GetNavAgentLocation() const override;
	ENGINE_API virtual void GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const override;
	ENGINE_API virtual bool ShouldPostponePathUpdates() const override;
	ENGINE_API virtual bool IsFollowingAPath() const override;
	ENGINE_API virtual IPathFollowingAgentInterface* GetPathFollowingAgent() const override;
	//~ End INavAgentInterface Interface
	
	/** Aborts the move the controller is currently performing */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	ENGINE_API virtual void StopMovement();

	/**
	 * Locks or unlocks movement input, consecutive calls stack up and require the same amount of calls to undo, or can all be undone using ResetIgnoreMoveInput.
	 * @param bNewMoveInput	If true, move input is ignored. If false, input is not ignored.
	 */
	UFUNCTION(BlueprintCallable, Category=Input)
	ENGINE_API virtual void SetIgnoreMoveInput(bool bNewMoveInput);

	/** Stops ignoring move input by resetting the ignore move input state. */
	UFUNCTION(BlueprintCallable, Category=Input, meta=(Keywords = "ClearIgnoreMoveInput"))
	ENGINE_API virtual void ResetIgnoreMoveInput();

	/** Returns true if movement input is ignored. */
	UFUNCTION(BlueprintCallable, Category=Input)
	ENGINE_API virtual bool IsMoveInputIgnored() const;

	/**
	* Locks or unlocks look input, consecutive calls stack up and require the same amount of calls to undo, or can all be undone using ResetIgnoreLookInput.
	* @param bNewLookInput	If true, look input is ignored. If false, input is not ignored.
	*/
	UFUNCTION(BlueprintCallable, Category=Input)
	ENGINE_API virtual void SetIgnoreLookInput(bool bNewLookInput);

	/** Stops ignoring look input by resetting the ignore look input state. */
	UFUNCTION(BlueprintCallable, Category=Input, meta=(Keywords="ClearIgnoreLookInput"))
	ENGINE_API virtual void ResetIgnoreLookInput();

	/** Returns true if look input is ignored. */
	UFUNCTION(BlueprintCallable, Category=Input)
	ENGINE_API virtual bool IsLookInputIgnored() const;

	/** Reset move and look input ignore flags. */
	UFUNCTION(BlueprintCallable, Category=Input)
	ENGINE_API virtual void ResetIgnoreInputFlags();

	/** Called when the level this controller is in is unloaded via streaming. */
	ENGINE_API virtual void CurrentLevelUnloaded();

	/** Returns whether this controller should persist through seamless travel */
	ENGINE_API virtual bool ShouldParticipateInSeamlessTravel() const;

protected:
	/** State entered when inactive (no possessed pawn, not spectating, etc). */
	ENGINE_API virtual void BeginInactiveState();

	/** Called when leaving the inactive state */
	ENGINE_API virtual void EndInactiveState();

	/** Event when this controller instigates ANY damage */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly)
	ENGINE_API void ReceiveInstigatedAnyDamage(float Damage, const class UDamageType* DamageType, class AActor* DamagedActor, class AActor* DamageCauser);

private:
	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();

};


// INLINES

inline bool AController::IsPlayerController() const
{
	return bIsPlayerController;
}

inline bool AController::IsLocalPlayerController() const
{
	return IsPlayerController() && IsLocalController();
}
