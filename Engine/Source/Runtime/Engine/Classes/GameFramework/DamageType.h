// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DamageType.generated.h"

/**
 * A DamageType is intended to define and describe a particular form of damage and to provide an avenue 
 * for customizing responses to damage from various sources.
 *
 * For example, a game could make a DamageType_Fire set it up to ignite the damaged actor.
 *
 * DamageTypes are never instanced and should be treated as immutable data holders with static code
 * functionality.  They should never be stateful.
 */
UCLASS(MinimalAPI, const, Blueprintable, BlueprintType)
class UDamageType : public UObject
{
	GENERATED_UCLASS_BODY()

	/** True if this damagetype is caused by the world (falling off level, into lava, etc). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DamageType)
	uint32 bCausedByWorld:1;

	/** True to scale imparted momentum by the receiving pawn's mass for pawns using character movement */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DamageType)
	uint32 bScaleMomentumByMass:1;

	/** When applying radial impulses, whether to treat as impulse or velocity change. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RigidBody)
	uint32 bRadialDamageVelChange : 1;

	/** The magnitude of impulse to apply to the Actors damaged by this type. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RigidBody)
	float DamageImpulse;

	/** How large the impulse should be applied to destructible meshes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Destruction)
	float DestructibleImpulse;

	/** How much the damage spreads on a destructible mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Destruction)
	float DestructibleDamageSpreadScale;

	/** Damage fall-off for radius damage (exponent).  Default 1.0=linear, 2.0=square of distance, etc. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DamageType)
	float DamageFalloff;
};



