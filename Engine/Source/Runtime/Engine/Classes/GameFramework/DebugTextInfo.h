// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DebugTextInfo.generated.h"

class AActor;
class UFont;

/*
 * Single entry of a debug text item to render. 
 *
 * @see AHud
 * @see AddDebugText(), RemoveDebugText() and DrawDebugTextList()
 */
USTRUCT()
struct FDebugTextInfo
{
	GENERATED_USTRUCT_BODY()

	/**  AActor related to text item */
	UPROPERTY()
	TObjectPtr<AActor> SrcActor;

	/** Offset from SrcActor.Location to apply */
	UPROPERTY()
	FVector SrcActorOffset;

	/** Desired offset to interpolate to */
	UPROPERTY()
	FVector SrcActorDesiredOffset;

	/** Text to display */
	UPROPERTY()
	FString DebugText;

	/** Time remaining for the debug text, -1.f == infinite */
	UPROPERTY(transient)
	float TimeRemaining;

	/** Duration used to lerp desired offset */
	UPROPERTY()
	float Duration;

	/** Text color */
	UPROPERTY()
	FColor TextColor;

	/** whether the offset should be treated as absolute world location of the string */
	UPROPERTY()
	uint32 bAbsoluteLocation:1;

	/** If the actor moves does the text also move with it? */
	UPROPERTY()
	uint32 bKeepAttachedToActor:1;

	/** Whether to draw a shadow for the text */
	UPROPERTY()
	uint32 bDrawShadow:1;

	/** When we first spawn store off the original actor location for use with bKeepAttachedToActor */
	UPROPERTY()
	FVector OrigActorLocation;

	/** The Font which to display this as.  Will Default to GetSmallFont()**/
	UPROPERTY()
	TObjectPtr<UFont> Font;

	/** Scale to apply to font when rendering */
	UPROPERTY()
	float FontScale;

	FDebugTextInfo()
		: SrcActor(NULL)
		, SrcActorOffset(ForceInit)
		, SrcActorDesiredOffset(ForceInit)
		, TimeRemaining(0)
		, Duration(0)
		, TextColor(ForceInit)
		, bAbsoluteLocation(false)
		, bKeepAttachedToActor(false)
		, bDrawShadow(false)
		, OrigActorLocation(ForceInit)
		, Font(NULL)
		, FontScale(1.0f)
	{
	}

};
