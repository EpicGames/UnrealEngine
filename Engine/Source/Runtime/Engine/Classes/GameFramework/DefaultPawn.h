// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// DefaultPawns are simple pawns that can fly around the world.
//
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Pawn.h"
#include "DefaultPawn.generated.h"

class UInputComponent;
class UPawnMovementComponent;
class USphereComponent;
class UStaticMeshComponent;

/**
 * DefaultPawn implements a simple Pawn with spherical collision and built-in flying movement.
 * @see UFloatingPawnMovement
 */
UCLASS(config=Game, Blueprintable, BlueprintType, MinimalAPI)
class ADefaultPawn : public APawn
{
	GENERATED_UCLASS_BODY()

	// Begin Pawn overrides
	ENGINE_API virtual UPawnMovementComponent* GetMovementComponent() const override;
	ENGINE_API virtual void SetupPlayerInputComponent(UInputComponent* InInputComponent) override;
	ENGINE_API virtual void UpdateNavigationRelevance() override;
	// End Pawn overrides

	/**
	 * Input callback to move forward in local space (or backward if Val is negative).
	 * @param Val Amount of movement in the forward direction (or backward if negative).
	 * @see APawn::AddMovementInput()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn")
	ENGINE_API virtual void MoveForward(float Val);

	/**
	 * Input callback to strafe right in local space (or left if Val is negative).
	 * @param Val Amount of movement in the right direction (or left if negative).
	 * @see APawn::AddMovementInput()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn")
	ENGINE_API virtual void MoveRight(float Val);

	/**
	 * Input callback to move up in world space (or down if Val is negative).
	 * @param Val Amount of movement in the world up direction (or down if negative).
	 * @see APawn::AddMovementInput()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn")
	ENGINE_API virtual void MoveUp_World(float Val);

	/**
	 * Called via input to turn at a given rate.
	 * @param Rate	This is a normalized rate, i.e. 1.0 means 100% of desired turn rate
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn")
	ENGINE_API virtual void TurnAtRate(float Rate);

	/**
	* Called via input to look up at a given rate (or down if Rate is negative).
	* @param Rate	This is a normalized rate, i.e. 1.0 means 100% of desired turn rate
	*/
	UFUNCTION(BlueprintCallable, Category="Pawn")
	ENGINE_API virtual void LookUpAtRate(float Rate);
	
	/** Base turn rate, in deg/sec. Other scaling may affect final turn rate. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Pawn")
	float BaseTurnRate;

	/** Base lookup rate, in deg/sec. Other scaling may affect final lookup rate. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Pawn")
	float BaseLookUpRate;

public:
	/** Name of the MovementComponent.  Use this name if you want to use a different class (with ObjectInitializer.SetDefaultSubobjectClass). */
	static ENGINE_API FName MovementComponentName;

protected:
	/** DefaultPawn movement component */
	UPROPERTY(Category = Pawn, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPawnMovementComponent> MovementComponent;

public:
	/** Name of the CollisionComponent. */
	static ENGINE_API FName CollisionComponentName;

private:
	/** DefaultPawn collision component */
	UPROPERTY(Category = Pawn, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USphereComponent> CollisionComponent;
public:

	/** Name of the MeshComponent. Use this name if you want to prevent creation of the component (with ObjectInitializer.DoNotCreateDefaultSubobject). */
	static ENGINE_API FName MeshComponentName;

private:
	/** The mesh associated with this Pawn. */
	UPROPERTY(Category = Pawn, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> MeshComponent;
public:

	/** If true, adds default input bindings for movement and camera look. */
	UPROPERTY(Category=Pawn, EditAnywhere, BlueprintReadOnly)
	uint32 bAddDefaultMovementBindings:1;

	/** Returns CollisionComponent subobject **/
	USphereComponent* GetCollisionComponent() const { return CollisionComponent; }
	/** Returns MeshComponent subobject **/
	UStaticMeshComponent* GetMeshComponent() const { return MeshComponent; }
};

