// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// DefaultPhysicsVolume:  the default physics volume for areas of the level with
// no physics volume specified
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/PhysicsVolume.h"
#include "DefaultPhysicsVolume.generated.h"

/**
* DefaultPhysicsVolume determines the physical effects an actor will experience if they are not inside any user specified PhysicsVolume
*
* @see APhysicsVolume
*/
UCLASS(notplaceable, transient, MinimalAPI)
class ADefaultPhysicsVolume : public APhysicsVolume
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
		virtual bool ShouldCheckCollisionComponentForErrors() const override { return false; }
#endif // WITH_EDITOR
};



