// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/LocalMessage.h"
#include "EngineMessage.generated.h"

UCLASS(abstract, MinimalAPI)
class  UEngineMessage : public ULocalMessage
{
	GENERATED_UCLASS_BODY()

	/** Message displayed in message dialog when player pawn fails to spawn because no playerstart was available. */
	UPROPERTY()
	FString FailedPlaceMessage;

	/** Message when player join attempt is refused because the server is at capacity. */
	UPROPERTY()
	FString MaxedOutMessage;

	/** Message when a new player enters the game. */
	UPROPERTY()
	FString EnteredMessage;

	/** Message when a player leaves the game. */
	UPROPERTY()
	FString LeftMessage;

	/** Message when a player changes their name. */
	UPROPERTY()
	FString GlobalNameChange;

	/** Message when a new spectator enters the server (if spectator has a player name). */
	UPROPERTY()
	FString SpecEnteredMessage;

	/** Message when a new player enters the server (if player is unnamed). */
	UPROPERTY()
	FString NewPlayerMessage;

	/** Message when a new spectator enters the server (if spectator is unnamed). */
	UPROPERTY()
	FString NewSpecMessage;

	//~ Begin ULocalMessage Interface
	virtual void ClientReceive(const FClientReceiveData& ClientData) const override;
	//~ End ULocalMessage Interface
};
