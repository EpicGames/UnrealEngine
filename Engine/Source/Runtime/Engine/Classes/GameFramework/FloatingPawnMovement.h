// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Movement component updates position of associated PrimitiveComponent during its tick
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/PawnMovementComponent.h"
#include "FloatingPawnMovement.generated.h"

/**
 * FloatingPawnMovement is a movement component that provides simple movement for any Pawn class.
 * Limits on speed and acceleration are provided, while gravity is not implemented.
 *
 * Normally the root component of the owning actor is moved, however another component may be selected (see SetUpdatedComponent()).
 * During swept (non-teleporting) movement only collision of UpdatedComponent is considered, attached components will teleport to the end location ignoring collision.
 */
UCLASS(ClassGroup = Movement, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UFloatingPawnMovement : public UPawnMovementComponent
{
	GENERATED_UCLASS_BODY()

	//Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//End UActorComponent Interface

	//Begin UMovementComponent Interface
	virtual float GetMaxSpeed() const override { return MaxSpeed; }

protected:
	ENGINE_API virtual bool ResolvePenetrationImpl(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation) override;
public:
	//End UMovementComponent Interface

	/** Maximum velocity magnitude allowed for the controlled Pawn. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FloatingPawnMovement)
	float MaxSpeed;

	/** Acceleration applied by input (rate of change of velocity) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FloatingPawnMovement)
	float Acceleration;

	/** Deceleration applied when there is no input (rate of change of velocity) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FloatingPawnMovement)
	float Deceleration;

	/**
	 * Setting affecting extra force applied when changing direction, making turns have less drift and become more responsive.
	 * Velocity magnitude is not allowed to increase, that only happens due to normal acceleration. It may decrease with large direction changes.
	 * Larger values apply extra force to reach the target direction more quickly, while a zero value disables any extra turn force.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FloatingPawnMovement, meta=(ClampMin="0", UIMin="0"))
	float TurningBoost;

protected:

	/** Update Velocity based on input. Also applies gravity. */
	ENGINE_API virtual void ApplyControlInputToVelocity(float DeltaTime);

	/** Prevent Pawn from leaving the world bounds (if that restriction is enabled in WorldSettings) */
	ENGINE_API virtual bool LimitWorldBounds();

	/** Set to true when a position correction is applied. Used to avoid recalculating velocity when this occurs. */
	UPROPERTY(Transient)
	uint32 bPositionCorrected:1;
};

