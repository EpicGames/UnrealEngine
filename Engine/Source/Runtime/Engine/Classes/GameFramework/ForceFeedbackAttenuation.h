// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Engine/Attenuation.h"
#include "ForceFeedbackAttenuation.generated.h"

/** The struct for defining the properties used when determining attenuation for a force feedback effect */
USTRUCT(BlueprintType)
struct FForceFeedbackAttenuationSettings : public FBaseAttenuationSettings
{
	GENERATED_BODY()

	// Currently no unique properties for force feedback, but using a unique type to make future expansion possible
};

/** 
 * Wrapper class that can be created as an asset for force feedback attenuation properties which allows reuse
 * of the properties for multiple attenuation components
 */
UCLASS(BlueprintType, hidecategories=Object, editinlinenew, MinimalAPI)
class UForceFeedbackAttenuation : public UObject 
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Settings)
	FForceFeedbackAttenuationSettings Attenuation;
};
