// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Curves/CurveFloat.h"
#include "ForceFeedbackParameters.h"
#include "GameFramework/InputDevicePropertyHandle.h"
#include "ForceFeedbackEffect.generated.h"

class UForceFeedbackEffect;
struct FForceFeedbackValues;
class UInputDeviceProperty;

USTRUCT()
struct FForceFeedbackChannelDetails
{
	GENERATED_USTRUCT_BODY()

	/** Please note the final channel mapping depends on the software and hardware capabilities of the platform used to run the engine or the game. Refer to documentation for more information. */
	UPROPERTY(EditAnywhere, Category="ChannelDetails")
	uint32 bAffectsLeftLarge:1;

	/** Please note the final channel mapping depends on the software and hardware capabilities of the platform used to run the engine or the game. Refer to documentation for more information. */
	UPROPERTY(EditAnywhere, Category="ChannelDetails")
	uint32 bAffectsLeftSmall:1;

	/** Please note the final channel mapping depends on the software and hardware capabilities of the platform used to run the engine or the game. Refer to documentation for more information. */
	UPROPERTY(EditAnywhere, Category="ChannelDetails")
	uint32 bAffectsRightLarge:1;

	/** Please note the final channel mapping depends on the software and hardware capabilities of the platform used to run the engine or the game. Refer to documentation for more information. */
	UPROPERTY(EditAnywhere, Category="ChannelDetails")
	uint32 bAffectsRightSmall:1;

	UPROPERTY(EditAnywhere, Category="ChannelDetails")
	FRuntimeFloatCurve Curve;

	FForceFeedbackChannelDetails()
		: bAffectsLeftLarge(true)
		, bAffectsLeftSmall(true)
		, bAffectsRightLarge(true)
		, bAffectsRightSmall(true)
	{
	}
};

USTRUCT()
struct FActiveForceFeedbackEffect
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<class UForceFeedbackEffect> ForceFeedbackEffect;

	FForceFeedbackParameters Parameters;
	float PlayTime;

	/** The platform user that should receive this effect */
	FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;

	/** Set to true after this force feedback effect has activated it's device properties */
	bool bActivatedDeviceProperties;

	/** Array of device properties that have been activated by this force feedback effect */
	UPROPERTY()
	TSet<FInputDevicePropertyHandle> ActiveDeviceProperties;

	FActiveForceFeedbackEffect()
		: ForceFeedbackEffect(nullptr)
		, PlayTime(0.f)
		, PlatformUser(PLATFORMUSERID_NONE)
		, bActivatedDeviceProperties(false)
	{
	}

	FActiveForceFeedbackEffect(UForceFeedbackEffect* InEffect, FForceFeedbackParameters InParameters, FPlatformUserId InPlatformUser)
		: ForceFeedbackEffect(InEffect)
		, Parameters(InParameters)
		, PlayTime(0.f)
		, PlatformUser(InPlatformUser)
		, bActivatedDeviceProperties(false)
	{
	}

	ENGINE_API ~FActiveForceFeedbackEffect();

	// Updates the final force feedback values based on this effect.  Returns true if the effect should continue playing, false if it is finished.
	ENGINE_API bool Update(float DeltaTime, FForceFeedbackValues& Values);

	// Activates all the device properties with the input device subsystem
	ENGINE_API void ActivateDeviceProperties();

	/** Reset any device properties that may need to be after the duration of this effect has ended. */
	ENGINE_API void ResetDeviceProperties();

	// Gets the current values at the stored play time
	ENGINE_API void GetValues(FForceFeedbackValues& Values) const;
};

/** A wrapper struct for setting channel details on a per-platform basis */
USTRUCT()
struct FForceFeedbackEffectOverridenChannelDetails
{
	GENERATED_BODY()

	FForceFeedbackEffectOverridenChannelDetails();

	UPROPERTY(EditAnywhere, Category="ForceFeedbackEffect")
	TArray<FForceFeedbackChannelDetails> ChannelDetails;
};

/**
 * A predefined force-feedback effect to be played on a controller
 */
UCLASS(BlueprintType, MinimalAPI)
class UForceFeedbackEffect : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category="ForceFeedbackEffect")
	TArray<FForceFeedbackChannelDetails> ChannelDetails;

	/** A map of platform name -> ForceFeedback channel details */
	UPROPERTY(EditAnywhere, Category = "ForceFeedbackEffect", Meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FForceFeedbackEffectOverridenChannelDetails> PerDeviceOverrides;

	/** A map of input device properties that we want to set while this effect is playing */
	UPROPERTY(EditAnywhere, Instanced, Category = "ForceFeedbackEffect")
	TArray<TObjectPtr<UInputDeviceProperty>> DeviceProperties;

	/** Duration of force feedback pattern in seconds. */
	UPROPERTY(Category=Info, AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly)
	float Duration;

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	float GetDuration();

	/** Returns the longest duration of any active UInputDeviceProperty's that this effect has on it. */
	float GetTotalDevicePropertyDuration();

	void GetValues(const float EvalTime, FForceFeedbackValues& Values, const FPlatformUserId PlatformUser, float ValueMultiplier = 1.f) const;

	/**
	 * Returns the channel details that should currently be used for the given platform.
	 * This will return one of the OverridenDetails if the current platform has one specified.
	 * If there isn't an override specified, then it will return the generic "ChannelDetails" array.
	 **/
	const TArray<FForceFeedbackChannelDetails>& GetCurrentChannelDetails(const FPlatformUserId PlatformUser) const;
};
