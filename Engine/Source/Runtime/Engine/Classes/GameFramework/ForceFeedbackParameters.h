// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ForceFeedbackParameters.generated.h"

/** This structure is used to pass arguments to ClientPlayForceFeedback() client RPC function */
USTRUCT()
struct FForceFeedbackParameters
{
	GENERATED_BODY()

		FForceFeedbackParameters()
		: bLooping(false)
		, bIgnoreTimeDilation(false)
		, bPlayWhilePaused(false)
	{}

	UPROPERTY()
		FName Tag;

	UPROPERTY()
		bool bLooping;

	UPROPERTY()
		bool bIgnoreTimeDilation;

	UPROPERTY()
		bool bPlayWhilePaused;
};
