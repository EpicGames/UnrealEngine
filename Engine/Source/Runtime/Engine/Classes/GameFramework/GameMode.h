// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/GameModeBase.h"
#include "GameMode.generated.h"

class APlayerState;
class ULocalMessage;
class UNetDriver;

/** Possible state of the current match, where a match is all the gameplay that happens on a single map */
namespace MatchState
{
	extern ENGINE_API const FName EnteringMap;			// We are entering this map, actors are not yet ticking
	extern ENGINE_API const FName WaitingToStart;		// Actors are ticking, but the match has not yet started
	extern ENGINE_API const FName InProgress;			// Normal gameplay is occurring. Specific games will have their own state machine inside this state
	extern ENGINE_API const FName WaitingPostMatch;		// Match has ended so we aren't accepting new players, but actors are still ticking
	extern ENGINE_API const FName LeavingMap;			// We are transitioning out of the map to another location
	extern ENGINE_API const FName Aborted;				// Match has failed due to network issues or other problems, cannot continue

	// If a game needs to add additional states, you may need to override HasMatchStarted and HasMatchEnded to deal with the new states
	// Do not add any states before WaitingToStart or after WaitingPostMatch
}

/**
 * GameMode is a subclass of GameModeBase that behaves like a multiplayer match-based game.
 * It has default behavior for picking spawn points and match state.
 * If you want a simpler base, inherit from GameModeBase instead.
 */
UCLASS(MinimalAPI)
class AGameMode : public AGameModeBase
{
	GENERATED_UCLASS_BODY()

	// Code to deal with the match state machine

	/** Returns the current match state, this is an accessor to protect the state machine flow */
	UFUNCTION(BlueprintCallable, Category="Game")
	FName GetMatchState() const { return MatchState; }

	/** Returns true if the match state is InProgress or other gameplay state */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API virtual bool IsMatchInProgress() const;

	/** Transition from WaitingToStart to InProgress. You can call this manually, will also get called if ReadyToStartMatch returns true */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API virtual void StartMatch();

	/** Transition from InProgress to WaitingPostMatch. You can call this manually, will also get called if ReadyToEndMatch returns true */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API virtual void EndMatch();

	/** Restart the game, by default travel to the current map */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API virtual void RestartGame();

	/** Report that a match has failed due to unrecoverable error */
	UFUNCTION(BlueprintCallable, Category="Game")
	ENGINE_API virtual void AbortMatch();

protected:

	/** What match state we are currently in */
	UPROPERTY(Transient)
	FName MatchState;

	/** Updates the match state and calls the appropriate transition functions */
	ENGINE_API virtual void SetMatchState(FName NewState);

	/** Overridable virtual function to dispatch the appropriate transition functions before GameState and Blueprints get SetMatchState calls. */
	ENGINE_API virtual void OnMatchStateSet();

	/** Implementable event to respond to match state changes */
	UFUNCTION(BlueprintImplementableEvent, Category="Game", meta=(DisplayName="OnSetMatchState", ScriptName="OnSetMatchState"))
	ENGINE_API void K2_OnSetMatchState(FName NewState);

	// Games should override these functions to deal with their game specific logic

	/** Called when the state transitions to WaitingToStart */
	ENGINE_API virtual void HandleMatchIsWaitingToStart();

	/** Returns true if ready to Start Match. Games should override this */
	UFUNCTION(BlueprintNativeEvent, Category="Game")
	ENGINE_API bool ReadyToStartMatch();

	/** Called when the state transitions to InProgress */
	ENGINE_API virtual void HandleMatchHasStarted();

	/** Returns true if ready to End Match. Games should override this */
	UFUNCTION(BlueprintNativeEvent, Category="Game")
	ENGINE_API bool ReadyToEndMatch();

	/** Called when the map transitions to WaitingPostMatch */
	ENGINE_API virtual void HandleMatchHasEnded();

	/** Called when the match transitions to LeavingMap */
	ENGINE_API virtual void HandleLeavingMap();

	/** Called when the match transitions to Aborted */
	ENGINE_API virtual void HandleMatchAborted();

public:

	/** Whether the game should immediately start when the first player logs in. Affects the default behavior of ReadyToStartMatch */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="GameMode")
	uint32 bDelayedStart : 1;

	/** Current number of spectators. */
	UPROPERTY(BlueprintReadOnly, Category=GameMode)
	int32 NumSpectators;    

	/** Current number of human players. */
	UPROPERTY(BlueprintReadOnly, Category=GameMode)
	int32 NumPlayers;    

	/** number of non-human players (AI controlled but participating as a player). */
	UPROPERTY(BlueprintReadOnly, Category=GameMode)
	int32 NumBots;    

	/** Minimum time before player can respawn after dying. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=GameMode, meta=(DisplayName="Minimum Respawn Delay"))
	float MinRespawnDelay;

	/** Number of players that are still traveling from a previous map */
	UPROPERTY(BlueprintReadOnly, Category=GameMode)
	int32 NumTravellingPlayers;

	/** Contains strings describing localized game agnostic messages. */
	UPROPERTY()
	TSubclassOf<class ULocalMessage> EngineMessageClass;

	/** PlayerStates of players who have disconnected from the server (saved in case they reconnect) */
	UPROPERTY()
	TArray<TObjectPtr<class APlayerState>> InactivePlayerArray;    

protected:

	/** Time a playerstate will stick around in an inactive state after a player logout */
	UPROPERTY(EditAnywhere, Category=GameMode)
	float InactivePlayerStateLifeSpan;

	/** The maximum number of inactive players before we kick the oldest ones out */
	UPROPERTY(EditAnywhere, Category = GameMode)
	int32 MaxInactivePlayers;

	/** If true, dedicated servers will record replays when HandleMatchHasStarted/HandleMatchHasStopped is called */
	UPROPERTY(config)
	bool bHandleDedicatedServerReplays;

public:

	/** Get local address */
	ENGINE_API virtual FString GetNetworkNumber();
	
	/** Will remove the controller from the correct count then add them to the spectator count. **/
	ENGINE_API void PlayerSwitchedToSpectatorOnly(APlayerController* PC);

	/** Removes the passed in player controller from the correct count for player/spectator/tranistioning **/
	ENGINE_API void RemovePlayerControllerFromPlayerCount(APlayerController* PC);

	/** Return true if we want to travel_absolute, used by RestartGame by default */
	ENGINE_API virtual bool GetTravelType();

	/** Exec command to broadcast a string to all players */
	UFUNCTION(Exec, BlueprintCallable, Category = AI)
	ENGINE_API virtual void Say(const FString& Msg);

	/** Broadcast a string to all players. */
	ENGINE_API virtual void Broadcast( AActor* Sender, const FString& Msg, FName Type = NAME_None );

	/**
	 * Broadcast a localized message to all players.
	 * Most message deal with 0 to 2 related PlayerStates.
	 * The LocalMessage class defines how the PlayerState's and optional actor are used.
	 */
	ENGINE_API virtual void BroadcastLocalized( AActor* Sender, TSubclassOf<ULocalMessage> Message, int32 Switch = 0, APlayerState* RelatedPlayerState_1 = NULL, APlayerState* RelatedPlayerState_2 = NULL, UObject* OptionalObject = NULL );

	/** Add PlayerState to the inactive list, remove from the active list */
	ENGINE_API virtual void AddInactivePlayer(APlayerState* PlayerState, APlayerController* PC);

	/** Attempt to find and associate an inactive PlayerState with entering PC.  
	  * @Returns true if a PlayerState was found and associated with PC. */
	ENGINE_API virtual bool FindInactivePlayer(APlayerController* PC);

	/** Override PC's PlayerState with the values in OldPlayerState as part of the inactive player handling */
	ENGINE_API virtual void OverridePlayerState(APlayerController* PC, APlayerState* OldPlayerState);

	/** SetViewTarget of player control on server change */
	ENGINE_API virtual void SetSeamlessTravelViewTarget(APlayerController* PC);

	/**
	 * Called from CommitMapChange before unloading previous level. Used for asynchronous level streaming
	 * @param PreviousMapName - Name of the previous persistent level
	 * @param NextMapName - Name of the persistent level being streamed to
	 */
	ENGINE_API virtual void PreCommitMapChange(const FString& PreviousMapName, const FString& NextMapName);

	/** Called from CommitMapChange after unloading previous level and loading new level+sublevels. Used for asynchronous level streaming */
	ENGINE_API virtual void PostCommitMapChange();

	/** 
	 * Called when a connection closes before getting to PostLogin() 
	 *
	 * @param ConnectionUniqueId the unique id on the connection, if known (may be very early and impossible to know)
	 */
	ENGINE_API virtual void NotifyPendingConnectionLost(const FUniqueNetIdRepl& ConnectionUniqueId);

	/** Handles when a player is disconnected, before the session does */
	ENGINE_API virtual void HandleDisconnect(UWorld* InWorld, UNetDriver* NetDriver);

	//~ Begin AActor Interface
	ENGINE_API virtual void Tick(float DeltaSeconds) override;
	//~ End AActor Interface

	//~ Begin AGameModeBase Interface
	ENGINE_API virtual void InitGame(const FString& MapName, const FString& Options, FString& ErrorMessage) override;
	ENGINE_API virtual void StartPlay() override;
	ENGINE_API virtual bool HasMatchStarted() const override;
	/** Returns true if the match state is WaitingPostMatch or later */
	ENGINE_API virtual bool HasMatchEnded() const override;
	ENGINE_API virtual void PostLogin(APlayerController* NewPlayer) override;
	ENGINE_API virtual void Logout(AController* Exiting) override;
	ENGINE_API virtual int32 GetNumPlayers() override;
	ENGINE_API virtual int32 GetNumSpectators() override;
	ENGINE_API virtual bool IsHandlingReplays() override;
	ENGINE_API virtual void HandleStartingNewPlayer_Implementation(APlayerController* NewPlayer) override;
	ENGINE_API virtual bool PlayerCanRestart_Implementation(APlayerController* Player) override;
	ENGINE_API virtual void PostSeamlessTravel() override;
	ENGINE_API virtual void HandleSeamlessTravelPlayer(AController*& C) override;
	ENGINE_API virtual void InitSeamlessTravelPlayer(AController* NewController) override;
	ENGINE_API virtual bool CanServerTravel(const FString& URL, bool bAbsolute) override;
	ENGINE_API virtual void StartToLeaveMap() override;
	ENGINE_API virtual bool SpawnPlayerFromSimulate(const FVector& NewLocation, const FRotator& NewRotation) override;
	//~ End AGameModeBase Interface
};
