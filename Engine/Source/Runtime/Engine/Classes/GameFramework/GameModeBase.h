// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Info.h"
#include "Engine/ServerStatReplicator.h"
#include "Online/CoreOnline.h"
#include "Net/Core/Connection/NetEnums.h"
#include "GameFramework/PlayerController.h"
#include "GameModeBase.generated.h"

class AGameSession;
class AGameStateBase;
class AHUD;
class APlayerState;
class ASpectatorPawn;
class UNetConnection;
class UPlayer;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogGameMode, Log, All);

/** Default delegate that provides an implementation for those that don't have special needs other than a toggle */
DECLARE_DELEGATE_RetVal(bool, FCanUnpause);

/**
 * The GameModeBase defines the game being played. It governs the game rules, scoring, what actors
 * are allowed to exist in this game type, and who may enter the game.
 *
 * It is only instanced on the server and will never exist on the client. 
 *
 * A GameModeBase actor is instantiated when the level is initialized for gameplay in
 * C++ UGameEngine::LoadMap().  
 * 
 * The class of this GameMode actor is determined by (in order) either the URL ?game=xxx, 
 * the GameMode Override value set in the World Settings, or the DefaultGameMode entry set 
 * in the game's Project Settings.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/GameMode/index.html
 */
UCLASS(config = Game, notplaceable, BlueprintType, Blueprintable, Transient, hideCategories = (Info, Rendering, MovementReplication, Replication, Actor), meta = (ShortTooltip = "Game Mode Base defines the game being played, its rules, scoring, and other facets of the game type."), MinimalAPI)
class AGameModeBase : public AInfo
{
	GENERATED_UCLASS_BODY()

public:

	//~=============================================================================
	// Initializing the game

	/**
	 * Initialize the game.
	 * The GameMode's InitGame() event is called before any other functions (including PreInitializeComponents() )
	 * and is used by the GameMode to initialize parameters and spawn its helper classes.
	 * @warning: this is called before actors' PreInitializeComponents.
	 */
	ENGINE_API virtual void InitGame(const FString& MapName, const FString& Options, FString& ErrorMessage);

	/**
	 * Initialize the GameState actor with default settings
	 * called during PreInitializeComponents() of the GameMode after a GameState has been spawned
	 * as well as during Reset()
	 */
	ENGINE_API virtual void InitGameState();

	/** Save options string and parse it when needed */
	UPROPERTY(BlueprintReadOnly, Category=GameMode)
	FString OptionsString;


	//~=============================================================================
	// Accessors for classes spawned by game

	/** Return GameSession class to use for this game  */
	ENGINE_API virtual TSubclassOf<AGameSession> GetGameSessionClass() const;

	/** Returns default pawn class for given controller */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category=Classes)
	ENGINE_API UClass* GetDefaultPawnClassForController(AController* InController);

	/** Class of GameSession, which handles login approval and online game interface */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Classes)
	TSubclassOf<AGameSession> GameSessionClass;

	/** Class of GameState associated with this GameMode. */
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category=Classes)
	TSubclassOf<AGameStateBase> GameStateClass;

	/** The class of PlayerController to spawn for players logging in. */
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category=Classes)
	TSubclassOf<APlayerController> PlayerControllerClass;

	/** A PlayerState of this class will be associated with every player to replicate relevant player information to all clients. */
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category=Classes)
	TSubclassOf<APlayerState> PlayerStateClass;

	/** HUD class this game uses. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Classes, meta = (DisplayName = "HUD Class"))
	TSubclassOf<AHUD> HUDClass;

	/** The default pawn class used by players. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Classes)
	TSubclassOf<APawn> DefaultPawnClass;

	/** The pawn class used by the PlayerController for players when spectating. */
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category=Classes)
	TSubclassOf<ASpectatorPawn> SpectatorClass;

	/** The PlayerController class used when spectating a network replay. */
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category=Classes)
	TSubclassOf<APlayerController> ReplaySpectatorPlayerControllerClass;

	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category = Classes)
	TSubclassOf<AServerStatReplicator> ServerStatReplicatorClass;

	/** Return if the game mode is requesting a specific replication system to be used for the GameNetDriver */
	EReplicationSystem GetGameNetDriverReplicationSystem() const { return GameNetDriverReplicationSystem; }

	//~=============================================================================
	// Accessors for current state

	/** Game Session handles login approval, arbitration, online game interface */
	UPROPERTY(Transient)
	TObjectPtr<AGameSession> GameSession;

	/** GameState is used to replicate game state relevant properties to all clients. */
	UPROPERTY(Transient)
	TObjectPtr<AGameStateBase> GameState;

	/** Helper template to returns the current GameState casted to the desired type. */
	template< class T >
	T* GetGameState() const
	{
		return Cast<T>(GameState);
	}

	UPROPERTY(Transient)
	TObjectPtr<AServerStatReplicator> ServerStatReplicator;

	/** Returns number of active human players, excluding spectators */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual int32 GetNumPlayers();

	/** Returns number of human players currently spectating */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual int32 GetNumSpectators();


	//~=============================================================================
	// Starting/pausing/resetting the game

	/** Transitions to calls BeginPlay on actors. */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void StartPlay();

	/** Returns true if the match start callbacks have been called */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual bool HasMatchStarted() const;

	/** Returns true if the match can be considered ended */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual bool HasMatchEnded() const;

	/**
	 * Adds the delegate to the list if the player Controller has the right to pause
	 * the game. The delegate is called to see if it is ok to unpause the game, e.g.
	 * the reason the game was paused has been cleared.
	 * @param PC the player Controller to check for admin privs
	 * @param CanUnpauseDelegate the delegate to query when checking for unpause
	 */
	ENGINE_API virtual bool SetPause(APlayerController* PC, FCanUnpause CanUnpauseDelegate = FCanUnpause());

	/**
	 * Checks the list of delegates to determine if the pausing can be cleared. If
	 * the delegate says it's ok to unpause, that delegate is removed from the list
	 * and the rest are checked. The game is considered unpaused when the list is
	 * empty.
	 */
	ENGINE_API virtual bool ClearPause();

	/**
	 * Forcibly removes an object's CanUnpause delegates from the list of pausers.  If any of the object's CanUnpause delegate
	 * handlers were in the list, triggers a call to ClearPause().
	 *
	 * Called when the player controller is being destroyed to prevent the game from being stuck in a paused state when a PC that
	 * paused the game is destroyed before the game is unpaused.
	 */
	ENGINE_API void ForceClearUnpauseDelegates(AActor* PauseActor);

	/** Returns true if the player is allowed to pause the game */
	ENGINE_API virtual bool AllowPausing(APlayerController* PC = nullptr);

	/** Returns true if the game is paused */
	ENGINE_API virtual bool IsPaused() const;

	/**
	 * Overridable function to determine whether an Actor should have Reset called when the game has Reset called on it.
	 * Default implementation returns true
	 * @param ActorToReset The actor to make a determination for
	 * @return true if ActorToReset should have Reset() called on it while restarting the game,
	 *		   false if the GameMode will manually reset it or if the actor does not need to be reset
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API bool ShouldReset(AActor* ActorToReset);

	/**
	 * Overridable function called when resetting level. This is used to reset the game state while staying in the same map
	 * Default implementation calls Reset() on all actors except GameMode and Controllers
	 */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void ResetLevel();

	/** Return to main menu, and disconnect any players */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void ReturnToMainMenuHost();

	/** Returns true if allowed to server travel */
	ENGINE_API virtual bool CanServerTravel(const FString& URL, bool bAbsolute);

	/** Handles request for server to travel to a new URL, with all players */
	ENGINE_API virtual void ProcessServerTravel(const FString& URL, bool bAbsolute = false);

	/** 
	 * called on server during seamless level transitions to get the list of Actors that should be moved into the new level
	 * PlayerControllers, Role < ROLE_Authority Actors, and any non-Actors that are inside an Actor that is in the list
	 * (i.e. Object.Outer == Actor in the list)
	 * are all automatically moved regardless of whether they're included here
	 * only dynamic actors in the PersistentLevel may be moved (this includes all actors spawned during gameplay)
	 * this is called for both parts of the transition because actors might change while in the middle (e.g. players might join or leave the game)
	 * @see also PlayerController::GetSeamlessTravelActorList() (the function that's called on clients)
	 * @param bToTransition true if we are going from old level to transition map, false if we are going from transition map to new level
	 * @param ActorList (out) list of actors to maintain
	 */
	ENGINE_API virtual void GetSeamlessTravelActorList(bool bToTransition, TArray<AActor*>& ActorList);

	/**
	 * Used to swap a viewport/connection's PlayerControllers when seamless traveling and the new GameMode's
	 * controller class is different than the previous
	 * includes network handling
	 * @param OldPC - the old PC that should be discarded
	 * @param NewPC - the new PC that should be used for the player
	 */
	ENGINE_API virtual void SwapPlayerControllers(APlayerController* OldPC, APlayerController* NewPC);

	/**
	 * Gets the class that should be used for spawning a player controller during seamless travel
	 * @param PreviousPlayerController The player controller from the prior level
	 * @return The class that should be used for spawning the player controller
	 */
	ENGINE_API virtual TSubclassOf<APlayerController> GetPlayerControllerClassToSpawnForSeamlessTravel(APlayerController* PreviousPlayerController);

	/**
	 * Handles reinitializing players that remained through a seamless level transition
	 * called from C++ for players that finished loading after the server
	 * @param C the Controller to handle
	 */
	ENGINE_API virtual void HandleSeamlessTravelPlayer(AController*& C);

	/**
	 * Called after a seamless level transition has been completed on the *new* GameMode.
	 * Used to reinitialize players already in the game as they won't have *Login() called on them
	 */
	ENGINE_API virtual void PostSeamlessTravel();

	/** Start the transition out of the current map. Called at start of seamless travel, or right before map change for hard travel. */
	ENGINE_API virtual void StartToLeaveMap();


	//~=============================================================================
	// Player joining and leaving

	/** 
	 * Allows game to send network messages to provide more information to the client joining the game via NMT_GameSpecific
	 * (for example required DLC)
	 * the out string RedirectURL is built in and send automatically if only a simple URL is needed
	 */
	ENGINE_API virtual void GameWelcomePlayer(UNetConnection* Connection, FString& RedirectURL);

	/**
	 * Accept or reject a player attempting to join the server.  Fails login if you set the ErrorMessage to a non-empty string.
	 * PreLogin is called before Login.  Significant game time may pass before Login is called
	 *
	 * @param	Options					The URL options (e.g. name/spectator) the player has passed
	 * @param	Address					The network address of the player
	 * @param	UniqueId				The unique id the player has passed to the server
	 * @param	ErrorMessage			When set to a non-empty value, the player will be rejected using the error message set
	 */
	ENGINE_API virtual void PreLogin(const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, FString& ErrorMessage);

	DECLARE_DELEGATE_OneParam(FOnPreLoginCompleteDelegate, const FString& /*ErrorMsg*/);

	/**
	 * Async version of PreLogin that MUST call OnComplete when done, otherwise incoming client connections will hang or timeout.
	 * This allows servers to make requests to other services for verification of login credentials without blocking the game thread.
	 * By default, this calls PreLogin() and OnComplete immediately to not break backwards compatibility. When overriding this, be sure
	 * checks in PreLogin are also run, and that the GameModePreLoginEvent is broadcast.
	 */
	ENGINE_API virtual void PreLoginAsync(const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, const FOnPreLoginCompleteDelegate& OnComplete);

	/**
	 * Called to login new players by creating a player controller, overridable by the game
	 *
	 * Sets up basic properties of the player (name, unique id, registers with backend, etc) and should not be used to do
	 * more complicated game logic.  The player controller is not fully initialized within this function as far as networking is concerned.
	 * Save "game logic" for PostLogin which is called shortly afterward.
	 *
	 * @param NewPlayer pointer to the UPlayer object that represents this player (either local or remote)
	 * @param RemoteRole the remote role this controller has
	 * @param Portal desired portal location specified by the client
	 * @param Options game options passed in by the client at login
	 * @param UniqueId platform specific unique identifier for the logging in player
	 * @param ErrorMessage [out] error message, if any, why this login will be failing
	 *
	 * If login is successful, returns a new PlayerController to associate with this player. Login fails if ErrorMessage string is set.
	 *
	 * @return a new player controller for the logged in player, NULL if login failed for any reason
	 */
	ENGINE_API virtual APlayerController* Login(UPlayer* NewPlayer, ENetRole InRemoteRole, const FString& Portal, const FString& Options, const FUniqueNetIdRepl& UniqueId, FString& ErrorMessage);

	/** Called after a successful login.  This is the first place it is safe to call replicated functions on the PlayerController. */
	ENGINE_API virtual void PostLogin(APlayerController* NewPlayer);

	/** Called as part of the PostLogin process.  This is the last step before we spawn a new player. */
	UE_DEPRECATED(5.6, "This functionality has been moved into OnPostLogin. You cannot call that function directly, it is executed at the end of PostLogin.")
	ENGINE_API void DispatchPostLogin(AController* NewPlayer);

protected:
	/** Called as part of the PostLogin process.  This is the last step before we spawn a new player. */
	ENGINE_API virtual void OnPostLogin(AController* NewPlayer);

public:
	/** Notification that a player has successfully logged in, and has been given a player controller */
	UFUNCTION(BlueprintImplementableEvent, Category=Game, meta = (DisplayName = "OnPostLogin", ScriptName = "OnPostLogin"))
	ENGINE_API void K2_PostLogin(APlayerController* NewPlayer);

	/** Called when a Controller with a PlayerState leaves the game or is destroyed */
	ENGINE_API virtual void Logout(AController* Exiting);

	/** Implementable event when a Controller with a PlayerState leaves the game. */
	UFUNCTION(BlueprintImplementableEvent, Category=Game, meta = (DisplayName = "OnLogout", ScriptName = "OnLogout"))
	ENGINE_API void K2_OnLogout(AController* ExitingController);

	/**
	 * Spawns the appropriate PlayerController for the given options; split out from Login() for easier overriding.
	 * Override this to conditionally spawn specialized PlayerControllers, for instance.
	 *
	 * @param RemoteRole the role this controller will play remotely
	 * @param Options the options string from the new player's URL
	 *
	 * @return PlayerController for the player, NULL if there is any reason this player shouldn't exist or due to some error
	 */
	ENGINE_API virtual APlayerController* SpawnPlayerController(ENetRole InRemoteRole, const FString& Options);

	/** Signals that a player is ready to enter the game, which may start it up */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API void HandleStartingNewPlayer(APlayerController* NewPlayer);

	/** Returns true if NewPlayerController may only join the server as a spectator. */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API bool MustSpectate(APlayerController* NewPlayerController) const;

	/** Return whether Viewer is allowed to spectate from the point of view of ViewTarget. */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API bool CanSpectate(APlayerController* Viewer, APlayerState* ViewTarget);

	/** The default player name assigned to players that join with no name specified. */
	UPROPERTY(EditAnywhere, Category=Game)
	FText DefaultPlayerName;

	/** 
	 * Sets the name for a controller 
	 * @param Controller	The controller of the player to change the name of
	 * @param NewName		The name to set the player to
	 * @param bNameChange	Whether the name is changing or if this is the first time it has been set
	 */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void ChangeName(AController* Controller, const FString& NewName, bool bNameChange);

	/** 
	 * Overridable event for GameMode blueprint to respond to a change name call
	 * @param Controller	The controller of the player to change the name of
	 * @param NewName		The name to set the player to
	 * @param bNameChange	Whether the name is changing or if this is the first time it has been set
	 */
	UFUNCTION(BlueprintImplementableEvent,Category=Game,meta=(DisplayName="OnChangeName", ScriptName="OnChangeName"))
	ENGINE_API void K2_OnChangeName(AController* Other, const FString& NewName, bool bNameChange);


	//~=============================================================================
	// Spawning the player's pawn

	/**
	 * Return the 'best' player start for this player to spawn from
	 * Default implementation looks for a random unoccupied spot
	 * 
	 * @param Player is the controller for whom we are choosing a playerstart
	 * @returns AActor chosen as player start (usually a PlayerStart)
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API AActor* ChoosePlayerStart(AController* Player);

	/**
	 * Return the specific player start actor that should be used for the next spawn
	 * This will either use a previously saved startactor, or calls ChoosePlayerStart
	 * 
	 * @param Player The AController for whom we are choosing a Player Start
	 * @param IncomingName Specifies the tag of a Player Start to use
	 * @returns Actor chosen as player start (usually a PlayerStart)
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API AActor* FindPlayerStart(AController* Player, const FString& IncomingName = TEXT(""));

	/**
	 * Return the specific player start actor that should be used for the next spawn
	 * This will either use a previously saved startactor, or calls ChoosePlayerStart
	 *
	 * @param Player The AController for whom we are choosing a Player Start
	 * @param IncomingName Specifies the tag of a Player Start to use
	 * @returns Actor chosen as player start (usually a PlayerStart)
	 */
	UFUNCTION(BlueprintPure, Category=Game, meta = (DisplayName = "FindPlayerStart"))
	ENGINE_API AActor* K2_FindPlayerStart(AController* Player, const FString& IncomingName = TEXT(""));

	/** Returns true if it's valid to call RestartPlayer. By default will call Player->CanRestartPlayer */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category=Game)
	ENGINE_API bool PlayerCanRestart(APlayerController* Player);

	/** Tries to spawn the player's pawn, at the location returned by FindPlayerStart */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void RestartPlayer(AController* NewPlayer);

	/** Tries to spawn the player's pawn at the specified actor's location */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void RestartPlayerAtPlayerStart(AController* NewPlayer, AActor* StartSpot);

	/** Tries to spawn the player's pawn at a specific location */
	UFUNCTION(BlueprintCallable, Category=Game)
	ENGINE_API virtual void RestartPlayerAtTransform(AController* NewPlayer, const FTransform& SpawnTransform);

	/**
	 * Called during RestartPlayer to actually spawn the player's pawn, when using a start spot
	 * @param	NewPlayer - Controller for whom this pawn is spawned
	 * @param	StartSpot - Actor at which to spawn pawn
	 * @return	a pawn of the default pawn class
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API APawn* SpawnDefaultPawnFor(AController* NewPlayer, AActor* StartSpot);

	/**
	 * Called during RestartPlayer to actually spawn the player's pawn, when using a transform
	 * @param	NewPlayer - Controller for whom this pawn is spawned
	 * @param	SpawnTransform - Transform at which to spawn pawn
	 * @return	a pawn of the default pawn class
	 */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API APawn* SpawnDefaultPawnAtTransform(AController* NewPlayer, const FTransform& SpawnTransform);

	/** Called from RestartPlayerAtPlayerStart, can be used to initialize the start spawn actor */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API void InitStartSpot(AActor* StartSpot, AController* NewPlayer);

	/** Implementable event called at the end of RestartPlayer */
	UFUNCTION(BlueprintImplementableEvent, Category=Game, meta = (DisplayName = "OnRestartPlayer", ScriptName = "OnRestartPlayer"))
	ENGINE_API void K2_OnRestartPlayer(AController* NewPlayer);

	/** Initializes player pawn back to starting values, called from RestartPlayer */
	ENGINE_API virtual void SetPlayerDefaults(APawn* PlayerPawn);


	//~=============================================================================
	// Engine hooks

	/** Return true if player should be allowed to use cheats by default */
	ENGINE_API virtual bool AllowCheats(APlayerController* P);

	/** Returns true if replays will start/stop during gameplay starting/stopping */
	ENGINE_API virtual bool IsHandlingReplays();

	/** Used in the editor to spawn a PIE player after the game has already started */
	ENGINE_API virtual bool SpawnPlayerFromSimulate(const FVector& NewLocation, const FRotator& NewRotation);

	//~ Begin AActor Interface
	ENGINE_API virtual void PreInitializeComponents() override;
	ENGINE_API virtual void Reset() override;
	//~ End AActor Interface

protected:
	/**
	 *	Attempts to initialize the 'StartSpot' of the Player.
	 * 	@param	Player				The controller for the player.
	 *	@param	OutErrorMessage		Any error messages.
	 *	@return	bool	true if we updated the start spot, otherwise false.
	 */
	ENGINE_API virtual bool UpdatePlayerStartSpot(AController* Player, const FString& Portal, FString& OutErrorMessage);

	/**
	 *	Check to see if we should start in cinematic mode 
	 * 	@param	OutHidePlayer		Whether to hide the player
	 *	@param	OutHideHud			Whether to hide the HUD		
	 *	@param	OutDisableMovement	Whether to disable movement
	 * 	@param	OutDisableTurning	Whether to disable turning
	 *	@return	bool			true if we should turn on cinematic mode, 
	 *							false if if we should not.
	 */
	ENGINE_API virtual bool ShouldStartInCinematicMode(APlayerController* Player, bool& OutHidePlayer, bool& OutHideHud, bool& OutDisableMovement, bool& OutDisableTurning);

	/**
	 * Used to notify the game type that it is ok to update a player's gameplay
	 * specific muting information now. The playercontroller needs to notify
	 * the server when it is possible to do so or the unique net id will be
	 * incorrect and the muting not work.
	 *
	 * @param aPlayer the playercontroller that is ready for updates
	 */
	ENGINE_API virtual void UpdateGameplayMuteList(APlayerController* aPlayer);

	/**
	 * Customize incoming player based on URL options
	 *
	 * @param NewPlayerController player logging in
	 * @param UniqueId unique id for this player
	 * @param Options URL options that came at login
	 *
	 */
	ENGINE_API virtual FString InitNewPlayer(APlayerController* NewPlayerController, const FUniqueNetIdRepl& UniqueId, const FString& Options, const FString& Portal = TEXT(""));

	/** Initialize the AHUD object for a player. Games can override this to do something different */
	UFUNCTION(BlueprintNativeEvent, Category=Game)
	ENGINE_API void InitializeHUDForPlayer(APlayerController* NewPlayer);

	/**
	 * Handles all player initialization that is shared between the travel methods
	 * (i.e. called from both PostLogin() and HandleSeamlessTravelPlayer())
	 */
	ENGINE_API virtual void GenericPlayerInitialization(AController* C);

	/** Replicates the current level streaming status to the given PlayerController */
	ENGINE_API virtual void ReplicateStreamingStatus(APlayerController* PC);

	/** Return true if FindPlayerStart should use the StartSpot stored on Player instead of calling ChoosePlayerStart */
	ENGINE_API virtual bool ShouldSpawnAtStartSpot(AController* Player);

	/** Handles second half of RestartPlayer */
	ENGINE_API virtual void FinishRestartPlayer(AController* NewPlayer, const FRotator& StartRotation);

	/**
	 * Called in the event that we fail to spawn a controller's pawn, which maybe it didn't have one or maybe it tried
	 * to spawn and was destroyed due to collision.
	 */
	ENGINE_API virtual void FailedToRestartPlayer(AController* NewPlayer);

	/**
	 * Notifies all clients to travel to the specified URL.
	 *
	 * @param	URL				a string containing the mapname (or IP address) to travel to, along with option key/value pairs
	 * @param	bSeamless		indicates whether the travel should use seamless travel or not.
	 * @param	bAbsolute		indicates which type of travel the server will perform (i.e. TRAVEL_Relative or TRAVEL_Absolute)
	 */
	ENGINE_API virtual APlayerController* ProcessClientTravel(FString& URL, bool bSeamless, bool bAbsolute);

	/** Handles initializing a seamless travel player, handles logic similar to InitNewPlayer */
	ENGINE_API virtual void InitSeamlessTravelPlayer(AController* NewController);

	/** Called when a PlayerController is swapped to a new one during seamless travel */
	UFUNCTION(BlueprintImplementableEvent, Category=Game, meta=(DisplayName="OnSwapPlayerControllers", ScriptName="OnSwapPlayerControllers"))
	ENGINE_API void K2_OnSwapPlayerControllers(APlayerController* OldPC, APlayerController* NewPC);

	/** Does the work of spawning a player controller of the given class at the given transform. */
	ENGINE_API virtual APlayerController* SpawnPlayerControllerCommon(ENetRole InRemoteRole, FVector const& SpawnLocation, FRotator const& SpawnRotation, TSubclassOf<APlayerController> InPlayerControllerClass);

public:
	/** Whether the game perform map travels using SeamlessTravel() which loads in the background and doesn't disconnect clients */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=GameMode)
	uint32 bUseSeamlessTravel : 1;
protected:

	/** Whether players should immediately spawn when logging in, or stay as spectators until they manually spawn */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=GameMode)
	uint32 bStartPlayersAsSpectators : 1;

	/** Whether the game is pauseable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=GameMode)
	uint32 bPauseable : 1;

	/** 
	 * Can be used to request a specific replication system for a GameNetDriver that will replicate this game mode.
	 * Leave to Default to use the game engine's preferred system. 
	 * Useful when migrating from one repsystem to another and a game mode does not fully support both repsystem yet.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = GameMode)
	EReplicationSystem GameNetDriverReplicationSystem = EReplicationSystem::Default;

	/** The list of delegates to check before unpausing a game */
	TArray<FCanUnpause> Pausers;

private:
	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();
};

/** GameModeBase events, particularly for use by plugins */
class FGameModeEvents
{
public:

	/**
	 * GameMode initialization has occurred
	 * - Called at the end of AGameModeBase::InitGame 
	 * - AGameSession has also been initialized
	 * - Possible some child level initialization hasn't finished
	 *
	 * @param GameMode the game mode actor that has been initialized
	 */
	DECLARE_EVENT_OneParam(AGameModeBase, FGameModeInitializedEvent, AGameModeBase* /*GameMode*/);

	/**
	 * Client pre login event, triggered when a client first contacts a server
	 *
	 * @param GameMode the game mode actor that has been initialized
	 * @param NewPlayer the unique id of the player attempting to join
	 * @param ErrorMessage current state of any error messages, setting this value non empty will reject the player
	 */
	DECLARE_EVENT_ThreeParams(AGameModeBase, FGameModePreLoginEvent, AGameModeBase* /*GameMode*/, const FUniqueNetIdRepl& /*NewPlayer*/, FString& /*ErrorMessage*/);

	/** 
	 * Post login event, triggered when a player joins the game as well as after non-seamless ServerTravel
	 *
	 * This is called after the player has finished initialization
	 */
	DECLARE_EVENT_TwoParams(AGameModeBase, FGameModePostLoginEvent, AGameModeBase* /*GameMode*/, APlayerController* /*NewPlayer*/);

	/**
	 * Logout event, triggered when a player leaves the game as well as during non-seamless ServerTravel
	 *
	 * Note that this is called before performing any cleanup of the specified AController
	 */
	DECLARE_EVENT_TwoParams(AGameModeBase, FGameModeLogoutEvent, AGameModeBase* /*GameMode*/, AController* /*Exiting*/);

	/**
	 * Match state has changed via SetMatchState()
	 *
	 * @param MatchState new match state
	 */
	DECLARE_EVENT_OneParam(AGameModeBase, FGameModeMatchStateSetEvent, FName /*MatchState*/);

public: 
	
	static FGameModeInitializedEvent& OnGameModeInitializedEvent() { return GameModeInitializedEvent; } 
	static FGameModePreLoginEvent& OnGameModePreLoginEvent() { return GameModePreLoginEvent; }
	static FGameModePostLoginEvent& OnGameModePostLoginEvent() { return GameModePostLoginEvent; }
	static FGameModeLogoutEvent& OnGameModeLogoutEvent() { return GameModeLogoutEvent; }
	static FGameModeMatchStateSetEvent& OnGameModeMatchStateSetEvent() { return GameModeMatchStateSetEvent; }

	static ENGINE_API FGameModeInitializedEvent GameModeInitializedEvent;
	static ENGINE_API FGameModePreLoginEvent GameModePreLoginEvent;
	static ENGINE_API FGameModePostLoginEvent GameModePostLoginEvent;
	static ENGINE_API FGameModeLogoutEvent GameModeLogoutEvent;
	static ENGINE_API FGameModeMatchStateSetEvent GameModeMatchStateSetEvent;
};


