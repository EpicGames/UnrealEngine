// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Info.h"
#include "GameNetworkManager.generated.h"

/** Describes which standby detection event occured so the game can take appropriate action. */
UENUM()
enum EStandbyType : int
{
	STDBY_Rx,
	STDBY_Tx,
	STDBY_BadPing,
	STDBY_MAX,
};


/**
 * Handles game-specific networking management (cheat detection, bandwidth management, etc.).
 */

UCLASS(config=Game, notplaceable, MinimalAPI)
class AGameNetworkManager : public AInfo
{
	GENERATED_UCLASS_BODY()

	//======================================================================================================================
	// Analytics

	/* If packet loss goes over this value, we have bad packet loss. Value is between 0 and 1.*/
	UPROPERTY(GlobalConfig)
	float BadPacketLossThreshold = 0.05f;

	/* If the packet loss goes over this threshold, we have severe packet loss. Value is between 0 and 1*/
	UPROPERTY(GlobalConfig)
	float SeverePacketLossThreshold = 0.15f;

	/** If average ping is higher than this threshold in ms, determine the server is either delaying packets or has bad upstream. */
	UPROPERTY(GlobalConfig)
	int32 BadPingThreshold;

	/** Similar to BadPingThreshold, but used to track exceptionally bad pings. */
	UPROPERTY(GlobalConfig)
	int32 SeverePingThreshold;

	//======================================================================================================================
	// Listen/dedicated server dynamic bandwidth (NetSpeed) adjustment
	
	/** Current adjusted bandwidth per player, based on total and dynamic bandwidth */
	UPROPERTY()
	int32 AdjustedNetSpeed;

	/** Last time AdjustedNetSpeed was updated for server (by client entering or leaving) */
	UPROPERTY()
	float LastNetSpeedUpdateTime;

	/** Total available bandwidth (in bytes/sec) for listen server, split dynamically across net connections */
	UPROPERTY(globalconfig)
	int32 TotalNetBandwidth;

	/** Minimum bandwidth set per connection after splitting TotalNetBandwidth */
	UPROPERTY(globalconfig)
	int32 MinDynamicBandwidth;

	/** Maximum bandwidth set per connection after splitting TotalNetBandwidth */
	UPROPERTY(globalconfig)
	int32 MaxDynamicBandwidth;

	//======================================================================================================================
	// Standby cheat detection
	
	/** Used to determine if checking for standby cheats should occur */
	UPROPERTY(config)
	uint32 bIsStandbyCheckingEnabled:1;

	/** Used to determine whether we've already caught a cheat or not */
	UPROPERTY()
	uint32 bHasStandbyCheatTriggered:1;

	/** The amount of time without packets before triggering the cheat code */
	UPROPERTY(config)
	float StandbyRxCheatTime;

	/** The amount of time without packets before triggering the cheat code */
	UPROPERTY(config)
	float StandbyTxCheatTime;

	/** The percentage of clients missing RX data before triggering the standby code */
	UPROPERTY(config)
	float PercentMissingForRxStandby;

	/** The percentage of clients missing TX data before triggering the standby code */
	UPROPERTY(config)
	float PercentMissingForTxStandby;

	/** The percentage of clients with bad ping before triggering the standby code */
	UPROPERTY(config)
	float PercentForBadPing;

	/** The amount of time to wait before checking a connection for standby issues */
	UPROPERTY(config)
	float JoinInProgressStandbyWaitTime;

	//======================================================================================================================
	// Player replication
	
	/** Average size of replicated move packet (ServerMove() packet size) from player */
	UPROPERTY(GlobalConfig)
	float MoveRepSize;

	/** MAXPOSITIONERRORSQUARED is the square of the max position error that is accepted (not corrected) in net play */
	UPROPERTY(GlobalConfig)
	float MAXPOSITIONERRORSQUARED;

	/** MAXNEARZEROVELOCITYSQUARED is the square of the max velocity that is considered zero (not corrected) in net play */
	UPROPERTY(GlobalConfig)
	float MAXNEARZEROVELOCITYSQUARED;

	/** CLIENTADJUSTUPDATECOST is the bandwidth cost in bytes of sending a client adjustment update. 180 is greater than the actual cost, but represents a tweaked value reserving enough bandwidth for
	other updates sent to the client.  Increase this value to reduce client adjustment update frequency, or if the amount of data sent in the clientadjustment() call increases */
	UPROPERTY(GlobalConfig)
	float CLIENTADJUSTUPDATECOST;

	/** MAXCLIENTUPDATEINTERVAL is the maximum time between movement updates from the client before the server forces an update. */
	UPROPERTY(GlobalConfig)
	float MAXCLIENTUPDATEINTERVAL;

	/** MaxClientForcedUpdateDuration is the maximum time duration over which the server will force updates, after MAXCLIENTUPDATEINTERVAL is initially exceeded. */
	UPROPERTY(GlobalConfig)
	float MaxClientForcedUpdateDuration;
	
	/** Ignore forced client movement updates when server hitches for longer than this duration. */
	UPROPERTY(GlobalConfig)
	float ServerForcedUpdateHitchThreshold;

	/** Ignore forced client movement updates when server hitch was detected within this amount of time in the past. */
	UPROPERTY(GlobalConfig)
	float ServerForcedUpdateHitchCooldown;

	/** MaxMoveDeltaTime is the default maximum time delta of CharacterMovement ServerMoves. Should be less than or equal to MAXCLIENTUPDATEINTERVAL, otherwise server will interfere by forcing position updates. */
	UPROPERTY(GlobalConfig)
	float MaxMoveDeltaTime;

	/** MaxClientSmoothingDeltaTime is the maximum delta time between server updates that clients are allowed to smooth between for position interpolation. This was previously (2 * MaxMoveDeltaTime). */
	UPROPERTY(GlobalConfig)
	float MaxClientSmoothingDeltaTime;

	/**
	 * ClientNetSendMoveDeltaTime is the default minimum time delta of CharacterMovement client moves to the server. When updates occur more frequently, they may be combined to save bandwidth.
	 * This value is not used when player count is over ClientNetSendMoveThrottleOverPlayerCount or player net speed is <= ClientNetSendMoveThrottleAtNetSpeed (see ClientNetSendMoveDeltaTimeThrottled).
	 */
	UPROPERTY(GlobalConfig)
	float ClientNetSendMoveDeltaTime;

	/** ClientNetSendMoveDeltaTimeThrottled is used in place of ClientNetSendMoveDeltaTime when player count is high or net speed is low. See ClientNetSendMoveDeltaTime for more info. */
	UPROPERTY(GlobalConfig)
	float ClientNetSendMoveDeltaTimeThrottled;

	/** ClientNetSendMoveDeltaTimeStationary is used when players are determined to not be moving or changing their view. See ClientNetSendMoveDeltaTime for more info. */
	UPROPERTY(GlobalConfig)
	float ClientNetSendMoveDeltaTimeStationary;

	/** When player net speed (CurrentNetSpeed, based on ConfiguredInternetSpeed or ConfiguredLanSpeed) is less than or equal to this amount, ClientNetSendMoveDeltaTimeThrottled is used instead of ClientNetSendMoveDeltaTime. */
	UPROPERTY(GlobalConfig)
	int32 ClientNetSendMoveThrottleAtNetSpeed;

	/** When player count is greater than this amount, ClientNetSendMoveDeltaTimeThrottled is used instead of ClientNetSendMoveDeltaTime. */
	UPROPERTY(GlobalConfig)
	int32 ClientNetSendMoveThrottleOverPlayerCount;

	/** Minimum delay between the server sending error corrections to a client, in seconds. */
	UPROPERTY(GlobalConfig)
	float ClientErrorUpdateRateLimit;

	/** Minimum delay between calls to ServerUpdateCamera, in seconds. */
	UPROPERTY(GlobalConfig)
	float ClientNetCamUpdateDeltaTime;

	/** Camera position change limit, when exceeded allows an immediate ServerUpdateCamera call. */
	UPROPERTY(GlobalConfig)
	float ClientNetCamUpdatePositionLimit;

	/** If client update is within MAXPOSITIONERRORSQUARED of what the server expects then the client is authoritative on it's final position */
	UPROPERTY(GlobalConfig)
	bool ClientAuthorativePosition;

	//======================================================================================================================
	// Movement Time Discrepancy settings for Characters (speed hack detection and prevention)

	/** Whether movement time discrepancy (speed hack) detection is enabled. */
	UPROPERTY(GlobalConfig)
	bool bMovementTimeDiscrepancyDetection;

	/** Whether movement time discrepancy resolution is enabled (when detected, make client movement "pay back" excessive time discrepancies) */
	UPROPERTY(GlobalConfig)
	bool bMovementTimeDiscrepancyResolution;

	/** Maximum time client can be ahead before triggering movement time discrepancy detection/resolution (if enabled). */
	UPROPERTY(GlobalConfig)
	float MovementTimeDiscrepancyMaxTimeMargin;

	/** Maximum time client can be behind. */
	UPROPERTY(GlobalConfig)
	float MovementTimeDiscrepancyMinTimeMargin;

	/** 
	 * During time discrepancy resolution, we "pay back" the time discrepancy at this rate for future moves until total error is zero.
	 * 1.0 = 100% resolution rate, meaning the next X ServerMoves from the client are fully paying back the time, 
	 * 0.5 = 50% resolution rate, meaning future ServerMoves will spend 50% of tick continuing to move the character and 50% paying back.
	 * Lowering from 100% could be used to produce less severe/noticeable corrections, although typically we would want to correct
	 * the client as quickly as possible.
	 */
	UPROPERTY(GlobalConfig)
	float MovementTimeDiscrepancyResolutionRate;

	/** 
	 * Accepted drift in clocks between client and server as a percent per second allowed. 
	 *
	 * 0.0 is "no forgiveness" and all logic would run on raw values, no tampering on the server side.
	 * 0.02 would be a 2% per second difference "forgiven" - if the time discrepancy in a given second was less than 2%,
	 * the error handling/detection code effectively ignores it.
	 * 
	 * Increasing this value above 0% lessens the chance of false positives on time discrepancy (burst packet loss, performance
	 * hitches), but also means anyone tampering with their client time below that percent will not be detected and no resolution
	 * action will be taken, and anyone above that threshold will still gain the advantage of this % of time boost (if running at 
	 * 10% speed-up and this value is 0.05 or 5% allowance, they would only be resolved down to a 5% speed boost).
	 *
	 * Time discrepancy detection code DOES keep track of LifetimeRawTimeDiscrepancy, which is unaffected by this drift allowance,
	 * so cheating below DriftAllowance percent could be tracked and acted on outside of an individual game. For example, if DriftAllowance
	 * was 0.05 (meaning we're not going to actively prevent any cheating below 5% boosts to ensure less false positives for normal players),
	 * we could still post-process analytics of the game showing that Player X regularly runs at 4% speed boost and take action.
	 */
	UPROPERTY(GlobalConfig)
	float MovementTimeDiscrepancyDriftAllowance;

	/** 
	 * Whether client moves should be force corrected during time discrepancy resolution, useful for projects that have lenient 
	 * move error tolerance/ClientAuthorativePosition enabled.
	 */
	UPROPERTY(GlobalConfig)
	bool bMovementTimeDiscrepancyForceCorrectionsDuringResolution;

	/**  Update network speeds for listen servers based on number of connected players.  */
	ENGINE_API virtual void UpdateNetSpeeds(bool bIsLanMatch);

	/** Timer which calls UpdateNetSpeeds() once a second. */
	ENGINE_API virtual void UpdateNetSpeedsTimer();

	/** Returns true if we should be in low bandwidth mode */
	ENGINE_API virtual bool IsInLowBandwidthMode();

	//======================================================================================================================
	// Player replication

	/** @return true if last player client to server update was sufficiently recent.  Used to limit frequency of corrections if connection speed is limited. */
	ENGINE_API virtual bool WithinUpdateDelayBounds(class APlayerController* PC, float LastUpdateTime) const;

	/** @return true if position error exceeds max allowable amount */
	ENGINE_API virtual bool ExceedsAllowablePositionError(FVector LocDiff) const;

	/** @return true if velocity vector passed in is considered near zero for networking purposes */
	ENGINE_API virtual bool NetworkVelocityNearZero(FVector InVelocity) const;
	ENGINE_API virtual void PostInitializeComponents() override;

	/** @RETURN new per/client bandwidth given number of players in the game */
	ENGINE_API virtual int32 CalculatedNetSpeed();
	
	//======================================================================================================================
	// Standby cheat detection
	/**
	 * Turns standby detection on/off
	 * @param bIsEnabled true to turn it on, false to disable it
	 */
	ENGINE_API virtual void EnableStandbyCheatDetection(bool bIsEnabled);

	/**
	 * Notifies the game code that a standby cheat was detected
	 * @param StandbyType the type of cheat detected
	 */
	ENGINE_API virtual void StandbyCheatDetected(EStandbyType StandbyType);

	/** If true, actor network relevancy is constrained by whether they are within their NetCullDistanceSquared from the client's view point. */
	UPROPERTY(globalconfig)
	bool	bUseDistanceBasedRelevancy;

protected:

	/** Handle for efficient management of UpdateNetSpeeds timer */
	FTimerHandle TimerHandle_UpdateNetSpeedsTimer;	
};



