// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Multiplayer game session.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Online/CoreOnline.h"
#include "GameFramework/Info.h"
#include "GameSession.generated.h"

class APlayerController;
class Error;
struct FUniqueNetIdRepl;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogGameSession, Log, All);

/**
Acts as a game-specific wrapper around the session interface. The game code makes calls to this when it needs to interact with the session interface.
A game session exists only the server, while running an online game.
*/
UCLASS(config=Game, notplaceable, MinimalAPI)
class AGameSession : public AInfo
{
	GENERATED_UCLASS_BODY()

	/** Maximum number of spectators allowed by this server. */
	UPROPERTY(globalconfig)
	int32 MaxSpectators;

	/** Maximum number of players allowed by this server. */
	UPROPERTY(globalconfig)
	int32 MaxPlayers;

	/** Restrictions on the largest party that can join together */
	UPROPERTY()
	int32 MaxPartySize;

	/** Maximum number of splitscreen players to allow from one connection */
	UPROPERTY(globalconfig)
	uint8 MaxSplitscreensPerConnection;

    /** Is voice enabled always or via a push to talk keybinding */
	UPROPERTY(globalconfig)
	bool bRequiresPushToTalk;

	/** SessionName local copy from PlayerState class.  should really be define in this class, but need to address replication issues */
	UPROPERTY()
	FName SessionName;

	/** Initialize options based on passed in options string */
	ENGINE_API virtual void InitOptions(const FString& Options);

	/** @return A new unique player ID */
	ENGINE_API int32 GetNextPlayerID();

	//=================================================================================
	// LOGIN

	/** 
	 * Allow an online service to process a login if specified on the commandline with -auth_login/-auth_password
	 * @return true if login is in progress, false otherwise
	 */
	ENGINE_API virtual bool ProcessAutoLogin();

    /** Delegate triggered on auto login completion */
	ENGINE_API virtual void OnAutoLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& Error);

	/** 
	 * Called from GameMode.PreLogin() and Login().
	 * @param	Options	The URL options (e.g. name/spectator) the player has passed
	 * @return	Non-empty Error String if player not approved
	 */
	ENGINE_API virtual FString ApproveLogin(const FString& Options);

	/**
	 * Register a player with the online service session
	 * @param NewPlayer player to register
	 * @param UniqueId uniqueId they sent over on Login
	 * @param bWasFromInvite was this from an invite
	 */
	ENGINE_API virtual void RegisterPlayer(APlayerController* NewPlayer, const FUniqueNetIdRepl& UniqueId, bool bWasFromInvite);

	/**
	 * Called by GameMode::PostLogin to give session code chance to do work after PostLogin
	 *
	 * @param NewPlayer player logging in
	 */
	ENGINE_API virtual void PostLogin(APlayerController* NewPlayer);

	/** @return true if there is no room on the server for an additional player */
	ENGINE_API virtual bool AtCapacity(bool bSpectator);

	//=================================================================================
	// LOGOUT

	/**
	 * Called when a PlayerController logs out of game.
	 *
	 * @param PC player controller currently logging out 
	 */
	ENGINE_API virtual void NotifyLogout(const APlayerController* PC);

	/**
	 * Called when a player logs out of game.
	 *
	 * @param SessionName session related to the log out
	 * @param UniqueId unique id of the player logging out
	 */
	ENGINE_API virtual void NotifyLogout(FName InSessionName, const FUniqueNetIdRepl& UniqueId);

	/**
	 * Unregister a player from the online service session
	 *
	 * @param SessionName name of session to unregister from
	 * @param UniqueId id of the player to unregister
	 */
	ENGINE_API virtual void UnregisterPlayer(FName InSessionName, const FUniqueNetIdRepl& UniqueId);

	ENGINE_API virtual void UnregisterPlayers(FName InSessionName, const TArray< FUniqueNetIdRepl >& Players);
	
	/**
	 * Unregister a player from the online service session
	 *
	 * @param ExitingPlayer the player to unregister
	 */
	ENGINE_API virtual void UnregisterPlayer(const APlayerController* ExitingPlayer);

	/**
	 * Add a player to the admin list of this session
	 *
	 * @param AdminPlayer player to add to the list
	 */
	ENGINE_API virtual void AddAdmin(APlayerController* AdminPlayer);

	/**
	 * Remove a player from the admin list of this session
	 *
	 * @param AdminPlayer player to remove from the list
	 */
	ENGINE_API virtual void RemoveAdmin(APlayerController* AdminPlayer);

	/** 
	 * Forcibly remove player from the server
	 *
	 * @param KickedPlayer player to kick
	 * @param KickReason text reason to display to player
	 *
	 * @return true if player was able to be kicked, false otherwise
	 */
	ENGINE_API virtual bool KickPlayer(APlayerController* KickedPlayer, const FText& KickReason);

	/**
	 * Forcibly remove player from the server and ban them permanently
	 *
	 * @param BannedPlayer player to ban
	 * @param KickReason text reason to display to player
	 *
	 * @return true if player was able to be banned, false otherwise
	 */
	ENGINE_API virtual bool BanPlayer(APlayerController* BannedPlayer, const FText& BanReason);

	/** Gracefully tell all clients then local players to return to lobby */
	ENGINE_API virtual void ReturnToMainMenuHost();

	/** 
	 * called after a seamless level transition has been completed on the *new* GameMode
	 * used to reinitialize players already in the game as they won't have *Login() called on them
	 */
	ENGINE_API virtual void PostSeamlessTravel();

	//=================================================================================
	// SESSION INFORMATION

	/** Restart the session	 */
	virtual void Restart() {}

	/** Allow a dedicated server a chance to register itself with an online service */
	ENGINE_API virtual void RegisterServer();

	/** Callback when autologin was expected but failed */
	ENGINE_API virtual void RegisterServerFailed();

	/**
	 * Get the current joinability settings for a given session
	 * 
	 * @param session to query
	 * @param OutSettings [out] struct that will be filled in with current settings
	 * 
	 * @return true if session exists and data is valid, false otherwise
	 */
	ENGINE_API virtual bool GetSessionJoinability(FName InSessionName, FJoinabilitySettings& OutSettings);

	/**
	 * Update session join parameters
	 *
	 * @param SessionName name of session to update
	 * @param bPublicSearchable can the game be found via matchmaking
	 * @param bAllowInvites can you invite friends
	 * @param bJoinViaPresence anyone who can see you can join the game
	 * @param bJoinViaPresenceFriendsOnly can only friends actively join your game 
	 */
	ENGINE_API virtual void UpdateSessionJoinability(FName InSessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly);

    /**
     * Does the session require push to talk
     * @return true if a push to talk keybinding is required or if voice is always enabled
     */
	virtual bool RequiresPushToTalk() const { return bRequiresPushToTalk; }

	/** Dump session info to log for debugging.	  */
	ENGINE_API virtual void DumpSessionState();

	//=================================================================================
	// MATCH INTERFACE

	/** @RETURNS true if GameSession handled the request, in case it wants to stall for some reason. Otherwise, game mode will start immediately */
	ENGINE_API virtual bool HandleStartMatchRequest();

	/** Handle when the match enters waiting to start */
	ENGINE_API virtual void HandleMatchIsWaitingToStart();

	/** Handle when the match has started */
	ENGINE_API virtual void HandleMatchHasStarted();

	/** Handle when the match has completed */
	ENGINE_API virtual void HandleMatchHasEnded();

	/** Called from GameMode.RestartGame(). */
	ENGINE_API virtual bool CanRestartGame();

private:
	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();

protected:
	/**
	 * Delegate called when StartSession has completed
	 *
	 * @param InSessionName name of session involved
	 * @param bWasSuccessful true if the call was successful, false otherwise
	 */
	ENGINE_API virtual void OnStartSessionComplete(FName InSessionName, bool bWasSuccessful);

	/**
	 * Delegate called when EndSession has completed
	 *
	 * @param InSessionName name of session involved
	 * @param bWasSuccessful true if the call was successful, false otherwise
	 */
	ENGINE_API virtual void OnEndSessionComplete(FName InSessionName, bool bWasSuccessful);

	/**
	 * PostReloadConfig override to reapply config property overrides.
	 */
	ENGINE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;

private:

	/** Override for the default value of MaxPlayers passed to InitOptions. */
	TOptional<int32> MaxPlayersOptionOverride;

	/** Override for the default value of MaxSpectators passed to InitOptions. */
	TOptional<int32> MaxSpectatorsOptionOverride;
};

/**
 * Returns the player controller associated with this net id
 * @param PlayerNetId the id to search for
 * @return the player controller if found, otherwise NULL
 */
ENGINE_API APlayerController* GetPlayerControllerFromNetId(UWorld* World, const FUniqueNetIdRepl& PlayerNetId);


