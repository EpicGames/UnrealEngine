// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/GameStateBase.h"
#include "GameState.generated.h"

/**
 * GameState is a subclass of GameStateBase that behaves like a multiplayer match-based game.
 * It is tied to functionality in GameMode.
 */
UCLASS(MinimalAPI)
class AGameState : public AGameStateBase
{
	GENERATED_UCLASS_BODY()

	// Code to deal with the match state machine

	/** Returns the current match state, this is an accessor to protect the state machine flow */
	FName GetMatchState() const { return MatchState; }

	/** Returns true if we're in progress */
	ENGINE_API virtual bool IsMatchInProgress() const;

	/** Updates the match state and calls the appropriate transition functions, only valid on server */
	ENGINE_API void SetMatchState(FName NewState);

protected:

	/** What match state we are currently in */
	UPROPERTY(ReplicatedUsing=OnRep_MatchState, BlueprintReadOnly, VisibleInstanceOnly, Category = GameState)
	FName MatchState;

	/** Previous map state, used to handle if multiple transitions happen per frame */
	UPROPERTY(BlueprintReadOnly, VisibleInstanceOnly, Category = GameState)
	FName PreviousMatchState;

	/** Called when the state transitions to WaitingToStart */
	ENGINE_API virtual void HandleMatchIsWaitingToStart();

	/** Called when the state transitions to InProgress */
	ENGINE_API virtual void HandleMatchHasStarted();

	/** Called when the map transitions to WaitingPostMatch */
	ENGINE_API virtual void HandleMatchHasEnded();

	/** Called when the match transitions to LeavingMap */
	ENGINE_API virtual void HandleLeavingMap();

public:

	/** Elapsed game time since match has started. */
	UPROPERTY(replicatedUsing=OnRep_ElapsedTime, BlueprintReadOnly, Category = GameState)
	int32 ElapsedTime;

	/** Match state has changed */
	UFUNCTION()
	ENGINE_API virtual void OnRep_MatchState();

	/** Gives clients the chance to do something when time gets updates */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ElapsedTime();

	/** Called periodically, overridden by subclasses */
	ENGINE_API virtual void DefaultTimer();

	//~ Begin AActor Interface
	ENGINE_API virtual void PostInitializeComponents() override;
	//~ End AActor Interface

	//~ Begin AGameStateBase Interface
	ENGINE_API virtual void ReceivedGameModeClass() override;
	ENGINE_API virtual bool HasMatchStarted() const override;
	/** Returns true if match is WaitingPostMatch or later */
	ENGINE_API virtual bool HasMatchEnded() const override;
	ENGINE_API virtual void HandleBeginPlay() override;
	ENGINE_API virtual float GetPlayerStartTime(class AController* Controller) const override;
	ENGINE_API virtual float GetPlayerRespawnDelay(class AController* Controller) const override;
	//~ End AGameStateBase Interface

protected:

	/** Handle for efficient management of DefaultTimer timer */
	FTimerHandle TimerHandle_DefaultTimer;

};



