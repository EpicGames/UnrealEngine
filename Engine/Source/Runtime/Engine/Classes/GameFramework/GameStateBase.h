// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Info.h"
#include "GameFramework/GameModeBase.h"
#include "Stats/Stats.h"
#include "GameStateBase.generated.h"

class APlayerState;
class ASpectatorPawn;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogGameState, Log, All);

DECLARE_CYCLE_STAT_EXTERN( TEXT( "GetPlayerStateFromUniqueId" ), STAT_GetPlayerStateFromUniqueId, STATGROUP_Game , ENGINE_API);

class AGameModeBase;
class ASpectatorPawn;
class APlayerState;
class AController;

/**
 * GameStateBase is a class that manages the game's global state, and is spawned by GameModeBase.
 * It exists on both the client and the server and is fully replicated.
 */
UCLASS(config=Game, notplaceable, BlueprintType, Blueprintable, MinimalAPI)
class AGameStateBase : public AInfo
{
	GENERATED_UCLASS_BODY()

public:

	//~=============================================================================
	// General accessors and variables

	/** Class of the server's game mode, assigned by GameModeBase. */
	UPROPERTY(Transient, BlueprintReadOnly, Category=GameState, ReplicatedUsing=OnRep_GameModeClass)
	TSubclassOf<AGameModeBase>  GameModeClass;

	/** Instance of the current game mode, exists only on the server. For non-authority clients, this will be NULL. */
	UPROPERTY(Transient, BlueprintReadOnly, Category=GameState)
	TObjectPtr<AGameModeBase> AuthorityGameMode;

	/** Class used by spectators, assigned by GameModeBase. */
	UPROPERTY(Transient, BlueprintReadOnly, Category=GameState, ReplicatedUsing=OnRep_SpectatorClass)
	TSubclassOf<ASpectatorPawn> SpectatorClass;

	/** Array of all PlayerStates, maintained on both server and clients (PlayerStates are always relevant) */
	UPROPERTY(Transient, BlueprintReadOnly, Category=GameState)
	TArray<TObjectPtr<APlayerState>> PlayerArray;

	/** Allow game states to react to asset packages being loaded asynchronously */
	virtual void AsyncPackageLoaded(UObject* Package) {}

	/** Helper to return the default object of the GameModeBase class corresponding to this GameState. This object is not safe to modify. */
	ENGINE_API const AGameModeBase* GetDefaultGameMode() const;

	/** Helper template to returns the GameModeBase default object cast to the right type */
	template< class T >
	const T* GetDefaultGameMode() const
	{
		return Cast<T>(GetDefaultGameMode());
	}

	/** Returns the simulated TimeSeconds on the server, will be synchronized on client and server */
	UFUNCTION(BlueprintCallable, Category=GameState)
	ENGINE_API virtual double GetServerWorldTimeSeconds() const;

	/** Returns true if the world has started play (called BeginPlay on actors) */
	UFUNCTION(BlueprintCallable, Category=GameState)
	ENGINE_API virtual bool HasBegunPlay() const;

	/** Returns true if the world has started match (called MatchStarted callbacks) */
	UFUNCTION(BlueprintCallable, Category=GameState)
	ENGINE_API virtual bool HasMatchStarted() const;

	/** Returns true if the match can be considered ended. Defaults to false. */
	UFUNCTION(BlueprintCallable, Category = Game)
	ENGINE_API virtual bool HasMatchEnded() const;

	/** Returns the time that should be used as when a player started */
	UFUNCTION(BlueprintCallable, Category=GameState)
	ENGINE_API virtual float GetPlayerStartTime(AController* Controller) const;

	/** Returns how much time needs to be spent before a player can respawn */
	UFUNCTION(BlueprintCallable, Category=GameState)
	ENGINE_API virtual float GetPlayerRespawnDelay(AController* Controller) const;

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > &OutLifetimeProps) const;

	/** Returns the player state for a specified unique player Id */
	ENGINE_API APlayerState* GetPlayerStateFromUniqueNetId(const FUniqueNetIdWrapper& InPlayerId) const;

	//~=============================================================================
	// Interaction with GameModeBase

	/** Called when the GameClass property is set (at startup for the server, after the variable has been replicated on clients) */
	ENGINE_API virtual void ReceivedGameModeClass();

	/** Called when the SpectatorClass property is set (at startup for the server, after the variable has been replicated on clients) */
	ENGINE_API virtual void ReceivedSpectatorClass();

	/** Called during seamless travel transition twice (once when the transition map is loaded, once when destination map is loaded) */
	ENGINE_API virtual void SeamlessTravelTransitionCheckpoint(bool bToTransitionMap);

	/** Add PlayerState to the PlayerArray */
	ENGINE_API virtual void AddPlayerState(APlayerState* PlayerState);

	/** Remove PlayerState from the PlayerArray. */
	ENGINE_API virtual void RemovePlayerState(APlayerState* PlayerState);

	/** Called by game mode to set the started play bool */
	ENGINE_API virtual void HandleBeginPlay();

	//~ Begin AActor Interface
	ENGINE_API virtual void PostInitializeComponents() override;
	//~ End AActor Interface

protected:

	/** GameModeBase class notification callback. */
	UFUNCTION()
	ENGINE_API virtual void OnRep_GameModeClass();

	/** Callback when we receive the spectator class */
	UFUNCTION()
	ENGINE_API virtual void OnRep_SpectatorClass();

	/** By default calls BeginPlay and StartMatch */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedHasBegunPlay();

	/** Replicated when GameModeBase->StartPlay has been called so the client will also start play */
	UPROPERTY(Transient, ReplicatedUsing = OnRep_ReplicatedHasBegunPlay)
	bool bReplicatedHasBegunPlay;

	/** Called periodically to update ReplicatedWorldTimeSecondsDouble */
	ENGINE_API virtual void UpdateServerTimeSeconds();

	/** Allows clients to calculate ServerWorldTimeSecondsDelta */
	UFUNCTION()
	ENGINE_API virtual void OnRep_ReplicatedWorldTimeSecondsDouble();

	/** Server TimeSeconds. Useful for syncing up animation and gameplay. */
	UPROPERTY(Transient, ReplicatedUsing = OnRep_ReplicatedWorldTimeSecondsDouble)
	double ReplicatedWorldTimeSecondsDouble;

	/** The difference from the local world's TimeSeconds and the server world's TimeSeconds. */
	UPROPERTY(Transient)
	float ServerWorldTimeSecondsDelta;

	/** How often the server updates the replicated TimeSeconds from the world. Values mean "update every X seconds", so 0.5 means update every half second. Set to zero to disable periodic updates. */
	UPROPERTY(EditDefaultsOnly, Category=GameState)
	float ServerWorldTimeSecondsUpdateFrequency;

	/** Handle for efficient management of the UpdateServerTimeSeconds timer */
	FTimerHandle TimerHandle_UpdateServerTimeSeconds;

	/** Cumulative sum of computed server world time deltas for smoothed-averaging */
	double SumServerWorldTimeSecondsDelta;
	/** The number of server world time deltas accumulated in SumServerWorldTimeSecondsDelta - used for computing the mean */
	uint32 NumServerWorldTimeSecondsDeltas;

private:
	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();

	friend class UDemoNetDriver;
	friend class FReplayHelper;
};



