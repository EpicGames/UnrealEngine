// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "GenericPlatform/GenericWindow.h"
#include "Scalability.h"
#include "GameUserSettings.generated.h"

#if !CPP      //noexport class

/** Supported windowing modes (mirrored from GenericWindow.h) */
UENUM(BlueprintType)
namespace EWindowMode
{
	enum Type : int
	{
		/** The window is in true fullscreen mode */
		Fullscreen,
		/** The window has no border and takes up the entire area of the screen */
		WindowedFullscreen,
		/** The window has a border and may not take up the entire screen area */
		Windowed,
	};
}

#endif

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnGameUserSettingsUINeedsUpdate);

/**
 * Stores user settings for a game (for example graphics and sound settings), with the ability to save and load to and from a file.
 */
UCLASS(config=GameUserSettings, configdonotcheckdefaults, MinimalAPI)
class UGameUserSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** @return the location where GGameUserSettingsIni will be found */
	static ENGINE_API FString GetConfigDir();

	/** Applies all current user settings to the game and saves to permanent storage (e.g. file), optionally checking for command line overrides. */
	UFUNCTION(BlueprintCallable, Category=Settings, meta=(bCheckForCommandLineOverrides=true))
	ENGINE_API virtual void ApplySettings(bool bCheckForCommandLineOverrides);
	
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void ApplyNonResolutionSettings();

	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void ApplyResolutionSettings(bool bCheckForCommandLineOverrides);

	/** Returns the user setting for game screen resolution, in pixels. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API FIntPoint GetScreenResolution() const;

	/** Returns the last confirmed user setting for game screen resolution, in pixels. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API FIntPoint GetLastConfirmedScreenResolution() const;

	/** Returns user's desktop resolution, in pixels. */
	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API FIntPoint GetDesktopResolution() const;

	/** Sets the user setting for game screen resolution, in pixels. */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetScreenResolution(FIntPoint Resolution);

	/** Returns the user setting for game window fullscreen mode. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API EWindowMode::Type GetFullscreenMode() const;

	/** Returns the last confirmed user setting for game window fullscreen mode. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API EWindowMode::Type GetLastConfirmedFullscreenMode() const;

	/** Sets the user setting for the game window fullscreen mode. See UGameUserSettings::FullscreenMode. */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetFullscreenMode(EWindowMode::Type InFullscreenMode);

	/** Returns the user setting for game window fullscreen mode. */
	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API EWindowMode::Type GetPreferredFullscreenMode() const;

	/** Returns the user setting for game window display ID. See UGameUserSettings::DisplayID. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API FString GetDisplayID() const;

	/** Returns the last confirmed user setting for game window display ID. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API FString GetLastConfirmedDisplayID() const;

	/** Returns the user setting for game window display index. See UGameUserSettings::DisplayIndex. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API int32 GetDisplayIndex() const;

	/** Returns the last confirmed user setting for game window display index. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API int32 GetLastConfirmedDisplayIndex() const;

	/** Returns the closest monitor index based on display ID and display index. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API int32 GetClosestMonitorIndex() const;

	/** Sets the user setting for the game window display. */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetDisplayProperties(const FString& ID, int32 Index);

	/** Sets the user setting for vsync. See UGameUserSettings::bUseVSync. */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetVSyncEnabled(bool bEnable);

	/** Returns the user setting for vsync. */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API bool IsVSyncEnabled() const;

	/** Sets the user setting for dynamic resolution. See UGameUserSettings::bUseDynamicResolution. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetDynamicResolutionEnabled(bool bEnable);

	/** Returns the user setting for dynamic resolution. */
	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API virtual bool IsDynamicResolutionEnabled() const;

	/** Checks if the Screen Resolution user setting is different from current */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API bool IsScreenResolutionDirty() const;

	/** Checks if the FullscreenMode user setting is different from current */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API bool IsFullscreenModeDirty() const;

	/** Checks if the vsync user setting is different from current system setting */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API bool IsVSyncDirty() const;

	/** Checks if the dynamic resolution user setting is different from current system setting */
	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API bool IsDynamicResolutionDirty() const;

	/** Mark current video mode settings (fullscreenmode/resolution) as being confirmed by the user */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void ConfirmVideoMode();

	/** Revert video mode (fullscreenmode/resolution) back to the last user confirmed values */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void RevertVideoMode();

	/** Set scalability settings to sensible fallback values, for use when the benchmark fails or potentially causes a crash */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetBenchmarkFallbackValues();

	/** Sets the user's audio quality level setting */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetAudioQualityLevel(int32 QualityLevel);

	/** Returns the user's audio quality level setting */
	UFUNCTION(BlueprintPure, Category=Settings)
	int32 GetAudioQualityLevel() const { return AudioQualityLevel; }

	/** Sets the user's frame rate limit (0 will disable frame rate limiting) */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetFrameRateLimit(float NewLimit);

	/** Gets the user's frame rate limit (0 indiciates the frame rate limit is disabled) */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API float GetFrameRateLimit() const;

	// Changes all scalability settings at once based on a single overall quality level
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void SetOverallScalabilityLevel(int32 Value);

	// Returns the overall scalability level (can return -1 if the settings are custom)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual int32 GetOverallScalabilityLevel() const;

	// Returns the current resolution scale and the range
	UFUNCTION(BlueprintCallable, Category=Settings, meta=(DisplayName="GetResolutionScaleInformation"))
	ENGINE_API void GetResolutionScaleInformationEx(float& CurrentScaleNormalized, float& CurrentScaleValue, float& MinScaleValue, float& MaxScaleValue) const;

	// Gets the current resolution scale as a normalized 0..1 value between MinScaleValue and MaxScaleValue
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API float GetResolutionScaleNormalized() const;

	// Sets the current resolution scale
	UFUNCTION(BlueprintCallable, Category=Settings, meta=(DisplayName="SetResolutionScaleValue"))
	ENGINE_API void SetResolutionScaleValueEx(float NewScaleValue);

	// Sets the current resolution scale as a normalized 0..1 value between MinScaleValue and MaxScaleValue
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetResolutionScaleNormalized(float NewScaleNormalized);

	// Sets the view distance quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetViewDistanceQuality(int32 Value);

	// Returns the view distance quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetViewDistanceQuality() const;

	// Sets the shadow quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetShadowQuality(int32 Value);

	// Returns the shadow quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetShadowQuality() const;

	// Sets the global illumination quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetGlobalIlluminationQuality(int32 Value);

	// Returns the global illumination quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetGlobalIlluminationQuality() const;

	// Sets the reflection quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetReflectionQuality(int32 Value);

	// Returns the reflection quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetReflectionQuality() const;

	// Sets the anti-aliasing quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetAntiAliasingQuality(int32 Value);

	// Returns the anti-aliasing quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetAntiAliasingQuality() const;

	// Sets the texture quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic  (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetTextureQuality(int32 Value);

	// Returns the texture quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetTextureQuality() const;

	// Sets the visual effects quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetVisualEffectQuality(int32 Value);

	// Returns the visual effects quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetVisualEffectQuality() const;

	// Sets the post-processing quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetPostProcessingQuality(int32 Value);

	// Returns the post-processing quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetPostProcessingQuality() const;
	
	// Sets the foliage quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SetFoliageQuality(int32 Value);

	// Returns the foliage quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API int32 GetFoliageQuality() const;

	// Sets the shading quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetShadingQuality(int32 Value);

	// Returns the shading quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API int32 GetShadingQuality() const;

	// Sets the landscape quality (0..4, higher is better)
	// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetLandscapeQuality(int32 Value);

	// Returns the landscape quality (0..4, higher is better)
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API int32 GetLandscapeQuality() const;

	/** Checks if any user settings is different from current */
	UFUNCTION(BlueprintPure, Category=Settings)
	ENGINE_API virtual bool IsDirty() const;

	/** Validates and resets bad user settings to default. Deletes stale user settings file if necessary. */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void ValidateSettings();

	/** Loads the user settings from persistent storage */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void LoadSettings(bool bForceReload = false);

	/** Save the user settings to persistent storage (automatically happens as part of ApplySettings) */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void SaveSettings();

	/** This function resets all settings to the current system settings */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void ResetToCurrentSettings();

	/** Sets the window position in save slot index 0. */
	ENGINE_API virtual void SetWindowPosition(int32 WindowPosX, int32 WindowPosY);

	/** Sets the window position in the given index, which must be less than GetMaxNumWindowPositionSaveSlots(). */
	ENGINE_API virtual void SetWindowPosition(int32 WindowPosX, int32 WindowPosY, int32 SaveSlotIndex);

	/** Returns the window position in save slot index 0. */
	ENGINE_API virtual FIntPoint GetWindowPosition();

	/** Returns the window position in the given index, or the default if the index is invalid. */
	ENGINE_API virtual FIntPoint GetWindowPosition(int32 SaveSlotIndex);
	
	/** Reloads the WindowPositions array from config files. Use before SaveConfig to avoid clobbering other processes' window positions. */
	ENGINE_API void ReloadWindowPositions();

	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void SetToDefaults();

	/** Gets the desired resolution quality based on DesiredScreenWidth/Height and the current screen resolution */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual float GetDefaultResolutionScale();

	/** Gets the recommended resolution quality based on LastRecommendedScreenWidth/Height and the current screen resolution */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API virtual float GetRecommendedResolutionScale();

	/** Loads the resolution settings before is object is available */
	static ENGINE_API void PreloadResolutionSettings(bool bAllowCmdLineOverrides = true);

	/** Returns the default resolution when no resolution is set */
	UFUNCTION(BlueprintCallable, Category=Settings)
	static ENGINE_API FIntPoint GetDefaultResolution();

	/** Returns the default window position when no position is set */
	UFUNCTION(BlueprintCallable, Category=Settings)
	static ENGINE_API FIntPoint GetDefaultWindowPosition();

	/** Returns the default window mode when no mode is set */
	UFUNCTION(BlueprintCallable, Category=Settings)
	static ENGINE_API EWindowMode::Type GetDefaultWindowMode();

	/** Gets the current vsync interval setting */
	UE_DEPRECATED(4.25, "Please use GetFramePace to get the paced frame rate")
	UFUNCTION(BlueprintPure, Category = Settings)
	static ENGINE_API int32 GetSyncInterval();

	/** Gets the current frame pacing frame rate in fps, or 0 if none */
	UFUNCTION(BlueprintPure, Category = Settings)
	static ENGINE_API int32 GetFramePace();

	/** Loads the user .ini settings into GConfig */
	static ENGINE_API void LoadConfigIni(bool bForceReload = false);

	/** Request a change to the specified resolution and window mode. Optionally apply cmd line overrides. */
	static ENGINE_API void RequestResolutionChange(int32 InResolutionX, int32 InResolutionY, EWindowMode::Type InWindowMode, bool bInDoOverrides = true, int32 MonitorIndex = -1);

	/** Returns the game local machine settings (resolution, windowing mode, scalability settings, etc...) */
	UFUNCTION(BlueprintCallable, Category=Settings)
	static ENGINE_API UGameUserSettings* GetGameUserSettings();

	/** Runs the hardware benchmark and populates ScalabilityQuality as well as the last benchmark results config members, but does not apply the settings it determines. Designed to be called in conjunction with ApplyHardwareBenchmarkResults */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void RunHardwareBenchmark(int32 WorkScale = 10, float CPUMultiplier = 1.0f, float GPUMultiplier = 1.0f);

	/** Applies the settings stored in ScalabilityQuality and saves settings */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API virtual void ApplyHardwareBenchmarkResults();

	/** Whether the curently running system supports HDR display output */
	UFUNCTION(BlueprintCallable, Category=Settings, meta = (DisplayName = "Supports HDR Display Output"))
	ENGINE_API virtual bool SupportsHDRDisplayOutput() const;

	/** Enables or disables HDR display output. Can be called again to change the desired nit level */
	UFUNCTION(BlueprintCallable, Category=Settings, meta=(DisplayName = "Enable HDR Display Output"))
	ENGINE_API void EnableHDRDisplayOutput(bool bEnable, int32 DisplayNits = 1000);

	/** Returns 0 if HDR isn't supported or is turned off */
	UFUNCTION(BlueprintCallable, Category = Settings, meta = (DisplayName = "Get Current HDR Display Nits"))
	ENGINE_API int32 GetCurrentHDRDisplayNits() const;

	/** Sets the maximum luminance that we want to display. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetMaximumHDRDisplayNits(int32 DisplayNits);

	/** Gets the maximum luminance that we want to display. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API int32 GetMaximumHDRDisplayNits() const;

	/** Sets if HDR calibration is used (only used for platforms without system HDR calibration) */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetHDRCalibrationUsed(bool bUseCalibration);

	/** Gets if HDR calibration is used (only used for platforms without system HDR calibration) */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API bool IsHDRCalibrationUsed() const;

	/** Sets HDR paper white to control scene brightness, also UI unless bIsHDRUILuminanceSeparate is true */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetHDRPaperWhiteNits(float Nits);

	/** Gets HDR paper white to control scene brightness, also UI unless bIsHDRUILuminanceSeparate is true */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API float GetHDRPaperWhiteNits() const;

	/** Sets the HDR UI luminance value that is used if luminance is separate*/
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetHDRUILuminanceNits(float Nits);

	/** Gets the HDR UI luminance value that is used if luminance is separate  */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API float GetHDRUILuminanceNits() const;

	/** Sets whether HDR UI should use its own value or use paper white */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void SetHDRUILuminanceSeparate(bool bIndependent);

	/** Gets whether HDR UI should use its own value or use paper white */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API bool IsHDRUILuminanceSeparate() const;

	/** Sets if HDR is enabled (ignored if support is unavailable) */
	UFUNCTION(BlueprintCallable, Category = Settings, meta = (DisplayName = "Set HDR Enabled"))
	ENGINE_API void SetHDREnabled(bool bEnableHDR);

	UFUNCTION(BlueprintCallable, Category = Settings, meta = (DisplayName = "Is HDR Enabled"))
	ENGINE_API bool IsHDREnabled() const;

	/**
	 * @return True if Lumen Global Illumination is allowed by scalability and supported by the device. This does not take into account whether Lumen is enabled in the project settings, or a Post Process Volume.
	 * This is useful for the game to setup exposure depending on whether interiors can be darkened by skylight shadowing or not.
	 */
	UFUNCTION(BlueprintCallable, Category = Settings, meta = (DisplayName = "Is Global Illumination Allowed", WorldContext = "WorldContextObject"))
	ENGINE_API bool IsGlobalIlluminationAllowed(UObject* WorldContextObject) const;

	/** Whether to use VSync or not. (public to allow UI to connect to it) */
	UPROPERTY(config)
	bool bUseVSync;

	/** Whether to use dynamic resolution or not. (public to allow UI to connect to it) */
	UPROPERTY(config)
	bool bUseDynamicResolution;

	// cached for the UI, current state if stored in console variables
	Scalability::FQualityLevels ScalabilityQuality;

protected:
	/** Game screen resolution width, in pixels. */
	UPROPERTY(config)
	uint32 ResolutionSizeX;

	/** Game screen resolution height, in pixels. */
	UPROPERTY(config)
	uint32 ResolutionSizeY;

	/** Game screen resolution width, in pixels. */
	UPROPERTY(config)
	uint32 LastUserConfirmedResolutionSizeX;

	/** Game screen resolution height, in pixels. */
	UPROPERTY(config)
	uint32 LastUserConfirmedResolutionSizeY;

	/** Window PosX */
	UE_DEPRECATED(5.6, "Use the WindowPositions array instead.")
	UPROPERTY(config)
	int32 WindowPosX_DEPRECATED;

	/** Window PosY */
	UE_DEPRECATED(5.6, "Use the WindowPositions array instead.")
	UPROPERTY(config)
	int32 WindowPosY_DEPRECATED;

	/** Saved window positions. Multiple instances running simultaneously can use different ones. */
	UPROPERTY(config)
	TArray<FIntPoint> WindowPositions;

	/**
	 * Game window fullscreen mode
	 *	0 = Fullscreen
	 *	1 = Windowed fullscreen
	 *	2 = Windowed
	 */
	UPROPERTY(config)
	int32 FullscreenMode;

	/** Last user confirmed fullscreen mode setting. */
	UPROPERTY(config)
	int32 LastConfirmedFullscreenMode;

	/** Fullscreen mode to use when toggling between windowed and fullscreen. Same values as r.FullScreenMode. */
	UPROPERTY(config)
	int32 PreferredFullscreenMode;

	/** String that attempts to uniquely identify a display even if FDisplayMetrics::MonitorInfo is reordered. */
	UPROPERTY(config)
	FString DisplayID;

	/** Index of a display in FDisplayMetrics::MonitorInfo. */
	UPROPERTY(config)
	int32 DisplayIndex;

	/** Last user confirmed display by ID. */
	UPROPERTY(config)
	FString LastUserConfirmedDisplayID;

	/** Last user confirmed display by index. */
	UPROPERTY(config)
	int32 LastUserConfirmedDisplayIndex;

	/** All settings will be wiped and set to default if the serialized version differs from UE_GAMEUSERSETTINGS_VERSION. */
	UPROPERTY(config)
	uint32 Version;

	UPROPERTY(config)
	int32 AudioQualityLevel;

	UPROPERTY(config)
	int32 LastConfirmedAudioQualityLevel;

	/** Frame rate cap */
	UPROPERTY(config)
	float FrameRateLimit;

	/** Dynamic resolution frame target */
	UPROPERTY(config)
	float DynamicResolutionFrameTarget;

	/** Min resolution scale we allow in current display mode */
	UE_DEPRECATED(5.3, "MinResolutionScale is now always Scalability::MinResolutionScale=0 that fallbacks to default project wide behavior defined by r.ScreenPercentage.Default")
	float MinResolutionScale;

	/** Desired screen width used to calculate the resolution scale when user changes display mode */
	UPROPERTY(config)
	int32 DesiredScreenWidth;

	/** If true, the desired screen height will be used to scale the render resolution automatically. */
	UPROPERTY(globalconfig)
	bool bUseDesiredScreenHeight;

	/** Desired screen height used to calculate the resolution scale when user changes display mode */
	UPROPERTY(config)
	int32 DesiredScreenHeight;

	/** Desired screen width used to calculate the resolution scale when user changes display mode */
	UPROPERTY(config)
	int32 LastUserConfirmedDesiredScreenWidth;

	/** Desired screen height used to calculate the resolution scale when user changes display mode */
	UPROPERTY(config)
	int32 LastUserConfirmedDesiredScreenHeight;

	/** Result of the last benchmark; calculated resolution to use. */
	UPROPERTY(config)
	float LastRecommendedScreenWidth;

	/** Result of the last benchmark; calculated resolution to use. */
	UPROPERTY(config)
	float LastRecommendedScreenHeight;

	/** Result of the last benchmark (CPU); -1 if there has not been a benchmark run */
	UPROPERTY(config)
	float LastCPUBenchmarkResult;

	/** Result of the last benchmark (GPU); -1 if there has not been a benchmark run */
	UPROPERTY(config)
	float LastGPUBenchmarkResult;

	/** Result of each individual sub-section of the last CPU benchmark; empty if there has not been a benchmark run */
	UPROPERTY(config)
	TArray<float> LastCPUBenchmarkSteps;

	/** Result of each individual sub-section of the last GPU benchmark; empty if there has not been a benchmark run */
	UPROPERTY(config)
	TArray<float> LastGPUBenchmarkSteps;

	/**
	 * Multiplier used against the last GPU benchmark
	 */
	UPROPERTY(config)
	float LastGPUBenchmarkMultiplier;

	/** Whether to display output in HDR if support is available */
	UPROPERTY(config)
	bool bUseHDRDisplayOutput;

	/** Whether to override platform HDR display information */
	UPROPERTY(config)
	bool bUseHDRDisplayCalibration;

	/** The maximum nits of which the HDR display is capable (application should offer calibration flow) */
	UPROPERTY(config)
	int32 HDRDisplayOutputNits;

	/** Paper white luminance to control scene brightness in HDR, also UI brightness if bIsHDRUILuminanceSeparate is false */
	UPROPERTY(config)
	float HDRPaperWhiteNits;

	/** Paper white luminance to control UI brightness in HDR if bIsHDRUILuminanceSeparate is true */
	UPROPERTY(config)
	float HDRUILuminanceNits;

	/** Whether to override HDRPaperWhiteNits with HDRUILuminanceNits for UI brightness */
	UPROPERTY(config)
	bool bIsHDRUILuminanceSeparate;

	/** When set to false, scalability settings (ScalabilityQuality) are not considered user settings
	  * i.e. they are not saved and applied.
	  */
	bool bEnableScalabilitySettings = true;

public:
	/** Returns the last CPU benchmark result (set by RunHardwareBenchmark) */
	float GetLastCPUBenchmarkResult() const
	{
		return LastCPUBenchmarkResult;
	}

	/** Returns the last GPU benchmark result (set by RunHardwareBenchmark) */
	float GetLastGPUBenchmarkResult() const
	{
		return LastGPUBenchmarkResult;
	}

	/** Returns each individual step of the last CPU benchmark result (set by RunHardwareBenchmark) */
	TArray<float> GetLastCPUBenchmarkSteps() const
	{
		return LastCPUBenchmarkSteps;
	}

	/** Returns each individual step of the last GPU benchmark result (set by RunHardwareBenchmark) */
	TArray<float> GetLastGPUBenchmarkSteps() const
	{
		return LastGPUBenchmarkSteps;
	}

	void RequestUIUpdate()
	{
		OnGameUserSettingsUINeedsUpdate.Broadcast();
	}

	/** Returns the maximum supported number save slots for window positions. */
	int32 GetMaxNumWindowPositionSaveSlots() const;

	/**
	 * We call this to refresh the config file from a listening Cloud subsystem, before we open it.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FUpdateGameUserSettingsFileFromCloud, const FString&);
	FUpdateGameUserSettingsFileFromCloud OnUpdateGameUserSettingsFileFromCloud;

	/**
	 * We call this to notify any listening Cloud subsystem that we have updated the config file.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FUpdateCloudDataFromGameUserSettings, const FString&);
	FUpdateCloudDataFromGameUserSettings OnUpdateCloudDataFromGameUserSettings;

	/**
	 * We call this to notify UI to refresh relevant settings.
	 */
	DECLARE_MULTICAST_DELEGATE(FOnGameUserSettingsVideoRevert);
	FOnGameUserSettingsVideoRevert OnGameUserSettingsVideoRevert;

protected:
	/**
	 * Check if the current version of the game user settings is valid. Sub-classes can override this to provide game-specific versioning as necessary.
	 * @return True if the current version is valid, false if it is not
	 */
	ENGINE_API virtual bool IsVersionValid();

	/** Update the version of the game user settings to the current version */
	ENGINE_API virtual void UpdateVersion();

	/** Picks the best resolution quality for a given screen size */
	ENGINE_API float FindResolutionQualityForScreenSize(float Width, float Height);

	/** Sets the frame rate limit CVar to the passed in value, 0.0 indicates no limit */
	static ENGINE_API void SetFrameRateLimitCVar(float InLimit);

	/** Sets the input latency mode 0 and 2 */
	static ENGINE_API void SetSyncTypeCVar(int32 InInterval);

	/** Returns the effective frame rate limit (by default it returns the FrameRateLimit member) */
	ENGINE_API virtual float GetEffectiveFrameRateLimit();

	ENGINE_API void UpdateResolutionQuality();

	ENGINE_API void EnableHDRDisplayOutputInternal(bool bEnable, int32 DisplayNits, bool bFromUserSettings);

private:

	UPROPERTY(BlueprintAssignable, meta = (AllowPrivateAccess = "true"))
	FOnGameUserSettingsUINeedsUpdate OnGameUserSettingsUINeedsUpdate;

	ENGINE_API void SetPreferredFullscreenMode(int32 Mode);
};
