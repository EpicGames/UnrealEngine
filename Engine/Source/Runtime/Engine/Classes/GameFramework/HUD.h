// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "GameFramework/HUDHitBox.h"
#include "GameFramework/DebugTextInfo.h"
#include "HUD.generated.h"

class AHUD;
class APawn;
class APlayerController;
class FCanvas;
class FDebugDisplayInfo;
class UCanvas;
class UFont;
class UMaterialInterface;
class UTexture;

DECLARE_TS_MULTICAST_DELEGATE_FiveParams(FOnShowDebugInfo, AHUD* /* HUD */, UCanvas* /* Canvas */, const FDebugDisplayInfo& /* DisplayInfo */, float& /* YL */, float& /* YPos */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnHUDPostRender, AHUD* /* HUD */, UCanvas* /* Canvas */);

/** 
 * Base class of the heads-up display. This has a canvas and a debug canvas on which primitives can be drawn.
 * It also contains a list of simple hit boxes that can be used for simple item click detection.
 * A method of rendering debug text is also included.
 * Provides some simple methods for rendering text, textures, rectangles and materials which can also be accessed from blueprints.
 * @see UCanvas
 * @see FHUDHitBox
 * @see FDebugTextInfo
 */
UCLASS(config=Game, hidecategories=(Rendering,Actor,Input,Replication), showcategories=("Input|MouseInput", "Input|TouchInput"), notplaceable, transient, BlueprintType, Blueprintable, MinimalAPI)
class AHUD : public AActor
{
	GENERATED_UCLASS_BODY()

	/** PlayerController which owns this HUD. */
	UPROPERTY(BlueprintReadOnly, Category=HUD)
	TObjectPtr<APlayerController> PlayerOwner;    

	/** Tells whether the game was paused due to lost focus */
	UPROPERTY(BlueprintReadOnly, Category=HUD)
	uint8 bLostFocusPaused:1;

	/** Whether or not the HUD should be drawn. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=HUD)
	uint8 bShowHUD:1;    

	/** If true, current ViewTarget shows debug information using its DisplayDebug(). */
	UPROPERTY(BlueprintReadWrite, Category=HUD)
	uint8 bShowDebugInfo:1;    

	/** Current target in our considered Targets list for 'showdebug' */
	UPROPERTY(Transient)
	int32 CurrentTargetIndex;

	/** If true, show hitbox debugging info. */
	UPROPERTY(BlueprintReadWrite, Category=HUD)
	uint8 bShowHitBoxDebugInfo:1;    

	/** If true, render actor overlays. */
	UPROPERTY(BlueprintReadWrite, Category=HUD)
	uint8 bShowOverlays:1;

	/** Put shadow on debug strings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=HUD)
	uint8 bEnableDebugTextShadow:1;	

private:
	/** if true show debug info for 'ShowDebugTargetActor', otherwise for Camera Viewtarget */
	uint8 bShowDebugForReticleTarget:1;
	
public:
	/** Holds a list of Actors that need PostRender() calls. */
	UPROPERTY()
	TArray<TObjectPtr<AActor>> PostRenderedActors;

	/** Used to calculate delta time between HUD rendering. */
	float LastHUDRenderTime;

	/** Time since last HUD render. */
	float RenderDelta;

	/** Array of names specifying what debug info to display for viewtarget actor. */
	UPROPERTY(globalconfig)
	TArray<FName> DebugDisplay;

	/** Array of names specifying what subsets of debug info to display for viewtarget actor. */
	UPROPERTY(globalconfig)
	TArray<FName> ToggledDebugCategories;

protected:
	/** Canvas to Draw HUD on.  Only valid during PostRender() event.  */
	UPROPERTY()
	TObjectPtr<UCanvas> Canvas;

	/** 'Foreground' debug canvas, will draw in front of Slate UI. */
	UPROPERTY()
	TObjectPtr<UCanvas> DebugCanvas;

	/** List of debug strings attached to actors, sorted by actor first, then by order of addition */
	UPROPERTY()
	TArray<struct FDebugTextInfo> DebugTextList;

public:
	//=============================================================================
	// Utils

	/** hides or shows HUD */
	UFUNCTION(exec)
	ENGINE_API virtual void ShowHUD();

	/**
	 * Toggles displaying properties of player's current ViewTarget
	 * DebugType input values supported by base engine include "AI", "physics", "net", "camera", and "collision"
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void ShowDebug(FName DebugType = NAME_None);

	/**
	 * Toggles sub categories of show debug to customize display
	 */
	UFUNCTION(exec)
	ENGINE_API void ShowDebugToggleSubCategory(FName Category);

	/** Toggles 'ShowDebug' from showing debug info between reticle target actor (of subclass <DesiredClass>) and camera view target */
	UFUNCTION(exec)
	ENGINE_API void ShowDebugForReticleTargetToggle(TSubclassOf<AActor> DesiredClass);

protected:
	/** Class filter for selecting 'ShowDebugTargetActor' when 'bShowDebugForReticleTarget' is true. */
	UPROPERTY()
	TSubclassOf<AActor> ShowDebugTargetDesiredClass;

	/** Show Debug Actor used if 'bShowDebugForReticleTarget' is true, only updated if trace from reticle hit a new Actor of class 'ShowDebugTargetDesiredClass'*/
	UPROPERTY()
	TObjectPtr<AActor> ShowDebugTargetActor;

public:
	/**
	 * Add debug text for a specific actor to be displayed via DrawDebugTextList().  If the debug text is invalid then it will
	 * attempt to remove any previous entries via RemoveDebugText().
	 * 
	 * @param DebugText				Text to draw
	 * @param SrcActor				Actor to which this relates
	 * @param Duration				Duration to display the string
	 * @param Offset 				Initial offset to render text
	 * @param DesiredOffset 		Desired offset to render text - the text will move to this location over the given duration
	 * @param TextColor 			Color of text to render
	 * @param bSkipOverwriteCheck 	skips the check to see if there is already debug text for the given actor
	 * @param bAbsoluteLocation 	use an absolute world location
	 * @param bKeepAttachedToActor 	if this is true the text will follow the actor, otherwise it will be drawn at the location when the call was made
	 * @param InFont 				font to use
	 * @param FontScale 			scale
	 * @param bDrawShadow 			Draw shadow on this string
	 */
	UFUNCTION(reliable, client, SealedEvent)
	ENGINE_API void AddDebugText(const FString& DebugText, AActor* SrcActor = NULL, float Duration = 0, FVector Offset = FVector(ForceInit), FVector DesiredOffset = FVector(ForceInit), FColor TextColor = FColor(ForceInit), bool bSkipOverwriteCheck = false, bool bAbsoluteLocation = false, bool bKeepAttachedToActor = false, UFont* InFont = NULL, float FontScale = 1.0, bool bDrawShadow = false);

	/**
	 * Remove all debug strings added via AddDebugText
	 */
	UFUNCTION(reliable, client, SealedEvent)
	ENGINE_API void RemoveAllDebugStrings();

	/**
	 * Remove debug strings for the given actor
	 *
	 * @param	SrcActor			Actor whose string you wish to remove
	 * @param	bLeaveDurationText	when true text that has a finite duration will be removed, otherwise all will be removed for given actor
	 */
	UFUNCTION(reliable, client, SealedEvent)
	ENGINE_API void RemoveDebugText(AActor* SrcActor, bool bLeaveDurationText = false);

	/** 
	 *	Hook to allow blueprints to do custom HUD drawing. @see bSuppressNativeHUD to control HUD drawing in base class. 
	 *	Note:  the canvas resource used for drawing is only valid during this event, it will not be valid if drawing functions are called later (e.g. after a Delay node).
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic)
	ENGINE_API void ReceiveDrawHUD(int32 SizeX, int32 SizeY);

	/** Called when a hit box is clicked on. Provides the name associated with that box. */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, meta=(DisplayName = "HitBoxClicked"))
	ENGINE_API void ReceiveHitBoxClick(const FName BoxName);
	/** Native handler, called when a hit box is clicked on. Provides the name associated with that box. */
	ENGINE_API virtual void NotifyHitBoxClick(FName BoxName);

	/** Called when a hit box is unclicked. Provides the name associated with that box. */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, meta=(DisplayName = "HitBoxReleased"))
	ENGINE_API void ReceiveHitBoxRelease(const FName BoxName);
	/** Native handler, called when a hit box is unclicked. Provides the name associated with that box. */
	ENGINE_API virtual void NotifyHitBoxRelease(FName BoxName);

	/** Called when a hit box is moused over. */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, meta=(DisplayName = "HitBoxBeginCursorOver"))
	ENGINE_API void ReceiveHitBoxBeginCursorOver(const FName BoxName);
	/** Native handler, called when a hit box is moused over. */
	ENGINE_API virtual void NotifyHitBoxBeginCursorOver(FName BoxName);

	/** Called when a hit box no longer has the mouse over it. */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, meta=(DisplayName = "HitBoxEndCursorOver"))
	ENGINE_API void ReceiveHitBoxEndCursorOver(const FName BoxName);
	/** Native handler, called when a hit box no longer has the mouse over it. */
	ENGINE_API virtual void NotifyHitBoxEndCursorOver(FName BoxName);

	//=============================================================================
	// Kismet API for simple HUD drawing.

	/**
	 * Returns the width and height of a string.
	 * @param Text				String to draw
	 * @param OutWidth			Returns the width in pixels of the string.
	 * @param OutHeight			Returns the height in pixels of the string.
	 * @param Font				Font to draw text.  If NULL, default font is chosen.
	 * @param Scale				Scale multiplier to control size of the text.
	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API void GetTextSize(const FString& Text, float& OutWidth, float& OutHeight, UFont* Font=NULL, float Scale=1.f) const;

	/**
	 * Draws a string on the HUD.
	 * @param Text				String to draw
	 * @param TextColor			Color to draw string
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the string.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the string.
	 * @param Font				Font to draw text.  If NULL, default font is chosen.
	 * @param Scale				Scale multiplier to control size of the text.
	 * @param bScalePosition	Whether the "Scale" parameter should also scale the position of this draw call.
	 */
	UFUNCTION(BlueprintCallable, Category=HUD, meta=(TextColor="(R=0,G=0,B=0,A=1)"))
	ENGINE_API void DrawText(const FString& Text, FLinearColor TextColor, float ScreenX, float ScreenY, UFont* Font=NULL, float Scale=1.f, bool bScalePosition=false);

	/**
	 * Draws a 2D line on the HUD.
	 * @param StartScreenX		Screen-space X coordinate of start of the line.
	 * @param StartScreenY		Screen-space Y coordinate of start of the line.
	 * @param EndScreenX		Screen-space X coordinate of end of the line.
	 * @param EndScreenY		Screen-space Y coordinate of end of the line.
	 * @param LineColor			Color to draw line
	 * @param LineThickness		Thickness of the line to draw
	 */
	UFUNCTION(BlueprintCallable, Category=HUD, meta=(LineColor="(R=0,G=0,B=0,A=1)"))
	ENGINE_API void DrawLine(float StartScreenX, float StartScreenY, float EndScreenX, float EndScreenY, FLinearColor LineColor, float LineThickness=0.f);

	/**
	 * Draws a colored untextured quad on the HUD.
	 * @param RectColor			Color of the rect. Can be translucent.
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the quad.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the quad.
	 * @param ScreenW			Screen-space width of the quad (in pixels).
	 * @param ScreenH			Screen-space height of the quad (in pixels).
	 */
	UFUNCTION(BlueprintCallable, Category=HUD, meta=(RectColor="(R=0,G=0,B=0,A=1)"))
	ENGINE_API void DrawRect(FLinearColor RectColor, float ScreenX, float ScreenY, float ScreenW, float ScreenH);

	/**
	 * Draws a textured quad on the HUD.
	 * @param Texture			Texture to draw.
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the quad.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the quad.
	 * @param ScreenW			Screen-space width of the quad (in pixels).
	 * @param ScreenH			Screen-space height of the quad (in pixels).
	 * @param TextureU			Texture-space U coordinate of upper left corner of the quad
	 * @param TextureV			Texture-space V coordinate of upper left corner of the quad.
	 * @param TextureUWidth		Texture-space width of the quad (in normalized UV distance).
	 * @param TextureVHeight	Texture-space height of the quad (in normalized UV distance).
	 * @param TintColor			Vertex color for the quad.
	 * @param BlendMode			Controls how to blend this quad with the scene. Translucent by default.
	 * @param Scale				Amount to scale the entire texture (horizontally and vertically)
	 * @param bScalePosition	Whether the "Scale" parameter should also scale the position of this draw call.
	 * @param Rotation			Amount to rotate this quad
	 * @param RotPivot			Location (as proportion of quad, 0-1) to rotate about
	 */
	UFUNCTION(BlueprintCallable, Category=HUD, meta=(AdvancedDisplay = "9"))
	ENGINE_API void DrawTexture(UTexture* Texture, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float TextureU, float TextureV, float TextureUWidth, float TextureVHeight, FLinearColor TintColor=FLinearColor::White, EBlendMode BlendMode=BLEND_Translucent, float Scale=1.f, bool bScalePosition=false, float Rotation=0.f, FVector2D RotPivot=FVector2D::ZeroVector);

	/**
	 * Draws a textured quad on the HUD. Assumes 1:1 texel density.
	 * @param Texture			Texture to draw.
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the quad.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the quad.
	 * @param Scale				Scale multiplier to control size of the text.
	 * @param bScalePosition	Whether the "Scale" parameter should also scale the position of this draw call.
	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API void DrawTextureSimple(UTexture* Texture, float ScreenX, float ScreenY, float Scale=1.f, bool bScalePosition=false);

	/**
	 * Draws a material-textured quad on the HUD.
	 * @param Material			Material to use
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the quad.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the quad.
	 * @param ScreenW			Screen-space width of the quad (in pixels).
	 * @param ScreenH			Screen-space height of the quad (in pixels).
	 * @param MaterialU			Texture-space U coordinate of upper left corner of the quad
	 * @param MaterialV			Texture-space V coordinate of upper left corner of the quad.
	 * @param MaterialUWidth	Texture-space width of the quad (in normalized UV distance).
	 * @param MaterialVHeight	Texture-space height of the quad (in normalized UV distance).
	 * @param Scale				Amount to scale the entire texture (horizontally and vertically)
	 * @param bScalePosition	Whether the "Scale" parameter should also scale the position of this draw call.
	 * @param Rotation			Amount to rotate this quad
	 * @param RotPivot			Location (as proportion of quad, 0-1) to rotate about
	 */
	UFUNCTION(BlueprintCallable, Category=HUD, meta=(AdvancedDisplay = "9"))
	ENGINE_API void DrawMaterial(UMaterialInterface* Material, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float MaterialU, float MaterialV, float MaterialUWidth, float MaterialVHeight, float Scale=1.f, bool bScalePosition=false, float Rotation=0.f, FVector2D RotPivot=FVector2D::ZeroVector);

	/**
	 * Draws a material-textured quad on the HUD.  Assumes UVs such that the entire material is shown.
	 * @param Material			Material to use
	 * @param ScreenX			Screen-space X coordinate of upper left corner of the quad.
	 * @param ScreenY			Screen-space Y coordinate of upper left corner of the quad.
	 * @param ScreenW			Screen-space width of the quad (in pixels).
	 * @param ScreenH			Screen-space height of the quad (in pixels).
	 * @param Scale				Amount to scale the entire texture (horizontally and vertically)
	 * @param bScalePosition	Whether the "Scale" parameter should also scale the position of this draw call.
	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API void DrawMaterialSimple(UMaterialInterface* Material, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float Scale=1.f, bool bScalePosition=false);

	UFUNCTION(BlueprintCallable, Category = HUD)
	ENGINE_API void DrawMaterialTriangle(UMaterialInterface* Material, FVector2D V0_Pos, FVector2D V1_Pos, FVector2D V2_Pos, FVector2D V0_UV, FVector2D V1_UV, FVector2D V2_UV, FLinearColor V0_Color = FLinearColor::White, FLinearColor V1_Color = FLinearColor::White, FLinearColor V2_Color = FLinearColor::White);
	
	/** Transforms a 3D world-space vector into 2D screen coordinates
	 * @param Location			The world-space position to transform
	 * @param bClampToZeroPlane	If true, 2D screen coordinates behind the viewing plane (-Z) will have Z set to 0 (leaving X and Y alone)
	 * @return The transformed vector
	 */
	UFUNCTION(BlueprintCallable, Category = HUD)
	ENGINE_API FVector Project(FVector Location, bool bClampToZeroPlane = true) const;
	
	/** Transforms a 2D screen location into a 3D location and direction */
	UFUNCTION(BlueprintCallable, Category = HUD)
	ENGINE_API void Deproject(float ScreenX, float ScreenY, FVector& WorldPosition, FVector& WorldDirection) const;

	/**
	 * Returns the array of actors inside a selection rectangle, with a class filter.
	 *
	 * Sample usage:
	 *
	 *      TArray<AStaticMeshActor*> ActorsInSelectionRect;
	 * 		GetActorsInSelectionRectangle<AStaticMeshActor>(FirstPoint,SecondPoint,ActorsInSelectionRect);
	 *
	 *
	 * @param FirstPoint					The first point, or anchor of the marquee box. Where the dragging of the marquee started in screen space.
	 * @param SecondPoint					The second point, where the mouse cursor currently is / the other point of the box selection, in screen space.
	 * @return OutActors					The actors that are within the selection box according to selection rule
	 * @param bIncludeNonCollidingComponents 	Whether to include even non-colliding components of the actor when determining its bounds
	 * @param bActorMustBeFullyEnclosed  	The Selection rule: whether the selection box can partially intersect Actor, or must fully enclose the Actor.
	 *
	 * returns false if selection could not occur. Make sure template class is extending AActor.
	 */
	template <typename ClassFilter>
	bool GetActorsInSelectionRectangle(const FVector2D& FirstPoint, const FVector2D& SecondPoint, TArray<ClassFilter*>& OutActors, bool bIncludeNonCollidingComponents = true, bool bActorMustBeFullyEnclosed = false)
	{
		//Is Actor subclass?
		if (!ClassFilter::StaticClass()->IsChildOf(AActor::StaticClass()))
		{
			return false;
		}

		//Run Inner Function, output to Base AActor Array
		TArray<AActor*> OutActorsBaseArray;
		GetActorsInSelectionRectangle(ClassFilter::StaticClass(), FirstPoint, SecondPoint, OutActorsBaseArray, bIncludeNonCollidingComponents, bActorMustBeFullyEnclosed);

		//Construct casted template type array
		for (AActor* EachActor : OutActorsBaseArray)
		{
			OutActors.Add(CastChecked<ClassFilter>(EachActor));
		}

		return true;
	};

	/**
	 * Returns the array of actors inside a selection rectangle, with a class filter.
	 *
	 * Sample usage:
	 *
	 *       TArray<AStaticMeshActor*> ActorsInSelectionRect;
	 * 		Canvas->GetActorsInSelectionRectangle<AStaticMeshActor>(FirstPoint,SecondPoint,ActorsInSelectionRect);
	 *
	 *
	 * @param FirstPoint					The first point, or anchor of the marquee box. Where the dragging of the marquee started in screen space.
	 * @param SecondPoint					The second point, where the mouse cursor currently is / the other point of the box selection, in screen space.
	 * @return OutActors					The actors that are within the selection box according to selection rule
	 * @param bIncludeNonCollidingComponents 	Whether to include even non-colliding components of the actor when determining its bounds
	 * @param bActorMustBeFullyEnclosed  	The Selection rule: whether the selection box can partially intersect Actor, or must fully enclose the Actor.
	 *
	 */
	UFUNCTION(BlueprintPure, Category=HUD)
	ENGINE_API void GetActorsInSelectionRectangle(TSubclassOf<AActor> ClassFilter, const FVector2D& FirstPoint, const FVector2D& SecondPoint, TArray<AActor*>& OutActors, bool bIncludeNonCollidingComponents = true, bool bActorMustBeFullyEnclosed = false);

	/**
	 * Add a hitbox to the hud
	 * @param Position			Coordinates of the top left of the hit box.
	 * @param Size				Size of the hit box.
	 * @param Name				Name of the hit box.
	 * @param bConsumesInput	Whether click processing should continue if this hit box is clicked.
	 * @param Priority			The priority of the box used for layering. Larger values are considered first.  Equal values are considered in the order they were added.
	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API void AddHitBox(FVector2D Position, FVector2D Size, FName InName, bool bConsumesInput, int32 Priority = 0);

	/** Returns the PlayerController for this HUD's player.	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API APlayerController* GetOwningPlayerController() const;

	/** Returns the Pawn for this HUD's player.	 */
	UFUNCTION(BlueprintCallable, Category=HUD)
	ENGINE_API APawn* GetOwningPawn() const;

	/**
	 * Draws a colored line between two points
	 * @param Start - start of line
	 * @param End - end if line
	 * @param LineColor
	 */
	ENGINE_API void Draw3DLine(FVector Start, FVector End, FColor LineColor);

	/**
	 * Draws a colored line between two points
	 * @param X1 - start of line x
	 * @param Y1 - start of line y
	 * @param X2 - end of line x
	 * @param Y3 - end of line y
	 * @param LineColor
	 */
	ENGINE_API void Draw2DLine(int32 X1, int32 Y1, int32 X2, int32 Y2, FColor LineColor);

	/** Set the canvas and debug canvas to use during drawing */
	ENGINE_API void SetCanvas(UCanvas* InCanvas, UCanvas* InDebugCanvas);

	ENGINE_API virtual void PostInitializeComponents() override;

	/** draw overlays for actors that were rendered this tick and have added themselves to the PostRenderedActors array	*/
	ENGINE_API virtual void DrawActorOverlays(FVector Viewpoint, FRotator ViewRotation);

	/** Draw the safe zone debugging overlay when enabled */
	ENGINE_API virtual void DrawSafeZoneOverlay();

	/** Called in PostInitializeComponents or postprocessing chain has changed (happens because of the worldproperties can define it's own chain and this one is set late). */
	ENGINE_API virtual void NotifyBindPostProcessEffects();

	/************************************************************************************************************
	 Actor Render - These functions allow for actors in the world to gain access to the hud and render
	 information on it.
	************************************************************************************************************/

	/** remove an actor from the PostRenderedActors array */
	ENGINE_API virtual void RemovePostRenderedActor(AActor* A);

	/** add an actor to the PostRenderedActors array */
	ENGINE_API virtual void AddPostRenderedActor(AActor* A);

	/**
	 * check if we should be display debug information for particular types of debug messages.
	 * @param DebugType - type of debug message.
	 * @return true if it should be displayed, false otherwise.
	 */
	ENGINE_API virtual bool ShouldDisplayDebug(const FName& DebugType) const;

	/** 
	 * Entry point for basic debug rendering on the HUD.  Activated and controlled via the "showdebug" console command.  
	 * Can be overridden to display custom debug per-game. 
	 */
	ENGINE_API virtual void ShowDebugInfo(float& YL, float& YPos);

	/** Get Target to view 'showdebug' on */
	ENGINE_API virtual AActor* GetCurrentDebugTargetActor();
	
	/** Utility function to add an actor to our consideration list for 'showdebug' 
	 * Only consider visible, non destroyed Actors in the same world the player is in. */
	static ENGINE_API void AddActorToDebugList(AActor* InActor, TArray<AActor*>& InOutList, UWorld* InWorld);

	/** Utility function to add a component's owner to our consideration list for 'showdebug' */
	static ENGINE_API void AddComponentOwnerToDebugList(UActorComponent* InComponent, TArray<AActor*>& InOutList, UWorld* InWorld);

	/** Get list of considered targets for 'showdebug'
	 * This list is built contextually based on which 'showdebug' flags have been enabled. */
	ENGINE_API virtual void GetDebugActorList(TArray<AActor*>& InOutList);

	/** Cycle to next target in our considered targets list for 'showdebug' */
	UFUNCTION(exec)
	ENGINE_API virtual void NextDebugTarget();

	/** Cycle to previous target in our considered targets list for 'showdebug' */
	UFUNCTION(exec)
	ENGINE_API virtual void PreviousDebugTarget();

	// Callback allowing external systems to register to show debug info
	static ENGINE_API FOnShowDebugInfo OnShowDebugInfo;

	// Called from ::PostRender. For less player/actor centered debugging
	static ENGINE_API FOnHUDPostRender OnHUDPostRender;

	/** PostRender is the main draw loop. */
	ENGINE_API virtual void PostRender();

	/** The Main Draw loop for the hud.  Gets called before any messaging.  Should be subclassed */
	ENGINE_API virtual void DrawHUD();

	//=============================================================================
	// Messaging.
	//=============================================================================

	/** @return UFont* from given FontSize index */
	ENGINE_API virtual  UFont* GetFontFromSizeIndex(int32 FontSizeIndex) const;

	/**
	 *	Pauses or unpauses the game due to main window's focus being lost.
	 *	@param Enable - tells whether to enable or disable the pause state
	 */
	ENGINE_API virtual void OnLostFocusPause(bool bEnable);

	/**
	 * Iterate through list of debug text and draw it over the associated actors in world space.
	 * Also handles culling null entries, and reducing the duration for timed debug text.
	 */
	ENGINE_API void DrawDebugTextList();

	/** Gives the HUD a chance to display project-specific data when taking a "bug" screen shot.	 */
	virtual void HandleBugScreenShot() { }

	/** Array of hitboxes for this frame. */
	TArray< FHUDHitBox >	HitBoxMap;
	
	/** Array of hitboxes that have been hit for this frame. */
	TArray< FHUDHitBox* >	HitBoxHits;

	/** Set of hitbox (by name) that are currently moused over or have a touch contacting them */
	TSet< FName > HitBoxesOver;

	/**
	 * Debug renderer for this frames hitboxes.
	 * @param	Canvas	Canvas on which to render debug boxes.
	 */
	ENGINE_API void RenderHitBoxes( FCanvas* InCanvas );
	
	/**
	 * Update the list of hitboxes and dispatch events for any hits.
	 * @param   ClickLocation	Location of the click event
	 * @param	InEventType		Type of input event that triggered the call.
	 */
	ENGINE_API bool UpdateAndDispatchHitBoxClickEvents(const FVector2D ClickLocation, const EInputEvent InEventType);
	
	/**
	 * Update a the list of hitboxes that have been hit this frame.
	 * @param	Canvas	Canvas on which to render debug boxes.
	 */
	ENGINE_API void UpdateHitBoxCandidates( TArray<FVector2D> InContactPoints );
	
	/** Have any hitboxes been hit this frame. */
	ENGINE_API bool AnyCurrentHitBoxHits() const;	

	/**
	 * Find the first hitbox containing the given coordinates.
	 * @param	InHitLocation		Coordinates to check
	 * @param	bConsumingInput		If true will return the first hitbox that would consume input at this coordinate
	 * @return	returns the hitbox at the given coordinates or NULL if none match.
	 */
	ENGINE_API const FHUDHitBox* GetHitBoxAtCoordinates( FVector2D InHitLocation, bool bConsumingInput = false ) const;

	/**
	* Finds all the hitboxes containing the given coordinates.
	* @param	InHitLocation		Coordinates to check
	* @param	OutHitBoxes			Reference parameter with hit box references at the coordinates
	*/
	ENGINE_API void GetHitBoxesAtCoordinates(FVector2D InHitLocation, TArray<const FHUDHitBox*>& OutHitBoxes) const;
	
	/**
	 * Return the hitbox with the given name
	 * @param	InName	Name of required hitbox
	 * @return returns the hitbox with the given name NULL if none match.
	 */
	ENGINE_API const FHUDHitBox* GetHitBoxWithName( const FName InName ) const;

protected:
	ENGINE_API bool IsCanvasValid_WarnIfNot() const;

private:
	// Helper function to deal with screen offset and splitscreen mapping of coordinates to HUD
	ENGINE_API FVector2D GetCoordinateOffset() const;

	/** Helper function to find the first and last index (if any) of elements from DebugTextList with the matching actor : 
	* @return number of elements found.
	*/
	ENGINE_API int32 FindDebugTextListIntervalForActor(AActor* InSrcActor, int32& OutFirstIdx, int32& OutLastIdx) const;
};



