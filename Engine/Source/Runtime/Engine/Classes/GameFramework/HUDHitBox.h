// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Hitbox used for clickable HUD elements.
 * These are named regions that receive various events when clicked upon.
 * @see AHud
 * @see AddHitBox(), ReceiveHitBoxClick(), ReceiveHitBoxRelease(), ReceiveHitBoxBeginCursorOver(), ReceiveHitBoxEndCursorOver()
 * @see NotifyHitBoxClick(), NotifyHitBoxRelease(), NotifyHitBoxBeginCursorOver(), NotifyHitBoxEndCursorOver()
 * @see GetHitBoxAtCoordinates(), GetHitBoxesAtCoordinates(), GetHitBoxWithName()
 */
class FHUDHitBox
{
public:
	/** 
	 * Constructor for a hitbox.
	 * @param	InCoords		Coordinates of top left of hitbox.
	 * @param	InSize			Size of the box.
	 */
	ENGINE_API FHUDHitBox( FVector2D InCoords, FVector2D InSize, const FName& InName, bool bInConsumesInput, int32 InPriority );

	/** 
	 * Are the given coordinates within this hitbox.
	 * @param	InCoords		Coordinates to check.
	 * @returns true if coordinates are within this hitbox.
	 */
	ENGINE_API bool Contains( FVector2D InCoords ) const;

	/** 
	 * Debug render for this hitbox.
	 * @param	InCanvas		Canvas on which to render.
	 * @param	InColor			Color to render the box.
	 */
	ENGINE_API void Draw( class FCanvas* InCanvas, const FLinearColor& InColor ) const;
	
	/** Get the name of this hitbox.  */
	const FName& GetName() const { return Name;};

	/** Should other boxes be processed if this box is clicked.  */
	const bool ConsumesInput() const { return bConsumesInput; };

	/** Get the priority of this hitbox.  */
	const int32 GetPriority() const { return Priority; };

private:
	/** Coordinates of top left of hitbox.  */
	FVector2D	Coords;

	/** Size of this hitbox.  */
	FVector2D	Size;
	
	/** The name of this hitbox.  */
	FName		Name;	

	/** Wether or not this hitbox should prevent hit checks to other hitboxes.  */
	bool bConsumesInput;

	/** The priority of this hitbox. Higher boxes are given priority. */
	int32 Priority;
};
