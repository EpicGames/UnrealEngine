// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Info, the root of all information holding classes.
 * Doesn't have any movement / collision related code.
  */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Info.generated.h"

/**
 * Info is the base class of an Actor that isn't meant to have a physical representation in the world, used primarily
 * for "manager" type classes that hold settings data about the world, but might need to be an Actor for replication purposes.
 */
UCLASS(abstract, hidecategories=(Input, Movement, Collision, Rendering, HLOD, WorldPartition, DataLayers, Transformation), showcategories=("Input|MouseInput", "Input|TouchInput"), MinimalAPI, NotBlueprintable)
class AInfo : public AActor
{
	GENERATED_BODY()

public:
	ENGINE_API AInfo(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	ENGINE_API virtual void PostLoad() override;
#endif

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
#endif

#if WITH_EDITORONLY_DATA
private:
	/** Billboard Component displayed in editor */
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;
public:
#endif

	/** Indicates whether this actor should participate in level bounds calculations. */
	virtual bool IsLevelBoundsRelevant() const override { return false; }

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif

public:
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	ENGINE_API class UBillboardComponent* GetSpriteComponent() const;
#endif
};



