// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameFramework/InputDevicePropertyHandle.h"

#include "InputDeviceLibrary.generated.h"

enum class EInputDeviceConnectionState : uint8;
class APlayerController;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogInputDevices, Log, All);

/**
 * A static BP library that exposes Input Device data to blueprints
 *
 * @see IPlatformInputDeviceMapper
 * @note Keep any function comments up to date with those in GenericPlatformInputDeviceMapper.h!
 */
UCLASS(MinimalAPI)
class UInputDeviceLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get the player controller who has the given Platform User ID. */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	static ENGINE_API APlayerController* GetPlayerControllerFromPlatformUser(const FPlatformUserId UserId);

	/** Get the player controller who owns the given input device id */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	static ENGINE_API APlayerController* GetPlayerControllerFromInputDevice(const FInputDeviceId DeviceId);

	/** Returns true if the given handle is valid */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input Devices", meta = (ReturnDisplayName = "Is Valid"))
	static ENGINE_API bool IsDevicePropertyHandleValid(const FInputDevicePropertyHandle& InHandle);
	
	/**
	 * Populates the OutInputDevices array with any InputDeviceID's that are mapped to the given platform user
	 *
	 * @param UserId				The Platform User to gather the input devices of.
	 * @param OutInputDevices		Array of input device ID's that will be populated with the mapped devices.
	 * @return						The number of mapped devices, INDEX_NONE if the user was not found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta=(ReturnDisplayName="Number of Devices"))
	static ENGINE_API int32 GetAllInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices);

	/**
	 * Get all mapped input devices on this platform regardless of their connection state.
	 * 
	 * @param OutInputDevices	Array of input devices to populate
	 * @return					The number of connected input devices
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta=(ReturnDisplayName="Number of Devices"))
	static ENGINE_API int32 GetAllInputDevices(TArray<FInputDeviceId>& OutInputDevices);

	/**
	 * Gather all currently connected input devices
	 * 
	 * @param OutInputDevices	Array of input devices to populate
	 * @return					The number of connected input devices
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta=(ReturnDisplayName="Number of Devices"))
	static ENGINE_API int32 GetAllConnectedInputDevices(TArray<FInputDeviceId>& OutInputDevices);

	/**
	 * Get all currently active platform ids, anyone who has a mapped input device
	 *
	 * @param OutUsers		Array that will be populated with the platform users.
	 * @return				The number of active platform users
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta=(ReturnDisplayName="Number of Users"))
	static ENGINE_API int32 GetAllActiveUsers(TArray<FPlatformUserId>& OutUsers);

	/**
	 * Returns the platform user id that is being used for unmapped input devices.
	 * Will be PLATFORMUSERID_NONE if platform does not support this (this is the default behavior)
	 */
	UFUNCTION(BlueprintPure, Category = "Input Devices")
	static ENGINE_API FPlatformUserId GetUserForUnpairedInputDevices();

	/**
	 * Returns the 'Primary' Platform user for this platform.
	 * This typically has an internal ID of '0' and is used as the default platform user to
	 * map devices such as the keyboard and mouse that don't get assigned unique ID's from their
	 * owning platform code.
	 */
	UFUNCTION(BlueprintPure, Category = "Input Devices")
	static ENGINE_API FPlatformUserId GetPrimaryPlatformUser();

	/** Returns true if the given Platform User Id is the user for unpaired input devices on this platform. */
	UFUNCTION(BlueprintPure, Category = "Input Devices")
	static ENGINE_API bool IsUnpairedUserId(const FPlatformUserId PlatformId);

	/** Returns true if the given input device is mapped to the unpaired platform user id. */
	UFUNCTION(BlueprintPure, Category = "Input Devices")
	static ENGINE_API bool IsInputDeviceMappedToUnpairedUser(const FInputDeviceId InputDevice);

	/** Returns the default device id used for things like keyboard/mouse input */
	UFUNCTION(BlueprintPure, Category = "Input Devices")
	static ENGINE_API FInputDeviceId GetDefaultInputDevice();

	/** Returns the platform user attached to this input device, or PLATFORMUSERID_NONE if invalid */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	static ENGINE_API FPlatformUserId GetUserForInputDevice(FInputDeviceId DeviceId);

	/** Returns the primary input device used by a specific player, or INPUTDEVICEID_NONE if invalid */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	static ENGINE_API FInputDeviceId GetPrimaryInputDeviceForUser(FPlatformUserId UserId);

	/**
	 * Gets the connection state of the given input device.
	 * 
	 * @param DeviceId		The device to get the connection state of
	 * @return				The connection state of the given device. EInputDeviceConnectionState::Unknown if the device is not mapped
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	static ENGINE_API EInputDeviceConnectionState GetInputDeviceConnectionState(const FInputDeviceId DeviceId);

	/**
	 * Check if the given input device is valid
	 * 
	 * @return	True if the given input device is valid (it has been assigned an ID by the PlatformInputDeviceMapper)
	 */
	UFUNCTION(BlueprintPure, Category = "Input Devices", meta=(ReturnDisplayName="Is Valid"))
	static ENGINE_API bool IsValidInputDevice(FInputDeviceId DeviceId);

	/**
	 * Check if the given platform ID is valid
	 * 
	 * @return	True if the platform ID is valid (it has been assigned by the PlatformInputDeviceMapper)
	 */
	UFUNCTION(BlueprintPure, Category = "Input Devices", meta=(ReturnDisplayName="Is Valid"))
	static ENGINE_API bool IsValidPlatformId(FPlatformUserId UserId);

	/** Static invalid platform user */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "None", ScriptConstantHost = "/Script/CoreUObject.PlatformUserId"), Category = "Input Devices")
	static ENGINE_API FPlatformUserId PlatformUserId_None();

	/** Static invalid input device */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "None", ScriptConstantHost = "/Script/CoreUObject.InputDeviceId"), Category = "Input Devices")
	static ENGINE_API FInputDeviceId InputDeviceId_None();

	/** Returns true if PlatformUserId A is equal to PlatformUserId B (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (PlatformUserId)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Input Devices")
	static ENGINE_API bool EqualEqual_PlatformUserId(FPlatformUserId A, FPlatformUserId B);

	/** Returns true if PlatformUserId A is not equal to PlatformUserId B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (PlatformUserId)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Input Devices")
	static ENGINE_API bool NotEqual_PlatformUserId(FPlatformUserId A, FPlatformUserId B);
	
	/** Returns true if InputDeviceId A is equal to InputDeviceId B (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (InputDeviceId)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Input Devices")
	static ENGINE_API bool EqualEqual_InputDeviceId(FInputDeviceId A, FInputDeviceId B);
	
	/** Returns true if InputDeviceId A is not equal to InputDeviceId B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (InputDeviceId)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Input Devices")
	static ENGINE_API bool NotEqual_InputDeviceId(FInputDeviceId A, FInputDeviceId B);
};
