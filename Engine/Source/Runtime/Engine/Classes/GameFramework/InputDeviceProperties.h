// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GenericPlatform/IInputInterface.h"
#include "GameFramework/InputDeviceSubsystem.h"

#include "InputDeviceProperties.generated.h"

class UCurveLinearColor;
class UCurveFloat;

#if WITH_EDITOR
	struct FPropertyChangedChainEvent;
#endif	// WITH_EDITOR

/**
* Base class that represents a single Input Device Property. An Input Device Property
* represents a feature that can be set on an input device. Things like what color a
* light is, advanced rumble patterns, or trigger haptics.
* 
* This top level object can then be evaluated at a specific time to create a lower level
* FInputDeviceProperty, which the IInputInterface implementation can interpret however it desires.
* 
* The behavior of device properties can vary depending on the current platform. Some platforms may not
* support certain device properties. An older gamepad may not have any advanced trigger haptics for 
* example. 
*/
UCLASS(Abstract, Blueprintable, BlueprintType, EditInlineNew, CollapseCategories, meta = (ShowWorldContextPin), MinimalAPI)
class UInputDeviceProperty : public UObject
{
	friend class UInputDeviceSubsystem;

	GENERATED_BODY()
public:

	ENGINE_API UInputDeviceProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:

	/**
	* Evaluate this device property for a given duration. 
	* If overriding in Blueprints, make sure to call the parent function!
	* 
 	* @param PlatformUser		The platform user that should receive this device property change
	* @param DeltaTime			Delta time
	* @param Duration			The number of seconds that this property has been active. Use this to get things like curve data over time.
	* @return					A pointer to the evaluated input device property.
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "InputDevice")
	ENGINE_API void EvaluateDeviceProperty(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration);

	/** 
	* Native C++ implementation of EvaluateDeviceProperty.
	* 
	* Override this to alter your device property in native code.
	* @see UInputDeviceProperty::EvaluateDeviceProperty
	*/
	ENGINE_API virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration);

	/**
	* Reset the current device property. Provides an opportunity to reset device state after evaluation is complete. 
	* If overriding in Blueprints, make sure to call the parent function!
	* 
	* @param PlatformUser		The platform user that should receive this device property change
	*/
	UFUNCTION(BlueprintNativeEvent, Category = "InputDevice")
	ENGINE_API void ResetDeviceProperty(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset = false);

	/**
	* Native C++ implementation of ResetDeviceProperty
	* Override this in C++ to alter the device property behavior in native code. 
	* 
	* @see ResetDeviceProperty
	*/
	ENGINE_API virtual void ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset = false);

	/**
	* Apply the device property from GetInternalDeviceProperty to the given platform user. 
	* Note: To remove any applied affects of this device property, call ResetDeviceProperty.
	* 
	* @param UserId		The owning Platform User whose input device this property should be applied to.
	*/
	UFUNCTION(Category = "InputDevice")
	ENGINE_API virtual void ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId);
	
	/** Gets a pointer to the current input device property that the IInputInterface can use. */
	virtual FInputDeviceProperty* GetInternalDeviceProperty() { return nullptr; };
	
public:

	/**
	* The duration that this device property should last. Override this if your property has any dynamic curves 
	* to be the max time range.
	*/
	ENGINE_API float GetDuration() const;
	
	/**
	 * Recalculates this device property's duration. This should be called whenever there are changes made
	 * to things like curves, or other time sensitive properties.
	 */
	ENGINE_API virtual float RecalculateDuration();

	// Post edit change property to update the duration if there are any dynamic options like for curves
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif	// WITH_EDITOR

protected:

	/**
	* Apply the given device property
	*
	* @param UserId			The owning Platform User whose input device this property should be applied to.
	* @param RawProperty	The internal input device property to apply.
	*/
	static ENGINE_API void ApplyDeviceProperty_Internal(const FPlatformUserId UserId, const FInputDeviceId DeviceId, FInputDeviceProperty* RawProperty);

	/** Returns the device specific data for the given platform user. Returns the default data if none are given */
	template<class TDataLayout>
	const TDataLayout* GetDeviceSpecificData(const FPlatformUserId UserId, const FInputDeviceId DeviceId, const TMap<FName, TDataLayout>& InMap) const;

	template<class TDataLayout>
	TDataLayout* GetDeviceSpecificDataMutable(const FPlatformUserId UserId, const FInputDeviceId DeviceId, TMap<FName, TDataLayout>& InMap)
	{
		return GetDeviceSpecificData<TDataLayout>(UserId, DeviceId, InMap);
	}

	/**
	* The duration that this device property should last. Override this if your property has any dynamic curves 
	* to be the max time range.
	* 
	* A duration of 0 means that the device property will be treated as a "One Shot" effect, being applied once
	* before being removed by the Input Device Subsystem.
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Info")
	float PropertyDuration = 0.0f;
};

template<class TDataLayout>
const TDataLayout* UInputDeviceProperty::GetDeviceSpecificData(const FPlatformUserId UserId, const FInputDeviceId DeviceId, const TMap<FName, TDataLayout>& InDeviceData) const
{
	if (const UInputDeviceSubsystem* SubSystem = UInputDeviceSubsystem::Get())
	{
		// If we were given a valid device, use that. Otherwise, default to the player's most recently used device
		const FHardwareDeviceIdentifier Hardware = DeviceId.IsValid() ? SubSystem->GetInputDeviceHardwareIdentifier(DeviceId) : SubSystem->GetMostRecentlyUsedHardwareDevice(UserId);
		
		// Check if there are any per-input device overrides available
		if (const TDataLayout* DeviceDetails = InDeviceData.Find(Hardware.HardwareDeviceIdentifier))
		{
			return DeviceDetails;
		}
	}

	return nullptr;
}

///////////////////////////////////////////////////////////////////////
// UColorInputDeviceProperty

/** Data required for setting the Input Device Color */
USTRUCT(BlueprintType)
struct FDeviceColorData
{
	GENERATED_BODY()

	/** True if the light should be enabled at all */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	bool bEnable = true;
	
	/** If true, the light color will be reset to "off" after this property has been evaluated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	bool bResetAfterCompletion = false;

	/** The color to set the light on  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	FColor LightColor = FColor::White;
};

/**
* Set the color of an Input Device to a static color. This will NOT reset the device color when the property
* is done evaluating. You can think of this as a "One Shot" effect, where you set the device property color.
* 
* NOTE: This property has platform specific implementations and may behave differently per platform.
* See the docs for more details on each platform.
*/
UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "Device Color (Static)"))
class UColorInputDeviceProperty : public UInputDeviceProperty
{
	GENERATED_BODY()

public:

	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual void ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset = false) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;

	/** Default color data that will be used by default. Device Specific overrides will be used when the current input device matches */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color")
	FDeviceColorData ColorData;

	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FDeviceColorData> DeviceOverrideData;

private:

	/** The internal light color property that this represents; */
	FInputDeviceLightColorProperty InternalProperty;
};


///////////////////////////////////////////////////////////////////////
// UColorInputDeviceCurveProperty

/** Data required for setting the Input Device Color */
USTRUCT(BlueprintType)
struct FDeviceColorCurveData
{
	GENERATED_BODY()

	/** True if the light should be enabled at all */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	bool bEnable = true;

	/** If true, the light color will be reset to "off" after the curve values are finished evaluating. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	bool bResetAfterCompletion = true;

	/** The color the device light should be */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color")
	TObjectPtr<UCurveLinearColor> DeviceColorCurve;
};

/** 
* A property that can be used to change the color of an input device's light over time with a curve
* 
* NOTE: This property has platform specific implementations and may behave differently per platform.
* See the docs for more details on each platform.
*/
UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "Device Color (Curve)"))
class UColorInputDeviceCurveProperty : public UInputDeviceProperty
{
	GENERATED_BODY()

public:

	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual void ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset = false) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	virtual float RecalculateDuration() override;

protected:
	/** Default color data that will be used by default. Device Specific overrides will be used when the current input device matches */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color")
	FDeviceColorCurveData ColorData;

	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FDeviceColorCurveData> DeviceOverrideData;

private:

	/** The internal light color property that this represents; */
	FInputDeviceLightColorProperty InternalProperty;
};


///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerEffect

USTRUCT(BlueprintType)
struct FDeviceTriggerBaseData
{
	GENERATED_BODY()

	/** Which trigger this property should effect */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Triggers")
	EInputDeviceTriggerMask AffectedTriggers = EInputDeviceTriggerMask::All;

	/** True if the triggers should be reset after the duration of this device property */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Triggers")
	bool bResetUponCompletion = true;
};

/** A property that effect the triggers on a gamepad */
UCLASS(Abstract, meta = (DisplayName = "Base Trigger Effect"), MinimalAPI)
class UInputDeviceTriggerEffect : public UInputDeviceProperty
{
	GENERATED_BODY()

public:	

	ENGINE_API virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	ENGINE_API virtual void ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset = false) override;
	ENGINE_API virtual void ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId) override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Triggers")
	FDeviceTriggerBaseData BaseTriggerData;

protected:

	/** Internal property that can be used to reset a given trigger */
	FInputDeviceTriggerResetProperty ResetProperty = {};
};

///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerFeedbackProperty

USTRUCT(BlueprintType)
struct FDeviceTriggerFeedbackData
{
	GENERATED_BODY()

	/** What position on the trigger that the feedback should be applied to over time (1-9) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<UCurveFloat> FeedbackPositionCurve;

	/** How strong the feedback is over time (1-8) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<UCurveFloat> FeedbackStrenghCurve;
};

/** 
* Sets simple trigger feedback
* 
* NOTE: This property has platform specific implementations and may behave differently per platform.
* See the docs for more details on each platform.
*/
UCLASS(Blueprintable, meta = (DisplayName = "Trigger Feedback"))
class UInputDeviceTriggerFeedbackProperty : public UInputDeviceTriggerEffect
{
	GENERATED_BODY()

public:
	
	UInputDeviceTriggerFeedbackProperty();
	
	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;	
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	virtual float RecalculateDuration() override;

	/** What position on the trigger that the feedback should be applied to over time (1-9) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger")
	FDeviceTriggerFeedbackData TriggerData;

	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Trigger", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FDeviceTriggerFeedbackData> DeviceOverrideData;

private:

	int32 GetPositionValue(const FDeviceTriggerFeedbackData* Data, const float Duration) const;
	int32 GetStrengthValue(const FDeviceTriggerFeedbackData* Data, const float Duration) const;

	/** The internal property that represents this trigger feedback. */
	FInputDeviceTriggerFeedbackProperty InternalProperty;
};

///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerResistanceProperty

USTRUCT(BlueprintType)
struct FDeviceTriggerTriggerResistanceData
{
	GENERATED_BODY()

	/** The position that the trigger should start providing resistance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty", meta = (UIMin = "0", UIMAX = "9"))
	int32 StartPosition = 0;

	/** How strong the resistance is */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty", meta = (UIMin = "0", UIMAX = "8"))
	int32 StartStrengh = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty", meta = (UIMin = "0", UIMAX = "9"))
	int32 EndPosition = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty", meta = (UIMin = "0", UIMAX = "8"))
	int32 EndStrengh = 0;
};

/** 
* Provides linear resistance to a trigger while it is being pressed between a start and end value
* 
* NOTE: This property has platform specific implementations and may behave differently per platform.
* See the docs for more details on each platform.
*/
UCLASS(Blueprintable, meta = (DisplayName = "Trigger Resistance"))
class UInputDeviceTriggerResistanceProperty : public UInputDeviceTriggerEffect
{
	GENERATED_BODY()

public:

	UInputDeviceTriggerResistanceProperty();

	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;

protected:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger")
	FDeviceTriggerTriggerResistanceData TriggerData;


	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Trigger", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FDeviceTriggerTriggerResistanceData> DeviceOverrideData;

private:

	/** The internal property that represents this trigger resistance */
	FInputDeviceTriggerResistanceProperty InternalProperty;
};


///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerVibrationProperty


USTRUCT(BlueprintType)
struct FDeviceTriggerTriggerVibrationData
{
	GENERATED_BODY()

	/** What position on the trigger that the feedback should be applied to over time (1-9) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<UCurveFloat> TriggerPositionCurve;

	/** The frequency of the vibration */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<UCurveFloat> VibrationFrequencyCurve;

	/** The amplitude of the vibration */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<UCurveFloat> VibrationAmplitudeCurve;
};

/**
* Sets trigger vibration
*
* NOTE: This property has platform specific implementations and may behave differently per platform.
* See the docs for more details on each platform.
*/
UCLASS(Blueprintable, meta = (DisplayName = "Trigger Vibration"))
class UInputDeviceTriggerVibrationProperty : public UInputDeviceTriggerEffect
{
	GENERATED_BODY()

public:

	UInputDeviceTriggerVibrationProperty();

	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	virtual float RecalculateDuration() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger")
	FDeviceTriggerTriggerVibrationData TriggerData;

	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Trigger", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FDeviceTriggerTriggerVibrationData> DeviceOverrideData;

private:

	int32 GetTriggerPositionValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const;
	int32 GetVibrationFrequencyValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const;
	int32 GetVibrationAmplitudeValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const;

	/** The internal property that represents this trigger feedback. */
	FInputDeviceTriggerVibrationProperty InternalProperty;
};

///////////////////////////////////////////////////////////////////////
// UInputDeviceSoundBasedVibrationProperty

class USoundBase;
class UEndpointSubmix;

USTRUCT(BlueprintType)
struct FAudioBasedVibrationData
{
	GENERATED_BODY()

	FAudioBasedVibrationData();
	
	/** The sound to play on the gamepad. Make sure the set the sound's submix sends to the gamepad audio and vibration endpoints! */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DeviceProperty")
	TObjectPtr<USoundBase> Sound;
};

/**
 * Plays a sound to an input device's speaker. On platforms that support it, this sound will be played
 * in the form of a vibration where the left and right audio channels represent the left and right side
 * of the controller.
 */
UCLASS(Blueprintable, meta = (DisplayName = "Audio Based Vibration (Experimental)"))
class UInputDeviceAudioBasedVibrationProperty : public UInputDeviceProperty
{
	GENERATED_BODY()

public:
	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual void ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	virtual float RecalculateDuration() override;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger")
	FAudioBasedVibrationData Data;

	/** A map of device specific color data. If no overrides are specified, the Default hardware data will be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Trigger", meta = (GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	TMap<FName, FAudioBasedVibrationData> DeviceOverrideData;

private:

	/** Returns the data that is relevant to the current input device */
	const FAudioBasedVibrationData* GetRelevantData(const FPlatformUserId UserId, const FInputDeviceId DeviceId) const;
};
