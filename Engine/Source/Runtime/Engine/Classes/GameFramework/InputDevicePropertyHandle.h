// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputDevicePropertyHandle.generated.h"

/** A handle to an active input device property that is being used by the InputDeviceSubsytem. */
USTRUCT(BlueprintType)
struct FInputDevicePropertyHandle
{
	friend class UInputDeviceSubsystem;

	GENERATED_BODY()

	ENGINE_API FInputDevicePropertyHandle();
	~FInputDevicePropertyHandle() = default;

	/** Returns true if this handle is valid */
	ENGINE_API bool IsValid() const;

	/** An invalid Input Device Property handle */
	static ENGINE_API FInputDevicePropertyHandle InvalidHandle;

	ENGINE_API uint32 GetTypeHash() const;

	ENGINE_API friend uint32 GetTypeHash(const FInputDevicePropertyHandle& InHandle);

	ENGINE_API bool operator==(const FInputDevicePropertyHandle& Other) const;
	ENGINE_API bool operator!=(const FInputDevicePropertyHandle& Other) const;

	ENGINE_API FString ToString() const;

private:

	// Private constructor because we don't want any other users to make a valid device property handle.
	ENGINE_API FInputDevicePropertyHandle(uint32 InternalID);

	/** Static function to get a unique device handle. */
	static ENGINE_API FInputDevicePropertyHandle AcquireValidHandle();

	/** The internal ID of this handle. Populated by the private constructor in AcquireValidHandle */
	UPROPERTY()
	uint32 InternalId;
};
