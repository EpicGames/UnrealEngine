// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputSettings.h"
#include "GameFramework/InputDevicePropertyHandle.h"
#include "Subsystems/EngineSubsystem.h"
#include "Tickable.h"
#include "Templates/SubclassOf.h"
#include "InputDeviceSubsystem.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogInputDeviceProperties, Log, All);

class UInputDeviceProperty;

/** Parameters for the UInputDeviceSubsystem::ActivateDeviceProperty function */
USTRUCT(BlueprintType)
struct FActivateDevicePropertyParams
{
	GENERATED_BODY()

	ENGINE_API FActivateDevicePropertyParams();
	
	/** The Platform User whose device's should receive the device property */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Activation Options")
	FPlatformUserId UserId;

	/** 
	* The Input Device that should receive the device property. If nothing is specified here,
	* then the Platform User's default input device will be used. 
	* 
	* The default input device is obtained from IPlatformInputDeviceMapper::GetPrimaryInputDeviceForUser
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Activation Options")
	FInputDeviceId DeviceId;

	/**
	* If true, then the input device property will not be removed after it's evaluation time has completed.
	* Instead, it will remain active until manually removed with a RemoveDeviceProperty call.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Activation Options")
	uint8 bLooping : 1;

	/** If true, then this device property will ignore dilated delta time and use the Applications delta time instead */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Activation Options")
	uint8 bIgnoreTimeDilation : 1;

	/** If true, then this device property will be played even if the game world is paused. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Activation Options")
	uint8 bPlayWhilePaused : 1;
};

/** Contains a pointer to an active device property and keeps track of how long it has been evaluated for */
USTRUCT()
struct FActiveDeviceProperty
{
	GENERATED_BODY()

	ENGINE_API FActiveDeviceProperty();
	
	/** Active properties can just use the hash of their FInputDevicePropertyHandle for a fast and unique lookup */
	ENGINE_API friend uint32 GetTypeHash(const FActiveDeviceProperty& InProp);
	ENGINE_API friend bool operator==(const FActiveDeviceProperty& ActiveProp, const FInputDevicePropertyHandle& Handle);
	ENGINE_API friend bool operator!=(const FActiveDeviceProperty& ActiveProp, const FInputDevicePropertyHandle& Handle);

	ENGINE_API bool operator==(const FActiveDeviceProperty& Other) const;
	ENGINE_API bool operator!=(const FActiveDeviceProperty& Other) const;

	/** The active device property */
	UPROPERTY()
	TWeakObjectPtr<UInputDeviceProperty> Property;

	/** How long this property has been evaluated for. DeltaTime is added to this on tick */
	double EvaluatedDuration;

	/** The platform user that is actively receiving this device property */
	FPlatformUserId PlatformUser;

	/** The input device id that should receive this property. */
	FInputDeviceId DeviceId;

	/** The handle of this active property. */
	FInputDevicePropertyHandle PropertyHandle;

	/**
	* If true, then the input device property will not be removed after it's evaluation time has completed.
	* Instead, it will remain active until manually removed with a RemoveDeviceProperty call.
	*/
	uint8 bLooping : 1;

	/** If true, then this device property will ignore dilated delta time and use the Applications delta time instead */
	uint8 bIgnoreTimeDilation : 1;

	/** If true, then this device property will be played even if the game world is paused. */
	uint8 bPlayWhilePaused : 1;

	/**
	 * This is set to true when this device property has been applied.
	 * All device properties should be applied at least one time, no matter what.
	 * This handles cases where the evaluation time is longer then the duration of a property.
	 * I.e., your property is set to a duration of 0.1, but you get a delta time of .12 seconds for some reason
	 * (choppy frames, low perf client, debugging, etc). 
	 */
	uint8 bHasBeenAppliedAtLeastOnce : 1;
};

/**
 * Delegate called when a user changed the hardware they are using for input.
 *
 * @param UserId		The Platform user whose device has changed
 */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FHardwareInputDeviceChanged, const FPlatformUserId, UserId, const FInputDeviceId, DeviceId);

/** Native delegate for non-UObject listeners for when new input device metadata is available. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FHardwareInputDeviceChangedNative, FPlatformUserId, FInputDeviceId);

/**
* The input device subsystem provides an interface to allow users to set Input Device Properties
* on any Platform User. 
*/
UCLASS(BlueprintType, MinimalAPI)
class UInputDeviceSubsystem : public UEngineSubsystem, public FTickableGameObject
{
	friend class FInputDeviceSubsystemProcessor;
	friend class FInputDeviceDebugTools;
	
	GENERATED_BODY()
public:

	/**
	 * Returns a pointer to the Input Device Engine Subsystem if it is available.
	 */
	static ENGINE_API UInputDeviceSubsystem* Get();

	//~ Begin UEngineSubsystem interface
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	ENGINE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	//~ End UEngineSubsystem interface
	
	//~ Begin FTickableGameObject interface	
	ENGINE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	ENGINE_API virtual ETickableTickType GetTickableTickType() const override;
	ENGINE_API virtual bool IsTickable() const override;
	ENGINE_API virtual bool IsTickableInEditor() const override;
	ENGINE_API virtual TStatId GetStatId() const override;
	ENGINE_API virtual void Tick(float InDeltaTime) override;
	//~ End FTickableGameObject interface
	
	/**
	 * Starts tracking the given device property as an "Active" property. This means that the property will be evaluted and applied to its platform user
	 *
	 * NOTE: This does NOT make a new instance of the given property. If you pass in the same object before it is completely
	 * evaluated, then you see undesired effects.
	 */
	ENGINE_API FInputDevicePropertyHandle ActivateDeviceProperty(UInputDeviceProperty* Property, const FActivateDevicePropertyParams& Params);

	/** Spawn a new instance of the given device property class and activate it. */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta = (AutoCreateRefTerm = "Params", ReturnDisplayName = "Device Property Handle"))
	ENGINE_API FInputDevicePropertyHandle ActivateDevicePropertyOfClass(TSubclassOf<UInputDeviceProperty> PropertyClass, const FActivateDevicePropertyParams& Params);
	
	/** Returns a pointer to the active input device property with the given handle. Returns null if the property doesn't exist */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta=(ReturnDisplayName="Device Property"))
	ENGINE_API UInputDeviceProperty* GetActiveDeviceProperty(const FInputDevicePropertyHandle Handle) const;
	
	/** Returns true if the property associated with the given handle is currently active, and it is not pending removal */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta = (ReturnDisplayName = "Is Active"))
	ENGINE_API bool IsPropertyActive(const FInputDevicePropertyHandle Handle) const;	

	/**
	* Remove a single device property based on it's handle
	*
	* @param FInputDevicePropertyHandle		Device property handle to be removed	
	*
	* @return								The number of removed device properties.
	*/
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	ENGINE_API void RemoveDevicePropertyByHandle(const FInputDevicePropertyHandle HandleToRemove);

	/**
	* Remove a set of device properties based on their handles. 
	* 
	* @param HandlesToRemove	The set of device property handles to remove
	* 
	* @return					The number of removed device properties
	*/
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	ENGINE_API void RemoveDevicePropertyHandles(const TSet<FInputDevicePropertyHandle>& HandlesToRemove);

	/** Removes all the current Input Device Properties that are active, regardless of the Platform User */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	ENGINE_API void RemoveAllDeviceProperties();

	/** Gets the most recently used hardware input device for the given platform user */
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	ENGINE_API FHardwareDeviceIdentifier GetMostRecentlyUsedHardwareDevice(const FPlatformUserId InUserId) const;

	/**
	 * Returns the most recently used FInputDeviceId for the given platform user id.
	 * 
	 * This will be INPUTDEVICEID_NONE if there is no known device for the given user.
	 * 
	 * @param InUserId	The platform user to find the most recent device for
	 * @param OfType	The type of input device to filter by. If unspecified, then the latest device regardless of its type will be returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "Input Devices", meta = (ReturnDisplayName = "Latest Device Id"))
	ENGINE_API FInputDeviceId GetMostRecentlyUsedInputDeviceId(const FPlatformUserId InUserId, const EHardwareDevicePrimaryType OfType = EHardwareDevicePrimaryType::Unspecified) const;
	
	UFUNCTION(BlueprintCallable, Category = "Input Devices")
	ENGINE_API FHardwareDeviceIdentifier GetInputDeviceHardwareIdentifier(const FInputDeviceId InputDevice) const;

	/**
	 * Returns the latest used input device for a specific type of hardware
	 */
	ENGINE_API FInputDeviceId GetLatestDeviceOfType(const FPlatformUserId InUserId, const EHardwareDevicePrimaryType OfType) const;

	/** A delegate that is fired when a platform user changes what Hardware Input device they are using */
	UPROPERTY(BlueprintAssignable, Category = "Input Devices")
	FHardwareInputDeviceChanged OnInputHardwareDeviceChanged;

	/** Native companion to OnInputHardwareDeviceChanged — supports AddRaw/AddWeakLambda for non-UObject listeners. */
	FHardwareInputDeviceChangedNative OnInputHardwareDeviceChangedNative;
	
protected:

	void HandleDeviceConnectionChange(EInputDeviceConnectionState NewState, FPlatformUserId User, FInputDeviceId DeviceId);

	/**
	 * Handler subscribed to FInputDeviceRegistry::OnInputDeviceDescriptorChanged. Routes a descriptor
	 * change into SetMostRecentlyUsedHardwareDevice so OnInputHardwareDeviceChanged becomes the single
	 * delegate gameplay code needs to subscribe to in order to be notified of device-identity changes
	 * (initial connect, descriptor refresh as the platform layer finalizes hardware metadata, etc).
	 */
	void HandleInputDeviceDescriptorChanged(const struct FInputDeviceDescriptor& Descriptor);

	/** Set the most recently used hardware device from the input processor */
	ENGINE_API void SetMostRecentlyUsedHardwareDevice(const FInputDeviceId InDeviceId, const FHardwareDeviceIdentifier& InHardwareId);

	// Callbacks for when PIE is started/stopped. We will likely want to pause/resume input device properties
	// when this happens, or just remove all active properties when PIE stops. This will make designer iteration a little easier
#if WITH_EDITOR
	ENGINE_API void OnPrePIEStarted(bool bSimulating);
	ENGINE_API void OnPIEPaused(bool bSimulating);
	ENGINE_API void OnPIEResumed(bool bSimulating);
	ENGINE_API void OnPIEStopped(bool bSimulating);

	bool bIsPIEPlaying = false;

#endif	// WITH_EDITOR

#if WITH_EDITORONLY_DATA

	struct FResetPIEData
	{
	 	TObjectPtr<UClass> ClassToReset;
	 	FInputDeviceId DeviceId;
	 	FPlatformUserId UserId;
	};
	
	/** 
	 * A set of device properties that should be reset when PIE ends.
	 * This ensures that each device is reset for the editor
	 **/
	TArray<FResetPIEData> PropertyClassesRequiringReset;

#endif	// WITH_EDITORONLY_DATA
	
	/**
	* Set of currently active input device properties that will be evaluated on tick
	*/
	UPROPERTY(Transient)
	TSet<FActiveDeviceProperty> ActiveProperties;

	/**
	 * Set of property handles the properties that are currently pending manual removal.
	 * This is populated by the "Remove device property" functions. 
	 */
	UPROPERTY(Transient)
	TSet<FInputDevicePropertyHandle> PropertiesPendingRemoval;
	
	/**
	 * A map of an input device to it's most recent hardware device identifier
	 */
	TMap<FInputDeviceId, FHardwareDeviceIdentifier> LatestInputDeviceIdentifiers;

	/**
	 * Data store about the input device that was most recently used by a specific FPlatformUserId
	 */
	struct FLatestInputDeviceData
	{
		/**
		 * The most recently used input device by this user
		 */
		FInputDeviceId LatestDeviceId = INPUTDEVICEID_NONE;
		
		/**
		 * A map of the latest used input device of each type for a single user.
		 * This allows us to query the most recently used device of a specific type for the player
		 */
		TMap<EHardwareDevicePrimaryType, FInputDeviceId> LatestDevices;
	};
	
	/** A map of platform user's to their most recent hardware device identifiers */
	TMap<FPlatformUserId, FLatestInputDeviceData> LatestUserDeviceIdentifiers;

	/** An input processor that is used to determine the most recently used hardware device for each user*/
	TSharedPtr<class FInputDeviceSubsystemProcessor> InputPreprocessor;
};