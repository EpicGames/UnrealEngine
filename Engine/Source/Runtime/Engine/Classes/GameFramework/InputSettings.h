// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "UObject/SoftObjectPath.h"
#include "GameFramework/PlayerInput.h"
#include "GenericPlatform/InputDeviceMappingPolicy.h"
#include "Components/InputComponent.h"
#include "Engine/PlatformSettings.h"

#include "InputSettings.generated.h"

/**
 * Project wide settings for input handling
 * 
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
UCLASS(config=Input, defaultconfig, MinimalAPI)
class UInputSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Properties of Axis controls */
	UPROPERTY(config, EditAnywhere, EditFixedSize, Category="Bindings", meta=(ToolTip="List of Axis Properties"), AdvancedDisplay)
	TArray<struct FInputAxisConfigEntry> AxisConfig;

	/**
	 * Platform specific settings for Input.
	 * @see UInputPlatformSettings
	 */
	UPROPERTY(EditAnywhere, Category = "Platforms")
	FPerPlatformSettings PlatformSettings;

	UPROPERTY(config, EditAnywhere, Category="Bindings", AdvancedDisplay)
	uint8 bAltEnterTogglesFullscreen:1;

	UPROPERTY(config, EditAnywhere, Category = "Bindings", AdvancedDisplay)
	uint8 bF11TogglesFullscreen : 1;

	// Allow mouse to be used for touch
	UPROPERTY(config, EditAnywhere, Category="MouseProperties", AdvancedDisplay)
	uint8 bUseMouseForTouch:1;

	// Mouse smoothing control
	UPROPERTY(config, EditAnywhere, Category="MouseProperties", AdvancedDisplay)
	uint8 bEnableMouseSmoothing:1;

	// Scale the mouse based on the player camera manager's field of view
	UPROPERTY(config, EditAnywhere, Category="MouseProperties", AdvancedDisplay)
	uint8 bEnableFOVScaling:1;

	/** Controls if the viewport will capture the mouse on Launch of the application */
	UPROPERTY(config, EditAnywhere, Category = "ViewportProperties")
	uint8 bCaptureMouseOnLaunch:1;

	/** Enable the use of legacy input scales on the player controller (InputYawScale, InputPitchScale, and InputRollScale) */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bEnableLegacyInputScales:1;
	
	/**
	 * If set to false, then the player controller's InputMotion function will never be called.
	 * This will effectively disable any motion input (tilt, rotation, acceleration, etc) on
	 * the GameViewportClient.
	 * 
	 * @see GameViewportClient::InputMotion
	 */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bEnableMotionControls:1;

	/**
	 * If true, then the PlayerController::InputKey function will only process an input event if it
	 * came from an input device that is owned by the PlayerController's Platform User.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bFilterInputByPlatformUser:1;

	/**
	 * If true, then the Player Controller will have it's Pressed Keys flushed when the input mode is changed
	 * to Game and UI mode or the game viewport loses focus. The default behavior is true.
	 * 
	 * @see UGameViewportClient::LostFocus
	 * @see APlayerController::ShouldFlushKeysWhenViewportFocusChanges
	 */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bShouldFlushPressedKeysOnViewportFocusLost:1;

	/**
	 * Should components that are dynamically added via the 'AddComponent' function at runtime have input delegates bound to them?
	 * @see AActor::FinishAddComponent
	 */
	UE_DEPRECATED(5.7, "bEnableDynamicComponentInputBinding is deprecated.")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "bEnableDynamicComponentInputBinding is deprecated, it will be treated as always being true."))
	uint8 bEnableDynamicComponentInputBinding:1;
	
	/** Should the touch input interface be shown always, or only when the platform has a touch screen? */
	UPROPERTY(config, EditAnywhere, Category="Mobile")
	uint8 bAlwaysShowTouchInterface:1;

	/** Whether or not to show the console on 4 finger tap, on mobile platforms */
	UPROPERTY(config, EditAnywhere, Category="Mobile")
	uint8 bShowConsoleOnFourFingerTap:1;

	/** Whether or not to use the gesture recognition system to convert touches in to gestures that can be bound and queried */
	UPROPERTY(config, EditAnywhere, Category = "Mobile")
	uint8 bEnableGestureRecognizer:1;

	/** If enabled, virtual keyboards will have autocorrect enabled. Currently only supported on mobile devices. */
	UPROPERTY(config, EditAnywhere, Category = "Virtual Keyboard (Mobile)")
	uint8 bUseAutocorrect:1;
	
	/**
	 * If true, then any input action or axis mappings which are marked as deprecated will throw a warning.
	 */
	UPROPERTY(config, EditAnywhere, Category="Bindings")
	uint8 bEnabledLegacyMappingDeprecationWarnings : 1;

	/**
	 * If true, then before creating an IInputDeviceModule, check for a list of "preferred"
	 * input API names set by the user in their GGameUserSettingsIni.
	 *
	 * This will check the following value:
	 * 
	 *		; In GGameUserSettingsIni
	 *		[GamepadAPIPreference]
	 *		PreferredAPI=Example,API,List
	 *
	 *	The "PreferredAPI" config value is a comma seperated string value which can be modified in your gameplay code.
	 *	If the value is not set, or empty, then the default value of "DefaultPreferredInputAPIList" will be checked.
	 *
	 * If the module is not contained in the comma-seperated list of preferred APIs,
	 * then the input API will not be used.
	 * 
	 * This may be a useful setting for your project if you would like to provide users with multiple options
	 * for their Input API to choose from to enjoy the best compatibility for their device. Because this setting
	 * is config driven, it will allow you to package your game with all the input plugins you would like to support
	 * enabled, but determine which to use at runtime, similar to how many projects allow you to select between different
	 * rendering APIs. 
	 *
	 * This feature is only for Desktop platforms, and is experimental in 5.8
	 */
	UE_EXPERIMENTAL(5.8, "This setting is experimental")
	UPROPERTY(config, EditAnywhere, Category = "Input", meta=(InlineEditConditionToggle))
	uint8 bEnablePreferredInputAPIPreferences : 1;

	/**
	 * This feature is only for Desktop platforms, and is experimental in 5.8
	 */
	UE_EXPERIMENTAL(5.8, "This setting is experimental")
	UPROPERTY(config, EditAnywhere, Category = "Input", AdvancedDisplay, meta=(EditCondition="bEnablePreferredInputAPIPreferences"))
	FString DefaultPreferredInputAPIList;

	/** 
	 * Disables autocorrect for these operating systems, even if autocorrect is enabled. Use the format "[platform] [osversion]"
	 * (e.g., "iOS 11.2" or "Android 6"). More specific versions will disable autocorrect for fewer devices ("iOS 11" will disable
	 * autocorrect for all devices running iOS 11, but "iOS 11.2.2" will not disable autocorrect for devices running 11.2.1).
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Virtual Keyboard (Mobile)")
	TArray<FString> ExcludedAutocorrectOS;

	/** Disables autocorrect for these cultures, even if autocorrect is turned on. These should be ISO-compliant language and country codes, such as "en" or "en-US". */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Virtual Keyboard (Mobile)")
	TArray<FString> ExcludedAutocorrectCultures;

	/** 
	 * Disables autocorrect for these device models, even if autocorrect is turned in. Model IDs listed here will match against the start of the device's
	 * model (e.g., "SM-" will match all device model IDs that start with "SM-"). This is currently only supported on Android devices.
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Virtual Keyboard (Mobile)")
	TArray<FString> ExcludedAutocorrectDeviceModels;

	/** The default mouse capture mode for the game viewport */
	UPROPERTY(config, EditAnywhere, Category = "ViewportProperties")
	EMouseCaptureMode DefaultViewportMouseCaptureMode;

	/** The default mouse lock state behavior when the viewport acquires capture */
	UPROPERTY(config, EditAnywhere, Category = "ViewportProperties")
	EMouseLockMode DefaultViewportMouseLockMode;

	// The scaling value to multiply the field of view by
	UPROPERTY(config, EditAnywhere, Category="MouseProperties", AdvancedDisplay, meta=(editcondition="bEnableFOVScaling"))
	float FOVScale;

	/** If a key is pressed twice in this amount of time it is considered a "double click" */
	UPROPERTY(config, EditAnywhere, Category="MouseProperties", AdvancedDisplay)
	float DoubleClickTime;

private:
	/** List of Action Mappings */
	UPROPERTY(config, EditAnywhere, Category="Bindings")
	TArray<struct FInputActionKeyMapping> ActionMappings;

	/** List of Axis Mappings */
	UPROPERTY(config, EditAnywhere, Category="Bindings")
	TArray<struct FInputAxisKeyMapping> AxisMappings;

	/**
	 * List of FNames, where each name is either an Action or Axis mapping, which are marked as deprecated.
	 * 
	 * Deprecated action and axis mappings will trigger Blueprint compilation warnings and log warnings when bound
	 * to in C++ code.
	 */
	UPROPERTY(config, VisibleAnywhere, Category="Bindings")
	TSet<FName> DeprecatedActionAndAxisNames;

	/** List of Speech Mappings */
	UE_DEPRECATED(5.7, "SpeechMappings are deprecated. These were only used by a now deprecated platform.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(config, EditAnywhere, Category = "Bindings")
	TArray<struct FInputActionSpeechMapping> SpeechMappings;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Default class type for player input object. May be overridden by player controller. */
	UPROPERTY(config, EditAnywhere, NoClear, Category = DefaultClasses)
	TSoftClassPtr<UPlayerInput> DefaultPlayerInputClass;

	/** Default class type for pawn input components. */
	UPROPERTY(config, EditAnywhere, NoClear, Category = DefaultClasses)
	TSoftClassPtr<UInputComponent> DefaultInputComponentClass;

public:
	/** The default on-screen touch input interface for the game (can be null to disable the onscreen interface) */
	UPROPERTY(config, EditAnywhere, Category="Mobile", meta=(AllowedClasses="/Script/Engine.TouchInterface"))
	FSoftObjectPath DefaultTouchInterface;

	/** The keys which open the console. */
	UPROPERTY(config, EditAnywhere, Category="Console")
	TArray<FKey> ConsoleKeys;

	// UObject interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	ENGINE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;

	ENGINE_API void RemoveInvalidKeys();

	ENGINE_API virtual void PostInitProperties() override;
	// End of UObject interface

	/** Returns the game local input settings (action mappings, axis mappings, etc...) */
	UFUNCTION(BlueprintPure, Category = Settings)
	static ENGINE_API UInputSettings* GetInputSettings();

	/** Programmatically add an action mapping to the project defaults */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void AddActionMapping(const FInputActionKeyMapping& KeyMapping, bool bForceRebuildKeymaps = true);

	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API void GetActionMappingByName(const FName InActionName, TArray<FInputActionKeyMapping>& OutMappings) const;

	/** Programmatically remove an action mapping to the project defaults */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void RemoveActionMapping(const FInputActionKeyMapping& KeyMapping, bool bForceRebuildKeymaps = true);

	/** Programmatically add an axis mapping to the project defaults */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void AddAxisMapping(const FInputAxisKeyMapping& KeyMapping, bool bForceRebuildKeymaps = true);

	/** Retrieve all axis mappings by a certain name. */
	UFUNCTION(BlueprintPure, Category = Settings)
	ENGINE_API void GetAxisMappingByName(const FName InAxisName, TArray<FInputAxisKeyMapping>& OutMappings) const;

	/** Programmatically remove an axis mapping to the project defaults */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void RemoveAxisMapping(const FInputAxisKeyMapping& KeyMapping, bool bForceRebuildKeymaps = true);

#if WITH_EDITOR	
	/**
	 * Returns all known legacy action and axis names that is useful for
	 * properties that you want a drop down selection of the available names
	 * on a UPROPERTY
	 * i.e.
	 *
	 * meta=(GetOptions="Engine.InputSettings.GetAllActionAndAxisNames")
	 */
	UFUNCTION()
	static ENGINE_API const TArray<FName>& GetAllActionAndAxisNames();
#endif// WITH_EDITOR

	/** Flush the current mapping values to the config file */
	UFUNCTION(BlueprintCallable, Category=Settings)
	ENGINE_API void SaveKeyMappings();

	/** Populate a list of all defined action names */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void GetActionNames(TArray<FName>& ActionNames) const;

	ENGINE_API bool IsActionMappingDeprecated(const FName ActionName) const;
	ENGINE_API void SetActionMappingDeprecated(const FName ActionName, const bool bIsDeprecated);

	/** Populate a list of all defined axis names */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void GetAxisNames(TArray<FName>& AxisNames) const;

	/** When changes are made to the default mappings, push those changes out to PlayerInput key maps */
	UFUNCTION(BlueprintCallable, Category = Settings)
	ENGINE_API void ForceRebuildKeymaps();

	/** Finds unique action name based on existing action names */
	ENGINE_API FName GetUniqueActionName(const FName BaseActionMappingName);
	/** Finds unique axis name based on existing action names */
	ENGINE_API FName GetUniqueAxisName(const FName BaseAxisMappingName);

	/** Append new mapping to existing list */
	ENGINE_API void AddActionMapping(FInputActionKeyMapping& NewMapping);
	/** Append new mapping to existing list */
	ENGINE_API void AddAxisMapping(FInputAxisKeyMapping& NewMapping);

	/** Ask for all the action mappings */
	ENGINE_API const TArray <FInputActionKeyMapping>& GetActionMappings() const;
	/** Ask for all the axis mappings */
	ENGINE_API const TArray <FInputAxisKeyMapping>& GetAxisMappings() const;

	/** Ask for all the speech mappings */
	UE_DEPRECATED(5.7, "Speech Mappings are deprecated. These were only used by a now deprecated platform.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ENGINE_API const TArray <FInputActionSpeechMapping>& GetSpeechMappings() const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Finds unique action name based on existing action names */
	ENGINE_API bool DoesActionExist(const FName InActionName);
	/** Finds unique axis name based on existing action names */
	ENGINE_API bool DoesAxisExist(const FName InAxisName);
	
	/** Finds unique speech name based on existing speech names */
	UE_DEPRECATED(5.7, "Speech Mappings are deprecated. These were only used by a now deprecated platform.")
	ENGINE_API bool DoesSpeechExist(const FName InSpeechName);


	/** Get the member name for the details panel */
	static ENGINE_API const FName GetActionMappingsPropertyName();
	/** Get the member name for the details panel */
	static ENGINE_API const FName GetAxisMappingsPropertyName();

	// Class accessors
	static ENGINE_API UClass* GetDefaultPlayerInputClass();
	static ENGINE_API UClass* GetDefaultInputComponentClass();
	
	/**
	 * Set the default player input class.
	 *
	 * @param NewDefaultPlayerInputClass The new class to use.
	 */
	static ENGINE_API void SetDefaultPlayerInputClass(TSubclassOf<UPlayerInput> NewDefaultPlayerInputClass);

	/**
	 * Set the default input component class.
	 *
	 * @param NewDefaultInputComponentClass The new class to use.
	 */
	static ENGINE_API void SetDefaultInputComponentClass(TSubclassOf<UInputComponent> NewDefaultInputComponentClass);
	
private:
	void PopulateAxisConfigs();
	void AddInternationalConsoleKey();
};


/**
 * A bitmask of supported features that a hardware device has. 
 */
UENUM(BlueprintType, Meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
namespace EHardwareDeviceSupportedFeatures
{
	enum Type : int32
	{
		/** A device that has not specified the type  */
		Unspecified			= 0x00000000,

		/** This device can support basic key presses */
		Keypress			= 0x00000001,

		/** This device can handle basic pointer behavior, such as a mouse */
		Pointer				= 0x00000002,

		/** This device has basic gamepad support */
		Gamepad				= 0x00000004,

		/** This device supports touch in some capactiy (tablet, controller with a touch pad, etc) */
		Touch				= 0x00000008,

		/** Does this device have a camera on it that we can access? */
		Camera				= 0x00000010,

		/** Can this device track motion in a 3D space? (VR controllers, headset, etc) */
		MotionTracking		= 0x00000020,

		/** This hardware supports setting a light color (such as an LED light bar) */
		Lights				= 0x00000040,

		/** Does this device have trigger haptics available? */
		TriggerHaptics		= 0x00000080,

		/** Flagged true if this device supports force feedback */
		ForceFeedback		= 0x00000100,

		/** Does this device support vibrations sourced from an audio file? */
		AudioBasedVibrations= 0x00000200,

		/** This device can track acceleration in the users physical space */
		Acceleration		= 0x00000400,

		/** This is a virtual device simulating input, not a physical device */
		Virtual				= 0x00000800,

		/** This device has a microphone on it that you can get audio from */
		Microphone			= 0x00001000,

		/** This device can track the orientation in world space, such as a gyroscope */
		Orientation			= 0x00002000,

		/** This device has the capabilities of a guitar (whammy bar, tilt, etc) */
		Guitar				= 0x00004000,

		/** This device has the capabilities of drums (symbols, foot pedal, etc) */
		Drums				= 0x00008000,

		/** Some custom flags that can be used in your game if you have custom hardware! */
		CustomA				= 0x01000000,
		CustomB				= 0x02000000,
		CustomC				= 0x04000000,
		CustomD				= 0x08000000,

		/** A flag for ALL supported device flags */
		All = 0x7FFFFFFF UMETA(Hidden),
	};
}

ENUM_CLASS_FLAGS(EHardwareDeviceSupportedFeatures::Type);

/**
 * What is the primary use of an input device type? 
 * Each hardware device can only be one primary type.
 */
UENUM(BlueprintType)
enum class EHardwareDevicePrimaryType : uint8
{
	Unspecified,
	KeyboardAndMouse,
	Gamepad,
	Touch,
	MotionTracking,
	RacingWheel,
	FlightStick,
	Camera,
	Instrument,

	// Some custom devices that can be used for your game specific hardware if desired
	CustomTypeA,
	CustomTypeB,
	CustomTypeC,
	CustomTypeD,
};

/**
* An identifier that can be used to determine what input devices are available based on the FInputDeviceScope.
* These mappings should match a FInputDeviceScope that is used by an IInputDevice
*/
USTRUCT(BlueprintType)
struct FHardwareDeviceIdentifier
{
	GENERATED_BODY()

	ENGINE_API FHardwareDeviceIdentifier();
	
	ENGINE_API FHardwareDeviceIdentifier(
		const FName InClassName,
		const FName InHardwareDeviceIdentifier,
		EHardwareDevicePrimaryType InPrimaryType = EHardwareDevicePrimaryType::Unspecified,
		EHardwareDeviceSupportedFeatures::Type Flags = EHardwareDeviceSupportedFeatures::Unspecified);
	
	/** 
	* The name of the Input Class that uses this hardware device.
	* This should correspond with a FInputDeviceScope that is used by an IInputDevice
	*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Hardware")
	FName InputClassName;

	/**
	 * The name of this hardware device. 
	 * This should correspond with a FInputDeviceScope that is used by an IInputDevice
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Hardware")
	FName HardwareDeviceIdentifier;

	/** The generic type that this hardware identifies as. This can be used to easily determine behaviors  */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Hardware")
	EHardwareDevicePrimaryType PrimaryDeviceType;
	
	/** Flags that represent this hardware device's traits */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Hardware", meta=(Bitmask, BitmaskEnum="/Script/Engine.EHardwareDeviceSupportedFeatures"))
	int32 SupportedFeaturesMask;

	/** Returns true if this hardware device has ANY of the given supported feature flags */
	ENGINE_API bool HasAnySupportedFeatures(const EHardwareDeviceSupportedFeatures::Type FlagsToCheck) const;

	/** Returns true if this hardware device has ALL of the given supported feature flags */
	ENGINE_API bool HasAllSupportedFeatures(const EHardwareDeviceSupportedFeatures::Type FlagsToCheck) const;

	/** Returns true if this hardware device Identifier has valid names */
	ENGINE_API bool IsValid() const;

	/**
	 * Returns a string containing the Input Class Name and HardwareDeviceIdentifier properties
	 * concatenated together.
	 */
	ENGINE_API FString ToString() const;
	
	/** An Invalid Hardware Device Identifier. */
	static ENGINE_API FHardwareDeviceIdentifier Invalid;

	/** Hardware device ID that represents a keyboard and mouse. This is what will be set when an Input Event's FKey is not a gamepad key. */
	static ENGINE_API FHardwareDeviceIdentifier DefaultKeyboardAndMouse;

	/** Hardware device ID that represents a default, generic, gamepad. */
	static ENGINE_API FHardwareDeviceIdentifier DefaultGamepad;

	/** Hardware device id that represents a default, generic, mobile touch input (tablet, phone, etc) */
	static ENGINE_API FHardwareDeviceIdentifier DefaultMobileTouch;

	ENGINE_API bool operator==(const FHardwareDeviceIdentifier& Other) const;
	ENGINE_API bool operator!=(const FHardwareDeviceIdentifier& Other) const;

	ENGINE_API friend uint32 GetTypeHash(const FHardwareDeviceIdentifier& InDevice);
	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FHardwareDeviceIdentifier& InDevice);
};

/** Per-Platform input options */
UCLASS(config=Input, defaultconfig, MinimalAPI)
class UInputPlatformSettings : public UPlatformSettings
{
	GENERATED_BODY()

public:

	ENGINE_API UInputPlatformSettings();
	
	static ENGINE_API UInputPlatformSettings* Get();

#if WITH_EDITOR
	/**
	* Returns an array of Hardware device names from every registered platform ini.
	* For use in the editor so that you can get a list of all known input devices and 
	* make device-specific options. For example, you can map any data type to a specific input 
	* device
	* 
	* UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(GetOptions="Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	* TMap<FString, UFooData> DeviceSpecificMap;
	* 
	* and the editor will make a nice drop down for you with all the current options that are in the settings.
	*/
	UFUNCTION()
	static ENGINE_API const TArray<FName>& GetAllHardwareDeviceNames();

	/**
	 * Returns all FHardwareDeviceIdentifier types known across all platform configurations.
	 * Unlike GetAllHardwareDeviceNames, this returns the full structs (including InputClassName
	 * and PrimaryDeviceType) so callers can build descriptors for device simulation.
	 */
	static ENGINE_API TArray<FHardwareDeviceIdentifier> GetAllHardwareDeviceIdentifiers();
#endif	// WITH_EDITOR


	/** Returns the first matching FHardwareDeviceIdentifier that has the given HardwareDeviceIdentifier.  */
	ENGINE_API const FHardwareDeviceIdentifier* GetHardwareDeviceForClassName(const FName InHardwareDeviceIdentifier) const;
	
	/** Add the given hardware device identifier to this platform's settings. */
	ENGINE_API void AddHardwareDeviceIdentifier(const FHardwareDeviceIdentifier& InHardwareDevice);

	/** Returns an array of all Hardware Device Identifiers known to this platform */
	ENGINE_API const TArray<FHardwareDeviceIdentifier>& GetHardwareDevices() const;

	////////////////////////////////////////////////////
	// Input Device Mapping

	/**
	* The max number of FPlatformUserId's which can be allocated on this platform.
	* 
	* This is checked in FGenericPlatformInputDeviceMapper::AllocateNewUserId
	*/
	UPROPERTY(config, EditAnywhere, Category = "Device Mapping")
	int32 MaxPlatformUserCount;

	/**
	 * The input device mapping policy controls how Human Interface Devices (HID's)
	 * are "mapped" or "assigned" to local players in your game. Depending on this
	 * policy, the creation of FPlatformUserId's and FInputDeviceId's can change.
	 *
	 * This policy is only in affect on platforms which do NOT have a managed user login
	 * or some kind of dedicated input mapping OS functionality (consoles have this).
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Mapping", meta=(ConsoleVariable="input.DeviceMappingPolicy"))
	EInputDeviceMappingPolicy DeviceMappingPolicy;
	
	////////////////////////////////////////////////////
	// Trigger Feedback
	
	/**
	 * The maximum position that a trigger can be set to
	 * 
	 * @see UInputDeviceTriggerFeedbackProperty
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Properties|Trigger Feedback", meta = (UIMin = "0"))
	int32 MaxTriggerFeedbackPosition;

	/**
	 * The maximum strength that trigger feedback can be set to
	 * 
	 * @see UInputDeviceTriggerFeedbackProperty
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Properties|Trigger Feedback", meta = (UIMin = "0"))
	int32 MaxTriggerFeedbackStrength;

	////////////////////////////////////////////////////
	// Trigger Vibrations
	
	/**
	 * The max position that a vibration trigger effect can be set to.
	 * 
	 * @see UInputDeviceTriggerVibrationProperty::GetTriggerPositionValue
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Properties|Trigger Vibration", meta = (UIMin = "0"))
	int32 MaxTriggerVibrationTriggerPosition;

	/**
	 * The max frequency that a trigger vibration can occur
	 * 
	 * @see UInputDeviceTriggerVibrationProperty::GetVibrationFrequencyValue
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Properties|Trigger Vibration", meta = (UIMin = "0"))
	int32 MaxTriggerVibrationFrequency;

	/**
	 * The maximum amplitude that can be set on trigger vibrations
	 * 
	 * @see UInputDeviceTriggerVibrationProperty::GetVibrationAmplitudeValue
	 */
	UPROPERTY(config, EditAnywhere, Category = "Device Properties|Trigger Vibration", meta = (UIMin = "0"))
	int32 MaxTriggerVibrationAmplitude;
	
protected:

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	/** A list of identifiable hardware devices available on this platform */
	UPROPERTY(config, EditAnywhere, Category = "Hardware")
	TArray<FHardwareDeviceIdentifier> HardwareDevices;
};
