// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/PhysicsVolume.h"
#include "KillZVolume.generated.h"

/**
* KillZVolume is a volume used to determine when actors should be killed. Killing logic is overridden in FellOutOfWorld
* 
* @see FellOutOfWorld
*/

UCLASS(MinimalAPI)
class AKillZVolume : public APhysicsVolume
{
	GENERATED_UCLASS_BODY()
	
	//Begin PhysicsVolume Interface
	ENGINE_API virtual void ActorEnteredVolume(class AActor* Other) override;
	//End PhysicsVolume Interface
};



