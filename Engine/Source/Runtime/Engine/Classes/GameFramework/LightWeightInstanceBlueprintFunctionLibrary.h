// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/ActorInstanceHandle.h"

#include "LightWeightInstanceBlueprintFunctionLibrary.generated.h"

UCLASS(MinimalAPI, Deprecated, meta = (DeprecationMessage ="Deprecated in 5.8. Consider using InstancedActors for similar functionality"))
class UDEPRECATED_LightWeightInstanceBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Returns a handle to a new light weight instance that represents an object of type ActorClass
	UFUNCTION(BlueprintCallable, Category = "Light Weight Instance")
	static ENGINE_API FActorInstanceHandle CreateNewLightWeightInstance(UClass* ActorClass, FTransform Transform, UDataLayerInstance* Layer, UWorld* World);

	// Returns a handle to the light weight representation and destroys Actor if successful; Returns a handle to Actor otherwise
	UFUNCTION(BlueprintCallable, Category = "Light Weight Instance")
	static ENGINE_API FActorInstanceHandle ConvertActorToLightWeightInstance(AActor* Actor);
};
