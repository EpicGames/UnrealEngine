// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/ActorInstanceHandle.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/SharedPointer.h"

#include "GameFramework/Actor.h"
#include "Engine/ActorInstanceManagerInterface.h"

#include "LightWeightInstanceManager.generated.h"


class AActor;
struct FActorSpawnParameters;

// Used for initializing light weight instances.
struct FLWIData
{
	FTransform Transform;
};


UCLASS(BlueprintType, Blueprintable, Experimental, MinimalAPI, Deprecated, meta = (DeprecationMessage ="Deprecated in 5.8. Consider using InstancedActors for similar functionality"))
class ADEPRECATED_LightWeightInstanceManager : public AActor, public IActorInstanceManagerInterface
{
	GENERATED_UCLASS_BODY()

	friend struct FLightWeightInstanceSubsystem;
	friend struct FActorInstanceHandle;
	friend class ULightWeightInstanceBlueprintFunctionLibrary;

public:
	ENGINE_API virtual ~ADEPRECATED_LightWeightInstanceManager();

	ENGINE_API virtual void Tick(float DeltaSeconds);

	// Returns the location of the instance specified by Handle
	ENGINE_API FVector GetLocation(const FActorInstanceHandle& Handle) const;

	// Returns the rotation of the instance specified by Handle
	ENGINE_API FRotator GetRotation(const FActorInstanceHandle& Handle) const;

	// Returns the name of the instance specified by Handle
	ENGINE_API FString GetName(const FActorInstanceHandle& Handle) const;

	// Returns true if this manager stores instances that can be turned into full weight objects of class OtherClass
	ENGINE_API bool DoesRepresentClass(const UClass* OtherClass) const;

	// Returns true if this manager is capable of representing objects of type OtherClass
	ENGINE_API bool DoesAcceptClass(const UClass* OtherClass) const;

	// Returns the base class of types that this can manage
	ENGINE_API UClass* GetAcceptedClass() const;

	// Sets the specific class that this manages
	ENGINE_API virtual void SetRepresentedClass(UClass* ActorClass);

	// Returns the index of the light weight instance associated with InActor if one exists; otherwise we return INDEX_NONE
	ENGINE_API int32 FindIndexForActor(const AActor* InActor) const;

	// Returns a handle to a light weight instance representing the same object as InActor and calls destroy on InActor if successful.
	ENGINE_API FActorInstanceHandle ConvertActorToLightWeightInstance(AActor* InActor);

	// LWI grid size to use for this manager.
	ENGINE_API virtual int32 GetGridSize() const;

	// Helper that converts a position (world space) into a coordinate for the LWI grid.
	ENGINE_API FInt32Vector3 ConvertPositionToCoord(const FVector& InPosition) const;

	// Helper that retrieve the world space bounds of the LWI grid cell encompassing the provided coordinate.
	ENGINE_API FBox ConvertPositionToGridBounds(const FVector& InPosition) const;

	ENGINE_API bool HasAnyValidInstancesOrManagedActors() const;

	// IActorInstanceManagerInterface begin
	ENGINE_API virtual int32 ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const;
	ENGINE_API virtual AActor* FindActor(const FActorInstanceHandle& Handle) override;
	ENGINE_API virtual AActor* FindOrCreateActor(const FActorInstanceHandle& Handle) override;
	virtual UClass* GetRepresentedClass(const int32 InstanceIndex) const override { return GetRepresentedClassInternal(); }
	virtual ULevel* GetLevelForInstance(const int32 InstanceIndex) const override { return GetLevel(); }
	ENGINE_API virtual FTransform GetTransform(const FActorInstanceHandle& Handle) const;
	// IActorInstanceManagerInterface end

protected:
	UClass* GetRepresentedClassInternal() const;

	// Creates an actor to replace the instance specified by Handle
	ENGINE_API AActor* ConvertInstanceToActor(const FActorInstanceHandle& Handle);

	ENGINE_API int32 AddNewInstance(FLWIData* InitData);

	// Adds a new instance at the specified index. This function should only be called by AddNewInstance.
	// Provided to allow subclasses to update their per instance data
	ENGINE_API virtual void AddNewInstanceAt(FLWIData* InitData, int32 Index);

	// Removes the instance
	ENGINE_API virtual void RemoveInstance(int32 Index);

	// Returns true if we have current information for an instance at Index
	ENGINE_API bool IsIndexValid(int32 Index) const;

	// Checks if we already have an actor for this handle. If an actor already exists we set the Actor variable on Handle and return true.
	ENGINE_API bool FindActorForHandle(const FActorInstanceHandle& Handle) const;

	// Sets the parameters for actor spawning.
	ENGINE_API virtual void SetSpawnParameters(FActorSpawnParameters& SpawnParams);

	// Returns the class to use when spawning a new actor
	ENGINE_API virtual UClass* GetActorClassToSpawn(const FActorInstanceHandle& Handle) const;

	// Called after actor construction but before other systems see this actor spawn
	ENGINE_API virtual void PreSpawnInitialization(const FActorInstanceHandle& Handle, AActor* SpawnedActor);

	// Called after spawning a new actor from a light weight instance
	ENGINE_API virtual void PostActorSpawn(const FActorInstanceHandle& Handle);

	// Called after a spawned actor is destroyed
	ENGINE_API virtual void OnSpawnedActorDestroyed(AActor* DestroyedActor, const int32 DestroyedActorInstanceIndex);

	// Helper functions for converting between our internal storage indices and the indices used by external bookkeeping
	virtual int32 ConvertInternalIndexToHandleIndex(int32 InInternalIndex) const { return InInternalIndex; }
	virtual int32 ConvertHandleIndexToInternalIndex(int32 InHandleIndex) const { return InHandleIndex; }

	//
	// Data and replication functions
	//

public:

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

private:
	UFUNCTION()
	ENGINE_API void OnSpawnedActorDestroyed(AActor* DestroyedActor);

protected:

	FString BaseInstanceName;

	UPROPERTY(EditDefaultsOnly, Replicated, Category=LightWeightInstance)
	TSubclassOf<AActor> RepresentedClass;

	UPROPERTY()
	TSubclassOf<AActor> AcceptedClass;

	//
	// Per instance data
	// Stored in separate arrays to make ticking more efficient when we need to update everything
	// 

	// Current per instance transforms
	UPROPERTY(ReplicatedUsing=OnRep_Transforms, EditInstanceOnly, Category=LightWeightInstance)
	TArray<FTransform> InstanceTransforms;
	UFUNCTION()
	ENGINE_API virtual void OnRep_Transforms();

	//
	// Subclasses should override the following functions if they need to store any additional data 
	//

	// Increases the size of all of our data arrays to NewSize
	ENGINE_API virtual void GrowDataArrays();

	// Populates the data array entries at Index with the data stored in InData;
	ENGINE_API virtual void UpdateDataAtIndex(FLWIData* InData, int32 Index);

	// Allocates the appropriate subclass of FLWIData to initialize instances for this manager. The caller is responsible for freeing the memory when they are finished.
	ENGINE_API virtual FLWIData* AllocateInitData() const;

	// Gathers info from InActor and stores it in InData. Returns true if the relevant data was successfully copied.
	ENGINE_API virtual bool SetDataFromActor(FLWIData* InData, AActor* InActor) const;

	//
	// Bookkeeping info
	//

	// keep track of which instances are currently represented by an actor
	UPROPERTY(Transient)
	TMap<int32, TObjectPtr<AActor>> Actors;

	TArray<int32> DestroyedActorIndices;

	// list of indices that we are no longer using
	UPROPERTY(Replicated)
	TArray<int32> FreeIndices;

	// handy way to check indices quickly so we don't need to iterate through the free indices list
	UPROPERTY(Replicated)
	TArray<bool> ValidIndices;
#if WITH_EDITOR
	//
	// Editor functions
	//
protected:
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;	
#endif // WITH_EDITOR

#if !UE_BUILD_SHIPPING
	TArray<int32> InstanceToActorConversionsInProgress;
#endif //!UE_BUILD_SHIPPING
};
