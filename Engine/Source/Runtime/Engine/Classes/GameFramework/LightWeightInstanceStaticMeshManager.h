// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"
#include "LightWeightInstanceManager.h"
#include "Elements/SMInstance/SMInstanceManager.h"

#include "LightWeightInstanceStaticMeshManager.generated.h"

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(FActorInstanceHandle, FOnActorReady, FActorInstanceHandle, InHandle);

UCLASS(BlueprintType, Blueprintable, Experimental, MinimalAPI, Deprecated, meta = (DeprecationMessage ="Deprecated in 5.8. Consider using InstancedActors for similar functionality"))
class ADEPRECATED_LightWeightInstanceStaticMeshManager : public ADEPRECATED_LightWeightInstanceManager, public ISMInstanceManager
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void SetRepresentedClass(UClass* ActorClass) override;

	// Sets the static mesh to use based on the info contained in InActor
	ENGINE_API virtual void SetStaticMeshFromActor(AActor* InActor);

	// Clears the static mesh used for rendering instances
	ENGINE_API void ClearStaticMesh();

	ENGINE_API virtual int32 ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const override;

protected:
	ENGINE_API virtual void AddNewInstanceAt(FLWIData* InitData, int32 Index) override;
	ENGINE_API virtual void RemoveInstance(int32 Index) override;

	ENGINE_API void AddInstanceToRendering(int32 DataIndex);
	ENGINE_API void RemoveInstanceFromRendering(int32 DataIndex);

	ENGINE_API void PostRemoveInstanceFromRendering();

	// sets the parameters on the instanced static mesh component
	ENGINE_API virtual void SetInstancedStaticMeshParams();

	// Called when we set the static mesh
	ENGINE_API void OnStaticMeshSet();

	ENGINE_API virtual void OnRep_Transforms() override;

	ENGINE_API virtual void PostActorSpawn(const FActorInstanceHandle& Handle) override;

public:

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PostLoad() override;
#endif

protected:

	//~ ISMInstanceManager interface
	ENGINE_API virtual FText GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const override;
	ENGINE_API virtual FText GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const override;
	ENGINE_API virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const override;
	ENGINE_API virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const override;
	ENGINE_API virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	ENGINE_API virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;
	ENGINE_API virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) override;
	ENGINE_API virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) override;
	ENGINE_API virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) override;

	ENGINE_API virtual void DuplicateLWIInstances(TArrayView<const int32> DataIndices, TArray<int32>& OutNewDataIndices);
	ENGINE_API void GetLWIDataIndices(TArrayView<const FSMInstanceId> InstanceIds, TArray<int32>& OutDataIndices) const;

	ENGINE_API virtual int32 ConvertInternalIndexToHandleIndex(int32 InternalIndex) const override;
	ENGINE_API virtual int32 ConvertHandleIndexToInternalIndex(int32 HandleIndex) const override;

protected:

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = Debug, ReplicatedUsing = OnRep_StaticMesh)
	TSoftObjectPtr<UStaticMesh> StaticMesh;
	UFUNCTION()
	ENGINE_API void OnRep_StaticMesh();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.4, "Doesn't need a UHierarchicalInstancedStaticMeshComponent anymore but is replaced by UInstancedStaticMeshComponent (see ISMComponent property).")
	UPROPERTY()
	TObjectPtr<class UHierarchicalInstancedStaticMeshComponent> InstancedStaticMeshComponent_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Debug, AdvancedDisplay, meta = (BlueprintProtected = "true", AllowPrivateAccess = "true"))
	TObjectPtr<class UInstancedStaticMeshComponent> ISMComponent;

	//
	// Bookkeeping info
	//

	// keep track of the relationship between our data and the rendering data stored in the instanced static mesh component
	UPROPERTY(Replicated)
	TArray<int32> RenderingIndicesToDataIndices;
	UPROPERTY(Replicated)
	TArray<int32> DataIndicesToRenderingIndices;

	// Data indices that we are going to delete later in the frame, we impose a small delay to group deletions to avoid cases where we might try to delete the same index multiple times in a frame.
	TArray<int32> DataIndicesToBeDeleted;
};
