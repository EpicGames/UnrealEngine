// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "GameFramework/LightWeightInstanceManager.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/SharedPointer.h"


ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogLightWeightInstance, Log, All);

class FAutoConsoleVariableRef;

struct FLightWeightInstanceSubsystem
{
	static FLightWeightInstanceSubsystem& Get()
	{
		static TSharedPtr<FLightWeightInstanceSubsystem, ESPMode::ThreadSafe> LWISubsystemSingleton = []
		{
			return MakeShareable(new FLightWeightInstanceSubsystem());
		}();
		return *LWISubsystemSingleton;
	}

	ENGINE_API FLightWeightInstanceSubsystem();
	ENGINE_API ~FLightWeightInstanceSubsystem();

	// Returns the instance manager that handles the given handle
	ENGINE_API ADEPRECATED_LightWeightInstanceManager* FindLightWeightInstanceManager(const FActorInstanceHandle& Handle) const;

	// Returns the instance manager that handles actors of type ActorClass in level Level
	ENGINE_API ADEPRECATED_LightWeightInstanceManager* FindLightWeightInstanceManager(UClass& ActorClass, UWorld& World, const FVector& InPos, const UDataLayerInstance* DataLayer = nullptr) const;

	// Returns the instance manager that handles instances of type Class that live in Level
	ENGINE_API ADEPRECATED_LightWeightInstanceManager* FindOrAddLightWeightInstanceManager(UClass& ActorClass, UWorld& World, const FVector& InPos, const UDataLayerInstance* DataLayer = nullptr);

	// Returns the actor specified by Handle if it exists. Returns nullptr if it doesn't
	ENGINE_API AActor* GetActor_NoCreate(const FActorInstanceHandle& Handle) const;

	// Returns the class of the actor specified by Handle.
	ENGINE_API UClass* GetActorClass(const FActorInstanceHandle& Handle);

	ENGINE_API FVector GetLocation(const FActorInstanceHandle& Handle);

	ENGINE_API FString GetName(const FActorInstanceHandle& Handle);

	ENGINE_API ULevel* GetLevel(const FActorInstanceHandle& Handle);

	// Returns true if the object represented by Handle is in InLevel
	ENGINE_API bool IsInLevel(const FActorInstanceHandle& Handle, const ULevel* InLevel);

	// Returns a handle to a new light weight instance that represents an object of type ActorClass
	ENGINE_API FActorInstanceHandle CreateNewLightWeightInstance(UClass* ActorClass, FLWIData* InitData, UDataLayerInstance* Layer, UWorld* World);

	// deletes the instance identified by Handle
	ENGINE_API void DeleteInstance(const FActorInstanceHandle& Handle);

	// Add a manager to the subsystem, thread safe.
	ENGINE_API bool AddManager(ADEPRECATED_LightWeightInstanceManager* Manager);

	// Remove a manager from the subsystem, thread safe.
	ENGINE_API bool RemoveManager(ADEPRECATED_LightWeightInstanceManager* Manager);

protected:
	// Returns the class of the instance manager best suited to support instances of type ActorClass
	ENGINE_API UClass* FindBestInstanceManagerClass(const UClass* ActorClass);

	// Returns the index associated with Manager
	ENGINE_API int32 GetManagerIndex(const ADEPRECATED_LightWeightInstanceManager* Manager) const;

	// Returns the light weight instance manager at index Index
	ENGINE_API const ADEPRECATED_LightWeightInstanceManager* GetManagerAt(int32 Index) const;

private:
	// TODO: preallocate the size of this based on a config variable
	TArray<ADEPRECATED_LightWeightInstanceManager*> LWInstanceManagers;

	// Mutex to make sure we don't change the LWInstanceManagers array while reading/writing it.
	mutable FTransactionallySafeRWLock LWIManagersRWLock;

#ifdef WITH_EDITOR
private:
	FDelegateHandle OnLevelActorAddedHandle;
	FDelegateHandle OnLevelActorDeletedHandle;
#endif // WITH_EDITOR
};
