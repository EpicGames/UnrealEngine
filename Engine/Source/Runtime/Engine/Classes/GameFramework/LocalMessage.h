// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// LocalMessage
//
// LocalMessages are abstract classes which contain an array of localized text.
// The PlayerController function ReceiveLocalizedMessage() is used to send messages
// to a specific player by specifying the LocalMessage class and index.  This allows
// the message to be localized on the client side, and saves network bandwidth since
// the text is not sent.  Actors (such as the GameMode) use one or more LocalMessage
// classes to send messages.  
//
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "LocalMessage.generated.h"

class APlayerController;
class APlayerState;

/** Handles the many pieces of data passed into Client Receive */
USTRUCT()
struct FClientReceiveData
{
	//always need to be here
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<APlayerController> LocalPC;

	UPROPERTY()
	FName MessageType;

	UPROPERTY()
	int32 MessageIndex;

	UPROPERTY()
	FString MessageString;

	UPROPERTY()
	TObjectPtr<APlayerState> RelatedPlayerState_1;

	UPROPERTY()
	TObjectPtr<APlayerState> RelatedPlayerState_2;

	UPROPERTY()
	TObjectPtr<UObject> OptionalObject;

	ENGINE_API FClientReceiveData();
};

UCLASS(abstract, MinimalAPI)
class ULocalMessage : public UObject
{
	GENERATED_UCLASS_BODY()
	/** send message to client */
	virtual void ClientReceive(const FClientReceiveData& ClientData) const PURE_VIRTUAL(ULocalMessage::ClientReceive,);
};



