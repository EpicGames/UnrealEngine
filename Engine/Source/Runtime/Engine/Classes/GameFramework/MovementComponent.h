// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Movement component updates position of associated PrimitiveComponent during its tick.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "MovementComponent.generated.h"

class AActor;
class APhysicsVolume;
class UPrimitiveComponent;
struct FCollisionQueryParams;
struct FCollisionResponseParams;
struct FCollisionShape;



namespace MovementComponentCVars
{
	// Typically we want to depenetrate regardless of direction, so we can get all the way out of penetration quickly.
	// Our rules for "moving with depenetration normal" only get us so far out of the object. We'd prefer to pop out by the full MTD amount.
	// Depenetration moves (in ResolvePenetration) then ignore blocking overlaps to be able to move out by the MTD amount.
	extern int32 MoveIgnoreFirstBlockingOverlap;

	extern float PenetrationOverlapCheckInflation;

	extern float PenetrationPullbackDistance;
}


/**
 * Setting that controls behavior when movement is restricted to a 2D plane defined by a specific axis/normal,
 * so that movement along the locked axis is not be possible.
 */
UENUM(BlueprintType)
enum class EPlaneConstraintAxisSetting : uint8
{
	/** Lock movement to a user-defined axis. */
	Custom,
	/** Lock movement in the X axis. */
	X,
	/** Lock movement in the Y axis. */
	Y,
	/** Lock movement in the Z axis. */
	Z,
	/** Use the global physics project setting. */
	UseGlobalPhysicsSetting
};


/**
 * MovementComponent is an abstract component class that defines functionality for moving a PrimitiveComponent (our UpdatedComponent) each tick.
 * Base functionality includes:
 *    - Restricting movement to a plane or axis.
 *    - Utility functions for special handling of collision results (SlideAlongSurface(), ComputeSlideVector(), TwoWallAdjust()).
 *    - Utility functions for moving when there may be initial penetration (SafeMoveUpdatedComponent(), ResolvePenetration()).
 *    - Automatically registering the component tick and finding a component to move on the owning Actor.
 * Normally the root component of the owning actor is moved, however another component may be selected (see SetUpdatedComponent()).
 * During swept (non-teleporting) movement only collision of UpdatedComponent is considered, attached components will teleport to the end location ignoring collision.
 */
UCLASS(ClassGroup=Movement, abstract, BlueprintType, MinimalAPI)
class UMovementComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	/**
	 * The component we move and update.
	 * If this is null at startup and bAutoRegisterUpdatedComponent is true, the owning Actor's root component will automatically be set as our UpdatedComponent at startup.
	 * @see bAutoRegisterUpdatedComponent, SetUpdatedComponent(), UpdatedPrimitive
	 */
	UPROPERTY(BlueprintReadOnly, Transient, DuplicateTransient, Category=MovementComponent)
	TObjectPtr<USceneComponent> UpdatedComponent;

	/**
	 * UpdatedComponent, cast as a UPrimitiveComponent. May be invalid if UpdatedComponent was null or not a UPrimitiveComponent.
	 */
	UPROPERTY(BlueprintReadOnly, Transient, DuplicateTransient, Category=MovementComponent)
	TObjectPtr<UPrimitiveComponent> UpdatedPrimitive;

	/**
	 * Flags that control the behavior of calls to MoveComponent() on our UpdatedComponent.
	 * @see EMoveComponentFlags
	 */
	EMoveComponentFlags MoveComponentFlags;

	/** Current velocity of updated component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Velocity)
	FVector Velocity;

protected:

	/**
	 * The normal or axis of the plane that constrains movement, if bConstrainToPlane is enabled.
	 * If for example you wanted to constrain movement to the X-Z plane (so that Y cannot change), the normal would be set to X=0 Y=1 Z=0.
	 * This is recalculated whenever PlaneConstraintAxisSetting changes. It is normalized once the component is registered with the game world.
	 * @see bConstrainToPlane, SetPlaneConstraintNormal(), SetPlaneConstraintFromVectors()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlanarMovement, meta=(editcondition=bConstrainToPlane))
	FVector PlaneConstraintNormal;

	/**
	 * The origin of the plane that constrains movement, if plane constraint is enabled. 
	 * This defines the behavior of snapping a position to the plane, such as by SnapUpdatedComponentToPlane().
	 * @see bConstrainToPlane, SetPlaneConstraintOrigin().
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlanarMovement, meta=(editcondition=bConstrainToPlane))
	FVector PlaneConstraintOrigin;

	/**
	 * Helper to compute the plane constraint axis from the current setting.
	 * 
	 * @param  AxisSetting Setting to use when computing the axis.
	 * @return Plane constraint axis/normal.
	 */
	ENGINE_API FVector GetPlaneConstraintNormalFromAxisSetting(EPlaneConstraintAxisSetting AxisSetting) const;

public:

	/** If true, skips TickComponent() if UpdatedComponent was not recently rendered. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MovementComponent)
	uint8 bUpdateOnlyIfRendered:1;

	/**
	 * If true, whenever the updated component is changed, this component will enable or disable its tick dependent on whether it has something to update.
	 * This will NOT enable tick at startup if bAutoActivate is false, because presumably you have a good reason for not wanting it to start ticking initially.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MovementComponent)
	uint8 bAutoUpdateTickRegistration:1;

	/**
	 * If true, after registration we will add a tick dependency to tick before our owner (if we can both tick).
	 * This is important when our tick causes an update in the owner's position, so that when the owner ticks it uses the most recent position without lag.
	 * Disabling this can improve performance if both objects tick but the order of ticks doesn't matter.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MovementComponent)
	uint8 bTickBeforeOwner:1;

	/** If true, registers the owner's Root component as the UpdatedComponent if there is not one currently assigned. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MovementComponent)
	uint8 bAutoRegisterUpdatedComponent:1;

	/**
	 * If true, movement will be constrained to a plane.
	 * @see PlaneConstraintNormal, PlaneConstraintOrigin, PlaneConstraintAxisSetting
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlanarMovement)
	uint8 bConstrainToPlane:1;

	/** If true and plane constraints are enabled, then the updated component will be snapped to the plane when first attached. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlanarMovement, meta=(editcondition=bConstrainToPlane))
	uint8 bSnapToPlaneAtStart:1;

	/**
	 * If true, then applies the value of bComponentShouldUpdatePhysicsVolume to the UpdatedComponent. If false, will not change bShouldUpdatePhysicsVolume on the UpdatedComponent at all.
	 * @see bComponentShouldUpdatePhysicsVolume
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MovementComponent)
	uint8 bAutoRegisterPhysicsVolumeUpdates:1;

	/**
	 * If true, enables bShouldUpdatePhysicsVolume on the UpdatedComponent during initialization from SetUpdatedComponent(), otherwise disables such updates.
	 * Only enabled if bAutoRegisterPhysicsVolumeUpdates is true.
	 * WARNING: UpdatePhysicsVolume is potentially expensive if overlap events are also *disabled* because it requires a separate query against all physics volumes in the world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MovementComponent, meta=(editcondition=bAutoRegisterPhysicsVolumeUpdates))
	uint8 bComponentShouldUpdatePhysicsVolume:1;

private:

#if WITH_EDITOR
	// Warned about trying to move something with static mobility.
	uint8 bEditorWarnedStaticMobilityMove:1;
#endif

	/** Transient flag indicating whether we are executing OnRegister(). */
	bool bInOnRegister;

	/** Transient flag indicating whether we are executing InitializeComponent(). */
	bool bInInitializeComponent;

	/**
	 * Setting that controls behavior when movement is restricted to a 2D plane defined by a specific axis/normal,
	 * so that movement along the locked axis is not be possible.
	 * @see SetPlaneConstraintAxisSetting
	 */
	UPROPERTY(EditAnywhere, Category=PlanarMovement, meta=(editcondition=bConstrainToPlane))
	EPlaneConstraintAxisSetting PlaneConstraintAxisSetting;

public:
	//~ Begin ActorComponent Interface 
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Deactivate() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Overridden to auto-register the updated component if it starts NULL, and we can find a root component on our owner. */
	ENGINE_API virtual void InitializeComponent() override;

	/** Overridden to update component properties that should be updated while being edited. */	
	ENGINE_API virtual void OnRegister() override;


#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	static ENGINE_API void PhysicsLockedAxisSettingChanged();
#endif // WITH_EDITOR

	//~ End ActorComponent Interface

	/** Returns gravity that affects this component */
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	ENGINE_API virtual float GetGravityZ() const;

	/** Returns maximum speed of component in current movement mode. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	virtual float GetMaxSpeed() const;

	/**
	 * Returns true if the current velocity is exceeding the given max speed (usually the result of GetMaxSpeed()), within a small error tolerance.
	 * Note that under normal circumstances updates cause by acceleration will not cause this to be true, however external forces or changes in the max speed limit
	 * can cause the max speed to be violated.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	ENGINE_API virtual bool IsExceedingMaxSpeed(float MaxSpeed) const;

	/** Stops movement immediately (zeroes velocity, usually zeros acceleration for components with acceleration). */
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	virtual void StopMovementImmediately();

	/**
	 * Possibly skip update if moved component is not rendered or can't move.
	 * @param DeltaTime @todo this parameter is not used in the function.
	 * @return true if component movement update should be skipped
	 */
	ENGINE_API virtual bool ShouldSkipUpdate(float DeltaTime) const;

	/** Returns the PhysicsVolume this MovementComponent is using, or the world's default physics volume if none. **/
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	ENGINE_API virtual APhysicsVolume* GetPhysicsVolume() const;

	/** Delegate when PhysicsVolume of UpdatedComponent has been changed **/
	UFUNCTION()
	ENGINE_API virtual void PhysicsVolumeChanged(class APhysicsVolume* NewVolume);

	/** Assign the component we move and update. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement")
	ENGINE_API virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent);

	/** Returns true if it's in PhysicsVolume with water flag **/
	ENGINE_API virtual bool IsInWater() const;

	/** Update tick registration state, determined by bAutoUpdateTickRegistration. Called by SetUpdatedComponent. */
	ENGINE_API virtual void UpdateTickRegistration();

	/** 
	 * Called for Blocking impact
	 * @param Hit: Describes the collision.
	 * @param TimeSlice: Time period for the simulation that produced this hit.  Useful for
	 *		  putting Hit.Time in context.  Can be zero in certain situations where it's not appropriate, 
	 *		  be sure to handle that.
	 * @param MoveDelta: Attempted move that resulted in the hit.
	 */
	ENGINE_API virtual void HandleImpact(const FHitResult& Hit, float TimeSlice=0.f, const FVector& MoveDelta = FVector::ZeroVector);

	/** Update ComponentVelocity of UpdatedComponent. This needs to be called by derived classes at the end of an update whenever Velocity has changed.	 */
	ENGINE_API virtual void UpdateComponentVelocity();

	/** Initialize collision params appropriately based on our collision settings. Use this before any Line, Overlap, or Sweep tests. */
	ENGINE_API virtual void InitCollisionParams(FCollisionQueryParams &OutParams, FCollisionResponseParams& OutResponseParam) const;

	/** Return true if the given collision shape overlaps other geometry at the given location and rotation. The collision params are set by InitCollisionParams(). */
	ENGINE_API virtual bool OverlapTest(const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor) const;

	/**
	 * Moves our UpdatedComponent by the given Delta, and sets rotation to NewRotation. Respects the plane constraint, if enabled.
	 * @note This simply calls the virtual MoveUpdatedComponentImpl() which can be overridden to implement custom behavior.
	 * @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat).
	 * @note The 'Teleport' flag is currently always treated as 'None' (not teleporting) when used in an active FScopedMovementUpdate.
	 * @return True if some movement occurred, false if no movement occurred. Result of any impact will be stored in OutHit.
	 */
	bool MoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation,    bool bSweep, FHitResult* OutHit = NULL, ETeleportType Teleport = ETeleportType::None);
	bool MoveUpdatedComponent(const FVector& Delta, const FRotator& NewRotation, bool bSweep, FHitResult* OutHit = NULL, ETeleportType Teleport = ETeleportType::None);

protected:

	ENGINE_API virtual bool MoveUpdatedComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit = NULL, ETeleportType Teleport = ETeleportType::None);

public:
	
	/**
	 * Moves our UpdatedComponent by the given Delta, and sets rotation to NewRotation.
	 * Respects the plane constraint, if enabled.
	 * @return True if some movement occurred, false if no movement occurred. Result of any impact will be stored in OutHit.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement", meta=(DisplayName = "MoveUpdatedComponent", ScriptName = "MoveUpdatedComponent", AdvancedDisplay="bTeleport"))
	ENGINE_API bool K2_MoveUpdatedComponent(FVector Delta, FRotator NewRotation, FHitResult& OutHit, bool bSweep = true, bool bTeleport = false);

	/**
	 * Calls MoveUpdatedComponent(), handling initial penetrations by calling ResolvePenetration().
	 * If this adjustment succeeds, the original movement will be attempted again.
	 * @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat).
	 * @note The 'Teleport' flag is currently always treated as 'None' (not teleporting) when used in an active FScopedMovementUpdate.
	 * @note When overriding, it is recommended to either call super or at least maintain the intended operation that includes resolving initial penetration.
	 * @note Also when overriding, make sure to include `using UMovementComponent::SafeMoveUpdatedComponent` next to the declaration. The FRotator version will be hidden otherwise and cause compilation issues when invoked.
	 * @return result of the final MoveUpdatedComponent() call.
	 */
	ENGINE_API virtual bool SafeMoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport = ETeleportType::None);
	bool SafeMoveUpdatedComponent(const FVector& Delta, const FRotator& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport = ETeleportType::None);

	/**
	 * Calculate a movement adjustment to try to move out of a penetration from a failed move.
	 * @param Hit the result of the failed move
	 * @return The adjustment to use after a failed move, or a zero vector if no attempt should be made.
	 */
	ENGINE_API virtual FVector GetPenetrationAdjustment(const FHitResult& Hit) const;
	
	/**
	 * Try to move out of penetration in an object after a failed move. This function should respect the plane constraint if applicable.
	 * @note This simply calls the virtual ResolvePenetrationImpl() which can be overridden to implement custom behavior.
	 * @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 * @param Adjustment	The requested adjustment, usually from GetPenetrationAdjustment()
	 * @param Hit			The result of the failed move
	 * @return True if the adjustment was successful and the original move should be retried, or false if no repeated attempt should be made.
	 */
	bool ResolvePenetration(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation);
	bool ResolvePenetration(const FVector& Adjustment, const FHitResult& Hit, const FRotator& NewRotation);

protected:

	ENGINE_API virtual bool ResolvePenetrationImpl(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation);

public:

	/**
	 * Compute a vector to slide along a surface, given an attempted move, time, and normal.
	 * @param Delta:	Attempted move.
	 * @param Time:		Amount of move to apply (between 0 and 1).
	 * @param Normal:	Normal opposed to movement. Not necessarily equal to Hit.Normal.
	 * @param Hit:		HitResult of the move that resulted in the slide.
	 */
	ENGINE_API virtual FVector ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit) const;

	/**
	 * Slide smoothly along a surface, and slide away from multiple impacts using TwoWallAdjust if necessary. Calls HandleImpact for each surface hit, if requested.
	 * Uses SafeMoveUpdatedComponent() for movement, and ComputeSlideVector() to determine the slide direction.
	 * @param Delta:	Attempted movement vector.
	 * @param Time:		Percent of Delta to apply (between 0 and 1). Usually equal to the remaining time after a collision: (1.0 - Hit.Time).
	 * @param Normal:	Normal opposing movement, along which we will slide.
	 * @param Hit:		[In] HitResult of the attempted move that resulted in the impact triggering the slide. [Out] HitResult of last attempted move.
	 * @param bHandleImpact:	Whether to call HandleImpact on each hit.
	 * @return The percentage of requested distance (Delta * Percent) actually applied (between 0 and 1). 0 if no movement occurred, non-zero if movement occurred.
	 */
	ENGINE_API virtual float SlideAlongSurface(const FVector& Delta, float Time, const FVector& Normal, FHitResult &Hit, bool bHandleImpact = false);

	/**
	 * Compute a movement direction when contacting two surfaces.
	 * @param Delta:		[In] Amount of move attempted before impact. [Out] Computed adjustment based on impacts.
	 * @param Hit:			Impact from last attempted move
	 * @param OldHitNormal:	Normal of impact before last attempted move
	 * @return Result in Delta that is the direction to move when contacting two surfaces.
	 */
	ENGINE_API virtual void TwoWallAdjust(FVector &Delta, const FHitResult& Hit, const FVector &OldHitNormal) const;

	/**
	 * Adds force from radial force components.
	 * Intended to be overridden by subclasses; default implementation does nothing.
	 * @param	Origin		The origin of the force
	 * @param	Radius		The radius in which the force will be applied
	 * @param	Strength	The strength of the force
	 * @param	Falloff		The falloff from the force's origin
	 */
	ENGINE_API virtual void AddRadialForce(const FVector& Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff);

	/**
	 * Adds impulse from radial force components.
	 * Intended to be overridden by subclasses; default implementation does nothing.
	 * @param	Origin		The origin of the force
	 * @param	Radius		The radius in which the force will be applied
	 * @param	Strength	The strength of the force
	 * @param	Falloff		The falloff from the force's origin
	 * @param	bVelChange	If true, the Strength is taken as a change in velocity instead of an impulse (ie. mass will have no effect).
	 */
	ENGINE_API virtual void AddRadialImpulse(const FVector& Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange);

	/**
	 * Set the plane constraint axis setting.
	 * Changing this setting will modify the current value of PlaneConstraintNormal.
	 * 
	 * @param  NewAxisSetting New plane constraint axis setting.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual void SetPlaneConstraintAxisSetting(EPlaneConstraintAxisSetting NewAxisSetting);

	/**
	 * Get the plane constraint axis setting.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	EPlaneConstraintAxisSetting GetPlaneConstraintAxisSetting() const;

	/**
	 * Sets the normal of the plane that constrains movement, enforced if the plane constraint is enabled.
	 * Changing the normal automatically sets PlaneConstraintAxisSetting to "Custom".
	 *
	 * @param PlaneNormal	The normal of the plane. If non-zero in length, it will be normalized.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual void SetPlaneConstraintNormal(FVector PlaneNormal);

	/** Uses the Forward and Up vectors to compute the plane that constrains movement, enforced if the plane constraint is enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual void SetPlaneConstraintFromVectors(FVector Forward, FVector Up);

	/** Sets the origin of the plane that constrains movement, enforced if the plane constraint is enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual void SetPlaneConstraintOrigin(FVector PlaneOrigin);
	
	/** Sets whether or not the plane constraint is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Components|Movement|Planar")
	ENGINE_API virtual void SetPlaneConstraintEnabled(bool bEnabled);

	/** Returns the normal of the plane that constrains movement, enforced if the plane constraint is enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API const FVector& GetPlaneConstraintNormal() const;

	/**
	 * Get the plane constraint origin. This defines the behavior of snapping a position to the plane, such as by SnapUpdatedComponentToPlane().
	 * @return The origin of the plane that constrains movement, if the plane constraint is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API const FVector& GetPlaneConstraintOrigin() const;

	/**
	 * Constrain a direction vector to the plane constraint, if enabled.
	 * @see SetPlaneConstraint
	 */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual FVector ConstrainDirectionToPlane(FVector Direction) const;

	/** Constrain a position vector to the plane constraint, if enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual FVector ConstrainLocationToPlane(FVector Location) const;

	/** Constrain a normal vector (of unit length) to the plane constraint, if enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual FVector ConstrainNormalToPlane(FVector Normal) const;

	/** Snap the updated component to the plane constraint, if enabled. */
	UFUNCTION(BlueprintCallable, Category="Components|Movement|Planar")
	ENGINE_API virtual void SnapUpdatedComponentToPlane();

	/** Called by owning Actor upon successful teleport from AActor::TeleportTo(). */
	virtual void OnTeleported() {};
};


//////////////////////////////////////////////////////////////////////////
// Inlines

inline float UMovementComponent::GetMaxSpeed() const
{
	return 0.f;
}

inline void UMovementComponent::StopMovementImmediately()
{
	Velocity = FVector::ZeroVector;
	UpdateComponentVelocity();
}

inline EPlaneConstraintAxisSetting UMovementComponent::GetPlaneConstraintAxisSetting() const
{
	return PlaneConstraintAxisSetting;
}

inline bool UMovementComponent::SafeMoveUpdatedComponent(const FVector& Delta, const FRotator& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport)
{
	return SafeMoveUpdatedComponent(Delta, NewRotation.Quaternion(), bSweep, OutHit, Teleport);
}

inline bool UMovementComponent::MoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* OutHit, ETeleportType Teleport)
{
	return MoveUpdatedComponentImpl(Delta, NewRotation, bSweep, OutHit, Teleport);
}

inline bool UMovementComponent::MoveUpdatedComponent(const FVector& Delta, const FRotator& NewRotation, bool bSweep, FHitResult* OutHit, ETeleportType Teleport)
{
	return MoveUpdatedComponentImpl(Delta, NewRotation.Quaternion(), bSweep, OutHit, Teleport);
}

inline bool UMovementComponent::ResolvePenetration(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation)
{
	return ResolvePenetrationImpl(Adjustment, Hit, NewRotation);
}

inline bool UMovementComponent::ResolvePenetration(const FVector& Adjustment, const FHitResult& Hit, const FRotator& NewRotation)
{
	return ResolvePenetrationImpl(Adjustment, Hit, NewRotation.Quaternion());
}
