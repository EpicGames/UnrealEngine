// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Movement component that is compatible with the navigation system's PathFollowingComponent
 */

#pragma once

#include "CoreMinimal.h"
#include "NavMovementInterface.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "GameFramework/MovementComponent.h"
#include "NavMovementComponent.generated.h"

class UCapsuleComponent;

/**
 * NavMovementComponent defines base functionality for MovementComponents that move any 'agent' that may be involved in AI pathfinding.
 */
UCLASS(abstract, config=Engine, MinimalAPI)
class UNavMovementComponent : public UMovementComponent, public INavMovementInterface
{
	GENERATED_UCLASS_BODY()

protected:
	/** Braking distance override used with acceleration driven path following (bUseAccelerationForPaths) */
	UE_DEPRECATED(5.5, "FixedPathBrakingDistance is deprecated, please use NavMovementProperties.FixedPathBrakingDistance instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "FixedPathBrakingDistance is deprecated, please use NavMovementProperties.FixedPathBrakingDistance instead."))
	float FixedPathBrakingDistance_DEPRECATED;

	/** If set to true NavAgentProps' radius and height will be updated with Owner's collision capsule size */
	UE_DEPRECATED(5.5, "bUpdateNavAgentWithOwnersCollision is deprecated, please use NavMovementProperties.bUpdateNavAgentWithOwnersCollision instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bUpdateNavAgentWithOwnersCollision is deprecated, please use NavMovementProperties.bUpdateNavAgentWithOwnersCollision instead."))
	uint8 bUpdateNavAgentWithOwnersCollision_DEPRECATED:1;

	/** If set, pathfollowing will control character movement via acceleration values. If false, it will set velocities directly. */
	UE_DEPRECATED(5.5, "bUseAccelerationForPaths is deprecated, please use NavMovementProperties.bUseAccelerationForPaths instead.")
	UPROPERTY(GlobalConfig, meta = (DeprecatedProperty, DeprecationMessage = "bUseAccelerationForPaths is deprecated, please use NavMovementProperties.bUseAccelerationForPaths instead."))
	uint8 bUseAccelerationForPaths_DEPRECATED : 1;

	/** If set, FixedPathBrakingDistance will be used for path following deceleration */
	UE_DEPRECATED(5.5, "bUseFixedBrakingDistanceForPaths is deprecated, please use NavMovementProperties.bUseFixedBrakingDistanceForPaths instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bUseFixedBrakingDistanceForPaths is deprecated, please use NavMovementProperties.bUseFixedBrakingDistanceForPaths instead."))
	uint8 bUseFixedBrakingDistanceForPaths_DEPRECATED : 1;

	/** If set, StopActiveMovement call will abort current path following request */
	UE_DEPRECATED(5.5, "bStopMovementAbortPaths is deprecated, please use NavMovementProperties.bStopMovementAbortPaths instead.")
	uint8 bStopMovementAbortPaths_DEPRECATED:1;
	
	UPROPERTY(EditAnywhere, Category = NavMovement)
	FNavMovementProperties NavMovementProperties;

public:
	/** Properties that define how the component can move. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NavMovement, meta = (DisplayName = "Movement Capabilities", Keywords = "Nav Agent"))
	FNavAgentProperties NavAgentProps;

	/** Expresses runtime state of character's movement. Put all temporal changes to movement properties here */
	UPROPERTY()
	FMovementProperties MovementState;

private:
	/** object implementing IPathFollowingAgentInterface. Is private to control access to it.
	 *	@see SetPathFollowingAgent, GetPathFollowingAgent */
	UPROPERTY()
	TObjectPtr<UObject> PathFollowingComp;

public:
	/** Get the owner of the object consuming nav movement */
	virtual UObject* GetOwnerAsObject() const override { return GetOwner(); }
	
	/** Get the component this movement component is updating */
	virtual TObjectPtr<UObject> GetUpdatedObject() const override { return UpdatedComponent; }

	/** Get axis-aligned cylinder around this actor, used for simple collision checks in nav movement */
	ENGINE_API virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const override;

	/** Returns collision extents vector for this object, based on GetSimpleCollisionCylinder. */
	ENGINE_API virtual FVector GetSimpleCollisionCylinderExtent() const override;
	
	/** Get forward vector of the object being driven by nav movement */
	ENGINE_API virtual FVector GetForwardVector() const override;
	
	/** Get the current velocity of the movement component */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual FVector GetVelocityForNavMovement() const override { return Velocity; }

	virtual float GetMaxSpeedForNavMovement() const override { return GetMaxSpeed(); }

	// Overridden to also call StopActiveMovement().
	virtual void StopMovementImmediately() override;

	ENGINE_API void SetUpdateNavAgentWithOwnersCollisions(bool bUpdateWithOwner);
	inline bool ShouldUpdateNavAgentWithOwnersCollision() const { return NavMovementProperties.bUpdateNavAgentWithOwnersCollision != 0; }

	/** Returns location of controlled actor - meaning center of collision bounding box */
	inline FVector GetActorLocation() const { return UpdatedComponent ? UpdatedComponent->GetComponentLocation() : FVector(FLT_MAX); }
	/** Returns location of controlled actor's "feet" meaning center of bottom of collision bounding box */
	virtual FVector GetActorFeetLocation() const { return UpdatedComponent ? (UpdatedComponent->GetComponentLocation() - FVector(0,0,UpdatedComponent->Bounds.BoxExtent.Z)) : FNavigationSystem::InvalidLocation; }
	/** Returns based location of controlled actor */
	ENGINE_API virtual FBasedPosition GetActorFeetLocationBased() const;
	/** Returns navigation location of controlled actor */
	inline FVector GetActorNavLocation() const { INavAgentInterface* MyOwner = Cast<INavAgentInterface>(GetOwner()); return MyOwner ? MyOwner->GetNavAgentLocation() : FNavigationSystem::InvalidLocation; }
	/** Returns the full world-coordinates transform of the associated scene component (the UpdatedComponent) */
	inline FTransform GetActorTransform() const { return UpdatedComponent ? UpdatedComponent->GetComponentTransform() : FTransform(); }

	/** Returns location of controlled actor - meaning center of collision bounding box */
	inline virtual FVector GetLocation() const override { return GetActorLocation(); }
	/** Returns location of controlled actor's "feet" meaning center of bottom of collision bounding box */
	virtual FVector GetFeetLocation() const override { return GetActorFeetLocation(); }
	/** Returns based location of controlled actor */
	virtual FBasedPosition GetFeetLocationBased() const override { return GetActorFeetLocationBased(); };

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR	
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	
	/** Set nav agent properties from an object */
	ENGINE_API virtual void UpdateNavAgent(const UObject& ObjectToUpdateFrom) override;
	
	/** path following: request new velocity */
	ENGINE_API virtual void RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed) override;

	/** path following: request new move input (normal vector = full strength) */
	ENGINE_API virtual void RequestPathMove(const FVector& MoveInput) override;

	/** check if current move target can be reached right now if positions are matching
	 *  (e.g. performing scripted move and can't stop) */
	ENGINE_API virtual bool CanStopPathFollowing() const override;

	/** Get Nav movement props struct this component uses */
	virtual FNavMovementProperties* GetNavMovementProperties() override { return &NavMovementProperties; }
	/** Returns the NavMovementProps(const) */
	virtual const FNavMovementProperties& GetNavMovementProperties() const override{ return NavMovementProperties; } 

	/** clears fixed braking distance */
	ENGINE_API void ClearFixedBrakingDistance();

	virtual void SetPathFollowingAgent(IPathFollowingAgentInterface* InPathFollowingAgent) override { PathFollowingComp = Cast<UObject>(InPathFollowingAgent); }
	virtual IPathFollowingAgentInterface* GetPathFollowingAgent() override { return Cast<IPathFollowingAgentInterface>(PathFollowingComp); }
	virtual const IPathFollowingAgentInterface* GetPathFollowingAgent() const override { return Cast<const IPathFollowingAgentInterface>(PathFollowingComp); }

	/** Returns the NavAgentProps(const) */
	inline const FNavAgentProperties& GetNavAgentPropertiesRef() const  override { return NavAgentProps; }
	/** Returns the NavAgentProps */
	inline FNavAgentProperties& GetNavAgentPropertiesRef()  override { return NavAgentProps; }

	/** Resets runtime movement state to character's movement capabilities */
	virtual void ResetMoveState() override { MovementState = NavAgentProps; }

	/** Returns true if path following can start */
	virtual bool CanStartPathFollowing() const override { return true; }

	/** Returns true if component is allowed to jump */
	inline bool IsJumpAllowed() const { return CanEverJump() && MovementState.bCanJump; }

	/** Sets whether this component is allowed to jump */
	inline void SetJumpAllowed(bool bAllowed) { MovementState.bCanJump = bAllowed; }

	/** Returns true if currently crouching */ 
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual bool IsCrouching() const override;

	/** Returns true if currently falling (not flying, in a non-fluid volume, and not on the ground) */ 
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual bool IsFalling() const override;

	/** Returns true if currently moving on the ground (e.g. walking or driving) */ 
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual bool IsMovingOnGround() const override;
	
	/** Returns true if currently swimming (moving through a fluid volume) */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual bool IsSwimming() const override;

	/** Returns true if currently flying (moving through a non-fluid volume without resting on the ground) */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	virtual bool IsFlying() const override;
};


inline bool UNavMovementComponent::IsCrouching() const
{
	return false;
}

inline bool UNavMovementComponent::IsFalling() const
{
	return false;
}

inline bool UNavMovementComponent::IsMovingOnGround() const
{
	return false;
}

inline bool UNavMovementComponent::IsSwimming() const
{
	return false;
}

inline bool UNavMovementComponent::IsFlying() const
{
	return false;
}

inline void UNavMovementComponent::StopMovementImmediately()
{
	Super::StopMovementImmediately();
	StopActiveMovement();
}
