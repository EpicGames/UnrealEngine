// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Engine/EngineTypes.h"
#include "AI/Navigation/NavigationTypes.h"

#include "NavMovementInterface.generated.h"

#define UE_API ENGINE_API

class IPathFollowingAgentInterface;

/**
 * Interface for navigation movement - should be implemented on movement objects that control an object directly
 */
UINTERFACE(MinimalAPI, NotBlueprintable)
class UNavMovementInterface : public UInterface
{
	GENERATED_BODY()
};

class INavMovementInterface
{
	GENERATED_BODY()

public:
	// Begin: Functions to be implemented by objects inheriting this interface
	/** Returns true if path following can start */
	virtual bool CanStartPathFollowing() const = 0;

	/** check if current move target can be reached right now if positions are matching
	 *  (e.g. performing scripted move and can't stop) */
	virtual bool CanStopPathFollowing() const = 0;

	/** Returns the Nav movement properties struct used by NavMovementInterface and PathFollowing */
	virtual FNavMovementProperties* GetNavMovementProperties() = 0;
	/** Returns the Nav movement properties struct (const) used by NavMovementInterface and PathFollowing */
	virtual const FNavMovementProperties& GetNavMovementProperties() const = 0;

	/** Returns the NavAgentProperties used by NavMovementInterface and PathFollowing */
	virtual FNavAgentProperties& GetNavAgentPropertiesRef() = 0;
	/** Returns the NavAgentProperties(const) used by NavMovementInterface and PathFollowing */
	virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const = 0;
	
	/** Set the path following agent this interface uses. Must implement IPathFollowingAgentInterface. */
	virtual void SetPathFollowingAgent(IPathFollowingAgentInterface* InPathFollowingAgent) = 0;
	/** Get path following agent this interface uses */
	virtual IPathFollowingAgentInterface* GetPathFollowingAgent() = 0;
	/** Get path following agent this interface uses(const) */
	virtual const IPathFollowingAgentInterface* GetPathFollowingAgent() const = 0;
	
	/** path following: request movement through a velocity directly */
	UFUNCTION(BlueprintCallable, Category="AI|NavMovement")
	virtual void RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed) = 0;

	/** path following: request movement through a new move input (normal vector = full strength) */
	UFUNCTION(BlueprintCallable, Category="AI|NavMovement")
	virtual void RequestPathMove(const FVector& MoveInput) = 0;

	/** Stops movement by setting velocity to zero - Note: depending on the movement system this may take effect next tick */
	virtual void StopMovementImmediately() = 0;

	/** Resets runtime movement state to default movement capabilities */
	virtual void ResetMoveState() = 0;
	
	// Movement specific code but not specific to nav movement - could probably be pulled out into a separate interface if desired
	/** Returns location of controlled agent - meaning center of collision shape */
	virtual FVector GetLocation() const = 0;
	
	/** Returns location of controlled agent's "feet" meaning center of bottom of collision shape */
	virtual FVector GetFeetLocation() const = 0;

	/** Returns based location of controlled agent */
	virtual FBasedPosition GetFeetLocationBased() const = 0;

	/** Get the owner of the object consuming nav movement */
	virtual UObject* GetOwnerAsObject() const = 0;
	
	/** Get the Object this movement interface is updating */
	virtual TObjectPtr<UObject> GetUpdatedObject() const = 0;

	/** Get axis-aligned cylinder around this agent, used for simple collision checks in nav movement */
	virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const = 0;

	/** Returns collision extents vector for this object */
	virtual FVector GetSimpleCollisionCylinderExtent() const = 0;
	
	/** Get forward vector of the object being driven by nav movement */
	virtual FVector GetForwardVector() const = 0;
	
	/** Set nav agent properties from an object */
	virtual void UpdateNavAgent(const UObject& ObjectToUpdateFrom) = 0;
	
	/** Get the current velocity of the agent for nav movement */
	UFUNCTION(BlueprintCallable, Category="AI|NavMovement")
	virtual FVector GetVelocityForNavMovement() const = 0;

	/** Get maximum movement speed of the agent */
	UFUNCTION(BlueprintCallable, Category="AI|NavMovement")
	virtual float GetMaxSpeedForNavMovement() const = 0;

	/** Returns true if currently crouching */ 
	UFUNCTION(BlueprintCallable, Category="Movement")
	virtual bool IsCrouching() const = 0;

	/** Returns true if currently falling (not flying, in a non-fluid volume, and not on the ground) */ 
	UFUNCTION(BlueprintCallable, Category="Movement")
	virtual bool IsFalling() const = 0;

	/** Returns true if currently moving on the ground (e.g. walking or driving) */ 
	UFUNCTION(BlueprintCallable, Category="Movement")
	virtual bool IsMovingOnGround() const = 0;
	
	/** Returns true if currently swimming (moving through a fluid volume) */
	UFUNCTION(BlueprintCallable, Category="Movement")
	virtual bool IsSwimming() const = 0;

	/** Returns true if currently flying (moving through a non-fluid volume without resting on the ground) */
	UFUNCTION(BlueprintCallable, Category="Movement")
	virtual bool IsFlying() const = 0;
	// End: Functions to be implemented by objects inheriting this interface
	
	
	// Begin: Functions to be implemented here for shared functionality
	/** Stops applying further movement (usually zeros acceleration). */
	UFUNCTION(BlueprintCallable, Category="AI|Movement")
	UE_API virtual void StopActiveMovement();

	/** Stops movement immediately (reset velocity) but keeps following current path */
	UFUNCTION(BlueprintCallable, Category="AI|Movement")
	UE_API virtual void StopMovementKeepPathing();
	
	/** Returns navigation location of controlled agent */
	UE_API FVector GetNavLocation() const;

	/** Returns braking distance for acceleration driven path following */
	UE_API virtual float GetPathFollowingBrakingDistance(float MaxSpeed) const;

	/** Set fixed braking distance */
	UE_API void SetFixedBrakingDistance(float DistanceToEndOfPath);

	/** Returns true if acceleration should be used for path following */
	UE_API bool UseAccelerationForPathFollowing() const;

	/** Returns true if agent can crouch */
	inline virtual bool CanEverCrouch() const { return GetNavAgentPropertiesRef().bCanCrouch; }

	/** Returns true if agent can jump */
	inline virtual bool CanEverJump() const { return GetNavAgentPropertiesRef().bCanJump; }

	/** Returns true if agent can move along the ground (walk, drive, etc) */
	inline virtual bool CanEverMoveOnGround() const { return GetNavAgentPropertiesRef().bCanWalk; }

	/** Returns true if agent can swim */
	inline virtual bool CanEverSwim() const { return GetNavAgentPropertiesRef().bCanSwim; }

	/** Returns true if agent can fly */
	inline virtual bool CanEverFly() const { return GetNavAgentPropertiesRef().bCanFly; }
	// End: Functions to be implemented here for shared functionality
};

#undef UE_API
