// Copyright Epic Games, Inc. All Rights Reserved.

//
// Unreal networking serialization helpers
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Online/CoreOnline.h"
#include "OnlineReplStructs.generated.h"

class FJsonValue;
enum class EUniqueIdEncodingFlags : uint8;

/**
 * Wrapper for opaque type FUniqueNetId
 *
 * Makes sure that the opaque aspects of FUniqueNetId are properly handled/serialized 
 * over network RPC and actor replication
 */
USTRUCT(BlueprintType, DisplayName = "Unique Net Id")
struct FUniqueNetIdRepl : public FUniqueNetIdWrapper
{
	GENERATED_USTRUCT_BODY()

	FUniqueNetIdRepl()
	{
	}

	FUniqueNetIdRepl(TYPE_OF_NULLPTR)
	{
	}

	FUniqueNetIdRepl(const FUniqueNetIdRepl& InWrapper)
		: FUniqueNetIdWrapper(InWrapper.Variant)
	{
	}

	FUniqueNetIdRepl(const FUniqueNetIdWrapper& InWrapper)
		: FUniqueNetIdWrapper(InWrapper)
	{
	}

	FUniqueNetIdRepl(const FUniqueNetIdRef& InUniqueNetId)
		: FUniqueNetIdWrapper(InUniqueNetId)
	{
	}
 
	FUniqueNetIdRepl(const FUniqueNetIdPtr& InUniqueNetId)
		: FUniqueNetIdWrapper(InUniqueNetId)
	{
	}
	
	FUniqueNetIdRepl(const FUniqueNetId& InUniqueNetId)
		: FUniqueNetIdWrapper(InUniqueNetId)
	{
	}

	virtual ~FUniqueNetIdRepl() = default;

	virtual void SetUniqueNetId(const FUniqueNetIdPtr& UniqueNetId) override
	{
		ReplicationBytes.Empty();
		FUniqueNetIdWrapper::SetUniqueNetId(UniqueNetId);
	}

	virtual void SetAccountId(const UE::Online::FAccountId& AccountId) override
	{
		ReplicationBytes.Empty();
		FUniqueNetIdWrapper::SetAccountId(AccountId);
	}

	/** Export contents of this struct as a string */
	bool ExportTextItem(FString& ValueStr, FUniqueNetIdRepl const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;

	/** Import string contexts and try to map them into a unique id */
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/** Network serialization */
	ENGINE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	void NetSerializeLoadV1Encoded(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess);
	void NetSerializeLoadV1Unencoded(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess);
	void NetSerializeLoadV2(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess);

	/** Serialization to any FArchive */
	ENGINE_API friend FArchive& operator<<( FArchive& Ar, FUniqueNetIdRepl& UniqueNetId);

	/** Serialization to any FArchive */
	bool Serialize(FArchive& Ar);
	
	/** Convert this unique id to a json value */
	ENGINE_API TSharedRef<FJsonValue> ToJson() const;
	/** Create a unique id from a json string */
	ENGINE_API void FromJson(const FString& InValue);

	/**
	 * For FUniqueNetIdRepl objects, we use the same hashing function as any other wrapper.
	 */
	friend inline uint32 GetTypeHash(FUniqueNetIdRepl const& Value)
	{
		if (Value.IsValid())
		{
			return GetTypeHash(*Value);
		}
		else
		{
			// If we hit this, something went wrong and we have received an unhashable wrapper.
			return INDEX_NONE;
		}
	}

protected:

	/** Helper to create an FUniqueNetId from a string and its type */
	void UniqueIdFromString(FName Type, const FString& Contents);
	/** Helper to make network serializable representation */
	void MakeReplicationData();
	void MakeReplicationDataV1();
	void MakeReplicationDataV2();
	/** Network serialized data cache */
	UPROPERTY(Transient)
	TArray<uint8> ReplicationBytes;
	
	static bool ShouldExportTextItemAsQuotedString(const FString& NetIdStr);
};

/** Specify type trait support for various low level UPROPERTY overrides */
template<>
struct TStructOpsTypeTraits<FUniqueNetIdRepl> : public TStructOpsTypeTraitsBase2<FUniqueNetIdRepl>
{
	enum 
	{
		// Can be copied via assignment operator
		WithCopy = true,
		// Requires custom serialization
		WithSerializer = true,
		// Requires custom net serialization
		WithNetSerializer = true,
		// Can share serialization state across connections
		WithNetSharedSerialization = true,
		// Requires custom Identical operator for rep notifies in PostReceivedBunch()
		WithIdenticalViaEquality = true,
		// Export contents of this struct as a string (displayall, obj dump, etc)
		WithExportTextItem = true,
		// Import string contents as a unique id
		WithImportTextItem = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** Test harness for Unique Id replication */
ENGINE_API void TestUniqueIdRepl(class UWorld* InWorld);
