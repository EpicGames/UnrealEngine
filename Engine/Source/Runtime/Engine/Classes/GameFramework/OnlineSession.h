// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Everything a local player will use to manage an online session.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Online/CoreOnline.h"
#include "OnlineSession.generated.h"

class FOnlineSessionSearchResult;

UCLASS(config=Game, MinimalAPI)
class UOnlineSession : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Register all delegates needed to manage online sessions.  */
	virtual void RegisterOnlineDelegates() {};

	/** Tear down all delegates used to manage online sessions.	 */
	virtual void ClearOnlineDelegates() {};

	/** Called to tear down any online sessions and return to main menu	 */
	ENGINE_API virtual void HandleDisconnect(UWorld *World, class UNetDriver *NetDriver);

	/** Start the online session specified */
	virtual void StartOnlineSession(FName SessionName) {};

	/** End the online session specified */
	virtual void EndOnlineSession(FName SessionName) {};

	/** Called when a user accepts an invite */
	virtual void OnSessionUserInviteAccepted(const bool bWasSuccess, const int32 ControllerId, FUniqueNetIdPtr UserId, const FOnlineSessionSearchResult & InviteResult) {};
};



