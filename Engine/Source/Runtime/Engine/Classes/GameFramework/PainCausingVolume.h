// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// PhysicsVolume:  a bounding volume which affects actor physics
// Each  AActor  is affected at any time by one PhysicsVolume
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/PhysicsVolume.h"
#include "GameFramework/DamageType.h"
#include "PainCausingVolume.generated.h"

/**
 * Volume that causes damage over time to any Actor that overlaps its collision.
 */
UCLASS(MinimalAPI)
class APainCausingVolume : public APhysicsVolume
{
	GENERATED_UCLASS_BODY()

	/** Whether volume currently causes damage. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PainCausingVolume)
	uint32 bPainCausing:1;

	/** Damage done per second to actors in this volume when bPainCausing=true */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PainCausingVolume)
	float DamagePerSec;

	/** Type of damage done */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PainCausingVolume)
	TSubclassOf<UDamageType> DamageType;

	/** If pain causing, time between damage applications. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PainCausingVolume)
	float PainInterval;

	/** if bPainCausing, cause pain when something enters the volume in addition to damage each second */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PainCausingVolume)
	uint32 bEntryPain:1;

	/** Checkpointed bPainCausing value */
	UPROPERTY()
	uint32 BACKUP_bPainCausing:1;

	/** Controller that gets credit for any damage caused by this volume */
	UPROPERTY()
	TObjectPtr<class AController> DamageInstigator;

	/** Damage overlapping actors if pain-causing. */
	ENGINE_API virtual void PainTimer();

#if WITH_EDITOR
	//Begin AVolume Interface
	ENGINE_API virtual void CheckForErrors() override;
	//End AVolume Interface
#endif

	//Begin AActor Interface
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	/* reset actor to initial state - used when restarting level without reloading. */
	ENGINE_API virtual void Reset() override;
	//End AActor Interface

	//Being PhysicsVolume Interface
	/** If bEntryPain is true, call CausePainTo() on entering actor. */
	ENGINE_API virtual void ActorEnteredVolume(class AActor* Other) override;
	//End PhysicsVolume Interface

	/** damage overlapping actors if pain causing. */
	ENGINE_API virtual void CausePainTo(class AActor* Other);

protected:

	/** Handle for efficient management of OnTimerTick timer */
	FTimerHandle TimerHandle_PainTimer;
};



