// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "UObject/CoreNet.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavAgentInterface.h"

#include "Pawn.generated.h"

struct FMovementBaseInterfaceData;
class AController;
class APhysicsVolume;
class APlayerController;
class APlayerState;
class FDebugDisplayInfo;
class UCanvas;
class UDamageType;
class UInputComponent;
class UPawnMovementComponent;
class UPawnNoiseEmitterComponent;
class UPlayer;
class UPrimitiveComponent;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogDamage, Warning, All);

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPawnRestartedSignature, APawn, ReceiveRestartedDelegate, APawn*, Pawn);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(FPawnControllerChangedSignature, APawn, ReceiveControllerChangedDelegate, APawn*, Pawn, AController*, OldController, AController*, NewController);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPawnBeginPlay, APawn*);

/** 
 * Pawn is the base class of all actors that can be possessed by players or AI.
 * They are the physical representations of players and creatures in a level.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Pawn/
 */
UCLASS(config=Game, BlueprintType, Blueprintable, hideCategories=(Navigation), meta=(ShortTooltip="A Pawn is an actor that can be 'possessed' and receive input from a controller."), MinimalAPI)
class APawn : public AActor, public INavAgentInterface
{
	GENERATED_BODY()

public:
	/** Default UObject constructor. */
	ENGINE_API APawn(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual void PreReplication( IRepChangedPropertyTracker & ChangedPropertyTracker ) override;

	/** Return our PawnMovementComponent, if we have one. By default, returns the first PawnMovementComponent found. Native classes that create their own movement component should override this method for more efficiency. */
	UFUNCTION(BlueprintCallable, meta=(Tooltip="Return our PawnMovementComponent, if we have one."), Category=Pawn)
	ENGINE_API virtual UPawnMovementComponent* GetMovementComponent() const;

	/** Return PrimitiveComponent we are based on (standing on, attached to, and moving on). */
	UE_DEPRECATED(5.8, "GetMovementBase is deprecated. Use GetMovementBaseObject or GetMovementBaseInterfaceData instead.")
	virtual UPrimitiveComponent* GetMovementBase() const { return nullptr; }

	/** Return UObject we are based on (standing on, attached to, and moving on). */
	virtual UObject* GetMovementBaseObject() const { return nullptr; }

	/** Return MovementBaseInterfaceData for the object we are based on (standing on, attached to, and moving on). */
	virtual const FMovementBaseInterfaceData* GetMovementBaseInterfaceData() const { return nullptr; }
	
	/** Return mutable MovementBaseInterfaceData for the object we are based on (standing on, attached to, and moving on). */
	virtual FMovementBaseInterfaceData* GetMovementBaseInterfaceData_Mutable(){ return nullptr; }
	
	/** If true, this Pawn's pitch will be updated to match the Controller's ControlRotation pitch, if controlled by a PlayerController. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pawn)
	uint32 bUseControllerRotationPitch:1;

	/** If true, this Pawn's yaw will be updated to match the Controller's ControlRotation yaw, if controlled by a PlayerController. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pawn)
	uint32 bUseControllerRotationYaw:1;

	/** If true, this Pawn's roll will be updated to match the Controller's ControlRotation roll, if controlled by a PlayerController. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Pawn)
	uint32 bUseControllerRotationRoll:1;

	/**
	 *	If set to false (default) given pawn instance will never affect navigation generation (but components could).
	 *	Setting it to true will result in using regular AActor's navigation relevancy 
	 *	calculation to check if this pawn instance should affect navigation generation.
	 *	@note Use SetCanAffectNavigationGeneration() to change this value at runtime.
	 *	@note Modifying this value at runtime will result in any navigation change only if runtime navigation generation is enabled.
	 *	@note Override UpdateNavigationRelevance() to propagate the flag to the desired components.
	 *	@see SetCanAffectNavigationGeneration(), UpdateNavigationRelevance()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Pawn)
	uint32 bCanAffectNavigationGeneration : 1;

private:
	/** Whether this Pawn's input handling is enabled.  Pawn must still be possessed to get input even if this is true */
	uint32 bInputEnabled:1;

	/** Used to prevent re-entry of OutsideWorldBounds event. */
	uint32 bProcessingOutsideWorldBounds : 1;

protected:
	UPROPERTY(Transient)
	uint32 bIsLocalViewTarget : 1;

public:
	/** Base eye height above collision center. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float BaseEyeHeight;

	/**
	 * Determines which PlayerController, if any, should automatically possess the pawn when the level starts or when the pawn is spawned.
	 * @see AutoPossessAI
	 */
	UPROPERTY(EditAnywhere, Category=Pawn)
	TEnumAsByte<EAutoReceiveInput::Type> AutoPossessPlayer;

	/**
	 * Determines when the Pawn creates and is possessed by an AI Controller (on level start, when spawned, etc).
	 * Only possible if AIControllerClassRef is set, and ignored if AutoPossessPlayer is enabled.
	 * @see AutoPossessPlayer
	 */
	UPROPERTY(EditAnywhere, Category=Pawn)
	EAutoPossessAI AutoPossessAI;

protected:
	/** Replicated so we can see where remote clients are looking. */
	UPROPERTY(replicated)
	uint16 RemoteViewPitch16;

public:

	/** DEPRECTED, use RemoteViewPitch16 instead. */
	UE_DEPRECATED(5.6, "This property is deprecated, use GetRemoteViewPitch()/RemoteViewPitch16 instead.")
	UPROPERTY(replicated)
	uint8 RemoteViewPitch;

	/** Default class to use when pawn is controlled by AI. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(DisplayName="AI Controller Class"), Category=Pawn)
	TSubclassOf<AController> AIControllerClass;

	/**
	 * Return our PawnNoiseEmitterComponent, if any. Default implementation returns the first PawnNoiseEmitterComponent found in the components array.
	 * If one isn't found, then it tries to find one on the Pawn's current Controller.
	 */
	ENGINE_API virtual UPawnNoiseEmitterComponent* GetPawnNoiseEmitterComponent() const;

	/**
	 * Inform AIControllers that you've made a noise they might hear (they are sent a HearNoise message if they have bHearNoises==true)
	 * The instigator of this sound is the pawn which is used to call MakeNoise.
	 *
	 * @param Loudness - is the relative loudness of this noise (range 0.0 to 1.0).  Directly affects the hearing range specified by the AI's HearingThreshold.
	 * @param NoiseLocation - Position of noise source.  If zero vector, use the actor's location.
	 * @param bUseNoiseMakerLocation - If true, use the location of the NoiseMaker rather than NoiseLocation.  If false, use NoiseLocation.
	 * @param NoiseMaker - Which actor is the source of the noise.  Not to be confused with the Noise Instigator, which is responsible for the noise (and is the pawn on which this function is called).  If not specified, the pawn instigating the noise will be used as the NoiseMaker
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=AI)
	ENGINE_API void PawnMakeNoise(float Loudness, FVector NoiseLocation, bool bUseNoiseMakerLocation = true, AActor* NoiseMaker = nullptr);
	
	/** Returns local Player Controller viewing this pawn, whether it is controlling or spectating */
	UFUNCTION(BlueprintCallable, Category = "Pawn")
	ENGINE_API APlayerController* GetLocalViewingPlayerController() const;

	// Is this pawn the ViewTarget of a local PlayerController?  Helpful for determining whether the pawn is
	// visible/critical for any VFX.  NOTE: Technically there may be some cases where locally controlled pawns return
	// false for this, such as if you are using a remote camera view of some sort.  But generally it will be true for
	// locally controlled pawns, and it will always be true for pawns that are being spectated in-game or in Replays.
	UFUNCTION(BlueprintCallable, Category = "Pawn")
	ENGINE_API bool IsLocallyViewed() const;

	ENGINE_API bool IsLocalPlayerControllerViewingAPawn() const;

	/** a delegate that broadcasts a notification whenever BeginPlay() is called */
	static ENGINE_API FOnPawnBeginPlay OnPawnBeginPlay;

private:
	/** If Pawn is possessed by a player, points to its Player State.  Needed for network play as controllers are not replicated to clients. */
	UPROPERTY(replicatedUsing=OnRep_PlayerState, BlueprintReadOnly, Category=Pawn, meta=(AllowPrivateAccess="true"))
	TObjectPtr<APlayerState> PlayerState;

public:

	/** Set the Pawn's Player State. Keeps bi-directional reference of Pawn to Player State and back in sync. */
	ENGINE_API void SetPlayerState(APlayerState* NewPlayerState);

	/** If Pawn is possessed by a player, returns its Player State.  Needed for network play as controllers are not replicated to clients. */
	APlayerState* GetPlayerState() const { return PlayerState; }

	TObjectPtr<APlayerState> GetPlayerStateObjPtr() const { return PlayerState; }

	/** Templated convenience version of GetPlayerState. */
	template<class T>
	T* GetPlayerState() const { return Cast<T>(PlayerState); }

	/** Templated convenience version of GetPlayerState which checks the type is as presumed. */
	template<class T>
	T* GetPlayerStateChecked() const { return CastChecked<T>(PlayerState); }

protected:
	/** Called on both the clientand server when ever SetPlayerState is called on this pawn. */
	virtual void OnPlayerStateChanged(APlayerState* NewPlayerState, APlayerState* OldPlayerState) { }

public:

	/** Controller of the last Actor that caused us damage. */
	UPROPERTY(BlueprintReadOnly, transient, Category="Pawn")
	TObjectPtr<AController> LastHitBy;

	/** Controller currently possessing this Actor */
	UPROPERTY(replicatedUsing=OnRep_Controller)
	TObjectPtr<AController> Controller;

	/** Previous controller that was controlling this pawn since the last controller change notification */
	UPROPERTY(transient)
	TObjectPtr<AController> PreviousController;

	/** Max difference between pawn's Rotation.Yaw and GetDesiredRotation().Yaw for pawn to be considered as having reached its desired rotation */
	float AllowedYawError;

	/** Freeze pawn - stop sounds, animations, physics, weapon firing */
	ENGINE_API virtual void TurnOff();

	/** Handle StartFire() passed from PlayerController */
	ENGINE_API virtual void PawnStartFire(uint8 FireModeNum = 0);

	/** Replicated pitch value so we can see where remote clients are looking. */
	inline uint16 GetRemoteViewPitch() const { return RemoteViewPitch16; }

	/**
	 * Set Pawn ViewPitch, so we can see where remote clients are looking.
	 * Maps 360.0 degrees into 16 bits
	 * @param	NewRemoteViewPitch	Pitch component to replicate to remote (non owned) clients.
	 */
	ENGINE_API void SetRemoteViewPitch(double NewRemoteViewPitch);

	/** Return Physics Volume for this Pawn */
	ENGINE_API virtual APhysicsVolume* GetPhysicsVolume() const override;

	/** Gets the owning actor of the Movement Base Component on which the pawn is standing. */
	UFUNCTION(BlueprintPure, Category=Pawn)
	static ENGINE_API AActor* GetMovementBaseActor(const APawn* Pawn);

	/** Gets the owning UObject of the Movement Base object on which the pawn is standing. */
	UFUNCTION(BlueprintPure, Category=Pawn)
	static ENGINE_API UObject* GetMovementBaseOwner(const APawn* Pawn);
	
	/** Return true if yaw is within AllowedYawError of desired yaw */
	ENGINE_API virtual bool ReachedDesiredRotation();

	/** Returns The half-height of the default Pawn, scaled by the component scale. By default returns the half-height of the RootComponent, regardless of whether it is registered or collidable. */
	ENGINE_API virtual float GetDefaultHalfHeight() const;

	/** See if this actor is currently being controlled */
	UE_DEPRECATED(4.24, "IsControlled is deprecated. To check if this pawn is controlled by anything, then call IsPawnControlled. To check if this pawn is controlled only by the player then call IsPlayerControlled")
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API bool IsControlled() const;

	/** Check if this actor is currently being controlled at all (the actor has a valid Controller, which will be false for remote clients) */
	UFUNCTION(BlueprintCallable, Category = Pawn)
	ENGINE_API virtual bool IsPawnControlled() const;

	/** Returns controller for this actor. */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	AController* GetController() const;
	
	/** Returns controller for this actor cast to the template type. May return NULL is the cast fails. */
	template < class T >
	T* GetController() const
	{
		return Cast<T>(GetController());
	}

	/** Sets controller for this actor */
	ENGINE_API void SetController(AController* InController);

	/** Get the rotation of the Controller, often the 'view' rotation of this Pawn. */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API FRotator GetControlRotation() const;

	/** Called when Controller is replicated */
	UFUNCTION()
	ENGINE_API virtual void OnRep_Controller();

	/** PlayerState Replication Notification Callback */
	UFUNCTION()
	ENGINE_API virtual void OnRep_PlayerState();

	//~ Begin AActor Interface.
	ENGINE_API virtual FVector GetVelocity() const override;
	ENGINE_API virtual void Reset() override;
	ENGINE_API virtual FString GetHumanReadableName() const override;
	ENGINE_API virtual bool ShouldTickIfViewportsOnly() const override;
	ENGINE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	ENGINE_API virtual void PostNetReceiveLocationAndRotation() override;
	ENGINE_API virtual void PostNetReceiveVelocity(const FVector& NewVelocity) override;
	ENGINE_API virtual void DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	ENGINE_API virtual void GetActorEyesViewPoint( FVector& Location, FRotator& Rotation ) const override;
	ENGINE_API virtual void OutsideWorldBounds() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void PreInitializeComponents() override;
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual const AActor* GetNetOwner() const override;
	ENGINE_API virtual UPlayer* GetNetOwningPlayer() override;
	ENGINE_API virtual UPlayer* GetNetOwningPlayerAnyRole() override;
	ENGINE_API virtual class UNetConnection* GetNetConnection() const override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual float TakeDamage(float Damage, struct FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser) override;
	ENGINE_API virtual void BecomeViewTarget(APlayerController* PC) override;
	ENGINE_API virtual void EndViewTarget(APlayerController* PC) override;
	ENGINE_API virtual void EnableInput(APlayerController* PlayerController) override;
	ENGINE_API virtual void DisableInput(APlayerController* PlayerController) override;
	ENGINE_API virtual void TeleportSucceeded(bool bIsATest) override;
	ENGINE_API virtual bool IsBasedOnActor(const AActor* Other) const override;

	/** Overridden to defer to the RootComponent's CanCharacterStepUpOn setting if it is explicitly Yes or No. If set to Owner, will return Super::CanBeBaseForCharacter(). */
	ENGINE_API virtual bool CanBeBaseForCharacter(APawn* APawn) const override;
	//~ End AActor Interface

	/**
	 * Use SetCanAffectNavigationGeneration to change this value at runtime.
	 * Note that calling this function at runtime will result in any navigation change only if runtime navigation generation is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation", meta=(AdvancedDisplay="bForceUpdate"))
	ENGINE_API void SetCanAffectNavigationGeneration(bool bNewValue, bool bForceUpdate = false);

	/** Update all components relevant for navigation generators to match bCanAffectNavigationGeneration flag */
	virtual void UpdateNavigationRelevance() {}

	//~ Begin INavAgentInterface Interface
	ENGINE_API virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const override;
	/** Basically retrieved pawn's location on navmesh */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	virtual FVector GetNavAgentLocation() const override { return GetActorLocation() - FVector(0.f, 0.f, BaseEyeHeight); }
	ENGINE_API virtual void GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const override;
	//~ End INavAgentInterface Interface

	/** Updates MovementComponent's parameters used by navigation system */
	ENGINE_API void UpdateNavAgent();

	/**
	 * Return true if we are in a state to take damage (checked at the start of TakeDamage.
	 * Subclasses may check this as well if they override TakeDamage and don't want to potentially trigger TakeDamage actions by checking if it returns zero in the super class.
	 */
	ENGINE_API virtual bool ShouldTakeDamage(float Damage, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser) const;

#if WITH_EDITOR
	ENGINE_API virtual void EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Returns vector direction of gravity */
	ENGINE_API virtual FVector GetGravityDirection() const;

	/** Returns a quaternion transforming from world space into gravity relative space */
	ENGINE_API virtual FQuat GetGravityTransform() const;

	/** Make sure pawn properties are back to default. */
	ENGINE_API virtual void SetPlayerDefaults();

	/** Set BaseEyeHeight based on current state. */
	ENGINE_API virtual void RecalculateBaseEyeHeight();

	/** Whether this Pawn's input handling is enabled.  Pawn must still be possessed to get input even if this is true */
	bool InputEnabled() const { return bInputEnabled; }


	/** 
	 * Called when this Pawn is possessed. Only called on the server (or in standalone).
	 * @param NewController The controller possessing this pawn
	 */
	ENGINE_API virtual void PossessedBy(AController* NewController);

	/** Event called when the Pawn is possessed by a Controller. Only called on the server (or in standalone) */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly, meta=(DisplayName= "Possessed"))
	ENGINE_API void ReceivePossessed(AController* NewController);

	/** Called when our Controller no longer possesses us. Only called on the server (or in standalone). */
	ENGINE_API virtual void UnPossessed();

	/** Event called when the Pawn is no longer possessed by a Controller. Only called on the server (or in standalone) */
	UFUNCTION(BlueprintImplementableEvent, BlueprintAuthorityOnly, meta=(DisplayName= "Unpossessed"))
	ENGINE_API void ReceiveUnpossessed(AController* OldController);


	/** Event called after a pawn's controller has changed, on the server and owning client. This will happen at the same time as the delegate on GameInstance */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void ReceiveControllerChanged(AController* OldController, AController* NewController);

	/** Event called after a pawn's controller has changed, on the server and owning client. This will happen at the same time as the delegate on GameInstance */
	UPROPERTY(BlueprintAssignable, Category = Pawn)
	FPawnControllerChangedSignature ReceiveControllerChangedDelegate;

	/** Call to notify about a change in controller, on both the server and owning client. This calls the above event and delegate */
	ENGINE_API virtual void NotifyControllerChanged();


	/** Event called after a pawn has been restarted, usually by a possession change. This is called on the server for all pawns and the owning client for player pawns */
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void ReceiveRestarted();

	/** Event called after a pawn has been restarted, usually by a possession change. This is called on the server for all pawns and the owning client for player pawns */
	UPROPERTY(BlueprintAssignable, Category = Pawn)
	FPawnRestartedSignature ReceiveRestartedDelegate;

	/**
	 * Notifies other systems that a pawn has been restarted. By default this is called on the server for all pawns and the owning client for player pawns.
	 * This can be overridden by subclasses to delay the notification of restart until data has loaded/replicated
	 */
	ENGINE_API virtual void NotifyRestarted();

	/** Called when the Pawn is being restarted (usually by being possessed by a Controller). This is called on the server for all pawns and the owning client for player pawns  */
	ENGINE_API virtual void Restart();

	/** Called on the owning client of a player-controlled Pawn when it is restarted, this calls Restart() */
	ENGINE_API virtual void PawnClientRestart();

	/** Wrapper function to call correct restart functions, enable bCallClientRestart if this is a locally owned player pawn or equivalent */
	ENGINE_API void DispatchRestart(bool bCallClientRestart);


	/** Returns true if controlled by a local (not network) Controller.	 */
	UFUNCTION(BlueprintPure, Category=Pawn)
	ENGINE_API virtual bool IsLocallyControlled() const;

	/**
	 * Returns the Platform User ID of the PlayerController that is controlling this character.
	 *
	 * Returns an invalid Platform User ID if this character is not controlled by a local player.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Pawn)
	ENGINE_API FPlatformUserId GetPlatformUserId() const;
  
	/** Returns true if controlled by a human player (possessed by a PlayerController).	This returns true for players controlled by remote clients */
	UFUNCTION(BlueprintPure, Category=Pawn)
	ENGINE_API virtual bool IsPlayerControlled() const;

	/** Returns true if controlled by a bot.	 */
	UFUNCTION(BlueprintPure, Category = Pawn)
	ENGINE_API virtual bool IsBotControlled() const;

	/**
	 * Get the view rotation of the Pawn (direction they are looking, normally Controller->ControlRotation).
	 * @return The view rotation of the Pawn.
	 */
	ENGINE_API virtual FRotator GetViewRotation() const;

	/** Returns	Pawn's eye location */
	ENGINE_API virtual FVector GetPawnViewLocation() const;

	/**
	 * Return the aim rotation for the Pawn.
	 * If we have a controller, by default we aim at the player's 'eyes' direction
	 * that is by default the Pawn rotation for AI, and camera (crosshair) rotation for human players.
	 */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual FRotator GetBaseAimRotation() const;

	/** Returns true if player is viewing this Pawn in FreeCam */
	ENGINE_API virtual bool InFreeCam() const;

	/** Updates Pawn's rotation to the given rotation, assumed to be the Controller's ControlRotation. Respects the bUseControllerRotation* settings. */
	ENGINE_API virtual void FaceRotation(FRotator NewControlRotation, float DeltaTime = 0.f);

	/** Call this function to detach safely pawn from its controller, knowing that we will be destroyed soon.	 */
	UFUNCTION(BlueprintCallable, Category=Pawn, meta=(Keywords = "Delete"))
	ENGINE_API virtual void DetachFromControllerPendingDestroy();

	/** Spawn default controller for this Pawn, and get possessed by it. */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual void SpawnDefaultController();

protected:
	/** Get the controller instigating the damage. If the damage is caused by the world and the supplied controller is nullptr or is this pawn's controller, uses LastHitBy as the instigator. */
	ENGINE_API virtual AController* GetDamageInstigator(AController* InstigatedBy, const UDamageType& DamageType) const;

	/** Creates an InputComponent that can be used for custom input bindings. Called upon possession by a PlayerController. Return null if you don't want one. */
	ENGINE_API virtual UInputComponent* CreatePlayerInputComponent();

	/** Destroys the player input component and removes any references to it. */
	ENGINE_API virtual void DestroyPlayerInputComponent();

	/** Allows a Pawn to set up custom input bindings. Called upon possession by a PlayerController, using the InputComponent created by CreatePlayerInputComponent(). */
	virtual void SetupPlayerInputComponent(UInputComponent* PlayerInputComponent) { /* No bindings by default.*/ }

public:
	/**
	 * Add movement input along the given world direction vector (usually normalized) scaled by 'ScaleValue'. If ScaleValue < 0, movement will be in the opposite direction.
	 * Base Pawn classes won't automatically apply movement, it's up to the user to do so in a Tick event. Subclasses such as Character and DefaultPawn automatically handle this input and move.
	 *
	 * @param WorldDirection	Direction in world space to apply input
	 * @param ScaleValue		Scale to apply to input. This can be used for analog input, ie a value of 0.5 applies half the normal value, while -1.0 would reverse the direction.
	 * @param bForce			If true always add the input, ignoring the result of IsMoveInputIgnored().
	 * @see GetPendingMovementInputVector(), GetLastMovementInputVector(), ConsumeMovementInputVector()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="AddInput"))
	ENGINE_API virtual void AddMovementInput(FVector WorldDirection, float ScaleValue = 1.0f, bool bForce = false);

	/**
	 * Return the pending input vector in world space. This is the most up-to-date value of the input vector, pending ConsumeMovementInputVector() which clears it,
	 * Usually only a PawnMovementComponent will want to read this value, or the Pawn itself if it is responsible for movement.
	 *
	 * @return The pending input vector in world space.
	 * @see AddMovementInput(), GetLastMovementInputVector(), ConsumeMovementInputVector()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="GetMovementInput GetInput"))
	ENGINE_API FVector GetPendingMovementInputVector() const;

	/**
	 * Return the last input vector in world space that was processed by ConsumeMovementInputVector(), which is usually done by the Pawn or PawnMovementComponent.
	 * Any user that needs to know about the input that last affected movement should use this function.
	 * For example an animation update would want to use this, since by default the order of updates in a frame is:
	 * PlayerController (device input) -> MovementComponent -> Pawn -> Mesh (animations)
	 *
	 * @return The last input vector in world space that was processed by ConsumeMovementInputVector().
	 * @see AddMovementInput(), GetPendingMovementInputVector(), ConsumeMovementInputVector()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="GetMovementInput GetInput"))
	ENGINE_API FVector GetLastMovementInputVector() const;

	/**
	 * Returns the pending input vector and resets it to zero.
	 * This should be used during a movement update (by the Pawn or PawnMovementComponent) to prevent accumulation of control input between frames.
	 * Copies the pending input vector to the saved input vector (GetLastMovementInputVector()).
	 * @return The pending input vector.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="ConsumeInput"))
	ENGINE_API virtual FVector ConsumeMovementInputVector();

	/**
	 * Add input (affecting Pitch) to the Controller's ControlRotation, if it is a local PlayerController.
	 * This value is multiplied by the PlayerController's InputPitchScale value.
	 * @param Val Amount to add to Pitch. This value is multiplied by the PlayerController's InputPitchScale value.
	 * @see PlayerController::InputPitchScale
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="up down addpitch"))
	ENGINE_API virtual void AddControllerPitchInput(float Val);

	/**
	 * Add input (affecting Yaw) to the Controller's ControlRotation, if it is a local PlayerController.
	 * This value is multiplied by the PlayerController's InputYawScale value.
	 * @param Val Amount to add to Yaw. This value is multiplied by the PlayerController's InputYawScale value.
	 * @see PlayerController::InputYawScale
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="left right turn addyaw"))
	ENGINE_API virtual void AddControllerYawInput(float Val);

	/**
	 * Add input (affecting Roll) to the Controller's ControlRotation, if it is a local PlayerController.
	 * This value is multiplied by the PlayerController's InputRollScale value.
	 * @param Val Amount to add to Roll. This value is multiplied by the PlayerController's InputRollScale value.
	 * @see PlayerController::InputRollScale
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input", meta=(Keywords="addroll"))
	ENGINE_API virtual void AddControllerRollInput(float Val);

	/** Helper to see if move input is ignored. If our controller is a PlayerController, checks Controller->IsMoveInputIgnored(). */
	UFUNCTION(BlueprintCallable, Category="Pawn|Input")
	ENGINE_API virtual bool IsMoveInputIgnored() const;

	UFUNCTION(BlueprintCallable, Category = "Pawn|Input")
	ENGINE_API TSubclassOf<UInputComponent> GetOverrideInputComponentClass() const;

protected:
	/**
	 * Accumulated control input vector, stored in world space. This is the pending input, which is cleared (zeroed) once consumed.
	 * @see GetPendingMovementInputVector(), AddMovementInput()
	 */
	UPROPERTY(Transient)
	FVector ControlInputVector;

	/**
	 * The last control input vector that was processed by ConsumeMovementInputVector().
	 * @see GetLastMovementInputVector()
	 */
	UPROPERTY(Transient)
	FVector LastControlInputVector;

	/** If set, then this InputComponent class will be used instead of the Input Settings' DefaultInputComponentClass */
	UPROPERTY(EditDefaultsOnly, Category = "Pawn|Input")
	TSubclassOf<UInputComponent> OverrideInputComponentClass;

public:
	/** Internal function meant for use only within Pawn or by a PawnMovementComponent. Adds movement input if not ignored, or if forced. */
	ENGINE_API void Internal_AddMovementInput(FVector WorldAccel, bool bForce = false);

	/** Internal function meant for use only within Pawn or by a PawnMovementComponent. Returns the value of ControlInputVector. */
	inline FVector Internal_GetPendingMovementInputVector() const { return ControlInputVector; }

	/** Internal function meant for use only within Pawn or by a PawnMovementComponent. Returns the value of LastControlInputVector. */
	inline FVector Internal_GetLastMovementInputVector() const { return LastControlInputVector; }

	/** Internal function meant for use only within Pawn or by a PawnMovementComponent. LastControlInputVector is updated with initial value of ControlInputVector. Returns ControlInputVector and resets it to zero. */
	ENGINE_API FVector Internal_ConsumeMovementInputVector();

	/** Add an Actor to ignore by Pawn's movement collision */
	ENGINE_API void MoveIgnoreActorAdd(AActor* ActorToIgnore);

	/** Remove an Actor to ignore by Pawn's movement collision */
	ENGINE_API void MoveIgnoreActorRemove(AActor* ActorToIgnore);

};


//////////////////////////////////////////////////////////////////////////
// Pawn inlines

inline AController* APawn::GetController() const
{
	return Controller;
}
