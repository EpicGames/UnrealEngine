// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Movement component meant for use with Pawns.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/NavMovementComponent.h"
#include "PawnMovementComponent.generated.h"

class APawn;
class UPrimitiveComponent;

/**
 * Enumerates all the action types that could be applied to the physics state
 */
UENUM()
enum class EPhysicsStateAction : uint8
{
	AddForce,
	AddTorque,
	AddForceAtPosition,
	AddLinearVelocity,
	AddAngularVelocity,
	AddVelocityAtPosition,
	AddLinearImpulse,
	AddAngularImpulse,
	AddImpulseAtPosition,
	AddAcceleration,
	NumActions
};
  
/** 
 * PawnMovementComponent can be used to update movement for an associated Pawn.
 * It also provides ways to accumulate and read directional input in a generic way (with AddInputVector(), ConsumeInputVector(), etc).
 * @see APawn
 */
UCLASS(abstract, MinimalAPI)
class UPawnMovementComponent : public UNavMovementComponent
{
	GENERATED_BODY()

public:

	/** Overridden to only allow registration with components owned by a Pawn. */
	ENGINE_API virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent) override;

	/**
	 * Adds the given vector to the accumulated input in world space. Input vectors are usually between 0 and 1 in magnitude. 
	 * They are accumulated during a frame then applied as acceleration during the movement update.
	 *
	 * @param WorldVector		Direction in world space to apply input
	 * @param bForce			If true always add the input, ignoring the result of IsMoveInputIgnored().
	 * @see APawn::AddMovementInput()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement")
	ENGINE_API virtual void AddInputVector(FVector WorldVector, bool bForce = false);

	/**
	 * Return the pending input vector in world space. This is the most up-to-date value of the input vector, pending ConsumeMovementInputVector() which clears it.
	 * PawnMovementComponents implementing movement usually want to use either this or ConsumeInputVector() as these functions represent the most recent state of input.
	 * @return The pending input vector in world space.
	 * @see AddInputVector(), ConsumeInputVector(), GetLastInputVector()
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement", meta=(Keywords="GetInput"))
	ENGINE_API FVector GetPendingInputVector() const;

	/**
	* Return the last input vector in world space that was processed by ConsumeInputVector(), which is usually done by the Pawn or PawnMovementComponent.
	* Any user that needs to know about the input that last affected movement should use this function.
	* @return The last input vector in world space that was processed by ConsumeInputVector().
	* @see AddInputVector(), ConsumeInputVector(), GetPendingInputVector()
	*/
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement", meta=(Keywords="GetInput"))
	ENGINE_API FVector GetLastInputVector() const;

	/* Returns the pending input vector and resets it to zero.
	 * This should be used during a movement update (by the Pawn or PawnMovementComponent) to prevent accumulation of control input between frames.
	 * Copies the pending input vector to the saved input vector (GetLastMovementInputVector()).
	 * @return The pending input vector.
	 */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement")
	ENGINE_API virtual FVector ConsumeInputVector();

	/** Helper to see if move input is ignored. If there is no Pawn or UpdatedComponent, returns true, otherwise defers to the Pawn's implementation of IsMoveInputIgnored(). */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement")
	ENGINE_API virtual bool IsMoveInputIgnored() const;

	/** Return the Pawn that owns UpdatedComponent. */
	UFUNCTION(BlueprintCallable, Category="Pawn|Components|PawnMovement")
	ENGINE_API class APawn* GetPawnOwner() const;

	/** Notify of collision in case we want to react, such as waking up avoidance or pathing code. */
	virtual void NotifyBumpedPawn(APawn* BumpedPawn) {}

	// UNavMovementComponent override for input operations
	ENGINE_API virtual void RequestPathMove(const FVector& MoveInput) override;

	ENGINE_API virtual void OnTeleported() override;

	/** Apply the async physics state action onto the solver */
	ENGINE_API void ApplyAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName& BoneName, 
		const EPhysicsStateAction ActionType, const FVector& ActionDatas, const FVector& ActionPosition = FVector::Zero());

protected:

	/** Pawn that owns this component. */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<class APawn> PawnOwner;

	/** Returns this component's associated controller, typically from the owning Pawn. May be null. May be overridden for special handling when the controller isn't paired with the Pawn that owns this component. */
	ENGINE_API virtual AController* GetController() const;

	/**
	 * Attempts to mark the PlayerCameraManager as dirty, if the controller has one.
	 * This will have no effect if called from the server.
	 */
	ENGINE_API void MarkForClientCameraUpdate();

public:

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

private:

	/** Send the physics command to execute to the server */
	UFUNCTION(Server, Reliable)
	ENGINE_API void ServerAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName BoneName, const FAsyncPhysicsTimestamp Timestamp,
			const EPhysicsStateAction ActionType, const FVector ActionDatas, const FVector ActionPosition = FVector::Zero());

	/** Dispatch back the physics command onto all the clients */
	UFUNCTION(NetMulticast, Reliable)
	ENGINE_API void MulticastAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName BoneName, const FAsyncPhysicsTimestamp Timestamp,
			const EPhysicsStateAction ActionType, const FVector ActionDatas, const FVector ActionPosition = FVector::Zero());

	/** Execute the async physics state action at a given timestamp */
	ENGINE_API void ExecuteAsyncPhysicsStateAction(const UPrimitiveComponent* ActionComponent, const FName& BoneName, const FAsyncPhysicsTimestamp& Timestamp,
		const EPhysicsStateAction ActionType, const FVector& ActionDatas, const FVector& ActionPosition = FVector::Zero());
};
