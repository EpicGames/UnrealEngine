// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "PhysicsVolume.generated.h"

/**
 * PhysicsVolume: A bounding volume which affects actor physics.
 * Each AActor is affected at any time by one PhysicsVolume.
 */

UCLASS(MinimalAPI)
class APhysicsVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
#endif // WITH_EDITOR	
	//~ End UObject Interface.

	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	//~======================================================================================
	// Character Movement related properties

	/** Terminal velocity of pawns using CharacterMovement when falling. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CharacterMovement)
	float TerminalVelocity;

	/** Determines which PhysicsVolume takes precedence if they overlap (higher number = higher priority). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CharacterMovement)
	int32 Priority;

	/** This property controls the amount of friction applied by the volume as pawns using CharacterMovement move through it. The higher this value, the harder it will feel to move through */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CharacterMovement)
	float FluidFriction;

	/** True if this volume contains a fluid like water */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CharacterMovement)
	uint32 bWaterVolume:1;

	//~======================================================================================
	// Physics related properties
	
	/**	By default, the origin of an AActor must be inside a PhysicsVolume for it to affect the actor. However if this flag is true, the other actor only has to touch the volume to be affected by it. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Volume)
	uint32 bPhysicsOnContact:1;

public:
	/** @Returns the Z component of the current world gravity. */
	ENGINE_API virtual float GetGravityZ() const;
	
	// Called when actor enters a volume
	ENGINE_API virtual void ActorEnteredVolume(class AActor* Other);
	
	// Called when actor leaves a volume, Other can be NULL
	ENGINE_API virtual void ActorLeavingVolume(class AActor* Other);

	/** Given a known overlap with the given component, validate that it meets the rules imposed by bPhysicsOnContact. */
	ENGINE_API virtual bool IsOverlapInVolume(const class USceneComponent& TestComponent) const;
};
