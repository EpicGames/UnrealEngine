// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "Containers/SortedMap.h"
#include "Containers/StaticArray.h"
#include "Misc/Guid.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Engine/NetSerialization.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/LatentActionManager.h"
#include "SceneTypes.h"
#include "GameFramework/Controller.h"
#include "UObject/TextProperty.h"
#include "GameFramework/PlayerMuteList.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Camera/PlayerCameraManager.h"
#include "GameFramework/ForceFeedbackParameters.h"
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"
#include "GenericPlatform/ICursor.h"
#include "GenericPlatform/IInputInterface.h"
#include "Physics/AsyncPhysicsData.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "EngineDefines.h"		// For UE_ENABLE_DEBUG_DRAWING
#include "InputKeyEventArgs.h"

#include "PlayerController.generated.h"

class ACameraActor;
class AHUD;
class APawn;
class ASpectatorPawn;
class FDebugDisplayInfo;
class SWidget;
class UActorChannel;
class UCheatManager;
class UGameViewportClient;
class UInputComponent;
class ULocalMessage;
class UNetConnection;
class UPlayer;
class UPlayerInput;
class UPrimitiveComponent;
namespace EControllerAnalogStick { enum Type : int; }
struct FActiveForceFeedbackEffect;
struct FActiveHapticFeedbackEffect;
struct FCollisionQueryParams;
struct FForceFeedbackEffectHistoryEntry;
struct FInputKeyParams;
struct FPlatformUserId;
class UAsyncPhysicsInputComponent;
enum class EGestureEvent : uint8;
enum class EGesturePhase : uint8;

/** Default delegate that provides an implementation for those that don't have special needs other than a toggle */
DECLARE_DELEGATE_RetVal(bool, FCanUnpause);

/** delegate used to override default viewport audio listener position calculated from camera */
DECLARE_DELEGATE_ThreeParams(FGetAudioListenerPos, FVector& /*Location*/, FVector& /*ProjFront*/, FVector& /*ProjRight*/);

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogPlayerController, Log, All);
DECLARE_STATS_GROUP(TEXT("PlayerController"), STATGROUP_PlayerController, STATCAT_Advanced);

UENUM()
namespace EDynamicForceFeedbackAction
{
	enum Type : int
	{
		Start,
		Update,
		Stop,
	};
}

typedef uint64 FDynamicForceFeedbackHandle;

struct FDynamicForceFeedbackDetails
{
	uint32 bAffectsLeftLarge:1;
	uint32 bAffectsLeftSmall:1;
	uint32 bAffectsRightLarge:1;
	uint32 bAffectsRightSmall:1;

	float Intensity;

	FDynamicForceFeedbackDetails()
		: bAffectsLeftLarge(true)
		, bAffectsLeftSmall(true)
		, bAffectsRightLarge(true)
		, bAffectsRightSmall(true)
		, Intensity(0.f)
	{}

	void Update(FForceFeedbackValues& Values) const;
};

/** Abstract base class for Input Mode structures */
struct FInputModeDataBase
{
protected:
	virtual ~FInputModeDataBase() = default;

	/** Derived classes override this function to apply the necessary settings for the desired input mode */
	virtual void ApplyInputMode(class FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const = 0;

	virtual bool ShouldFlushInputOnViewportFocus() const { return true; };

	/** Utility functions for derived classes. */
	ENGINE_API void SetFocusAndLocking(FReply& SlateOperations, TSharedPtr<class SWidget> InWidgetToFocus, bool bLockMouseToViewport, TSharedRef<class SViewport> InViewportWidget) const;

	friend class APlayerController;

public:

#if UE_ENABLE_DEBUG_DRAWING
	/** Returns the name of this input mode for debug display when you call the "showdebug input" command. */
	ENGINE_API virtual const FString& GetDebugDisplayName() const;
#endif	// UE_ENABLE_DEBUG_DRAWING
};

/** Data structure used to setup an input mode that allows only the UI to respond to user input. */
struct FInputModeUIOnly : public FInputModeDataBase
{
	ENGINE_API FInputModeUIOnly();

	ENGINE_API virtual ~FInputModeUIOnly();

	/** Widget to focus */
	ENGINE_API FInputModeUIOnly& SetWidgetToFocus(TSharedPtr<SWidget> InWidgetToFocus);

	/** Sets the mouse locking behavior of the viewport */
	ENGINE_API FInputModeUIOnly& SetLockMouseToViewportBehavior(EMouseLockMode InMouseLockMode);

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API virtual const FString& GetDebugDisplayName() const override;
#endif	// UE_ENABLE_DEBUG_DRAWING

protected:
	TSharedPtr<SWidget> WidgetToFocus;
	EMouseLockMode MouseLockMode;

	ENGINE_API virtual void ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const override;
};

/** This structure is used to pass arguments to ClientUpdateMultipleLevelsStreamingStatus() client RPC function */
USTRUCT()
struct FUpdateLevelStreamingLevelStatus
{
	GENERATED_BODY();

	/** Name of the level package name used for loading. */
	UPROPERTY()
	FName PackageName;

	/** Current LOD index for a streaming level */
	UPROPERTY()
	int32 LODIndex = 0;

	/** Whether the level should be loaded */
	UPROPERTY()
	bool bNewShouldBeLoaded = false;
	
	/** Whether the level should be visible if it is loaded */
	UPROPERTY()
	bool bNewShouldBeVisible = false;
	
	/** Whether we want to force a blocking load */
	UPROPERTY()
	bool bNewShouldBlockOnLoad = false;

	/** Whether we want to force a blocking unload */
	UPROPERTY()
	bool bNewShouldBlockOnUnload = false;
};

/** Data structure used to setup an input mode that allows the UI to respond to user input, and if the UI doesn't handle it player input / player controller gets a chance. */
struct FInputModeGameAndUI : public FInputModeDataBase
{
	ENGINE_API FInputModeGameAndUI();

	ENGINE_API virtual ~FInputModeGameAndUI();

	/** Widget to focus */
	FInputModeGameAndUI& SetWidgetToFocus(TSharedPtr<SWidget> InWidgetToFocus) { WidgetToFocus = InWidgetToFocus; return *this; }

	/** Sets the mouse locking behavior of the viewport */
	FInputModeGameAndUI& SetLockMouseToViewportBehavior(EMouseLockMode InMouseLockMode) { MouseLockMode = InMouseLockMode; return *this; }

	/** Whether to hide the cursor during temporary mouse capture caused by a mouse down */
	FInputModeGameAndUI& SetHideCursorDuringCapture(bool InHideCursorDuringCapture) { bHideCursorDuringCapture = InHideCursorDuringCapture; return *this; }

	virtual bool ShouldFlushInputOnViewportFocus() const override { return false; }

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API virtual const FString& GetDebugDisplayName() const override;
#endif	// UE_ENABLE_DEBUG_DRAWING

protected:

	TSharedPtr<SWidget> WidgetToFocus;
	EMouseLockMode MouseLockMode;
	bool bHideCursorDuringCapture;

	ENGINE_API virtual void ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const override;
};

/** Data structure used to setup an input mode that allows only the player input / player controller to respond to user input. */
struct FInputModeGameOnly : public FInputModeDataBase
{
	ENGINE_API FInputModeGameOnly();

	ENGINE_API virtual ~FInputModeGameOnly();

	/** Whether the mouse down that causes capture should be consumed, and not passed to player input processing */
	FInputModeGameOnly& SetConsumeCaptureMouseDown(bool InConsumeCaptureMouseDown) { bConsumeCaptureMouseDown = InConsumeCaptureMouseDown; return *this; }

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API virtual const FString& GetDebugDisplayName() const override;
#endif	// UE_ENABLE_DEBUG_DRAWING
	
protected:
	bool bConsumeCaptureMouseDown;

	ENGINE_API virtual void ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const override;
};

USTRUCT()
struct FAsyncPhysicsTimestamp
{
	GENERATED_BODY()

	/** Check if the server and local frames are valid */
	bool IsValid() const
	{
		return ServerFrame != INDEX_NONE && LocalFrame != INDEX_NONE;
	}
	UPROPERTY()
	int32 ServerFrame = INDEX_NONE;

	UPROPERTY()
	int32 LocalFrame = INDEX_NONE;

};

/**
 * PlayerControllers are used by human players to control Pawns.
 *
 * ControlRotation (accessed via GetControlRotation()), determines the aiming
 * orientation of the controlled Pawn.
 *
 * In networked games, PlayerControllers exist on the server for every player-controlled pawn,
 * and also on the controlling client's machine. They do NOT exist on a client's
 * machine for pawns controlled by remote players elsewhere on the network.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Controller/PlayerController/
 */
UCLASS(config=Game, BlueprintType, Blueprintable, meta=(ShortTooltip="A Player Controller is an actor responsible for controlling a Pawn used by the player."), MinimalAPI)
class APlayerController : public AController, public IWorldPartitionStreamingSourceProvider
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	ENGINE_API APlayerController(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** UPlayer associated with this PlayerController.  Could be a local player or a net connection. */
	UPROPERTY()
	TObjectPtr<UPlayer> Player;

	/** Used in net games so client can acknowledge it possessed a specific pawn. */
	UPROPERTY()
	TObjectPtr<APawn> AcknowledgedPawn;

	/** Heads up display associated with this PlayerController. */
	UPROPERTY()
	TObjectPtr<AHUD> MyHUD;

	// ******************************************************************************
	// Camera/view related variables

	/** Camera manager associated with this Player Controller. */
	UPROPERTY(BlueprintReadOnly, Category=PlayerController)
	TObjectPtr<APlayerCameraManager> PlayerCameraManager;

	/** PlayerCamera class should be set for each game, otherwise Engine.PlayerCameraManager is used */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PlayerController)
	TSubclassOf<APlayerCameraManager> PlayerCameraManagerClass;

	/** 
	 * True to allow this player controller to manage the camera target for you,
	 * typically by using the possessed pawn as the camera target. Set to false
	 * if you want to manually control the camera target.
	 */
	UPROPERTY(EditAnywhere, Category=PlayerController)
	bool bAutoManageActiveCameraTarget;

	/** Used to replicate the view rotation of targets not owned/possessed by this PlayerController. */ 
	UPROPERTY(replicated)
	FRotator TargetViewRotation; 

	/** Setter for TargetViewRotation. Marks it dirty if APlayerController is configured to use push-based dirtiness. */
	ENGINE_API void SetTargetViewRotation(const FRotator& InRotation);

	/** Smoothed version of TargetViewRotation to remove jerkiness from intermittent replication updates. */
	FRotator BlendedTargetViewRotation;

	/** Interp speed for blending remote view rotation for smoother client updates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerController)
	float SmoothTargetViewRotationSpeed;

	/**
	 * Last used FOV based multiplier to distance to an object when determining if it exceeds the object's cull distance
	 * @note: only valid for local player
	 */
	float LocalPlayerCachedLODDistanceFactor;

	/** The actors which the camera shouldn't see - e.g. used to hide actors which the camera penetrates */
	UPROPERTY()
	TArray<TObjectPtr<class AActor>> HiddenActors;

	/** Explicit components the camera shouldn't see (helpful for external systems to hide a component from a single player) */
	UPROPERTY()
	TArray< TWeakObjectPtr<UPrimitiveComponent> > HiddenPrimitiveComponents;

	/** Whether to render primitives component. */
	bool bRenderPrimitiveComponents;

	/** Used to make sure the client is kept synchronized when in a spectator state */
	UPROPERTY()
	float LastSpectatorStateSynchTime;

	/** Last location synced on the server for a spectator. */
	UPROPERTY(Transient)
	FVector LastSpectatorSyncLocation;

	/** Last rotation synced on the server for a spectator. */
	UPROPERTY(Transient)
	FRotator LastSpectatorSyncRotation;

	/** Cap set by server on bandwidth from client to server in bytes/sec (only has impact if >=2600) */
	UPROPERTY()
	int32 ClientCap;
	
	/**
	 * Object that manages "cheat" commands.
	 *
	 * By default:
	 *	 - In Shipping configurations, the manager is always disabled because UE_WITH_CHEAT_MANAGER is 0
	 *   - When playing in the editor, cheats are always enabled
	 *   - In other cases, cheats are enabled by default in single player games but can be forced on with the EnableCheats console command
	 * 
	 * This behavior can be changed either by overriding APlayerController::EnableCheats or AGameModeBase::AllowCheats.
	 */
	UPROPERTY(Transient, BlueprintReadOnly, Category="Cheat Manager")
	TObjectPtr<UCheatManager> CheatManager;
	
	/**
	 * Class of my CheatManager.
	 * @see CheatManager for more information about when it will be instantiated.
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Cheat Manager")
	TSubclassOf<UCheatManager> CheatClass;

	/** Object that manages player input. */
	UPROPERTY(Transient)
	TObjectPtr<UPlayerInput> PlayerInput;    
	
	UPROPERTY(Transient)
	TArray<FActiveForceFeedbackEffect> ActiveForceFeedbackEffects;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** For debugging, shows the last force feeback effects that played */
	TArray<FForceFeedbackEffectHistoryEntry> ForceFeedbackEffectHistoryEntries;
#endif

protected:
	ENGINE_API virtual void OnReplicationStartedForIris(const FOnReplicationStartedParams&) override;

private:

	struct FDynamicForceFeedbackAction
	{
		/** Time over which interpolation should happen */
		float TotalTime;
		/** Time so far elapsed for the interpolation */
		float TimeElapsed;
		/** Current feedback intensity values */
		FDynamicForceFeedbackDetails ForceFeedbackDetails;
		/** Unique ID for the action */
		FDynamicForceFeedbackHandle Handle;
		/** Unique ID generation static */
		static FDynamicForceFeedbackHandle HandleAllocator;

		/** 
		 * Updates Values with this action's details.
		 * @param DeltaTime	Time since last update
		 * @param Values	Values structure to update
		 * @return Returns false if the elapsed time has exceeded the total time.
		 */
		bool Update(const float DeltaTime, FForceFeedbackValues& Values);
	};

	/** Map of dynamic force feedback effects invoked from native */
	TSortedMap<FDynamicForceFeedbackHandle, FDynamicForceFeedbackAction> DynamicForceFeedbacks;

	/** Map of dynamic force feedback effects invoked from blueprints */
	TSortedMap<int32, FDynamicForceFeedbackDetails*> LatentDynamicForceFeedbacks;

	friend class FLatentDynamicForceFeedbackAction;

public:

	/** Currently playing haptic effects for both the left and right hand */
	TSharedPtr<struct FActiveHapticFeedbackEffect> ActiveHapticEffect_Left;
	TSharedPtr<struct FActiveHapticFeedbackEffect> ActiveHapticEffect_Right;
	TSharedPtr<struct FActiveHapticFeedbackEffect> ActiveHapticEffect_Gun;
	TSharedPtr<struct FActiveHapticFeedbackEffect> ActiveHapticEffect_HMD;

	/** Currently active force feedback weights */
	FForceFeedbackValues ForceFeedbackValues;

	/** List of names of levels the server is in the middle of sending us for a PrepareMapChange() call */
	TArray<FName> PendingMapChangeLevelNames;

	/**
	 * When true, reduces connect timeout from InitialConnectionTimeOut to ConnectionTimeout.  
	 * Set once initial level load is complete (client may be unresponsive during level loading).
	 */
	uint32 bShortConnectTimeOut:1;

	/** Is this player currently in cinematic mode?  Prevents rotation/movement/firing/etc */
	uint32 bCinematicMode:1;
	
	/** When cinematic mode is true, signifies that this controller's pawn should be hidden */
	uint32 bHidePawnInCinematicMode:1;

	/** Whether this controller is using streaming volumes.  **/
	uint32 bIsUsingStreamingVolumes:1;

	/** True if PlayerController is currently waiting for the match to start or to respawn. Only valid in Spectating state. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category=PlayerController)
	uint32 bPlayerIsWaiting:1;

	/** Indicate that the Spectator is waiting to join/respawn. */
	UFUNCTION(server, reliable, WithValidation, Category=PlayerController)
	ENGINE_API void ServerSetSpectatorWaiting(bool bWaiting);

	/** Indicate that the Spectator is waiting to join/respawn. */
	UFUNCTION(client, reliable, Category=PlayerController)
	ENGINE_API void ClientSetSpectatorWaiting(bool bWaiting);

	/**
	 * Index identifying players using the same base connection (splitscreen clients)
	 * Used by netcode to match replicated PlayerControllers to the correct splitscreen viewport and child connection
	 * replicated via special internal code, not through normal variable replication
	 */
	UPROPERTY(DuplicateTransient)
	uint8 NetPlayerIndex;

	/** List of muted players in various categories */
	FPlayerMuteList MuteList;

	/**
	 * This is set on the OLD PlayerController when performing a swap over a network connection
	 * so we know what connection we're waiting on acknowledgment from to finish destroying this PC
	 * (or when the connection is closed)
	 * @see GameModeBase::SwapPlayerControllers()
	 */
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UNetConnection> PendingSwapConnection;

	/** The net connection this controller is communicating on, nullptr for local players on server */
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UNetConnection> NetConnection;

	/** Input axes values, accumulated each tick. */
	FRotator RotationInput;

	/** Yaw input speed scaling */
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use the Enhanced Input plugin Scalar Modifier instead. See UInputSettings::bEnableLegacyInputScales to enable legacy behavior"))
	float InputYawScale_DEPRECATED = 1.0f;

	/** Pitch input speed scaling */
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use the Enhanced Input plugin Scalar Modifier instead. See UInputSettings::bEnableLegacyInputScales to enable legacy behavior"))
	float InputPitchScale_DEPRECATED = 1.0f;

	/** Roll input speed scaling */
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use the Enhanced Input plugin Scalar Modifier instead. See UInputSettings::bEnableLegacyInputScales to enable legacy behavior"))
	float InputRollScale_DEPRECATED = 1.0f;

	/** A getter for the deprecated InputYawScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "GetDeprecatedInputYawScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API float GetDeprecatedInputYawScale() const;
	
	/** A getter for the deprecated InputPitchScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "GetDeprecatedInputPitchScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API float GetDeprecatedInputPitchScale() const;
	
	/** A getter for the deprecated InputRollScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "GetDeprecatedInputRollScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.)")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API float GetDeprecatedInputRollScale() const;

	/** A getter for the deprecated InputYawScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "SetDeprecatedInputYawScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API void SetDeprecatedInputYawScale(float NewValue);

	/** A getter for the deprecated InputPitchScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "SetDeprecatedInputPitchScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API void SetDeprecatedInputPitchScale(float NewValue);

	/** A getter for the deprecated InputRollScale property. This should only be used if UInputSettings::bEnableLegacyInputScales is turned on. */
	UE_DEPRECATED(5.0, "SetDeprecatedInputRollScale is deprecated, please use the Enhanced Input plugin Scalar Modifier instead.)")
	UFUNCTION(BlueprintCallable, Category = PlayerController)
	ENGINE_API void SetDeprecatedInputRollScale(float NewValue);

	/** Whether the mouse cursor should be displayed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter=SetShowMouseCursor, Category=MouseInterface)
	uint32 bShowMouseCursor:1;

	/** Whether actor/component click events should be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface)
	uint32 bEnableClickEvents:1;

	/** Whether actor/component touch events should be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface)
	uint32 bEnableTouchEvents:1;

	/** Whether actor/component mouse over events should be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface)
	uint32 bEnableMouseOverEvents:1;

	/** Whether actor/component touch over events should be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface)
	uint32 bEnableTouchOverEvents:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Game|Feedback")
	uint32 bForceFeedbackEnabled:1;

	/** Whether or not to consider input from motion sources (tilt, acceleration, etc) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetMotionControlsEnabled, Category="Input")
	uint32 bEnableMotionControls:1;

	UFUNCTION(BlueprintSetter)
	ENGINE_API void SetMotionControlsEnabled(bool bEnabled);

	/** Whether the PlayerController should be used as a World Partiton streaming source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition)
	uint32 bEnableStreamingSource:1;

	/** Whether the PlayerController streaming source should activate cells after loading. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition, meta=(EditCondition="bEnableStreamingSource"))
	uint32 bStreamingSourceShouldActivate:1;

	/** Whether the PlayerController streaming source should block on slow streaming. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition, meta=(EditCondition="bEnableStreamingSource"))
	uint32 bStreamingSourceShouldBlockOnSlowStreaming:1;

	/** PlayerController streaming source priority. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition, meta=(EditCondition="bEnableStreamingSource"))
	EStreamingSourcePriority StreamingSourcePriority;

	/** Color used for debugging. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition, meta = (EditCondition = "bEnableStreamingSource"))
	FColor StreamingSourceDebugColor;

	/** Optional aggregated shape list used to build a custom shape for the streaming source. When empty, fallbacks sphere shape with a radius equal to grid's loading range. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPartition, meta = (EditCondition = "bEnableStreamingSource"))
	TArray<FStreamingSourceShape> StreamingSourceShapes;

	/** Scale applied to force feedback values */
	UPROPERTY(config)
	float ForceFeedbackScale;

	/** List of keys that will cause click events to be forwarded, default to left click */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface, meta=(EditCondition="bEnableClickEvents"))
	TArray<FKey> ClickEventKeys;

	/** Type of mouse cursor to show by default */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MouseInterface)
	TEnumAsByte<EMouseCursor::Type> DefaultMouseCursor;

	/** Currently visible mouse cursor */
	UPROPERTY(BlueprintReadWrite, Category=MouseInterface)
	TEnumAsByte<EMouseCursor::Type> CurrentMouseCursor;

	/** Default trace channel used for determining what world object was clicked on. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MouseInterface)
	TEnumAsByte<ECollisionChannel> DefaultClickTraceChannel;

	/** Trace channel currently being used for determining what world object was clicked on. */
	UPROPERTY(BlueprintReadWrite, Category=MouseInterface)
	TEnumAsByte<ECollisionChannel> CurrentClickTraceChannel;

	/** Distance to trace when computing click events */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MouseInterface, meta=(DisplayName="Trace Distance"))
	float HitResultTraceDistance;

	/** Counter for this players seamless travels (used along with the below value, to restrict ServerNotifyLoadedWorld) */
	UPROPERTY()
	uint16 SeamlessTravelCount;

	/** The value of SeamlessTravelCount, upon the last call to GameModeBase::HandleSeamlessTravelPlayer; used to detect seamless travel */
	UPROPERTY()
	uint16 LastCompletedSeamlessTravelCount;

public:
	/** Run from the console to try and manually enable cheats which are disabled by default in multiplayer, games can override this */
	UFUNCTION(exec)
	ENGINE_API virtual void EnableCheats();

	/** Timer used by RoundEnded and Inactive states to accept player input again */
	ENGINE_API virtual void UnFreeze();

	/** Calculate minimal respawn delay */
	ENGINE_API virtual float GetMinRespawnDelay();

	/** Set the field of view to NewFOV */ 
	UFUNCTION(exec)
	ENGINE_API virtual void FOV(float NewFOV);

	/** Restarts the current level */
	UFUNCTION(exec)
	ENGINE_API virtual void RestartLevel();

	/** Causes the client to travel to the given URL */
	UFUNCTION(exec)
	ENGINE_API virtual void LocalTravel(const FString& URL);

	/** RPC used by ServerExec. Not intended to be called directly */
	UFUNCTION(Reliable, Server, WithValidation)
	ENGINE_API void ServerExecRPC(const FString& Msg);

	/** Executes command on server (non shipping builds only) */
	UFUNCTION(exec)
	ENGINE_API void ServerExec(const FString& Msg);

	/** Return the client to the main menu gracefully */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientReturnToMainMenuWithTextReason(const FText& ReturnReason);

	/** Development RPC for testing object reference replication */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientRepObjRef(UObject* Object);

	/**
	 * Locally try to pause game (call serverpause to pause network game); returns success indicator.  Calls GameModeBase's SetPause().
	 * @return true if succeeded to pause
	 */
	ENGINE_API virtual bool SetPause(bool bPause, FCanUnpause CanUnpauseDelegate = FCanUnpause());

	/** Command to try to pause the game. */
	UFUNCTION(exec)
	ENGINE_API virtual void Pause();

	/** Tries to set the player's name to the given name. */
	UFUNCTION(exec)
	ENGINE_API virtual void SetName(const FString& S);

	/** SwitchLevel to the given MapURL. */
	UFUNCTION(exec)
	ENGINE_API virtual void SwitchLevel(const FString& URL);

	/** 
	 * Called to notify the server when the client has loaded a new world via seamless traveling
	 * @param WorldPackageName the name of the world package that was loaded
	 * @param bFinalDest whether this world is the destination map for the travel (i.e. not the transition level)
	 */
	ENGINE_API virtual void NotifyLoadedWorld(FName WorldPackageName, bool bFinalDest);

	/**
	 * Processes player input (immediately after PlayerInput gets ticked) and calls UpdateRotation().
	 * PlayerTick is only called if the PlayerController has a PlayerInput object. Therefore, it will only be called for locally controlled PlayerControllers.
	 */
	ENGINE_API virtual void PlayerTick(float DeltaTime);

	/** Method called prior to processing input */
	ENGINE_API virtual void PreProcessInput(const float DeltaTime, const bool bGamePaused);

	/** Method called after processing input */
	ENGINE_API virtual void PostProcessInput(const float DeltaTime, const bool bGamePaused);

	/** Adjust input based on cinematic mode 
	  * @param	bInCinematicMode	specify true if the player is entering cinematic mode; false if the player is leaving cinematic mode.
	  * @param	bAffectsMovement	specify true to disable movement in cinematic mode, enable it when leaving
	  * @param	bAffectsTurning		specify true to disable turning in cinematic mode or enable it when leaving
	  */
	ENGINE_API virtual void SetCinematicMode( bool bInCinematicMode, bool bAffectsMovement, bool bAffectsTurning);

	/** Reset move and look input ignore flags to defaults */
	ENGINE_API virtual void ResetIgnoreInputFlags() override;

	/** Returns hit results from doing a collision query at a certain location on the screen */
	ENGINE_API bool GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ECollisionChannel TraceChannel, const FCollisionQueryParams& CollisionQueryParams, FHitResult& HitResult) const;
	ENGINE_API bool GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;
	ENGINE_API bool GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;
	ENGINE_API bool GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const;

	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(DeprecatedFunction, DeprecationMessage = "Use new GetHitResultUnderCursorByChannel or GetHitResultUnderCursorForObject", TraceChannel=ECC_Visibility, bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderCursor(ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;

	/** Performs a collision query under the mouse cursor, looking on a trace channel */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderCursorByChannel(ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;

	/** Performs a collision query under the mouse cursor, looking for object types */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderCursorForObjects(const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const;

	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(DeprecatedFunction, DeprecationMessage = "Use new GetHitResultUnderFingerByChannel or GetHitResultUnderFingerForObject", TraceChannel=ECC_Visibility, bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderFinger(ETouchIndex::Type FingerIndex, ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;

	/** Performs a collision query under the finger, looking on a trace channel */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderFingerByChannel(ETouchIndex::Type FingerIndex, ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const;

	/** Performs a collision query under the finger, looking for object types */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(bTraceComplex=true))
	ENGINE_API bool GetHitResultUnderFingerForObjects(ETouchIndex::Type FingerIndex, const  TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const;

	/** Convert current mouse 2D position to World Space 3D position and direction. Returns false if unable to determine value. **/
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta = (DisplayName = "Convert Mouse Location To World Space", Keywords = "deproject"))
	ENGINE_API bool DeprojectMousePositionToWorld(FVector& WorldLocation, FVector& WorldDirection) const;

	/** Convert 2D screen position to World Space 3D position and direction. Returns false if unable to determine value. **/
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta = (DisplayName = "Convert Screen Location To World Space", Keywords = "deproject"))
	ENGINE_API bool DeprojectScreenPositionToWorld(float ScreenX, float ScreenY, FVector& WorldLocation, FVector& WorldDirection) const;

	/**
	 * Convert a World Space 3D position into a 2D Screen Space position.
	 * @return true if the world coordinate was successfully projected to the screen.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta = (DisplayName = "Convert World Location To Screen Location", Keywords = "project"))
	ENGINE_API bool ProjectWorldLocationToScreen(FVector WorldLocation, FVector2D& ScreenLocation, bool bPlayerViewportRelative = false) const;

	/**
	 * Convert a World Space 3D position into a 3D Screen Space position.
	 * @return true if the world coordinate was successfully projected to the screen.
	 */
	ENGINE_API bool ProjectWorldLocationToScreenWithDistance(FVector WorldLocation, FVector& ScreenLocation, bool bPlayerViewportRelative = false) const;
	
	/**
	 * After successful world to screen projection, allows custom post-processing of the resulting ScreenLocation.
	 * @return Whether projected location remains valid.
	 */
	ENGINE_API virtual bool PostProcessWorldToScreen(FVector WorldLocation, FVector2D& ScreenLocation, bool bPlayerViewportRelative) const;

	/** Positions the mouse cursor in screen space, in pixels. */
	UFUNCTION( BlueprintCallable, Category="Game|Player", meta = (DisplayName = "Set Mouse Position", Keywords = "mouse" ))
	ENGINE_API void SetMouseLocation( const int X, const int Y );

	/**
	 * Updates the rotation of player, based on ControlRotation after RotationInput has been applied.
	 * This may then be modified by the PlayerCamera, and is passed to Pawn->FaceRotation().
	 */
	ENGINE_API virtual void UpdateRotation(float DeltaTime);

	/**
	* Whether the PlayerController should be used as a World Partiton streaming source. 
	* Default implementation returns bEnableStreamingSource but can be overriden in child classes.
	* @return true if it should.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	virtual bool IsStreamingSourceEnabled() const { return bEnableStreamingSource; }

	/**
	* Whether the PlayerController streaming source should activate cells after loading.
	* Default implementation returns bStreamingSourceShouldActivate but can be overriden in child classes.
	* @return true if it should.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	virtual bool StreamingSourceShouldActivate() const { return bEnableStreamingSource && bStreamingSourceShouldActivate; }

	/**
	* Whether the PlayerController streaming source should block on slow streaming.
	* Default implementation returns bStreamingSourceShouldBlockOnSlowStreaming but can be overriden in child classes.
	* @return true if it should.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	virtual bool StreamingSourceShouldBlockOnSlowStreaming() const { return bEnableStreamingSource && bStreamingSourceShouldBlockOnSlowStreaming; }

	/**
	* Gets the streaming source priority.
	* Default implementation returns StreamingSourcePriority but can be overriden in child classes.
	* @return the streaming source priority.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	virtual EStreamingSourcePriority GetStreamingSourcePriority() const { return StreamingSourcePriority; }

	/**
	* Gets the streaming source location and rotation.
	* Default implementation returns APlayerController::GetPlayerViewPoint but can be overriden in child classes.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	ENGINE_API virtual void GetStreamingSourceLocationAndRotation(FVector& OutLocation, FRotator& OutRotation) const;
	
	/**
	* Gets the streaming source priority.
	* Default implementation returns StreamingSourceShapes but can be overriden in child classes.
	* @return the streaming source priority.
	*/
	UFUNCTION(BlueprintCallable, Category = WorldPartition)
	ENGINE_API virtual void GetStreamingSourceShapes(TArray<FStreamingSourceShape>& OutShapes) const;

	/**
	 * Gets the PlayerController's streaming sources
	 * @return the streaming sources.
	 */
	ENGINE_API virtual bool GetStreamingSources(TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const final;

protected:
	ENGINE_API virtual bool GetStreamingSourcesInternal(TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const;
	ENGINE_API virtual bool GetStreamingSource(FWorldPartitionStreamingSource& OutStreamingSource) const final;
	virtual const UObject* GetStreamingSourceOwner() const override final { return this; }
	
	/** Pawn has been possessed, so changing state to NAME_Playing. Start it walking and begin playing with it. */
	ENGINE_API virtual void BeginPlayingState();

	/** Leave playing state. */ 
	ENGINE_API virtual void EndPlayingState();

	/** Overridden to return that player controllers are capable of RPCs */
	ENGINE_API virtual bool HasNetOwner() const override;

public:
	/** Fire the player's currently selected weapon with the optional firemode. */
	UFUNCTION(exec)
	ENGINE_API virtual void StartFire(uint8 FireModeNum = 0);

	/** Notify player of change to level */
	ENGINE_API void LevelStreamingStatusChanged(class ULevelStreaming* LevelObject, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, int32 LODIndex);
	ENGINE_API void LevelStreamingStatusChanged(class ULevelStreaming* LevelObject, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, bool bNewShouldBlockOnUnload, int32 LODIndex);

	/** Used to wait until a map change can be prepared when one was already in progress */
	ENGINE_API virtual void DelayedPrepareMapChange();

	/**
	 * Called on client during seamless level transitions to get the list of Actors that should be moved into the new level
	 * PlayerControllers, Role < ROLE_Authority Actors, and any non-Actors that are inside an Actor that is in the list
	 * (i.e. Object.Outer == Actor in the list)
	 * are all automatically moved regardless of whether they're included here
	 * only dynamic actors in the PersistentLevel may be moved (this includes all actors spawned during gameplay)
	 * this is called for both parts of the transition because actors might change while in the middle (e.g. players might join or leave the game)
	 * @see also GameModeBase::GetSeamlessTravelActorList() (the function that's called on servers)
	 * @param bToEntry true if we are going from old level -> entry, false if we are going from entry -> new level
	 * @param ActorList (out) list of actors to maintain
	 */
	ENGINE_API virtual void GetSeamlessTravelActorList(bool bToEntry, TArray<class AActor*>& ActorList);

	/**
	 * Called when seamless traveling and we are being replaced by the specified PC
	 * clean up any persistent state (post process chains on LocalPlayers, for example)
	 * (not called if PlayerController is the same for the from and to GameModes)
	 */
	ENGINE_API virtual void SeamlessTravelTo(class APlayerController* NewPC);

	/**
	 * Called when seamless traveling and the specified PC is being replaced by this one
	 * copy over data that should persist
	 * (not called if PlayerController is the same for the from and to GameModes)
	 */
	ENGINE_API virtual void SeamlessTravelFrom(class APlayerController* OldPC);

	/** 
	 * Called after this player controller has transitioned through seamless travel, but before that player is initialized
	 * This is called both when a new player controller is created, and when it is maintained
	 */
	ENGINE_API virtual void PostSeamlessTravel();

	/**
	 * Called when player controller gets added to its owning world player controller list. 
	 */
	ENGINE_API void OnAddedToPlayerControllerList();

	/**
	 * Called when player controller gets removed from its owning world player controller list.
	 */
	ENGINE_API void OnRemovedFromPlayerControllerList();

	/** 
	 * Tell the client to enable or disable voice chat (not muting)
	 * @param bEnable enable or disable voice chat
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientEnableNetworkVoice(bool bEnable);

	/** 
	 * Acknowledge received LevelVisibilityTransactionId
	 * @param PackageName - Identifying the level that we are acknowledging levelvisibility for
	 * @param TransactionId - TransactionId being acknowledged
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientAckUpdateLevelVisibility(FName PackageName, FNetLevelVisibilityTransactionId TransactionId, bool bClientAckCanMakeVisible);

	/** Enable voice chat transmission */
	ENGINE_API void StartTalking();

	/** Disable voice chat transmission */
	ENGINE_API void StopTalking();

	/** 
	 * Toggle voice chat on and off
	 * @param bSpeaking enable or disable voice chat
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void ToggleSpeaking(bool bInSpeaking);

	/**
	 * Tells the client that the server has all the information it needs and that it
	 * is ok to start sending voice packets. The server will already send voice packets
	 * when this function is called, since it is set server side and then forwarded
	 *
	 * NOTE: This is done as an RPC instead of variable replication because ordering matters
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientVoiceHandshakeComplete();

	/**
	 * Tell the server to mute a player for this controller
	 * @param PlayerId player id to mute
	 */
	UFUNCTION(server, reliable, WithValidation)
	ENGINE_API virtual void ServerMutePlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the server to unmute a player for this controller
	 * @param PlayerId player id to unmute
	 */
	UFUNCTION(server, reliable, WithValidation )
	ENGINE_API virtual void ServerUnmutePlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the client to mute a player for this controller
	 * @param PlayerId player id to mute
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientMutePlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the client to unmute a player for this controller
	 * @param PlayerId player id to unmute
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientUnmutePlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the client to block a player for this controller
	 * @param PlayerId player id to block
	 */
	UFUNCTION(server, reliable, WithValidation)
	ENGINE_API virtual void ServerBlockPlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the client to unblock a player for this controller
	 * @param PlayerId player id to unblock
	 */
	UFUNCTION(server, reliable, WithValidation)
	ENGINE_API virtual void ServerUnblockPlayer(FUniqueNetIdRepl PlayerId);

	/**
	 * Tell the client to unmute an array of players for this controller
	 * @param PlayerIds player ids to unmute
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API virtual void ClientUnmutePlayers(const TArray<FUniqueNetIdRepl>& PlayerIds);

	/**
	 * Mutes a remote player on the server and then tells the client to mute
	 *
	 * @param PlayerNetId the remote player to mute
	 */
	ENGINE_API void GameplayMutePlayer(const FUniqueNetIdRepl& PlayerNetId);

	/**
	 * Unmutes a remote player on the server and then tells the client to unmute
	 *
	 * @param PlayerNetId the remote player to unmute
	 */
	ENGINE_API void GameplayUnmutePlayer(const FUniqueNetIdRepl& PlayerNetId);

	/**
	 * Unmutes all remote players muted due to gameplay rules on the server and then tells the client to unmute
	 */
	ENGINE_API void GameplayUnmuteAllPlayers();

	/**
	* Get a remote player controller on the server for muting
	*
	* @param PlayerNetId the remote player to find
	*/
	ENGINE_API virtual class APlayerController* GetPlayerControllerForMuting(const FUniqueNetIdRepl& PlayerNetId);

	/**
	 * Is the specified player muted by this controlling player
	 * for any reason (gameplay, system, etc), check voice interface IsMuted() for system mutes
	 *
	 * @param PlayerId potentially muted player
	 * @return true if player is muted, false otherwise
	 */
	ENGINE_API virtual bool IsPlayerMuted(const class FUniqueNetId& PlayerId);


	/** Console control commands, useful when remote debugging so you can't touch the console the normal way */
	UFUNCTION(exec)
	ENGINE_API virtual void ConsoleKey(FKey Key);

	/** Sends a command to the console to execute if not shipping version */
	UFUNCTION(exec)
	ENGINE_API virtual void SendToConsole(const FString& Command);

	/** Adds a location to the texture streaming system for the specified duration. */
	UFUNCTION(reliable, client, SealedEvent)
	ENGINE_API void ClientAddTextureStreamingLoc(FVector InLoc, float Duration, bool bOverrideLocation);

	/** Tells client to cancel any pending map change. */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientCancelPendingMapChange();

	/** Set CurrentNetSpeed to the lower of its current value and Cap. */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientCapBandwidth(int32 Cap);

	/** Actually performs the level transition prepared by PrepareMapChange(). */
	UFUNCTION(Reliable, Client)	
	ENGINE_API void ClientCommitMapChange();

	/**
	 * Tells the client to block until all pending level streaming actions are complete
	 * happens at the end of the tick
	 * primarily used to force update the client ASAP at join time
	 */
	UFUNCTION(reliable, client, SealedEvent)
	ENGINE_API void ClientFlushLevelStreaming();

	/** Forces GC at the end of the tick on the client */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientForceGarbageCollection();

	/** 
	 * Replicated function called by GameHasEnded().
	 * @param	EndGameFocus - actor to view with camera
	 * @param	bIsWinner - true if this controller is on winning team
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientGameEnded(class AActor* EndGameFocus, bool bIsWinner);

	/** 
	 * Server uses this to force client into NewState .
	 * @Note ALL STATE NAMES NEED TO BE DEFINED IN name table in UnrealNames.h to be correctly replicated (so they are mapped to the same thing on client and server).
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientGotoState(FName NewState);

	/** Calls IgnoreLookInput on client */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientIgnoreLookInput(bool bIgnore);

	/** Calls IgnoreMoveInput on client */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientIgnoreMoveInput(bool bIgnore);

	/**
	 * Outputs a message to HUD
	 * @param S - message to display
	 * @param Type - @todo document
	 * @param MsgLifeTime - Optional length of time to display 0 = default time
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientMessage(const FString& S, FName Type = NAME_None, float MsgLifeTime = 0.f);

	/** 
	 * Play Camera Shake 
	 * @param Shake - Camera shake animation to play
	 * @param Scale - Scalar defining how "intense" to play the anim
	 * @param PlaySpace - Which coordinate system to play the shake in (used for CameraAnims within the shake).
	 * @param UserPlaySpaceRot - Matrix used when PlaySpace = CAPS_UserDefined
	 */
	UFUNCTION(unreliable, client, BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientStartCameraShake(TSubclassOf<class UCameraShakeBase> Shake, float Scale = 1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/** 
	 * Play Camera Shake localized to a given source
	 * @param Shake - Camera shake animation to play
	 * @param SourceComponent - The source from which the camera shakes originates
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientStartCameraShakeFromSource(TSubclassOf<class UCameraShakeBase> Shake, class UCameraShakeSourceComponent* SourceComponent);

	/**
	 * Play sound client-side (so only the client will hear it)
	 * @param Sound	- Sound to play
	 * @param VolumeMultiplier - Volume multiplier to apply to the sound
	 * @param PitchMultiplier - Pitch multiplier to apply to the sound
	 */
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientPlaySound(class USoundBase* Sound, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f );

	/**
	 * Play sound client-side at the specified location
	 * @param Sound	- Sound to play
	 * @param Location - Location to play the sound at
	 * @param VolumeMultiplier - Volume multiplier to apply to the sound
	 * @param PitchMultiplier - Pitch multiplier to apply to the sound
	 */
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientPlaySoundAtLocation(class USoundBase* Sound, FVector Location, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f);

	/** Asynchronously loads the given level in preparation for a streaming map transition.
	 * the server sends one function per level name since dynamic arrays can't be replicated
	 * @param LevelNames - the names of the level packages to load. LevelNames[0] will be the new persistent (primary) level
	 * @param bFirst - whether this is the first item in the list (so clear the list first)
	 * @param bLast - whether this is the last item in the list (so start preparing the change after receiving it)
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientPrepareMapChange(FName LevelName, bool bFirst, bool bLast);

	/**
	 * Forces the streaming system to disregard the normal logic for the specified duration and
	 * instead always load all mip-levels for all textures used by the specified actor.
	 * @param ForcedActor		- The actor whose textures should be forced into memory.
	 * @param ForceDuration		- Number of seconds to keep all mip-levels in memory, disregarding the normal priority logic.
	 * @param bEnableStreaming	- Whether to start (true) or stop (false) streaming
	 * @param CinematicTextureGroups	- Bitfield indicating which texture groups that use extra high-resolution mips
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientPrestreamTextures(class AActor* ForcedActor, float ForceDuration, bool bEnableStreaming, int32 CinematicTextureGroups = 0);

	/** Tell client to reset the PlayerController */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientReset();

	/** Tell client to restart the level */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientRestart(class APawn* NewPawn);

	/** 
	 * Tells the client to block until all pending level streaming actions are complete.
	 * Happens at the end of the tick primarily used to force update the client ASAP at join time.
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetBlockOnAsyncLoading();

	/**
	 * Tell client to fade camera
	 * @Param bEnableFading - true if we should apply FadeColor/FadeAmount to the screen
	 * @Param FadeColor - Color to fade to
	 * @Param FadeAlpha - Contains the start fade (X) and end fade (Y) values to apply. A start fade of less than 0 will use the screen's current fade value
	 * @Param FadeTime - length of time for fade to occur over
	 * @Param bFadeAudio - true to apply fading of audio alongside the video
	 * @param bHoldWhenFinished - True for fade to hold at the ToAlpha until fade is disabled
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetCameraFade(bool bEnableFading, FColor FadeColor = FColor(ForceInit), FVector2D FadeAlpha = FVector2D(-1.0f, 0.0f), float FadeTime = 0, bool bFadeAudio = false, bool bHoldWhenFinished = false);

	/**
	 * Replicated function to set camera style on client
	 * @param	NewCamMode, name defining the new camera mode
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetCameraMode(FName NewCamMode);

	/** Called by the server to synchronize cinematic transitions with the client */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetCinematicMode(bool bInCinematicMode, bool bAffectsMovement, bool bAffectsTurning, bool bAffectsHUD);

	/**
	 * Forces the streaming system to disregard the normal logic for the specified duration and
	 * instead always load all mip-levels for all textures used by the specified material.
	 *
	 * @param Material		- The material whose textures should be forced into memory.
	 * @param ForceDuration	- Number of seconds to keep all mip-levels in memory, disregarding the normal priority logic.
	 * @param CinematicTextureGroups	- Bitfield indicating which texture groups that use extra high-resolution mips
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetForceMipLevelsToBeResident(class UMaterialInterface* Material, float ForceDuration, int32 CinematicTextureGroups = 0);

	/** Set the client's class of HUD and spawns a new instance of it. If there was already a HUD active, it is destroyed. */
	UFUNCTION(BlueprintCallable, Category="HUD", Reliable, Client)
	ENGINE_API void ClientSetHUD(TSubclassOf<AHUD> NewHUDClass);

	/** Helper to get the size of the HUD canvas for this player controller.  Returns 0 if there is no HUD */
	UFUNCTION(BlueprintCallable, Category="HUD")
	ENGINE_API void GetViewportSize(int32& SizeX, int32& SizeY) const;

	/** Gets the HUD currently being used by this player controller */
	UFUNCTION(BlueprintCallable, Category="HUD")
	ENGINE_API AHUD* GetHUD() const;

	/** Templated version of GetHUD, will return nullptr if cast fails */
	template<class T>
	T* GetHUD() const
	{
		return Cast<T>(MyHUD);
	}

	/**
	 * Sets the Widget for the Mouse Cursor to display 
	 * @param Cursor - the cursor to set the widget for
	 * @param CursorWidget - the widget to set the cursor to
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API void SetMouseCursorWidget(EMouseCursor::Type Cursor, UPARAM(Required) class UUserWidget* CursorWidget);

	/** Set the view target
	 * @param A - new actor to set as view target
	 * @param TransitionParams - parameters to use for controlling the transition
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientSetViewTarget(class AActor* A, struct FViewTargetTransitionParams TransitionParams = FViewTargetTransitionParams());
	
	/** Spawn a camera lens effect (e.g. blood). */	
	UFUNCTION(unreliable, client, BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientSpawnGenericCameraLensEffect(UPARAM(meta=(MustImplement ="CameraLensEffectInterface")) TSubclassOf<class AActor>  LensEffectEmitterClass);

	UFUNCTION(unreliable, client, Category="Game|Feedback", meta=(DeprecatedFunction, DeprecationMessage="Prefer the version taking ICameraLensEffectInterface (ClientSpawnGenericCameraLensEffect)"))
	ENGINE_API void ClientSpawnCameraLensEffect(TSubclassOf<class AEmitterCameraLensEffectBase>  LensEffectEmitterClass);

	/** Removes all Camera Lens Effects. */
	UFUNCTION(reliable, client, BlueprintCallable, Category="Game|Feedback")
	ENGINE_API virtual void ClientClearCameraLensEffects();

	/** Stop camera shake on client.  */
	UFUNCTION(reliable, client, BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientStopCameraShake(TSubclassOf<class UCameraShakeBase> Shake, bool bImmediately = true);

	/** Stop camera shake on client.  */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientStopCameraShakesFromSource(class UCameraShakeSourceComponent* SourceComponent, bool bImmediately = true);

	/** 
	 * Play a force feedback pattern on the player's controller
	 * @param	ForceFeedbackEffect		The force feedback pattern to play
	 * @param	bLooping				Whether the pattern should be played repeatedly or be a single one shot
	 * @param	bIgnoreTimeDilation		Whether the pattern should ignore time dilation
	 * @param	bPlayWhilePaused		Whether the pattern should continue to play while the game is paused
	 * @param	Tag						A tag that allows stopping of an effect.  If another effect with this Tag is playing, it will be stopped and replaced
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback", meta=(DisplayName="Client Play Force Feedback", AdvancedDisplay="bIgnoreTimeDilation,bPlayWhilePaused"))
	ENGINE_API void K2_ClientPlayForceFeedback(class UForceFeedbackEffect* ForceFeedbackEffect, FName Tag, bool bLooping, bool bIgnoreTimeDilation, bool bPlayWhilePaused);

private:
	/** 
	 * Internal replicated version of client play force feedback event. 
	 * Cannot be named ClientPlayForceFeedback as redirector for blueprint function version to K2_... does not work in that case
	 */
	UFUNCTION(unreliable, client)
	ENGINE_API void ClientPlayForceFeedback_Internal(class UForceFeedbackEffect* ForceFeedbackEffect, FForceFeedbackParameters Params = FForceFeedbackParameters());

public:

	/** 
	 * Play a force feedback pattern on the player's controller
	 * @param	ForceFeedbackEffect		The force feedback pattern to play
	 * @param	Params					Parameter struct to customize playback behavior of the feedback effect
	 */
	void ClientPlayForceFeedback(class UForceFeedbackEffect* ForceFeedbackEffect, FForceFeedbackParameters Params = FForceFeedbackParameters())
	{
		ClientPlayForceFeedback_Internal(ForceFeedbackEffect, Params);
	}

	/** 
	 * Stops a playing force feedback pattern
	 * @param	ForceFeedbackEffect		If set only patterns from that effect will be stopped
	 * @param	Tag						If not none only the pattern with this tag will be stopped
	 */
	UFUNCTION(reliable, client, BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ClientStopForceFeedback(class UForceFeedbackEffect* ForceFeedbackEffect, FName Tag);

public:
	/** 
	 * Latent action that controls the playing of force feedback 
	 * Begins playing when Start is called.  Calling Update or Stop if the feedback is not active will have no effect.
	 * Completed will execute when Stop is called or the duration ends.
	 * When Update is called the Intensity, Duration, and affect values will be updated with the current inputs
	 * @param	Intensity				How strong the feedback should be.  Valid values are between 0.0 and 1.0
	 * @param	Duration				How long the feedback should play for.  If the value is negative it will play until stopped
	 * @param   bAffectsLeftLarge		Whether the intensity should be applied to the large left servo
	 * @param   bAffectsLeftSmall		Whether the intensity should be applied to the small left servo
	 * @param   bAffectsRightLarge		Whether the intensity should be applied to the large right servo
	 * @param   bAffectsRightSmall		Whether the intensity should be applied to the small right servo
	 */
	UFUNCTION(BlueprintCallable, meta=(Latent, LatentInfo="LatentInfo", ExpandEnumAsExecs="Action", Duration="-1", bAffectsLeftLarge="true", bAffectsLeftSmall="true", bAffectsRightLarge="true", bAffectsRightSmall="true", AdvancedDisplay="bAffectsLeftLarge,bAffectsLeftSmall,bAffectsRightLarge,bAffectsRightSmall"), Category="Game|Feedback")
	ENGINE_API void PlayDynamicForceFeedback(float Intensity, float Duration, bool bAffectsLeftLarge, bool bAffectsLeftSmall, bool bAffectsRightLarge, bool bAffectsRightSmall, TEnumAsByte<EDynamicForceFeedbackAction::Type> Action, FLatentActionInfo LatentInfo);

private:
	//~ This method is purely for debugging purposes.
	//~ It will trigger a ServerUpdateLevelVisibilityCall with the provided package name.
	UFUNCTION(Exec)
	ENGINE_API void TestServerLevelVisibilityChange(const FName PackageName, const FName FileName);

public:
	/** 
	 * Allows playing of a dynamic force feedback event from native code
	 * Begins playing when Start is called.  Calling with Action set to Update or Stop if the feedback is not active will have no effect.
	 * When Update is called the Intensity, Duration, and affect values will be updated
	 * @param	Intensity				How strong the feedback should be.  Valid values are between 0.0 and 1.0
	 * @param	Duration				How long the feedback should play for.  If the value is negative it will play until stopped
	 * @param   bAffectsLeftLarge		Whether the intensity should be applied to the large left servo
	 * @param   bAffectsLeftSmall		Whether the intensity should be applied to the small left servo
	 * @param   bAffectsRightLarge		Whether the intensity should be applied to the large right servo
	 * @param   bAffectsRightSmall		Whether the intensity should be applied to the small right servo
	 * @param   Action					Whether to (re)start, update, or stop the action
	 * @param	UniqueID				The ID returned by the start action when wanting to restart, update, or stop the action
	 * @return  The index to pass in to the function to update the latent action in the future if needed. Returns 0 if the feedback was stopped
	 *          or the specified UniqueID did not map to an action that can be started/updated.
	 */
	ENGINE_API FDynamicForceFeedbackHandle PlayDynamicForceFeedback(float Intensity, float Duration, bool bAffectsLeftLarge, bool bAffectsLeftSmall, bool bAffectsRightLarge, bool bAffectsRightSmall, EDynamicForceFeedbackAction::Type Action = EDynamicForceFeedbackAction::Start, FDynamicForceFeedbackHandle ActionHandle = 0);

	/**
	 * Play a haptic feedback curve on the player's controller
	 * @param	HapticEffect			The haptic effect to play
	 * @param	Hand					Which hand to play the effect on
	 * @param	Scale					Scale between 0.0 and 1.0 on the intensity of playback
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void PlayHapticEffect(class UHapticFeedbackEffect_Base* HapticEffect, EControllerHand Hand, float Scale = 1.f,  bool bLoop = false);

	/**
	 * Stops a playing haptic feedback curve
	 * @param	HapticEffect			The haptic effect to stop
	 * @param	Hand					Which hand to stop the effect for
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void StopHapticEffect(EControllerHand Hand);

	/**
	 * Sets the value of the haptics for the specified hand directly, using frequency and amplitude.  NOTE:  If a curve is already
	 * playing for this hand, it will be cancelled in favour of the specified values.
	 *
	 * @param	Frequency				The normalized frequency [0.0, 1.0] to play through the haptics system
	 * @param	Amplitude				The normalized amplitude [0.0, 1.0] to set the haptic feedback to
	 * @param	Hand					Which hand to play the effect on
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void SetHapticsByValue(const float Frequency, const float Amplitude, EControllerHand Hand);
	
	/**
	 * Allows the player controller to disable all haptic requests from being fired, e.g. in the case of a level loading
	 *
	 * @param	bNewDisabled	If TRUE, the haptics will stop and prevented from being enabled again until set to FALSE
	 */
	UFUNCTION(BlueprintCallable, Category = "Game|Feedback")
	ENGINE_API virtual void SetDisableHaptics(bool bNewDisabled);

	/**
	 * Sets the light color of the player's controller
	 * @param	Color					The color for the light to be
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void SetControllerLightColor(FColor Color);
	
	/**
	 * Resets the light color of the player's controller to default
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Feedback")
	ENGINE_API void ResetControllerLightColor();

	/**
	 * Sets the deadzones of the player's controller
	 * @param	LeftDeadZone 	Inner DeadZone for the left analog stick
	 * @param	RightDeadZone	Inner DeadZone for the right analog stick
	 */
	UFUNCTION(BlueprintCallable, Category = "Input")
	ENGINE_API void SetControllerDeadZones(const float LeftDeadZone, const float RightDeadZone);

	/**
	 * Resets the player's controller deadzones to default
	 */
	UFUNCTION(BlueprintCallable, Category = "Input")
	ENGINE_API void ResetControllerDeadZones();

	/**
	 * Sets the trigger release thresholds of the player's controller
	 * @param	LeftThreshold 	Release Threshold for the left trigger
	 * @param	RightThreshold	Release Threshold for the right trigger
	 */
	UFUNCTION(BlueprintCallable, Category = "Input")
	ENGINE_API void SetControllerTriggerReleaseThresholds(const float LeftThreshold, const float RightThreshold);

	/**
	 * Resets the player's controller trigger release thresholds to default
	 */
	UFUNCTION(BlueprintCallable, Category = "Input")
	ENGINE_API void ResetControllerTriggerReleaseThresholds();

	/**
	 * Sets whether the player's controller's gyro auto calibration is enabled
	 * @param	bEnabled	 	Whether gyro auto calibration should be enabled
	 */
	UFUNCTION(BlueprintCallable, Category = "Input")
	ENGINE_API void SetControllerGyroAutoCalibration(bool bEnabled);

	/**
	 * Travel to a different map or IP address. Calls the PreClientTravel event before doing anything.
	 * NOTE: This is implemented as a locally executed wrapper for ClientTravelInternal, to avoid API compatability breakage
	 *
	 * @param URL				A string containing the mapname (or IP address) to travel to, along with option key/value pairs
	 * @param TravelType 		specifies whether the client should append URL options used in previous travels; if true is specified
	 *							for the bSeamlesss parameter, this value must be TRAVEL_Relative.
	 * @param bSeamless			Indicates whether to use seamless travel (requires TravelType of TRAVEL_Relative)
	 * @param MapPackageGuid	The GUID of the map package to travel to - this is used to find the file when it has been autodownloaded,
	 * 							so it is only needed for clients
	 */
	UFUNCTION()
	ENGINE_API void ClientTravel(const FString& URL, enum ETravelType TravelType, bool bSeamless = false, FGuid MapPackageGuid = FGuid());

	/**
	 * Internal clientside implementation of ClientTravel - use ClientTravel to call this
	 *
	 * @param URL				A string containing the mapname (or IP address) to travel to, along with option key/value pairs
	 * @param TravelType 		specifies whether the client should append URL options used in previous travels; if true is specified
	 *							for the bSeamlesss parameter, this value must be TRAVEL_Relative.
	 * @param bSeamless			Indicates whether to use seamless travel (requires TravelType of TRAVEL_Relative)
	 * @param MapPackageGuid	The GUID of the map package to travel to - this is used to find the file when it has been autodownloaded,
	 * 							so it is only needed for clients
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientTravelInternal(const FString& URL, enum ETravelType TravelType, bool bSeamless = false, FGuid MapPackageGuid = FGuid());

	/**
	 * Replicated Update streaming status
	 * @param PackageName - Name of the level package name used for loading.
	 * @param bNewShouldBeLoaded - Whether the level should be loaded
	 * @param bNewShouldBeVisible - Whether the level should be visible if it is loaded	
	 * @param bNewShouldBlockOnLoad - Whether we want to force a blocking load
	 * @param LODIndex				- Current LOD index for a streaming level
	 * @param TransactionId			- Optional parameter used when communicating LevelVisibility changes between server and client
	 * @param bNewShouldBlockOnUnload - Optional parameter used to force a blocking unload or not
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientUpdateLevelStreamingStatus(FName PackageName, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, int32 LODIndex, FNetLevelVisibilityTransactionId TransactionId = FNetLevelVisibilityTransactionId(), bool bNewShouldBlockOnUnload = false);

	/**
	 * Replicated Update streaming status.  This version allows for the streaming state of many levels to be sent in a single RPC.
	 * @param LevelStatuses	The list of levels the client should have either streamed in or not, depending on state.
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientUpdateMultipleLevelsStreamingStatus(const TArray<FUpdateLevelStreamingLevelStatus>& LevelStatuses);

	/** Notify client they were kicked from the server */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientWasKicked(const FText& KickReason);

	/** Notify client that the session is starting */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientStartOnlineSession();

	/** Notify client that the session is about to start */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientEndOnlineSession();

	/** Assign Pawn to player, but avoid calling ClientRestart if we have already accepted this pawn */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientRetryClientRestart(class APawn* NewPawn);

	/** Call ClientRetryClientRestart, but only if the current pawn is not the currently acknowledged pawn (and throttled to avoid saturating the network). */
	ENGINE_API virtual void SafeRetryClientRestart();

	/** send client localized message id */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientReceiveLocalizedMessage(TSubclassOf<ULocalMessage> Message, int32 Switch = 0, class APlayerState* RelatedPlayerState_1 = nullptr, class APlayerState* RelatedPlayerState_2 = nullptr, class UObject* OptionalObject = nullptr);

	/** acknowledge possession of pawn */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerAcknowledgePossession(class APawn* P);

	/** change mode of camera */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerCamera(FName NewMode);

	/** Change name of server */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerChangeName(const FString& S);

	/** 
	 * Called to notify the server when the client has loaded a new world via seamless traveling
	 * @param WorldPackageName the name of the world package that was loaded
	 */
	UFUNCTION(reliable, server, WithValidation, SealedEvent)
	ENGINE_API void ServerNotifyLoadedWorld(FName WorldPackageName);

	/** Replicate pause request to the server */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerPause();

	/** Attempts to restart this player, generally called from the client upon respawn request. */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerRestartPlayer();

	/** When spectating, updates spectator location/rotation and pings the server to make sure spectating should continue. */
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerSetSpectatorLocation(FVector NewLoc, FRotator NewRot);

	/** Calls ServerSetSpectatorLocation but throttles it to reduce bandwidth and only calls it when necessary. */
	ENGINE_API void SafeServerUpdateSpectatorState();

	/** Tells the server to make sure the possessed pawn is in sync with the client. */
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerCheckClientPossession();

	/** Reliable version of ServerCheckClientPossession to be used when there is no likely danger of spamming the network. */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerCheckClientPossessionReliable();

	/** Call ServerCheckClientPossession on the server, but only if the current pawn is not the acknowledged pawn (and throttled to avoid saturating the network). */
	ENGINE_API virtual void SafeServerCheckClientPossession();

	/** Notifies the server that the client has ticked gameplay code, and should no longer get the extended "still loading" timeout grace period */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerShortTimeout();

	/** If PlayerCamera.bUseClientSideCameraUpdates is set, client will replicate camera positions to the server. */
	// @TODO - combine pitch/yaw into one int, maybe also send location compressed
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerUpdateCamera(FVector_NetQuantize CamLoc, int32 CamPitchAndYaw);

	/** 
	 * Called when the client adds/removes a streamed level.
	 * The server will only replicate references to Actors in visible levels so that it's impossible to send references to
	 * Actors the client has not initialized.
	 *
	 * @param LevelVisibility	Visibility state for the level whose state changed.
	 */
	UFUNCTION(reliable, server, WithValidation, SealedEvent)
	ENGINE_API void ServerUpdateLevelVisibility(const FUpdateLevelVisibilityLevelInfo& LevelVisibility);

	/** 
	 * Called when the client adds/removes a streamed level.  This version of the function allows you to pass the state of 
	 * multiple levels at once, to reduce the number of RPC events that will be sent.
	 *
	 * @param	LevelVisibilities	Visibility state for each level whose state has changed
	 */
	UFUNCTION(reliable, server, WithValidation, SealedEvent)
	ENGINE_API void ServerUpdateMultipleLevelsVisibility( const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities );

	UFUNCTION(Reliable, Server, SealedEvent)
	ENGINE_API void ServerUpdateContainerVisibility(UObject* Container, bool bVisible);

	/** Used by client to request server to confirm current viewtarget (server will respond with ClientSetViewTarget() ). */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerVerifyViewTarget();

	/** Move camera to next player on round ended or spectating*/
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerViewNextPlayer();

	/** Move camera to previous player on round ended or spectating */
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerViewPrevPlayer();

	/** Move camera to current user */
	UFUNCTION(unreliable, server, WithValidation)
	ENGINE_API void ServerViewSelf(struct FViewTargetTransitionParams TransitionParams = FViewTargetTransitionParams());

	/** @todo document */
	UFUNCTION(Reliable, Client)
	ENGINE_API void ClientTeamMessage(class APlayerState* SenderPlayerState, const FString& S, FName Type, float MsgLifeTime = 0);

	/** Used by UGameplayDebuggingControllerComponent to replicate messages for AI debugging in network games. */
	UFUNCTION(reliable, server, WithValidation)
	ENGINE_API void ServerToggleAILogging();

	/**
	 * Add Pitch (look up) input. This value is multiplied by InputPitchScale.
	 * @param Val Amount to add to Pitch. This value is multiplied by InputPitchScale.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(Keywords="up down"))
	ENGINE_API virtual void AddPitchInput(float Val);

	/**
	 * Add Yaw (turn) input. This value is multiplied by InputYawScale.
	 * @param Val Amount to add to Yaw. This value is multiplied by InputYawScale.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(Keywords="left right turn"))
	ENGINE_API virtual void AddYawInput(float Val);

	/**
	 * Add Roll input. This value is multiplied by InputRollScale.
	 * @param Val Amount to add to Roll. This value is multiplied by InputRollScale.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API virtual void AddRollInput(float Val);

	/** Returns true if the given key/button is pressed on the input of the controller (if present) */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API bool IsInputKeyDown(FKey Key) const;

	/** Returns true if the given key/button was up last frame and down this frame. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API bool WasInputKeyJustPressed(FKey Key) const;

	/** Returns true if the given key/button was down last frame and up this frame. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API bool WasInputKeyJustReleased(FKey Key) const;

	/** Returns the analog value for the given key/button.  If analog isn't supported, returns 1 for down and 0 for up. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API float GetInputAnalogKeyState(FKey Key) const;

	/** Returns the vector value for the given key/button. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API FVector GetInputVectorKeyState(FKey Key) const;

	/** Retrieves the X and Y screen coordinates of the specified touch key. Returns false if the touch index is not down */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API void GetInputTouchState(ETouchIndex::Type FingerIndex, float& LocationX, float& LocationY, bool& bIsCurrentlyPressed) const;
	ENGINE_API void GetInputTouchState(ETouchIndex::Type FingerIndex, double& LocationX, double& LocationY, bool& bIsCurrentlyPressed) const;	// LWC_TODO: Temp stand in for native calls with FVector2D components.

	/** Retrieves the current motion state of the player's input device */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API void GetInputMotionState(FVector& Tilt, FVector& RotationRate, FVector& Gravity, FVector& Acceleration) const;

	/** Retrieves the X and Y screen coordinates of the mouse cursor. Returns false if there is no associated mouse device */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API bool GetMousePosition(float& LocationX, float& LocationY) const;
	ENGINE_API bool GetMousePosition(double& LocationX, double& LocationY) const;	// LWC_TODO: Temp stand in for native calls with FVector2D components.

	/** Returns how long the given key/button has been down.  Returns 0 if it's up or it just went down this frame. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API float GetInputKeyTimeDown(FKey Key) const;

	/** Retrieves how far the mouse moved this frame. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API void GetInputMouseDelta(float& DeltaX, float& DeltaY) const;
	ENGINE_API void GetInputMouseDelta(double& DeltaX, double& DeltaY) const;	// LWC_TODO: Temp stand in for native calls with FVector2D components.
	
	/** Retrieves the X and Y displacement of the given analog stick. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API void GetInputAnalogStickState(EControllerAnalogStick::Type WhichStick, float& StickX, float& StickY) const;
	ENGINE_API void GetInputAnalogStickState(EControllerAnalogStick::Type WhichStick, double& StickX, double& StickY) const;	// LWC_TODO: Temp stand in for native calls with FVector2D components.

	/** Activates a new touch interface for this player controller */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API virtual void ActivateTouchInterface(class UTouchInterface* NewTouchInterface);

	/** Set the virtual joystick visibility. */
	UFUNCTION(BlueprintCallable, Category="Game|Player")
	ENGINE_API virtual void SetVirtualJoystickVisibility(bool bVisible);

	/** Setup an input mode. */
	ENGINE_API virtual void SetInputMode(const FInputModeDataBase& InData);

	/**
	 * Change Camera mode
	 * @param	New camera mode to set
	 */
	UFUNCTION(exec)
	ENGINE_API virtual void Camera(FName NewMode);

	/**
	 * Set the view target blending with variable control
	 * @param NewViewTarget - new actor to set as view target
	 * @param BlendTime - time taken to blend
	 * @param BlendFunc - Cubic, Linear etc functions for blending
	 * @param BlendExp -  Exponent, used by certain blend functions to control the shape of the curve. 
	 * @param bLockOutgoing - If true, lock outgoing viewtarget to last frame's camera position for the remainder of the blend.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Player", meta=(Keywords = "Camera"))
	ENGINE_API virtual void SetViewTargetWithBlend(class AActor* NewViewTarget, float BlendTime = 0, enum EViewTargetBlendFunction BlendFunc = VTBlend_Linear, float BlendExp = 0, bool bLockOutgoing = false);

	/** 
	* Make this player a member of a netcondition group. 
	* Any subobject registered in the group may now be replicated to this player's connection.
	*/
	ENGINE_API void IncludeInNetConditionGroup(FName NetGroup);

	/** Remove this player from a netcondition group. */
	ENGINE_API void RemoveFromNetConditionGroup(FName NetGroup);

	/** Returns true if the player controller is a member of the netcondition group */
	bool IsMemberOfNetConditionGroup(FName NetGroup) const
	{
		return NetConditionGroups.Find(NetGroup) != INDEX_NONE;
	}

	/** Returns the list of netcondition groups we are part of. */
	const TArray<FName>& GetNetConditionGroups() const { return NetConditionGroups; }

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// The current input component stack which is built every frame in APlayerController::BuildInputStack.
	// This is useful for building any external debug tools where you have access to the player controller
	// It is not recommended to access this outside of debug tooling because this is re-built every frame.
	TConstArrayView<TWeakObjectPtr<const UInputComponent>> Debug_GetMostRecentInputStack() const { return { LastUsedInputStack }; }
#endif
	
private:
	
	/** List of netcondition groups we are currently a member of. */
	TArray<FName> NetConditionGroups;

protected:
	/** Clickable object currently under the mouse cursor. */
	TWeakObjectPtr<UPrimitiveComponent> CurrentClickablePrimitive;

	/** Touchable objects currently under fingers. */
	TWeakObjectPtr<UPrimitiveComponent> CurrentTouchablePrimitives[EKeys::NUM_TOUCH_KEYS];

	/** Internal.  Current stack of InputComponents. */
	TArray< TWeakObjectPtr<UInputComponent> > CurrentInputStack;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Debug tracking of the most recent *complete* input stack.
	TArray<TWeakObjectPtr<const UInputComponent>> LastUsedInputStack;
#endif
	
	/** InputComponent we use when player is in Inactive state. */
	UPROPERTY()
	TObjectPtr<UInputComponent> InactiveStateInputComponent;

	/** Sets up input bindings for the input component pushed on the stack in the inactive state. */
	ENGINE_API virtual void SetupInactiveStateInputComponent(UInputComponent* InComponent);

	/** Refresh state specific input components */
	ENGINE_API virtual void UpdateStateInputComponents();

	/** The state of the inputs from cinematic mode */
	uint32 bCinemaDisableInputMove:1;
	uint32 bCinemaDisableInputLook:1;

	/** Whether we fully tick when the game is paused, if our tick function is allowed to do so. If false, we do a minimal update during the tick. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PlayerController)
	uint32 bShouldPerformFullTickWhenPaused : 1;

private:
	/* Whether the PlayerController's input handling is enabled. */
	uint32 bInputEnabled:1;

protected:

	/** The virtual touch interface */
	TSharedPtr<class SVirtualJoystick> VirtualJoystick;

	/** Create virtual touch interface */
	ENGINE_API virtual TSharedPtr<class SVirtualJoystick> CreateVirtualJoystick();

	/** The currently set touch interface */
	UPROPERTY()
	TObjectPtr<class UTouchInterface> CurrentTouchInterface;

	/** If set, then this UPlayerInput class will be used instead of the Input Settings' DefaultPlayerInputClass */
	UPROPERTY(EditDefaultsOnly, Category = Input)
	TSubclassOf<UPlayerInput> OverridePlayerInputClass;

	/** Handle for efficient management of UnFreeze timer */
	FTimerHandle TimerHandle_UnFreeze;

private:
	/** Handle for efficient management of DelayedPrepareMapChange timer */
	FTimerHandle TimerHandle_DelayedPrepareMapChange;

	/** Handle for efficient management of ClientCommitMapChange timer */
	FTimerHandle TimerHandle_ClientCommitMapChange;

public:
	/** Adds an inputcomponent to the top of the input stack. */
	ENGINE_API virtual void PushInputComponent(UInputComponent* Input);

	/** Removes given inputcomponent from the input stack (regardless of if it's the top, actually). */
	ENGINE_API virtual bool PopInputComponent(UInputComponent* Input);

	/** Returns true if the given input component is in this PlayerController's CurrentInputStack */
	ENGINE_API virtual bool IsInputComponentInStack(const UInputComponent* Input) const;

	/** Flushes the current key state. */
	ENGINE_API virtual void FlushPressedKeys();

	/**
	 * If true, then the GameViewportClient should call FlushPressedKeys on this controller when it loses focus.
	 * The default behavior here is to return true if the PlayerController is in any input mode other than GameAndUI
	 */
	virtual bool ShouldFlushKeysWhenViewportFocusChanges() const { return bShouldFlushInputWhenViewportFocusChanges; }

	UFUNCTION(BlueprintCallable, Category = Input)
	ENGINE_API TSubclassOf<UPlayerInput> GetOverridePlayerInputClass() const;

	/** Handles a key press */
	UE_DEPRECATED(5.6, "Use the version which takes a FInputKeyEventArgs instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ENGINE_API virtual bool InputKey(const FInputKeyParams& Params) final;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Handles a key press */
	ENGINE_API virtual bool InputKey(const FInputKeyEventArgs& Params);

	/** Handles a touch screen action */
	UE_DEPRECATED(5.8, "Use the overload that uses FTouchId instead")
	ENGINE_API virtual bool InputTouch(
		const FInputDeviceId DeviceId,
		const uint32 Handle,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint32 TouchpadIndex,
		const uint64 Timestamp) final;

	/**
	 * Allow the player-controller to process a touch event received by the viewport-client.
	 * If the player-controller uses the event, it should return true.  If true is returned, the viewport-client will not offer it to the next
	 * candidate.
	 * 
	 * @param	TouchId - Identifier unique to this ongoing touch (i.e. device + finger).
	 * @param	Type - What kind of touch event this is (see ETouchType).
	 * @param	TouchLocation - Viewport-relative position of the touch.
	 * @param	Force - How hard the touch is.
	 * @param	Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the event, false to pass it on.
	 */
	ENGINE_API virtual bool InputTouch(
		const FTouchId TouchId,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint64 Timestamp);

	/** Handles motion control */
	UE_DEPRECATED(5.6, "Use the version which takes in a DeviceId and Timestamp instead.")
	ENGINE_API virtual bool InputMotion(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration) final;

	/** Handles motion control */
	ENGINE_API virtual bool InputMotion(const FInputDeviceId DeviceId, const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, const uint64 Timestamp);

	/** Handles a gesture input event, maps EGestureEvent to FKey and forwards to PlayerInput */
	ENGINE_API virtual bool InputGesture(const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp);

	/** Associate a new UPlayer with this PlayerController. */
	ENGINE_API virtual void SetPlayer(UPlayer* InPlayer);

	/** Returns the ULocalPlayer for this controller if it exists, or null otherwise */
	ENGINE_API class ULocalPlayer* GetLocalPlayer() const;

	/**
	 * Returns the platform user that is assigned to this Player Controller's Local Player.
	 * If there is no local player, then this will return PLATFORMUSERID_NONE
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Game|Player")
	ENGINE_API FPlatformUserId GetPlatformUserId() const;

	/**
	 * Called client-side to smoothly interpolate received TargetViewRotation (result is in BlendedTargetViewRotation)
	 * @param TargetPawn   is the pawn which is the current ViewTarget
	 * @param DeltaSeconds is the time interval since the last smoothing update
	 */
	ENGINE_API virtual void SmoothTargetViewRotation(APawn* TargetPawn, float DeltaSeconds);

	/**
	 * Executes the Exec() command on the UPlayer object
	 *
	 * @param Command command to execute (string of commands optionally separated by a | (pipe))
	 * @param bWriteToLog write out to the log
	 */
	ENGINE_API virtual FString ConsoleCommand(const FString& Command, bool bWriteToLog = true);

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const FUObjectMigrationContext& MigrationContext) override;
#endif
	//~ End UObject Interface

	//~ Begin AActor Interface
	ENGINE_API virtual void GetActorEyesViewPoint(FVector& Location, FRotator& Rotation) const override;
	ENGINE_API virtual void CalcCamera(float DeltaTime, struct FMinimalViewInfo& OutResult) override;
	ENGINE_API virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
	ENGINE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	ENGINE_API virtual void FellOutOfWorld(const class UDamageType& dmgType) override;
	ENGINE_API virtual void Reset() override;
protected:
	ENGINE_API virtual void OnPossess(APawn* aPawn) override;
	ENGINE_API virtual void OnUnPossess() override;
public:
	ENGINE_API virtual void CleanupPlayerState() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void OnActorChannelOpen(class FInBunch& InBunch, class UNetConnection* Connection) override;
	ENGINE_API virtual bool UseShortConnectTimeout() const override;
	ENGINE_API virtual void OnSerializeNewActor(class FOutBunch& OutBunch) override;
	ENGINE_API virtual void OnNetCleanup(class UNetConnection* Connection) override;
	ENGINE_API virtual float GetNetPriority(const FVector& ViewPos, const FVector& ViewDir, AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth) override;
	ENGINE_API virtual const AActor* GetNetOwner() const override;
	ENGINE_API virtual class UPlayer* GetNetOwningPlayer() override;
	ENGINE_API virtual class UPlayer* GetNetOwningPlayerAnyRole() override;
	ENGINE_API virtual class UNetConnection* GetNetConnection() const override;
	ENGINE_API virtual bool DestroyNetworkActorHandled() override;
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual void EnableInput(class APlayerController* PlayerController) override;
	ENGINE_API virtual void DisableInput(class APlayerController* PlayerController) override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

protected:
	ENGINE_API virtual void BeginPlay() override;
	//~ End AActor Interface

public:
	//~ Begin AController Interface
	ENGINE_API virtual void GameHasEnded(class AActor* EndGameFocus = nullptr, bool bIsWinner = false) override;
	ENGINE_API virtual bool IsLocalController() const override;
	ENGINE_API virtual void GetPlayerViewPoint(FVector& out_Location, FRotator& out_Rotation) const override;
	ENGINE_API virtual void SetInitialLocationAndRotation(const FVector& NewLocation, const FRotator& NewRotation) override;
	ENGINE_API virtual void ChangeState(FName NewState) override;
	ENGINE_API virtual class AActor* GetViewTarget() const override;
	ENGINE_API virtual void BeginInactiveState() override;
	ENGINE_API virtual void EndInactiveState() override;
	ENGINE_API virtual void FailedToSpawnPawn() override;
	ENGINE_API virtual void SetPawn(APawn* InPawn) override;
	//~ End AController Interface

	/**
	 * Called on the server when the client sends a message indicating it was unable to initialize an Actor channel,
	 * most commonly because the desired Actor's archetype couldn't be serialized
	 * the default is to do nothing (Actor simply won't exist on the client), but this function gives the game code
	 * an opportunity to try to correct the problem
	 */
	virtual void NotifyActorChannelFailure(UActorChannel* ActorChan) {}

	/**
	 * Builds a list of actors that are hidden based upon gameplay
	 * @param ViewLocation the view point to hide/unhide from
	 */
	virtual void UpdateHiddenActors(const FVector& ViewLocation) {}

	/**
	 * Builds a list of components that are hidden based upon gameplay
	 * @param ViewLocation the view point to hide/unhide from
	 * @param HiddenComponents the list to add to/remove from
	 */
	virtual void UpdateHiddenComponents(const FVector& ViewLocation, TSet<FPrimitiveComponentId>& /*HiddenComponents*/) {}

	/**
	 * Builds a list of components that are hidden based upon gameplay.
	 * This calls both UpdateHiddenActors and UpdateHiddenComponents, merging the two lists.
	 * @param ViewLocation the view point to hide/unhide from
	 * @param HiddenComponents this list will have all components that should be hidden added to it
	 */
	ENGINE_API void BuildHiddenComponentList(const FVector& ViewLocation, TSet<FPrimitiveComponentId>& HiddenComponentsOut);

	/** spawn cameras for servers and owning players */
	ENGINE_API virtual void SpawnPlayerCameraManager();

	/** get audio listener position and orientation */
	ENGINE_API virtual void GetAudioListenerPosition(FVector& OutLocation, FVector& OutFrontDir, FVector& OutRightDir) const;

	/** Gets the attenuation position override. */
	ENGINE_API virtual bool GetAudioListenerAttenuationOverridePosition(FVector& OutLocation) const;

	/**
	 * Used to override the default positioning of the audio listener
	 * 
	 * @param AttachToComponent Optional component to attach the audio listener to
	 * @param Location Depending on whether Component is attached this is either an offset from its location or an absolute position
	 * @param Rotation Depending on whether Component is attached this is either an offset from its rotation or an absolute rotation
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Audio")
	ENGINE_API void SetAudioListenerOverride(USceneComponent* AttachToComponent, FVector Location, FRotator Rotation);

	/**
	 * Clear any overrides that have been applied to audio listener
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Audio")
	ENGINE_API void ClearAudioListenerOverride();

	UFUNCTION(BlueprintCallable, Category = "Game|Audio")
	ENGINE_API void SetAudioListenerAttenuationOverride(USceneComponent* AttachToComponent, FVector AttenuationLocationOVerride);

	UFUNCTION(BlueprintCallable, Category = "Game|Audio")
	ENGINE_API void ClearAudioListenerAttenuationOverride();


protected:
	/** Whether to override the normal audio listener positioning method */
	uint32 bOverrideAudioListener:1;
	/** Whether to override the attenuation listener position. */
	uint32 bOverrideAudioAttenuationListener:1;
	/** Component that is currently driving the audio listener position/orientation */
	TWeakObjectPtr<USceneComponent> AudioListenerComponent;
	/** Component that is used to only override where attenuation calculations are computed from. */
	TWeakObjectPtr<USceneComponent> AudioListenerAttenuationComponent;
	/** Currently overridden location of audio listener */
	FVector AudioListenerLocationOverride;
	/** Currently overridden rotation of audio listener */
	FRotator AudioListenerRotationOverride;
	/** Currently overridden vector used to do attenuation calculations for listener. */
	FVector AudioListenerAttenuationOverride;

	/** Internal. */
	ENGINE_API void TickPlayerInput(const float DeltaSeconds, const bool bGamePaused);
	ENGINE_API virtual void ProcessPlayerInput(const float DeltaTime, const bool bGamePaused);
	ENGINE_API virtual void BuildInputStack(TArray<UInputComponent*>& InputStack);
	ENGINE_API void ProcessForceFeedbackAndHaptics(const float DeltaTime, const bool bGamePaused);
	ENGINE_API virtual void UpdateForceFeedback(IInputInterface* InputInterface, const int32 ControllerId);
	ENGINE_API virtual bool IsInViewportClient(UGameViewportClient* ViewportClient) const;
	ENGINE_API virtual int32 GetInputIndex() const;
	ENGINE_API virtual ACameraActor* GetAutoActivateCameraForPlayer() const;

	UE_DEPRECATED(5.8, "Use the overload with FTouchId instead")
	ENGINE_API virtual void ProcessTouchHitResult(const FInputDeviceId DeviceId, uint32 Handle, ETouchType::Type Type, const FVector2D& TouchLocation, const FHitResult& HitResult, bool bHit) final;

	ENGINE_API virtual void ProcessTouchHitResult(const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const FHitResult& HitResult, bool bHit);

	/** Allows the PlayerController to set up custom input bindings. */
	ENGINE_API virtual void SetupInputComponent();

public:
	/**
	 * Store the net speed 
	 * @param NewSpeed current speed of network
	 */
	ENGINE_API void SetNetSpeed(int32 NewSpeed);

	/** 
	 * Get the local players network address
	 * @return the address
	 */
	ENGINE_API FString GetPlayerNetworkAddress();

	/** 
	 * Get the server network address
	 * @return the address
	 */
	ENGINE_API FString GetServerNetworkAddress();

	/** Handles remapping a package name for networking, call on both the client and server when sending package names manually for RPCs */
	ENGINE_API FName NetworkRemapPath(FName InPackageName, bool bReading);

	/** Clears out 'left-over' audio components. */
	ENGINE_API virtual void CleanUpAudioComponents();

	/** Called to try and enable cheats for this player, happens during initialization or from AllowCheats command */
	ENGINE_API virtual void AddCheats(bool bForce = false);

	/** Spawn a HUD (make sure that PlayerController always has valid HUD, even if ClientSetHUD() hasn't been called */
	ENGINE_API virtual void SpawnDefaultHUD();

	/** Create the touch interface, and activate an initial touch interface (if touch interface is desired) */
	ENGINE_API virtual void CreateTouchInterface();

	/** Gives the PlayerController an opportunity to cleanup any changes it applied to the game viewport, primarily for the touch interface */
	ENGINE_API virtual void CleanupGameViewport();

	/** Called on the client to do local pawn setup after possession, before calling ServerAcknowledgePossession */
	ENGINE_API virtual void AcknowledgePossession(class APawn* P);

	/** Clean up when a Pawn's player is leaving a game. Base implementation destroys the pawn. */
	ENGINE_API virtual void PawnLeavingGame();

	/** Takes ping updates from the net driver (both clientside and serverside), and passes them on to PlayerState::UpdatePing */
	ENGINE_API virtual void UpdatePing(float InPing);

	/**
	 * Get next active viewable player in PlayerArray.
	 * @param dir is the direction to go in the array
	 */
	ENGINE_API virtual class APlayerState* GetNextViewablePlayer(int32 dir);

	/**
	 * View next active player in PlayerArray.
	 * @param dir is the direction to go in the array
	 */
	ENGINE_API virtual void ViewAPlayer(int32 dir);

	/** Returns true if this controller thinks it's able to restart. Called from GameModeBase::PlayerCanRestart */
	UFUNCTION(BlueprintCallable, Category = "Game|Player")
	ENGINE_API virtual bool CanRestartPlayer();

	/**
	 * Server/SP only function for changing whether the player is in cinematic mode.  Updates values of various state variables, then replicates the call to the client
	 * to sync the current cinematic mode.
	 * @param	bInCinematicMode	specify true if the player is entering cinematic mode; false if the player is leaving cinematic mode.
	 * @param	bHidePlayer			specify true to hide the player's pawn (only relevant if bInCinematicMode is true)
	 * @param	bAffectsHUD			specify true if we should show/hide the HUD to match the value of bCinematicMode
	 * @param	bAffectsMovement	specify true to disable movement in cinematic mode, enable it when leaving
	 * @param	bAffectsTurning		specify true to disable turning in cinematic mode or enable it when leaving
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Cinematic", meta=(bHidePlayer="true", bAffectsHUD="true"))
	ENGINE_API virtual void SetCinematicMode(bool bInCinematicMode, bool bHidePlayer, bool bAffectsHUD, bool bAffectsMovement, bool bAffectsTurning);

	/**
	 * Determines whether this player is playing split-screen.
	 * @param	OutSplitscreenPlayerIndex	receives the index [into the player's local GamePlayers array] for this player, if playing splitscreen.
	 * @return	true if this player is playing splitscreen.
	 */
	ENGINE_API bool IsSplitscreenPlayer(int32* OutSplitscreenPlayerIndex = nullptr) const;

	/**
	 * Wrapper for determining whether this player is the first player on their console.
	 * @return	true if this player is not using splitscreen, or is the first player in the split-screen layout.
	 */
	UFUNCTION(BlueprintPure, Category = "Game|Player")
	ENGINE_API bool IsPrimaryPlayer() const;

	/**
	 * Returns the PlayerState associated with the player at the specified index.
	 * @param	PlayerIndex		the index [into the local player's GamePlayers array] for the player PlayerState to find
	 * @return	the PlayerState associated with the player at the specified index, or None if the player is not a split-screen player or
	 *			the index was out of range.
	 */
	ENGINE_API class APlayerState* GetSplitscreenPlayerByIndex(int32 PlayerIndex = 1) const;

	/**
	 * Returns the number of split-screen players playing on this player's machine.
	 * @return	the total number of players on the player's local machine, or 0 if this player isn't playing split-screen.
	 */
	ENGINE_API int32 GetSplitscreenPlayerCount() const;

	/** Update the camera manager; this is called after all actors have been ticked.	 */ 
	ENGINE_API virtual void UpdateCameraManager(float DeltaSeconds);

	/**
	 * This function will be called to notify the player controller that the world has received its game class. In the case of a client
	 * we need to initialize the Input System here.
	 *
	 * @Param GameModeClass - The Class of the game that was replicated
	 */
	ENGINE_API virtual void ReceivedGameModeClass(TSubclassOf<class AGameModeBase> GameModeClass);

	/** Notify the server that client data was received on the Pawn.
	 * @return true if InPawn is acknowledged on the server, false otherwise. */
	ENGINE_API virtual bool NotifyServerReceivedClientData(APawn* InPawn, float TimeStamp);

	/** Start spectating mode, as the only mode allowed. */
	ENGINE_API virtual void StartSpectatingOnly();

	/**
	 * Default implementation of pausing check for 'CanUnpause' delegates
	 * @return True if pausing is allowed
	 */
	ENGINE_API virtual bool DefaultCanUnpause();

	/** Returns true if game is currently paused. */
	ENGINE_API bool IsPaused() const;

	bool InputEnabled() const { return bInputEnabled; }

	/** Returns true if we fully tick when paused (and if our tick function is enabled when paused).	 */
	ENGINE_API bool ShouldPerformFullTickWhenPaused() const;

	/** returns whether the client has completely loaded the server's current world (valid on server only) */
	ENGINE_API bool HasClientLoadedCurrentWorld();

	/** Set the view target
	 * @param A - new actor to set as view target
	 * @param TransitionParams - parameters to use for controlling the transition
	 */
	ENGINE_API virtual void SetViewTarget(class AActor* NewViewTarget, FViewTargetTransitionParams TransitionParams = FViewTargetTransitionParams());

	/**
	 * If bAutoManageActiveCameraTarget is true, then automatically manage the active camera target.
	 * If there a CameraActor placed in the level with an auto-activate player assigned to it, that will be preferred, otherwise SuggestedTarget will be used.
	 */
	ENGINE_API virtual void AutoManageActiveCameraTarget(AActor* SuggestedTarget);

	/**
	 * Notify from server that Visual Logger is recording, to show that information on client about possible performance issues 
	 */
	UFUNCTION(Reliable, Client)
	ENGINE_API void OnServerStartedVisualLogger(bool bIsLogging);

	ENGINE_API void SetShowMouseCursor(bool bShow);

	/** Returns true if the mouse cursor should be shown */
	ENGINE_API virtual bool ShouldShowMouseCursor() const;

	/** Returns the current mouse cursor, or None */
	ENGINE_API virtual EMouseCursor::Type GetMouseCursor() const;

	// Spectating

	/** Get the Pawn used when spectating. nullptr when not spectating. */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ASpectatorPawn* GetSpectatorPawn() const { return SpectatorPawn; }

	/** Returns the first of GetPawn() or GetSpectatorPawn() that is not nullptr, or nullptr otherwise. */
	ENGINE_API APawn* GetPawnOrSpectator() const;

	/** Called to notify the controller that the spectator class has been received. */
	ENGINE_API virtual void ReceivedSpectatorClass(TSubclassOf<ASpectatorPawn> SpectatorClass);

	/**
	 * Returns the location the PlayerController is focused on.
	 *  If there is a possessed Pawn, returns the Pawn's location.
	 *  If there is a spectator Pawn, returns that Pawn's location.
	 *  Otherwise, returns the PlayerController's spawn location (usually the last known Pawn location after it has died).
	 */
	UFUNCTION(BlueprintCallable, Category=Pawn)
	ENGINE_API virtual FVector GetFocalLocation() const;

protected:
	/** Event when spectating begins. */
	ENGINE_API virtual void BeginSpectatingState();

	/** Event when no longer spectating. */
	ENGINE_API virtual void EndSpectatingState();

	/** Set the spectator pawn. Will also call AttachToPawn() using the new spectator. */
	ENGINE_API virtual void SetSpectatorPawn(ASpectatorPawn* NewSpectatorPawn);

	/** Spawn a SpectatorPawn to use as a spectator and initialize it. By default it is spawned at the PC's current location and rotation. */
	ENGINE_API virtual ASpectatorPawn* SpawnSpectatorPawn();

	/** Destroys the SpectatorPawn and sets it to nullptr. */
	ENGINE_API virtual void DestroySpectatorPawn();

	/** Useful to spectate other pawn without un-possessing the current pawn */
	virtual bool ShouldKeepCurrentPawnUponSpectating() const { return false; }

private:
	/** The pawn used when spectating (nullptr if not spectating). */
	UPROPERTY()
	TObjectPtr<ASpectatorPawn> SpectatorPawn;

	/** Used to delay calling ClientRestart() again when it hasn't been appropriately acknowledged. */
	float		LastRetryPlayerTime;

	/** Set during SpawnActor once and never again to indicate the intent of this controller instance (SERVER ONLY) */
	UPROPERTY()
	mutable bool bIsLocalPlayerController;

protected:
	/** The location used internally when there is no pawn or spectator, to know where to spawn the spectator or focus the camera on death. */
	UPROPERTY(Replicated)
	FVector SpawnLocation;

	/** Set the SpawnLocation for use when changing states or when there is no pawn or spectator. */
	ENGINE_API virtual void SetSpawnLocation(const FVector& NewLocation);

	/** Last real time (undilated) recorded in TickActor() when checking for forced client movement updates. */
	float LastMovementUpdateTime;

	/** Last real time (undilated) a hitch was detected in TickActor() when checking for forced client movement updates. */
	float LastMovementHitch;

public:
	/** Get the location used when initially created, or when changing states when there is no pawn or spectator. */
	FVector GetSpawnLocation() const { return SpawnLocation; }

	/** Called after this PlayerController's viewport/net connection is associated with this player controller. */
	ENGINE_API virtual void ReceivedPlayer();

	/** 
	 * Spawn the appropriate class of PlayerInput.
	 * Only called for playercontrollers that belong to local players.
	 */
	ENGINE_API virtual void InitInputSystem();

	/** Returns true if input should be frozen (whether UnFreeze timer is active) */
	ENGINE_API virtual bool IsFrozen();

	/**
	 * Called when the local player is about to travel to a new map or IP address.  Provides subclass with an opportunity
	 * to perform cleanup or other tasks prior to the travel.
	 */
	ENGINE_API virtual void PreClientTravel(const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel);

	/** Set new camera mode */
	ENGINE_API virtual void SetCameraMode(FName NewCamMode);

	/** Reset Camera Mode to default. */
	ENGINE_API virtual void ResetCameraMode();

	/**
	 * Called on server at end of tick, to let client Pawns handle updates from the server.
	 * Done this way to avoid ever sending more than one ClientAdjustment per server tick.
	 */
	ENGINE_API virtual void SendClientAdjustment();

	/**
	 * Designate this player controller as local (public for GameModeBase to use, not expected to be called anywhere else)
	 */
	void SetAsLocalPlayerController() { bIsLocalPlayerController = true; }

	/**
	 * Whether this controller should persist through seamless travel
	 * Player controllers should always be included in seamless travel
	 */
	virtual bool ShouldParticipateInSeamlessTravel() const override { return true; }
	
#if UE_ENABLE_DEBUG_DRAWING
	/** Keep track of the current input mode debug string here. Set in APlayerController::SetInputMode */
	ENGINE_API const FString& GetCurrentInputModeDebugString() const;
#endif

private:
	/** If true, prevent any haptic effects from playing */
	bool bDisableHaptics : 1;

	/**
	 * If true, then the GameViewportCliet will call FlushPressedKeys when it loses focus ( UGameViewportClient::LostFocus ).
	 * By default this is true for all Input Modes except for GameAndUI. When you are using the "GameAndUI" input mode 
	 * then you don't normally want the input to be flushed because it would reset the player controller's inputs if you bring
	 * up any in-game UI.
	 */
	bool bShouldFlushInputWhenViewportFocusChanges : 1;

#if UE_ENABLE_DEBUG_DRAWING
	/** Keep track of the current input mode debug string here. Set in APlayerController::SetInputMode */
	FString CurrentInputModeDebugString;
#endif

	/** The static offset between the local async physics tick frame number and the server's, kept in sync via time-dilation
	* This is used to synchronize events that happen in the async physics tick 
	* This is 0 on the server */
	int32 NetworkPhysicsTickOffset = 0;
	bool bNetworkPhysicsTickOffsetAssigned = false;

	/** Tick Offset sync interval timer */
	float TickOffsetSyncCountdown = 0.f;

	/** Time to add to the tick offset, works as a buffer on the server to handle fluctuating network conditions */
	float TickOffsetBufferTime;

	/** Time that tick offset has been desynced for since it was last in sync, used to trigger a correction */
	float NetworkPhysicsTickOffsetDesyncAccumulatedTime = 0.0f;

public:

	/** Update the tick offset in between the local client and the server */
	ENGINE_API void UpdateServerAsyncPhysicsTickOffset();

	/** Server receives the clients FAsyncPhysicsTimestamp with its predicted ServerFrame and clients LocalFrame, to updates the time dilation needed to keep the ServerFrame prediction in sync */
	UFUNCTION(Server, Unreliable)
	ENGINE_API void ServerSendLatestAsyncPhysicsTimestamp(FAsyncPhysicsTimestamp Timestamp);

	/** Client receives the setup of the FAsyncPhysicsTimestamp ServerFrame and LocalFrame offset from the server. */
	UFUNCTION(Client, Reliable)
	ENGINE_API void ClientSetupNetworkPhysicsTimestamp(FAsyncPhysicsTimestamp Timestamp);

	/** Client receives the time dilation value it needs to use to keep its ServerFrame to LocalFrame offset in sync */
	UFUNCTION(Client, Unreliable)
	ENGINE_API void ClientAckTimeDilation(float TimeDilation, int32 ServerStep);

	/** Enqueues a command to run at the time specified by AsyncPhysicsTimestamp. Note that if the time specified was missed the command is triggered as soon as possible as part of the async tick.
	These commands are all run on the game thread. If you want to run on the physics thread see FPhysicsSolverBase::RegisterSimOneShotCallback
	If OwningObject is not null this command will only fire as long as the object is still alive. This allows a lambda to still use the owning object's data for read/write (assuming data is designed for async physics tick)
	*/
	ENGINE_API void ExecuteAsyncPhysicsCommand(const FAsyncPhysicsTimestamp& AsyncPhysicsTimestamp, UObject* OwningObject, const TFunction<void()>& Command, const bool bEnableResim = true);

	/** Generates a timestamp for the upcoming physics step. Useful for synchronizing client and server events on a specific physics step 
	* On the server LocalFrame and ServerFrame are the same, on the client LocalFrame is the upcoming physics frame and ServerFrame is the server-side frame that the LocalFrame corresponds to.
	*/
	ENGINE_API FAsyncPhysicsTimestamp GetPhysicsTimestamp();

	UE_DEPRECATED(5.7, "Deprecated, use the GetPhysicsTimestamp() without the DeltaSeconds parameter, it will return the upcoming frame as intended.")
	FAsyncPhysicsTimestamp GetPhysicsTimestamp(float DeltaSeconds) { return GetPhysicsTimestamp(); }

	/** Get the physics frame number offset between Local and Server, recommended is to use GetPhysicsTimestamp() which takes this offset into account on the client for the ServerFrame value */
	int32 GetNetworkPhysicsTickOffset() const { return NetworkPhysicsTickOffset; }

	/** True if the NetworkPhysicsTickOffset is setup, before this is setup the GetNetworkPhysicsTickOffset() and GetPhysicsTimestamp() functions will not return valid results */
	bool GetNetworkPhysicsTickOffsetAssigned() const { return bNetworkPhysicsTickOffsetAssigned; }

	/** Get the current buffer added to the physics tick-offset */
	float GetTickOffsetBufferTime() const { return TickOffsetBufferTime; }

	/** Set a new buffer time to add to the physics tick-offset */
	void SetTickOffsetBufferTime(float NewBufferTime);

	/** Return the handshake cookie of the connection this player controller is associated. */
	ENGINE_API uint32 GetClientHandshakeId() const;

	/** Associate this player controller with a connection using it's handshake cookie. */
	ENGINE_API void SetClientHandshakeId(uint32 InClientHandshakeId);

	/** Set the connection identifier of the local user on the client that will be using this player controller. */
	ENGINE_API void SetLocalPlayerConnectionIdentifier(int32 Identifier);
	
	/** Get the connection identifier of the local user on the client that will be using this player controller. */
	ENGINE_API int32 GetLocalPlayerConnectionIdentifier() const;

private:

	/** The player id associated with this player controller. */
	UPROPERTY()
	FUniqueNetIdRepl CachedConnectionPlayerId;

	/** An optional client handshake id used by the connection associated with this player controller. */
	UPROPERTY(DuplicateTransient)
	uint32 ClientHandshakeId = 0;

	int32 LocalPlayerConnectionIdentifier;
};

/** 
 * A specialized player controller used by server connections that are created using NMT_JoinNoPawn and NMT_JoinNoPawnSplit.
 */
UCLASS(MinimalAPI)
class ANoPawnPlayerController : public APlayerController
{
public:

	GENERATED_BODY()
	ANoPawnPlayerController(const FObjectInitializer& ObjectInitializer);

	virtual void PostInitializeComponents() override;
	virtual class AActor* GetViewTarget() const override;
	virtual class UNetConnection* GetNetConnection() const override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual FRemoteServerId GetPreferredRemoteServer() const override;
#endif

	/** Set a view target position for use in replication relevancy. */
	UFUNCTION(Server, Unreliable)
	void ServerSetViewTargetPosition(FVector ViewTargetPos);

	/** When disabled, any connection using this player controller won't replicate actors down to it's client. Enabled by default. */
	UFUNCTION(Server, Reliable)
	void ServerEnableReplicationToConnection(bool bEnable);

	bool IsEnableReplicationToConnection() const { return bEnableReplicationToConnection; }

private:
	
	bool bEnableReplicationToConnection = true;
};
