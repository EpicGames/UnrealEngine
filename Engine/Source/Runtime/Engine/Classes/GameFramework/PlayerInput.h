// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// PlayerInput
// Object within PlayerController that manages player input.
// Only spawned on client.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "GestureRecognizer.h"
#include "InputCoreTypes.h"
#include "InputKeyEventArgs.h"
#include "KeyState.h"
#include "Containers/StaticArray.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "PlayerInput.generated.h"

class FDebugDisplayInfo;
class UInputComponent;
class ULocalPlayer;
struct FDelegateDispatchDetails;
struct FInputActionBinding;
struct FInputAxisBinding;
struct FInputKeyBinding;


/** Struct containing mappings for legacy method of binding keys to exec commands. */
USTRUCT()
struct FKeyBind
{
	GENERATED_USTRUCT_BODY()

	/** The key to be bound to the command */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	FKey Key;

	/** The command to execute when the key is pressed/released */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	FString Command;

	/** Whether the control key needs to be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 Control:1;

	/** Whether the shift key needs to be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 Shift:1;

	/** Whether the alt key needs to be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 Alt:1;

	/** Whether the command key needs to be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 Cmd:1;

	/** Whether the control key must not be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bIgnoreCtrl:1;

	/** Whether the shift key must not be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bIgnoreShift:1;

	/** Whether the alt key must not be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bIgnoreAlt:1;

	/** Whether the command key must not be held when the key event occurs */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	uint8 bIgnoreCmd:1;

	UPROPERTY(transient)
	uint8 bDisabled : 1;

	FKeyBind()
		: Control(false)
		, Shift(false)
		, Alt(false)
		, Cmd(false)
		, bIgnoreCtrl(false)
		, bIgnoreShift(false)
		, bIgnoreAlt(false)
		, bIgnoreCmd(false)
		, bDisabled(false)
	{
	}
};

/** Configurable properties for control axes, used to transform raw input into game ready values. */
USTRUCT()
struct FInputAxisProperties
{
	GENERATED_USTRUCT_BODY()

	/** What the dead zone of the axis is.  For control axes such as analog sticks. */
	UPROPERTY(EditAnywhere, Category="Input")
	float DeadZone;

	/** Scaling factor to multiply raw value by. */
	UPROPERTY(EditAnywhere, Category="Input")
	float Sensitivity;

	/** For applying curves to [0..1] axes, e.g. analog sticks */
	UPROPERTY(EditAnywhere, Category="Input")
	float Exponent;

	/** Inverts reported values for this axis */
	UPROPERTY(EditAnywhere, Category="Input")
	uint8 bInvert:1;

	FInputAxisProperties()
		: DeadZone(0.2f)
		, Sensitivity(1.f)
		, Exponent(1.f)
		, bInvert(false)
	{}
	
};

/** Configurable properties for control axes. */
USTRUCT()
struct FInputAxisConfigEntry
{
	GENERATED_USTRUCT_BODY()

	/** Axis Key these properties apply to */
	UPROPERTY(VisibleAnywhere, Category="Input")
	FName AxisKeyName;

	/** Properties for the Axis Key */
	UPROPERTY(EditAnywhere, Category="Input")
	struct FInputAxisProperties AxisProperties;
};

/** 
 * Defines a mapping between an action and key 
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
USTRUCT( BlueprintType )
struct FInputActionKeyMapping
{
	GENERATED_USTRUCT_BODY()

	/** Friendly name of action, e.g "jump" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	FName ActionName;

	/** true if one of the Shift keys must be down when the KeyEvent is received to be acknowledged */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	uint8 bShift:1;

	/** true if one of the Ctrl keys must be down when the KeyEvent is received to be acknowledged */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	uint8 bCtrl:1;

	/** true if one of the Alt keys must be down when the KeyEvent is received to be acknowledged */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	uint8 bAlt:1;

	/** true if one of the Cmd keys must be down when the KeyEvent is received to be acknowledged */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	uint8 bCmd:1;

	/** Key to bind it to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	FKey Key;

	bool operator==(const FInputActionKeyMapping& Other) const
	{
		return (   ActionName == Other.ActionName
				&& Key == Other.Key
				&& bShift == Other.bShift
				&& bCtrl == Other.bCtrl
				&& bAlt == Other.bAlt
				&& bCmd == Other.bCmd);
	}

	bool operator<(const FInputActionKeyMapping& Other) const
	{
		bool bResult = false;
		if (ActionName.LexicalLess(Other.ActionName))
		{
			bResult = true;
		}
		else if (ActionName == Other.ActionName)
		{
			bResult = (Key < Other.Key);
		}
		return bResult;
	}

	FInputActionKeyMapping(const FName InActionName = NAME_None, const FKey InKey = EKeys::Invalid, const bool bInShift = false, const bool bInCtrl = false, const bool bInAlt = false, const bool bInCmd = false)
		: ActionName(InActionName)
		, bShift(bInShift)
		, bCtrl(bInCtrl)
		, bAlt(bInAlt)
		, bCmd(bInCmd)
		, Key(InKey)
	{}
};

/** 
 * Defines a mapping between an axis and key 
 * 
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
**/
USTRUCT( BlueprintType )
struct FInputAxisKeyMapping
{
	GENERATED_USTRUCT_BODY()

	/** Friendly name of axis, e.g "MoveForward" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	FName AxisName;

	/** Multiplier to use for the mapping when accumulating the axis value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	float Scale;

	/** Key to bind it to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Input")
	FKey Key;

	bool operator==(const FInputAxisKeyMapping& Other) const
	{
		return (   AxisName == Other.AxisName
				&& Key == Other.Key
				&& Scale == Other.Scale);
	}

	bool operator<(const FInputAxisKeyMapping& Other) const
	{
		bool bResult = false;
		if (AxisName.LexicalLess(Other.AxisName))
		{
			bResult = true;
		}
		else if (AxisName == Other.AxisName)
		{
			if (Key < Other.Key)
			{
				bResult = true;
			}
			else if (Key == Other.Key)
			{
				bResult = (Scale < Other.Scale);

			}
		}
		return bResult;
	}

	FInputAxisKeyMapping(const FName InAxisName = NAME_None, const FKey InKey = EKeys::Invalid, const float InScale = 1.f)
		: AxisName(InAxisName)
		, Scale(InScale)
		, Key(InKey)
	{}
};


/** 
 * Defines a mapping between an action and speech recognition 
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.7, "Speech mappings are deprecated. These were only used by a now deprecated platform.") FInputActionSpeechMapping
{
	GENERATED_USTRUCT_BODY()

	static FName GetKeyCategory()
	{
		return FName("Speech");
	}
	FName GetActionName() const
	{
		return ActionName;
	}
	FName GetSpeechKeyword() const
	{
		return SpeechKeyword;
	}
	FName GetKeyName() const
	{
		return FName(*FString::Printf(TEXT("%s_%s"), *GetKeyCategory().ToString(), *SpeechKeyword.ToString()));
	}
private:
	/** Friendly name of action, e.g "jump" */
	UPROPERTY(EditAnywhere, Category="Input")
	FName ActionName;

	/** Key to bind it to. */
	UPROPERTY(EditAnywhere, Category="Input")
	FName SpeechKeyword;
public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool operator==(const FInputActionSpeechMapping& Other) const
	{
		return (   ActionName == Other.ActionName
				&& SpeechKeyword == Other.SpeechKeyword);
	}

	bool operator<(const FInputActionSpeechMapping& Other) const
	{
		bool bResult = false;
		if (ActionName.LexicalLess(Other.ActionName))
		{
			bResult = true;
		}
		else if (ActionName == Other.ActionName)
		{
			bResult = (SpeechKeyword.LexicalLess(Other.SpeechKeyword));
		}
		return bResult;
	}

	FInputActionSpeechMapping(const FName InActionName = NAME_None, const FName InSpeechKeyword = NAME_None)
		: ActionName(InActionName)
		, SpeechKeyword(InSpeechKeyword)
	{}
	
	~FInputActionSpeechMapping() = default;
	FInputActionSpeechMapping(const FInputActionSpeechMapping&) = default;
	FInputActionSpeechMapping(FInputActionSpeechMapping&&) = default;
	FInputActionSpeechMapping& operator=(const FInputActionSpeechMapping&) = default;
	FInputActionSpeechMapping& operator=(FInputActionSpeechMapping&&) = default;
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** Paramaters to be considered when calling UPlayerInput::InputKey. */
struct UE_DEPRECATED(5.6, "Use FInputKeyEventArgs instead") FInputKeyParams
{
	FInputKeyParams() = default;
	
	FInputKeyParams(FKey InKey, enum EInputEvent InEvent, FVector InDelta, bool bGamepadOverride = false, FInputDeviceId InInputDevice = INPUTDEVICEID_NONE)
		: Key(InKey)
		, InputDevice(InInputDevice)
		, Event(InEvent)
		, Delta(InDelta)
		, bIsGamepadOverride(bGamepadOverride)
	{};

	FInputKeyParams(FKey InKey, enum EInputEvent InEvent, double InDelta, bool bGamepadOverride = false, FInputDeviceId InInputDevice = INPUTDEVICEID_NONE)
		: Key(InKey)
		, InputDevice(InInputDevice)
		, Event(InEvent)
		, Delta(FVector(InDelta, 0.0, 0.0))
		, bIsGamepadOverride(bGamepadOverride)
	{};
	
	FInputKeyParams(FKey InKey, double InDelta, float InDeltaTime, int32 InNumSamples, bool bGamepadOverride = false, FInputDeviceId InInputDevice = INPUTDEVICEID_NONE)
		: Key(InKey)
		, InputDevice(InInputDevice)
		, NumSamples(InNumSamples)
		, DeltaTime(InDeltaTime)
		, Delta(FVector(InDelta, 0.0, 0.0))
		, bIsGamepadOverride(bGamepadOverride)
	{};

	FInputKeyParams(FKey InKey, FVector InDelta, float InDeltaTime, int32 InNumSamples, bool bGamepadOverride = false, FInputDeviceId InInputDevice = INPUTDEVICEID_NONE)
		: Key(InKey)
		, InputDevice(InInputDevice)
		, NumSamples(InNumSamples)
		, DeltaTime(InDeltaTime)
		, Delta(InDelta)
		, bIsGamepadOverride(bGamepadOverride)
	{};

	/** The key that has been pressed */
	FKey Key = EKeys::Invalid;

	/** The input device that has triggered this input */
	FInputDeviceId InputDevice = INPUTDEVICEID_NONE;

	/** The event that has caused a Button key to be considered */
	enum EInputEvent Event = EInputEvent::IE_Pressed;

	/** The number of samples to be taken into account with the FKeyState::SampleCountAccumulator */
	int32 NumSamples = 0;

	/** The time between the previous frame and the current one */
	float DeltaTime = 1 / 60.f;
	
	/** The Delta that the given key/axis has been changed by */
	FVector Delta = FVector::ZeroVector;

	/** If set to true, treat this input event as if it were from a gamepad, whether the FKey is a gamepad key or not. */
	bool bIsGamepadOverride = false;

	/** Returns true if the Key used for this input is a gamepad key */
	bool IsGamepad() const { return Key.IsGamepadKey() || bIsGamepadOverride; }

	/** Get the delta of the given axis for 1D axis */
	double Get1DAxisDelta() const { return Delta.X; }

	/** Get the delta of the given axis for 2D axis */
	FVector2D Get2DAxisDelta() const { return FVector2D((float)Delta.X, (float)Delta.Y); }

	/** Get the delta of the given axis for 3D axis */
	FVector Get3DAxisDelta() const { return Delta; }
};

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogPlayerInput, Log, All);

namespace UE
{
	struct FTouchState
	{
		FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
		FVector2f ViewportRelativeLocation = FVector2f(-1.0f, -1.0f);
		float Pressure = 0.0f;
	};
}

/**
 * Object within PlayerController that processes player input.
 * Only exists on the client in network games.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
UCLASS(config=Input, transient, MinimalAPI)
class UPlayerInput : public UObject
{
	GENERATED_BODY()

public:

	ENGINE_API UPlayerInput();

	// NOTE: These touch vectors are calculated and set directly, they do not go through the .ini Bindings
	// @todo: We have 10 touches to match the number of Touch* entries in EKeys (not easy to make this an enum or define or anything)
	UE_DEPRECATED(5.8, "Use GetTouchStates() instead")
	FVector Touches[EKeys::NUM_TOUCH_KEYS];

	/** The most recent state reported for each finger. */
	TArrayView<const UE::FTouchState> GetTouchStates() const
	{
		return TouchStates;
	}

	/** Used to store paired touch locations for event ids during the frame and flushed when processed. */
	TMap<uint32, FVector> TouchEventLocations;

	// Mouse smoothing sample data
	float ZeroTime[2];    /** How long received mouse movement has been zero. */
	float SmoothedMouse[2];    /** Current average mouse movement/sample */
	int32 MouseSamples;    /** Number of mouse samples since mouse movement has been zero */
	float MouseSamplingTotal;    /** DirectInput's mouse sampling total time */


private:
	TEnumAsByte<EInputEvent> CurrentEvent;

public:
	/** Generic bindings of keys to Exec()-compatible strings for development purposes only */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	TArray<struct FKeyBind> DebugExecBindings;

	/** This player's version of the Axis Properties */
	TArray<struct FInputAxisConfigEntry> AxisConfig;

	/** This player's version of the Action Mappings */
	TArray<struct FInputActionKeyMapping> ActionMappings;

	/** This player's version of Axis Mappings */
	TArray<struct FInputAxisKeyMapping> AxisMappings;

	/** List of Axis Mappings that have been inverted */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	TArray<FName> InvertedAxis;

	/** Gets the axis properties for a given AxisKey.  Returns if true if AxisKey was found in the AxisConfig array. */
	ENGINE_API bool GetAxisProperties(const FKey AxisKey, FInputAxisProperties& AxisProperties);

	/** Gets the axis properties for a given AxisKey.  Returns if true if AxisKey was found in the AxisConfig array. */
	ENGINE_API void SetAxisProperties(const FKey AxisKey, const FInputAxisProperties& AxisProperties);

	/** Exec function to change the mouse sensitivity */
	ENGINE_API void SetMouseSensitivity(const float SensitivityX, const float SensitivityY);

	/** Sets both X and Y axis sensitivity to the supplied value. */
	UFUNCTION(exec)
	void SetMouseSensitivity(const float Sensitivity) { SetMouseSensitivity(Sensitivity, Sensitivity); }

	/** Exec function to add a debug exec command.
	* Use + to addmodifiers; do not include any spaces between modifiers, so that it is not conflated with the command to be bound.
	* example usage: 
	* SetBind k serverexec slomo 0.1
	* SetBind shift+k serverexec slomo 1
	*/
	UFUNCTION(exec)
	ENGINE_API void SetBind(FName BindName, const FString& Command);

	/** Returns the mouse sensitivity along the X-axis, or the Y-axis, or 1.0 if none are known. */
	ENGINE_API float GetMouseSensitivityX();

	/** Returns the mouse sensitivity along the Y-axis, or 1.0 if none are known. */
	ENGINE_API float GetMouseSensitivityY();

	/** Returns whether an Axis Key is inverted */
	ENGINE_API bool GetInvertAxisKey(const FKey AxisKey);

	/** Returns whether an Axis Mapping is inverted */
	ENGINE_API bool GetInvertAxis(const FName AxisName);

	/** Exec function to invert an axis key */
	UFUNCTION(exec)
	ENGINE_API void InvertAxisKey(const FKey AxisKey);

	/** Exec function to invert an axis mapping */
	UFUNCTION(exec)
	ENGINE_API void InvertAxis(const FName AxisName);

	/** Exec function to reset mouse smoothing values */
	UFUNCTION(exec)
	ENGINE_API void ClearSmoothing();

	/** Add a player specific action mapping. */
	ENGINE_API void AddActionMapping(const FInputActionKeyMapping& KeyMapping);

	/** Remove a player specific action mapping. */
	ENGINE_API void RemoveActionMapping(const FInputActionKeyMapping& KeyMapping);

	/** Add a player specific axis mapping. */
	ENGINE_API void AddAxisMapping(const FInputAxisKeyMapping& KeyMapping);

	/** Remove a player specific axis mapping. */
	ENGINE_API void RemoveAxisMapping(const FInputAxisKeyMapping& KeyMapping);

	/** Add an engine defined action mapping that cannot be remapped. */
	static ENGINE_API void AddEngineDefinedActionMapping(const FInputActionKeyMapping& ActionMapping);

	/** Add an engine defined axis mapping that cannot be remapped. */
	static ENGINE_API void AddEngineDefinedAxisMapping(const FInputAxisKeyMapping& AxisMapping);

	/** Clear the current cached key maps and rebuild from the source arrays. */
	ENGINE_API void ForceRebuildingKeyMaps(const bool bRestoreDefaults = false);

	/** Return's this object casted to a player controller. This can be null if there is no player controller. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input")
	ENGINE_API APlayerController* GetOuterAPlayerController() const;

	/**
	* Returns the owning local player of this Input Object. This will only be valid if there is a 
	* current outer player controller.
	*/
	ENGINE_API ULocalPlayer* GetOwningLocalPlayer() const;
	
private:

	/** Runtime struct that caches the list of mappings for a given Action Name and the capturing chord if applicable */
	struct FActionKeyDetails
	{
		/** List of all action key mappings that correspond to the action name in the containing map */
		TArray<FInputActionKeyMapping> Actions;

		/** For paired actions only, this represents the chord that is currently held and when it is released will represent the release event */
		FInputChord CapturingChord;
	};

	/** Runtime struct that caches the list of mappings for a given Axis Name and whether that axis is currently inverted */
	struct FAxisKeyDetails
	{
		/** List of all axis key mappings that correspond to the axis name in the containing map */
		TArray<FInputAxisKeyMapping> KeyMappings;

		/** Whether this axis should invert its outputs */
		uint8 bInverted:1;

		FAxisKeyDetails()
			: bInverted(false)
		{
		}
	};

	/** Internal structure for storing axis config data. */
	TMap<FKey,FInputAxisProperties> AxisProperties;

	/** Map of Action Name to details about the keys mapped to that action */
	mutable TMap<FName,FActionKeyDetails> ActionKeyMap;

	/** Map of Axis Name to details about the keys mapped to that axis */
	mutable TMap<FName,FAxisKeyDetails> AxisKeyMap;

	/** The current game view of each key */
	TMap<FKey,FKeyState> KeyStateMap;

	mutable uint32 KeyMapBuildIndex;

	mutable uint8 bKeyMapsBuilt:1;

public:
	
	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual UWorld* GetWorld() const override;
	//~ End UObject Interface

	/** Flushes the current key state. */
	ENGINE_API virtual void FlushPressedKeys();

	/** Flushes the current key state of the keys associated with the action name passed in */
	ENGINE_API void FlushPressedActionBindingKeys(FName ActionName);

	/** Handles a key input event.  Returns true if there is an action that handles the specified key. */
	UE_DEPRECATED(5.6, "Use the version which takes a FInputKeyEventArgs instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ENGINE_API virtual bool InputKey(const FInputKeyParams& Params) final;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Handles a key input event.  Returns true if there is an action that handles the specified key. */
	ENGINE_API virtual bool InputKey(const FInputKeyEventArgs& Params);

	/** Handles a touch input event.  Returns true. */
	UE_DEPRECATED(5.8, "Use the overload that uses FTouchId instead")
	ENGINE_API bool InputTouch(
		const FInputDeviceId DeviceId,
		const uint32 Handle,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint32 TouchpadIndex,
		const uint64 Timestamp);

	/** Handles a touch input event.  Returns true. */
	ENGINE_API bool InputTouch(
		const FTouchId TouchId,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint64 Timestamp);

	/** Handles a motion input event.  Returns true. */
	UE_DEPRECATED(5.6, "Use the version which takes in a FInputDeviceId and a timestamp instead.")
	ENGINE_API bool InputMotion(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration);

	/** Handles a motion input event.  Returns true. */
	ENGINE_API bool InputMotion(
		const FInputDeviceId DeviceId,
		const FVector& Tilt,
		const FVector& RotationRate,
		const FVector& Gravity,
		const FVector& Acceleration,
		const uint64 Timestamp);

	/** Handles a gesture input event.  Returns true. */
	UE_DEPRECATED(5.6, "Use the version which takes a device id and timestamp instead.")
	ENGINE_API bool InputGesture(const FKey Gesture, const EInputEvent Event, const float Value);

	/** Handles a gesture input event.  Returns true. */
	ENGINE_API bool InputGesture(
		const FInputDeviceId DeviceId,
		const FKey Gesture,
		const EInputEvent Event,
		const float Value,
		const uint64 Timestamp);

	/** Handles a gesture input event with 2D value (e.g. pan delta). Returns true. */
	ENGINE_API bool InputGesture(const FInputKeyEventArgs& Params);

	/** Manually update the GestureRecognizer AnchorDistance using the current locations of the touches */
	ENGINE_API void UpdatePinchStartDistance();

	/** Per frame tick function. Primarily for gesture recognition */
	ENGINE_API void Tick(float DeltaTime);

	/** Process the frame's input events given the current input component stack. */
	ENGINE_API virtual void ProcessInputStack(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused);

protected:
	
	/**
	 * Evaluates the current state of the KeyStateMap this tick.
	 * Should only be called from UPlayerInput::ProcessInputStack
	 */
	ENGINE_API virtual void EvaluateKeyMapState(const float DeltaTime, const bool bGamePaused, OUT TArray<TPair<FKey, FKeyState*>>& KeysWithEvents);

	/**
	 * Evaluates the state of any bound input delegates on the given Input Component Stack.
	 * This should only be called from UPlayerInput::ProcessInputStack
	 */
	ENGINE_API virtual void EvaluateInputDelegates(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents);

	/** 
	* Called just before iteration every input component on the stack to process their bound listener delegates.
	* This provides an extension point to handle modifying the current input state before determining which input delegates should be fired.
	*/
	virtual void PrepareInputDelegatesForEvaluation(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents) {}

	/**
	 * Evaluates a single input component on the component stack and executes its bound delegates.
	 * 
	 * @return True if this input component should BLOCK the rest of the input stack, preventing further components from evaluating their delegates.
	 */
	ENGINE_API virtual bool EvaluateInputComponentDelegates(UInputComponent* const IC, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents, const float DeltaTime, const bool bGamePaused);
	
	/** 
	* Called on any input component which was not evaluated due to the input being blocked by another component earlier in the stack.
	* These input components will not have had the EvaluateInputComponentDelegates flow executed, because they are blocked.
	* This is a good spot to reset any delegate values if desired to reflect that they didn't execute for next frame.
	*/
	ENGINE_API virtual void EvaluateBlockedInputComponent(UInputComponent* InputComponent);
	
public:

	/** Rather than processing input, consume it and discard without doing anything useful with it.  Like calling ProcessInputStack() and ignoring all results. */
	ENGINE_API void DiscardPlayerInput();

	/** 
	Smooth mouse movement, because mouse sampling doesn't match up with tick time.
	 * @note: if we got sample event for zero mouse samples (so we
				didn't have to guess whether a 0 was caused by no sample occuring during the tick (at high frame rates) or because the mouse actually stopped)
	 * @param: aMouse is the mouse axis movement received from DirectInput
	 * @param: SampleCount is the number of mouse samples received from DirectInput
	 * @param: Index is 0 for X axis, 1 for Y axis
	 * @return the smoothed mouse axis movement
	 */
	ENGINE_API virtual float SmoothMouse(float aMouse, uint8& SampleCount, int32 Index);

	/**
	 * Draw important PlayerInput variables on canvas.  HUD will call DisplayDebug() on the current ViewTarget when the ShowDebug exec is used
	 *
	 * @param Canvas - Canvas to draw on
	 * @param DebugDisplay - Contains information about what debug data to display
	 * @param YL - Height of the current font
	 * @param YPos - Y position on Canvas. YPos += YL, gives position to draw text for next debug line.
	 */
	ENGINE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);

	/** @return key state of the InKey */
	FKeyState* GetKeyState(FKey InKey) { return KeyStateMap.Find(InKey); }
	const FKeyState* GetKeyState(FKey InKey) const { return KeyStateMap.Find(InKey); }

	/** @return true if InKey is currently held */
	ENGINE_API bool IsPressed( FKey InKey ) const;
	
	/** @return true if InKey went from up to down since player input was last processed. */
	ENGINE_API bool WasJustPressed( FKey InKey ) const;

	/** return true if InKey went from down to up since player input was last processed. */
	ENGINE_API bool WasJustReleased( FKey InKey ) const;

	/** @return how long the key has been held down, or 0.f if not down. */
	ENGINE_API float GetTimeDown( FKey InKey ) const;

	/** @return current state of the InKey */
	ENGINE_API float GetKeyValue( FKey InKey ) const;

	/** @return current state of the InKey */
	ENGINE_API float GetRawKeyValue( FKey InKey ) const;

	/** @return processed value of the InKey */
	ENGINE_API FVector GetProcessedVectorKeyValue(FKey InKey) const;

	/** @return raw value of the InKey */
	ENGINE_API FVector GetRawVectorKeyValue(FKey InKey) const;

	/** @return true if alt key is pressed */
	ENGINE_API bool IsAltPressed() const;

	/** @return true if ctrl key is pressed */
	ENGINE_API bool IsCtrlPressed() const;

	/** @return true if shift key is pressed */
	ENGINE_API bool IsShiftPressed() const;

	/** @return true if cmd key is pressed */
	ENGINE_API bool IsCmdPressed() const;

	uint32 GetKeyMapBuildIndex() const { return KeyMapBuildIndex; }

#if !UE_BUILD_SHIPPING
	/**
	 * Exec handler
	 */
	ENGINE_API bool Exec(UWorld* UInWorld, const TCHAR* Cmd,FOutputDevice& Ar);

	/** Returns the command for a given key in the legacy binding system */
	ENGINE_API FString GetBind(FKey Key);

	/** Get the legacy Exec key binding for the given command. */
	ENGINE_API FKeyBind GetExecBind(FString const& ExecCommand);

	/** Execute input commands within the legacy key binding system. */
	ENGINE_API bool ExecInputCommands( UWorld* InWorld, const TCHAR* Cmd, class FOutputDevice& Ar);
#endif

	/** Returns the list of keys mapped to the specified Action Name */
	ENGINE_API virtual const TArray<FInputActionKeyMapping>& GetKeysForAction(const FName ActionName) const;

	/** Returns the list of keys mapped to the specified Axis Name */
	ENGINE_API virtual const TArray<FInputAxisKeyMapping>& GetKeysForAxis(const FName AxisName) const;

	static const TArray<FInputActionKeyMapping>& GetEngineDefinedActionMappings() { return EngineDefinedActionMappings; }
	static const TArray<FInputAxisKeyMapping>& GetEngineDefinedAxisMappings() { return EngineDefinedAxisMappings; }

protected:
	TMap<FKey, FKeyState>& GetKeyStateMap()
	{ return KeyStateMap; }

	/**
	* Given raw keystate value of a vector axis, returns the "massaged" value. Override for any custom behavior,
	* such as input changes dependent on a particular game state.
	*/
	ENGINE_API virtual FVector MassageVectorAxisInput(FKey Key, FVector RawValue);

	/**
	* Given raw keystate value, returns the "massaged" value. Override for any custom behavior,
	* such as input changes dependent on a particular game state.
	*/
	ENGINE_API virtual float MassageAxisInput(FKey Key, float RawValue);

private:

	/** Process non-axes keystates */
	ENGINE_API void ProcessNonAxesKeys(FKey Inkey, FKeyState* KeyState);
	
	// finished processing input for this frame, clean up for next update
	ENGINE_API void FinishProcessingPlayerInput();

	/** key event processing
	 * @param Key - name of key causing event
	 * @param Event - types of event, includes IE_Pressed
	 * @return true if just pressed
	 */
	ENGINE_API bool KeyEventOccurred(FKey Key, EInputEvent Event, TArray<uint32>& EventIndices, const FKeyState* KeyState = nullptr) const;

	/* Collects the chords and the delegates they invoke for an action binding
	 * @param ActionBinding - the action to determine whether it occurred
	 * @param bGamePaused - whether the game is currently paused
	 * @param FoundChords - the list of chord/delegate pairs to add to
	 * @param KeysToConsume - array to collect the keys associated with this binding that should be consumed
	 */
	ENGINE_API void GetChordsForAction(const FInputActionBinding& ActionBinding, const bool bGamePaused, TArray<struct FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume);

	/* Helper function for GetChordsForAction to examine each keymapping that belongs to the ActionBinding
	 * @param KeyMapping - the key mapping to determine whether it occured
	 * @param ActionBinding - the action to determine whether it occurred
	 * @param bGamePaused - whether the game is currently paused
	 * @param FoundChords - the list of chord/delegate pairs to add to
	 * @param KeysToConsume - array to collect the keys associated with this binding that should be consumed
	 */
	ENGINE_API void GetChordsForKeyMapping(const FInputActionKeyMapping& KeyMapping, const FInputActionBinding& ActionBinding, const bool bGamePaused, TArray<FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume, const FKeyState* KeyState = nullptr);

	/* Collects the chords and the delegates they invoke for a key binding
	 * @param KeyBinding - the key to determine whether it occurred
	 * @param bGamePaused - whether the game is currently paused
	 * @param FoundChords - the list of chord/delegate pairs to add to
	 * @param KeysToConsume - array to collect the keys associated with this binding that should be consumed
	 */
	ENGINE_API void GetChordForKey(const FInputKeyBinding& KeyBinding, const bool bGamePaused, TArray<struct FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume, const FKeyState* KeyState = nullptr);

	/* Returns the summed values of all the components of this axis this frame
	 * @param AxisBinding - the action to determine if it ocurred
	 * @param KeysToConsume - array to collect the keys associated with this binding that should be consumed
	 * @param bHadAnyNonConsumedKeys - If false, then all keys related to the axis binding have been consumed and there were none that could be used to determine the value
	 *
	 * @return The value of the axis based on the all the key mappings to it added together
	 */
	ENGINE_API float DetermineAxisValue(const FInputAxisBinding& AxisBinding, const bool bGamePaused, TArray<FKey>& KeysToConsume, OUT bool& bHadAnyNonConsumedKeys) const;

	/** Utility function to ensure the key mapping cache maps are built */
	inline void ConditionalBuildKeyMappings() const
	{
		if (!bKeyMapsBuilt)
		{
			ConditionalBuildKeyMappings_Internal();
		}
	}

	ENGINE_API virtual void ConditionalBuildKeyMappings_Internal() const;

	/** Set the Key consumed for the frame so that subsequent input components will not be notified they were pressed */
	ENGINE_API void ConsumeKey(FKey Key);

	/** @return true if InKey is being consumed */
	ENGINE_API bool IsKeyConsumed(FKey Key, const FKeyState* KeyState = nullptr) const;

protected:

	/** Initialized axis properties (i.e deadzone values) if needed */
	ENGINE_API void ConditionalInitAxisProperties();

	/** @return True if a key is handled by an action binding */
	ENGINE_API virtual bool IsKeyHandledByAction(FKey Key) const;

	/** Retrieves the actions bound to the given input component which are triggered by a given key. */
	ENGINE_API void GetActionsBoundToKey(UInputComponent* InputComponent, FKey Key, TArray<TSharedPtr<FInputActionBinding>>& Actions);

	TStaticArray<UE::FTouchState, EKeys::NUM_TOUCH_KEYS> TouchStates;

private:

	/** Gesture recognizer object */
	// @todo: Move this up to Slate?
	FGestureRecognizer GestureRecognizer;
	friend FGestureRecognizer;

	/** Static empty array to be able to return from GetKeysFromAction when there are no keys mapped to the requested action name */
	static ENGINE_API const TArray<FInputActionKeyMapping> NoKeyMappings;

	/** Static empty array to be able to return from GetKeysFromAxis when there are no axis mapped to the requested axis name */
	static ENGINE_API const TArray<FInputAxisKeyMapping> NoAxisMappings;

	/** Action Mappings defined by engine systems that cannot be remapped by users */
	static ENGINE_API TArray<FInputActionKeyMapping> EngineDefinedActionMappings;

	/** Axis Mappings defined by engine systems that cannot be remapped by users */
	static ENGINE_API TArray<FInputAxisKeyMapping> EngineDefinedAxisMappings;

	// Temporary array used as part of input processing
	TArray<uint32> EventIndices;

	/** A counter used to track the order in which events occurred since the last time the input stack was processed */
	uint32 EventCount;

	/** Cache the last time dilation so as to be able to clear smoothing when it changes */
	float LastTimeDilation;

	friend class UEnhancedPlayerInput;	// TEMP: Support for ongoing input rework
};
