// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API ENGINE_API

/**
 * The UE_PLAYER_INPUT_INCLUDE_DEBUG macro will allow the engine to broadcast a debug delegate whenever
 *
 * a Player input event is executed. This is useful
 * 
 * If you do not wish to have this enabled for your build target, you can add this to your *.Target.cs file:
 *		GlobalDefinitions.Add("UE_PLAYER_INPUT_INCLUDE_DEBUG=0");
 */
#ifndef UE_PLAYER_INPUT_INCLUDE_DEBUG
	#define UE_PLAYER_INPUT_INCLUDE_DEBUG	!UE_BUILD_SHIPPING || UE_BUILD_TEST
#endif

#if UE_PLAYER_INPUT_INCLUDE_DEBUG

class UInputComponent;
class UPlayerInput;
class UObject;

namespace UE::Input
{
	
/**
 * Payload data for when a player input event has occured.
 */
struct FPlayerInputDebuggingArgs
{
	/**
	 * The input component which has a delegate bound to this
	 */
	TWeakObjectPtr<const UInputComponent> InputComponent;

	/**
	 * The object which is receiving this input delegate
	 */
	TWeakObjectPtr<const UObject> ListeningObject;

	/**
	 * The name of the input delegate which is being fired.
	 * For legacy action/axis mappings, this will simply be their FName found in
	 * the project settings. For Enhanced Input actions, this will be the Object Path of the UInputAction.
	 */
	FString ActionName = TEXT("");
	
	/**
	 * The value of the input delegate which has been fired. 
	 *
	 * - For legacy IE_Pressed events, the X value will be 1.
	 * - For legacy IE_Released events, the Y value will be 1.
	 * - For legacy "axis" events, this value will be the raw accumulated value of that axis.
	 * Keep in mind that legacy axis events fire every tick so long as they are bound, so
	 * applying a deadzone may be desirable for any debug tooling here.
	 * - For Enhanced input events the evaluated input action instance will be used.
	 */
	FVector InputValue = FVector::ZeroVector;
};

/**
 * A class which contains several player input debugging related delegates which may be useful
 * for users building debugging tools for input.
 *
 * This is especially useful when combined with the SlateDebugging.h, because you can receive notifications
 * of when a "raw" slate input event occurs, as well as when a player event occurs. This makes it easier to
 * track down issues with input consumption and create event driven visualizing tools.
 */
class FPlayerInputDebugging
{
public:

	/** Event fired when a player has executed an input event. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlayerInputDelegateExecuted, const UPlayerInput*, const FPlayerInputDebuggingArgs&);
	static UE_API FOnPlayerInputDelegateExecuted OnPlayerInputEventExecuted;

	/** Event fired when the given player's input has been flushed. For example, when the viewport loses focus. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlayerInputFlushed, const UPlayerInput*);
	static UE_API FOnPlayerInputFlushed OnPlayerInputFlushed;
	
	static UE_API void BroadcastPlayerInputDelegateExecuted(const UPlayerInput* PlayerInput, const FPlayerInputDebuggingArgs& EventData);
	static UE_API void BroadcastPlayerInputFlushed(const UPlayerInput* PlayerInput);
};
};	// namespace UE::Input

#endif	// UE_PLAYER_INPUT_INCLUDE_DEBUG

#undef UE_API