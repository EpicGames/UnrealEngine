// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Online/CoreOnline.h"
#include "Misc/EnumRange.h"
#include "PlayerMuteList.generated.h"

struct FUniqueNetIdRepl;

UENUM(meta = (Bitflags))
enum class EVoiceBlockReasons : uint8
{
	None		= 0,		// Communication with this client has no filter reasons
	Muted		= 1 << 0,	// The owning player controller has explicitly muted the player
	Gameplay	= 1 << 1,	// The player was muted for gameplay reasons
	Blocked		= 1 << 2,	// The owning player controller has blocked the player
	BlockedBy	= 1 << 3,	// The owning player controller was blocked by the player
};
ENUM_CLASS_FLAGS(EVoiceBlockReasons);
ENUM_RANGE_BY_VALUES(EVoiceBlockReasons, EVoiceBlockReasons::Muted, EVoiceBlockReasons::Gameplay, EVoiceBlockReasons::Blocked, EVoiceBlockReasons::BlockedBy);

/**
 * Container responsible for managing the mute state of a player controller
 * at the gameplay level (VoiceInterface handles actual muting)
 */
USTRUCT()
struct FPlayerMuteList
{
	GENERATED_USTRUCT_BODY()

public:

	FPlayerMuteList() :
		bHasVoiceHandshakeCompleted(false),
		VoiceChannelIdx(0)
	{
	}

	/** Map of player ids and a bitfield containing filter reasons. Non-zero entries imply communication is blocked for that player*/
	TUniqueNetIdMap<EVoiceBlockReasons> VoicePacketFilterMap;

	/** Has server and client handshake completed */
	UPROPERTY()
	bool bHasVoiceHandshakeCompleted;
	UPROPERTY()
	int32 VoiceChannelIdx;

public:

	/**
	 * Add a filter reason for an id to this player's mutelist
	 *
	 * @param OwningPC player id to add a filter reason
	 * @param VoiceBlockReason reason to add
	 * 
	 * @return true if it's the first reason to block voice from user
	 */
	bool AddVoiceBlockReason(const FUniqueNetIdPtr& PlayerId, EVoiceBlockReasons VoiceBlockReason);

	/**
	 * Remove a filter reason for an id from this player's mutelist
	 *
	 * @param UniqueIdToRemove player id to remove a filter reason
	 * @param VoiceBlockReason reason to remove
	 *
	 * @return true if it's the last reason blocking voice from user
	 */
	bool RemoveVoiceBlockReason(const FUniqueNetIdPtr& PlayerId, EVoiceBlockReasons VoiceBlockReason);

	/**
	 * Tell the server to mute a given player
	 *
	 * @param OwningPC player controller that would like to mute another player
	 * @param MuteId player id that should be muted
	 */
	void ServerMutePlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& MuteId);

	/**
	 * Tell the server to unmute a given player
	 *
	 * @param OwningPC player controller that would like to unmute another player
	 * @param UnmuteId player id that should be unmuted
	 */
	void ServerUnmutePlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& UnmuteId);

	/**
	 * Server muting based on gameplay rules
	 *
	 * @param OwningPC player controller that would like to mute another player
	 * @param MuteId player id that should be muted
	 */
	void GameplayMutePlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& MuteId);

	/**
	 * Server unmuting based on gameplay rules
	 *
	 * @param OwningPC player controller that would like to unmute another player
	 * @param UnmuteId player id that should be unmuted
	 */
	void GameplayUnmutePlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& UnmuteId);

	/**
	 * Server unmuting all players muted based on gameplay rules
	 *
	 * @param OwningPC player controller that would like to unmute all players
	 */
	void GameplayUnmuteAllPlayers(class APlayerController* OwningPC);

	/**
	 * Tell the server to block a given player
	 *
	 * @param OwningPC player controller that would like to block another player
	 * @param BlockerId player id that should be blocked
	 */
	void ServerBlockPlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& BlockId);

	/**
	 * Tell the server to unblock a given player
	 * @param UnblockedPC player controller that would like to unblock another player
	 * @param BlockerId player id that should be unblocked
	 */
	void ServerUnblockPlayer(class APlayerController* OwningPC, const FUniqueNetIdRepl& UnblockId);

	/**
	 * Is a given player currently muted
	 * 
	 * @param PlayerId player to query mute state
	 *
	 * @return true if the playerid is muted, false otherwise
	 */
	bool IsPlayerMuted(const class FUniqueNetId& PlayerId);
};

/** Dump out information about all player controller mute state */
ENGINE_API FString DumpMutelistState(UWorld* World);
