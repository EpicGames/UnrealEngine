// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/NavigationObjectBase.h"
#include "PlayerStart.generated.h"

/** 
 *	This class indicates a location where a player can spawn when the game begins.
 *	
 * 
 *	@see https://docs.unrealengine.com/latest/INT/Engine/Actors/PlayerStart/
 */
UCLASS(Blueprintable, ClassGroup=Common, hidecategories=Collision, MinimalAPI)
class APlayerStart : public ANavigationObjectBase
{
	GENERATED_BODY()
public:

	ENGINE_API APlayerStart(const FObjectInitializer& ObjectInitializer);

	/*~ To take more control over PlayerStart selection, you can override the virtual AGameModeBase::FindPlayerStart and AGameModeBase::ChoosePlayerStart functions. */

	/** Used when searching for which playerstart to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Object)
	FName PlayerStartTag;

	/** Arrow component to indicate forward direction of start */
#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
public:
#endif

#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	ENGINE_API class UArrowComponent* GetArrowComponent() const;
#endif
};
