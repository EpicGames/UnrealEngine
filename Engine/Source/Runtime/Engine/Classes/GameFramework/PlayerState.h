// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Online/CoreOnline.h"
#include "GameFramework/OnlineReplStructs.h"
#include "GameFramework/Info.h"
#include "PlayerState.generated.h"

/**
 * Struct containing one seconds worth of accumulated ping data (for averaging)
 * NOTE: Maximum PingCount is 7, and maximum PingSum is 8191 (1170*7)
 */
struct PingAvgData
{
	/** The sum of all accumulated pings (used to calculate avg later) */
	uint16	PingSum : 13;

	/** The number of accumulated pings */
	uint8	PingCount : 3;

	/** Default constructor */
	PingAvgData()
		: PingSum(0)
		, PingCount(0)
	{
	}
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnPlayerStatePawnSet, APlayerState*, Player, APawn*, NewPawn, APawn*, OldPawn);

/**
 * A PlayerState is created for every player on a server (or in a standalone game).
 * PlayerStates are replicated to all clients, and contain network game relevant information about the player, such as playername, score, etc.
 */
UCLASS(BlueprintType, Blueprintable, notplaceable, MinimalAPI)
class APlayerState : public AInfo
{
	GENERATED_UCLASS_BODY()

private:

	/** Player's current score. */
	UPROPERTY(ReplicatedUsing = OnRep_Score, Category = PlayerState, BlueprintGetter = GetScore)
	float Score;

	/** Unique net id number. Actual value varies based on current online subsystem, use it only as a guaranteed unique number per player. */
	UPROPERTY(ReplicatedUsing = OnRep_PlayerId, Category = PlayerState, BlueprintGetter = GetPlayerId)
	int32 PlayerId;

	/** Replicated compressed ping for this player (holds ping in msec divided by 4) */
	UPROPERTY(Replicated, Category=PlayerState, BlueprintGetter=GetCompressedPing, meta=(AllowPrivateAccess))
	uint8 CompressedPing;

	/** The current PingBucket index that is being filled */
	uint8 CurPingBucket;

	/**
	 * Whether or not this player's replicated CompressedPing value is updated automatically.
	 * Since player states are always relevant by default, in cases where there are many players replicating,
	 * replicating the ping value can cause additional unnecessary overhead on servers if the value isn't
	 * needed on clients.
	 */
	UPROPERTY(EditDefaultsOnly, Category=PlayerState)
	uint8 bShouldUpdateReplicatedPing:1;

	/** Whether this player is currently a spectator */
	UPROPERTY(Replicated, Category=PlayerState, BlueprintGetter=IsSpectator)
	uint8 bIsSpectator:1;

	/** Whether this player can only ever be a spectator */
	UPROPERTY(Replicated)
	uint8 bOnlySpectator:1;

	/** True if this PlayerState is associated with an AIController */
	UPROPERTY(Replicated, Category=PlayerState, BlueprintGetter=IsABot)
	uint8 bIsABot:1;

public:
	/** client side flag - whether this player has been welcomed or not (player entered message) */
	uint8 bHasBeenWelcomed:1;

private:

	/** Means this PlayerState came from the GameMode's InactivePlayerArray */
	UPROPERTY(ReplicatedUsing=OnRep_bIsInactive)
	uint8 bIsInactive:1;

	/** indicates this is a PlayerState from the previous level of a seamless travel,
	 * waiting for the player to finish the transition before creating a new one
	 * this is used to avoid preserving the PlayerState in the InactivePlayerArray if the player leaves
	 */
	UPROPERTY(Replicated)
	uint8 bFromPreviousLevel:1;

public:

	/** if set, GetPlayerName() will call virtual GetPlayerNameCustom() to allow custom access */
	uint8 bUseCustomPlayerNames : 1;

private:

	/** Elapsed time on server when this PlayerState was first created.  */
	UPROPERTY(Replicated)
	int32 StartTime;

	/** The id used by the network to uniquely identify a player.
	 * NOTE: the internals of this property should *never* be exposed to the player as it's transient
	 * and opaque in meaning (ie it might mean date/time followed by something else).
	 * It is OK to use and pass around this property, though. */
	UPROPERTY(ReplicatedUsing = OnRep_UniqueId)
	FUniqueNetIdRepl UniqueId;

public:
	/** This is used for sending game agnostic messages that can be localized */
	UPROPERTY()
	TSubclassOf<class ULocalMessage> EngineMessageClass;

	/** Exact ping in milliseconds as float (rounded and compressed in replicated CompressedPing) */
	float ExactPing;

	/** Used to match up InactivePlayerState with rejoining playercontroller. */
	UPROPERTY()
	FString SavedNetworkAddress;

	/** The session that the player needs to join/remove from as it is created/leaves */
	FName SessionName;

	/** Broadcast whenever this player's possessed pawn is set */
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnPlayerStatePawnSet OnPawnSet;

private:

	friend struct FSetPlayerStatePawn;

	/** The pawn that is controlled by by this player state. */
	UPROPERTY(BlueprintReadOnly, Category=PlayerState, meta=(AllowPrivateAccess="true"))
	TObjectPtr<APawn> PawnPrivate;

	ENGINE_API void SetPawnPrivate(APawn* InPawn);

	UFUNCTION()
	ENGINE_API void OnPawnPrivateDestroyed(AActor* InActor);

	/**
	 * Stores the last 4 seconds worth of ping data (one second per 'bucket').
	 * It is stored in this manner, to allow calculating a moving average,
	 * without using up a lot of space, while also being tolerant of changes in ping update frequency
	 */
	PingAvgData		PingBucket[4];

	/** The timestamp for when the current PingBucket began filling */
	float			CurPingBucketTimestamp;

	/** Player name, or blank if none. */
	UPROPERTY(ReplicatedUsing = OnRep_PlayerName)
	FString PlayerNamePrivate;

	/** Previous player name.  Saved on client-side to detect player name changes. */
	FString OldNamePrivate;

public:
	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_Score();

	UFUNCTION()
	ENGINE_API virtual void OnRep_PlayerName();

	UFUNCTION()
	ENGINE_API virtual void OnRep_bIsInactive();

	UFUNCTION()
	ENGINE_API virtual void OnRep_PlayerId();

	UFUNCTION()
	ENGINE_API virtual void OnRep_UniqueId();

	//~ Begin AActor Interface
	ENGINE_API virtual void PostInitializeComponents() override; 
	ENGINE_API virtual void Destroyed() override;
	ENGINE_API virtual void Reset() override;
	ENGINE_API virtual FString GetHumanReadableName() const override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual FRemoteServerId GetPreferredRemoteServer() const override;
#endif
	//~ End AActor Interface

	/** Return the pawn controlled by this Player State. */
	UFUNCTION(BlueprintCallable, Category = "PlayerState")
	APawn* GetPawn() const { return PawnPrivate; }

	/** Convenience helper to return a cast version of the pawn controlled by this Player State. */
	template<class T>
	T* GetPawn() const { return Cast<T>(PawnPrivate); }

	/** Returns the AI or player controller that created this player state, or null for remote clients */
	ENGINE_API class AController* GetOwningController() const;

	/** Return the player controller that created this player state, or null for remote clients */
	UFUNCTION(BlueprintCallable, Category = "PlayerState")
	ENGINE_API class APlayerController* GetPlayerController() const;
	
	/** Called by Controller when its PlayerState is initially replicated. */
	ENGINE_API virtual void ClientInitialize(class AController* C);

	/**
	 * Receives ping updates for the client (both clientside and serverside), from the net driver
	 * NOTE: This updates much more frequently clientside, thus the clientside ping will often be different to what the server displays
	 */
	ENGINE_API virtual void UpdatePing(float InPing);

	/** Recalculates the replicated Ping value once per second (both clientside and serverside), based upon collected ping data */
	ENGINE_API virtual void RecalculateAvgPing();

	/**
	 * Returns true if should broadcast player welcome/left messages.
	 * Current conditions: must be a human player a network game 
	 */
	ENGINE_API virtual bool ShouldBroadCastWelcomeMessage(bool bExiting = false);

	/** set the player name to S */
	ENGINE_API virtual void SetPlayerName(const FString& S);

	/** set the player name to S locally, does not trigger net updates */
	ENGINE_API virtual void SetPlayerNameInternal(const FString& S);

	/** returns current player name */
	UFUNCTION(BlueprintPure, Category = PlayerState)
	ENGINE_API FString GetPlayerName() const;

	/** custom access to player name, called only when bUseCustomPlayerNames is set */
	ENGINE_API virtual FString GetPlayerNameCustom() const;

	/** returns previous player name */
	ENGINE_API virtual FString GetOldPlayerName() const;

	/** set the player name to S */
	ENGINE_API virtual void SetOldPlayerName(const FString& S);

	/** 
	 * Register a player with the online subsystem
	 * @param bWasFromInvite was this player invited directly
	 */
	ENGINE_API virtual void RegisterPlayerWithSession(bool bWasFromInvite);

	/** Unregister a player with the online subsystem */
	ENGINE_API virtual void UnregisterPlayerWithSession();

	/** Create duplicate PlayerState (for saving Inactive PlayerState)	*/
	ENGINE_API virtual class APlayerState* Duplicate();

	/** Called on the server when the owning player has disconnected, by default this method destroys this player state */
	ENGINE_API virtual void OnDeactivated();

	/** Called on the server when the owning player has reconnected and this player state is added to the active players array */
	ENGINE_API virtual void OnReactivated();

	/** called by seamless travel when initializing a player on the other side - copy properties to the new PlayerState that should persist */
	ENGINE_API virtual void SeamlessTravelTo(class APlayerState* NewPlayerState);

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

	/** calls OverrideWith and triggers OnOverrideWith for BP extension */
	ENGINE_API void DispatchOverrideWith(APlayerState* PlayerState);

	ENGINE_API void DispatchCopyProperties(APlayerState* PlayerState);

protected:

	ENGINE_API virtual void OverrideWith(APlayerState* PlayerState);

	/** Copy properties which need to be saved in inactive PlayerState */
	ENGINE_API virtual void CopyProperties(APlayerState* PlayerState);

	/*
	* Can be implemented in Blueprint Child to move more properties from old to new PlayerState when reconnecting
	*
	* @param OldPlayerState		Old PlayerState, which we use to fill the new one with
	*/
	UFUNCTION(BlueprintImplementableEvent, Category = PlayerState, meta = (DisplayName = "OverrideWith"))
	ENGINE_API void ReceiveOverrideWith(APlayerState* OldPlayerState);

	/*
	* Can be implemented in Blueprint Child to move more properties from old to new PlayerState when traveling to a new level
	*
	* @param NewPlayerState		New PlayerState, which we fill with the current properties
	*/
	UFUNCTION(BlueprintImplementableEvent, Category = PlayerState, meta = (DisplayName = "CopyProperties"))
	ENGINE_API void ReceiveCopyProperties(APlayerState* NewPlayerState);

	/** Sets whether or not the replicated ping value is updated automatically. */
	void SetShouldUpdateReplicatedPing(bool bInShouldUpdateReplicatedPing) { bShouldUpdateReplicatedPing = bInShouldUpdateReplicatedPing; }

	/** called after receiving player name */
	ENGINE_API virtual void HandleWelcomeMessage();

private:
	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();

	//~ Begin Methods for Replicated Members.
public:

	/** Gets the literal value of Score. */
	UFUNCTION(BlueprintGetter)
	float GetScore() const
	{
		return Score;
	}

	/** Sets the value of Score without causing other side effects to this instance. */
	ENGINE_API void SetScore(const float NewScore);

	/** Gets the literal value of PlayerId. */
	UFUNCTION(BlueprintGetter)
	int32 GetPlayerId() const
	{
		return PlayerId;
	}

	/** Sets the value of PlayerId without causing other side effects to this instance. */
	ENGINE_API void SetPlayerId(const int32 NewId);

	/** Gets the literal value of the compressed Ping value (Ping = PingInMS / 4). */
	UFUNCTION(BlueprintGetter)
	uint8 GetCompressedPing() const
	{
		return CompressedPing;
	}

	/** Sets the value of CompressedPing without causing other side effects to this instance. */
	ENGINE_API void SetCompressedPing(const uint8 NewPing);

	/**
	 * Returns the ping (in milliseconds)
	 *
	 * Returns ExactPing if available (local players or when running on the server), and
	 * the replicated CompressedPing (converted back to milliseconds) otherwise.
	 * 
	 * Note that replication of CompressedPing is controlled by bShouldUpdateReplicatedPing,
	 * and if disabled then this will return 0 or a stale value on clients for player states
	 * that aren't related to local players
	 */
	UFUNCTION(BlueprintCallable, Category = "PlayerState")
	ENGINE_API float GetPingInMilliseconds() const;

	/** Gets the literal value of bIsSpectator. */
	UFUNCTION(BlueprintGetter)
	bool IsSpectator() const
	{
		return bIsSpectator;
	}

	/** Sets the value of bIsSpectator without causing other side effects to this instance. */
	ENGINE_API void SetIsSpectator(const bool bNewSpectator);

	/** Gets the literal value of bOnlySpectator. */
	UFUNCTION(BlueprintCallable, Category = "PlayerState")
	bool IsOnlyASpectator() const
	{
		return bOnlySpectator;
	}

	/** Sets the value of bOnlySpectator without causing other side effects to this instance. */
	ENGINE_API void SetIsOnlyASpectator(const bool bNewSpectator);

	/** Gets the literal value of bIsABot. */
	UFUNCTION(BlueprintGetter)
	bool IsABot() const
	{
		return bIsABot;
	}

	/** Sets the value of bIsABot without causing other side effects to this instance. */
	ENGINE_API void SetIsABot(const bool bNewIsABot);

	/** Gets the literal value of bIsInactive. */
	bool IsInactive() const
	{
		return bIsInactive;
	}

	/** Sets the value of bIsInactive without causing other side effects to this instance. */
	ENGINE_API void SetIsInactive(const bool bNewInactive);

	/** Gets the literal value of bFromPreviousLevel. */
	bool IsFromPreviousLevel() const
	{
		return bFromPreviousLevel;
	}

	/** Sets the value of bFromPreviousLevel without causing other side effects to this instance. */
	ENGINE_API void SetIsFromPreviousLevel(const bool bNewFromPreviousLevel);

	/** Gets the literal value of StartTime. */
	int32 GetStartTime() const
	{
		return StartTime;
	}

	/** Sets the value of StartTime without causing other side effects to this instance. */
	ENGINE_API void SetStartTime(const int32 NewStartTime);

	/** Gets the literal value of UniqueId. */
	const FUniqueNetIdRepl& GetUniqueId() const
	{
		return UniqueId;
	}

	/** Gets the online unique id for a player. If a player is logged in this will be consistent across all clients and servers. */
	UFUNCTION(BlueprintCallable, Category = "PlayerState", meta = (DisplayName = "Get Unique Net Id"))
	ENGINE_API FUniqueNetIdRepl BP_GetUniqueId() const;

	/**
	 * Associate an online unique id with this player
	 * @param InUniqueId the unique id associated with this player
	 */
	ENGINE_API void SetUniqueId(const FUniqueNetIdRepl& NewUniqueId);
	
	/**
	 * Associate an online unique id with this player
	 * @param InUniqueId the unique id associated with this player
	 */
	ENGINE_API void SetUniqueId(FUniqueNetIdRepl&& NewUniqueId);

	/** Called on both the client and server when unique ID has been modified */
	ENGINE_API virtual void OnSetUniqueId();

	//~ End Methods for Replicated Members.
};

struct FSetPlayerStatePawn
{
private:

	friend APawn;

	FSetPlayerStatePawn(APlayerState* PlayerState, APawn* Pawn)
	{
		APawn* OldPawn = PlayerState->PawnPrivate;
		PlayerState->SetPawnPrivate(Pawn);
		PlayerState->OnPawnSet.Broadcast(PlayerState, PlayerState->PawnPrivate, OldPawn); 
	}
};
