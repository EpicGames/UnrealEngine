// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PrimitiveComponentUtilities.generated.h"

#define UE_API ENGINE_API

UCLASS(meta=(BlueprintThreadSafe, ScriptName="PrimitiveComponentUtilities"), MinimalAPI)
class UPrimitiveComponentUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
public:
	/** Adds an additional "owner" for visibility purposes, like OwnerNoSee or OnlyOwnerSee. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API void AddVisibilityOwner(UPrimitiveComponent* Component, AActor* Owner);

	/** Adds an additional "owner" for visibility purposes, like OwnerNoSee or OnlyOwnerSee. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API void AddVisibilityOwnerActor(AActor* Actor, AActor* Owner, bool bRecursivelyIncludeAttachedActors);

	/** Removes an actor previously added by AddVisibilityOwner. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API void RemoveVisibilityOwner(UPrimitiveComponent* Component, const AActor* Owner);

	/** Removes an actor previously added by AddVisibilityOwner. */
 	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API void RemoveVisibilityOwnerActor(AActor* Actor, AActor* Owner, bool bRecursivelyIncludeAttachedActors);

	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API  void ClearVisibilityOwners(UPrimitiveComponent* Component);
 
	/** Changes the value of bOnlyOwnerSee on each primitive component on the actor. */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static UE_API void SetOnlyVisibleToOwner(AActor* Actor, bool bNewOnlyOwnerSee, bool bRecursivelyIncludeAttachedActors);

	static UE_API TArray<const AActor*> GetVisibilityOwners(const UPrimitiveComponent* Component);
 
	static UE_API void ForEachVisibilityOwner(const UPrimitiveComponent* Component, TFunctionRef<void(TObjectPtr<const AActor>)> Func);

	static UE_API void ForEachActorPrimitiveComponent(AActor* Actor, TFunctionRef<void(TObjectPtr<UPrimitiveComponent>)> Func, bool bRecursivelyIncludeAttachedActors);
};

#undef UE_API