// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/MovementComponent.h"
#include "UObject/ObjectMacros.h"
#include "ProjectileMovementComponent.generated.h"

struct FHitResult;

/**
 * ProjectileMovementComponent updates the position of another component during its tick.
 *
 * Behavior such as bouncing after impacts and homing toward a target are supported.
 *
 * Normally the root component of the owning actor is moved, however another component may be selected (see SetUpdatedComponent()).
 * If the updated component is simulating physics, only the initial launch parameters (when initial velocity is non-zero)
 * will affect the projectile, and the physics sim will take over from there.
 *
 * @see UMovementComponent
 */
UCLASS(ClassGroup=Movement, meta=(BlueprintSpawnableComponent), ShowCategories=(Velocity), MinimalAPI)
class UProjectileMovementComponent : public UMovementComponent
{
	GENERATED_UCLASS_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FOnProjectileBounceDelegate, const FHitResult&, ImpactResult, const FVector&, ImpactVelocity );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FOnProjectileStopDelegate, const FHitResult&, ImpactResult );

	/** Initial speed of projectile. If greater than zero, this will override the initial Velocity value and instead treat Velocity as a direction. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile)
	float InitialSpeed;

	/** Limit on speed of projectile (0 means no limit). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile)
	float MaxSpeed;

	/** If true, this projectile will have its rotation updated each frame to match the direction of its velocity. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile)
	uint8 bRotationFollowsVelocity:1;

	/** If true, this projectile will have its rotation updated each frame to maintain the rotations Yaw only. (bRotationFollowsVelocity is required to be true) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile, meta = (EditCondition = "bRotationFollowsVelocity"))
	uint8 bRotationRemainsVertical:1;

	/** If true, simple bounces will be simulated. Set this to false to stop simulating on contact. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces)
	uint8 bShouldBounce:1;

	/**
	 * If true, the initial Velocity is interpreted as being in local space upon startup.
	 * @see SetVelocityInLocalSpace()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile)
	uint8 bInitialVelocityInLocalSpace:1;

	/**
	 * If true, forces sub-stepping to break up movement into discrete smaller steps to improve accuracy of the trajectory.
	 * Objects that move in a straight line typically do *not* need to set this, as movement always uses continuous collision detection (sweeps) so collision is not missed.
	 * Sub-stepping is automatically enabled when under the effects of gravity or when homing towards a target.
	 * @see MaxSimulationTimeStep, MaxSimulationIterations
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileSimulation)
	uint8 bForceSubStepping:1;

	/**
	 * If true, does normal simulation ticking and update. If false, simulation is halted, but component will still tick (allowing interpolation to run).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileSimulation)
	uint8 bSimulationEnabled:1;

	/**
	 * If true, movement uses swept collision checks.
	 * If false, collision effectively teleports to the destination. Note that when this is disabled, movement will never generate blocking collision hits (though overlaps will be updated).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileSimulation)
	uint8 bSweepCollision:1;

	/**
	 * If true, we will accelerate toward our homing target. HomingTargetComponent must be set after the projectile is spawned.
	 * @see HomingTargetComponent, HomingAccelerationMagnitude
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Homing)
	uint8 bIsHomingProjectile:1;

	/**
	 * Controls the effects of friction on velocity parallel to the impact surface when bouncing.
	 * If true, friction will be modified based on the angle of impact, making friction higher for perpendicular impacts and lower for glancing impacts.
	 * If false, a bounce will retain a proportion of tangential velocity equal to (1.0 - Friction), acting as a "horizontal restitution".
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces)
	uint8 bBounceAngleAffectsFriction:1;

	/**
	 * If true, projectile is sliding / rolling along a surface.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category=ProjectileBounces)
	uint8 bIsSliding:1;

	/**
	 * If true and there is an interpolated component set, location (and optionally rotation) interpolation is enabled which allows the interpolated object to smooth uneven updates
	 * of the UpdatedComponent's location (usually to smooth network updates). This requires using SetInterpolatedComponent() to indicate the visual component that lags behind the collision,
	 * and using MoveInterpolationTarget() when the new target location/rotation is received (usually on a net update).
	 * @see SetInterpolatedComponent(), MoveInterpolationTarget()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation)
	uint8 bInterpMovement:1;

	/**
	 * If true and there is an interpolated component set, rotation interpolation is enabled which allows the interpolated object to smooth uneven updates
	 * of the UpdatedComponent's rotation (usually to smooth network updates).
	 * Rotation interpolation is *only* applied if bInterpMovement is also enabled.
	 * @see SetInterpolatedComponent(), MoveInterpolationTarget()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation)
	uint8 bInterpRotation:1;

	/**
	 * If true, throttle interpolation when not relevant.
	 * @see ThrottleInterpolationSkipFramesNotRecent, ThrottleInterpolationSkipFramesRecent, ThrottleInterpolationThresholdNotRenderedShortTime, ThrottleInterpolationThresholdNotRenderedLongTime
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation)
	uint8 bThrottleInterpolation:1;

protected:
	/**
	 * True if interpolation has reached the target, false if there is more interpolation required.
	 */
	uint8 bInterpolationComplete:1;

	/**
	 * Tracks the number of frames since the last interpolation.
	 */
	int32 ThrottleInterpolationFramesSinceInterp;

public:

	/**
	 * If true, uses FScopedMovementUpdate to avoid moving the UpdatedComponent more than once during a tick during simulation.
	 * This also defers overlap updates and some impact events until after the simulation update completes, so it may delay important events and continue deflection, so use with caution.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=ProjectileInterpolation)
	uint8 bSimulationUseScopedMovement:1;

	/**
	 * If true, uses FScopedMovementUpdate to avoid moving the attached interpolated object's children more than once during a tick when it would both interpolate and move during projectile simulation.
	 * This also defers overlap updates for the interpolated object until after the simulation update completes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=ProjectileInterpolation)
	uint8 bInterpolationUseScopedMovement:1;

	/** Saved HitResult Time (0 to 1) from previous simulation step. Equal to 1.0 when there was no impact. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category=ProjectileBounces)
	float PreviousHitTime;

	/** Saved HitResult Normal from previous simulation step that resulted in an impact. If PreviousHitTime is 1.0, then the hit was not in the last step. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category=ProjectileBounces)
	FVector PreviousHitNormal;

	/** Custom gravity scale for this projectile. Set to 0 for no gravity. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Projectile)
	float ProjectileGravityScale;

	/** Buoyancy of UpdatedComponent in fluid. 0.0=sinks as fast as in air, 1.0=neutral buoyancy */
	UPROPERTY()
	float Buoyancy;

	/**
	 * Percentage of velocity maintained after the bounce in the direction of the normal of impact (coefficient of restitution).
	 * 1.0 = no velocity lost, 0.0 = no bounce. Ignored if bShouldBounce is false.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces, meta=(ClampMin="0", UIMin="0"))
	float Bounciness;

	/**
	 * Coefficient of friction, affecting the resistance to sliding along a surface.
	 * Normal range is [0,1] : 0.0 = no friction, 1.0+ = very high friction.
	 * Also affects the percentage of velocity maintained after the bounce in the direction tangent to the normal of impact.
	 * Ignored if bShouldBounce is false.
	 * @see bBounceAngleAffectsFriction
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces, meta=(ClampMin="0", UIMin="0"))
	float Friction;

	/**
	 * If velocity is below this threshold after a bounce, stops simulating and triggers the OnProjectileStop event.
	 * Ignored if bShouldBounce is false, in which case the projectile stops simulating on the first impact.
	 * @see StopSimulating(), OnProjectileStop
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces)
	float BounceVelocityStopSimulatingThreshold;

	/**
	 * When bounce angle affects friction, apply at least this fraction of normal friction.
	 * Helps consistently slow objects sliding or rolling along surfaces or in valleys when the usual friction amount would take a very long time to settle.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileBounces, AdvancedDisplay, meta=(ClampMin="0", UIMin="0", ClampMax="1", UIMax="1"))
	float MinFrictionFraction;

	/**
	 * Returns true if velocity magnitude is less than BounceVelocityStopSimulatingThreshold.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement")
	bool IsVelocityUnderSimulationThreshold() const { return Velocity.SizeSquared() < FMath::Square(BounceVelocityStopSimulatingThreshold); }

	/** Called when projectile impacts something and bounces are enabled. */
	UPROPERTY(BlueprintAssignable)
	FOnProjectileBounceDelegate OnProjectileBounce;

	/** Called when projectile has come to a stop (velocity is below simulation threshold, bounces are disabled, or it is forcibly stopped). */
	UPROPERTY(BlueprintAssignable)
	FOnProjectileStopDelegate OnProjectileStop;

	/** The magnitude of our acceleration towards the homing target. Overall velocity magnitude will still be limited by MaxSpeed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Homing)
	float HomingAccelerationMagnitude;

	/**
	 * The current target we are homing towards. Can only be set at runtime (when projectile is spawned or updating).
	 * @see bIsHomingProjectile
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category=Homing)
	TWeakObjectPtr<USceneComponent> HomingTargetComponent;

	/** Sets the velocity to the new value, rotated into Actor space. */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement")
	ENGINE_API virtual void SetVelocityInLocalSpace(FVector NewVelocity);

	//Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	ENGINE_API virtual void PostLoad() override;
	//End UActorComponent Interface

	//Begin UMovementComponent Interface
	virtual float GetMaxSpeed() const override { return MaxSpeed; }
	ENGINE_API virtual void InitializeComponent() override;
	ENGINE_API virtual void UpdateTickRegistration() override;
	//End UMovementComponent Interface

	/**
	 * This will check to see if the projectile is still in the world.  It will check things like
	 * the KillZ, outside world bounds, etc. and handle the situation.
	 */
	ENGINE_API virtual bool CheckStillInWorld();

	/** @return Buoyancy of UpdatedComponent in fluid.  0.0=sinks as fast as in air, 1.0=neutral buoyancy*/
	float GetBuoyancy() const { return Buoyancy; };	

	bool ShouldApplyGravity() const { return ProjectileGravityScale != 0.f; }

	/**
	 * Given an initial velocity and a time step, compute a new velocity.
	 * Default implementation applies the result of ComputeAcceleration() to velocity.
	 * 
	 * @param  InitialVelocity Initial velocity.
	 * @param  DeltaTime Time step of the update.
	 * @return Velocity after DeltaTime time step.
	 */
	ENGINE_API virtual FVector ComputeVelocity(FVector InitialVelocity, float DeltaTime) const;

	/** Clears the reference to UpdatedComponent, fires stop event (OnProjectileStop), and stops ticking (if bAutoUpdateTickRegistration is true). */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement")
	ENGINE_API virtual void StopSimulating(const FHitResult& HitResult);

	bool HasStoppedSimulation() { return (UpdatedComponent == nullptr) || (IsActive() == false); }

	/**
	 * Compute remaining time step given remaining time and current iterations.
	 * The last iteration (limited by MaxSimulationIterations) always returns the remaining time, which may violate MaxSimulationTimeStep.
	 *
	 * @param RemainingTime		Remaining time in the tick.
	 * @param Iterations		Current iteration of the tick (starting at 1).
	 * @return The remaining time step to use for the next sub-step of iteration.
	 * @see MaxSimulationTimeStep, MaxSimulationIterations
	 * @see ShouldUseSubStepping()
	 */
	ENGINE_API float GetSimulationTimeStep(float RemainingTime, int32 Iterations) const;

	/**
	 * Determine whether or not to use substepping in the projectile motion update.
	 * If true, GetSimulationTimeStep() will be used to time-slice the update. If false, all remaining time will be used during the tick.
	 * @return Whether or not to use substepping in the projectile motion update.
	 * @see GetSimulationTimeStep()
	 */
	ENGINE_API virtual bool ShouldUseSubStepping() const;

	/**
	 * Max time delta for each discrete simulation step.
	 * Lowering this value can address precision issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationIterations, bForceSubStepping
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0.0166", ClampMax="0.50", UIMin="0.0166", UIMax="0.50"), Category=ProjectileSimulation)
	float MaxSimulationTimeStep;

	/**
	 * Max number of iterations used for each discrete simulation step.
	 * Increasing this value can address precision issues with fast-moving objects or complex collision scenarios, at the cost of performance.
	 *
	 * WARNING: if (MaxSimulationTimeStep * MaxSimulationIterations) is too low for the min framerate, the last simulation step may exceed MaxSimulationTimeStep to complete the simulation.
	 * @see MaxSimulationTimeStep, bForceSubStepping
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(ClampMin="1", ClampMax="25", UIMin="1", UIMax="25"), Category=ProjectileSimulation)
	int32 MaxSimulationIterations;

	/**
	 * On the first few bounces (up to this amount), allow extra iterations over MaxSimulationIterations if necessary.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ClampMax="4", UIMin="0", UIMax="4"), Category=ProjectileSimulation)
	int32 BounceAdditionalIterations;

	/**
	 * Assigns the component that will be used for network interpolation/smoothing. It is expected that this is a component attached somewhere below the UpdatedComponent.
	 * When network updates use MoveInterpolationTarget() to move the UpdatedComponent, the interpolated component's relative offset will be maintained and smoothed over
	 * the course of future component ticks. The current relative location and rotation of the component is saved as the target offset for future interpolation.
	 * @see MoveInterpolationTarget(), bInterpMovement, bInterpRotation
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement|Interpolation")
	ENGINE_API virtual void SetInterpolatedComponent(USceneComponent* Component);
	
	/**
	 * Returns the component used for network interpolation.
	 */
	ENGINE_API USceneComponent* GetInterpolatedComponent() const;

	/**
	 * Moves the UpdatedComponent, which is also the interpolation target for the interpolated component. If there is not interpolated component, this simply moves UpdatedComponent.
	 * Use this typically from PostNetReceiveLocationAndRotation() or similar from an Actor.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement|Interpolation")
	ENGINE_API virtual void MoveInterpolationTarget(const FVector& NewLocation, const FRotator& NewRotation);

	/**
	 * Resets interpolation so that interpolated component snaps back to the initial location/rotation without any additional offsets.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement|Interpolation")
	ENGINE_API virtual void ResetInterpolation();

	/**
	 * "Time" over which most of the location interpolation occurs, when the UpdatedComponent (target) moves ahead of the interpolated component.
	 * Since the implementation uses exponential lagged smoothing, this is a rough time value and experimentation should inform a final result.
	 * A value of zero is effectively instantaneous interpolation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, meta=(ClampMin="0"))
	float InterpLocationTime;

	/**
	 * "Time" over which most of the rotation interpolation occurs, when the UpdatedComponent (target) moves ahead of the interpolated component.
	 * Since the implementation uses exponential lagged smoothing, this is a rough time value and experimentation should inform a final result.
	 * A value of zero is effectively instantaneous interpolation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, meta=(ClampMin="0"))
	float InterpRotationTime;

	/**
	 * Max distance behind UpdatedComponent which the interpolated component is allowed to lag.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, meta=(ClampMin="0"))
	float InterpLocationMaxLagDistance;

	/**
	 * Max distance behind UpdatedComponent beyond which the interpolated component is snapped to the target location instead.
	 * For instance if the target teleports this far beyond the interpolated component, the interpolation is snapped to match the target.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, meta=(ClampMin="0"))
	float InterpLocationSnapToTargetDistance;

	/**
	 * Time after not rendered recently when we consider throttling interpolation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, AdvancedDisplay)
	float ThrottleInterpolationThresholdNotRenderedShortTime;

	/**
	 * Time after not rendered for a long time when we consider throttling interpolation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, AdvancedDisplay)
	float ThrottleInterpolationThresholdNotRenderedLongTime;

	/**
	 * When recently relevant, skip this many frames of interpolation if throttling is enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, AdvancedDisplay, meta=(ClampMin="0", UIMin="0"))
	int32 ThrottleInterpolationSkipFramesRecent;

	/**
	 * When not recently relevant, skip this many frames of interpolation if throttling is enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ProjectileInterpolation, AdvancedDisplay, meta=(ClampMin="0", UIMin="0"))
	int32 ThrottleInterpolationSkipFramesNotRecent;

	/**
	 * Returns whether interpolation is complete because the target has been reached. True when interpolation is disabled.
	 */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement|Interpolation")
	bool IsInterpolationComplete() const { return bInterpolationComplete || !bInterpMovement; }

protected:

	/**
	 * Update interpolation throttling for this frame. Uses ComputeThrottleInterpolationMaxFrames() to determine the number of frames to allow to be skipped.
	 * @return true if throttled this frame, false if there should be an update.
	 */
	ENGINE_API bool UpdateThrottleInterpolation(float DeltaTime, USceneComponent* InterpComponent);

	/**
	 * Determine the number of frames to allow to skip when interpolating. Returning 0 means not to allow throttling.
	 * Default implementation chooses between ThrottleInterpolationThresholdNotRenderedShortTime and ThrottleInterpolationThresholdNotRenderedLongTime based on WasRecentlyRendered(),
	 * extend or override this for custom behavior.
	 */
	ENGINE_API virtual int32 ComputeThrottleInterpolationMaxFrames(float DeltaTime, USceneComponent* InterpComponent);
	
	/**
	 * Custom hook to reset throttle interpolation tracking when it was throttling previously.
	 */
	ENGINE_API virtual void ResetThrottleInterpolation(float DeltaTime);

	// Enum indicating how simulation should proceed after HandleBlockingHit() is called.
	enum class EHandleBlockingHitResult
	{
		Deflect,				/** Assume velocity has been deflected, and trigger HandleDeflection(). This is the default return value of HandleBlockingHit(). */
		AdvanceNextSubstep,		/** Advance to the next simulation update. Typically used when additional slide/multi-bounce logic can be ignored,
								    such as when an object that blocked the projectile is destroyed and movement should continue. */
		Abort,					/** Abort all further simulation. Typically used when components have been invalidated or simulation should stop. */
	};

	/**
	 * Handle blocking hit during simulation update. Checks that simulation remains valid after collision.
	 * If simulating then calls HandleImpact(), and returns EHandleHitWallResult::Deflect by default to enable multi-bounce and sliding support through HandleDeflection().
	 * If no longer simulating then returns EHandleHitWallResult::Abort, which aborts attempts at further simulation.
	 *
	 * @param  Hit						Blocking hit that occurred.
	 * @param  TimeTick					Time delta of last move that resulted in the blocking hit.
	 * @param  MoveDelta				Movement delta for the current sub-step.
	 * @param  SubTickTimeRemaining		How much time to continue simulating in the current sub-step, which may change as a result of this function.
	 *									Initial default value is: TimeTick * (1.f - Hit.Time)
	 * @return Result indicating how simulation should proceed.
	 * @see EHandleHitWallResult, HandleImpact()
	 */
	 ENGINE_API virtual EHandleBlockingHitResult HandleBlockingHit(const FHitResult& Hit, float TimeTick, const FVector& MoveDelta, float& SubTickTimeRemaining);

	/**
	 * Applies bounce logic if enabled to affect velocity upon impact (using ComputeBounceResult()),
	 * or stops the projectile if bounces are not enabled or velocity is below BounceVelocityStopSimulatingThreshold.
	 * Triggers applicable events (OnProjectileBounce).
	 */
	ENGINE_API virtual void HandleImpact(const FHitResult& Hit, float TimeSlice=0.f, const FVector& MoveDelta = FVector::ZeroVector) override;

	/**
	 * Handle a blocking hit after HandleBlockingHit() returns a result indicating that deflection occured.
	 * Default implementation increments NumBounces, checks conditions that could indicate a slide, and calls HandleSliding() if necessary.
	 * 
	 * @param  Hit					Blocking hit that occurred. May be changed to indicate the last hit result of further movement.
	 * @param  OldVelocity			Velocity at the start of the simulation update sub-step. Current Velocity may differ (as a result of a bounce).
	 * @param  NumBounces			Number of bounces that have occurred thus far in the tick.
	 * @param  SubTickTimeRemaining	Time remaining in the simulation sub-step. May be changed to indicate change to remaining time.
	 * @return True if simulation of the projectile should continue, false otherwise.
	 * @see HandleSliding()
	 */
	 ENGINE_API virtual bool HandleDeflection(FHitResult& Hit, const FVector& OldVelocity, const uint32 NumBounces, float& SubTickTimeRemaining);

	/**
	 * Handle case where projectile is sliding along a surface.
	 * Velocity will be parallel to the impact surface upon entry to this method.
	 * 
	 * @param  InitialHit				Hit result of impact causing slide. May be modified by this function to reflect any subsequent movement.
	 * @param  SubTickTimeRemaining		Time remaining in the tick. This function may update this time with any reduction to the simulation time requested.
	 * @return True if simulation of the projectile should continue, false otherwise.
	 */
	ENGINE_API virtual bool HandleSliding(FHitResult& Hit, float& SubTickTimeRemaining);

	/** Computes result of a bounce and returns the new velocity. */
	ENGINE_API virtual FVector ComputeBounceResult(const FHitResult& Hit, float TimeSlice, const FVector& MoveDelta);

public:

	/** Don't allow velocity magnitude to exceed MaxSpeed, if MaxSpeed is non-zero. */
	UFUNCTION(BlueprintCallable, Category="Game|Components|ProjectileMovement")
	ENGINE_API FVector LimitVelocity(FVector NewVelocity) const;

	/** Compute the distance we should move in the given time, at a given a velocity. */
	ENGINE_API virtual FVector ComputeMoveDelta(const FVector& InVelocity, float DeltaTime) const;

	/** Compute the acceleration that will be applied */
	ENGINE_API virtual FVector ComputeAcceleration(const FVector& InVelocity, float DeltaTime) const;

	/** Allow the projectile to track towards its homing target. */
	ENGINE_API virtual FVector ComputeHomingAcceleration(const FVector& InVelocity, float DeltaTime) const;

	/** Adds a force which is accumulated until next tick, used by ComputeAcceleration() to affect Velocity. */
	ENGINE_API void AddForce(FVector Force);

	/** Returns the sum of pending forces from AddForce(). */
	FVector GetPendingForce() const { return PendingForce; }

	/** Clears any pending forces from AddForce(). If bClearImmediateForce is true, clears any force being processed during this update as well. */
	ENGINE_API void ClearPendingForce(bool bClearImmediateForce = false);
	
protected:

	// Double-buffer of pending force so that updates can use the accumulated value and reset the data so other AddForce() calls work correctly.
	// Also prevents accumulation over frames where the update aborts for whatever reason, and works with substepping movement.
	FVector PendingForceThisUpdate;

	ENGINE_API virtual void TickInterpolation(float DeltaTime);
	
	FVector InterpLocationOffset;
	FVector InterpInitialLocationOffset;
	TWeakObjectPtr<USceneComponent> InterpolatedComponentPtr;
	FQuat InterpRotationOffset;
	FQuat InterpInitialRotationOffset;

private:

	// Pending force for next tick.
	FVector PendingForce;

public:
	/** Compute gravity effect given current physics volume, projectile gravity scale, etc. */
	ENGINE_API virtual float GetGravityZ() const override;

protected:
	/** Minimum delta time considered when ticking. Delta times below this are not considered. This is a very small non-zero positive value to avoid potential divide-by-zero in simulation code. */
	static ENGINE_API const float MIN_TICK_TIME;
};



