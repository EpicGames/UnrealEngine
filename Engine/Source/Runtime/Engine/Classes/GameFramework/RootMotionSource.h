// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Engine/NetSerialization.h"
#include "Animation/AnimationAsset.h"
#include "RootMotionSource.generated.h"

class ACharacter;
class UCharacterMovementComponent;
class UCurveFloat;
class UCurveVector;

#define ROOT_MOTION_DEBUG (1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#if ROOT_MOTION_DEBUG
struct RootMotionSourceDebug
{
	static ENGINE_API TAutoConsoleVariable<int32> CVarDebugRootMotionSources;
	static ENGINE_API void PrintOnScreen(const ACharacter& InCharacter, const FString& InString);
	static ENGINE_API void PrintOnScreenServerMsg(const FString& InString);
};
#endif

/** 
 * Enumeration for RootMotionSource accumulation mode
 */
UENUM()
enum class ERootMotionAccumulateMode : uint8
{
	// Sets velocity to this value directly
	Override	= 0, 
	// Gets added on top of existing velocity
	Additive	= 1
};

/** 
 * Enumeration for RootMotionSource status flags
 */
UENUM()
enum class ERootMotionSourceStatusFlags : uint8
{
	// Source has been Prepared and can contribute to root motion 
	Prepared					= 0x01,
	// Source has timed out or otherwise reached a state where it is considered "done"
	Finished					= 0x02,
	// Source has been queued for removal, will be removed next PrepareRootMotion() by the SourceGroup
	MarkedForRemoval			= 0x04
};

/** 
 * Enumeration for RootMotionSource settings
 */
UENUM()
enum class ERootMotionSourceSettingsFlags : uint8
{
	// Source will switch character to Falling mode with any "Z up" velocity added.
	// Use this for jump-like root motion. If not enabled, uses default jump impulse
	// detection (which keeps you stuck on ground in Walking fairly strongly)
	UseSensitiveLiftoffCheck	= 0x01,
	// If Duration of Source would end partway through the last tick it is active,
	// do not reduce SimulationTime. Disabling this is useful for sources that
	// are more about providing velocity (like jumps), vs. sources that need
	// the precision of partial ticks for say ending up at an exact location (MoveTo)
	DisablePartialEndTick		= 0x02,
	// Whether to ignore impact to Z when accumulating output to Velocity
	// Setting this flag on override sources provides the same behavior as
	// animation root motion
	IgnoreZAccumulate			= 0x04

};

enum class ERootMotionSourceID : uint16 { Invalid = 0 };

/** 
 * Structure for mapping RootMotionSource server IDs to those on this client
 */
struct FRootMotionServerToLocalIDMapping
{
	ENGINE_API FRootMotionServerToLocalIDMapping();

	uint16 ServerID; // ID of root motion source on server
	uint16 LocalID; // ID of root motion source on local client
	float TimeStamp; // Last time this ID mapping was updated/still valid

	// Given CurrentTimeStamp, returns whether this mapping is still valid (has expired yet)
	ENGINE_API bool IsStillValid(float CurrentTimeStamp);
};

/** 
 * Structure for RootMotionSource status flags 
 * (used for convenience instead of having to manually manipulate flag bitfields)
 */
USTRUCT()
struct FRootMotionSourceStatus
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint8 Flags;

	ENGINE_API FRootMotionSourceStatus();

	ENGINE_API void Clear();

	ENGINE_API void SetFlag(ERootMotionSourceStatusFlags Flag);
	ENGINE_API void UnSetFlag(ERootMotionSourceStatusFlags Flag);
	ENGINE_API bool HasFlag(ERootMotionSourceStatusFlags Flag) const;
};

/** 
 * Structure for RootMotionSource option flags 
 * (used for convenience instead of having to manually manipulate flag bitfields)
 */
USTRUCT()
struct FRootMotionSourceSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint8 Flags;

	ENGINE_API FRootMotionSourceSettings();

	ENGINE_API void Clear();

	ENGINE_API void SetFlag(ERootMotionSourceSettingsFlags Flag);
	ENGINE_API void UnSetFlag(ERootMotionSourceSettingsFlags Flag);
	ENGINE_API bool HasFlag(ERootMotionSourceSettingsFlags Flag) const;

	// Accumulate settings with one another
	ENGINE_API FRootMotionSourceSettings& operator+=(const FRootMotionSourceSettings& Other);
};

UENUM()
enum class ERootMotionFinishVelocityMode : uint8
{
	// Maintain the last velocity root motion gave to the character
	MaintainLastRootMotionVelocity = 0,
	// Set Velocity to the specified value (for example, 0,0,0 to stop the character)
	SetVelocity,
	// Clamp velocity magnitude to the specified value. Note that it will not clamp Z if negative (falling). it will clamp Z positive though. 
	ClampVelocity,
};

/** 
 * Struct for RootMotion Finish Velocity options.
 */
USTRUCT()
struct FRootMotionFinishVelocitySettings
{
	GENERATED_USTRUCT_BODY()

	// Mode, see ERootMotionFinishVelocityMode
	UPROPERTY()
	ERootMotionFinishVelocityMode Mode;

	// Clamp Velocity if Move == ClampVelocity
	UPROPERTY()
	float ClampVelocity;

	// Set Velocity if Mode == SetVelocity
	UPROPERTY()
	FVector SetVelocity;

	FRootMotionFinishVelocitySettings()
		: Mode(ERootMotionFinishVelocityMode::MaintainLastRootMotionVelocity)
		, ClampVelocity(0.f)
		, SetVelocity(FVector::ZeroVector)
	{}
};

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FRootMotionSourceFixLayout
{
	virtual ~FRootMotionSourceFixLayout() = default;
};

/** 
*	Generalized source of Root Motion to a CharacterMovementComponent.
*	
*	Has priorities and different accumulation modes (override, additive, set, ignore).
*
*	RootMotionSources are accumulated from highest priority to lowest, once all values
*	are "finalized" (meaning no more accumulation to those values), root motion accumulation
*	is finished. Having all of translation and rotation in a transform being overridden during
*	root motion accumulation means that CharacterMovementComponent Phys*() functions no longer
*	need to CalcVelocity() since it'll just get overridden anyway.
*
*	Some example source possibilities:
*		Full Animation Root Motion (matching only form of root motion we've supported historically):
*			- Overrides all translation axes or rotation or both
*			- Very High priority so it gets applied first and since it is override no other
*			  RootMotionSources have an influence that are lower priority 
*			- Scale on translation and/or rotation is set to 1 to match animation 1 for 1
*			- Provides local space transform (character-orientation relative)
*
*			- End result: Same as before - overrides velocity/rotation to values in AnimMontage
*			- But can now scale individual translation axes if desired, or instead of override
*			  can use "set" to allow other additive sources of root motion to be applied on top
*			  of the animation root motion (modify velocity to home root motion melee attacks, etc.)
*
*		Jump pad volumes that work during animation root montage attacks
*			- Constant velocity modification Z up, additive and high priority
*			- Attack/melee animation root motion uses "set" instead of "override" to allow other sources
*			- Animation root motion is applied, then the Z up additive of this source is so you are
*			  affected by the jump pad
*
*		Attack/melee homing
*			- Additive, lower priority so it gets applied after any animation/significant root motion
*			- World space, sourced by logic modifying velocity to go towards nearby enemies
*			- Could point to a UAbilityTask node from the GameplayAbilities system
*
*		Character movement controlled by splines/curves in-editor instead of requiring animation tweaks
*			- Same as animation root motion but source from spline/curve data
*	
*/
USTRUCT()
struct FRootMotionSource
#if CPP
	: public FRootMotionSourceFixLayout
#endif
{
	GENERATED_USTRUCT_BODY()

	/** Priority of this source relative to other sources - higher number being the highest priority/first applied. */
	UPROPERTY()
	uint16 Priority;

	/** ID local to this client or server instance. Used for matching up FRootMotionSources between SavedMoves
	 *  and allow a mapping between server LocalIDs and client LocalIDs for correction/comparison */
	UPROPERTY()
	uint16 LocalID;

	/** 
	 *	This name allows us to find the source later so that we can end it. 
	 **/
	UPROPERTY()
	FName InstanceName;

	/** 
	 *  Time this source should start (in character movement client time) 
	 *  This is used to handle cases of inconsistent tick times (ServerMoves ticking for 1 second when root motion should
	 *  have only applied for the last 0.1 seconds, or root motion source ending halfway through a 0.5 second tick)
	 */
	UPROPERTY(NotReplicated)
	float StartTime;

	/** Time elapsed so far for this source */
	UPROPERTY()
	float CurrentTime;

	/** The last Time entry we had before the last SetTime() - used for simulated catchup */
	UPROPERTY(NotReplicated)
	float PreviousTime;

	/** The length of this root motion - < 0 for infinite (to be removed manually) */
	UPROPERTY()
	float Duration;

	/** Status of this source */
	UPROPERTY()
	FRootMotionSourceStatus Status;

	/** Settings of this source */
	UPROPERTY(NotReplicated)
	FRootMotionSourceSettings Settings;

	/** 
	*  Accumulation mode for this source (whether or not to additively apply this root motion or override completely)
	**/
	UPROPERTY()
	ERootMotionAccumulateMode AccumulateMode;

	/** True when this RootMotionSource is contributing local space accumulation (false for world space) */
	UPROPERTY()
	bool bInLocalSpace;

	/** 
	 *  True when this RootMotionSource has been marked for simulated catchup - this Simulated version
	 *  of the Source needs to catch up to where it was before being corrected by authoritative version.
	 */
	bool bNeedsSimulatedCatchup;

	/** 
	 *  True when this RootMotionSource is running on a SimulatedProxy and has marked itself as needing
	 *  location/rotation smoothing. This smoothing logic is what runs whenever the SimulatedProxy receives
	 *  a replication update from the server (SimulatedTick in CharacterMovementComponent, see bCorrectedToServer 
	 *  and SmoothCorrection), this flag allows the RootMotionSource to signal to the CMC that a change
	 *  was done significant enough that needs smoothing. 
	 */
	bool bSimulatedNeedsSmoothing;

	/** Root Motion generated by this Source */
	UPROPERTY(NotReplicated)
	FRootMotionMovementParams RootMotionParams;

	/** Finish Velocity Parameters */
	UPROPERTY(NotReplicated)
	FRootMotionFinishVelocitySettings FinishVelocityParams;

	ENGINE_API FRootMotionSource();

	virtual ~FRootMotionSource() = default;

	/** @return the CurrentTime - amount of time elapsed so far for this source */
	ENGINE_API float GetTime() const;

	/** @return the StartTime - time this source should start (in character movement client time)  */
	ENGINE_API float GetStartTime() const;

	/** @return whether the start time has been set */
	ENGINE_API bool IsStartTimeValid() const;

	/** @return the Duration - the length of this root motion - < 0 for infinite (to be removed manually) */
	ENGINE_API float GetDuration() const;

	/** @return whether this source will be removed when CurrentTime reaches Duration */
	ENGINE_API virtual bool IsTimeOutEnabled() const;

	/** @return newly allocated copy of this FRootMotionSource. Must be overridden by child classes. */
	ENGINE_API virtual FRootMotionSource* Clone() const;

	/** 
	 *  @return Whether this is the same RootMotionSource as Other. 
	 *  This is used for networking when clients receive RootMotionSource data from the server and need
	 *  to decide which local RootMotionSource to compare and apply the corrections to.
	 *  This is required due to RootMotionSources in general being added independently on server and
	 *  clients, not needing to know about each other by default.
	 *  
	 *  For well-networked RootMotionSources, any given FRootMotionSource child class could implement
	 *  their own unique ID and simply use that in the Matches check. This "heuristic-style" default
	 *  was chosen to simplify addition of new RootMotionSources, and assumes that in a networked setting
	 *  a given RootMotionSource won't be applied many times in a given frame by the same instigator
	 *  to the same target with the exact same parameters.
	 *
	 *  Guaranteed uniqueness would also require a strict application order ("RootMotionSources can only
	 *  be added on Authority") or a prediction-based setup ("Apply on Autonomous and Simulated predictively,
	 *  then apply on Authority and confirm, and if Authority doesn't confirm remove them"). We avoid
	 *  that synchronization complexity for now.
	 *
	 *  See UCharacterMovementComponent::ConvertRootMotionServerIDsToLocalIDs
	 *
	 *  Should be overridden by child classes, as default implementation only contains basic equivalency checks 
	 **/
	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const;

	/** Checks that it Matches() and has the same state (time, track position, etc.) */
	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const;

	/** 
	 *  Mainly for server correction purposes - update this Source's state from
	 *  another's, usually the authoritative state from the server's version of the Source
	 *  @param bMarkForSimulatedCatchup sets the source for needing to "catch up" to current state next Prepare
	 *  @return Whether the update was sucessful or not. Note that failure to update
	 *          denotes a complete failure, and the Source will then be marked for removal.
	 *          We need to remove since we don't have a way of reverting partial updates
	 *          depending on where the update failed.
	 **/
	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false);

	/** True when this RootMotionSource should be affecting root motion */
	ENGINE_API virtual bool IsActive() const;

	/** Set the CurrentTime of this source. Use this setter so that sources based on duration can get correctly marked for end */
	ENGINE_API virtual void SetTime(float NewTime);

	/** Checks if this source has timed out and marks for removal if necessary */
	ENGINE_API virtual void CheckTimeOut();

	/** 
	*	Generates the RootMotion for this Source, can be used for both "live" generation
	*	or for playback (client prediction correction, simulated proxies, etc.)
	*
	*	Examples:
	*		- Animation RootMotionSources use Time as track time into AnimMontage and
	*		  extract the root motion from AnimMontage chunk of time (Position,Position+DeltaTime)
	*		- ConstantForce source uses Time as the time into the application
	*		  so if its duration ends halfway through the frame it knows how much root
	*		  motion it should have applied
	*		- Spline/curve-based sources use Time for knowing where on spline/curve to extract
	*		  from
	*	@param SimulationTime How far forward in time to simulate root motion
	*	@param MovementTickTime How much time the movement is going to take that this is being prepared for
	*
	*/
	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime,
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		);

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const;

	ENGINE_API virtual FString ToSimpleString() const;

	virtual void AddReferencedObjects(class FReferenceCollector& Collector) {}
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource > : public TStructOpsTypeTraitsBase2< FRootMotionSource >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/** ConstantForce applies a fixed force to the target */
USTRUCT()
struct FRootMotionSource_ConstantForce : public FRootMotionSource
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSource_ConstantForce();

	virtual ~FRootMotionSource_ConstantForce() = default;

	UPROPERTY()
	FVector Force;

	UPROPERTY()
	TObjectPtr<UCurveFloat> StrengthOverTime;

	ENGINE_API virtual FRootMotionSource* Clone() const override;

	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false) override;

	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		) override;

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const override;

	ENGINE_API virtual FString ToSimpleString() const override;

	ENGINE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource_ConstantForce > : public TStructOpsTypeTraitsBase2< FRootMotionSource_ConstantForce >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};


/** RadialForce applies a force pulling or pushing away from a given world location to the target */
USTRUCT()
struct FRootMotionSource_RadialForce : public FRootMotionSource
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSource_RadialForce();

	virtual ~FRootMotionSource_RadialForce() = default;

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	TObjectPtr<AActor> LocationActor;

	UPROPERTY()
	float Radius;

	UPROPERTY()
	float Strength;

	UPROPERTY()
	bool bIsPush;

	UPROPERTY()
	bool bNoZForce;

	UPROPERTY()
	TObjectPtr<UCurveFloat> StrengthDistanceFalloff;

	UPROPERTY()
	TObjectPtr<UCurveFloat> StrengthOverTime;

	UPROPERTY()
	bool bUseFixedWorldDirection;

	UPROPERTY()
	FRotator FixedWorldDirection;

	ENGINE_API virtual FRootMotionSource* Clone() const override;

	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false) override;

	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		) override;

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const override;

	ENGINE_API virtual FString ToSimpleString() const override;

	ENGINE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource_RadialForce > : public TStructOpsTypeTraitsBase2< FRootMotionSource_RadialForce >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};


/** MoveToForce moves the target to a given fixed location in world space over the duration */
USTRUCT()
struct FRootMotionSource_MoveToForce : public FRootMotionSource
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSource_MoveToForce();

	virtual ~FRootMotionSource_MoveToForce() = default;

	UPROPERTY()
	FVector StartLocation;

	UPROPERTY()
	FVector TargetLocation;

	UPROPERTY()
	bool bRestrictSpeedToExpected;

	UPROPERTY()
	TObjectPtr<UCurveVector> PathOffsetCurve;

	ENGINE_API FVector GetPathOffsetInWorldSpace(const float MoveFraction) const;

	ENGINE_API virtual FRootMotionSource* Clone() const override;

	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false) override;

	ENGINE_API virtual void SetTime(float NewTime) override;

	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		) override;

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const override;

	ENGINE_API virtual FString ToSimpleString() const override;

	ENGINE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource_MoveToForce > : public TStructOpsTypeTraitsBase2< FRootMotionSource_MoveToForce >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};


/** 
 * MoveToDynamicForce moves the target to a given location in world space over the duration, where the end location
 * is dynamic and can change during the move (meant to be used for things like moving to a moving target)
 */
USTRUCT()
struct FRootMotionSource_MoveToDynamicForce : public FRootMotionSource
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSource_MoveToDynamicForce();

	virtual ~FRootMotionSource_MoveToDynamicForce() = default;

	UPROPERTY()
	FVector StartLocation;

	UPROPERTY()
	FVector InitialTargetLocation;

	// Dynamically-changing location of target, which may be altered while this movement is ongoing 
	UPROPERTY()
	FVector TargetLocation;

	UPROPERTY()
	bool bRestrictSpeedToExpected;

	UPROPERTY()
	TObjectPtr<UCurveVector> PathOffsetCurve;

	UPROPERTY()
	TObjectPtr<UCurveFloat> TimeMappingCurve;

	ENGINE_API void SetTargetLocation(FVector NewTargetLocation);

	ENGINE_API FVector GetPathOffsetInWorldSpace(const float MoveFraction) const;

	ENGINE_API virtual FRootMotionSource* Clone() const override;

	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false) override;

	ENGINE_API virtual void SetTime(float NewTime) override;

	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		) override;

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const override;

	ENGINE_API virtual FString ToSimpleString() const override;

	ENGINE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource_MoveToDynamicForce > : public TStructOpsTypeTraitsBase2< FRootMotionSource_MoveToDynamicForce >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};


/** JumpForce moves the target in a jump-like manner (ends when landing, applied force is relative) */
USTRUCT()
struct FRootMotionSource_JumpForce : public FRootMotionSource
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSource_JumpForce();

	virtual ~FRootMotionSource_JumpForce() = default;

	UPROPERTY()
	FRotator Rotation;

	UPROPERTY()
	float Distance;

	UPROPERTY()
	float Height;

	UPROPERTY()
	bool bDisableTimeout;

	UPROPERTY()
	TObjectPtr<UCurveVector> PathOffsetCurve;

	UPROPERTY()
	TObjectPtr<UCurveFloat> TimeMappingCurve;

	FVector SavedHalfwayLocation;

	ENGINE_API FVector GetPathOffset(float MoveFraction) const;

	ENGINE_API FVector GetRelativeLocation(float MoveFraction) const;

	ENGINE_API virtual bool IsTimeOutEnabled() const override;

	ENGINE_API virtual FRootMotionSource* Clone() const override;

	ENGINE_API virtual bool Matches(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool MatchesAndHasSameState(const FRootMotionSource* Other) const override;

	ENGINE_API virtual bool UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup = false) override;

	ENGINE_API virtual void PrepareRootMotion(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
		) override;

	ENGINE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	ENGINE_API virtual UScriptStruct* GetScriptStruct() const override;

	ENGINE_API virtual FString ToSimpleString() const override;

	ENGINE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FRootMotionSource_JumpForce > : public TStructOpsTypeTraitsBase2< FRootMotionSource_JumpForce >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/**
 *	Group of Root Motion Sources that are applied
 *
 **/
USTRUCT()
struct FRootMotionSourceGroup
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FRootMotionSourceGroup();

	virtual ~FRootMotionSourceGroup() = default;

	/** Root Motion Sources currently applied in this Group */
	TArray< TSharedPtr<FRootMotionSource> > RootMotionSources;

	/** Root Motion Sources to be added next frame */
	TArray< TSharedPtr<FRootMotionSource> > PendingAddRootMotionSources;

	/** 
	 *  Whether this group has additive root motion sources
	 **/
	UPROPERTY()
	uint8 bHasAdditiveSources:1;

	/** 
	 *  Whether this group has override root motion sources
	 **/
	UPROPERTY()
	uint8 bHasOverrideSources:1;

	/** 
	 *  Whether this group has override root motion sources that have IgnoreZAccumulate flag
	 **/
	UPROPERTY()
	uint8 bHasOverrideSourcesWithIgnoreZAccumulate:1;

	/** True when we had additive velocity applied last tick, checked to know if we should restore
	 *  LastPreAdditiveVelocity before a Velocity computation */
	UPROPERTY()
	uint8 bIsAdditiveVelocityApplied:1;

	/** Aggregate Settings of the last group of accumulated sources */
	UPROPERTY()
	FRootMotionSourceSettings LastAccumulatedSettings;

	/** Saved off pre-additive-applied Velocity, used for being able to reliably add/remove additive
	 *  velocity from currently computed Velocity (otherwise we would be removing additive velocity
	 *  that no longer exists, like if you run into a wall and your Velocity becomes 0 - subtracting
	 *  the velocity that we added heading into the wall last tick would make you go backwards. With
	 *  this method we override that resulting Velocity due to obstructions */
	UPROPERTY()
	FVector_NetQuantize10 LastPreAdditiveVelocity;

	ENGINE_API void CleanUpInvalidRootMotion(float DeltaTime, const ACharacter& Character, UCharacterMovementComponent& MoveComponent);

	/** 
	 *  Generates root motion by accumulating transforms through current root motion sources. 
	 *  @param bForcePrepareAll - Used during "live" PerformMovements() to ensure all sources get prepared
	 *                            Needed due to SavedMove playback/server correction only applying corrections to
	 *                            Sources that need updating, so in that case we only Prepare those that need it.
	 */
	ENGINE_API void PrepareRootMotion(float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& InMoveComponent, bool bForcePrepareAll = false);

	/**  Helper function for accumulating override velocity into InOutVelocity */
	ENGINE_API void AccumulateOverrideRootMotionVelocity(float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, FVector& InOutVelocity) const;

	/**  Helper function for accumulating additive velocity into InOutVelocity */
	ENGINE_API void AccumulateAdditiveRootMotionVelocity(float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, FVector& InOutVelocity) const;

	/** Get rotation output of current override root motion source, returns true if OutRotation was filled */
	ENGINE_API bool GetOverrideRootMotionRotation(float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, FQuat& OutRotation) const;

	/** Gets whether any active root motion source has been marked to need NetworkSmoothing as a SimulatedProxy */
	ENGINE_API bool NeedsSimulatedSmoothing() const;

	/** Sets the StartTime of all pending root motion sources to be at least this time, can be used on servers to match client-side start times */
	ENGINE_API void SetPendingRootMotionSourceMinStartTimes(float NewStartTime);

	/** Applies a reset to the start time for each root motion when the time stamp is reset */
	ENGINE_API void ApplyTimeStampReset(float DeltaTime);

	/** @return true if Velocity will be overridden by root motion sources, meaning we can skip all normal movement-based velocity calculations */
	ENGINE_API bool HasOverrideVelocity() const;

	/** @return true if Velocity will be overridden by root motion sources that do not affect Z velocity (matching animation root motion behavior) */
	ENGINE_API bool HasOverrideVelocityWithIgnoreZAccumulate() const;

	/** @return true if any axis of velocity has additive velocity applied by root motion sources */
	ENGINE_API bool HasAdditiveVelocity() const;

	/** @return true if any axis of velocity is modified by root motion sources */
	ENGINE_API bool HasVelocity() const;

	/** @return true if we have Root Motion from any source to use in PerformMovement() physics. */
	ENGINE_API bool HasActiveRootMotionSources() const;

	/** @return true if we have Root Motion accumulated from sources to use in PerformMovement() physics. 
		Not valid outside of the scope of that function. Since RootMotion is extracted and used in it. */
	ENGINE_API bool HasRootMotionToApply() const;

	/** Apply a RootMotionSource to this Group 
	 *  @return LocalID for this RMS */
	ENGINE_API uint16 ApplyRootMotionSource(TSharedPtr<FRootMotionSource> SourcePtr);

	/** Get a RootMotionSource from this Group by name */
	ENGINE_API TSharedPtr<FRootMotionSource> GetRootMotionSource(FName InstanceName);

	/** Get a RootMotionSource from this Group by ID */
	ENGINE_API TSharedPtr<FRootMotionSource> GetRootMotionSourceByID(uint16 RootMotionSourceID);

	/** Remove a RootMotionSource from this Group by name */
	ENGINE_API void RemoveRootMotionSource(FName InstanceName);

	/** Remove a RootMotionSource from this Group by ID */
	ENGINE_API void RemoveRootMotionSourceByID(uint16 RootMotionSourceID);

	/** 
	 *  Update contained Sources to state in matching sources from other group.
	 *  Used for correcting root motion state when receiving authoritative state from server.
	 *  @param GroupToTakeStateFrom the Authoritative Group to take state from
	 *  @param bMarkForSimulatedCatchup marks Sources as needing to return to their current Time on next Prepare
	 *  @return whether it successfully updated state
	 **/
	ENGINE_API void UpdateStateFrom(const FRootMotionSourceGroup& GroupToTakeStateFrom, bool bMarkForSimulatedCatchup = false);

	/** Serialize the root motion sources and their states for this group */
	ENGINE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess, uint8 MaxNumRootMotionSourcesToSerialize = MAX_uint8);

	/** Clear the contents to return it to "empty" */
	ENGINE_API void Clear();

	/** Removes any Sources without a valid ID */
	ENGINE_API void CullInvalidSources();

	/** Copy operator - deep copy so it can be used for archiving/saving off moves */
	ENGINE_API FRootMotionSourceGroup& operator=(const FRootMotionSourceGroup& Other);

	/** Comparison operator - needs matching Sources along with identical states in those sources */
	ENGINE_API bool operator==(const FRootMotionSourceGroup& Other) const;

	/** Comparison operator */
	ENGINE_API bool operator!=(const FRootMotionSourceGroup& Other) const;

	/** Exposes references to GC system */
	ENGINE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;

protected:

	/** Accumulates contributions for velocity into InOutVelocity for a given type of root motion from this group */
	ENGINE_API void AccumulateRootMotionVelocity(ERootMotionAccumulateMode RootMotionType, float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, FVector& InOutVelocity) const;

	/** Accumulates contributions for velocity into InOutVelocity for a given type of root motion from this group */
	ENGINE_API void AccumulateRootMotionVelocityFromSource(const FRootMotionSource& RootMotionSource, float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, FVector& InOutVelocity) const;

	/** Helper function for serializing array of root motion sources */
	static ENGINE_API void NetSerializeRMSArray(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess, TArray< TSharedPtr<FRootMotionSource> >& RootMotionSourceArray, uint8 MaxNumRootMotionSourcesToSerialize = MAX_uint8);

};

template<>
struct TStructOpsTypeTraits<FRootMotionSourceGroup> : public TStructOpsTypeTraitsBase2<FRootMotionSourceGroup>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FRootMotionSource> Data is copied around
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithAddStructReferencedObjects = true,
	};
};
