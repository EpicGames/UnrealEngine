// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/MovementComponent.h"
#include "RotatingMovementComponent.generated.h"

/**
 * Performs continuous rotation of a component at a specific rotation rate.
 * Rotation can optionally be offset around a pivot point.
 * Collision testing is not performed during movement.
 */
UCLASS(ClassGroup=Movement, meta=(BlueprintSpawnableComponent), HideCategories=(Velocity), MinimalAPI)
class URotatingMovementComponent : public UMovementComponent
{
	GENERATED_UCLASS_BODY()

	/** How fast to update roll/pitch/yaw of the component we update. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RotatingComponent)
	FRotator RotationRate;

	/**
	 * Translation of pivot point around which we rotate, relative to current rotation.
	 * For instance, with PivotTranslation set to (X=+100, Y=0, Z=0), rotation will occur
	 * around the point +100 units along the local X axis from the center of the object,
	 * rather than around the object's origin (the default).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RotatingComponent)
	FVector PivotTranslation;

	/** Whether rotation is applied in local or world space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RotatingComponent)
	uint32 bRotationInLocalSpace:1;

	//Begin UActorComponent Interface
	/** Applies rotation to UpdatedComponent. */
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//End UActorComponent Interface
};



