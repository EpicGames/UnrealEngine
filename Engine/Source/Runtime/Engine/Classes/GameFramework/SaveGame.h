// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/CoreMiscDefines.h"
#include "SaveGame.generated.h"

class ULocalPlayer;
class APlayerController;

/** 
 *	This class acts as a base class for a save game object that can be used to save state about the game. 
 *	When you create your own save game subclass, you would add member variables for the information that you want to save.
 *	Then when you want to save a game, create an instance of this object using CreateSaveGameObject, fill in the data, and use SaveGameToSlot, providing a slot name.
 *	To load the game you then just use LoadGameFromSlot, and read the data from the resulting object.
 *
 *	@see https://docs.unrealengine.com/latest/INT/Gameplay/SaveGame
 */
UCLASS(abstract, Blueprintable, BlueprintType, MinimalAPI)
class USaveGame : public UObject
{
	/**
	 *	@see UGameplayStatics::CreateSaveGameObject
	 *	@see UGameplayStatics::SaveGameToSlot
	 *	@see UGameplayStatics::DoesSaveGameExist
	 *	@see UGameplayStatics::LoadGameFromSlot
	 *	@see UGameplayStatics::DeleteGameInSlot
	 */

	GENERATED_BODY()
};


DECLARE_DYNAMIC_DELEGATE_OneParam(FOnLocalPlayerSaveGameLoaded, class ULocalPlayerSaveGame*, SaveGame);
DECLARE_DELEGATE_OneParam(FOnLocalPlayerSaveGameLoadedNative, class ULocalPlayerSaveGame*);

/**
 * Abstract subclass of USaveGame that provides utility functions that let you associate a Save Game object with a specific local player.
 * These objects can also be loaded using the functions on GameplayStatics, but you would need to call functions like InitializeSaveGame manually.
 * For simple games, it is fine to blueprint this class directly and add parameters and override functions in blueprint,
 * but for complicated games you will want to subclass this in native code and set up proper versioning.
 */
UCLASS(abstract, Blueprintable, BlueprintType, MinimalAPI)
class ULocalPlayerSaveGame : public USaveGame
{
	GENERATED_BODY()
public:

	/**
	 * Synchronously loads a save game object in the specified slot for the local player, stalling the main thread until it completes.
	 * This will return null for invalid parameters, but will create a new instance if the parameters are valid but loading fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	static ENGINE_API ULocalPlayerSaveGame* LoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, APlayerController* LocalPlayerController, const FString& SlotName);

	/**
	 * Native version of above function, this takes a ULocalPlayer because you can have a local player before a player controller.
	 * This will return null for invalid parameters, but will create a new instance if the parameters are valid but loading fails.
	 */
	static ENGINE_API ULocalPlayerSaveGame* LoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName);

	/**
	 * Asynchronously loads a save game object in the specified slot for the local player, if this returns true the delegate will get called later.
	 * False means the load was never scheduled, otherwise it will create and initialize a new instance before calling the delegate if loading failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	static ENGINE_API bool AsyncLoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, APlayerController* LocalPlayerController, const FString& SlotName, FOnLocalPlayerSaveGameLoaded Delegate);

	/**
	 * Native version of above function, this takes a ULocalPlayer and calls a native delegate.
	 * False means the load was never scheduled, otherwise it will create and initialize a new instance before calling the delegate if loading failed.
	 */
	static ENGINE_API bool AsyncLoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName, FOnLocalPlayerSaveGameLoadedNative Delegate);

	/**
	 * Create a brand new save game, possibly ready for saving but without trying to load from disk. This will always succeed.
	 */
	static ENGINE_API ULocalPlayerSaveGame* CreateNewSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName);

	/**
	 * Synchronously save using the slot and user index, stalling the main thread until it completes. 
	 * This will return true if the save was requested, and errors should be handled by the HandlePostSave function that will be called immediately.
	 */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool SaveGameToSlotForLocalPlayer();

	/**
	 * Asynchronously save to the slot and user index.
	 * This will return true if the save was requested, and errors should be handled by the HandlePostSave function after the save succeeds or fails
	 */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool AsyncSaveGameToSlotForLocalPlayer();


	/** Returns the local player controller this is associated with, this will be valid if it is ready to save */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual APlayerController* GetLocalPlayerController() const;

	/** Returns the local player this is associated with, this will be valid if it is ready to save */
	ENGINE_API virtual const ULocalPlayer* GetLocalPlayer() const;

	/** Associates this save game with a local player, this is called automatically during load/create */
	ENGINE_API virtual void SetLocalPlayer(const ULocalPlayer* LocalPlayer);

	/** Returns the platform user to save to, based on Local Player by default */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual FPlatformUserId GetPlatformUserId() const;

	/** Returns the user index to save to, based on Local Player by default */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual int32 GetPlatformUserIndex() const;

	/** Returns the save slot name to use */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual FString GetSaveSlotName() const;

	/** Sets the save slot name for any future saves */
	ENGINE_API virtual void SetSaveSlotName(const FString& SlotName);

	/** Returns the game-specific version number this was last saved/loaded as */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual int32 GetSavedDataVersion() const;

	/** Returns the invalid save data version, which means it has never been saved/loaded */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual int32 GetInvalidDataVersion() const;

	/** Returns the latest save data version, this is used when the new data is saved */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual int32 GetLatestDataVersion() const;

	/** Returns true if this was loaded from an existing save */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool WasLoaded() const;

	/** Returns true if a save is in progress */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool IsSaveInProgress() const;

	/** Returns true if it has been saved at least once and the last save was successful */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool WasLastSaveSuccessful() const;

	/** Returns true if a save was ever requested, may still be in progress */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual bool WasSaveRequested() const;


	/** Initializes this save after either loading or initial creation, automatically called by load/create functions above */
	ENGINE_API virtual void InitializeSaveGame(const ULocalPlayer* LocalPlayer, FString InSlotName, bool bWasLoaded);

	/** Resets all saved data to default values, called when the load fails or manually */
	UFUNCTION(BlueprintCallable, Category = "SaveGame|LocalPlayer")
	ENGINE_API virtual void ResetToDefault();

	/** Blueprint event called to reset all saved data to default, called when the load fails or manually */
	UFUNCTION(BlueprintImplementableEvent, Category = "SaveGame|LocalPlayer")
	ENGINE_API void OnResetToDefault();

	/** Called after loading, this is not called for newly created saves */
	ENGINE_API virtual void HandlePostLoad();

	/** Blueprint event called after loading, is not called for newly created saves */
	UFUNCTION(BlueprintImplementableEvent, Category = "SaveGame|LocalPlayer")
	ENGINE_API void OnPostLoad();

	/** Called before saving, do any game-specific fixup here */
	ENGINE_API virtual void HandlePreSave();

	/** Blueprint event called before saving, do any game-specific fixup here  */
	UFUNCTION(BlueprintImplementableEvent, Category = "SaveGame|LocalPlayer")
	ENGINE_API void OnPreSave();

	/** Called after saving finishes with success/failure result */
	ENGINE_API virtual void HandlePostSave(bool bSuccess);

	/** Blueprint event called after saving finishes with success/failure result */
	UFUNCTION(BlueprintImplementableEvent, Category = "SaveGame|LocalPlayer")
	ENGINE_API void OnPostSave(bool bSuccess);

protected:
	/** Internal callback that will call HandlePostSave */
	ENGINE_API virtual void ProcessSaveComplete(const FString& SlotName, const int32 UserIndex, bool bSuccess, int32 SaveRequest);

	/** Internal helper function used by both the sync and async save */
	static ENGINE_API ULocalPlayerSaveGame* ProcessLoadedSave(USaveGame* BaseSave, const FString& SlotName, const int32 UserIndex, TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer);

	/** The local player this is connected to, can be null if subclasses override Get/Set Local Player or it hasn't been initialized */
	UPROPERTY(Transient)
	TObjectPtr<const ULocalPlayer> OwningPlayer;

	/** The slot name this was loaded from and that will be used to save to in the future */
	UPROPERTY(Transient)
	FString SaveSlotName;

	/** 
	 * The value of GetLatestDataVersion when this was last saved.
	 * Subclasses can override GetLatestDataVersion and then handle fixups in HandlePostLoad.
	 * This defaults to 0 so old save games that didn't previously subclass ULocalPlayerSaveGame will have 0 instead of the invalid version.
	 */
	UPROPERTY()
	int32 SavedDataVersion = 0;

	/**
	 * The value of SavedDataVersion when a save was last loaded, this will be -1 for newly created saves
	 */
	UPROPERTY(Transient)
	int32 LoadedDataVersion = -1;


	/** Integer that is incremented every time a save has been requested in the current session, can be used to know if one is in progress */
	UPROPERTY(Transient)
	int32 CurrentSaveRequest = 0;

	/** Integer that is set when a save completes successfully, if this equals RequestedSaveCount then the last save was successful */
	UPROPERTY(Transient)
	int32 LastSuccessfulSaveRequest = 0;

	/** Integer that is set when a save fails */
	UPROPERTY(Transient)
	int32 LastErrorSaveRequest = 0;

};
