// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Simple reticle. A very simple crosshair reticle that can be draw on the HUD canvas. 
 *
 * @see AHUD
 */
class FSimpleReticle
{
public:
	FSimpleReticle()
	{
		SetupReticle( 8.0f, 20.0f );
	}

	void SetupReticle( const float Length, const float InnerSize )
	{
		HorizontalOffsetMin.Set( InnerSize, 0.0f );
		HorizontalOffsetMax.Set( InnerSize + Length, 0.0f );
		VerticalOffsetMin.Set( 0.0f, InnerSize);
		VerticalOffsetMax.Set( 0.0f, InnerSize + Length );
	}
	ENGINE_API void Draw( class UCanvas* InCanvas, FLinearColor InColor );
private:
	FVector2D HorizontalOffsetMin;
	FVector2D HorizontalOffsetMax;
	FVector2D VerticalOffsetMin;
	FVector2D VerticalOffsetMax;	
};
