// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * SpectatorPawns are simple pawns that can fly around the world, used by
 * PlayerControllers when in the spectator state.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/DefaultPawn.h"
#include "SpectatorPawn.generated.h"

UCLASS(config=Game, Blueprintable, BlueprintType, notplaceable, MinimalAPI)
class ASpectatorPawn : public ADefaultPawn
{
	GENERATED_UCLASS_BODY()

	// Begin Pawn overrides
	/** Overridden to avoid changing network role. If subclasses want networked behavior, call the Pawn::PossessedBy() instead. */
	ENGINE_API virtual void PossessedBy(class AController* NewController) override;
	ENGINE_API virtual void TurnAtRate(float Rate) override;
	ENGINE_API virtual void LookUpAtRate(float Rate) override;
	// End Pawn overrides
};
