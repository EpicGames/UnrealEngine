// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Movement component used by SpectatorPawn.
 * Primarily exists to be able to ignore time dilation during tick.
 */
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/FloatingPawnMovement.h"
#include "SpectatorPawnMovement.generated.h"

UCLASS(MinimalAPI)
class USpectatorPawnMovement : public UFloatingPawnMovement
{
	GENERATED_UCLASS_BODY()

	/** If true, component moves at full speed no matter the time dilation. Default is false. */
	UPROPERTY()
	uint32 bIgnoreTimeDilation:1;

	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
};

