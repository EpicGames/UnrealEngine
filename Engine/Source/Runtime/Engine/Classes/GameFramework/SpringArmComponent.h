// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "SpringArmComponent.generated.h"

/**
 * This component tries to maintain its children at a fixed distance from the parent,
 * but will retract the children if there is a collision, and spring back when there is no collision.
 *
 * Example: Use as a 'camera boom' or 'selfie stick' to keep the follow camera for a player from colliding into the world.
 */

UCLASS(ClassGroup=Camera, meta=(BlueprintSpawnableComponent), hideCategories=(Mobility), MinimalAPI)
class USpringArmComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Natural length of the spring arm when there are no collisions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	float TargetArmLength;

	/** offset at end of spring arm; use this instead of the relative offset of the attached component to ensure the line trace works as desired */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	FVector SocketOffset;

	/** Offset at start of spring, applied in world space. Use this if you want a world-space offset from the parent component instead of the usual relative-space offset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Camera)
	FVector TargetOffset;

	/** How big should the query probe sphere be (in unreal units) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraCollision, meta=(editcondition="bDoCollisionTest"))
	float ProbeSize;

	/** Collision channel of the query probe (defaults to ECC_Camera) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraCollision, meta=(editcondition="bDoCollisionTest"))
	TEnumAsByte<ECollisionChannel> ProbeChannel;

	/** If true, do a collision test using ProbeChannel and ProbeSize to prevent camera clipping into level.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraCollision)
	uint32 bDoCollisionTest:1;

	/**
	 * If this component is placed on a pawn, should it use the view/control rotation of the pawn where possible?
	 * When disabled, the component will revert to using the stored RelativeRotation of the component.
	 * Note that this component itself does not rotate, but instead maintains its relative rotation to its parent as normal,
	 * and just repositions and rotates its children as desired by the inherited rotation settings. Use GetTargetRotation()
	 * if you want the rotation target based on all the settings (UsePawnControlRotation, InheritPitch, etc).
	 *
	 * @see GetTargetRotation(), APawn::GetViewRotation()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraSettings)
	uint32 bUsePawnControlRotation:1;

	/** Should we inherit pitch from parent component. Does nothing if using Absolute Rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraSettings)
	uint32 bInheritPitch : 1;

	/** Should we inherit yaw from parent component. Does nothing if using Absolute Rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraSettings)
	uint32 bInheritYaw : 1;

	/** Should we inherit roll from parent component. Does nothing if using Absolute Rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=CameraSettings)
	uint32 bInheritRoll : 1;

	/**
	 * If true, camera lags behind target position to smooth its movement.
	 * @see CameraLagSpeed
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag)
	uint32 bEnableCameraLag : 1;

	/**
	 * If true, camera lags behind target rotation to smooth its movement.
	 * @see CameraRotationLagSpeed
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag)
	uint32 bEnableCameraRotationLag : 1;

	/**
	 * If bUseCameraLagSubstepping is true, sub-step camera damping so that it handles fluctuating frame rates well (though this comes at a cost).
	 * @see CameraLagMaxTimeStep
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lag, AdvancedDisplay)
	uint32 bUseCameraLagSubstepping : 1;

	/**
	 * If true and camera location lag is enabled, draws markers at the camera target (in green) and the lagged position (in yellow).
	 * A line is drawn between the two locations, in green normally but in red if the distance to the lag target has been clamped (by CameraLagMaxDistance).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag)
	uint32 bDrawDebugLagMarkers : 1;

	/** If bEnableCameraLag is true, controls how quickly camera reaches target position. Low values are slower (more lag), high values are faster (less lag), while zero is instant (no lag). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag, meta=(editcondition="bEnableCameraLag", ClampMin="0.0", ClampMax="1000.0", UIMin = "0.0", UIMax = "1000.0"))
	float CameraLagSpeed;

	/** If bEnableCameraRotationLag is true, controls how quickly camera reaches target position. Low values are slower (more lag), high values are faster (less lag), while zero is instant (no lag). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag, meta=(editcondition = "bEnableCameraRotationLag", ClampMin="0.0", ClampMax="1000.0", UIMin = "0.0", UIMax = "1000.0"))
	float CameraRotationLagSpeed;
	
	/** Max time step used when sub-stepping camera lag. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lag, AdvancedDisplay, meta=(editcondition = "bUseCameraLagSubstepping", ClampMin="0.005", ClampMax="0.5", UIMin = "0.005", UIMax = "0.5"))
	float CameraLagMaxTimeStep;

	/** Max distance the camera target may lag behind the current location. If set to zero, no max distance is enforced. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Lag, meta=(editcondition="bEnableCameraLag", ClampMin="0.0", UIMin = "0.0"))
	float CameraLagMaxDistance;

	/** If true AND the view target is simulating using physics then use the same max timestep cap as the physics system. Prevents camera jitter when delta time is clamped within Chaos Physics. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lag)
	uint32 bClampToMaxPhysicsDeltaTime : 1;

	/**
	 * Get the target rotation we inherit, used as the base target for the boom rotation.
	 * This is derived from attachment to our parent and considering the UsePawnControlRotation and absolute rotation flags.
	 */
	UFUNCTION(BlueprintCallable, Category=SpringArm)
	ENGINE_API FRotator GetTargetRotation() const;

	/** Get the position where the camera should be without applying the Collision Test displacement */
	UFUNCTION(BlueprintCallable, Category=CameraCollision)
	ENGINE_API FVector GetUnfixedCameraPosition() const;

	/** Is the Collision Test displacement being applied? */
	UFUNCTION(BlueprintCallable, Category = CameraCollision)
	ENGINE_API bool IsCollisionFixApplied() const;

	/** Temporary variables when applying Collision Test displacement to notify if its being applied and by how much */
	bool bIsCameraFixed = false;
	FVector UnfixedCameraPosition;

	/** Temporary variables when using camera lag, to record previous camera position */
	FVector PreviousDesiredLoc;
	FVector PreviousArmOrigin;
	/** Temporary variable for lagging camera rotation, for previous rotation */
	FRotator PreviousDesiredRot;

	// UActorComponent interface
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	// End of UActorComponent interface

	// USceneComponent interface
	ENGINE_API virtual bool HasAnySockets() const override;
	ENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	ENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	// End of USceneComponent interface

	/** The name of the socket at the end of the spring arm (looking back towards the spring arm origin) */
	static ENGINE_API const FName SocketName;

	/** Returns the desired rotation for the spring arm, before the rotation constraints such as bInheritPitch etc are enforced. */
	ENGINE_API virtual FRotator GetDesiredRotation() const;

protected:
	/** Cached component-space socket location */
	FVector RelativeSocketLocation;
	/** Cached component-space socket rotation */
	FQuat RelativeSocketRotation;

protected:
	/** Updates the desired arm location, calling BlendLocations to do the actual blending if a trace is done */
	ENGINE_API virtual void UpdateDesiredArmLocation(bool bDoTrace, bool bDoLocationLag, bool bDoRotationLag, float DeltaTime);

	/**
	 * This function allows subclasses to blend the trace hit location with the desired arm location;
	 * by default it returns bHitSomething ? TraceHitLocation : DesiredArmLocation
	 */
	ENGINE_API virtual FVector BlendLocations(const FVector& DesiredArmLocation, const FVector& TraceHitLocation, bool bHitSomething, float DeltaTime);
};
