// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "TouchInterface.generated.h"

class UTexture2D;

USTRUCT()
struct FTouchInputControl
{
	GENERATED_USTRUCT_BODY()

	// basically mirroring SVirtualJoystick::FControlInfo but as an editable class
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="Set this to true to treat the joystick as a simple button"))
	bool bTreatAsButton = false;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="For sticks, this is the Thumb"))
	TObjectPtr<UTexture2D> Image1;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="For sticks, this is the Background"))
	TObjectPtr<UTexture2D> Image2;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="The initial center point of the control. If Time Until Reset is < 0, control resets back to here.\nUse negative numbers to invert positioning from top to bottom, left to right. (if <= 1.0, it's relative to screen, > 1.0 is absolute)"))
	FVector2D Center;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="The size of the control (if <= 1.0, it's relative to screen, > 1.0 is absolute)"))
	FVector2D VisualSize;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="For sticks, the size of the thumb (if <= 1.0, it's relative to screen, > 1.0 is absolute)"))
	FVector2D ThumbSize;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="The interactive size of the control. Measured outward from Center. (if <= 1.0, it's relative to screen, > 1.0 is absolute)"))
	FVector2D InteractionSize;
	UPROPERTY(EditAnywhere, Category = "Control", meta = (ToolTip = "The scale for control input"))
	FVector2D InputScale;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="The main input to send from this control (for sticks, this is the horizontal axis)"))
	FKey MainInputKey;
	UPROPERTY(EditAnywhere, Category="Control", meta=(ToolTip="The alternate input to send from this control (for sticks, this is the vertical axis)"))
	FKey AltInputKey;

	FTouchInputControl()
		: bTreatAsButton(false)
		, Image1(nullptr)
		, Image2(nullptr)
		, Center(ForceInitToZero)
		, VisualSize(ForceInitToZero)
		, ThumbSize(ForceInitToZero)
		, InteractionSize(ForceInitToZero)
		, InputScale(1.f, 1.f)
	{
	}
};


/**
 * Defines an interface by which touch input can be controlled using any number of buttons and virtual joysticks
 */
UCLASS(Blueprintable, BlueprintType, MinimalAPI)
class UTouchInterface : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category="TouchInterface")
	TArray<FTouchInputControl> Controls;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="Opacity (0.0 - 1.0) of all controls while any control is active"))
	float ActiveOpacity;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="Opacity (0.0 - 1.0) of all controls while no controls are active"))
	float InactiveOpacity;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="How long after user interaction will all controls fade out to Inactive Opacity"))
	float TimeUntilDeactive;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="How long after going inactive will controls reset/recenter themselves (0.0 will disable this feature)"))
	float TimeUntilReset;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="How long after joystick enabled for touch (0.0 will disable this feature)"))
	float ActivationDelay;

	UPROPERTY(EditAnywhere, Category="TouchInterface", meta=(ToolTip="Prevent joystick re-centering and moving from Center through user taps"))
	bool bPreventRecenter;

	UPROPERTY(EditAnywhere, Category = "TouchInterface", meta = (ToolTip = "Delay at startup before virtual joystick is drawn"))
	float StartupDelay;

	/** Make this the active set of touch controls */
	ENGINE_API virtual void Activate(TSharedPtr<SVirtualJoystick> VirtualJoystick);
};
