// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Misc/ScopeRWLock.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/UObjectAnnotation.h"

template<typename InAllocatorType>
struct FAnnotationSparseActorList
{
	bool IsDefault() const
	{
		return OwnerList.IsEmpty();
	}

	TArray<const AActor*, InAllocatorType> OwnerList;
};

// No automatic cleanup as we need to manually manage our reverse annotation map.
template<typename InAllocatorType>
class FUObjectAnnotationSparseActorList : public FUObjectAnnotationSparse<FAnnotationSparseActorList<InAllocatorType>, false>
{
	using Base = FUObjectAnnotationSparse<FAnnotationSparseActorList<InAllocatorType>, false>;

public:
	void AddOwner(const UObject* Object, const AActor* Owner);
	void RemoveOwner(const UObject* Object, const AActor* Owner);

protected:
	virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override;

private:
	TMap<const AActor*, TArray<const UObject*>> ReverseAnnotationMap;
};

template<typename InAllocatorType>
void FUObjectAnnotationSparseActorList<InAllocatorType>::AddOwner(const UObject* Object, const AActor* Owner)
{
	if (!Object || !Owner)
	{
		return;
	}

	UE::TRWScopeLock AnnotationMapLock(this->AnnotationMapCritical, SLT_Write);

	if (this->AnnotationMap.IsEmpty() && ReverseAnnotationMap.IsEmpty() && !this->bRegistered)
	{
		GUObjectArray.AddUObjectDeleteListener(this);
		this->bRegistered = true;
	}

	FAnnotationSparseActorList<InAllocatorType>& Entry = this->AnnotationMap.FindOrAdd(Object);
	Entry.OwnerList.AddUnique(Owner);
	ReverseAnnotationMap.FindOrAdd(Owner).AddUnique(Object);
}

template<typename InAllocatorType>
void FUObjectAnnotationSparseActorList<InAllocatorType>::RemoveOwner(const UObject* Object, const AActor* Owner)
{
	if (!Object || !Owner)
	{
		return;
	}

	UE::TRWScopeLock AnnotationMapLock(this->AnnotationMapCritical, SLT_Write);
	if (FAnnotationSparseActorList<InAllocatorType>* Entry = this->AnnotationMap.Find(Object))
	{
		Entry->OwnerList.Remove(Owner);

		// Remove the annotation entirely if we removed the last owner
		if (Entry->IsDefault())
		{
			this->AnnotationMap.Remove(Object);
		}
	}

	if (TArray<const UObject*>* Objects = ReverseAnnotationMap.Find(Owner))
	{
		Objects->Remove(Object);
		if (Objects->IsEmpty())
		{
			ReverseAnnotationMap.Remove(Owner);
		}
	}

	if (this->AnnotationMap.IsEmpty() && ReverseAnnotationMap.IsEmpty() && this->bRegistered)
	{
		GUObjectArray.RemoveUObjectDeleteListener(this);
		this->bRegistered = false;
	}
}

template<typename InAllocatorType>
void FUObjectAnnotationSparseActorList<InAllocatorType>::NotifyUObjectDeleted(const UObjectBase* Object, int32 Index)
{
	// UE::TRWScopeLock AnnotationMapLock
	{
		UE::TRWScopeLock AnnotationMapLock(this->AnnotationMapCritical, SLT_Write);

		// Object being deleted is potentially a owner
		if (const AActor* Owner = Cast<const AActor>(Object))
		{
			if (TArray<const UObject*>* Objects = ReverseAnnotationMap.Find(Owner))
			{
				for (const UObject* TempObject : *Objects)
				{
					if (FAnnotationSparseActorList<InAllocatorType>* Entry = this->AnnotationMap.Find(TempObject))
					{
						Entry->OwnerList.Remove(Owner);
						if (Entry->IsDefault())
						{
							this->AnnotationMap.Remove(TempObject);
						}
					}
				}
				ReverseAnnotationMap.Remove(Owner);
			}
		}
		// Object being deleted is a UObject, find the owners and remove them.
		else
		{
			const UObject* TempObject = Cast<const UObject>(Object);
			if (FAnnotationSparseActorList<InAllocatorType>* Owners = this->AnnotationMap.Find(TempObject))
			{
				for (TWeakObjectPtr<const AActor> ObjectOwner : Owners->OwnerList)
				{
					if (ObjectOwner.IsValid())
					{
						if (TArray<const UObject*>* Objects = ReverseAnnotationMap.Find(ObjectOwner.Get()))
						{
							Objects->Remove(TempObject);
							if (Objects->IsEmpty())
							{
								ReverseAnnotationMap.Remove(ObjectOwner.Get());
							}
						}
					}
				}
				this->AnnotationMap.Remove(TempObject);
			}
		}

		if (this->AnnotationMap.IsEmpty() && ReverseAnnotationMap.IsEmpty() && this->bRegistered)
		{
			GUObjectArray.RemoveUObjectDeleteListener(this);
			this->bRegistered = false;
		}
	}

	// Needs to be at the end because in non shipping builds it will check if we cleaned up our objects. 
	// Make sure you have released the lock before this point or you will deadlock.
	Base::NotifyUObjectDeleted(Object, Index);
}