// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UpdateLevelVisibilityLevelInfo.generated.h"

/** This structure is used to to identify NetLevelVisibility transactions between server and client */
USTRUCT()
struct FNetLevelVisibilityTransactionId
{
	GENERATED_BODY()

	enum : uint32
	{
		InvalidTransactionIndex = 0U,
		InvalidTransactionId = 0U,
		IsClientTransactionMask = 0x80000000U,
		ValueMask = ~IsClientTransactionMask,
	};

	FNetLevelVisibilityTransactionId();

	bool IsClientTransaction() const { return Data & IsClientTransactionMask; }
	bool IsValid() const { return Data != InvalidTransactionId; }
	void SetIsClientInstigator(bool bValue) { Data = (Data & ~IsClientTransactionMask) | (bValue ? IsClientTransactionMask : 0U); }
	uint32 GetTransactionIndex() const { return Data & ValueMask; }
	void SetTransactionIndex(uint32 TransactionIndex) { Data = (Data & IsClientTransactionMask) | (TransactionIndex & ValueMask); }

	uint32 IncrementTransactionIndex();
	bool operator==(const FNetLevelVisibilityTransactionId& Other) const { return Data == Other.Data; }
	ENGINE_API bool NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess);

private:
	FNetLevelVisibilityTransactionId(uint32 LevelVisibilityTransactionIndex, bool bIsClientInstigator);

	UPROPERTY()
	uint32 Data;
};

template<>
struct TStructOpsTypeTraits<FNetLevelVisibilityTransactionId> : public TStructOpsTypeTraitsBase2<FNetLevelVisibilityTransactionId>
{
	enum
	{
		WithNetSerializer = true
	};
};

/** This structure is used to pass arguments to ServerUpdateLevelVisibility() and ServerUpdateMultipleLevelsVisibility() server RPC functions */
USTRUCT()
struct FUpdateLevelVisibilityLevelInfo
{
	GENERATED_BODY();

	FUpdateLevelVisibilityLevelInfo()
		: PackageName(NAME_None)
		, CookPackageName(NAME_None)
		, FileName(NAME_None)
		, bIsVisible(false)
		, bTryMakeVisible(false)
		, bSkipCloseOnError(false)
	{
	}

	/**
	 * @param Level				Level to pull PackageName and FileName from.
	 * @param bInIsVisible		Default value for bIsVisible.
	 * @param bInTryMakeVisible	Whether the level is trying to be made visible or not.
	 */
	ENGINE_API FUpdateLevelVisibilityLevelInfo(const class ULevel* const Level, const bool bInIsVisible, const bool bInTryMakeVisible = false);

	/** The name of the package for the level whose status changed. */
	UPROPERTY()
	FName PackageName;

	/** The name of the cook package for the level whose status changed (can be none). 
	  * Can be used as a fallback for PackageName when a PIE client connects to a cooked server to update level visiblity.
	  */
	UPROPERTY()
	FName CookPackageName;

	/** The name / path of the asset file for the level whose status changed. */
	UPROPERTY()
	FName FileName;

	/** Identifies this visibility request when communicating with server */
	UPROPERTY()
	FNetLevelVisibilityTransactionId VisibilityRequestId;

	/** The new visibility state for this level. */
	UPROPERTY()
	uint32 bIsVisible : 1;

	/** Whether the level is in the state of making visible and waits for server to acknowledge. */
	UPROPERTY()
	uint32 bTryMakeVisible : 1;

	/** Skip connection close if level can't be found (not net serialized) */
	UPROPERTY(NotReplicated)
	uint32 bSkipCloseOnError : 1;

	ENGINE_API bool NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FUpdateLevelVisibilityLevelInfo> : public TStructOpsTypeTraitsBase2<FUpdateLevelVisibilityLevelInfo>
{
	enum
	{
		WithNetSerializer = true
	};
};

inline FNetLevelVisibilityTransactionId::FNetLevelVisibilityTransactionId()
: Data(InvalidTransactionId)
{
}

inline FNetLevelVisibilityTransactionId::FNetLevelVisibilityTransactionId(uint32 LevelVisibilityTransactionIndex, bool bIsClientInstigator)
: Data((bIsClientInstigator ? IsClientTransactionMask : 0U) | (LevelVisibilityTransactionIndex & ValueMask))
{
}

inline uint32 FNetLevelVisibilityTransactionId::IncrementTransactionIndex()
{
	uint32 NewIndex = (GetTransactionIndex() + 1U) & ValueMask;
	// Deal with wraparound since zero is reserved as the invalid index
	if (NewIndex == InvalidTransactionIndex)
	{
		++NewIndex;
	}
	SetTransactionIndex(NewIndex);
	return NewIndex;
}
