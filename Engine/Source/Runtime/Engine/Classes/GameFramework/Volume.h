// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Brush.h"
#include "Volume.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogVolume, Log, All);

/** 
 *	An editable 3D volume placed in a level. Different types of volumes perform different functions
 *	@see https://docs.unrealengine.com/latest/INT/Engine/Actors/Volumes
 */
UCLASS(showcategories=Collision, hidecategories=(Brush, Physics), abstract, ConversionRoot, MinimalAPI)
class AVolume : public ABrush
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	/** Delegate used for notifications when a volumes initial shape changes */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVolumeShapeChanged, AVolume& )

	/** Function to get the 'Volume imported' delegate */
	static FOnVolumeShapeChanged& GetOnVolumeShapeChangedDelegate()
	{
		return OnVolumeShapeChanged;
	}
private:
	/** Called during posteditchange after the volume's initial shape has changed */
	static ENGINE_API FOnVolumeShapeChanged OnVolumeShapeChanged;
public:
	//~ Begin AActor Interface
	/**
	* Function that gets called from within Map_Check to allow this actor to check itself
	* for any potential errors and register them with map check dialog.
	*/
	ENGINE_API virtual void CheckForErrors() override;
	virtual bool ShouldCheckCollisionComponentForErrors() const { return true; }
#endif // WITH_EDITOR
	
	ENGINE_API virtual bool IsLevelBoundsRelevant() const override;
	//~ End AActor Interface

	//~ Begin Brush Interface
	ENGINE_API virtual bool IsStaticBrush() const override;
	ENGINE_API virtual bool IsVolumeBrush() const override;
	//~ End Brush Interface

	/** @returns true if a sphere/point (with optional radius CheckRadius) overlaps this volume */
	ENGINE_API bool EncompassesPoint(FVector Point, float SphereRadius=0.f, float* OutDistanceToPoint = 0) const;

	/** @returns the coarse bounds of this volume */
	ENGINE_API FBoxSphereBounds GetBounds() const;

	//Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual FName GetCustomIconName() const override;
#endif // WITH_EDITOR
	//End UObject Interface


};



