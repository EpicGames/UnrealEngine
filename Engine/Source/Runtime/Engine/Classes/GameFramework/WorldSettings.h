// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "GameFramework/Actor.h"
#include "GameFramework/DamageType.h"
#include "GameFramework/Info.h"
#include "Sound/AudioVolume.h"
#include "UObject/ConstructorHelpers.h"
#include "WorldGridPreviewer.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldSettings.generated.h"

class UWorldPartition;
class UAssetUserData;
class UNetConnection;
class UNavigationSystemConfig;
class UAISystemBase;
class UHierarchicalLODSetup;
struct FHierarchicalSimplification;

UENUM()
enum EVisibilityAggressiveness : int
{
	VIS_LeastAggressive UMETA(DisplayName = "Least Aggressive"),
	VIS_ModeratelyAggressive UMETA(DisplayName = "Moderately Aggressive"),
	VIS_MostAggressive UMETA(DisplayName = "Most Aggressive"),
	VIS_Max UMETA(Hidden),
};

UENUM()
enum EVolumeLightingMethod : int
{
	/** 
	 * Lighting samples are computed in an adaptive grid which covers the entire Lightmass Importance Volume.  Higher density grids are used near geometry.
	 * The Volumetric Lightmap is interpolated efficiently on the GPU per-pixel, allowing accurate indirect lighting for dynamic objects and volumetric fog.
	 * Positions outside of the Importance Volume reuse the border texels of the Volumetric Lightmap (clamp addressing).
	 * On mobile, interpolation is done on the CPU at the center of each object's bounds.
	 */
	VLM_VolumetricLightmap UMETA(DisplayName = "Volumetric Lightmap"),

	/** 
	 * Volume lighting samples are placed on top of static surfaces at medium density, and everywhere else in the Lightmass Importance Volume at low density.  Positions outside of the Importance Volume will have no indirect lighting.
	 * This method requires CPU interpolation so the Indirect Lighting Cache is used to interpolate results for each dynamic object, adding Rendering Thread overhead.  
	 * Volumetric Fog cannot be affected by precomputed lighting with this method.
	 */
	VLM_SparseVolumeLightingSamples UMETA(DisplayName = "Sparse Volume Lighting Samples"),
};

USTRUCT(BlueprintType)
struct FLightmassWorldInfoSettings
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Warning: Setting this to less than 1 will greatly increase build times!
	 * Scale of the level relative to real world scale (1 Unreal Unit = 1 cm). 
	 * All scale-dependent Lightmass setting defaults have been tweaked to work well with real world scale, 
	 * Any levels with a different scale should use this scale to compensate. 
	 * For large levels it can drastically reduce build times to set this to 2 or 4.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, AdvancedDisplay, meta=(UIMin = "1.0", UIMax = "4.0"))
	float StaticLightingLevelScale;

	/** 
	 * Number of light bounces to simulate for point / spot / directional lights, starting from the light source. 
	 * 0 is direct lighting only, 1 is one bounce, etc. 
	 * Bounce 1 takes the most time to calculate and contributes the most to visual quality, followed by bounce 2.
	 * Successive bounces don't really affect build times, but have a much lower visual impact, unless the material diffuse colors are close to 1.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, meta=(UIMin = "1.0", UIMax = "10.0"))
	int32 NumIndirectLightingBounces;

	/** 
	 * Number of skylight and emissive bounces to simulate.  
	 * Lightmass uses a non-distributable radiosity method for skylight bounces whose cost is proportional to the number of bounces.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, meta=(UIMin = "1.0", UIMax = "10.0"))
	int32 NumSkyLightingBounces;

	/** 
	 * Warning: Setting this higher than 1 will greatly increase build times!
	 * Can be used to increase the GI solver sample counts in order to get higher quality for levels that need it.
	 * It can be useful to reduce IndirectLightingSmoothness somewhat (~.75) when increasing quality to get defined indirect shadows.
	 * Note that this can't affect compression artifacts, UV seams or other texture based artifacts.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, AdvancedDisplay, meta=(UIMin = "1.0", UIMax = "4.0"))
	float IndirectLightingQuality;

	/** 
	 * Smoothness factor to apply to indirect lighting.  This is useful in some lighting conditions when Lightmass cannot resolve accurate indirect lighting.
	 * 1 is default smoothness tweaked for a variety of lighting situations.
	 * Higher values like 3 smooth out the indirect lighting more, but at the cost of indirect shadows losing detail.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, AdvancedDisplay, meta=(UIMin = "0.5", UIMax = "6.0"))
	float IndirectLightingSmoothness;

	/** 
	 * Represents a constant color light surrounding the upper hemisphere of the level, like a sky.
	 * This light source currently does not get bounced as indirect lighting and causes reflection capture brightness to be incorrect.  Prefer using a Static Skylight instead.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral)
	FColor EnvironmentColor;

	/** Scales EnvironmentColor to allow independent color and brightness controls. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, meta=(UIMin = "0", UIMax = "10"))
	float EnvironmentIntensity;

	/** Scales the emissive contribution of all materials in the scene.  Currently disabled and should be removed with mesh area lights. */
	UPROPERTY()
	float EmissiveBoost;

	/** Scales the diffuse contribution of all materials in the scene. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, meta=(UIMin = "0.1", UIMax = "6.0"))
	float DiffuseBoost;

	/** Technique to use for providing precomputed lighting at all positions inside the Lightmass Importance Volume */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassVolumeLighting)
	TEnumAsByte<enum EVolumeLightingMethod> VolumeLightingMethod;

	/** If true, AmbientOcclusion will be enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion)
	uint8 bUseAmbientOcclusion:1;

	/** 
	 * Whether to generate textures storing the AO computed by Lightmass.
	 * These can be accessed through the PrecomputedAOMask material node, 
	 * Which is useful for blending between material layers on environment assets.
	 * Be sure to set DirectIlluminationOcclusionFraction and IndirectIlluminationOcclusionFraction to 0 if you only want the PrecomputedAOMask!
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion)
	uint8 bGenerateAmbientOcclusionMaterialMask:1;

	/** If true, override normal direct and indirect lighting with just the exported diffuse term. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassDebug, AdvancedDisplay)
	uint8 bVisualizeMaterialDiffuse:1;

	/** If true, override normal direct and indirect lighting with just the AO term. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassDebug, AdvancedDisplay)
	uint8 bVisualizeAmbientOcclusion:1;

	/** 
	 * Whether to compress lightmap textures.  Disabling lightmap texture compression will reduce artifacts but increase memory and disk size by 4x.
	 * Use caution when disabling this.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassGeneral, AdvancedDisplay)
	uint8 bCompressLightmaps:1;

	/** 
	 * Size of an Volumetric Lightmap voxel at the highest density (used around geometry), in world space units. 
	 * This setting has a large impact on build times and memory, use with caution.  
	 * Halving the DetailCellSize can increase memory by up to a factor of 8x.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassVolumeLighting, meta=(UIMin = "50", UIMax = "1000"))
	float VolumetricLightmapDetailCellSize;

	/** 
	 * Maximum amount of memory to spend on Volumetric Lightmap Brick data.  High density bricks will be discarded until this limit is met, with bricks furthest from geometry discarded first.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassVolumeLighting, meta=(UIMin = "1", UIMax = "500"), meta = (EditConditionHides, EditCondition = "!bWorldPartition"))
	float VolumetricLightmapMaximumBrickMemoryMb;

	/**  Size of an Volumetric Lightmap high detail loading cell.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = LightmassVolumeLighting, meta = (UIMin = "1600", UIMax = "102400"), meta = (EditConditionHides, EditCondition = "bWorldPartition"))
	float VolumetricLightmapLoadingCellSize;

	/** 
	 * Controls how much smoothing should be done to Volumetric Lightmap samples during Spherical Harmonic de-ringing.  
	 * Whenever highly directional lighting is stored in a Spherical Harmonic, a ringing artifact occurs which manifests as unexpected black areas on the opposite side.
	 * Smoothing can reduce this artifact.  Smoothing is only applied when the ringing artifact is present.
	 * 0 = no smoothing, 1 = strong smooth (little directionality in lighting).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassVolumeLighting, meta=(UIMin = "0", UIMax = "1"))
	float VolumetricLightmapSphericalHarmonicSmoothing;

	/** 
	 * Scales the distances at which volume lighting samples are placed.  Volume lighting samples are computed by Lightmass and are used for GI on movable components.
	 * Using larger scales results in less sample memory usage and reduces Indirect Lighting Cache update times, but less accurate transitions between lighting areas.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassVolumeLighting, AdvancedDisplay, meta=(UIMin = "0.1", UIMax = "100.0"))
	float VolumeLightSamplePlacementScale;

	/** How much of the AO to apply to direct lighting. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion, meta=(UIMin = "0", UIMax = "1"))
	float DirectIlluminationOcclusionFraction;

	/** How much of the AO to apply to indirect lighting. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion, meta=(UIMin = "0", UIMax = "1"))
	float IndirectIlluminationOcclusionFraction;

	/** Higher exponents increase contrast. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion, meta=(UIMin = ".5", UIMax = "8"))
	float OcclusionExponent;

	/** Fraction of samples taken that must be occluded in order to reach full occlusion. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion, meta=(UIMin = "0", UIMax = "1"))
	float FullyOccludedSamplesFraction;

	/** Maximum distance for an object to cause occlusion on another object. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=LightmassOcclusion)
	float MaxOcclusionDistance;

	UPROPERTY(transient)
	bool bWorldPartition;

	FLightmassWorldInfoSettings()
		: StaticLightingLevelScale(1)
		, NumIndirectLightingBounces(3)
		, NumSkyLightingBounces(1)
		, IndirectLightingQuality(1)
		, IndirectLightingSmoothness(1)
		, EnvironmentColor(ForceInit)
		, EnvironmentIntensity(1.0f)
		, EmissiveBoost(1.0f)
		, DiffuseBoost(1.0f)
		, VolumeLightingMethod(VLM_VolumetricLightmap)
		, bUseAmbientOcclusion(false)
		, bGenerateAmbientOcclusionMaterialMask(false)
		, bVisualizeMaterialDiffuse(false)
		, bVisualizeAmbientOcclusion(false)
		, bCompressLightmaps(true)
		, VolumetricLightmapDetailCellSize(200)
		, VolumetricLightmapMaximumBrickMemoryMb(30)
		, VolumetricLightmapLoadingCellSize(3200)		
		, VolumetricLightmapSphericalHarmonicSmoothing(.02f)
		, VolumeLightSamplePlacementScale(1)
		, DirectIlluminationOcclusionFraction(0.5f)
		, IndirectIlluminationOcclusionFraction(1.0f)
		, OcclusionExponent(1.0f)
		, FullyOccludedSamplesFraction(1.0f)
		, MaxOcclusionDistance(200.0f)
		, bWorldPartition(false)
	{
	}
};

/** stores information on a viewer that actors need to be checked against for relevancy */
USTRUCT()
struct FNetViewer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UNetConnection> Connection;

	/** The "controlling net object" associated with this view (typically player controller) */
	UPROPERTY()
	TObjectPtr<class AActor> InViewer;

	/** The actor that is being directly viewed, usually a pawn.  Could also be the net actor of consequence */
	UPROPERTY()
	TObjectPtr<class AActor> ViewTarget;

	/** Where the viewer is looking from */
	UPROPERTY()
	FVector ViewLocation;

	/** Direction the viewer is looking */
	UPROPERTY()
	FVector ViewDir;

	FNetViewer()
		: Connection(nullptr)
		, InViewer(nullptr)
		, ViewTarget(nullptr)
		, ViewLocation(ForceInit)
		, ViewDir(ForceInit)
	{
	}

	ENGINE_API FNetViewer(UNetConnection* InConnection, float DeltaSeconds);

	/** For use by replication graph, connection likely null */
	ENGINE_API FNetViewer(AController* InController);
};

USTRUCT()
struct FNaniteSettings
{
	GENERATED_BODY();

	FNaniteSettings()
	: bAllowMaskedMaterials(true)
	{
	}

	UPROPERTY(EditAnywhere, Category = Nanite)
	bool bAllowMaskedMaterials;
};

/** Settings pertaining to which PhysX broadphase to use, and settings for MBP if that is the chosen broadphase type */
USTRUCT()
struct FBroadphaseSettings
{
	GENERATED_BODY();

	FBroadphaseSettings()
		: bUseMBPOnClient(false)
		, bUseMBPOnServer(false)
		, bUseMBPOuterBounds(false)
		, MBPBounds(EForceInit::ForceInitToZero)
		, MBPOuterBounds(EForceInit::ForceInitToZero)
		, MBPNumSubdivs(2)
	{

	}

	/** Whether to use MBP (Multi Broadphase Pruning */
	UPROPERTY(EditAnywhere, Category = Broadphase)
	bool bUseMBPOnClient;

	UPROPERTY(EditAnywhere, Category = Broadphase)
	bool bUseMBPOnServer;

	/** Whether to have MBP grid over concentrated inner bounds with loose outer bounds */
	UPROPERTY(EditAnywhere, Category = Broadphase)
	bool bUseMBPOuterBounds;

	/** Total bounds for MBP, must cover the game world or collisions are disabled for out of bounds actors */
	UPROPERTY(EditAnywhere, Category = Broadphase, meta = (EditCondition = "bUseMBPOnClient || bUseMBPOnServer"))
	FBox MBPBounds;

	/** Total bounds for MBP, should cover absolute maximum bounds of the game world where physics is required */
	UPROPERTY(EditAnywhere, Category = Broadphase, meta = (EditCondition = "bUseMBPOnClient || bUseMBPOnServer"))
	FBox MBPOuterBounds;

	/** Number of times to subdivide the MBP bounds, final number of regions is MBPNumSubdivs^2 */
	UPROPERTY(EditAnywhere, Category = Broadphase, meta = (EditCondition = "bUseMBPOnClient || bUseMBPOnServer", ClampMin=1, ClampMax=16))
	uint32 MBPNumSubdivs;
};

/** Settings for dynamically increasing the loading range in PIE after BeginPlay - makes PIE startup quicker on large maps */
USTRUCT()
struct FDynamicLoadingRangeScaling
{
	GENERATED_BODY()

	/** Enable for this map, unless user force disabled using bForceDisableDynamicLoadingRangeScaling. */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay)
	bool bEnabled = false;

	/** Start scaling at which ratio, from 0.f to 100.f. 100.f is equivalent to disabling the feature. 0.f is not recommended if floor or terrain tiles are streamed, as the pawn will have nothing to stand on initially. */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta=(EditCondition = "bEnabled", Units="%", ClampMin = "0.0", ClampMax = "100.0", UIMin = "0.0", UIMax = "100.0"))
	float InitialDynamicLoadingRangeScaling = 100.f;

	/** Delay before scaling starts, to prevent additional world streaming while initial resources are also streaming in mips, LODs, etc. */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta=(EditCondition = "bEnabled", Units="s", ClampMin = "0.0", UIMin = "0.0"))
	float DynamicLoadingRangeScalingDelay = 0.f;

	/** How long to scale for after the delay, in seconds. */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta=(EditCondition = "bEnabled", Units="s", ClampMin = "0.0", UIMin = "0.0"))
	float DynamicLoadingRangeScalingDuration = 10.f;
};

/**
 * Actor containing all script accessible world properties.
 */
UCLASS(config=game, hidecategories=(Actor, Advanced, Display, Events, Object, Attachment, Info, Input, Blueprint, Layers, Tags, Replication, LevelInstance), showcategories=(Rendering, WorldPartition, "Input|MouseInput", "Input|TouchInput"), notplaceable, MinimalAPI)
class AWorldSettings : public AInfo, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

#if WITH_EDITOR
	bool GetDynamicLoadingRangeScalingEnable() const;
#endif // WITH_EDITOR

	/** PRECOMPUTED VISIBILITY SETTINGS **/

	/** 
	 * World space size of precomputed visibility cells in x and y.
	 * Smaller sizes produce more effective occlusion culling at the cost of increased runtime memory usage and lighting build times.
	 */
	UPROPERTY(EditAnywhere, Category=PrecomputedVisibility, AdvancedDisplay, meta=(DisplayAfter="bPlaceCellsOnlyAlongCameraTracks"))
	int32 VisibilityCellSize;

	/** 
	 * Determines how aggressive precomputed visibility should be.
	 * More aggressive settings cull more objects but also cause more visibility errors like popping.
	 */
	UPROPERTY(EditAnywhere, Category=PrecomputedVisibility, AdvancedDisplay, meta=(DisplayAfter="bPlaceCellsOnlyAlongCameraTracks"))
	TEnumAsByte<enum EVisibilityAggressiveness> VisibilityAggressiveness;

	/** 
	 * Whether to place visibility cells inside Precomputed Visibility Volumes and along camera tracks in this level. 
	 * Precomputing visibility reduces rendering thread time at the cost of some runtime memory and somewhat increased lighting build times.
	 */
	UPROPERTY(EditAnywhere, Category=PrecomputedVisibility)
	uint8 bPrecomputeVisibility:1;

	/** 
	 * Whether to place visibility cells only along camera tracks or only above shadow casting surfaces.
	 */
	UPROPERTY(EditAnywhere, Category=PrecomputedVisibility, AdvancedDisplay)
	uint8 bPlaceCellsOnlyAlongCameraTracks:1;

	/** DEFAULT BASIC PHYSICS SETTINGS **/

	/** If true, enables CheckStillInWorld checks */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, AdvancedDisplay)
	uint8 bEnableWorldBoundsChecks:1;

protected:
	/** if set to false navigation system will not get created (and all 
	 *	navigation functionality won't be accessible).
	 *	This flag is now deprecated. Use NavigationSystemConfig property to 
	 *	determine navigation system's properties or existence */
	UE_DEPRECATED(4.22, "This member will be removed. Please use NavigationSystemConfig instead.")
	UPROPERTY(BlueprintReadOnly, config, Category = World, meta=(DisplayName = "DEPRECATED_bEnableNavigationSystem"))
	uint8 bEnableNavigationSystem:1;

	/** if set to false AI system will not get created. Use it to disable all AI-related activity on a map */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, config, Category=AI, AdvancedDisplay)
	uint8 bEnableAISystem:1;

public:
	/** If set to true, use new UGC navigation (added through scene graph component). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, config, Category=AI, AdvancedDisplay, meta=(DisplayName = "Use Automatic UGC Navigation"))
	uint8 bUseAutomaticUGCNavigation:1;
	
	/** 
	 * Enables tools for composing a tiled world. 
	 * Level has to be saved and all sub-levels removed before enabling this option.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, meta=(EditConditionHides, EditCondition="WorldPartition == nullptr"))
	uint8 bEnableWorldComposition:1;

	/**
	 * Enables client-side streaming volumes instead of server-side.
	 * Expected usage scenario: server has all streaming levels always loaded, clients independently stream levels in/out based on streaming volumes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = World)
	uint8 bUseClientSideLevelStreamingVolumes:1;

	/** World origin will shift to a camera position when camera goes far away from current origin */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, AdvancedDisplay, meta=(EditConditionHides, EditCondition = "bEnableWorldComposition"))
	uint8 bEnableWorldOriginRebasing:1;
		
	/** if set to true, when we call GetGravityZ we assume WorldGravityZ has already been initialized and skip the lookup of DefaultGravityZ and GlobalGravityZ */
	UPROPERTY(transient)
	uint8 bWorldGravitySet:1;

	/** If set to true we will use GlobalGravityZ instead of project setting DefaultGravityZ */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(DisplayName = "Override World Gravity"), Category = Physics)
	uint8 bGlobalGravitySet:1;

	/** 
	 * Causes the BSP build to generate as few sections as possible.
	 * This is useful when you need to reduce draw calls but can reduce texture streaming efficiency and effective lightmap resolution.
	 * Note - changes require a rebuild to propagate.  Also, be sure to select all surfaces and make sure they all have the same flags to minimize section count.
	 */
	UPROPERTY(EditAnywhere, Category=World, AdvancedDisplay)
	uint8 bMinimizeBSPSections:1;

	/** 
	 * Whether to force lightmaps and other precomputed lighting to not be created even when the engine thinks they are needed.
	 * This is useful for improving iteration in levels with fully dynamic lighting and shadowing.
	 * Note that any lighting and shadowing interactions that are usually precomputed will be lost if this is enabled.
	 */
	UPROPERTY(EditAnywhere, Category=Lightmass, AdvancedDisplay)
	uint8 bForceNoPrecomputedLighting:1;

	/** 
	 * Force precomputed lighting to only use VolumetricLightmaps.
	 */
	UPROPERTY(EditAnywhere, Category=Lightmass, AdvancedDisplay)
	uint8 bForceVolumetricLightmapsOnly:1;

	/** when this flag is set, more time is allocated to background loading (replicated) */
	UPROPERTY(replicated)
	uint8 bHighPriorityLoading:1;

	/** copy of bHighPriorityLoading that is not replicated, for clientside-only loading operations */
	UPROPERTY()
	uint8 bHighPriorityLoadingLocal:1;

	UPROPERTY(config, EditAnywhere, Category = Broadphase)
	uint8 bOverrideDefaultBroadphaseSettings:1;

	/** If set to true, all eligible actors in this level will be added to a single cluster representing the entire level (used for small sublevels) */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem, AdvancedDisplay, meta=(EditConditionHides, EditCondition="WorldPartition == nullptr", DisplayAfter="HierarchicalLODSetup"))
	uint8 bGenerateSingleClusterForLevel : 1;

	/**
	 * Allows game logic to temporarily disable all forms of time dilation (when using GetEffectiveTimeDilation().
	 * When false, logic is still allowed to set time dilation values, but their values will be ignored when calling
	 * GetEffectiveTimeDilation() until this is set back to true.
	 *
	 * Transient because it's used for temporary overrides and shouldn't be saved/loaded from level packages. Not
	 * replicated because in multiplayer a server should modify the real time dilation, instead of ignoring it on
	 * a client (which will lead to position desyncs due to delta time mismatches).
	*/
	UPROPERTY(Transient)
	uint8 bAllowTimeDilation : 1;

#if WITH_EDITORONLY_DATA
	// This property is customized in FWorldPartitionDetails
	UPROPERTY()
	FDynamicLoadingRangeScaling DynamicLoadingRangeScaling;

	/** if set to true, this hide the streaming disabled warning available in the viewport */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta = (EditCondition = "WorldPartition != nullptr"))
	uint8 bHideEnableStreamingWarning : 1;
	
	/** 
	 * Whether Foliage actors of this world contain their grid size in their name. This should only be changed by UWorldPartitionFoliageBuilder or when creating new worlds so that older worlds are unaffected
	 * and is used by the UActorPartitionSubsystem to find existing foliage actors by name.
	 */
	UPROPERTY()
	uint8 bIncludeGridSizeInNameForFoliageActors : 1;

	/**
	 * Whether partitioned actors of this world contain their grid size in their name. This should only be changed when creating new worlds so that older worlds are unaffected
	 * and is used by the UActorPartitionSubsystem to find existing foliage actors by name.
	 */
	UPROPERTY()
	uint8 bIncludeGridSizeInNameForPartitionedActors : 1;
#endif

	/**
	 * Whether to configure the listening socket to allow reuse of the address and port. If this is true, be sure no other
	 * servers can run on the same port, otherwise this can lead to undefined behavior since packets will go to two servers.
	 */
	UPROPERTY(EditAnywhere, Config, Category=Network)
	uint8 bReuseAddressAndPort : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AI, meta=(MetaClass="/Script/Engine.AISystemBase", editcondition="bEnableAISystem"), AdvancedDisplay)
	TSoftClassPtr<UAISystemBase> AISystemClass;

	/** Additional transform applied when applying LevelStreaming Transform to LevelInstance */
	UPROPERTY()
	FVector LevelInstancePivotOffset;

protected:
	/** Holds parameters for NavigationSystem's creation. Set to Null will result
	 *	in NavigationSystem instance not being created for this world. Note that
	 *	if set NavigationSystemConfigOverride will be used instead. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, AdvancedDisplay, Instanced, NoClear, meta=(NoResetToDefault))
	TObjectPtr<UNavigationSystemConfig> NavigationSystemConfig;

	/** Overrides NavigationSystemConfig. */
	UPROPERTY(Transient)
	TObjectPtr<UNavigationSystemConfig> NavigationSystemConfigOverride;

	UPROPERTY(VisibleAnywhere, Category=WorldPartitionSetup, Instanced)
	TObjectPtr<UWorldPartition> WorldPartition;

public:
#if WITH_EDITORONLY_DATA
	/** Size of the grid for instanced foliage actors, only used for partitioned worlds */
	UPROPERTY(EditAnywhere, Category = Foliage)
	uint32 InstancedFoliageGridSize;

	UPROPERTY(EditAnywhere, Category = Foliage, Transient, SkipSerialization)
	bool bShowInstancedFoliageGrid;

	UPROPERTY(EditAnywhere, Category = Landscape)
	uint32 LandscapeSplineMeshesGridSize;

	/** Size of the grid for navigation data chunk actors */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint32 NavigationDataChunkGridSize;
	
	/**
	 * Loading cell size used when building navigation data iteratively.
	 * The actual cell size used will be rounded using the NavigationDataChunkGridSize.
	 * It's recommended to use a value as high as the hardware memory allows to load. 
	 */
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (ClampMin = "5000"))
	uint32 NavigationDataBuilderLoadingCellSize;
	
	/** Default size of the grid for placed elements from the editor */
	UPROPERTY()
	uint32 DefaultPlacementGridSize;
#endif

#if WITH_EDITOR
	mutable TUniquePtr<FWorldGridPreviewer> InstancedFoliageGridGridPreviewer;
#endif

	/**
	 * A list of runtime data layers that should be included in the base navmesh.
	 * Editor data layers and actors outside data layers will be included.
	 */
	UPROPERTY(EditAnywhere, Category = Navigation)
	TArray<TObjectPtr<UDataLayerAsset>> BaseNavmeshDataLayers;
	
	/** scale of 1uu to 1m in real world measurements, for HMD and other physically tracked devices (e.g. 1uu = 1cm would be 100.0) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=VR)
	float WorldToMeters;

	// any actor falling below this level gets destroyed
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category=World, meta=(editcondition = "bEnableWorldBoundsChecks"))
	float KillZ;

	// The type of damage inflicted when a actor falls below KillZ
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, AdvancedDisplay, meta=(editcondition = "bEnableWorldBoundsChecks"))
	TSubclassOf<UDamageType> KillZDamageType;

	// current gravity actually being used
	UPROPERTY(transient, ReplicatedUsing=OnRep_WorldGravityZ)
	float WorldGravityZ;

	UFUNCTION()
	ENGINE_API virtual void OnRep_WorldGravityZ();

	// Set a custom gravity value for this level, in distance units per second squared.
	// For example, when the project's distance unit is set to centimeters (the default), normal earth gravity would be -980.0.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Physics, meta=(editcondition = "bGlobalGravitySet"))
	float GlobalGravityZ;

	// level specific default physics volume
	UPROPERTY(EditAnywhere, noclear, BlueprintReadOnly, Category=Physics, AdvancedDisplay)
	TSubclassOf<class ADefaultPhysicsVolume> DefaultPhysicsVolumeClass;

	// Method to get the Physics Collision Handler Class
	virtual TSubclassOf<class UPhysicsCollisionHandler> GetPhysicsCollisionHandlerClass() { return PhysicsCollisionHandlerClass; }

	// optional level specific collision handler
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Physics, AdvancedDisplay)
	TSubclassOf<class UPhysicsCollisionHandler>	PhysicsCollisionHandlerClass;

	/************************************/
	
	/** GAMEMODE SETTINGS **/
	
	/** The default GameMode to use when starting this map in the game. If this value is NULL, the INI setting for default game type is used. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=GameMode, meta=(DisplayName="GameMode Override"))
	TSubclassOf<class AGameModeBase> DefaultGameMode;

	/** Class of GameNetworkManager to spawn for network games */
	UPROPERTY()
	TSubclassOf<class AGameNetworkManager> GameNetworkManagerClass;

	/************************************/
	
	/** RENDERING SETTINGS **/
	/** Maximum size of textures for packed light and shadow maps */
	UPROPERTY(EditAnywhere, Category=Lightmass, AdvancedDisplay)
	int32 PackedLightAndShadowMapTextureSize;

	/** Default color scale for the level */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=World, AdvancedDisplay)
	FVector DefaultColorScale;

	/** Max occlusion distance used by mesh distance fields, overridden if there is a movable skylight. */
	UPROPERTY(EditAnywhere, Category=Rendering, meta=(UIMin = "500", UIMax = "5000", DisplayName = "Default Max DistanceField Occlusion Distance"))
	float DefaultMaxDistanceFieldOcclusionDistance;

	/** Distance from the camera that the global distance field should cover. */
	UPROPERTY(EditAnywhere, Category=Rendering, meta=(UIMin = "10000", UIMax = "100000", DisplayName = "Global DistanceField View Distance"))
	float GlobalDistanceFieldViewDistance;

	/** 
	 * Controls the intensity of self-shadowing from capsule indirect shadows. 
	 * These types of shadows use approximate occluder representations, so reducing self-shadowing intensity can hide those artifacts.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering, meta=(UIMin = "0", UIMax = "1"))
	float DynamicIndirectShadowsSelfShadowingIntensity;

#if WITH_EDITORONLY_DATA
	/************************************/
	
	/** LIGHTMASS RELATED SETTINGS **/
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Lightmass)
	struct FLightmassWorldInfoSettings LightmassSettings;
#endif

	/**  Range in which volumetric lightmaps will be loaded.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = LightmassVolumeLighting, meta = (UIMin = "1600", UIMax = "102400"), meta = (EditConditionHides, EditCondition = "WorldPartition != nullptr"))
	float VolumetricLightmapLoadingRange;


	/************************************/
	/** NANITE SETTINGS **/
	UPROPERTY(EditAnywhere, config, replicated, ReplicatedUsing = OnRep_NaniteSettings, Category = Nanite)
	FNaniteSettings NaniteSettings;

	UFUNCTION()
	ENGINE_API virtual void OnRep_NaniteSettings();

	ENGINE_API virtual void SetAllowMaskedMaterials(bool bState);

	/************************************/
	/** AUDIO SETTINGS **/
	/** Default reverb settings used by audio volumes.													*/
	UPROPERTY(EditAnywhere, config, Category=Audio)
	FReverbSettings DefaultReverbSettings;

	/** Default interior settings applied to sounds that have "apply ambient volumes" set to true on their SoundClass. */
	UPROPERTY(EditAnywhere, config, Category=Audio)
	FInteriorSettings DefaultAmbientZoneSettings;

	/** Default Base SoundMix.																			*/
	UPROPERTY(EditAnywhere, Category=Audio)
	TObjectPtr<class USoundMix> DefaultBaseSoundMix;

#if WITH_EDITORONLY_DATA
	/** If set overrides the level settings and global project settings */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem, meta=(EditConditionHides, EditCondition = "WorldPartition == nullptr"))
	TSoftClassPtr<UHierarchicalLODSetup> HLODSetupAsset;

	/** If set overrides the project-wide base material used for Proxy Materials */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem, meta=(EditConditionHides, EditCondition = "WorldPartition == nullptr && HLODSetupAsset == nullptr"))
	TSoftObjectPtr<class UMaterialInterface> OverrideBaseMaterial;

protected:
	/** Hierarchical LOD Setup */
	UPROPERTY(EditAnywhere, Category = HLODSystem, config, meta=(EditConditionHides, EditCondition = "WorldPartition == nullptr && HLODSetupAsset == nullptr"))
	TArray<FHierarchicalSimplification> HierarchicalLODSetup;

public:
	UPROPERTY()
	int32 NumHLODLevels;

	/** Specify the transform to apply to the source meshes when building HLODs. */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem, AdvancedDisplay, meta=(EditConditionHides, EditCondition = "WorldPartition == nullptr", DisplayName = "HLOD Baking Transform"))
	FTransform HLODBakingTransform;

	/************************************/
	/** EDITOR ONLY SETTINGS **/

	/** Level Bookmarks: 10 should be MAX_BOOKMARK_NUMBER @fixmeconst */
	UE_DEPRECATED(4.21, "This member will be removed. Please use the Bookmark accessor functions instead.")
	UPROPERTY()
	TObjectPtr<class UBookMark> BookMarks[10];
#endif

	/************************************/

	/** 
	 * Normally 1 - scales real time passage.
	 * Warning - most use cases should use GetEffectiveTimeDilation() instead of reading from this directly
	 */
	UPROPERTY(transient, replicated)
	float TimeDilation;

	// Additional time dilation used by Sequencer slomo track.  Transient because this is often 
	// temporarily modified by the editor when previewing slow motion effects, yet we don't want it saved or loaded from level packages.
	UPROPERTY(transient, replicated)
	float CinematicTimeDilation;

	// Additional TimeDilation used to control demo playback speed
	UPROPERTY(transient)
	float DemoPlayTimeDilation;

	/** Lowest acceptable global time dilation. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MinGlobalTimeDilation;
	
	/** Highest acceptable global time dilation. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MaxGlobalTimeDilation;

	/** Lowest acceptable cinematic time dilation. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MinCinematicTimeDilation;

	/** Highest acceptable cinematic time dilation. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MaxCinematicTimeDilation;

	/** Smallest possible frametime, not considering dilation. Equiv to 1/FastestFPS. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MinUndilatedFrameTime;

	/** Largest possible frametime, not considering dilation. Equiv to 1/SlowestFPS. */
	UPROPERTY(config, EditAnywhere, Category = Tick, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0"))
	float MaxUndilatedFrameTime;

	UPROPERTY(config, EditAnywhere, Category = Broadphase)
	FBroadphaseSettings BroadphaseSettings;

	/** valid only during replication - information about the player(s) being replicated to
	 * (there could be more than one in the case of a splitscreen client)
	 */
	UPROPERTY()
	TArray<struct FNetViewer> ReplicationViewers;

	// ************************************

protected:

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = AssetUserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of editor-only user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = AssetUserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;
#endif

	// If paused, PlayerState of person pausing the game.
	UPROPERTY(transient, replicated)
	TObjectPtr<class APlayerState> PauserPlayerState;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FWorldPartitionPerWorldSettings DefaultWorldPartitionSettings;
#endif

public:
	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.


	//~ Begin AActor Interface.
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	virtual bool IsSelectable() const override { return false; }
	virtual bool SupportsExternalPackaging() const override { return false; }
	ENGINE_API virtual bool CanDeleteSelectedActor(FText& OutReason) const override;
	ENGINE_API virtual bool CanReplaceSelectedActor(FText& OutReason) const override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PreInitializeComponents() override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	//~ End AActor Interface.

	ENGINE_API UWorldPartition* GetWorldPartition() const;
	bool IsPartitionedWorld() const { return GetWorldPartition() != nullptr; }
	ENGINE_API void SetWorldPartition(UWorldPartition* InWorldPartition);
	ENGINE_API void ApplyWorldPartitionForcedSettings();

	virtual bool SupportsWorldPartitionStreaming() const { return true; }

#if WITH_EDITOR
	ENGINE_API void SupportsWorldPartitionStreamingChanged();
#endif

	/**
	 * Returns the Z component of the current world gravity and initializes it to the default
	 * gravity if called for the first time.
	 *
	 * @return Z component of current world gravity.
	 */
	ENGINE_API virtual float GetGravityZ() const;

	/**
	 * Multiplies all time dilation sources together and returns the effective overall time dilation. When bAllowTimeDilation is
	 * set to false, this will return 1.0 regardless of the time dilation sources. See SetAllowTimeDilation(...)/bAllowTimeDilation.
	 */
	virtual float GetEffectiveTimeDilation() const
	{
		if (bAllowTimeDilation)
		{
			return TimeDilation * CinematicTimeDilation * DemoPlayTimeDilation;
		}

		return 1.0f;
	}

	/**
	 * Returns the delta time to be used by the tick. Can be overridden if game specific logic is needed.
	 */
	ENGINE_API virtual float FixupDeltaSeconds(float DeltaSeconds, float RealDeltaSeconds);

	/** Sets the global time dilation value (subject to clamping). Returns the final value that was set (which may be clamped) even if bAllowTimeDilation is set to false. */
	ENGINE_API virtual float SetTimeDilation(float NewTimeDilation);

	/** Sets whether or not Time Dilation is allowed. When true, GetEffectiveTimeDilation() will return the time dilation as determined by the various dilation factors, when false it will return 1.0f. */
	ENGINE_API virtual void SetAllowTimeDilation(bool NewAllowTimeDilation);

	/** Sets the cinematic time dilation value (subject to clamping). Returns the final value that was set (which may be clamped) even if bAllowTimeDilation is set to false. */
	ENGINE_API virtual float SetCinematicTimeDilation(float NewCinematicTimeDilation);

	/** @return configuration for NavigationSystem's creation. Null means
	 *	no navigation system will be created*/
	UNavigationSystemConfig* const GetNavigationSystemConfig() const { return NavigationSystemConfigOverride ? NavigationSystemConfigOverride : NavigationSystemConfig; }

	/** 
	 * Sets a configuration override for NavigationSystem's creation. 
	 * If set, GetNavigationSystemConfig will return this configuration instead NavigationSystemConfig. 
	 */
	ENGINE_API void SetNavigationSystemConfigOverride(UNavigationSystemConfig* NewConfig);

	/** @return current configuration override for NavigationSystem's creation, if any. */
	const UNavigationSystemConfig* GetNavigationSystemConfigOverride() const { return NavigationSystemConfigOverride; }

	/** @return whether given world is configured to host any NavigationSystem */
	ENGINE_API bool IsNavigationSystemEnabled() const;

	/** @return whether given world is configured to host an AISystem */
	bool IsAISystemEnabled() const { return bEnableAISystem; }

	/** @return whether given world is restricting actors to +-HALF_WORLD_MAX bounds, destroying actors that move below KillZ */
	ENGINE_API bool AreWorldBoundsChecksEnabled() const;
	
	/**
	 * Called from GameStateBase, calls BeginPlay on all actors
	 */
	ENGINE_API virtual void NotifyBeginPlay();

	/** 
	 * Called from GameStateBase, used to notify native classes of match startup (such as level scripting)
	 */	
	ENGINE_API virtual void NotifyMatchStarted();

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITOR
	ENGINE_API const TArray<FHierarchicalSimplification>& GetHierarchicalLODSetup() const;
	ENGINE_API TArray<FHierarchicalSimplification>& GetHierarchicalLODSetup();
	ENGINE_API int32 GetNumHierarchicalLODLevels() const;
	ENGINE_API UMaterialInterface* GetHierarchicalLODBaseMaterial() const;
	ENGINE_API void ResetHierarchicalLODSetup();

	ENGINE_API void SaveDefaultWorldPartitionSettings();
	ENGINE_API void ResetDefaultWorldPartitionSettings();
	ENGINE_API const FWorldPartitionPerWorldSettings* GetDefaultWorldPartitionSettings() const;
#endif // WITH EDITOR

	inline class APlayerState* GetPauserPlayerState() const { return PauserPlayerState; }
	inline virtual void SetPauserPlayerState(class APlayerState* PlayerState) { PauserPlayerState = PlayerState; }

	ENGINE_API virtual void RewindForReplay() override;

private:

	// Hidden functions that don't make sense to use on this class.
	HIDE_ACTOR_TRANSFORM_FUNCTIONS();

	ENGINE_API virtual void Serialize( FArchive& Ar ) override;

private:
	ENGINE_API void InternalPostPropertyChanged(FName PropertyName);

	ENGINE_API void AdjustNumberOfBookmarks();
	ENGINE_API void UpdateNumberOfBookmarks();

	ENGINE_API void SanitizeBookmarkClasses();
	ENGINE_API void UpdateBookmarkClass();

	/**
	 * Maximum number of bookmarks allowed.
	 * Changing this will change the allocation of the bookmarks array, and when shrinking
	 * may cause some bookmarks to become eligible for GC.
	 */
	UPROPERTY(Config, Meta=(ClampMin=0))
	int32 MaxNumberOfBookmarks;

	/**
	 * Class that will be used when creating new bookmarks.
	 * Old bookmarks may be recreated with the new class where possible.
	 */
	UPROPERTY(Config, Meta=(AllowAbstract="false", ExactClass="false", AllowedClasses="/Script/Engine.BookmarkBase"))
	TSubclassOf<class UBookmarkBase> DefaultBookmarkClass;

	UPROPERTY()
	TArray<TObjectPtr<class UBookmarkBase>> BookmarkArray;

	// Tracked so we can detect changes from Config
	UPROPERTY()
	TSubclassOf<class UBookmarkBase> LastBookmarkClass;

#if WITH_EDITORONLY_DATA
	// Bookmark that will be applied when the level is loaded in the editor.
	UPROPERTY(EditAnywhere, Category=Bookmark)
	TObjectPtr<class UBookmarkBase> DefaultBookmark;
#endif 

public:
	ENGINE_API virtual FSoftClassPath GetAISystemClassName() const;

	/**
	 * The number of bookmarks that will have mapped keyboard shortcuts by default.
	 */
	static const uint32 NumMappedBookmarks = 10;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBookmarkClassChanged, AWorldSettings*);
	static ENGINE_API FOnBookmarkClassChanged OnBookmarkClassChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNumberOfBookmarksChanged, AWorldSettings*);
	static ENGINE_API FOnNumberOfBookmarksChanged OnNumberOfBoomarksChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDefaultBookmarkChanged, AWorldSettings*);
	static ENGINE_API FOnDefaultBookmarkChanged OnDefaultBookmarkChanged;
#endif

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNaniteSettingsChanged, AWorldSettings*);
	static ENGINE_API FOnNaniteSettingsChanged OnNaniteSettingsChanged;

	const int32 GetMaxNumberOfBookmarks() const
	{
		return MaxNumberOfBookmarks;
	}

	TSubclassOf<class UBookmarkBase> GetDefaultBookmarkClass() const
	{
		return DefaultBookmarkClass;
	}

	/**
	 * Gets the array of bookmarks.
	 * It's common for entries to be null as this is treated more like a sparse array.
	 */
	const TArray<class UBookmarkBase*>& GetBookmarks() const
	{
		return BookmarkArray;
	}

	/**
	 * Attempts to move bookmarks such that all bookmarks are adjacent in memory.
	 *
	 * Note, this will not rearrange any valid Bookmarks inside the mapped range, but
	 * may move bookmarks outside that range to fill up mapped bookmarks.
	 */
	ENGINE_API void CompactBookmarks();

	/**
	 * Gets the bookmark at the specified index, creating it if a bookmark doesn't exist.
	 *
	 * This will fail if the specified index is greater than MaxNumberOfBookmarks.
	 *
	 * For "plain" access that doesn't cause reallocation, use GetBookmarks
	 *
	 * @param bRecreateOnClassMismatch	Whether or not we should recreate an existing bookmark if it's class
	 *									doesn't match the default bookmark class.
	 */
	ENGINE_API class UBookmarkBase* GetOrAddBookmark(const uint32 BookmarkIndex, const bool bRecreateOnClassMismatch);

	/**
	 * Creates and adds a new bookmark of a different class.
	 *
	 * When the bookmark's class is not of the same class as the default bookmark class, the bookmark
	 * will be removed on the next update.
	 * This will fail if we've overrun MaxNumberOfBookmarks.
	 *
	 * @param	BookmarkClass	Class of the new bookmark.
	 * @param	bExpandIfNecessary	Will increase MaxNumberOfBookmarks if there's not enough add another.
	 *
	 * @return	The bookmark that was created.
	 *			Will be nullptr on failure.
	 */
	ENGINE_API class UBookmarkBase* AddBookmark(const TSubclassOf<UBookmarkBase> BookmarkClass, const bool bExpandIfNecessary);

	/**
	 * Clears the reference to the bookmark from the specified index.
	 */
	ENGINE_API void ClearBookmark(const uint32 BookmarkIndex);

	/**
	 * Clears all references to current bookmarks.
	 */
	ENGINE_API void ClearAllBookmarks();

#if WITH_EDITOR
	/**
	 * Get the bookmark that will be applied when the map is opened in the editor.
	 */
	ENGINE_API class UBookmarkBase* GetDefaultBookmark() const;

	/**
	 * Set the bookmark that will be applied when the map is opened in the editor.
	 */
	ENGINE_API void SetDefaultBookmark(class UBookmarkBase* InDefaultBookmark);
#endif

#if WITH_EDITORONLY_DATA
private: //DEPRECATED
	UPROPERTY()
	bool bEnableHierarchicalLODSystem_DEPRECATED;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="As of UE 5.1 all worlds are large. Set UE_USE_UE4_WORLD_MAX in EngineDefines.h to alter this."))
	uint8 bEnableLargeWorlds_DEPRECATED:1;
#endif
};
