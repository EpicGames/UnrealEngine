// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GenericPlatform/IInputInterface.h"
#include "HapticFeedbackEffect_Base.generated.h"

struct FHapticFeedbackValues;

UCLASS(MinimalAPI, BlueprintType)
class UHapticFeedbackEffect_Base : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void Initialize(FHapticFeedbackBuffer& HapticBuffer) {};

	virtual void GetValues(const float EvalTime, FHapticFeedbackValues& Values);

	virtual float GetDuration() const;
};


USTRUCT()
struct FActiveHapticFeedbackEffect
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<class UHapticFeedbackEffect_Base> HapticEffect;

	FActiveHapticFeedbackEffect()
		: HapticEffect(nullptr)
		, PlayTime(0.f)
		, Scale(1.f)
	{
	}

	FActiveHapticFeedbackEffect(UHapticFeedbackEffect_Base* InEffect, float InScale, bool bInLoop)
		: HapticEffect(InEffect)
		, bLoop(bInLoop)
		, PlayTime(0.f)
	{
		Scale = FMath::Clamp(InScale, 0.f, 10.f);
		HapticEffect->Initialize(HapticBuffer);
	}

	void Restart()
	{
		PlayTime = 0.f;
		HapticEffect->Initialize(HapticBuffer);
	}

	ENGINE_API bool Update(const float DeltaTime, struct FHapticFeedbackValues& Values);

	bool bLoop;

private:
	float PlayTime;
	float Scale;
	FHapticFeedbackBuffer HapticBuffer;
};
