// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "GenericPlatform/IInputInterface.h"
#include "HapticFeedbackEffect_Buffer.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UHapticFeedbackEffect_Buffer : public UHapticFeedbackEffect_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = "HapticFeedbackEffect_Buffer")
	TArray<uint8> Amplitudes;

	UPROPERTY(EditAnywhere, Category = "HapticFeedbackEffect_Buffer")
	int SampleRate;

	~UHapticFeedbackEffect_Buffer();
	
	void GetValues(const float EvalTime, FHapticFeedbackValues& Values) override;

	float GetDuration() const override;
	
	void Initialize(FHapticFeedbackBuffer& HapticBuffer) override;
};
