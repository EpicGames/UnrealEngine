// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/CurveFloat.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "HapticFeedbackEffect_Curve.generated.h"

struct FHapticFeedbackValues;

USTRUCT()
struct FHapticFeedbackDetails_Curve
{
	GENERATED_USTRUCT_BODY()

	/** The frequency to vibrate the haptic device at.  Frequency ranges vary by device! */
	UPROPERTY(EditAnywhere, Category = "HapticDetails")
	FRuntimeFloatCurve	Frequency;

	/** The amplitude to vibrate the haptic device at.  Amplitudes are normalized over the range [0.0, 1.0], with 1.0 being the max setting of the device */
	UPROPERTY(EditAnywhere, Category = "HapticDetails")
	FRuntimeFloatCurve	Amplitude;

	FHapticFeedbackDetails_Curve()
	{
	}
};

UCLASS(MinimalAPI, BlueprintType)
class UHapticFeedbackEffect_Curve : public UHapticFeedbackEffect_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = "HapticFeedbackEffect")
	FHapticFeedbackDetails_Curve HapticDetails;

	void GetValues(const float EvalTime, FHapticFeedbackValues& Values) override;

	float GetDuration() const override;
};
