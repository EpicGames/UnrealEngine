// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "GenericPlatform/IInputInterface.h"
#include "HapticFeedbackEffect_SoundWave.generated.h"

class USoundWave;

UCLASS(MinimalAPI, BlueprintType)
class UHapticFeedbackEffect_SoundWave : public UHapticFeedbackEffect_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = "HapticFeedbackEffect_SoundWave")
	TObjectPtr<USoundWave> SoundWave;

	/** If true on a vr controller the left and right stereo channels would be applied to the left and right controller, respectively. */
	UPROPERTY(EditAnywhere, Category = "HapticFeedbackEffect_SoundWave")
	bool bUseStereo;

	~UHapticFeedbackEffect_SoundWave();

	void Initialize(FHapticFeedbackBuffer& HapticBuffer) override;

	void GetValues(const float EvalTime, FHapticFeedbackValues& Values) override;

	float GetDuration() const override;

private:
	void PrepareSoundWaveBuffer();
	void PrepareSoundWaveMonoBuffer(uint8* PCMData, int32 RawPCMDataSize);
	void PrepareSoundWaveStereoBuffer(uint8* PCMData, int32 RawPCMDataSize);
	bool bPrepared;

	TArray<uint8> RawData;
};
