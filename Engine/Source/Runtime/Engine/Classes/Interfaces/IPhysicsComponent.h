// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicsObject.h"
#include "Chaos/Serialization/SolverSerializer.h"
#include "Containers/Array.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "IPhysicsComponent.generated.h"

/** Interface for components that contains physics bodies. **/
UINTERFACE(MinimalApi, Experimental, meta = (CannotImplementInterfaceInBlueprint))
class UPhysicsComponent : public UInterface
{
	GENERATED_BODY()
};

class IPhysicsComponent
{
	GENERATED_BODY()

public:
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const = 0;
	virtual Chaos::FPhysicsObject* GetPhysicsObjectByName(const FName& Name) const = 0;
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const = 0;
	
	virtual Chaos::FPhysicsObjectId GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const = 0;

#if UE_WITH_REMOTE_OBJECT_HANDLE

protected:
	virtual void MigratePhysicsState(FArchive& Ar) = 0;
	virtual void ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData = nullptr) = 0;
public:

#endif
};

template<typename TId>
TArray<Chaos::FPhysicsObjectHandle> GetAllPhysicsObjectsById(IPhysicsComponent* Component, const TArray<TId>& AllIds)
{
	constexpr bool bIsId = std::is_same_v<TId, Chaos::FPhysicsObjectId>;
	constexpr bool bIsName = std::is_same_v<TId, FName>;

	static_assert(bIsId || bIsName, "Invalid ID type passed to GetAllPhysicsObjectsById");
	if (!Component)
	{
		return {};
	}

	TArray<Chaos::FPhysicsObjectHandle> Objects;
	Objects.Reserve(AllIds.Num());

	for (const TId& Id : AllIds)
	{
		if constexpr (bIsId)
		{
			Objects.Add(Component->GetPhysicsObjectById(Id));
		}
		else if constexpr (bIsName)
		{
			Objects.Add(Component->GetPhysicsObjectByName(Id));
		}
	}

	return Objects;
}