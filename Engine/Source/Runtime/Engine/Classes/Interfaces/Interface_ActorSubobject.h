// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Templates/Casts.h"
#include "UObject/Interface.h"
#include "Interface_ActorSubobject.generated.h"

class UAssetUserData;

/** Interface for actor subobjects. WARNING: Experimental. This interface may change or be removed without notice. **/
UINTERFACE(MinimalApi, Experimental, meta=(CannotImplementInterfaceInBlueprint))
class UInterface_ActorSubobject : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IInterface_ActorSubobject
{
	GENERATED_IINTERFACE_BODY()

	virtual void OnCreatedFromReplication() {}
	virtual void OnDestroyedFromReplication() {}

};

