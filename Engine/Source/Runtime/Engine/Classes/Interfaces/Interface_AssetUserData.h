// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/Casts.h"
#include "UObject/Interface.h"
#include "Engine/AssetUserData.h"
#include "Interface_AssetUserData.generated.h"

template <typename T> class TSubclassOf;

class UAssetUserData;

/** Interface for assets/objects that can own UserData **/
UINTERFACE(MinimalApi, meta=(CannotImplementInterfaceInBlueprint))
class UInterface_AssetUserData : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IInterface_AssetUserData
{
	GENERATED_IINTERFACE_BODY()

	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData);
	
	/**
	* Returns an instance of the provided AssetUserData class if it's contained in the target asset.
	*
	* @param	InUserDataClass		UAssetUserData sub class to get
	*
	* @return	The instance of the UAssetUserData class contained, or null if it doesn't exist
	*/
	UFUNCTION(BlueprintCallable, Category=AssetUserData, meta=(DeterminesOutputType="InUserDataClass"))
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass);

	/**
	* Checks whether or not an instance of the provided AssetUserData class is contained.
	*
	* @param	InUserDataClass		UAssetUserData sub class to check for
	*
	* @return	Whether or not an instance of InUserDataClass was found
	*/
	UFUNCTION(BlueprintCallable, Category=AssetUserData)
	ENGINE_API virtual bool HasAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass);

	/**
	* Creates and adds an instance of the provided AssetUserData class to the target asset.
	*
	* @param	InUserDataClass		UAssetUserData sub class to create
	*
	* @return	Whether or not an instance of InUserDataClass was succesfully added
	*/
	UFUNCTION(BlueprintCallable, Category=AssetUserData)
	ENGINE_API virtual bool AddAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass);

	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const;

	template<typename T>
	T* GetAssetUserData()
	{
		return Cast<T>( GetAssetUserDataOfClass(T::StaticClass()) );
	}

	template<typename T>
	T* GetAssetUserDataChecked()
	{
		return CastChecked<T>(GetAssetUserDataOfClass(T::StaticClass()));
	}

	template<typename T>
	bool HasAssetUserData()
	{
		return HasAssetUserDataOfClass(T::StaticClass());
	}

	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass);

};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/SubclassOf.h"
#endif
