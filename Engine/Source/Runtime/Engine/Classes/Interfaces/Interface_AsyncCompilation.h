// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Interface for objects that provide async compilation
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Interface_AsyncCompilation.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UInterface_AsyncCompilation : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IInterface_AsyncCompilation
{
	GENERATED_IINTERFACE_BODY()

#if WITH_EDITOR
	/**	Returns whether or not the asset is currently being compiled */
	virtual bool IsCompiling() const = 0;
#else
	/**	Returns whether or not the asset is currently being compiled */
	virtual bool IsCompiling() const { return false; }
#endif
};

