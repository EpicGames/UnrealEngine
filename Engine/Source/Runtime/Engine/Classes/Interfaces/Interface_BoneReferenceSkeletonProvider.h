// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Interface_BoneReferenceSkeletonProvider.generated.h"

UINTERFACE(MinimalAPI)
class UBoneReferenceSkeletonProvider : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for objects to provide skeletons that can be used with FBoneReference's details customization.
 */
class IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()

public:

	UE_DEPRECATED(5.0, "Please use GetSkeleton(bool& bInvalidSkeletonIsError, const class IPropertyHandle* PropertyHandle)")
	virtual class USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError) { return GetSkeleton(bInvalidSkeletonIsError, nullptr); }

	/**
	 * Called to get the skeleton that FBoneReference's details customization will use to populate
	 * bone names.
	 *
	 * @param [out] bInvalidSkeletonIsError		When true, returning an invalid skeleton will be treated as an error.
	 *
	 * @return The skeleton we should use.
	 */
	virtual class USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const class IPropertyHandle* PropertyHandle) PURE_VIRTUAL(IBoneReferenceSkeletonProvider::GetSkeleton, return nullptr; );
};
