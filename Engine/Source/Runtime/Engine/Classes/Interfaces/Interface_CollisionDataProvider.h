// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Interface for objects that have a PhysX collision representation and need their geometry cooked
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Interface_CollisionDataProviderCore.h"
#include "Interface_CollisionDataProvider.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UInterface_CollisionDataProvider : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IInterface_CollisionDataProvider
{
	GENERATED_IINTERFACE_BODY()


	/**	 Interface for retrieving triangle mesh collision data from the implementing object 
	 *
	 * @param CollisionData - structure given by the caller to be filled with tri mesh collision data
	 * @return true if successful, false if unable to successfully fill in data structure
	 */
	virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) { return false; }

	/**	Returns an estimate of how much data would be retrieved by GetPhysicsTriMeshData.
	 *
	 * @param OutTriMeshEstimates - structure given by the caller to be filled with tri mesh estimate data
	 * @return true if successful, false if unable to successfully fill in data structure
	 */
	virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const { return false; }

	/**	 Interface for checking if the implementing objects contains triangle mesh collision data 
	 *
	 * @return true if the implementing object contains triangle mesh data, false otherwise
	 */
	virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const { return false; }

	/**
	 * Poll for availability of asynchronously-populated triangle mesh collision data
	 *
	 * @return true if the triangle mesh collision can be accessed without blocking, false otherwise
	 */
	virtual bool PollAsyncPhysicsTriMeshData(bool InUseAllTriData) const { return true; }

	/** Do we want to create a negative version of this mesh */
	virtual bool WantsNegXTriMesh() { return false; }

	/** An optional string identifying the mesh data. */
	virtual void GetMeshId(FString& OutMeshId) {}
};

