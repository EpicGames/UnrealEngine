// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Interface_PostProcessVolume.generated.h"

#define DEBUG_POST_PROCESS_VOLUME_ENABLE (!(UE_BUILD_SHIPPING))

struct FPostProcessSettings;

struct FPostProcessVolumeProperties
{
	const FPostProcessSettings* Settings;
	float Priority;
	float BlendRadius;
	float BlendWeight;
	bool bIsEnabled;
	bool bIsUnbound;

	// Size (volume of the PPV's bounding box), is used as an additional sort key if Priority is equal.  Smaller volumes take higher
	// priority.  Unbounded volumes are treated as having max volume.  If volume is also equal, an arbitrary guid is used as a final
	// sort key.  The goal is to improve determinism in the order equal Priority volumes are processed, which otherwise is affected
	// by the order in which they were loaded, which can vary between editor and engine, platform, and due to streaming.
	double Size = DBL_MAX;
	FGuid VolumeGuid;
};

/** Interface for general PostProcessVolume access **/
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UInterface_PostProcessVolume : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IInterface_PostProcessVolume
{
	GENERATED_IINTERFACE_BODY()

	// Returns true if the sphere around Point with radius SphereRadius intersects with the volume
	// OutDistanceToPoint is set to the distance to the nearest point on the volume,
	// or may be set to FLT_MAX if Point is outside the blending range of the volume.
	virtual bool EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint) = 0;
	virtual FPostProcessVolumeProperties GetProperties() const = 0;
#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	virtual FString GetDebugName() const = 0;
#endif
};
