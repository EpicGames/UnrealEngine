// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Interface_PreviewMeshProvider.generated.h"

class USkeletalMesh;

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UInterface_PreviewMeshProvider : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/** An asset that can provide a preview skeletal mesh (for editing) */
class IInterface_PreviewMeshProvider
{
	GENERATED_IINTERFACE_BODY()

public:
	/** 
	 * Set the preview mesh for this asset 
	 * @param	bMarkAsDirty	Passing true will call Modify() on the asset
	 */
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) = 0;

	/** Get the preview mesh for this asset */
	virtual USkeletalMesh* GetPreviewMesh() const = 0;

	/** 
	 * Get the preview mesh for this asset, non const. Allows the preview mesh to be modified if it is somehow invalid 
	 * @param	bFindIfNotSet	If true, attempts to find a suitable asset if one is not found
	 */
	virtual USkeletalMesh* GetPreviewMesh(bool bFindIfNotSet = false) { return static_cast<const IInterface_PreviewMeshProvider*>(this)->GetPreviewMesh(); }
};
