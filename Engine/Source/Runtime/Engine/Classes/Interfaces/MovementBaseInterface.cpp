// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementBaseInterface.h"

#include "IPhysicsComponent.h"
#include "Engine/HitResult.h"

DEFINE_LOG_CATEGORY_STATIC(LogCharacterMovementBase, Log, All);

FMovementBaseInterfaceData::FMovementBaseInterfaceData()
	: PhysicsObjectOwner(nullptr)
	, BodyInstanceOwner(nullptr)
{
}

FMovementBaseInterfaceData::FMovementBaseInterfaceData(const FMovementBaseInterfaceData* OtherMovementBaseData)
{
	if (OtherMovementBaseData)
	{
		BodyInstanceOwner = OtherMovementBaseData->BodyInstanceOwner;
		PhysicsObjectOwner = OtherMovementBaseData->PhysicsObjectOwner;
	}
	else
	{
		BodyInstanceOwner = nullptr;
		PhysicsObjectOwner = nullptr;
	}
}

FMovementBaseInterfaceData::FMovementBaseInterfaceData(TObjectPtr<UObject> PhysicsObjectOwner)
{
	Set(PhysicsObjectOwner);
}

FMovementBaseInterfaceData::FMovementBaseInterfaceData(TObjectPtr<UObject> PhysicsObjectOwner, IPhysicsBodyInstanceOwner* PhysicsBodyInstanceOwner)
	: PhysicsObjectOwner(PhysicsObjectOwner)
	, BodyInstanceOwner(PhysicsBodyInstanceOwner)
{
}

bool FMovementBaseInterfaceData::operator==(const FMovementBaseInterfaceData& Other) const
{
	return PhysicsObjectOwner == Other.PhysicsObjectOwner && BodyInstanceOwner == Other.BodyInstanceOwner;
}

bool FMovementBaseInterfaceData::operator!=(const FMovementBaseInterfaceData& Other) const
{
	return !(FMovementBaseInterfaceData::operator==(Other));
}

bool FMovementBaseInterfaceData::IsValid() const
{
	return PhysicsObjectOwner.IsValid() && BodyInstanceOwner != nullptr;
}

void FMovementBaseInterfaceData::Clear()
{
	PhysicsObjectOwner = nullptr;
	BodyInstanceOwner = nullptr;
}

void FMovementBaseInterfaceData::Set(TObjectPtr<UObject> NewPhysicsObjectOwner, const IPhysicsBodyInstanceOwner* NewPhysicsBodyInstanceOwner)
{
	PhysicsObjectOwner = NewPhysicsObjectOwner;
	BodyInstanceOwner = NewPhysicsBodyInstanceOwner;
}

void FMovementBaseInterfaceData::Set(TObjectPtr<UObject> NewPhysicsObjectOwner)
{
	IPhysicsBodyInstanceOwner* NewBodyInstanceOwner = nullptr;
	
	if (NewPhysicsObjectOwner)
	{
		// Todo: Pass along some sort of ID so we can get something useful for objects owning multiple physics objects
		if (IPhysicsBodyInstanceOwnerResolver* PhysicsObjectOwnerResolver = Cast<IPhysicsBodyInstanceOwnerResolver>(NewPhysicsObjectOwner))
		{
			Chaos::FConstPhysicsObjectHandle PhysicsObject = nullptr;
			if (const IPhysicsComponent* PhysicsComponent = Cast<IPhysicsComponent>(NewPhysicsObjectOwner))
			{
				PhysicsObject = PhysicsComponent->GetPhysicsObjectById(0);
			}
				
			NewBodyInstanceOwner = PhysicsObjectOwnerResolver->ResolvePhysicsBodyInstanceOwner(PhysicsObject);
		}
		else
		{
			UE_LOGF(LogCharacterMovementBase, Warning, "%ls: UObject passed into FMovementBaseInterfaceData::Set is expected to implement IPhysicsBodyInstanceOwnerResolver. If the Object passed in implements IPhysicsBodyInstanceOwner directly consider using FMovementBaseInterfaceData::Set(TObjectPtr<UObject> NewPhysicsObjectOwner, const IPhysicsBodyInstanceOwner* NewPhysicsBodyInstanceOwner) instead.", *GetNameSafe(NewPhysicsObjectOwner));
		}
	}
	
	PhysicsObjectOwner = NewPhysicsObjectOwner;
	BodyInstanceOwner = NewBodyInstanceOwner;
}

const IPhysicsBodyInstanceOwner* FMovementBaseInterfaceData::GetBodyInstanceOwner() const
{
	return PhysicsObjectOwner.IsValid() ? BodyInstanceOwner : nullptr;
}

UObject* FMovementBaseInterfaceData::GetMovementBaseObject() const
{
	return IsValid() ? BodyInstanceOwner->GetSourceObject() : nullptr;
}

UObject* FMovementBaseInterfaceData::GetMovementBaseObjectOwner() const
{
	return IsValid() ? BodyInstanceOwner->GetSourceObjectOwner() : nullptr;
}
