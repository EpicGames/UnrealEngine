// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "UObject/Interface.h"
#include "MovementBaseInterface.generated.h"

#define UE_API ENGINE_API

USTRUCT()
struct FMovementBaseInterfaceData
{
	GENERATED_BODY()

	// UObject that owns the physics representation of the movement base object
	UPROPERTY()
	TWeakObjectPtr<UObject> PhysicsObjectOwner;

protected:
	// IPhysicsBodyInstanceOwner reference that implements movement base functionality 
	const IPhysicsBodyInstanceOwner* BodyInstanceOwner;
	
public:
	UE_API FMovementBaseInterfaceData();
	UE_API FMovementBaseInterfaceData(const FMovementBaseInterfaceData* OtherMovementBaseData);
	UE_API FMovementBaseInterfaceData(TObjectPtr<UObject> PhysicsObjectOwner);
	UE_API FMovementBaseInterfaceData(TObjectPtr<UObject> PhysicsObjectOwner, IPhysicsBodyInstanceOwner* PhysicsBodyInstanceOwner);
	
	UE_API bool operator==(const FMovementBaseInterfaceData& Other) const;
	UE_API bool operator!=(const FMovementBaseInterfaceData& Other) const;
	explicit operator bool() const { return IsValid(); }
	
	/** Dangerous access: Ensures validity before returning */
	const IPhysicsBodyInstanceOwner* operator->() const 
	{ 
		const IPhysicsBodyInstanceOwner* Resolved = GetBodyInstanceOwner();
		check(Resolved);
		return Resolved; 
	}

	/** Returns true if this struct has all relevant movement base data set */
	UE_API bool IsValid() const;

	/** Clears relevant movement base data */
	UE_API void Clear();
	
	/** Sets relevant movement base data */
	UE_API void Set(TObjectPtr<UObject> NewPhysicsObjectOwner, const IPhysicsBodyInstanceOwner* NewPhysicsBodyInstanceOwner);
	
	/** 
	 * Sets relevant movement base data. 
	 * Uses IPhysicsBodyInstanceOwnerResolver and IPhysicsComponent to find and set IPhysicsBodyInstanceOwner
	 */
	UE_API void Set(TObjectPtr<UObject> NewPhysicsObjectOwner);
	
	/**
	 * Returns IPhysicsBodyInstanceOwner that implements movement base functionality
	 * Note: The IPhysicsBodyInstanceOwner returned is only valid if the PhysicsObjectOwner is as well so the IPhysicsBodyInstanceOwner returned shouldn't be cached
	 */
	UE_API const IPhysicsBodyInstanceOwner* GetBodyInstanceOwner() const;

	/**
	 * Helper function that returns UObject that owns movement base functionality
	 */
	UE_API UObject* GetMovementBaseObject() const;

	/**
	 * Helper function that returns the owner of the UObject that owns movement base functionality
	 */
	UE_API UObject* GetMovementBaseObjectOwner() const;
};

#undef UE_API
