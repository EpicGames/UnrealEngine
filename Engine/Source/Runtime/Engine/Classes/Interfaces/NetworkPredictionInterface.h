// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// INetworkPredictionInterface is an interface for objects that want to perform
// network prediction of movement. See UCharacterMovementComponent for an example implementation.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "NetworkPredictionInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNetworkPredictionInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INetworkPredictionInterface
{
	GENERATED_IINTERFACE_BODY()

	//--------------------------------
	// Server hooks
	//--------------------------------

	/** (Server) Send position to client if necessary, or just ack good moves. */
	virtual void SendClientAdjustment()					PURE_VIRTUAL(INetworkPredictionInterface::SendClientAdjustment,);

	/** (Server) Trigger a position update on clients, if the server hasn't heard from them in a while. @return Whether movement is performed. */
	virtual bool ForcePositionUpdate(float DeltaTime)	PURE_VIRTUAL(INetworkPredictionInterface::ForcePositionUpdate, return false;);

	//--------------------------------
	// Client hooks
	//--------------------------------

	/** (Client) After receiving a network update of position, allow some custom smoothing, given the old transform before the correction and new transform from the update. */
	virtual void SmoothCorrection(const FVector& OldLocation, const FQuat& OldRotation, const FVector& NewLocation, const FQuat& NewRotation) PURE_VIRTUAL(INetworkPredictionInterface::SmoothCorrection,);

	//--------------------------------
	// Other
	//--------------------------------

	/** @return FNetworkPredictionData_Client instance used for network prediction. */
	virtual class FNetworkPredictionData_Client* GetPredictionData_Client() const PURE_VIRTUAL(INetworkPredictionInterface::GetPredictionData_Client, return NULL;);
	
	/** @return FNetworkPredictionData_Server instance used for network prediction. */
	virtual class FNetworkPredictionData_Server* GetPredictionData_Server() const PURE_VIRTUAL(INetworkPredictionInterface::GetPredictionData_Server, return NULL;);

	/** Accessor to check if there is already client data, without potentially allocating it on demand.*/
	virtual bool HasPredictionData_Client() const PURE_VIRTUAL(INetworkPredictionInterface::HasPredictionData_Client, return false;);

	/** Accessor to check if there is already server data, without potentially allocating it on demand.*/
	virtual bool HasPredictionData_Server() const PURE_VIRTUAL(INetworkPredictionInterface::HasPredictionData_Server, return false;);

	/** Resets client prediction data. */
	virtual void ResetPredictionData_Client() PURE_VIRTUAL(INetworkPredictionInterface::ResetPredictionData_Client,);

	/** Resets server prediction data. */
	virtual void ResetPredictionData_Server() PURE_VIRTUAL(INetworkPredictionInterface::ResetPredictionData_Server,);
};


// Network data representation on the client
class FNetworkPredictionData_Client
{
public:

	FNetworkPredictionData_Client()
	{
	}

	virtual ~FNetworkPredictionData_Client() = default;

	/** Can be overridden to allow references to be accounted for during GC */
	virtual void AddStructReferencedObjects(FReferenceCollector& Collector) const {};
};


// Network data representation on the server
class FNetworkPredictionData_Server
{
public:

	FNetworkPredictionData_Server()
	: ServerTimeStamp(0.f)
	, ServerTimeBeginningForcedUpdates(0.f)
	, ServerTimeLastForcedUpdate(0.f)
	, bTriggeringForcedUpdates(false)
	, bForcedUpdateDurationExceeded(false)
	, bLastRequestNeedsForcedUpdates(false)
	{
	}

	virtual ~FNetworkPredictionData_Server() = default;

	virtual void ResetForcedUpdateState()
	{
		ServerTimeBeginningForcedUpdates = 0.0f;
		ServerTimeLastForcedUpdate = 0.0f;
		bTriggeringForcedUpdates = false;
		bForcedUpdateDurationExceeded = false;
	}

	/** Can be overridden to allow references to be accounted for during GC */
	virtual void AddStructReferencedObjects(FReferenceCollector& Collector) const {};

	/** Server clock time when last server move was received or movement was forced to be processed */
	float ServerTimeStamp;

	//////////////////////////////////////////////////////////////////////////
	// Forced update state

	/** Initial ServerTimeStamp that triggered a ForcedPositionUpdate series. Reset to 0 after no longer exceeding update interval. */
	float ServerTimeBeginningForcedUpdates;

	/** ServerTimeStamp last time we called ForcePositionUpdate. */
	float ServerTimeLastForcedUpdate;

	/** Set to true while requirements for ForcePositionUpdate interval are met, and set back to false after updates are received again. */
	uint8 bTriggeringForcedUpdates : 1;

	/** Set to true while bTriggeringForcedUpdates is true and after update duration has been exceeded (when server will stop forcing updates). */
	uint8 bForcedUpdateDurationExceeded : 1;

	/** Set to true if last received move request  is bad and needs correction */
	uint8 bLastRequestNeedsForcedUpdates : 1;

	virtual void Serialize(FArchive& Archive)
	{
		if (Archive.IsMigratingRemoteObjects())
		{
			Archive << ServerTimeStamp;
			Archive << ServerTimeBeginningForcedUpdates;
			Archive << ServerTimeLastForcedUpdate;
			
			bool bTempTriggeringForcedUpdates = bTriggeringForcedUpdates;
			Archive << bTempTriggeringForcedUpdates;
			bTriggeringForcedUpdates = bTempTriggeringForcedUpdates;

			bool bTempForcedUpdateDurationExceeded = bForcedUpdateDurationExceeded;
			Archive << bTempForcedUpdateDurationExceeded;
			bForcedUpdateDurationExceeded = bTempForcedUpdateDurationExceeded;

			bool bTempLastRequestNeedsForcedUpdates = bLastRequestNeedsForcedUpdates;
			Archive << bTempLastRequestNeedsForcedUpdates;
			bLastRequestNeedsForcedUpdates = bTempLastRequestNeedsForcedUpdates;
		}
	}
};

