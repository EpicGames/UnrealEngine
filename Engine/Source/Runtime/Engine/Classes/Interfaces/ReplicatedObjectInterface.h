// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Templates/Casts.h"
#include "UObject/Interface.h"
#include "ReplicatedObjectInterface.generated.h"

/** Interface for custom replicated objects. WARNING: Experimental. This interface may change or be removed without notice. **/
UINTERFACE(MinimalApi, Experimental, meta = (CannotImplementInterfaceInBlueprint))
class UReplicatedObjectInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IReplicatedObjectInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual bool ShouldClientDestroyWhenNotRelevant() { return false; }
	virtual void OnLostRelevancy() {}
	virtual bool IsStaticReplicator() { return false; }
};



