// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "BlueprintAsyncActionBase.generated.h"

class UGameInstance;

/**
* BlueprintCallable factory functions for classes which inherit from UBlueprintAsyncActionBase will have a special blueprint node created for it: UK2Node_AsyncAction
* You can stop this node spawning and create a more specific one by adding the UCLASS metadata "HasDedicatedAsyncNode"
*/

UCLASS(MinimalAPI)
class UBlueprintAsyncActionBase : public UObject
{
	GENERATED_BODY()
public:
	/** Default UObject constructor */
	ENGINE_API UBlueprintAsyncActionBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Called when the object is destroyed */
	ENGINE_API virtual void BeginDestroy() override;

	/** Called to trigger the action once the delegates have been bound */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true"))
	ENGINE_API virtual void Activate();

	/**
	 * Call to globally register this object with a game instance, it will not be destroyed until SetReadyToDestroy is called
	 * This allows having an action stay alive until SetReadyToDestroy is manually called, allowing it to be used inside loops or if the calling BP goes away
	 */
	ENGINE_API virtual void RegisterWithGameInstance(const UObject* WorldContextObject);

	/** Call when the action is completely done, this makes the action free to delete, and will unregister it with the game instance */
	ENGINE_API virtual void SetReadyToDestroy();

protected:
	ENGINE_API virtual void RegisterWithGameInstance(UGameInstance* GameInstance);

	TWeakObjectPtr<UGameInstance> RegisteredWithGameInstance;
};
