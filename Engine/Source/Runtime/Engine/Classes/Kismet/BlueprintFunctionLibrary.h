// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "BlueprintFunctionLibrary.generated.h"

// This class is a base class for any function libraries exposed to blueprints.
// Methods in subclasses are expected to be static, and no methods should be added to this base class.
UCLASS(Abstract, MinimalAPI)
class UBlueprintFunctionLibrary : public UObject
{
	GENERATED_UCLASS_BODY()

	// UObject interface
	ENGINE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	// End of UObject interface
};
