// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "StructUtils/InstancedStruct.h"
#include "BlueprintInstancedStructLibrary.generated.h"

struct FGenericStruct;

UENUM()
enum class EStructUtilsResult : uint8
{
	Valid,
	NotValid,
};

UCLASS(MinimalAPI)
class UBlueprintInstancedStructLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Create a new InstancedStruct from the given source value.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Utilities|Instanced Struct", meta = (CustomStructureParam = "Value", BlueprintInternalUseOnly="true", NativeMakeFunc))
	static ENGINE_API FInstancedStruct MakeInstancedStruct(const int32& Value);

	/**
	 * Sets the value of InstancedStruct from the given source value.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Utilities|Instanced Struct", meta = (CustomStructureParam = "Value", BlueprintInternalUseOnly="true"))
	static ENGINE_API void SetInstancedStructValue(UPARAM(Ref) FInstancedStruct& InstancedStruct, const int32& Value);

	/**
	 * Retrieves data from an InstancedStruct if it matches the output type.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Utilities|Instanced Struct", meta = (CustomStructureParam = "Value", ExpandEnumAsExecs = "ExecResult", BlueprintInternalUseOnly="true"))
	static ENGINE_API void GetInstancedStructValue(EStructUtilsResult& ExecResult, UPARAM(Ref) const FInstancedStruct& InstancedStruct, int32& Value);

	/**
	 * Resets an InstancedStruct.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Instanced Struct", meta = (AdvancedDisplay = "1"))
	static void Reset(UPARAM(Ref)FInstancedStruct& InstancedStruct, const UScriptStruct* StructType = nullptr)
	{
		InstancedStruct.InitializeAs(StructType, nullptr);
	}

	/** 
	 * Checks whether an InstancedStruct contains value.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Instanced Struct", meta = (DisplayName = "Is Valid (Branch)", ExpandEnumAsExecs = "ReturnValue"))
	static EStructUtilsResult IsInstancedStructValid(UPARAM(Ref) const FInstancedStruct& InstancedStruct)
	{
		return InstancedStruct.IsValid() ? EStructUtilsResult::Valid : EStructUtilsResult::NotValid;
	}

	/**
	 * Checks whether two InstancedStructs (and the values contained within) are equal.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Instanced Struct", meta = (CompactNodeTitle = "==", DisplayName = "Equal", Keywords = "== equal"))
	static bool EqualEqual_InstancedStruct(UPARAM(Ref) const FInstancedStruct& A, UPARAM(Ref) const FInstancedStruct& B) { return A == B; }

	/**
	 * Checks whether two InstancedStructs are not equal.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Instanced Struct", meta = (CompactNodeTitle = "!=", DisplayName = "Not Equal", Keywords = "!= not equal"))
	static bool NotEqual_InstancedStruct(UPARAM(Ref) const FInstancedStruct& A, UPARAM(Ref) const FInstancedStruct& B) { return A != B; }

	/**
	 * Checks whether the InstancedStruct contains value.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Instanced Struct", meta = ( DisplayName = "Is Valid"))
	static bool IsValid_InstancedStruct(UPARAM(Ref) const FInstancedStruct& InstancedStruct) { return InstancedStruct.IsValid(); }

private:
	DECLARE_FUNCTION(execMakeInstancedStruct);
	DECLARE_FUNCTION(execSetInstancedStructValue);
	DECLARE_FUNCTION(execGetInstancedStructValue);
};
