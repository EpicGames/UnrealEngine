// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintMapLibrary.generated.h"

UCLASS(meta=(BlueprintThreadSafe), MinimalAPI)
class UBlueprintMapLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Adds a key and value to the map. If something already uses the provided key it will be overwritten with the new value.
	 * After calling Key is guaranteed to be associated with Value until a subsequent mutation of the Map.
	 *
	 * @param	TargetMap		The map to add the key and value to
	 * @param	Key				The key that will be used to look the value up
	 * @param	Value			The value to be retrieved later
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Add", CompactNodeTitle = "ADD", MapParam = "TargetMap", MapKeyParam = "Key", MapValueParam = "Value", AutoCreateRefTerm = "Key, Value"), Category = "Utilities|Map")
	static ENGINE_API void Map_Add(const TMap<int32, int32>& TargetMap, const int32& Key, const int32& Value);
	
	/** 
	 * Removes a key and its associated value from the map.
	 *
	 * @param	TargetMap		The map to remove the key and its associated value from
	 * @param	Key				The key that will be used to look the value up
	 * @return	True if an item was removed (False indicates nothing in the map uses the provided key)
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Remove", CompactNodeTitle = "REMOVE", MapParam = "TargetMap", MapKeyParam = "Key",  AutoCreateRefTerm = "Key"), Category = "Utilities|Map")
	static ENGINE_API bool Map_Remove(const TMap<int32, int32>& TargetMap, const int32& Key);
	
	/** 
	 * Finds the value associated with the provided Key
	 *
	 * @param	TargetMap		The map to perform the lookup on
	 * @param	Key				The key that will be used to look the value up
	 * @param	Value			The value associated with the key, default constructed if key was not found
	 * @return	True if an item was found (False indicates nothing in the map uses the provided key)
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Find", CompactNodeTitle = "FIND", MapParam = "TargetMap", MapKeyParam = "Key", MapValueParam = "Value", AutoCreateRefTerm = "Key, Value", BlueprintThreadSafe), Category = "Utilities|Map")
	static ENGINE_API bool Map_Find(const TMap<int32, int32>& TargetMap, const int32& Key, int32& Value);
	
	/** 
	 * Checks whether key is in a provided Map
	 *
	 * @param	TargetMap		The map to perform the lookup on
	 * @param	Key				The key that will be used to lookup
	 * @return	True if an item was found (False indicates nothing in the map uses the provided key)
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Contains", CompactNodeTitle = "CONTAINS", MapParam = "TargetMap", MapKeyParam = "Key", AutoCreateRefTerm = "Key", BlueprintThreadSafe), Category = "Utilities|Map")
	static ENGINE_API bool Map_Contains(const TMap<int32, int32>& TargetMap, const int32& Key);
	
	/** 
	 * Outputs an array of all keys present in the map
	 *
	 * @param	TargetMap		The map to get the list of keys from
	 * @param	Keys			All keys present in the map
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Keys", CompactNodeTitle = "KEYS", MapParam = "TargetMap", MapKeyParam = "Keys", AutoCreateRefTerm = "Keys"), Category = "Utilities|Map")
	static ENGINE_API void Map_Keys(const TMap<int32, int32>& TargetMap, TArray<int32>& Keys);
	
	/** 
	 * Outputs an array of all values present in the map
	 *
	 * @param	TargetMap		The map to get the list of values from
	 * @param	Values			All values present in the map
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Values", CompactNodeTitle = "VALUES", MapParam = "TargetMap", MapValueParam = "Values", AutoCreateRefTerm = "Values"), Category = "Utilities|Map")
	static ENGINE_API void Map_Values(const TMap<int32, int32>& TargetMap, TArray<int32>& Values);

	/** 
	 * Determines the number of entries in a provided Map
	 *
	 * @param	TargetMap		The map in question
	 * @return	The number of entries in the map
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Length", CompactNodeTitle = "LENGTH", MapParam = "TargetMap", Keywords = "num size count", BlueprintThreadSafe), Category="Utilities|Map")
	static ENGINE_API int32 Map_Length(const TMap<int32, int32>& TargetMap);
	
	/**
	 * Check if the map does not have any entires
	 *
	 * @param	TargetMap		The map to check
	 * @return	A boolean indicating if the map has any entires
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Empty", CompactNodeTitle = "IS EMPTY", MapParam = "TargetMap"), Category = "Utilities|Map")
	static ENGINE_API bool Map_IsEmpty(const TMap<int32, int32>& TargetMap);

	/**
	 * Check if the map has any entries
	 *
	 * @param	TargetMap		The map to check
	 * @return	A boolean indicating if the map has any entires
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Not Empty", CompactNodeTitle = "IS NOT EMPTY", MapParam = "TargetMap"), Category = "Utilities|Map")
	static ENGINE_API bool Map_IsNotEmpty(const TMap<int32, int32>& TargetMap);

	/** 
	 * Clears a map of all entries, resetting it to empty
	 *
	 * @param	TargetMap		The map to clear
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Clear", CompactNodeTitle = "CLEAR", MapParam = "TargetMap" ), Category = "Utilities|Map")
	static ENGINE_API void Map_Clear(const TMap<int32, int32>& TargetMap);

	/** 
	 * Not exposed to users. Supports setting a map property on an object by name.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(BlueprintInternalUseOnly = "true", MapParam = "Value"))
	static ENGINE_API void SetMapPropertyByName(UObject* Object, FName PropertyName, const TMap<int32, int32>& Value);

	/**
	 * Given a map and an index, returns a copy of the key and value found at that index.
	 * Index may differ from the order in which they were added. This function is implemented
	 * for the purpose of iterating maps by index.
	 *
	 * @param	TargetMap	The map to get a key & value from
	 * @param	Index		The index in the map to get the key & value
	 * @param	Key			A copy of the key stored in the map at the index
	 * @param	Value		A copy of the value stored in the map at the index
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Get Key Value By Index", CompactNodeTitle = "GET KEY VALUE BY INDEX", MapParam = "TargetMap", MapKeyParam = "Key", MapValueParam = "Value", AutoCreateRefTerm = "Key, Value", BlueprintThreadSafe), Category = "Utilities|Map")
	static ENGINE_API void Map_GetKeyValueByIndex(const TMap<int32, int32>& TargetMap, int32 Index, int32& Key, int32& Value);

	/** 
	 * Determines the last valid index for a map
	 *
	 * @param	TargetMap		The map in question
	 * @return	The number of entries in the map
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Get Last Index", CompactNodeTitle = "GET LAST INDEX", MapParam = "TargetMap", BlueprintThreadSafe), Category="Utilities|Map")
	static ENGINE_API int32 Map_GetLastIndex(const TMap<int32, int32>& TargetMap);

	DECLARE_FUNCTION(execMap_Add)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since Key and Value aren't really an int, step the stack manually
		const FProperty* CurrKeyProp = MapProperty->KeyProp;
		const int32 KeyPropertySize = CurrKeyProp->GetElementSize() * CurrKeyProp->ArrayDim;
		void* KeyStorageSpace = FMemory_Alloca(KeyPropertySize);
		CurrKeyProp->InitializeValue(KeyStorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(KeyStorageSpace);
		
		const FProperty* CurrValueProp = MapProperty->ValueProp;
		const int32 ValuePropertySize = CurrValueProp->GetElementSize() * CurrValueProp->ArrayDim;
		void* ValueStorageSpace = FMemory_Alloca(ValuePropertySize);
		CurrValueProp->InitializeValue(ValueStorageSpace);
		
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(ValueStorageSpace);

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericMap_Add(MapAddr, MapProperty, KeyStorageSpace, ValueStorageSpace);
		P_NATIVE_END;
		
		CurrValueProp->DestroyValue(ValueStorageSpace);
		CurrKeyProp->DestroyValue(KeyStorageSpace);
	}

	DECLARE_FUNCTION(execMap_Remove)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since Key and Value aren't really an int, step the stack manually
		const FProperty* CurrKeyProp = MapProperty->KeyProp;
		const int32 KeyPropertySize = CurrKeyProp->GetElementSize() * CurrKeyProp->ArrayDim;
		void* KeyStorageSpace = FMemory_Alloca(KeyPropertySize);
		CurrKeyProp->InitializeValue(KeyStorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(KeyStorageSpace);
		
		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericMap_Remove(MapAddr, MapProperty, KeyStorageSpace);
		P_NATIVE_END;

		CurrKeyProp->DestroyValue(KeyStorageSpace);
	}

	DECLARE_FUNCTION(execMap_Find)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since Key and Value aren't really an int, step the stack manually
		const FProperty* CurrKeyProp = MapProperty->KeyProp;
		const int32 KeyPropertySize = CurrKeyProp->GetElementSize() * CurrKeyProp->ArrayDim;
		void* KeyStorageSpace = FMemory_Alloca(KeyPropertySize);
		CurrKeyProp->InitializeValue(KeyStorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(KeyStorageSpace);
		
		const FProperty* CurrValueProp = MapProperty->ValueProp;
		const int32 ValuePropertySize = CurrValueProp->GetElementSize() * CurrValueProp->ArrayDim;
		void* ValueStorageSpace = FMemory_Alloca(ValuePropertySize);
		CurrValueProp->InitializeValue(ValueStorageSpace);
		
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(ValueStorageSpace);
		const FFieldClass* CurrValuePropClass = CurrValueProp->GetClass();
		const FFieldClass* MostRecentPropClass = Stack.MostRecentProperty->GetClass();
		void* ItemPtr;
		// If the destination and the inner type are identical in size and their field classes derive from one another, then permit the writing out of the array element to the destination memory
		if (Stack.MostRecentPropertyAddress != NULL && (ValuePropertySize == Stack.MostRecentProperty->GetElementSize()*Stack.MostRecentProperty->ArrayDim) &&
			(MostRecentPropClass->IsChildOf(CurrValuePropClass) || CurrValuePropClass->IsChildOf(MostRecentPropClass)))
		{
			ItemPtr = Stack.MostRecentPropertyAddress;
		}
		else
		{
			ItemPtr = ValueStorageSpace;
		}
		
		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericMap_Find(MapAddr, MapProperty, KeyStorageSpace, ItemPtr);
		P_NATIVE_END;

		CurrValueProp->DestroyValue(ValueStorageSpace);
		CurrKeyProp->DestroyValue(KeyStorageSpace);
	}
	
	DECLARE_FUNCTION(execMap_Keys)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericMap_Keys(MapAddr, MapProperty, ArrayAddr, ArrayProperty);
		P_NATIVE_END

	}
	
	DECLARE_FUNCTION(execMap_Values)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericMap_Values(MapAddr, MapProperty, ArrayAddr, ArrayProperty);
		P_NATIVE_END
	}

	DECLARE_FUNCTION(execMap_Contains)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since Key and Value aren't really an int, step the stack manually
		const FProperty* CurrKeyProp = MapProperty->KeyProp;
		const int32 KeyPropertySize = CurrKeyProp->GetElementSize() * CurrKeyProp->ArrayDim;
		void* KeyStorageSpace = FMemory_Alloca(KeyPropertySize);
		CurrKeyProp->InitializeValue(KeyStorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(KeyStorageSpace);

		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericMap_Find(MapAddr, MapProperty, KeyStorageSpace, nullptr);
		P_NATIVE_END;

		CurrKeyProp->DestroyValue(KeyStorageSpace);
	}

	DECLARE_FUNCTION(execMap_Length)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		
		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericMap_Length(MapAddr, MapProperty);
		P_NATIVE_END
	}

	DECLARE_FUNCTION(execMap_IsEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(nullptr);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericMap_IsEmpty(MapAddr, MapProperty);
		P_NATIVE_END
	}

	DECLARE_FUNCTION(execMap_IsNotEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(nullptr);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericMap_IsNotEmpty(MapAddr, MapProperty);
		P_NATIVE_END
	}

	DECLARE_FUNCTION(execMap_Clear)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(NULL);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericMap_Clear(MapAddr, MapProperty);
		P_NATIVE_END
	}

	DECLARE_FUNCTION(execSetMapPropertyByName)
	{
		P_GET_OBJECT(UObject, OwnerObject);
		P_GET_PROPERTY(FNameProperty, MapPropertyName);

		Stack.StepCompiledIn<FMapProperty>(nullptr);
		void* SrcMapAddr = Stack.MostRecentPropertyAddress;

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericMap_SetMapPropertyByName(OwnerObject, MapPropertyName, SrcMapAddr);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execMap_GetKeyValueByIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(nullptr);
		void* MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_GET_PROPERTY(FIntProperty, Index);
		
		// Since Key isn't really an int, step the stack manually
		const FProperty* CurrKeyProp = MapProperty->KeyProp;
		const int32 KeyPropertySize = CurrKeyProp->GetElementSize() * CurrKeyProp->ArrayDim;
		void* KeyStorageSpace = FMemory_Alloca(KeyPropertySize);
		CurrKeyProp->InitializeValue(KeyStorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(KeyStorageSpace);
		const FFieldClass* CurrKeyPropClass = CurrKeyProp->GetClass();
		const FFieldClass* MostRecentPropClass = Stack.MostRecentProperty->GetClass();

		// If the destination and the inner type are identical in size and their field classes derive from one another, then permit the writing out of the array element to the destination memory
		void* KeyPtr = KeyStorageSpace;
		if (Stack.MostRecentPropertyAddress != nullptr && (KeyPropertySize == Stack.MostRecentProperty->GetElementSize() * Stack.MostRecentProperty->ArrayDim) &&
			(MostRecentPropClass->IsChildOf(CurrKeyPropClass) || CurrKeyPropClass->IsChildOf(MostRecentPropClass)))
		{
			KeyPtr = Stack.MostRecentPropertyAddress;
		}

		// Since the Value isn't really an int, step the stack manually
		const FProperty* CurrValueProp = MapProperty->ValueProp;
		const int32 ValuePropertySize = CurrValueProp->GetElementSize() * CurrValueProp->ArrayDim;
		void* ValueStorageSpace = FMemory_Alloca(ValuePropertySize);
		CurrValueProp->InitializeValue(ValueStorageSpace);
		
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(ValueStorageSpace);
		const FFieldClass* CurrValuePropClass = CurrValueProp->GetClass();
		MostRecentPropClass = Stack.MostRecentProperty->GetClass();

		// If the destination and the inner type are identical in size and their field classes derive from one another, then permit the writing out of the array element to the destination memory
		void* ItemPtr = ValueStorageSpace;
		if (Stack.MostRecentPropertyAddress != nullptr && (ValuePropertySize == Stack.MostRecentProperty->GetElementSize() * Stack.MostRecentProperty->ArrayDim) &&
			(MostRecentPropClass->IsChildOf(CurrValuePropClass) || CurrValuePropClass->IsChildOf(MostRecentPropClass)))
		{
			ItemPtr = Stack.MostRecentPropertyAddress;
		}
			
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericMap_GetKeyValueByIndex(MapAddr, MapProperty, Index, KeyPtr, ItemPtr);
		P_NATIVE_END;

		CurrValueProp->DestroyValue(ValueStorageSpace);
		CurrKeyProp->DestroyValue(KeyStorageSpace);
	}
	
	DECLARE_FUNCTION(execMap_GetLastIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FMapProperty>(nullptr);
		void *MapAddr = Stack.MostRecentPropertyAddress;
		FMapProperty* MapProperty = CastField<FMapProperty>(Stack.MostRecentProperty);
		if (!MapProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
	
		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericMap_GetLastIndex(MapAddr, MapProperty);
		P_NATIVE_END
	}

	static ENGINE_API void GenericMap_Add(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr, const void* ValuePtr);
	static ENGINE_API bool GenericMap_Remove(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr);
	static ENGINE_API bool GenericMap_Find(const void* TargetMap, const FMapProperty* MapProperty, const void* KeyPtr, void* ValuePtr);
	static ENGINE_API void GenericMap_Keys(const void* MapAddr, const FMapProperty* MapProperty, const void* ArrayAddr, const FArrayProperty* ArrayProperty);
	static ENGINE_API void GenericMap_Values(const void* MapAddr, const FMapProperty* MapProperty, const void* ArrayAddr, const FArrayProperty* ArrayProperty);
	static ENGINE_API int32 GenericMap_Length(const void* TargetMap, const FMapProperty* MapProperty);
	static ENGINE_API bool GenericMap_IsEmpty(const void* TargetMap, const FMapProperty* MapProperty);
	static ENGINE_API bool GenericMap_IsNotEmpty(const void* TargetMap, const FMapProperty* MapProperty);
	static ENGINE_API void GenericMap_Clear(const void* TargetMap, const FMapProperty* MapProperty);
	static ENGINE_API void GenericMap_SetMapPropertyByName(UObject* OwnerObject, FName MapPropertyName, const void* SrcMapAddr);
	static ENGINE_API void GenericMap_GetKeyValueByIndex(const void* TargetMap, const FMapProperty* MapProperty, int32 Index, void* KeyPtr, void* ValuePtr);
	static ENGINE_API int32 GenericMap_GetLastIndex(const void* TargetMap, const FMapProperty* MapProperty);

private:
	static constexpr int32 MaxSupportedMapSize = TNumericLimits<int32>::Max();
};
