// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/TextProperty.h"
#include "BlueprintPathsLibrary.generated.h"

/**
 * Function library to expose FPaths to Blueprints and Python
 *
 * Function signatures are preserved for the most part with adjustments made to some 
 * signatures to better match Blueprints / Python workflow
 */
UCLASS(meta = (ScriptName = "Paths"), MinimalAPI)
class UBlueprintPathsLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	* Should the "saved" directory structures be rooted in the user dir or relative to the "engine/game"
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool ShouldSaveToUserDir();

	/**
	* Returns the directory the application was launched from (useful for commandline utilities)
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString LaunchDir();

	/**
	* Returns the base directory of the "core" engine that can be shared across
	* several games or across games & mods. Shaders and base localization files
	* e.g. reside in the engine directory.
	*
	* @return engine directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineDir();

	/**
	* Returns the root directory for user-specific engine files. Always writable.
	*
	* @return root user directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineUserDir();

	/**
	* Returns the root directory for user-specific engine files which can be shared between versions. Always writable.
	*
	* @return root user directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineVersionAgnosticUserDir();

	/**
	* Returns the content directory of the "core" engine that can be shared across
	* several games or across games & mods.
	*
	* @return engine content directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineContentDir();

	/**
	* Returns the directory the root configuration files are located.
	*
	* @return root config directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineConfigDir();

	/**
	* Returns the intermediate directory of the engine
	*
	* @return content directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineIntermediateDir();

	/**
	* Returns the saved directory of the engine
	*
	* @return Saved directory.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineSavedDir();

	/**
	* Returns the plugins directory of the engine
	*
	* @return Plugins directory.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EnginePluginsDir();

	/**
	* Returns the base directory enterprise directory.
	*
	* @return enterprise directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EnterpriseDir();

	/**
	* Returns the enterprise plugins directory
	*
	* @return Plugins directory.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EnterprisePluginsDir();

	/**
	* Returns the enterprise FeaturePack directory
	*
	* @return FeaturePack directory.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EnterpriseFeaturePackDir();

	/**
	* Returns the root directory of the engine directory tree
	*
	* @return Root directory.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString RootDir();

	/**
	* Returns the base directory of the current project by looking at FApp::GetProjectName().
	* This is usually a subdirectory of the installation
	* root directory and can be overridden on the command line to allow self
	* contained mod support.
	*
	* @return base directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectDir();

	/**
	* Returns the root directory for user-specific game files.
	*
	* @return game user directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectUserDir();

	/**
	* Returns the content directory of the current game by looking at FApp::GetProjectName().
	*
	* @return content directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectContentDir();

	/**
	* Returns the directory the root configuration files are located.
	*
	* @return root config directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectConfigDir();

	/**
	* Returns the saved directory of the current game by looking at FApp::GetProjectName().
	*
	* @return saved directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectSavedDir();

	/**
	* Returns the intermediate directory of the current game by looking at FApp::GetProjectName().
	*
	* @return intermediate directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectIntermediateDir();

	/**
	 * Returns the Shader Working Directory
	 *
	 * @return shader working directory
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ShaderWorkingDir();

	/**
	* Returns the plugins directory of the current game by looking at FApp::GetProjectName().
	*
	* @return plugins directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectPluginsDir();

	/**
	* Returns the mods directory of the current project by looking at FApp::GetProjectName().
	*
	* @return mods directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectModsDir();

	/*
	* Returns true if a writable directory for downloaded data that persists across play sessions is available
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool HasProjectPersistentDownloadDir();

	/*
	* Returns the writable directory for downloaded data that persists across play sessions.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectPersistentDownloadDir();

	/**
	* Returns the directory the engine uses to look for the source leaf ini files. This
	* can't be an .ini variable for obvious reasons.
	*
	* @return source config directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString SourceConfigDir();

	/**
	* Returns the directory the engine saves generated config files.
	*
	* @return config directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GeneratedConfigDir();

	/**
	* Returns the directory the engine stores sandbox output
	*
	* @return sandbox directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString SandboxesDir();

	/**
	* Returns the directory the engine uses to output profiling files.
	*
	* @return log directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProfilingDir();

	/**
	* Returns the directory the engine uses to output screenshot files.
	*
	* @return screenshot directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ScreenShotDir();

	/**
	* Returns the directory the engine uses to output BugIt files.
	*
	* @return screenshot directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString BugItDir();

	/**
	* Returns the directory the engine uses to output user requested video capture files.
	*
	* @return Video capture directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString VideoCaptureDir();

	/**
	* Returns the directory the engine uses to output logs. This currently can't
	* be an .ini setting as the game starts logging before it can read from .ini
	* files.
	*
	* @return log directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ProjectLogDir();

	/** Returns the directory for automation save files */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString AutomationDir();

	/** Returns the directory for automation save files that are meant to be deleted every run */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString AutomationTransientDir();

	/** Returns the directory for automation log files */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString AutomationLogDir();

	/** Returns the directory for local files used in cloud emulation or support */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString CloudDir();

	/** Returns the directory that contains subfolders for developer-specific content */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GameDevelopersDir();

	/** Returns the directory that contains developer-specific content for the current user */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GameUserDeveloperDir();

	/** Returns the directory for temp files used for diffing */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString DiffDir();

	/**
	* Returns a list of engine-specific localization paths
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetEngineLocalizationPaths();

	/**
	* Returns a list of editor-specific localization paths
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetEditorLocalizationPaths();

	/**
	* Returns a list of property name localization paths
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetPropertyNameLocalizationPaths();

	/**
	* Returns a list of tool tip localization paths
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetToolTipLocalizationPaths();

	/**
	* Returns a list of game-specific localization paths
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetGameLocalizationPaths();

	/**
	* Returns a list of restricted/internal folder names (without any slashes) which may be tested against full paths to determine if a path is restricted or not.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const TArray<FString>& GetRestrictedFolderNames();

	/**
	* Determines if supplied path uses a restricted/internal subdirectory.	Note that slashes are normalized and character case is ignored for the comparison.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool IsRestrictedPath(const FString& InPath);

	/**
	* Returns the saved directory that is not game specific. This is usually the same as
	* EngineSavedDir().
	*
	* @return saved directory
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GameAgnosticSavedDir();

	/** Returns the directory where engine source code files are kept */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString EngineSourceDir();

	/** Returns the directory where game source code files are kept */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GameSourceDir();

	/** Returns the directory where feature packs are kept */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString FeaturePackDir();

	/**
	* Checks whether the path to the project file, if any, is set.
	*
	* @return true if the path is set, false otherwise.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool IsProjectFilePathSet();

	/**
	* Gets the path to the project file.
	*
	* @return Project file path.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetProjectFilePath();

	/**
	* Sets the path to the project file.
	*
	* @param NewGameProjectFilePath - The project file path to set.
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Paths")
	static ENGINE_API void SetProjectFilePath(const FString& NewGameProjectFilePath);

	/**
	* Gets the extension for this filename.
	*
	* @param	bIncludeDot		if true, includes the leading dot in the result
	*
	* @return	the extension of this filename, or an empty string if the filename doesn't have an extension.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetExtension(const FString& InPath, bool bIncludeDot = false);

	// Returns the filename (with extension), minus any path information.
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetCleanFilename(const FString& InPath);

	// Returns the same thing as GetCleanFilename, but without the extension
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetBaseFilename(const FString& InPath, bool bRemovePath = true);

	// Returns the path in front of the filename
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetPath(const FString& InPath);

	/** Changes the extension of the given filename (does nothing if the file has no extension) */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ChangeExtension(const FString& InPath, const FString& InNewExtension);

	/** Sets the extension of the given filename (like ChangeExtension, but also applies the extension if the file doesn't have one) */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString SetExtension(const FString& InPath, const FString& InNewExtension);

	/** Returns true if this file was found, false otherwise */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool FileExists(const FString& InPath);

	/** Returns true if this directory was found, false otherwise */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool DirectoryExists(const FString& InPath);

	/** Returns true if this path represents a root drive or volume  */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool IsDrive(const FString& InPath);

	/** Returns true if this path is relative to another path */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool IsRelative(const FString& InPath);

	/** Convert all / and \ to TEXT("/") */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void NormalizeFilename(const FString& InPath, FString& OutPath);

	/**
	* Checks if two paths are the same.
	*
	* @param PathA First path to check.
	* @param PathB Second path to check.
	*
	* @returns True if both paths are the same. False otherwise.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool IsSamePath(const FString& PathA, const FString& PathB);

	/** Normalize all / and \ to TEXT("/") and remove any trailing TEXT("/") if the character before that is not a TEXT("/") or a colon */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void NormalizeDirectoryName(const FString& InPath, FString& OutPath);

	/**
	* Takes a fully pathed string and eliminates relative pathing (eg: annihilates ".." with the adjacent directory).
	* Assumes all slashes have been converted to TEXT('/').
	* For example, takes the string:
	*	BaseDirectory/SomeDirectory/../SomeOtherDirectory/Filename.ext
	* and converts it to:
	*	BaseDirectory/SomeOtherDirectory/Filename.ext
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool CollapseRelativeDirectories(const FString& InPath, FString& OutPath);

	/**
	* Removes duplicate slashes in paths.
	* Assumes all slashes have been converted to TEXT('/').
	* For example, takes the string:
	*	BaseDirectory/SomeDirectory//SomeOtherDirectory////Filename.ext
	* and converts it to:
	*	BaseDirectory/SomeDirectory/SomeOtherDirectory/Filename.ext
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void RemoveDuplicateSlashes(const FString& InPath, FString& OutPath);

	/**
	* Make fully standard "Unreal" pathname:
	*    - Normalizes path separators [NormalizeFilename]
	*    - Removes extraneous separators  [NormalizeDirectoryName, as well removing adjacent separators]
	*    - Collapses internal ..'s
	*    - Makes relative to Engine\Binaries\<Platform> (will ALWAYS start with ..\..\..)
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void MakeStandardFilename(const FString& InPath, FString& OutPath);

	/** Takes an "Unreal" pathname and converts it to a platform filename. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void MakePlatformFilename(const FString& InPath, FString& OutPath);


	/**
	 * Assuming both paths (or filenames) are relative to the same base dir, converts InPath to be relative to InRelativeTo
	 *
	 * @param InPath Path to change to be relative to InRelativeTo
	 * @param InRelativeTo Path to use as the new relative base
	 * @param InPath New path relative to InRelativeTo
	 * @returns true if OutPath was changed to be relative
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API bool MakePathRelativeTo(const FString& InPath, const FString& InRelativeTo, FString& OutPath);

	/**
	* Converts a relative path name to a fully qualified name relative to the specified BasePath.
	* BasePath will be the process BaseDir() if not BasePath is given
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ConvertRelativePathToFull(const FString& InPath, const FString& InBasePath = TEXT(""));

	/**
	* Converts a normal path to a sandbox path (in Saved/Sandboxes).
	*
	* @param InSandboxName The name of the sandbox.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ConvertToSandboxPath(const FString& InPath, const FString& InSandboxName);

	/**
	* Converts a sandbox (in Saved/Sandboxes) path to a normal path.
	*
	* @param InSandboxName The name of the sandbox.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString ConvertFromSandboxPath(const FString& InPath, const FString& InSandboxName);

	/**
	* Creates a temporary filename with the specified prefix.
	*
	* @param Path The file pathname.
	* @param Prefix The file prefix.
	* @param Extension File extension ('.' required).
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString CreateTempFilename(const FString& Path, const FString& Prefix = TEXT(""), const FString& Extension = TEXT(".tmp"));

	/** Returns a string containing all invalid characters as dictated by the operating system */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetInvalidFileSystemChars();

	/**
	*	Returns a string that is safe to use as a filename because all items in
	*	GetInvalidFileSystemChars() are removed
	*
	*	Optionally specify the character to replace invalid characters with
	*
	* @param  InString
	* @param  InReplacementChar
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString MakeValidFileName(const FString& InString, const FString& InReplacementChar = TEXT(""));

	/**
	* Validates that the parts that make up the path contain no invalid characters as dictated by the operating system
	* Note that this is a different set of restrictions to those imposed by FPackageName
	*
	* @param InPath - path to validate
	* @param OutReason - If validation fails, this is filled with the failure reason
	* @param bDidSucceed - Whether the path could be validated
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void ValidatePath(const FString& InPath, bool& bDidSucceed, FText& OutReason);

	/**
	* Parses a fully qualified or relative filename into its components (filename, path, extension).
	*
	* @param	Path		[out] receives the value of the path portion of the input string
	* @param	Filename	[out] receives the value of the filename portion of the input string
	* @param	Extension	[out] receives the value of the extension portion of the input string
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API void Split(const FString& InPath, FString& PathPart, FString& FilenamePart, FString& ExtensionPart);

	/** Gets the relative path to get from BaseDir to RootDirectory  */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API const FString& GetRelativePathToRoot();

	/** Combine two or more Paths into one single Path */
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString Combine(const TArray<FString>& InPaths);

};
