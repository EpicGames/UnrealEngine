// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Engine/GameInstance.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintPlatformLibrary.generated.h"

class ILocalNotificationService;
enum class EDeviceScreenOrientation : uint8;

/**
 * The list of possible device/screen orientation for mobile devices
 */
UENUM(BlueprintType)
namespace EScreenOrientation
{
	enum Type : int
	{
		/** The orientation is not known */
		Unknown,

		/** The orientation is portrait with the home button at the bottom */
		Portrait,

		/** The orientation is portrait with the home button at the top */
		PortraitUpsideDown,

		/** The orientation is landscape with the home button at the right side */
		LandscapeLeft,

		/** The orientation is landscape with the home button at the left side */
		LandscapeRight,

		/** The orientation is as if place on a desk with the screen upward */
		FaceUp,

		/** The orientation is as if place on a desk with the screen downward */
		FaceDown,

		/** The orientation is portrait, oriented upright with the sensor */
		PortraitSensor,

		/** The orientation is landscape, oriented upright with the sensor */
		LandscapeSensor,

		/** The orientation is no longer locked and adjusts according to the sensor */
		FullSensor
	};
}

// application state when the game receives a notification
UENUM(BlueprintType)
namespace EApplicationState
{
	enum Type : int
	{
		/** The Application was in an unknown state when receiving the notification */
		Unknown,

		/** The Application was inactive when receiving the notification */
		Inactive,

		/** The Application was in the background when receiving the notification */
		Background,

		/** The Application was active when receiving the notification */
		Active,
	};
}

/** UObject based class for handling mobile events. Having this object as an option gives the app lifetime access to these global delegates. The component UApplicationLifecycleComponent is destroyed at level loads */
UCLASS(Blueprintable, BlueprintType, ClassGroup=Mobile, MinimalAPI)
class UPlatformGameInstance : public UGameInstance
{
	GENERATED_BODY()

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FPlatformDelegate);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlatformStartupArgumentsDelegate, const TArray<FString>&, StartupArguments);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlatformRegisteredForRemoteNotificationsDelegate, const TArray<uint8>&, inArray);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlatformRegisteredForUserNotificationsDelegate, int32, inInt);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlatformFailedToRegisterForRemoteNotificationsDelegate, FString, inString);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FPlatformReceivedRemoteNotificationDelegate, FString, inString, EApplicationState::Type, inAppState);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FPlatformReceivedLocalNotificationDelegate, FString, inString, int32, inInt, EApplicationState::Type, inAppState);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPlatformScreenOrientationChangedDelegate, EScreenOrientation::Type, inScreenOrientation);

	// This is called when the application is about to be deactivated (e.g., due to a phone call or SMS or the sleep button). 
	// The game should be paused if possible, etc... 
	UPROPERTY(BlueprintAssignable)
	FPlatformDelegate ApplicationWillDeactivateDelegate;
	
	// Called when the application has been reactivated (reverse any processing done in the Deactivate delegate)
	UPROPERTY(BlueprintAssignable)
	FPlatformDelegate ApplicationHasReactivatedDelegate;
	
	// This is called when the application is being backgrounded (e.g., due to switching  
	// to another app or closing it via the home button)  
	// The game should release shared resources, save state, etc..., since it can be  
	// terminated from the background state without any further warning.  
	UPROPERTY(BlueprintAssignable)	
	FPlatformDelegate ApplicationWillEnterBackgroundDelegate; // for instance, hitting the home button
	
	// Called when the application is returning to the foreground (reverse any processing done in the EnterBackground delegate)
	UPROPERTY(BlueprintAssignable)
	FPlatformDelegate ApplicationHasEnteredForegroundDelegate;
	
	// This *may* be called when the application is getting terminated by the OS.  
	// There is no guarantee that this will ever be called on a mobile device,  
	// save state when ApplicationWillEnterBackgroundDelegate is called instead.  
	UPROPERTY(BlueprintAssignable)
	FPlatformDelegate ApplicationWillTerminateDelegate;

	// Called when the OS is running low on resources and asks the application to free up any cached resources, drop graphics quality etc.
	UPROPERTY(BlueprintAssignable)
	FPlatformDelegate ApplicationShouldUnloadResourcesDelegate;

	// Called with arguments passed to the application on statup, perhaps meta data passed on by another application which launched this one.
	UPROPERTY(BlueprintAssignable)
	FPlatformStartupArgumentsDelegate ApplicationReceivedStartupArgumentsDelegate;

	// called when the user grants permission to register for remote notifications
    UPROPERTY(BlueprintAssignable)
	FPlatformRegisteredForRemoteNotificationsDelegate ApplicationRegisteredForRemoteNotificationsDelegate;

	// called when the user grants permission to register for notifications
    UPROPERTY(BlueprintAssignable)
	FPlatformRegisteredForUserNotificationsDelegate ApplicationRegisteredForUserNotificationsDelegate;

	// called when the application fails to register for remote notifications
    UPROPERTY(BlueprintAssignable)
	FPlatformFailedToRegisterForRemoteNotificationsDelegate ApplicationFailedToRegisterForRemoteNotificationsDelegate;

	// called when the application receives a remote notification
    UPROPERTY(BlueprintAssignable)
	FPlatformReceivedRemoteNotificationDelegate ApplicationReceivedRemoteNotificationDelegate;

	// called when the application receives a local notification
    UPROPERTY(BlueprintAssignable)
	FPlatformReceivedLocalNotificationDelegate ApplicationReceivedLocalNotificationDelegate;

	// called when the application receives a screen orientation change notification
    UPROPERTY(BlueprintAssignable)
	FPlatformScreenOrientationChangedDelegate ApplicationReceivedScreenOrientationChangedNotificationDelegate;

public:

    ENGINE_API virtual void PostInitProperties() override;
    ENGINE_API virtual void BeginDestroy() override;

private:
	/** Native handlers that get registered with the actual FCoreDelegates, and then proceed to broadcast to the delegates above */
	void ApplicationWillDeactivateDelegate_Handler() { ApplicationWillDeactivateDelegate.Broadcast(); }
	void ApplicationHasReactivatedDelegate_Handler() { ApplicationHasReactivatedDelegate.Broadcast(); }
	void ApplicationWillEnterBackgroundDelegate_Handler() 
	{ 
		SCOPED_ENTER_BACKGROUND_EVENT(STAT_UPlatformGameInstance_ApplicationWillEnterBackground);
		ApplicationWillEnterBackgroundDelegate.Broadcast();
	}
	void ApplicationHasEnteredForegroundDelegate_Handler() { ApplicationHasEnteredForegroundDelegate.Broadcast(); }
	void ApplicationWillTerminateDelegate_Handler() { ApplicationWillTerminateDelegate.Broadcast(); }
	void ApplicationShouldUnloadResourcesDelegate_Handler() { ApplicationShouldUnloadResourcesDelegate.Broadcast(); }
	void ApplicationReceivedStartupArgumentsDelegate_Handler(const TArray<FString>& StartupArguments) { ApplicationReceivedStartupArgumentsDelegate.Broadcast(StartupArguments); }
    void ApplicationRegisteredForRemoteNotificationsDelegate_Handler(TArray<uint8> inArray) { ApplicationRegisteredForRemoteNotificationsDelegate.Broadcast(inArray); }
    void ApplicationRegisteredForUserNotificationsDelegate_Handler(int32 inInt) { ApplicationRegisteredForUserNotificationsDelegate.Broadcast(inInt); }
    void ApplicationFailedToRegisterForRemoteNotificationsDelegate_Handler(FString inFString) { ApplicationFailedToRegisterForRemoteNotificationsDelegate.Broadcast(inFString); }
	void ApplicationReceivedRemoteNotificationDelegate_Handler(FString inFString, int32 inAppState);
	void ApplicationReceivedLocalNotificationDelegate_Handler(FString inFString, int32 inInt, int32 inAppState);
    void ApplicationReceivedScreenOrientationChangedNotificationDelegate_Handler(int32 inScreenOrientation);

};

UCLASS(meta=(ScriptName="PlatformLibrary"), MinimalAPI)
class UBlueprintPlatformLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
	static ENGINE_API ILocalNotificationService*	platformService;
	
public:
        
	/** Clear all pending local notifications. Typically this will be done before scheduling new notifications when going into the background */
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API void ClearAllLocalNotifications();

	/** Schedule a local notification at a specific time, inLocalTime specifies the current local time or if UTC time should be used 
	 * @param FireDateTime The time at which to fire the local notification
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param Title The title of the notification
	 * @param Body The more detailed description of the notification
	 * @param Action The text to be displayed on the slider controller
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	*/
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API int32 ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent);

	/** Schedule a local notification to fire inSecondsFromNow from now 
	 * @param inSecondsFromNow The seconds until the notification should fire
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param Title The title of the notification
	 * @param Body The more detailed description of the notification
	 * @param Action The text to be displayed on the slider controller
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	*/
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API int32 ScheduleLocalNotificationFromNow(int32 inSecondsFromNow, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent);

	/** Schedule a local notification badge at a specific time, inLocalTime specifies the current local time or if UTC time should be used
	 * @param FireDateTime The time at which to fire the local notification
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	*/
	UFUNCTION(BlueprintCallable, Category = "Platform|LocalNotification")
	static ENGINE_API int32 ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool LocalTime, const FString& ActivationEvent);
	
	/** Schedule a local notification badge to fire inSecondsFromNow from now
	 * @param inSecondsFromNow The seconds until the notification should fire
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	*/
	UFUNCTION(BlueprintCallable, Category = "Platform|LocalNotification")
	static ENGINE_API void ScheduleLocalNotificationBadgeFromNow(int32 inSecondsFromNow, const FString& ActivationEvent);

	/** Cancel a local notification given the ActivationEvent
	 * @param ActivationEvent The string passed into the Schedule call for the notification to be cancelled
	*/
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API void CancelLocalNotification(const FString& ActivationEvent);

	/** Cancel a local notification given the ActivationEvent
	 * @param NotificationId The Id returned from one of the ScheduleLocalNotification* functions
	 */
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API void CancelLocalNotificationById(int32 NotificationId);

	/** Get the local notification that was used to launch the app
	 * @param NotificationLaunchedApp Return true if a notification was used to launch the app
	 * @param ActivationEvent Returns the name of the ActivationEvent if a notification was used to launch the app
	 * @param FireDate Returns the time the notification was activated
	*/
	UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
	static ENGINE_API void GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate);

	/**
	 * Returns the current orientation of the device: will be either Portrait, LandscapeLeft, PortraitUpsideDown or LandscapeRight.
	 *
	 * @return An EDeviceScreenOrientation value.
	 */
	UFUNCTION(BlueprintPure, Category="Platform|LocalNotification")
	static ENGINE_API EScreenOrientation::Type GetDeviceOrientation();

	/**
	 * Returns the allowed orientation of the device. This is NOT the same as GetDeviceOrientation, which only returns Portrait, LandscapeLeft,
	 * PortraitUpsideDown or LandscapeRight. The allowed orientation limits what orientation your device can have. So if you set the allowed orientation
	 * to LandscapeLeft, GetDeviceOrientation will only ever return LandscapeLeft. But if you set the allowed orientation to LandscapeSensor, you are actually
	 * restricting the allowed orientations to LandscapeLeft OR LandscapeRight (depending on the sensor), so GetDeviceOrientation might return LandscapeLeft OR LandscapeRight.
	 *
	 * @return An EDeviceScreenOrientation value.
	 */
	UFUNCTION(BlueprintPure, Category = "Platform|LocalNotification")
	static ENGINE_API EScreenOrientation::Type GetAllowedDeviceOrientation();

	/**
	 * Set the allowed orientation of the device.
	 */
	UFUNCTION(BlueprintCallable, Category = "Platform|LocalNotification")
	static ENGINE_API void SetAllowedDeviceOrientation(EScreenOrientation::Type NewAllowedDeviceOrientation);

private:

	static EScreenOrientation::Type ConvertToScreenOrientation(EDeviceScreenOrientation DeviceScreenOrientation);

	static EDeviceScreenOrientation ConvertToDeviceScreenOrientation(EScreenOrientation::Type ScreenOrientation);
};
