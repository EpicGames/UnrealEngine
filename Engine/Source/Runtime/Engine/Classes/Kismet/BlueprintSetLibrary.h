// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintSetLibrary.generated.h"

UCLASS(meta=(BlueprintThreadSafe), MinimalAPI)
class UBlueprintSetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Adds item to set
	 *
	 * @param	TargetSet		The set to add item to
	 * @param	NewItem			The item to add to the set
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Add", CompactNodeTitle = "ADD", SetParam = "TargetSet|NewItem", AutoCreateRefTerm = "NewItem"), Category="Utilities|Set")
	static ENGINE_API void Set_Add(const TSet<int32>& TargetSet, const int32& NewItem);
	
	/**
	 * Adds all elements from an Array to a Set
	 *
	 * @param	TargetSet		The set to search for the item
	 * @param	NewItems		The items to add to the set
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Add Items", CompactNodeTitle = "ADD ITEMS", SetParam = "TargetSet|NewItems", AutoCreateRefTerm = "NewItems"), Category="Utilities|Set")
	static ENGINE_API void Set_AddItems(const TSet<int32>& TargetSet, const TArray<int32>& NewItems);

	/**
	 * Remove item from set. Output value indicates if something was actually removed. False
	 * indicates no equivalent item was found.
	 *
	 * @param	TargetSet		The set to remove from
	 * @param	Item			The item to remove from the set
	 * @return	True if an item was removed (False indicates no equivalent item was present)
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Remove", CompactNodeTitle = "REMOVE", SetParam = "TargetSet|Item", AutoCreateRefTerm = "Item"), Category="Utilities|Set")
	static ENGINE_API bool Set_Remove(const TSet<int32>& TargetSet, const int32& Item);
	
	/**
	 * Check if the set is empty
	 *
	 * @param	TargetSet		The set to check
	 * @return	A boolean indicating if the array is empty
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Empty", CompactNodeTitle = "IS EMPTY", SetParam = "TargetSet"), Category = "Utilities|Set")
	static ENGINE_API bool Set_IsEmpty(const TSet<int32>& TargetSet);

	/**
	 * Check if the set has any elements
	 *
	 * @param	TargetSet		The set to check
	 * @return	A boolean indicating if the array has any elements
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Not Empty", CompactNodeTitle = "IS NOT EMPTY", SetParam = "TargetSet"), Category = "Utilities|Set")
	static ENGINE_API bool Set_IsNotEmpty(const TSet<int32>& TargetSet);

	/**
	 * Removes all elements in an Array from a set.
	 *
	 * @param	TargetSet		The set to remove from
	 * @param	Items			The items to remove from the set
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Remove Items", CompactNodeTitle = "REMOVE ITEMS", SetParam = "TargetSet|Items", AutoCreateRefTerm = "Items"), Category="Utilities|Set")
	static ENGINE_API void Set_RemoveItems(const TSet<int32>& TargetSet, const  TArray<int32>& Items);

	/**
	 * Outputs an Array containing copies of the entries of a Set.
	 *
	 * @param		A		Set
	 * @param		Result	Array
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (DisplayName = "To Array", CompactNodeTitle = "TO ARRAY", SetParam = "A|Result"), Category = "Utilities|Set")
	static ENGINE_API void Set_ToArray(const TSet<int32>& A, TArray<int32>& Result);

	/**
	 * Clear a set, removes all content.
	 *
	 * @param	TargetSet		The set to clear
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Clear", CompactNodeTitle = "CLEAR", Keywords = "empty", SetParam = "TargetSet"), Category="Utilities|Set")
	static ENGINE_API void Set_Clear(const TSet<int32>& TargetSet);

	/**
	 * Get the number of items in a set.
	 *
	 * @param	TargetSet		The set to get the length of
	 * @return	The length of the set
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Length", CompactNodeTitle = "LENGTH", SetParam = "TargetSet", Keywords = "num size count", BlueprintThreadSafe), Category="Utilities|Set")
	static ENGINE_API int32 Set_Length(const TSet<int32>& TargetSet);

	/**
	 * Returns true if the set contains the given item.
	 *
	 * @param	TargetSet		The set to search for the item
	 * @param	ItemToFind		The item to look for
	 * @return	True if the item was found within the set
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Contains Item", CompactNodeTitle = "CONTAINS", SetParam = "TargetSet|ItemToFind", AutoCreateRefTerm = "ItemToFind", BlueprintThreadSafe), Category="Utilities|Set")
	static ENGINE_API bool Set_Contains(const TSet<int32>& TargetSet, const int32& ItemToFind);

	/**
	 * Assigns Result to the intersection of Set A and Set B. That is, Result will contain
	 * all elements that are in both Set A and Set B. To intersect with the empty set use
	 * Clear.
	 *
	 * @param		A		One set to intersect
	 * @param		B		Another set to intersect
	 * @param		Result	Set to store results in
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Intersection", CompactNodeTitle = "INTERSECTION", SetParam = "A|B|Result"), Category="Utilities|Set")
	static ENGINE_API void Set_Intersection(const TSet<int32>& A, const TSet<int32>& B, TSet<int32>& Result );

	/**
	 * Assigns Result to the union of two sets, A and B. That is, Result will contain
	 * all elements that are in Set A and in addition all elements in Set B. Note that 
	 * a Set is a collection of unique elements, so duplicates will be eliminated.
	 *
	 * @param		A		One set to union
	 * @param		B		Another set to union
	 * @param		Result	Set to store results in
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (DisplayName = "Union", CompactNodeTitle = "UNION", SetParam = "A|B|Result"), Category = "Utilities|Set")
	static ENGINE_API void Set_Union(const TSet<int32>& A, const TSet<int32>& B, TSet<int32>& Result );

	/**
	 * Assigns Result to the relative difference of two sets, A and B. That is, Result will 
	 * contain  all elements that are in Set A but are not found in Set B. Note that the 
	 * difference between two sets  is not commutative. The Set whose elements you wish to 
	 * preserve should be the first (top) parameter. Also called the relative complement.
	 *
	 * @param		A		Starting set
	 * @param		B		Set of elements to remove from set A
	 * @param		Result	Set containing all elements in A that are not found in B
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (DisplayName = "Difference", CompactNodeTitle = "DIFFERENCE", SetParam = "A|B|Result"), Category = "Utilities|Set")
	static ENGINE_API void Set_Difference(const TSet<int32>& A, const TSet<int32>& B, TSet<int32>& Result );

	/** 
	 * Not exposed to users. Supports setting a set property on an object by name.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(BlueprintInternalUseOnly = "true", SetParam = "Value"))
	static ENGINE_API void SetSetPropertyByName(UObject* Object, FName PropertyName, const TSet<int32>& Value);

	/**
	 * Given a set and an index, returns a copy of the item found at that index.
	 * Index may differ from the order in which they were added. This function is implemented
	 * for the purpose of iterating sets by index.
	 *
	 * @param	TargetSet		The set to get an item from
	 * @param	Index			The index in the set to get an item from
	 * @return	A copy of the item stored at the index
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Get Item By Index", CompactNodeTitle = "GET ITEM BY INDEX", SetParam = "TargetSet|Item", AutoCreateRefTerm = "Item", BlueprintThreadSafe), Category = "Utilities|Set")
	static void Set_GetItemByIndex(const TSet<int32>& TargetSet, int32 Index, int32& Item);

	/** 
	 * Retrieves the last valid index for a set
	 *
	 * @param	TargetSet		The set in question
	 * @return	The number of entries in the set
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Last Index", CompactNodeTitle = "GET LAST INDEX", SetParam = "TargetSet", BlueprintThreadSafe), Category="Utilities|Set")
	static int32 Set_GetLastIndex(const TSet<int32>& TargetSet);

	DECLARE_FUNCTION(execSet_Add)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since ItemPtr isn't really an int, step the stack manually
		const FProperty* ElementProp = SetProperty->ElementProp;
		const int32 PropertySize = ElementProp->GetElementSize() * ElementProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		ElementProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemPtr = StorageSpace;

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_Add(SetAddr, SetProperty, ItemPtr);
		P_NATIVE_END;

		ElementProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execSet_AddItems)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* TargetArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!TargetArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_AddItems(SetAddr, SetProperty, TargetArrayAddr, TargetArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Remove)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since ItemPtr isn't really an int, step the stack manually
		const FProperty* ElementProp = SetProperty->ElementProp;
		const int32 PropertySize = ElementProp->GetElementSize() * ElementProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		ElementProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = NULL;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemPtr = StorageSpace;

		P_FINISH;

		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericSet_Remove(SetAddr, SetProperty, ItemPtr);
		P_NATIVE_END;

		ElementProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execSet_RemoveItems)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* TargetArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!TargetArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_RemoveItems(SetAddr, SetProperty, TargetArrayAddr, TargetArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_ToArray)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* TargetArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!TargetArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_ToArray(SetAddr, SetProperty, TargetArrayAddr, TargetArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Clear)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		// Perform the search
		P_NATIVE_BEGIN;
		GenericSet_Clear(SetAddr, SetProperty);
		P_NATIVE_END;

	}

	DECLARE_FUNCTION(execSet_Length)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		// Perform the search
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericSet_Length(SetAddr, SetProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Contains)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			// @todo(dano): rename to 'container context failed'
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since ItemToFind isn't really an int, step the stack manually
		const FProperty* ElementProp = SetProperty->ElementProp;
		const int32 PropertySize = ElementProp->GetElementSize() * ElementProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		ElementProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemToFindPtr = StorageSpace;

		P_FINISH;

		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericSet_Contains(SetAddr, SetProperty, ItemToFindPtr);
		P_NATIVE_END;

		ElementProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execSet_IsEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(nullptr);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericSet_IsEmpty(SetAddr, SetProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_IsNotEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(nullptr);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericSet_IsNotEmpty(SetAddr, SetProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Intersection)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrA = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyA = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyA)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrB = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyB = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyB)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrResult = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyResult = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyResult)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_Intersect(SetAddrA, SetPropertyA, SetAddrB, SetPropertyB, SetAddrResult, SetPropertyResult);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Union)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrA = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyA = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyA)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrB = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyB = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyB)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrResult = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyResult = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyResult)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_Union(SetAddrA, SetPropertyA, SetAddrB, SetPropertyB, SetAddrResult, SetPropertyResult);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_Difference)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrA = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyA = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyA)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrB = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyB = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyB)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(NULL);
		void* SetAddrResult = Stack.MostRecentPropertyAddress;
		FSetProperty* SetPropertyResult = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetPropertyResult)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_Difference(SetAddrA, SetPropertyA, SetAddrB, SetPropertyB, SetAddrResult, SetPropertyResult);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSetSetPropertyByName)
	{
		P_GET_OBJECT(UObject, OwnerObject);
		P_GET_PROPERTY(FNameProperty, SetPropertyName);

		Stack.StepCompiledIn<FSetProperty>(nullptr);
		void* SrcSetAddr = Stack.MostRecentPropertyAddress;

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericSet_SetSetPropertyByName(OwnerObject, SetPropertyName, SrcSetAddr);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execSet_GetItemByIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(nullptr);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_GET_PROPERTY(FIntProperty, Index);

		// Since Item isn't really an int, step the stack manually
		const FProperty* ElementProp = SetProperty->ElementProp;
		const int32 PropertySize = ElementProp->GetElementSize() * ElementProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		ElementProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		const FFieldClass* ElementPropClass = ElementProp->GetClass();
		const FFieldClass* MostRecentPropClass = Stack.MostRecentProperty->GetClass();
		void* ItemPtr;

		// If the destination and the inner type are identical in size and their field classes derive from one another, then permit the writing out of the array element to the destination memory
		if (Stack.MostRecentPropertyAddress != nullptr && (PropertySize == Stack.MostRecentProperty->GetElementSize() * Stack.MostRecentProperty->ArrayDim) &&
			(MostRecentPropClass->IsChildOf(ElementPropClass) || ElementPropClass->IsChildOf(MostRecentPropClass)))
		{
			ItemPtr = Stack.MostRecentPropertyAddress;
		}
		else
		{
			ItemPtr = StorageSpace;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericSet_GetItemByIndex(SetAddr, SetProperty, Index, ItemPtr);
		P_NATIVE_END;
		ElementProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execSet_GetLastIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FSetProperty>(nullptr);
		void* SetAddr = Stack.MostRecentPropertyAddress;
		FSetProperty* SetProperty = CastField<FSetProperty>(Stack.MostRecentProperty);
		if (!SetProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
	
		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericSet_GetLastIndex(SetAddr, SetProperty);
		P_NATIVE_END
	}

	static ENGINE_API void GenericSet_Add(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemPtr);
	static ENGINE_API void GenericSet_AddItems(const void* TargetSet, const FSetProperty* SetProperty, const void* TargetArray, const FArrayProperty* ArrayProperty);
	static ENGINE_API bool GenericSet_Remove(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemPtr);
	static ENGINE_API void GenericSet_RemoveItems(const void* TargetSet, const FSetProperty* SetProperty, const void* TargetArray, const FArrayProperty* ArrayProperty);
	static ENGINE_API void GenericSet_ToArray(const void* TargetSet, const FSetProperty* SetProperty, void* TargetArray, const FArrayProperty* ArrayProperty);
	static ENGINE_API void GenericSet_Clear(const void* TargetSet, const FSetProperty* SetProperty);
	static ENGINE_API int32 GenericSet_Length(const void* TargetSet, const FSetProperty* SetProperty);
	static ENGINE_API bool GenericSet_Contains(const void* TargetSet, const FSetProperty* SetProperty, const void* ItemToFind);
	static ENGINE_API bool GenericSet_IsEmpty(const void* TargetSet, const FSetProperty* SetProperty);
	static ENGINE_API bool GenericSet_IsNotEmpty(const void* TargetSet, const FSetProperty* SetProperty);
	static ENGINE_API void GenericSet_Intersect(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult);
	static ENGINE_API void GenericSet_Union(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult);
	static ENGINE_API void GenericSet_Difference(const void* SetA, const FSetProperty* SetPropertyA, const void* SetB, const FSetProperty* SetPropertyB, const void* SetResult, const FSetProperty* SetPropertyResult);
	static ENGINE_API void GenericSet_SetSetPropertyByName(UObject* OwnerObject, FName SetPropertyName, const void* SrcSetAddr);
	static ENGINE_API void GenericSet_GetItemByIndex(void* TargetSet, const FSetProperty* SetProperty, int32 Index, void* Item);
	static ENGINE_API int32 GenericSet_GetLastIndex(const void* TargetSet, const FSetProperty* SetProperty);

private:
	static constexpr int32 MaxSupportedSetSize = TNumericLimits<int32>::Max();
};
