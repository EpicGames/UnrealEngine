// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintFunctionLibrary.h"

#include "BlueprintSpringMathLibrary.generated.h"

/** A simple struct that stores velocities for an FTransform. */
USTRUCT(BlueprintType)
struct FSpringTransformVelocity
{
	GENERATED_BODY()

	// Change in Location over time in cm/s
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Transform", meta= (ForceUnits = "cm/s"))
	FVector LinearVelocity = FVector::ZeroVector;

	// Change in rotation over time in deg/s
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Transform", meta = (ForceUnits = "deg/s"))
	FVector AngularVelocity = FVector::ZeroVector;

	// Change in scale over time
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Transform")
	FVector ScalarVelocity = FVector::ZeroVector;
};

UCLASS(MinimalAPI, meta=(BlueprintThreadSafe))
class UBlueprintSpringMathLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Simple Spring //

	/** Interpolates the value InOutX towards TargetX with the motion of a critically damped spring. The velocity of X is stored in InOutV.
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm="TargetX"))
	static ENGINE_API void CriticalSpringDampFloat(UPARAM(ref) float& InOutX, UPARAM(ref) float& InOutV, const float& TargetX, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the angle InOutAngle towards TargetAngle with the motion of a critically damped spring. The velocity of InOutAngle is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutAngle The value to be damped in degrees
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param TargetAngle The goal to damp towards in degrees
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutAngle to TargetAngle
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetAngle"))
	static ENGINE_API void CriticalSpringDampAngle(UPARAM(ref) float& InOutAngle, UPARAM(ref) float& InOutAngularVelocity, const float& TargetAngle, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutX towards TargetX with the motion of a critically damped spring. The velocity of X is stored in InOutV.
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetX"))
	static ENGINE_API void CriticalSpringDampVector(UPARAM(ref) FVector& InOutX, UPARAM(ref) FVector& InOutV, const FVector& TargetX, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutX towards TargetX with the motion of a critically damped spring. The velocity of X is stored in InOutV.
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetX"))
	static ENGINE_API void CriticalSpringDampVector2D(UPARAM(ref) FVector2D& InOutX, UPARAM(ref) FVector2D& InOutV, const FVector2D& TargetX, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutRotation towards TargetRotation with the motion of a critically damped spring. The velocity of InOutRotation is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param TargetRotation The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetRotation"))
	static ENGINE_API void CriticalSpringDampQuat(UPARAM(ref) FQuat& InOutRotation, UPARAM(ref) FVector& InOutAngularVelocity, const FQuat& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutRotation towards TargetRotation with the motion of a critically damped spring. The velocity of InOutRotation is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param TargetRotation The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetRotation"))
	static ENGINE_API void CriticalSpringDampRotator(UPARAM(ref) FRotator& InOutRotation, UPARAM(ref) FVector& InOutAngularVelocity, const FRotator& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutScale towards TargetScale with the motion of a critically damped spring.
	 *
	 * @param InOutScale The value to be damped
	 * @param InOutScalarVelocity The speed of the value to be damped
	 * @param TargetScale The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutScale to TargetScale
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetScale"))
	static ENGINE_API void CriticalSpringDampScale(UPARAM(ref) FVector& InOutScale, UPARAM(ref) FVector& InOutScalarVelocity, const FVector& TargetScale, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutTransform towards TargetTransform with the motion of a critically damped spring.
	 *
	 * @param InOutTransform The value to be damped
	 * @param InOutVelocity The speed of the value to be damped
	 * @param TargetTransform The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutTransform to TargetTransform
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetTransform"))
	static ENGINE_API void CriticalSpringDampTransform(UPARAM(ref) FTransform& InOutTransform, UPARAM(ref) FSpringTransformVelocity& InOutVelocity, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime = 0.2f);


	// Double Spring //

	/** Interpolates the value InOutX towards TargetX using a critically damped spring via an intermediate state - producing a more S-shaped curve. The velocity of X is stored in InOutV.
	 *
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param InOutXi The intermediate state of the double spring
	 * @param InOutVi The intermediate velocity of the double spring
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetX"))
	static ENGINE_API void CriticalDoubleSpringDampFloat(UPARAM(ref) float& InOutX, UPARAM(ref) float& InOutV, UPARAM(ref) float& InOutXi, UPARAM(ref) float& InOutVi, const float& TargetX, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the angle InOutAngle towards TargetAngle using a critically damped spring via an intermediate state - producing a more S-shaped curve. The velocity of InOutAngle is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutAngle The value to be damped in degrees
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param InOutIntermediateAngle The intermediate state of the double spring
	 * @param InOutIntermediateAngularVelocity The intermediate velocity of the double spring
	 * @param TargetAngle The goal to damp towards in degrees
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutAngle to TargetAngle
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetAngle"))
	static ENGINE_API void CriticalDoubleSpringDampAngle(UPARAM(ref) float& InOutAngle,UPARAM(ref) float& InOutAngularVelocity, UPARAM(ref) float& InOutIntermediateAngle, UPARAM(ref) float& InOutIntermediateAngularVelocity, const float& TargetAngle, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutX towards TargetX using a critically damped spring via an intermediate state - producing a more S-shaped curve. The velocity of X is stored in InOutV.
	 *
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param InOutXi The intermediate state of the double spring
	 * @param InOutVi The intermediate velocity of the double spring
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetAngle"))
	static ENGINE_API void CriticalDoubleSpringDampVector(UPARAM(ref) FVector& InOutX, UPARAM(ref) FVector& InOutV, UPARAM(ref) FVector& InOutXi, UPARAM(ref) FVector& InOutVi, const FVector& TargetX, float DeltaTime, float SmoothingTime = 0.2f);
	
	/** Interpolates the value InOutX towards TargetX using a critically damped spring via an intermediate state - producing a more S-shaped curve. The velocity of X is stored in InOutV.
	 *
	 * @param InOutX The value to be damped
	 * @param InOutV The speed of the value to be damped
	 * @param InOutXi The intermediate state of the double spring
	 * @param InOutVi The intermediate velocity of the double spring
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetX"))
	static ENGINE_API void CriticalDoubleSpringDampVector2D(UPARAM(ref) FVector2D& InOutX, UPARAM(ref) FVector2D& InOutV, UPARAM(ref) FVector2D& InOutXi, UPARAM(ref) FVector2D& InOutVi, const FVector2D& TargetX, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutRotation towards TargetRotation using a critically damped spring via an intermediate state - producing a more S-shaped curve. The velocity of InOutRotation is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param InOutIntermediateRotation The intermediate state of the double spring
	 * @param InOutIntermediateAngularVelocity The intermediate velocity of the double spring
	 * @param TargetRotation The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetRotation"))
	static ENGINE_API void CriticalDoubleSpringDampQuat(UPARAM(ref) FQuat& InOutRotation, UPARAM(ref) FVector& InOutAngularVelocity, UPARAM(ref) FQuat& InOutIntermediateRotation, UPARAM(ref) FVector& InOutIntermediateAngularVelocity, const FQuat& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutRotation towards TargetRotation using a critically damped spring via an intermediate value - producing a more S-shaped curve. The velocity of InOutRotation is stored in InOutAngularVelocity in deg/s.
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocity The speed of the value to be damped in deg/s
	 * @param InOutIntermediateRotation The intermediate state of the double spring
	 * @param InOutIntermediateAngularVelocity The intermediate velocity of the double spring
	 * @param TargetRotation The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetRotation"))
	static ENGINE_API void CriticalDoubleSpringDampRotator(UPARAM(ref) FRotator& InOutRotation, UPARAM(ref) FVector& InOutAngularVelocity, UPARAM(ref) FRotator& InOutIntermediateRotation, UPARAM(ref) FVector& InOutIntermediateAngularVelocity, const FRotator& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutScale towards TargetScale using a critically damped spring via an intermediate state - producing a more S-shaped curve.
	 *
	 * @param InOutScale The value to be damped
	 * @param InOutScalarVelocity The speed of the value to be damped
	 * @param InOutIntermediateScale The intermediate state of the double spring
	 * @param InOutIntermediateScalarVelocity The intermediate velocity of the double spring
	 * @param TargetScale The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutScale to TargetScale
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetScale"))
	static ENGINE_API void CriticalDoubleSpringDampScale(UPARAM(ref) FVector& InOutScale, UPARAM(ref) FVector& InOutScalarVelocity, UPARAM(ref) FVector& InOutIntermediateScale, UPARAM(ref) FVector& InOutIntermediateScalarVelocity, const FVector& TargetScale, float DeltaTime, float SmoothingTime = 0.2f);

	/** Interpolates the value InOutTransform towards TargetTransform using a critically damped spring via an intermediate state - producing a more S-shaped curve.
	 *
	 * @param InOutTransform The value to be damped
	 * @param InOutVelocity The speed of the value to be damped
	 * @param InOutIntermediateTransform The intermediate state of the double spring
	 * @param InOutIntermediateVelocity The intermediate velocity of the double spring
	 * @param TargetTransform The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutTransform to TargetTransform
	 * @param DeltaTime Timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring", AutoCreateRefTerm = "TargetTransform"))
	static ENGINE_API void CriticalDoubleSpringDampTransform(UPARAM(ref) FTransform& InOutTransform, UPARAM(ref) FSpringTransformVelocity& InOutVelocity, UPARAM(ref) FTransform& InOutIntermediateTransform, UPARAM(ref) FSpringTransformVelocity& InOutIntermediateTransformVelocity, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime = 0.2f);


	// Velocity Spring //

	/** A velocity spring will damp towards a target that follows a fixed linear target velocity, allowing control of the interpolation speed
	 * while still giving a smoothed behavior. A SmoothingTime of 0 will give a linear interpolation between X and TargetX
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutVi The intermediate target velocity of the value to be damped
	 * @param TargetX The target value of X to damp towards
	 * @param MaxSpeed The desired speed to achieve while damping towards X
	 * @param SmoothingTime The smoothing time to use while damping towards X. Higher values will give more smoothed behaviour. A value of 0 will give a linear interpolation of X to Target
	 * @param DeltaTime The timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring"))
	static ENGINE_API void VelocitySpringDampFloat(UPARAM(ref) float& InOutX, UPARAM(ref) float& InOutV, UPARAM(ref) float& InOutVi, float TargetX, float MaxSpeed, float DeltaTime, float SmoothingTime = 0.2f);

	/** A velocity spring will damp towards a target that follows a fixed linear target velocity, allowing control of the interpolation speed
	 * while still giving a smoothed behavior. A SmoothingTime of 0 will give a linear interpolation between X and TargetX
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutVi The intermediate target velocity of the value to be damped
	 * @param TargetX The target value of X to damp towards
	 * @param MaxSpeed The desired speed to achieve while damping towards X
	 * @param SmoothingTime The smoothing time to use while damping towards X. Higher values will give more smoothed behaviour. A value of 0 will give a linear interpolation of X to Target
	 * @param DeltaTime The timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring"))
	static ENGINE_API void VelocitySpringDampVector(UPARAM(ref) FVector& InOutX, UPARAM(ref) FVector& InOutV, UPARAM(ref) FVector& InOutVi, const FVector& TargetX, float MaxSpeed, float DeltaTime,  float SmoothingTime = 0.2f);

	/** A velocity spring will damp towards a target that follows a fixed linear target velocity, allowing control of the interpolation speed
	 * while still giving a smoothed behavior. A SmoothingTime of 0 will give a linear interpolation between X and TargetX
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutVi The intermediate target velocity of the value to be damped
	 * @param TargetX The target value of X to damp towards
	 * @param MaxSpeed The desired speed to achieve while damping towards X
	 * @param SmoothingTime The smoothing time to use while damping towards X. Higher values will give more smoothed behaviour. A value of 0 will give a linear interpolation of X to Target
	 * @param DeltaTime The timestep in seconds
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring"))
	static ENGINE_API void VelocitySpringDampVector2D(UPARAM(ref) FVector2D& InOutX, UPARAM(ref) FVector2D& InOutV, UPARAM(ref) FVector2D& InOutVi, const FVector2D& TargetX, float MaxSpeed, float DeltaTime,  float SmoothingTime = 0.2f);


	// Damper //

	/**
	 * Smooths a value using exponential damping towards a target.
	 * 
	 * 
	 * @param  Value The value to be smoothed
	 * @param  Target The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring"))
	static ENGINE_API float DampFloat(float Value, float Target, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths an angle in degrees using exponential damping towards a target.
	 *
	 *
	 * @param  Angle The angle to be smoothed in degrees
	 * @param  TargetAngle The target to smooth towards in degrees
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float DampAngle(float Angle, float TargetAngle, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a value using exponential damping towards a target.
	 * 
	 * 
	 * @param  Value The value to be smoothed
	 * @param  Target The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring", AutoCreateRefTerm = "Value, Target"))
	static ENGINE_API FVector DampVector(const FVector& Value, const FVector& Target, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a value using exponential damping towards a target.
	 * 
	 * 
	 * @param  Value The value to be smoothed
	 * @param  Target The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring", AutoCreateRefTerm = "Value, Target"))
	static ENGINE_API FVector2D DampVector2D(const FVector2D& Value, const FVector2D& Target, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a rotation using exponential damping towards a target.
	 * 
	 * 
	 * @param  Rotation The rotation to be smoothed
	 * @param  TargetRotation The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring", AutoCreateRefTerm = "Rotation, TargetRotation"))
	static ENGINE_API FQuat DampQuat(const FQuat& Rotation, const FQuat& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a value using exponential damping towards a target.
	 * 
	 * 
	 * @param  Rotation The rotation to be smoothed
	 * @param  TargetRotation The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring", AutoCreateRefTerm = "Rotation, TargetRotation"))
	static ENGINE_API FRotator DampRotator(const FRotator& Rotation, const FRotator& TargetRotation, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a scale using exponential damping towards a target.
	 *
	 *
	 * @param  Scale The scale to be smoothed
	 * @param  TargetScale The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", AutoCreateRefTerm = "Scale, TargetScale"))
	static ENGINE_API FVector DampScale(const FVector& Scale, const FVector& TargetScale, float DeltaTime, float SmoothingTime = 0.2f);

	/**
	 * Smooths a transform using exponential damping towards a target.
	 *
	 *
	 * @param  Transform The transform to be smoothed
	 * @param  TargetTransform The target to smooth towards
	 * @param  DeltaTime Time interval
	 * @param  SmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", AutoCreateRefTerm = "Transform, TargetTransform"))
	static ENGINE_API FTransform DampTransform(const FTransform& Transform, const FTransform& TargetTransform, float DeltaTime, float SmoothingTime = 0.2f);


	// Character //

	/** Update the position of a character given a target velocity using a simple damped spring
	 * 
	 * @param InOutPosition The position of the character
	 * @param InOutVelocity The velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutAcceleration The acceleration of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param TargetVelocity The target velocity of the character.
	 * @param DeltaTime The delta time to tick the character
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetVelocity"))
	static ENGINE_API void SpringCharacterUpdate(UPARAM(ref) FVector& InOutPosition, UPARAM(ref) FVector& InOutVelocity, UPARAM(ref) FVector& InOutAcceleration, const FVector& TargetVelocity, float DeltaTime, float SmoothingTime = 0.2f);

	/** Estimates an approximate stopping distance from a smoothing time for a character using a simple damped spring
	 *
	 * @param InitialVelocity The velocity of the character at the start of the stop
	 * @param InitialAcceleration The acceleration of the character at the start of the stop
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @return An estimation of the total distance traveled during the stop
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Stopping Distance"))
	static ENGINE_API float SpringCharacterStoppingDistance(const float InitialVelocity, const float InitialAcceleration, const float SmoothingTime = 0.2f);

	/** Estimates an approximate stopping time (how long until the velocity goes below the VelocityThreshold) from a smoothing time for a character using a simple damped spring. Assumes the initial acceleration is zero.
	 *
	 * @param InitialVelocity The velocity of the character at the start of the stop
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param VelocityThreshold The velocity threshold at which the character is considered to be stopped in cm/s
	 * @return An estimation of the time taken to come to a stop
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Stopping Time"))
	static ENGINE_API float SpringCharacterStoppingTime(const float InitialVelocity, const float SmoothingTime = 0.2f, const float VelocityThreshold = 1.0f);

	/** Estimates an approximate smoothing time from a stopping distance for a character using by a simple damped spring. Assumes the initial acceleration is zero.
	 *
	 * @param InitialVelocity The velocity of the character at the start of the stop
	 * @param StoppingDistance The total distance traveled during the stop
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromStoppingDistance(const float InitialVelocity, const float StoppingDistance = 100.0f);

	/** Estimates an approximate smoothing time from a stopping time for a character using by a simple damped spring. Assumes the initial acceleration is zero.
	 *
	 * @param InitialVelocity The velocity of the character at the start of the stop
	 * @param StoppingTime The time taken to come to a stop
	 * @param VelocityThreshold The velocity threshold at which the character is considered to be stopped in cm/s
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromStoppingTime(const float InitialVelocity, const float StoppingTime = 1.0f, const float VelocityThreshold = 10.0f);

	/** Estimates an approximate starting distance (distance traveled until the velocity is within the VelocityThreshold of the target) from a smoothing time for a character using a simple damped spring
	 *
	 * @param TargetVelocity The target velocity of the character
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param VelocityThreshold The velocity threshold from the target at which the character is considered to have started in cm/s
	 * @return An estimation of the total distance traveled during the start
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Starting Distance"))
	static ENGINE_API float SpringCharacterStartingDistance(const float TargetVelocity, const float SmoothingTime = 0.2f, const float VelocityThreshold = 10.0f);

	/** Estimates an approximate starting time (time until the velocity is within the VelocityThreshold of the target) from a smoothing time for a character using a simple damped spring.
	 *
	 * @param TargetVelocity The target velocity of the character
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param VelocityThreshold The velocity threshold from the target at which the character is considered to have started in cm/s
	 * @return An estimation of the time taken to come to a start
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Starting Time"))
	static ENGINE_API float SpringCharacterStartingTime(const float TargetVelocity, const float SmoothingTime = 0.2f, const float VelocityThreshold = 10.0f);

	/** Estimates an approximate smoothing time from a starting distance for a character using by a simple damped spring.
	 *
	 * @param TargetVelocity The target velocity of the character
	 * @param StartingDistance The total distance traveled during the start
	 * @param VelocityThreshold The velocity threshold from the target at which the character is considered to have started in cm/s
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromStartingDistance(const float TargetVelocity, const float StartingDistance = 100.0f, const float VelocityThreshold = 10.0f);

	/** Estimates an approximate smoothing time from a starting time for a character using by a simple damped spring.
	 *
	 * @param TargetVelocity The target velocity of the character
	 * @param StartingTime The time taken to come to a start
	 * @param VelocityThreshold The velocity threshold from the target at which the character is considered to have started in cm/s
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromStartingTime(const float TargetVelocity, const float StartingTime = 1.0f, const float VelocityThreshold = 10.0f);

	/** Estimates an approximate turning time (time until the angle is within the AngleThreshold of the target) from a smoothing time for a character using a simple damped spring. Assumes the initial angular velocity is zero.
	 *
	 * @param TargetAngle The target angle of the character in degrees
	 * @param SmoothingTime The time over which to smooth turning. It takes roughly the smoothing time in order for the character to reach the target angle.
	 * @param AngleThreshold The angle threshold from the target at which the character is considered to have turned in degrees
	 * @return An estimation of the time taken to turn
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Turning Time"))
	static ENGINE_API float SpringCharacterTurningTime(const float TargetAngle, const float SmoothingTime = 0.2f, const float AngleThreshold = 5.0f);

	/** Estimates an approximate smoothing time from a turning time for a character using by a simple damped spring.
	 *
	 * @param TargetAngle The target angle of the character in degrees
	 * @param TurningTime The time taken to come to turn
	 * @param AngleThreshold The angle threshold from the target at which the character is considered to have turned in degrees
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromTurningTime(const float TargetAngle, const float TurningTime = 1.0f, const float AngleThreshold = 5.0f);

	/** Estimates an approximate maximum acceleration from a smoothing time for a character using a simple damped spring.
	 *
	 * @param InitialVelocity The initial velocity of the character
	 * @param InitialAcceleration The initial acceleration of the character
	 * @param TargetVelocity The target velocity of the character
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @return An estimation of the maximum acceleration
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Maximum Acceleration"))
	static ENGINE_API float SpringCharacterMaximumAcceleration(const float InitialVelocity, const float InitialAcceleration, const float TargetVelocity, const float SmoothingTime = 0.2f);

	/** Estimates an approximate smoothing time from a maximum acceleration for a character using by a simple damped spring.
	 *
	 * @param InitialVelocity The initial velocity of the character
	 * @param TargetVelocity The target velocity of the character
	 * @param MaximumAcceleration The maximum acceleration of the character
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromMaximumAcceleration(const float InitialVelocity, const float TargetVelocity, const float MaximumAcceleration);

	/** Estimates an approximate maximum angular velocity from a smoothing time for a character using a simple damped spring.
	 *
	 * @param InitialAngle The initial angle of the character in degrees
	 * @param InitialAngularVelocity The initial angular velocity of the character in deg/s
	 * @param TargetAngle The target angle of the character
	 * @param SmoothingTime The time over which to smooth angle. It takes roughly the smoothing time in order for the character to reach the target angle.
	 * @return An estimation of the maximum angular velocity
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Maximum Angular Velocity"))
	static ENGINE_API float SpringCharacterMaximumAngularVelocity(const float InitialAngle, const float InitialAngularVelocity, const float TargetAngle, const float SmoothingTime = 0.2f);

	/** Estimates an approximate smoothing time from a maximum angular velocity for a character using by a simple damped spring.
	 *
	 * @param InitialAngle The initial angle of the character in degrees
	 * @param TargetAngle The target angle of the character
	 * @param MaximumAngularVelocity The maximum angular velocity of the character
	 * @return An estimation of the SmoothingTime of the spring
	 */
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring", ReturnDisplayName = "Smoothing Time"))
	static ENGINE_API float SpringCharacterSmoothingTimeFromMaximumAngularVelocity(const float InitialAngle, const float TargetAngle, const float MaximumAngularVelocity);

	/** Update a position representing a character given a target velocity using a velocity spring.
	 * A velocity spring tracks an intermediate velocity which moves at a maximum acceleration linearly towards a target.
	 * This means unlike the "SpringCharacterUpdate", it will take longer to reach a target velocity that is further away from the current velocity.
	 * 
	 * @param InOutPosition The position of the character
	 * @param InOutVelocity The velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutVelocityIntermediate The intermediate velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutAcceleration The acceleration of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param TargetVelocity The target velocity of the character.
	 * @param DeltaTime The delta time to tick the character
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param MaxAcceleration Puts a limit on the maximum acceleration that the intermediate velocity can do each frame. If MaxAccel is very large, the behaviour will be the same as SpringCharacterUpdate
	 */
	UFUNCTION(BlueprintCallable, meta=(Category="Math|Spring", AutoCreateRefTerm = "TargetVelocity"))
	static ENGINE_API void VelocitySpringCharacterUpdate(UPARAM(ref) FVector& InOutPosition, UPARAM(ref) FVector& InOutVelocity, UPARAM(ref) FVector& InOutVelocityIntermediate, UPARAM(ref) FVector& InOutAcceleration, const FVector& TargetVelocity, float DeltaTime, float SmoothingTime = 0.2f, float MaxAcceleration = 1.0f);



	// Conversion Methods //

	/** Convert from smoothing time to spring strength.
	* 
	* @param SmoothingTime The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
	* @return The spring strength. This corresponds to the undamped frequency of the spring in hz.
	*/
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring"))
	static ENGINE_API float ConvertSmoothingTimeToStrength(float SmoothingTime);

	/** Convert from spring strength to smoothing time.
 	* 
 	* @param Strength The spring strength. This corresponds to the undamped frequency of the spring in hz.
 	* @return The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
 	*/
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring"))
	static ENGINE_API float ConvertStrengthToSmoothingTime(float Strength);

	/** Convert a halflife to a smoothing time
 	* 
 	* @param HalfLife The half life of the spring. How long it takes the value to get halfway towards the target.
 	* @return The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
 	*/
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring"))
	static ENGINE_API float ConvertHalfLifeToSmoothingTime(float HalfLife);

	/** Convert a smoothing time to a half life
 	* 
 	* @param SmoothingTime The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate. 
 	* @return The half life of the spring. How long it takes the value to get halfway towards the target. 
 	*/
	UFUNCTION(BlueprintPure, meta=(Category="Math|Spring"))
	static ENGINE_API float ConvertSmoothingTimeToHalfLife(float SmoothingTime);


	// Velocity Tracking //

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityFloat(UPARAM(ref) float& InOutValue, UPARAM(ref) float& InOutVelocity, float InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityVector(UPARAM(ref) FVector& InOutValue, UPARAM(ref) FVector& InOutVelocity, FVector InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityVector2D(UPARAM(ref) FVector2D& InOutValue, UPARAM(ref) FVector2D& InOutVelocity, FVector2D InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityQuat(UPARAM(ref) FQuat& InOutValue, UPARAM(ref) FVector& InOutVelocity, FQuat InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityRotator(UPARAM(ref) FRotator& InOutValue, UPARAM(ref) FVector& InOutVelocity, FRotator InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityAngle(UPARAM(ref) float& InOutValue, UPARAM(ref) float& InOutVelocity, float InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityScale(UPARAM(ref) FVector& InOutValue, UPARAM(ref) FVector& InOutVelocity, FVector InValue, float DeltaTime);

	/** Update the value and velocity using the finite difference  */
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void TrackVelocityTransform(UPARAM(ref) FTransform& InOutValue, UPARAM(ref) FSpringTransformVelocity& InOutVelocity, FTransform InValue, float DeltaTime);

	// Spring Inertialization //

	/** Applies a spring inertialization to a float value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float SpringInertializeApplyToFloat(float InValue, float InValueOffset, float InVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionFloat(UPARAM(ref) float& InOutValueOffset, UPARAM(ref) float& InOutVelocityOffset, float InSrcValue, float InSrcVelocity, float InDstValue, float InDstVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a vector value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector SpringInertializeApplyToVector(FVector InValue, FVector InValueOffset, FVector InVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionVector(UPARAM(ref) FVector& InOutValueOffset, UPARAM(ref) FVector& InOutVelocityOffset, FVector InSrcValue, FVector InSrcVelocity, FVector InDstValue, FVector InDstVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a Vector2D value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector2D SpringInertializeApplyToVector2D(FVector2D InValue, FVector2D InValueOffset, FVector2D InVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionVector2D(UPARAM(ref) FVector2D& InOutValueOffset, UPARAM(ref) FVector2D& InOutVelocityOffset, FVector2D InSrcValue, FVector2D InSrcVelocity, FVector2D InDstValue, FVector2D InDstVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a quaternion, applying the appropriate offset given the time since transition.
	*
	* @param InRotation					The rotation to apply the inertialization to
	* @param InRotationOffset			Offset rotation at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The rotation with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FQuat SpringInertializeApplyToQuat(FQuat InRotation, FQuat InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutRotationOffset		Offset rotation to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcRotation				Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstRotation				Rotation of the destination (rotation transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionQuat(UPARAM(ref) FQuat& InOutRotationOffset, UPARAM(ref) FVector& InOutAngularVelocityOffset, FQuat InSrcRotation, FVector InSrcAngularVelocity, FQuat InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a rotator, applying the appropriate offset given the time since transition.
	*
	* @param InRotation					The rotation to apply the inertialization to
	* @param InRotationOffset			Offset rotation at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The rotation with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FRotator SpringInertializeApplyToRotator(FRotator InRotation, FRotator InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutRotationOffset		Offset rotation to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcRotation				Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstRotation				Rotation of the destination (rotation transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionRotator(UPARAM(ref) FRotator& InOutRotationOffset, UPARAM(ref) FVector& InOutAngularVelocityOffset, FRotator InSrcRotation, FVector InSrcAngularVelocity, FRotator InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a angle, adding the appropriate offset given the time since transition.
	*
	* @param InAngle					The angle to apply the inertialization to
	* @param InAngleOffset				Offset angle at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The angle with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float SpringInertializeApplyToAngle(float InAngle, float InAngleOffset, float InAngularVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutAngleOffset			Offset angle to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcAngle					Angle of the source (angle transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstAngle					Angle of the destination (angle transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionAngle(UPARAM(ref) float& InOutAngleOffset, UPARAM(ref) float& InOutAngularVelocityOffset, float InSrcAngle, float InSrcAngularVelocity, float InDstAngle, float InDstAngularVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a scale, applying the appropriate offset given the time since transition.
	*
	* @param InScale					The scale to apply the inertialization to
	* @param InScaleOffset				Offset scale at the time of transition
	* @param InScalarVelocityOffset		Offset scalar velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The scale with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector SpringInertializeApplyToScale(FVector InScale, FVector InScaleOffset, FVector InScalarVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutScaleOffset			Offset scale to update
	* @param InOutScalarVelocityOffset	Offset scalar velocity to update
	* @param InSrcScale					Scale of the source (scale transitioning from)
	* @param InSrcScalarVelocity		Scalar velocity of the source (scalar velocity transitioning from)
	* @param InDstScale					Scale of the destination (scale transitioning to)
	* @param InDstScalarVelocity		Scalar velocity of the destination (scalar velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionScale(UPARAM(ref) FVector& InOutScaleOffset, UPARAM(ref) FVector& InOutScalarVelocityOffset, FVector InSrcScale, FVector InSrcScalarVelocity, FVector InDstScale, FVector InDstScalarVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Applies a spring inertialization to a transform, applying the appropriate offset given the time since transition.
	*
	* @param InTransform				The transform to apply the inertialization to
	* @param InTransformOffset			Offset transform at the time of transition
	* @param InVelocityOffset			Offset velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The transform with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FTransform SpringInertializeApplyToTransform(FTransform InTransform, FTransform InTransformOffset, FSpringTransformVelocity InVelocityOffset, float TimeSinceTransition, float SmoothingTime = 0.2f);

	/** Transitions a spring inertialization, updating the appropriate offsets.
	*
	* @param InOutTransformOffset		Offset transform to update
	* @param InOutVelocityOffset		Offset velocity to update
	* @param InSrcTransform				Transform of the source (transform transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param InDstTransform				Transform of the destination (transform transitioning to)
	* @param InDstVelocity				Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param SmoothingTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void SpringInertializeTransitionTransform(UPARAM(ref) FTransform& InOutTransformOffset, UPARAM(ref) FSpringTransformVelocity& InOutVelocityOffset, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, FTransform InDstTransform, FSpringTransformVelocity InDstVelocity, float TimeSinceTransition, float SmoothingTime = 0.2f);

	// Cubic Inertialization //

	/** Applies a cubic inertialization to a float value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float CubicInertializeApplyToFloat(float InValue, float InValueOffset, float InVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionFloat(UPARAM(ref) float& InOutValueOffset, UPARAM(ref) float& InOutVelocityOffset, float InSrcValue, float InSrcVelocity, float InDstValue, float InDstVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a vector value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector CubicInertializeApplyToVector(FVector InValue, FVector InValueOffset, FVector InVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionVector(UPARAM(ref) FVector& InOutValueOffset, UPARAM(ref) FVector& InOutVelocityOffset, FVector InSrcValue, FVector InSrcVelocity, FVector InDstValue, FVector InDstVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a Vector2D value, adding the appropriate offset given the time since transition.
	*
	* @param InValue				The value to apply the inertialization to
	* @param InValueOffset			Offset value at the time of transition
	* @param InVelocityOffset		Offset velocity at the time of transition
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return						The value with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector2D CubicInertializeApplyToVector2D(FVector2D InValue, FVector2D InValueOffset, FVector2D InVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutValueOffset		Offset value to update
	* @param InOutVelocityOffset	Offset velocity to update
	* @param InSrcValue				Value of the source (value transitioning from)
	* @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	* @param InDstValue				Value of the destination (value transitioning to)
	* @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition	Time since the last transition
	* @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionVector2D(UPARAM(ref) FVector2D& InOutValueOffset, UPARAM(ref) FVector2D& InOutVelocityOffset, FVector2D InSrcValue, FVector2D InSrcVelocity, FVector2D InDstValue, FVector2D InDstVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a quaternion, applying the appropriate offset given the time since transition.
	*
	* @param InRotation					The rotation to apply the inertialization to
	* @param InRotationOffset			Offset rotation at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The rotation with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FQuat CubicInertializeApplyToQuat(FQuat InRotation, FQuat InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutRotationOffset		Offset rotation to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcRotation				Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstRotation				Rotation of the destination (rotation transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionQuat(UPARAM(ref) FQuat& InOutRotationOffset, UPARAM(ref) FVector& InOutAngularVelocityOffset, FQuat InSrcRotation, FVector InSrcAngularVelocity, FQuat InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a rotator, applying the appropriate offset given the time since transition.
	*
	* @param InRotation					The rotation to apply the inertialization to
	* @param InRotationOffset			Offset rotation at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The rotation with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FRotator CubicInertializeApplyToRotator(FRotator InRotation, FRotator InRotationOffset, FVector InAngularVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutRotationOffset		Offset rotation to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcRotation				Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstRotation				Rotation of the destination (rotation transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionRotator(UPARAM(ref) FRotator& InOutRotationOffset, UPARAM(ref) FVector& InOutAngularVelocityOffset, FRotator InSrcRotation, FVector InSrcAngularVelocity, FRotator InDstRotation, FVector InDstAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to an angle, applying the appropriate offset given the time since transition.
	*
	* @param InAngle					The angle to apply the inertialization to
	* @param InAngleOffset				Offset angle at the time of transition
	* @param InAngularVelocityOffset	Offset angular velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The angle with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float CubicInertializeApplyToAngle(float InAngle, float InAngleOffset, float InAngularVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutAngleOffset			Offset angle to update
	* @param InOutAngularVelocityOffset	Offset angular velocity to update
	* @param InSrcAngle					Angle of the source (angle transitioning from)
	* @param InSrcAngularVelocity		Angular velocity of the source (angular velocity transitioning from)
	* @param InDstAngle					Angle of the destination (angle transitioning to)
	* @param InDstAngularVelocity		Angular velocity of the destination (angular velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionAngle(UPARAM(ref) float& InOutAngleOffset, UPARAM(ref) float& InOutAngularVelocityOffset, float InSrcAngle, float InSrcAngularVelocity, float InDstAngle, float InDstAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a scale, applying the appropriate offset given the time since transition.
	*
	* @param InScale					The scale to apply the inertialization to
	* @param InScaleOffset				Offset scale at the time of transition
	* @param InScalarVelocityOffset		Offset scalar velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The scale with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector CubicInertializeApplyToScale(FVector InScale, FVector InScaleOffset, FVector InScalarVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutScaleOffset			Offset scale to update
	* @param InOutScalarVelocityOffset	Offset scalar velocity to update
	* @param InSrcScale					Scale of the source (scale transitioning from)
	* @param InSrcScalarVelocity		Scalar velocity of the source (scalar velocity transitioning from)
	* @param InDstScale					Scale of the destination (scale transitioning to)
	* @param InDstScalarVelocity		Scalar velocity of the destination (scalar velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionScale(UPARAM(ref) FVector& InOutScaleOffset, UPARAM(ref) FVector& InOutScalarVelocityOffset, FVector InSrcScale, FVector InSrcScalarVelocity, FVector InDstScale, FVector InDstScalarVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Applies a cubic inertialization to a transform, applying the appropriate offset given the time since transition.
	*
	* @param InTransform				The transform to apply the inertialization to
	* @param InTransformOffset			Offset transform at the time of transition
	* @param InVelocityOffset			Offset velocity at the time of transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @return							The transform with appropriate offset applied
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FTransform CubicInertializeApplyToTransform(FTransform InTransform, FTransform InTransformOffset, FSpringTransformVelocity InVelocityOffset, float TimeSinceTransition, float BlendTime = 0.2f);

	/** Transitions a cubic inertialization, updating the appropriate offsets.
	*
	* @param InOutTransformOffset		Offset transform to update
	* @param InOutVelocityOffset		Offset velocity to update
	* @param InSrcTransform				Transform of the source (transform transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param InDstTransform				Transform of the destination (transform transitioning to)
	* @param InDstVelocity				Velocity of the destination (velocity transitioning to)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void CubicInertializeTransitionTransform(UPARAM(ref) FTransform& InOutTransformOffset, UPARAM(ref) FSpringTransformVelocity& InOutVelocityOffset, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, FTransform InDstTransform, FSpringTransformVelocity InDstVelocity, float TimeSinceTransition, float BlendTime = 0.2f);

	// Dead Blending //

	/** Applies a dead blend to a float value, blending it with an extrapolation of the value and velocity at the point of the last transition.
	*
	* @param InValue					The value to apply the dead blend to
	* @param InValueTransition			The value at the time of the last transition
	* @param InVelocityTransition		The velocity at the time of the last transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return							The blended value
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float DeadBlendApplyToFloat(float InValue, float InValueTransition, float InVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutValueTransition		Value at the point of transition
	* @param InOutVelocityTransition	Velocity at the point of transition
	* @param InSrcValue					Value of the source (value transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionFloat(UPARAM(ref) float& InOutValueTransition, UPARAM(ref) float& InOutVelocityTransition, float InSrcValue, float InSrcVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a vector value, blending it with an extrapolation of the value and velocity at the point of the last transition.
	*
	* @param InValue					The value to apply the dead blend to
	* @param InValueTransition			The value at the time of the last transition
	* @param InVelocityTransition		The velocity at the time of the last transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return							The blended value
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector DeadBlendApplyToVector(FVector InValue, FVector InValueTransition, FVector InVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutValueTransition		Value at the point of transition
	* @param InOutVelocityTransition	Velocity at the point of transition
	* @param InSrcValue					Value of the source (value transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionVector(UPARAM(ref) FVector& InOutValueTransition, UPARAM(ref) FVector& InOutVelocityTransition, FVector InSrcValue, FVector InSrcVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a Vector2D value, blending it with an extrapolation of the value and velocity at the point of the last transition.
	*
	* @param InValue					The value to apply the dead blend to
	* @param InValueTransition			The value at the time of the last transition
	* @param InVelocityTransition		The velocity at the time of the last transition
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return							The blended value
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector2D DeadBlendApplyToVector2D(FVector2D InValue, FVector2D InValueTransition, FVector2D InVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutValueTransition		Value at the point of transition
	* @param InOutVelocityTransition	Velocity at the point of transition
	* @param InSrcValue					Value of the source (value transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionVector2D(UPARAM(ref) FVector2D& InOutValueTransition, UPARAM(ref) FVector2D& InOutVelocityTransition, FVector2D InSrcValue, FVector2D InSrcVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a quaternion, blending it with an extrapolation of the rotation and velocity at the point of the last transition.
	*
	* @param InRotation						The rotation to apply the dead blend to
	* @param InRotationTransition			The rotation at the time of the last transition
	* @param InAngularVelocityTransition	The angular velocity at the time of the last transition
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return								The blended rotation
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FQuat DeadBlendApplyToQuat(FQuat InRotation, FQuat InRotationTransition, FVector InAngularVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutRotationTransition		Rotation at the point of transition
	* @param InOutAngularVelocityTransition	Angular velocity at the point of transition
	* @param InSrcRotation					Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity			Angular velocity of the source (angular velocity transitioning from)
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionQuat(UPARAM(ref) FQuat& InOutRotationTransition, UPARAM(ref) FVector& InOutAngularVelocityTransition, FQuat InSrcRotation, FVector InSrcAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a rotator, blending it with an extrapolation of the rotation and velocity at the point of the last transition.
	*
	* @param InRotation						The rotation to apply the dead blend to
	* @param InRotationTransition			The rotation at the time of the last transition
	* @param InAngularVelocityTransition	The angular velocity at the time of the last transition
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return								The blended rotation
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FRotator DeadBlendApplyToRotator(FRotator InRotation, FRotator InRotationTransition, FVector InAngularVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutRotationTransition		Rotation at the point of transition
	* @param InOutAngularVelocityTransition	Angular velocity at the point of transition
	* @param InSrcRotation					Rotation of the source (rotation transitioning from)
	* @param InSrcAngularVelocity			Angular velocity of the source (angular velocity transitioning from)
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionRotator(UPARAM(ref) FRotator& InOutRotationTransition, UPARAM(ref) FVector& InOutAngularVelocityTransition, FRotator InSrcRotation, FVector InSrcAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to an angle, blending it with an extrapolation of the angle and velocity at the point of the last transition.
	*
	* @param InAngle						The angle to apply the dead blend to
	* @param InAngleTransition				The angle at the time of the last transition
	* @param InAngularVelocityTransition	The angular velocity at the time of the last transition
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return								The blended angle
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API float DeadBlendApplyToAngle(float InAngle, float InAngleTransition, float InAngularVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutAngleTransition			Angle at the point of transition
	* @param InOutAngularVelocityTransition	Angular velocity at the point of transition
	* @param InSrcAngle						Angle of the source (angle transitioning from)
	* @param InSrcAngularVelocity			Angular velocity of the source (angular velocity transitioning from)
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionAngle(UPARAM(ref) float& InOutAngleTransition, UPARAM(ref) float& InOutAngularVelocityTransition, float InSrcAngle, float InSrcAngularVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a scale, blending it with an extrapolation of the scale and velocity at the point of the last transition.
	*
	* @param InScale						The scale to apply the dead blend to
	* @param InScaleTransition				The scale at the time of the last transition
	* @param InScalarVelocityTransition		The scalar velocity at the time of the last transition
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return								The blended scale
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FVector DeadBlendApplyToScale(FVector InScale, FVector InScaleTransition, FVector InScalarVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutScaleTransition			Scale at the point of transition
	* @param InOutScalarVelocityTransition	Scalar velocity at the point of transition
	* @param InSrcScale						Scale of the source (scale transitioning from)
	* @param InSrcScalarVelocity			Scalar velocity of the source (scalar velocity transitioning from)
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionScale(UPARAM(ref) FVector& InOutScaleTransition, UPARAM(ref) FVector& InOutScalarVelocityTransition, FVector InSrcScale, FVector InSrcScalarVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Applies a dead blend to a transform, blending it with an extrapolation of the transform and velocity at the point of the last transition.
	*
	* @param InTransform					The transform to apply the dead blend to
	* @param InTransformTransition			The transform at the time of the last transition
	* @param InVelocityTransition			The velocity at the time of the last transition
	* @param TimeSinceTransition			Time since the last transition
	* @param BlendTime						Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime					Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	* @return								The blended transform
	*/
	UFUNCTION(BlueprintPure, meta = (Category = "Math|Spring"))
	static ENGINE_API FTransform DeadBlendApplyToTransform(FTransform InTransform, FTransform InTransformTransition, FSpringTransformVelocity InVelocityTransition, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	*
	* @param InOutTransformTransition	Transform at the point of transition
	* @param InOutVelocityTransition	Velocity at the point of transition
	* @param InSrcTransform				Transform of the source (transform transitioning from)
	* @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	* @param TimeSinceTransition		Time since the last transition
	* @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	* @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "Math|Spring"))
	static ENGINE_API void DeadBlendTransitionTransform(UPARAM(ref) FTransform& InOutTransformTransition, UPARAM(ref) FSpringTransformVelocity& InOutVelocityTransition, FTransform InSrcTransform, FSpringTransformVelocity InSrcVelocity, float TimeSinceTransition, float BlendTime = 0.2f, float SmoothingTime = 0.2f);

};
