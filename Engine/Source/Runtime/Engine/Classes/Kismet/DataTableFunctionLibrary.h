// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/DataTable.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "DataTableFunctionLibrary.generated.h"

struct FRealCurve;
class UCurveTable;

/** Enum used to indicate success or failure of EvaluateCurveTableRow. */
UENUM()
namespace EEvaluateCurveTableResult
{
    enum Type : int
    {
        /** Found the row successfully. */
        RowFound,
        /** Failed to find the row. */
        RowNotFound,
    };
}

UCLASS(MinimalAPI)
class UDataTableFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintCallable, Category = "DataTable", meta = (ExpandEnumAsExecs="OutResult", DataTablePin="CurveTable"))
	static ENGINE_API void EvaluateCurveTableRow(UCurveTable* CurveTable, FName RowName, float InXY, TEnumAsByte<EEvaluateCurveTableResult::Type>& OutResult, float& OutXY,const FString& ContextString);

	/**
	 * Convert a curve table to JSON.
	 * @param CurveTable The Curve Table to export.
	 * @return The CurveTable contents encoded as a string.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API FString CurveTableToJson(UCurveTable* CurveTable);

	/**
	 * Convert a curve table to JSON.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API TArray<FString> JsonToCurveTable(UCurveTable* CurveTable, const FString& InString);

	/**
	 * Adds a simple curve to a curve table.
	 * @param CurveTable The Curve Table to modify
	 * @param RowName The name of the new row
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API bool AddSimpleCurveToTable(UCurveTable* CurveTable, FName RowName);

	/**
	 * Returns the names of the rows in the curve table.
	 * @param CurveTable The Curve Table to query
	 * @return A list of names of rows.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API TArray<FName> GetCurveTableRowNames(UCurveTable* CurveTable);

	/**
	 * Adds a simple curve to a curve table.
	 * @param CurveTable The Curve Table to modify
	 * @param RowName The name of the row to modify
	 * @param DefaultValue The default value of the row.
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API bool SetCurveTableRowDefault(UCurveTable* CurveTable, FName RowName, float DefaultValue);
	
	/**
	 * Adds a key to a row in a CurveTable
	 * @param CurveTable The Curve Table to modify.
	 * @param RowName The name of the row to modify.
	 * @param InTime The time of the new key.
	 * @param InValue The value of the new key.
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API bool AddCurveTableKey(UCurveTable* CurveTable, FName RowName, float InTime, float InValue);

	/**
	 * Returns a list of keys in a row of a SimpleCurve table.
	 * @param CurveTable The Curve Table to query.
	 * @param RowName The name of the row to query.
	 * @return The list of keys in the curve, or empty if the curve doesn't exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API TArray<FSimpleCurveKey> GetCurveTableKeys(UCurveTable* CurveTable, FName RowName);

	/**
	 * Removes a curve from a curve table.
	 * @param CurveTable The Curve Table to modify
	 * @param RowName The name of the row to remove
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API bool RemoveCurveTableRow(UCurveTable* CurveTable, FName RowName);
	
	/**
	 * Renames a curve table row.
	 * @param CurveTable The Curve Table to modify
	 * @param RowName The name of the row to rename
	 * @param NewRowName The new name of the row
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable")
	static ENGINE_API bool RenameCurveTableRow(UCurveTable* CurveTable, const FName& RowName, const FName& NewRowName);

    
	/** Get the row struct used by the given Data Table, if any */
	UFUNCTION(BlueprintPure, Category = "DataTable", meta=(ScriptMethod="GetRowStruct"))
 	static ENGINE_API const UScriptStruct* GetDataTableRowStruct(const UDataTable* Table);

	// Returns whether or not Table contains a row named RowName
  	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="DoesRowExist"))
 	static ENGINE_API bool DoesDataTableRowExist(const UDataTable* Table, FName RowName);
    
	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="GetRowNames"))
	static ENGINE_API void GetDataTableRowNames(const UDataTable* Table, TArray<FName>& OutRowNames);

	/**
	 * Get the name of each column in this Data Table.
	 * @note These are always the raw property names (@see GetDataTableColumnAsString) rather than the friendly export name that would be used in a CSV/JSON export (@see GetDataTableColumnNameFromExportName).
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="GetColumnNames"))
	static ENGINE_API void GetDataTableColumnNames(const UDataTable* Table, TArray<FName>& OutColumnNames);

	/**
	 * Get the friendly export name of each column in this Data Table.
	 * @see GetDataTableColumnNameFromExportName.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="GetColumnExportNames"))
	static ENGINE_API void GetDataTableColumnExportNames(const UDataTable* Table, TArray<FString>& OutExportColumnNames);

	/**
	 * Get the raw property name of a data table column from its friendly export name.
	 * @return True if a column was found for the friendly name, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="GetColumnNameFromExportName"))
	static ENGINE_API bool GetDataTableColumnNameFromExportName(const UDataTable* Table, const FString& ColumnExportName, FName& OutColumnName);

	/**
	 * Export from the DataTable all the row for one column. Export it as string. The row name is not included.
	 * @see GetDataTableColumnNames.
	 * @see GetDataTableColumnNameFromExportName.
	 */
	UFUNCTION(BlueprintCallable, Category = "DataTable", meta=(ScriptMethod="GetColumnAsString"))
	static ENGINE_API TArray<FString> GetDataTableColumnAsString(const UDataTable* DataTable, FName PropertyName);

    /** Get a Row from a DataTable given a RowName */
    UFUNCTION(BlueprintCallable, CustomThunk, Category = "DataTable", meta=(CustomStructureParam = "OutRow", BlueprintInternalUseOnly="true"))
    static ENGINE_API bool GetDataTableRowFromName(UDataTable* Table, FName RowName, FTableRowBase& OutRow);
	static ENGINE_API bool Generic_GetDataTableRowFromName(const UDataTable* Table, FName RowName, void* OutRowPtr);
	DECLARE_FUNCTION(execGetDataTableRowFromName);

#if WITH_EDITOR
	/** 
	 * Empty and fill a Data Table from CSV string.
	 * @param	CSVString			The Data that representing the contents of a CSV file.
	 * @param	ImportRowStruct		Optional row struct to apply on import. If set will also force the import to run automated (no questions or dialogs).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName="Fill Data Table from CSV String", meta=(AdvancedDisplay="ImportRowStruct", ScriptMethod="FillFromCSVString"))
	static ENGINE_API bool FillDataTableFromCSVString(UDataTable* DataTable, const FString& CSVString, UScriptStruct* ImportRowStruct = nullptr);

	/** 
	 * Empty and fill a Data Table from CSV file.
	 * @param	CSVFilePath			The file path of the CSV file.
	 * @param	ImportRowStruct		Optional row struct to apply on import. If set will also force the import to run automated (no questions or dialogs).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Fill Data Table from CSV File", meta=(AdvancedDisplay="ImportRowStruct", ScriptMethod="FillFromCSVFile"))
	static ENGINE_API bool FillDataTableFromCSVFile(UDataTable* DataTable, const FString& CSVFilePath, UScriptStruct* ImportRowStruct = nullptr);

	/** 
	 * Empty and fill a Data Table from JSON string.
	 * @param	JSONString			The Data that representing the contents of a JSON file.
	 * @param	ImportRowStruct		Optional row struct to apply on import. If set will also force the import to run automated (no questions or dialogs).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Fill Data Table from JSON String", meta=(AdvancedDisplay="ImportRowStruct", ScriptMethod="FillFromJSONString"))
	static ENGINE_API bool FillDataTableFromJSONString(UDataTable* DataTable, const FString& JSONString, UScriptStruct* ImportRowStruct = nullptr);

	/** 
	 * Empty and fill a Data Table from JSON file.
	 * @param	JSONFilePath		The file path of the JSON file.
	 * @param	ImportRowStruct		Optional row struct to apply on import. If set will also force the import to run automated (no questions or dialogs).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Fill Data Table from JSON File", meta=(AdvancedDisplay="ImportRowStruct", ScriptMethod="FillFromJSONFile"))
	static ENGINE_API bool FillDataTableFromJSONFile(UDataTable* DataTable, const FString& JSONFilePath, UScriptStruct* ImportRowStruct = nullptr);

	/** 
	 * Export a Data Table to CSV string.
	 * @param	OutCSVString Output representing the contents of a CSV file.
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName="Export Data Table to CSV String", meta=(ScriptMethod="ExportToCSVString"))
	static ENGINE_API bool ExportDataTableToCSVString(const UDataTable* DataTable, FString& OutCSVString);

	/** 
	 * Export a Data Table to CSV file.
	 * @param	CSVFilePath	The file path of the CSV file to write (output file is UTF-8).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Export Data Table to CSV File", meta=(ScriptMethod="ExportToCSVFile"))
	static ENGINE_API bool ExportDataTableToCSVFile(const UDataTable* DataTable, const FString& CSVFilePath);

	/** 
	 * Export a Data Table to JSON string.
	 * @param	OutJSONString Output representing the contents of a JSON file.
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Export Data Table to JSON String", meta=(ScriptMethod="ExportToJSONString"))
	static ENGINE_API bool ExportDataTableToJSONString(const UDataTable* DataTable, FString& OutJSONString);

	/** 
	 * Export a Data Table to JSON file.
	 * @param	JSONFilePath The file path of the JSON file to write (output file is UTF-8).
	 * @return	True if the operation succeeds, check the log for errors if it didn't succeed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", DisplayName = "Export Data Table to JSON File", meta=(ScriptMethod="ExportToJSONFile"))
	static ENGINE_API bool ExportDataTableToJSONFile(const UDataTable* DataTable, const FString& JSONFilePath);

	/** Add a row to a Data Table with the provided name and data. */
    UFUNCTION(BlueprintCallable, CustomThunk, Category = "Editor Scripting | DataTable", meta=(AutoCreateRefTerm="RowName", CustomStructureParam="RowData"))
	static ENGINE_API void AddDataTableRow(UDataTable* const DataTable, const FName& RowName, const FTableRowBase& RowData);
    DECLARE_FUNCTION(execAddDataTableRow);

	/** Removes the row with the provided name from a Data Table. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DataTable", meta = (AutoCreateRefTerm = "RowName"))
	static ENGINE_API void RemoveDataTableRow(UDataTable* DataTable, const FName& RowName);
#endif //WITH_EDITOR
};
