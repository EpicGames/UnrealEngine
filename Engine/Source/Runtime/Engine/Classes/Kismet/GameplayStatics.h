// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "UObject/Interface.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "Engine/LatentActionManager.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Sound/DialogueTypes.h"
#include "GameplayStaticsTypes.h"
#include "Particles/WorldPSCPool.h"
#include "GameplayStatics.generated.h"

class ASceneCapture2D;
class UAudioComponent;
class UInitialActiveSoundParams;
class UBlueprint;
class UDecalComponent;
class UDialogueWave;
class UParticleSystem;
class UParticleSystemComponent;
class USaveGame;
class USceneCaptureComponent2D;
class USceneComponent;
class USoundAttenuation;
class USoundBase;
class USoundConcurrency;
class UStaticMesh;
class FMemoryReader;
class APlayerController;
struct FDialogueContext;



/** Delegate called from AsyncLoadGameFromSlot. First two parameters are passed in SlotName and UserIndex, third parameter is a bool indicating success (true) or failure (false). */
DECLARE_DELEGATE_ThreeParams(FAsyncSaveGameToSlotDelegate, const FString&, const int32, bool);

/** Delegate called from AsyncLoadGameFromSlot. First two parameters are passed in SlotName and UserIndex, third parameter is the returned SaveGame, or null if it failed to load. */
DECLARE_DELEGATE_ThreeParams(FAsyncLoadGameFromSlotDelegate, const FString&, const int32, USaveGame*);

/** Static class with useful gameplay utility functions that can be called from both Blueprint and C++ */
UCLASS(MinimalAPI)
class UGameplayStatics : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	// --- Create Object
	UFUNCTION(BlueprintCallable, Category = "Spawning", meta = (BlueprintInternalUseOnly = "true", DefaultToSelf = "Outer"))
	static ENGINE_API UObject* SpawnObject(TSubclassOf<UObject> ObjectClass, UObject* Outer);

	// --- Spawning functions ------------------------------

	/** Spawns an instance of a blueprint, but does not automatically run its construction script.  */
	UFUNCTION(BlueprintCallable, Category="Spawning", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction = "true", BlueprintInternalUseOnly = "true", DeprecatedFunction="Note", DeprecationMessage = "Use BeginSpawningActorFromClass"))
	static ENGINE_API class AActor* BeginSpawningActorFromBlueprint(const UObject* WorldContextObject, const class UBlueprint* Blueprint, const FTransform& SpawnTransform, bool bNoCollisionFail);

	/** Spawns an instance of an actor class, but does not automatically run its construction script.  */
	UFUNCTION(BlueprintCallable, Category = "Spawning", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true", BlueprintInternalUseOnly = "true"))
	static ENGINE_API class AActor* BeginDeferredActorSpawnFromClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, const FTransform& SpawnTransform, ESpawnActorCollisionHandlingMethod CollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::Undefined, AActor* Owner = nullptr, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::MultiplyWithRoot);

	/** 'Finish' spawning an actor.  This will run the construction script. */
	UFUNCTION(BlueprintCallable, Category="Spawning", meta=(UnsafeDuringActorConstruction = "true", BlueprintInternalUseOnly = "true"))
	static ENGINE_API class AActor* FinishSpawningActor(class AActor* Actor, const FTransform& SpawnTransform, ESpawnActorScaleMethod TransformScaleMethod = ESpawnActorScaleMethod::MultiplyWithRoot);

	// --- Actor functions ------------------------------

	/** Find the average location (centroid) of an array of Actors */
	UFUNCTION(BlueprintCallable, Category="Transformation")
	static ENGINE_API FVector GetActorArrayAverageLocation(const TArray<AActor*>& Actors);

	/** Bind the bounds of an array of Actors */
	UFUNCTION(BlueprintCallable, Category="Collision")
	static ENGINE_API void GetActorArrayBounds(const TArray<AActor*>& Actors, bool bOnlyCollidingComponents, FVector& Center, FVector& BoxExtent);
	
	/** 
	 *	Find the first Actor in the world of the specified class. 
	 *	@param	ActorClass	Class of Actor to find. Must be specified or result will be empty.
	 *	@return				Actor of the specified class.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor", meta=(WorldContext="WorldContextObject", DeterminesOutputType="ActorClass"))
	static ENGINE_API class AActor* GetActorOfClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass);

	/** 
	 *	Find all Actors in the world of the specified class. 
	 *	This will be slow if there are many actors of the specified class.
	 *	@param	ActorClass	Class of Actor to find. Must be specified or result array will be empty.
	 *	@param	OutActors	Output array of Actors of the specified class.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor",  meta=(WorldContext="WorldContextObject", DeterminesOutputType="ActorClass", DynamicOutputParam="OutActors"))
	static ENGINE_API void GetAllActorsOfClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, TArray<AActor*>& OutActors);

	/** 
	 *	Find all Actors in the world with the specified interface.
	 *	This is a very slow operation, as it will search over every actor in the world.
	 *	@param	Interface	Interface to find. Must be specified or result array will be empty.
	 *	@param	OutActors	Output array of Actors of the specified interface.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor",  meta=(WorldContext="WorldContextObject", DeterminesOutputType="Interface", DynamicOutputParam="OutActors"))
	static ENGINE_API void GetAllActorsWithInterface(const UObject* WorldContextObject, TSubclassOf<UInterface> Interface, TArray<AActor*>& OutActors);

	/**
	 *	Find all Actors in the world with the specified tag.
	 *	This is a very slow operation, as it will search over every actor in the world.
	 *	@param	Tag			Tag to find. Must be specified or result array will be empty.
	 *	@param	OutActors	Output array of Actors of the specified tag.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor",  meta=(WorldContext="WorldContextObject"))
	static ENGINE_API void GetAllActorsWithTag(const UObject* WorldContextObject, FName Tag, TArray<AActor*>& OutActors);

	/**
	 *	Find all Actors in the world of the specified class with the specified tag.
	 *	This will be slow if there are many actors of the specified class.
	 *	@param	Tag			Tag to find. Must be specified or result array will be empty.
	 *	@param	ActorClass	Class of Actor to find. Must be specified or result array will be empty.
	 *	@param	OutActors	Output array of Actors of the specified tag.
	 */
	UFUNCTION(BlueprintCallable, Category = "Actor", meta = (WorldContext="WorldContextObject", DeterminesOutputType="ActorClass", DynamicOutputParam="OutActors"))
	static ENGINE_API void GetAllActorsOfClassWithTag(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, FName Tag, TArray<AActor*>& OutActors);

	/**
	 *	Returns an Actor nearest to Origin from ActorsToCheck array.
	 *	@param	Origin			World Location from which the distance is measured.
	 *	@param	ActorsToCheck	Array of Actors to examine and return Actor nearest to Origin.
	 *	@param	Distance	Distance from Origin to the returned Actor.
	 *	@return				Nearest Actor.
	 */
	UFUNCTION(BlueprintPure, Category = "Actor")
	static ENGINE_API AActor* FindNearestActor(FVector Origin, const TArray<AActor*>& ActorsToCheck, float& Distance);

	// --- Player functions ------------------------------

	/** Returns the game instance object  */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API class UGameInstance* GetGameInstance(const UObject* WorldContextObject);

	/**
	 * Returns the number of active player states, there is one player state for every connected player even if they are a remote client.
	 * Indexes up to this can be use as PlayerStateIndex parameters for other functions.
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta = (WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API int32 GetNumPlayerStates(const UObject* WorldContextObject);

	/**
	 * Returns the player state at the given index in the game state's PlayerArray. 
	 * This will work on both the client and server and the index will be consistent.
	 * After initial replication, all clients and the server will have access to PlayerStates for all connected players.
	 *
	 * @param PlayerStateIndex	Index into the game state's PlayerArray
	 */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API class APlayerState* GetPlayerState(const UObject* WorldContextObject, int32 PlayerStateIndex);

	/**
	 * Returns the player state that matches the passed in online id, or null for an invalid one.
	 * This will work on both the client and server for local and remote players.
	 *
	 * @param UniqueId	The player's unique net/online id
	 */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API class APlayerState* GetPlayerStateFromUniqueNetId(const UObject* WorldContextObject, const FUniqueNetIdRepl& UniqueId);

	/**
	 * Returns the total number of available player controllers, including remote players when called on a server.
	 * Indexes up to this can be used as PlayerIndex parameters for the following functions.
	 */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetNumPlayerControllers(const UObject* WorldContextObject);

	/**
	 * Returns the number of fully initialized local players, this will be 0 on dedicated servers.
	 * Indexes up to this can be used as PlayerIndex parameters for the following functions, and you are guaranteed to get a local player controller.
	 */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetNumLocalPlayerControllers(const UObject* WorldContextObject);

	/** 
	 * Returns the player controller found while iterating through the local and available remote player controllers.
	 * On a network client, this will only include local players as remote player controllers are not available.
	 * The index will be consistent as long as no new players join or leave, but it will not be the same across different clients and servers.
	 *
	 * @param PlayerIndex	Index in the player controller list, starting first with local players and then available remote ones
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API class APlayerController* GetPlayerController(const UObject* WorldContextObject, int32 PlayerIndex);

	/**
	 * Returns the pawn for the player controller at the specified player index.
	 * This will not include pawns of remote clients with no available player controller, you can use the player states list for that.
	 *
	 * @param PlayerIndex	Index in the player controller list, starting first with local players and then available remote ones
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API class APawn* GetPlayerPawn(const UObject* WorldContextObject, int32 PlayerIndex);

	/**
	 * Returns the pawn for the player controller at the specified player index, will return null if the pawn is not a character.
	 * This will not include characters of remote clients with no available player controller, you can iterate the PlayerStates list for that.
	 *
	 * @param PlayerIndex	Index in the player controller list, starting first with local players and then available remote ones
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API class ACharacter* GetPlayerCharacter(const UObject* WorldContextObject, int32 PlayerIndex);

	/**
	 * Returns the camera manager for the Player Controller at the specified player index.
	 * This will not include remote clients with no player controller.
	 *
	 * @param PlayerIndex	Index in the player controller list, starting first with local players and then available remote ones
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API class APlayerCameraManager* GetPlayerCameraManager(const UObject* WorldContextObject, int32 PlayerIndex);

	/** 
	 * Determines if any local player controller's camera is within range of the specified location.
	 * @param Location		The location from which test range
	 * @param MaximumRange	The distance away from Location to test range
	 * @note This will always return false on dedicated servers.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Game", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API bool IsAnyLocalPlayerCameraWithinRange(const UObject* WorldContextObject, const FVector& Location, float MaximumRange);

	/** 
	 * Returns the player controller with the specified physical controller ID. This only works for local players.
	 *
	 * @param ControllerID	Physical controller ID, the same value returned from Get Player Controller ID
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(DisplayName="Get Local Player Controller From ID", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API class APlayerController* GetPlayerControllerFromID(const UObject* WorldContextObject, int32 ControllerID);

	/** 
	 * Returns the player controller with the specified physical controller ID. This only works for local players.
	 *
	 * @param ControllerID	Physical controller ID, the same value returned from Get Player Controller ID
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(DisplayName="Get Local Player Controller From Platform User", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API APlayerController* GetPlayerControllerFromPlatformUser(const UObject* WorldContextObject, FPlatformUserId UserId);

	/** 
	 * Create a new local player for this game, for cases like local multiplayer.
	 *
	 * @param ControllerId             The ID of the physical controller that the should control the newly created player. A value of -1 specifies to use the next available ID
	 * @param bSpawnPlayerController   Whether a player controller should be spawned immediately for this player. If false a player controller will not be created automatically until transition to the next map.
	 * @return                         The created player controller if one is created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Game", meta = (DisplayName="Create Local Player", WorldContext = "WorldContextObject", AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API class APlayerController* CreatePlayer(const UObject* WorldContextObject, int32 ControllerId = -1, bool bSpawnPlayerController = true);
	
	/** 
	 * Create a new local player for this game, for cases like local multiplayer.
	 *
	 * @param UserId             	   The platform user id that should control the newly created player. A valude of PLATFRMUSERID_NONE specifies to use the next available ID	
	 * @param bSpawnPlayerController   Whether a player controller should be spawned immediately for this player. If false a player controller will not be created automatically until transition to the next map.
	 * @return                         The created player controller if one is created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Game", meta = (AutoCreateRefTerm="UserId", DisplayName="Create Local Player For Platform User", WorldContext = "WorldContextObject", AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API class APlayerController* CreatePlayerFromPlatformUser(const UObject* WorldContextObject, FPlatformUserId UserId, bool bSpawnPlayerController = true);

	/** 
	 * Removes a local player from this game.
	 *
	 * @param Player			The player controller of the player to be removed
	 * @param bDestroyPawn		Whether the controlled pawn should be deleted as well
	 */
	UFUNCTION(BlueprintCallable, Category = "Game", meta = (DisplayName="Remove Local Player", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void RemovePlayer(APlayerController* Player, bool bDestroyPawn);

	/**
	 * Gets what physical controller ID a player is using. This only works for local player controllers.
	 *
	 * @param Player	The player controller of the player to get the ID of
	 * @return			The ID of the passed in player. -1 if there is no physical controller assigned to the passed in player
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(DisplayName="Get Local Player Controller ID", UnsafeDuringActorConstruction="true"))
	static ENGINE_API int32 GetPlayerControllerID(APlayerController* Player);

	/**
	 * Sets what physical controller ID a player should be using. This only works for local player controllers.
	 *
	 * @param Player			The player controller of the player to change the controller ID of
	 * @param ControllerId		The controller ID to assign to this player
	 */
	UFUNCTION(BlueprintCallable, Category="Game", meta=(DisplayName="Set Local Player Controller ID",UnsafeDuringActorConstruction="true"))
	static ENGINE_API void SetPlayerControllerID(APlayerController* Player, int32 ControllerId);
	
	/**
	 * Sets what platform user id a player should be using. This only works for local player controllers.
	 *
	 * @param Player			The player controller of the player to change the platform user of
	 * @param PlatformUserId	The platform user id to assign this player
	 */
	UFUNCTION(BlueprintCallable, Category="Game", meta=(DisplayName="Set Local Player Controller Platform User Id",UnsafeDuringActorConstruction="true"))
	static ENGINE_API void SetPlayerPlatformUserId(APlayerController* PlayerController, FPlatformUserId UserId);

	// --- Level Streaming functions ------------------------
	
	/** Stream the level (by Name); Calling again before it finishes has no effect */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", Latent = "", LatentInfo = "LatentInfo", DisplayName = "Load Stream Level (by Name)"), Category="Game")
	static ENGINE_API void LoadStreamLevel(const UObject* WorldContextObject, FName LevelName, bool bMakeVisibleAfterLoad, bool bShouldBlockOnLoad, FLatentActionInfo LatentInfo);

	/** Stream the level (by Object Reference); Calling again before it finishes has no effect */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", Latent = "", LatentInfo = "LatentInfo", DisplayName = "Load Stream Level (by Object Reference)"), Category="Game")
	static ENGINE_API void LoadStreamLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, bool bMakeVisibleAfterLoad, bool bShouldBlockOnLoad, FLatentActionInfo LatentInfo);
	
	/** Unload a streamed in level (by Name)*/
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", Latent = "", LatentInfo = "LatentInfo", DisplayName = "Unload Stream Level (by Name)"), Category="Game")
	static ENGINE_API void UnloadStreamLevel(const UObject* WorldContextObject, FName LevelName, FLatentActionInfo LatentInfo, bool bShouldBlockOnUnload);

	/** Unload a streamed in level (by Object Reference)*/
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", Latent = "", LatentInfo = "LatentInfo", DisplayName = "Unload Stream Level (by Object Reference)"), Category="Game")
	static ENGINE_API void UnloadStreamLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, FLatentActionInfo LatentInfo, bool bShouldBlockOnUnload);
	
	/** Returns level streaming object with specified level package name */
	UFUNCTION(BlueprintPure, meta=(WorldContext="WorldContextObject"), Category="Game")
	static ENGINE_API class ULevelStreaming* GetStreamingLevel(const UObject* WorldContextObject, FName PackageName);

	/** Flushes level streaming in blocking fashion and returns when all sub-levels are loaded / visible / hidden */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject"), Category = "Game")
	static ENGINE_API void FlushLevelStreaming(const UObject* WorldContextObject);
		
	/** Does nothing */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage = "Does nothing, do not use."), Category = "Game")
	static ENGINE_API void CancelAsyncLoading();

	/**
	 * Travel to another level
	 *
	 * @param	LevelName			the level to open
	 * @param	bAbsolute			if true options are reset, if false options are carried over from current level
	 * @param	Options				a string of options to use for the travel URL
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", AdvancedDisplay = "2", DisplayName = "Open Level (by Name)"), Category="Game")
	static ENGINE_API void OpenLevel(const UObject* WorldContextObject, FName LevelName, bool bAbsolute = true, FString Options = FString(TEXT("")));

	/**
	 * Travel to another level
	 *
	 * @param	Level				the level to open
	 * @param	bAbsolute			if true options are reset, if false options are carried over from current level
	 * @param	Options				a string of options to use for the travel URL
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", AdvancedDisplay = "2", DisplayName = "Open Level (by Object Reference)"), Category="Game")
	static ENGINE_API void OpenLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, bool bAbsolute = true, FString Options = FString(TEXT("")));
	
	/**
	* Get the name of the currently-open level.
	*
	* @param bRemovePrefixString	remove any streaming- or editor- added prefixes from the level name.
	*/
	UFUNCTION(BlueprintCallable, meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "1"), Category = "Game")
	static ENGINE_API FString GetCurrentLevelName(const UObject* WorldContextObject, bool bRemovePrefixString = true);

	// --- Global functions ------------------------------

	/** Returns the current GameModeBase or Null if it can't be retrieved, such as on the client */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API class AGameModeBase* GetGameMode(const UObject* WorldContextObject);

	/** Returns the current GameStateBase or Null if it can't be retrieved */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API class AGameStateBase* GetGameState(const UObject* WorldContextObject);

	/** Returns the class of a passed in Object, will always be valid if Object is not NULL */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Class", DeterminesOutputType = "Object"), Category="Utilities")
	static ENGINE_API class UClass *GetObjectClass(const UObject *Object);

	/**
	 * Returns whether or not the object passed in is of (or inherits from) the class type.
	 * @return True if the object is of (or inherits from) the class type.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IsA"), Category = "Utilities")
	static ENGINE_API bool ObjectIsA(const UObject* Object, TSubclassOf<UObject> ObjectClass);

	/**
	 * Gets the current global time dilation.
	 * @return Current time dilation.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API float GetGlobalTimeDilation(const UObject* WorldContextObject);

	/**
	 * Sets the global time dilation.
	 * @param	TimeDilation	value to set the global time dilation to
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Time", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API void SetGlobalTimeDilation(const UObject* WorldContextObject, float TimeDilation);

	/**
	 * Sets the game's paused state
	 * @param	bPaused		Whether the game should be paused or not
	 * @return	Whether the game was successfully paused/unpaused
	 */
	UFUNCTION(BlueprintCallable, Category="Game", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API bool SetGamePaused(const UObject* WorldContextObject, bool bPaused);

	/**
	 * Returns the game's paused state
	 * @return	Whether the game is currently paused or not
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API bool IsGamePaused(const UObject* WorldContextObject);

	/**
	 * Enables split screen
	 * @param	bDisable		Whether the viewport should split screen between local players or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Viewport", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetForceDisableSplitscreen(const UObject* WorldContextObject, bool bDisable);

	/**
	 * Returns the split screen state
	 * @return	Whether the game viewport is split screen or not
	 */
	UFUNCTION(BlueprintPure, Category = "Viewport", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool IsSplitscreenForceDisabled(const UObject* WorldContextObject);

	/**
	 * Enabled rendering of the world
	 * @param	bEnable		Whether the world should be rendered or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetEnableWorldRendering(const UObject* WorldContextObject, bool bEnable);

	/**
	 * Returns the world rendering state
	 * @return	Whether the world should be rendered or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool GetEnableWorldRendering(const UObject* WorldContextObject);

	/**
	 * Returns the current viewport mouse capture mode
	 */
	UFUNCTION(BlueprintPure, Category = "Viewport", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API EMouseCaptureMode GetViewportMouseCaptureMode(const UObject* WorldContextObject);

	/**
	 * Sets the current viewport mouse capture mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Viewport", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetViewportMouseCaptureMode(const UObject* WorldContextObject, const EMouseCaptureMode MouseCaptureMode);

	/** Hurt locally authoritative actors within the radius. Will only hit components that block the Visibility channel.
	 * @param BaseDamage - The base damage to apply, i.e. the damage at the origin.
	 * @param Origin - Epicenter of the damage area.
	 * @param DamageRadius - Radius of the damage area, from Origin
	 * @param DamageTypeClass - Class that describes the damage that was done.
	 * @param IgnoreActors - List of Actors to ignore
	 * @param DamageCauser - Actor that actually caused the damage (e.g. the grenade that exploded).  This actor will not be damaged and it will not block damage.
	 * @param InstigatedByController - Controller that was responsible for causing this damage (e.g. player who threw the grenade)
	 * @param bFullDamage - if true, damage not scaled based on distance from Origin
	 * @param DamagePreventionChannel - Damage will not be applied to victim if there is something between the origin and the victim which blocks traces on this channel
	 * @return true if damage was applied to at least one actor.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Game|Damage", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="IgnoreActors"))
	static ENGINE_API bool ApplyRadialDamage(const UObject* WorldContextObject, float BaseDamage, const FVector& Origin, float DamageRadius, TSubclassOf<class UDamageType> DamageTypeClass, const TArray<AActor*>& IgnoreActors, AActor* DamageCauser = NULL, AController* InstigatedByController = NULL, bool bDoFullDamage = false, ECollisionChannel DamagePreventionChannel = ECC_Visibility);
	
	/** Hurt locally authoritative actors within the radius. Will only hit components that block the Visibility channel.
	 * @param BaseDamage - The base damage to apply, i.e. the damage at the origin.
	 * @param Origin - Epicenter of the damage area.
	 * @param DamageInnerRadius - Radius of the full damage area, from Origin
	 * @param DamageOuterRadius - Radius of the minimum damage area, from Origin
	 * @param DamageFalloff - Falloff exponent of damage from DamageInnerRadius to DamageOuterRadius
	 * @param DamageTypeClass - Class that describes the damage that was done.
	 * @param IgnoreActors - List of Actors to ignore
	 * @param DamageCauser - Actor that actually caused the damage (e.g. the grenade that exploded)
	 * @param InstigatedByController - Controller that was responsible for causing this damage (e.g. player who threw the grenade)
	 * @param bFullDamage - if true, damage not scaled based on distance from Origin
	 * @param DamagePreventionChannel - Damage will not be applied to victim if there is something between the origin and the victim which blocks traces on this channel
	 * @return true if damage was applied to at least one actor.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Game|Damage", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="IgnoreActors"))
	static ENGINE_API bool ApplyRadialDamageWithFalloff(const UObject* WorldContextObject, float BaseDamage, float MinimumDamage, const FVector& Origin, float DamageInnerRadius, float DamageOuterRadius, float DamageFalloff, TSubclassOf<class UDamageType> DamageTypeClass, const TArray<AActor*>& IgnoreActors, AActor* DamageCauser = NULL, AController* InstigatedByController = NULL, ECollisionChannel DamagePreventionChannel = ECC_Visibility);
	

	/** Hurts the specified actor with the specified impact.
	 * @param DamagedActor - Actor that will be damaged.
	 * @param BaseDamage - The base damage to apply.
	 * @param HitFromDirection - Direction the hit came FROM
	 * @param HitInfo - Collision or trace result that describes the hit
	 * @param EventInstigator - Controller that was responsible for causing this damage (e.g. player who shot the weapon)
	 * @param DamageCauser - Actor that actually caused the damage (e.g. the grenade that exploded)
	 * @param DamageTypeClass - Class that describes the damage that was done.
	 * @return Actual damage the ended up being applied to the actor.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Game|Damage")
	static ENGINE_API float ApplyPointDamage(AActor* DamagedActor, float BaseDamage, const FVector& HitFromDirection, const FHitResult& HitInfo, AController* EventInstigator, AActor* DamageCauser, TSubclassOf<class UDamageType> DamageTypeClass);

	/** Hurts the specified actor with generic damage.
	 * @param DamagedActor - Actor that will be damaged.
	 * @param BaseDamage - The base damage to apply.
	 * @param EventInstigator - Controller that was responsible for causing this damage (e.g. player who shot the weapon)
	 * @param DamageCauser - Actor that actually caused the damage (e.g. the grenade that exploded)
	 * @param DamageTypeClass - Class that describes the damage that was done.
	 * @return Actual damage the ended up being applied to the actor.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Game|Damage")
	static ENGINE_API float ApplyDamage(AActor* DamagedActor, float BaseDamage, AController* EventInstigator, AActor* DamageCauser, TSubclassOf<class UDamageType> DamageTypeClass);

	// --- Camera functions ------------------------------

	/** Plays an in-world camera shake that affects all nearby local players, with distance-based attenuation. Does not replicate.
	 * @param WorldContextObject - Object that we can obtain a world context from
	 * @param Shake - Camera shake asset to use
	 * @param Epicenter - location to place the effect in world space
	 * @param InnerRadius - Cameras inside this radius are ignored
	 * @param OuterRadius - Cameras outside of InnerRadius and inside this are effected
	 * @param Falloff - Affects falloff of effect as it nears OuterRadius
	 * @param bOrientShakeTowardsEpicenter - Changes the rotation of shake to point towards epicenter instead of forward
	 */
	UFUNCTION(BlueprintCallable, Category="Camera", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void PlayWorldCameraShake(const UObject* WorldContextObject, TSubclassOf<class UCameraShakeBase> Shake, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff = 1.f, bool bOrientShakeTowardsEpicenter = false);

	// --- Particle functions ------------------------------

	/** Plays the specified effect at the given location and rotation, fire and forget. The system will go away when the effect is complete. Does not replicate.
	 * @param WorldContextObject - Object that we can obtain a world context from
	 * @param EmitterTemplate - particle system to create
	 * @param Location - location to place the effect in world space
	 * @param Rotation - rotation to place the effect in world space	
	 * @param Scale - scale to create the effect at
	 * @param bAutoDestroy - Whether the component will automatically be destroyed when the particle system completes playing or whether it can be reactivated
	 * @param PoolingMethod - Method used for pooling this component. Defaults to none.
	 * @param bAutoActivate - Whether the component will be automatically activated on creation.
	 */
	UFUNCTION(BlueprintCallable, Category="Effects", meta=(Keywords = "particle system", WorldContext="WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UParticleSystemComponent* SpawnEmitterAtLocation(const UObject* WorldContextObject, UParticleSystem* EmitterTemplate, FVector Location, FRotator Rotation = FRotator::ZeroRotator, FVector Scale = FVector(1.f), bool bAutoDestroy = true, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivateSystem = true);

	// Backwards compatible version of SpawnEmitterAttached for C++ without Scale
	static ENGINE_API UParticleSystemComponent* SpawnEmitterAtLocation(const UObject* WorldContextObject, UParticleSystem* EmitterTemplate, FVector Location, FRotator Rotation, bool bAutoDestroy, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivateSystem =true);

	/** Plays the specified effect at the given location and rotation, fire and forget. The system will go away when the effect is complete. Does not replicate.
	 * @param World - The World to spawn in
	 * @param EmitterTemplate - particle system to create
	 * @param SpawnTransform - transform with which to place the effect in world space
	 * @param bAutoDestroy - Whether the component will automatically be destroyed when the particle system completes playing or whether it can be reactivated
	 * @param PoolingMethod - Method used for pooling this component. Defaults to none.
	 * @param bAutoActivate - Whether the component will be automatically activated on creation.
	 */
	static ENGINE_API UParticleSystemComponent* SpawnEmitterAtLocation(UWorld* World, UParticleSystem* EmitterTemplate, const FTransform& SpawnTransform, bool bAutoDestroy = true, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivate = true);

private:
	static ENGINE_API UParticleSystemComponent* InternalSpawnEmitterAtLocation(UWorld* World, UParticleSystem* EmitterTemplate, FVector Location, FRotator Rotation, FVector Scale, bool bAutoDestroy, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivate = true);

public:

	/* Struct of parameters passed to SuggestProjectileVelocity function. */
	struct FSuggestProjectileVelocityParameters
	{
	public:
		const UObject* WorldContextObject = nullptr;
		FVector Start = FVector::ZeroVector;
		FVector End = FVector::ZeroVector;
		float TossSpeed = 0.f;
		bool bFavorHighArc = false;
		float CollisionRadius = 0.f;
		float OverrideGravityZ = 0;
		ESuggestProjVelocityTraceOption::Type TraceOption = ESuggestProjVelocityTraceOption::TraceFullPath;
		FCollisionResponseParams ResponseParam = FCollisionResponseParams::DefaultResponseParam;
		TArray<AActor*> ActorsToIgnore = TArray<AActor*>();
		bool bDrawDebug = false;
		bool bAcceptClosestOnNoSolutions = false;

		FSuggestProjectileVelocityParameters(const UObject* World, FVector StartLocation, FVector EndLocation, float Speed)
			: WorldContextObject{ World }, Start{ StartLocation }, End{ EndLocation }, TossSpeed{ Speed } {}
	};

	/** Plays the specified effect attached to and following the specified component. The system will go away when the effect is complete. Does not replicate.
	* @param EmitterTemplate - particle system to create
	 * @param AttachComponent - Component to attach to.
	 * @param AttachPointName - Optional named point within the AttachComponent to spawn the emitter at
	 * @param Location - Depending on the value of LocationType this is either a relative offset from the attach component/point or an absolute world location that will be translated to a relative offset (if LocationType is KeepWorldPosition).
	 * @param Rotation - Depending on the value of LocationType this is either a relative offset from the attach component/point or an absolute world rotation that will be translated to a relative offset (if LocationType is KeepWorldPosition).
	 * @param Scale - Depending on the value of LocationType this is either a relative scale from the attach component or an absolute world scale that will be translated to a relative scale (if LocationType is KeepWorldPosition).
	 * @param LocationType - Specifies whether Location is a relative offset or an absolute world position
	 * @param bAutoDestroy - Whether the component will automatically be destroyed when the particle system completes playing or whether it can be reactivated
	 * @param PoolingMethod - Method used for pooling this component. Defaults to none.
	 * @param bAutoActivate - Whether the component will be automatically activated on creation.
	 */
	UFUNCTION(BlueprintCallable, Category="Effects", meta=(Keywords = "particle system", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UParticleSystemComponent* SpawnEmitterAttached(class UParticleSystem* EmitterTemplate, class USceneComponent* AttachToComponent, FName AttachPointName = NAME_None, FVector Location = FVector(ForceInit), FRotator Rotation = FRotator::ZeroRotator, FVector Scale = FVector(1.f), EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bAutoDestroy = true, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivate=true);

	// Backwards compatible version of SpawnEmitterAttached for C++ without Scale
	static ENGINE_API UParticleSystemComponent* SpawnEmitterAttached(class UParticleSystem* EmitterTemplate, class USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bAutoDestroy = true, EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None, bool bAutoActivate=true);

	// --- Sound functions ------------------------------
	
	/**
	 * Determines if any audio listeners are within range of the specified location
	 * @param Location		The location from which test if a listener is in range
	 * @param MaximumRange	The distance away from Location to test if any listener is within
	 * @note This will always return false if there is no audio device, or the audio device is disabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool AreAnyListenersWithinRange(const UObject* WorldContextObject, const FVector& Location, float MaximumRange);
	

	/**
	 * Finds and returns the position of the closest listener to the specified location
	 * @param Location						The location from which we'd like to find the closest listener, in world space.
	 * @param MaximumRange					The maximum distance away from Location that a listener can be.
	 * @param bAllowAttenuationOverride		True for the adjusted listener position (if attenuation override is set), false for the raw listener position (for panning)
	 * @param ListenerPosition				[Out] The position of the closest listener in world space, if found.
	 * @return true if we've successfully found a listener within MaximumRange of Location, otherwise false.
 	 * @note This will always return false if there is no audio device, or the audio device is disabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool GetClosestListenerLocation(const UObject* WorldContextObject, const FVector& Location, float MaximumRange, const bool bAllowAttenuationOverride, FVector& ListenerPosition);

	/**
	* Sets a global pitch modulation scalar that will apply to all non-UI sounds
	*
	* * Fire and Forget.
	* * Not Replicated.
	* @param PitchModulation - A pitch modulation value to globally set.
	* @param TimeSec - A time value to linearly interpolate the global modulation pitch over from it's current value.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetGlobalPitchModulation(const UObject* WorldContextObject, float PitchModulation, float TimeSec);

	/** 
	* Linearly interpolates the attenuation distance scale value from it's current attenuation distance override value 
	* (1.0f it not overridden) to its new attenuation distance override, over the given amount of time
	*
	* * Fire and Forget.
	* * Not Replicated.
	* @param SoundClass - Sound class to to use to set the attenuation distance scale on.
	* @param DistanceAttenuationScale - A scalar for the attenuation distance used for computing distance attenuation. 
	* @param TimeSec - A time value to linearly interpolate from the current distance attenuation scale value to the new value.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetSoundClassDistanceScale(const UObject* WorldContextObject, USoundClass* SoundClass, float DistanceAttenuationScale, float TimeSec = 0.0f);


	/**
	* Sets the global listener focus parameters, which will scale focus behavior of sounds based on their focus azimuth
	* settings in their attenuation settings.
	*
	* * Fire and Forget.
	* * Not Replicated.
	* @param FocusAzimuthScale - An angle scale value used to scale the azimuth angle that defines where sounds are in-focus.
	* @param NonFocusAzimuthScale- An angle scale value used to scale the azimuth angle that defines where sounds are out-of-focus.
	* @param FocusDistanceScale - A distance scale value to use for sounds which are in-focus. Values < 1.0 will reduce perceived 
	*							  distance to sounds, values > 1.0 will increase perceived distance to in-focus sounds.
	* @param NonFocusDistanceScale - A distance scale value to use for sounds which are out-of-focus. Values < 1.0 will reduce 
	*								 perceived distance to sounds, values > 1.0 will increase perceived distance to in-focus sounds.
	* @param FocusVolumeScale- A volume attenuation value to use for sounds which are in-focus.
	* @param NonFocusVolumeScale- A volume attenuation value to use for sounds which are out-of-focus.
	* @param FocusPriorityScale - A priority scale value (> 0.0) to use for sounds which are in-focus. Values < 1.0 will reduce
	*							  the priority of in-focus sounds, values > 1.0 will increase the priority of in-focus sounds.
	* @param NonFocusPriorityScale - A priority scale value (> 0.0) to use for sounds which are out-of-focus. Values < 1.0 will
	*								 reduce the priority of sounds out-of-focus sounds, values > 1.0 will increase the priority of
	*								 out-of-focus sounds.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetGlobalListenerFocusParameters(const UObject* WorldContextObject, float FocusAzimuthScale = 1.0f, float NonFocusAzimuthScale = 1.0f, float FocusDistanceScale = 1.0f, float NonFocusDistanceScale = 1.0f, float FocusVolumeScale = 1.0f, float NonFocusVolumeScale = 1.0f, float FocusPriorityScale = 1.0f, float NonFocusPriorityScale = 1.0f);

	/**
	 * Plays a sound directly with no attenuation, perfect for UI sounds.
	 *
	 * * Fire and Forget.
	 * * Not Replicated.
	 * @param Sound - Sound to play.
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the sound to begin playback at
 	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param OwningActor - The actor to use as the "owner" for concurrency settings purposes. 
	 *						Allows PlaySound calls to do a concurrency limit per owner.
	 * @param bIsUISound - True if sound is UI related, else false
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Audio", meta=( WorldContext="WorldContextObject", AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true" ))
	static ENGINE_API void PlaySound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundConcurrency* ConcurrencySettings = nullptr, const AActor* OwningActor = nullptr, bool bIsUISound = true);

	/**
	 * This function allows users to create Audio Components with settings specifically for non-spatialized,
	 * non-distance-attenuated sounds. Audio Components created using this function by default will not have 
	 * Spatialization applied. Sound instances will begin playing upon spawning this Audio Component.
	 *
	 * * Not Replicated.
	 * @param Sound - Sound to play.
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the sound to begin playback at
	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param PersistAcrossLevelTransition - Whether the sound should continue to play when the map it was played in is unloaded
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes 
	 *						 (by completing or stopping) or whether it can be reactivated
	 * @return An audio component to manipulate the spawned sound
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Audio", meta=( WorldContext="WorldContextObject", AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true", Keywords = "play" ))
	static ENGINE_API UAudioComponent* SpawnSound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundConcurrency* ConcurrencySettings = nullptr, bool bPersistAcrossLevelTransition = false, bool bAutoDestroy = true);

	/**
	 * This function allows users to create Audio Components in advance of playback with settings specifically for non-spatialized,
	 * non-distance-attenuated sounds. Audio Components created using this function by default will not have Spatialization applied.
	 * @param Sound - Sound to create.
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far into the sound to begin playback at
	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param PersistAcrossLevelTransition - Whether the sound should continue to play when the map it was played in is unloaded
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes 
	 *						 (by completing or stopping), or whether it can be reactivated
	 * @return An audio component to manipulate the created sound
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Audio", meta=( WorldContext="WorldContextObject", AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true", Keywords = "play" ))
	static ENGINE_API UAudioComponent* CreateSound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundConcurrency* ConcurrencySettings = nullptr, bool bPersistAcrossLevelTransition = false, bool bAutoDestroy = true);

	/**
	 * Plays a sound at the given location. This is a fire and forget sound and does not travel with any actor. 
	 * Replication is also not handled at this point.
	 * @param Sound - sound to play
	 * @param Location - World position to play sound at
	 * @param Rotation - World rotation to play sound at
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the sound to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param OwningActor - The actor to use as the "owner" for concurrency settings purposes. Allows PlaySound calls
	 *						to do a concurrency limit per owner.
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext="WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API void PlaySoundAtLocation(const UObject* WorldContextObject, USoundBase* Sound, FVector Location, FRotator Rotation, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, class USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, const AActor* OwningActor = nullptr, const UInitialActiveSoundParams* InitialParams = nullptr);

	static void PlaySoundAtLocation(const UObject* WorldContextObject, USoundBase* Sound, FVector Location, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, class USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, const UInitialActiveSoundParams* InitialParams = nullptr)
	{
		PlaySoundAtLocation(WorldContextObject, Sound, Location, FRotator::ZeroRotator, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, ConcurrencySettings, nullptr, InitialParams);
	}

	/**
	 * Spawns a sound at the given location. This does not travel with any actor. Replication is also not handled at this point.
	 * @param Sound - sound to play
	 * @param Location - World position to play sound at
	 * @param Rotation - World rotation to play sound at
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the sound to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes 
	 *						 (by completing or stopping) or whether it can be reactivated
	 * @return An audio component to manipulate the spawned sound
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext="WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UAudioComponent* SpawnSoundAtLocation(const UObject* WorldContextObject, USoundBase* Sound, FVector Location, FRotator Rotation = FRotator::ZeroRotator, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, class USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, bool bAutoDestroy = true);

	/** This function allows users to create and play Audio Components attached to a specific Scene Component. 
	 *  Useful for spatialized and/or distance-attenuated sounds that need to follow another object in space.
	 * @param Sound - sound to play
	 * @param AttachComponent - Component to attach to.
	 * @param AttachPointName - Optional named point within the AttachComponent to play the sound at
	 * @param Location - Depending on the value of Location Type this is either a relative offset from 
	 *					 the attach component/point or an absolute world position that will be translated to a relative offset
	 * @param Rotation - Depending on the value of Location Type this is either a relative offset from
	 *					 the attach component/point or an absolute world rotation that will be translated to a relative offset
	 * @param LocationType - Specifies whether Location is a relative offset or an absolute world position
	 * @param bStopWhenAttachedToDestroyed - Specifies whether the sound should stop playing when the
	 *										 owner of the attach to component is destroyed.
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the sound to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 * @param ConcurrencySettings - Override concurrency settings package to play sound with
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes
	 *						 (by completing or stopping) or whether it can be reactivated
	 * @return An audio component to manipulate the spawned sound
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UAudioComponent* SpawnSoundAttached(USoundBase* Sound, USceneComponent* AttachToComponent, FName AttachPointName = NAME_None, FVector Location = FVector(ForceInit), FRotator Rotation = FRotator::ZeroRotator, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bStopWhenAttachedToDestroyed = false, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, bool bAutoDestroy = true);

	static class UAudioComponent* SpawnSoundAttached(USoundBase* Sound, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bStopWhenAttachedToDestroyed = false, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, bool bAutoDestroy = true)
	{
		return SpawnSoundAttached(Sound, AttachToComponent, AttachPointName, Location, FRotator::ZeroRotator, LocationType, bStopWhenAttachedToDestroyed, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, ConcurrencySettings, bAutoDestroy);
	}

	/**
	 * Plays a dialogue directly with no attenuation, perfect for UI.
	 *
	 * * Fire and Forget.
	 * * Not Replicated.
	 * @param Dialogue - dialogue to play
	 * @param Context - context the dialogue is to play in
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the dialogue to begin playback at
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Audio", meta=( WorldContext="WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true" ))
	static ENGINE_API void PlayDialogue2D(const UObject* WorldContextObject, UDialogueWave* Dialogue, const struct FDialogueContext& Context, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f);

	/**
	 * Spawns a DialogueWave, a special type of Asset that requires Context data in order to resolve a specific SoundBase,
	 * which is then passed on to the new Audio Component. Audio Components created using this function by default will not
	 * have Spatialization applied. Sound instances will begin playing upon spawning this Audio Component.
	 *
	 * * Not Replicated.
	 * @param Dialogue - dialogue to play
	 * @param Context - context the dialogue is to play in
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the dialogue to begin playback at
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound
	 *						 finishes (by completing or stopping) or whether it can be reactivated
	 * @return An audio component to manipulate the spawned sound
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Audio", meta=( WorldContext="WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true", Keywords = "play" ))
	static ENGINE_API UAudioComponent* SpawnDialogue2D(const UObject* WorldContextObject, UDialogueWave* Dialogue, const struct FDialogueContext& Context, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, bool bAutoDestroy = true);

	/** Plays a dialogue at the given location. This is a fire and forget sound and does not travel with any actor. 
	 *	Replication is also not handled at this point.
	 * @param Dialogue - dialogue to play
	 * @param Context - context the dialogue is to play in
	 * @param Location - World position to play dialogue at
	 * @param Rotation - World rotation to play dialogue at
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the dialogue to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext="WorldContextObject", AdvancedDisplay = "4", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void PlayDialogueAtLocation(const UObject* WorldContextObject, class UDialogueWave* Dialogue, const FDialogueContext& Context, FVector Location, FRotator Rotation, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr);

	static void PlayDialogueAtLocation(const UObject* WorldContextObject, UDialogueWave* Dialogue, const FDialogueContext& Context, FVector Location, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr)
	{
		PlayDialogueAtLocation(WorldContextObject, Dialogue, Context, Location, FRotator::ZeroRotator, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings);
	}

	/** Spawns a DialogueWave, a special type of Asset that requires Context data in order to resolve a specific SoundBase,
	 *  which is then passed on to the new Audio Component. This function allows users to create and play Audio Components at a
	 *  specific World Location and Rotation. Useful for spatialized and/or distance-attenuated dialogue.
	 * @param Dialogue - Dialogue to play
	 * @param Context - Context the dialogue is to play in
	 * @param Location - World position to play dialogue at
	 * @param Rotation - World rotation to play dialogue at
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far into the dialogue to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes
	 *						 (by completing or stopping) or whether it can be reactivated
	 * @return Audio Component to manipulate the playing dialogue with
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext="WorldContextObject", AdvancedDisplay = "4", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UAudioComponent* SpawnDialogueAtLocation(const UObject* WorldContextObject, UDialogueWave* Dialogue, const struct FDialogueContext& Context, FVector Location, FRotator Rotation = FRotator::ZeroRotator, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr, bool bAutoDestroy = true);

	/** Spawns a DialogueWave, a special type of Asset that requires Context data in order to resolve a specific SoundBase,
	 *	which is then passed on to the new Audio Component. This function allows users to create and play Audio Components
	 *	attached to a specific Scene Component. Useful for spatialized and/or distance-attenuated dialogue that needs to 
	 *	follow another object in space.
	 * @param Dialogue - dialogue to play
	 * @param Context - context the dialogue is to play in
	 * @param AttachComponent - Component to attach to.
	 * @param AttachPointName - Optional named point within the AttachComponent to play the sound at
	 * @param Location - Depending on the value of Location Type this is either a relative offset from the 
	 *					 attach component/point or an absolute world position that will be translated to a relative offset
	 * @param Rotation - Depending on the value of Location Type this is either a relative offset from the 
	 *					 attach component/point or an absolute world rotation that will be translated to a relative offset
	 * @param LocationType - Specifies whether Location is a relative offset or an absolute world position
	 * @param bStopWhenAttachedToDestroyed - Specifies whether the sound should stop playing when the owner its attached
	 *										 to is destroyed.
	 * @param VolumeMultiplier - A linear scalar multiplied with the volume, in order to make the sound louder or softer.
	 * @param PitchMultiplier - A linear scalar multiplied with the pitch.
	 * @param StartTime - How far in to the dialogue to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play sound with
	 * @param bAutoDestroy - Whether the returned audio component will be automatically cleaned up when the sound finishes 
	 *						 (by completing or stopping) or whether it can be reactivated
	 * @return Audio Component to manipulate the playing dialogue with
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UAudioComponent* SpawnDialogueAttached(UDialogueWave* Dialogue, const FDialogueContext& Context, USceneComponent* AttachToComponent, FName AttachPointName = NAME_None, FVector Location = FVector(ForceInit), FRotator Rotation = FRotator::ZeroRotator, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bStopWhenAttachedToDestroyed = false, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr, bool bAutoDestroy = true);

	static UAudioComponent* SpawnDialogueAttached(UDialogueWave* Dialogue, const FDialogueContext& Context, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bStopWhenAttachedToDestroyed = false, float VolumeMultiplier = 1.f, float PitchMultiplier = 1.f, float StartTime = 0.f, USoundAttenuation* AttenuationSettings = nullptr, bool bAutoDestroy = true)
	{
		return SpawnDialogueAttached(Dialogue, Context, AttachToComponent, AttachPointName, Location, FRotator::ZeroRotator, LocationType, bStopWhenAttachedToDestroyed, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, bAutoDestroy);
	}

	/** Plays a force feedback effect at the given location. This is a fire and forget effect and does not travel with any actor. Replication is also not handled at this point.
	 * @param ForceFeedbackEffect - effect to play
	 * @param Location - World position to center the effect at
	 * @param Rotation - World rotation to center the effect at
	 * @param IntensityMultiplier - Intensity multiplier 
	 * @param StartTime - How far in to the feedback effect to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play effect with
	 * @param bAutoDestroy - Whether the returned force feedback component will be automatically cleaned up when the feedback pattern finishes (by completing or stopping) or whether it can be reactivated
	 * @return Force Feedback Component to manipulate the playing feedback effect with
	 */
	UFUNCTION(BlueprintCallable, Category="ForceFeedback", meta=(WorldContext="WorldContextObject", AdvancedDisplay = "3", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UForceFeedbackComponent* SpawnForceFeedbackAtLocation(const UObject* WorldContextObject, UForceFeedbackEffect* ForceFeedbackEffect, FVector Location, FRotator Rotation = FRotator::ZeroRotator, bool bLooping = false, float IntensityMultiplier = 1.f, float StartTime = 0.f, UForceFeedbackAttenuation* AttenuationSettings = nullptr, bool bAutoDestroy = true);

	/** Plays a force feedback effect attached to and following the specified component. This is a fire and forget effect. Replication is also not handled at this point.
	 * @param ForceFeedbackEffect - effect to play
	 * @param AttachComponent - Component to attach to.
	 * @param AttachPointName - Optional named point within the AttachComponent to attach to
	 * @param Location - Depending on the value of Location Type this is either a relative offset from the attach component/point or an absolute world position that will be translated to a relative offset
	 * @param Rotation - Depending on the value of Location Type this is either a relative offset from the attach component/point or an absolute world rotation that will be translated to a relative offset
	 * @param LocationType - Specifies whether Location is a relative offset or an absolute world position
	 * @param bStopWhenAttachedToDestroyed - Specifies whether the feedback effect should stop playing when the owner of the attach to component is destroyed.
	 * @param IntensityMultiplier - Intensity multiplier 
	 * @param StartTime - How far in to the feedback effect to begin playback at
	 * @param AttenuationSettings - Override attenuation settings package to play effect with
	 * @param bAutoDestroy - Whether the returned force feedback component will be automatically cleaned up when the feedback patern finishes (by completing or stopping) or whether it can be reactivated
	 * @return Force Feedback Component to manipulate the playing feedback effect with
	*/
	UFUNCTION(BlueprintCallable, Category="ForceFeedback", meta=(AdvancedDisplay = "2", UnsafeDuringActorConstruction = "true", Keywords = "play"))
	static ENGINE_API UForceFeedbackComponent* SpawnForceFeedbackAttached(UForceFeedbackEffect* ForceFeedbackEffect, USceneComponent* AttachToComponent, FName AttachPointName = NAME_None, FVector Location = FVector(ForceInit), FRotator Rotation = FRotator::ZeroRotator, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, bool bStopWhenAttachedToDestroyed = false, bool bLooping = false, float IntensityMultiplier = 1.f, float StartTime = 0.f, class UForceFeedbackAttenuation* AttenuationSettings = nullptr, bool bAutoDestroy = true);

	/**
	 * Will set subtitles to be enabled or disabled.
	 * @param bEnabled will enable subtitle drawing if true, disable if false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Subtitles")
	static ENGINE_API void SetSubtitlesEnabled(bool bEnabled);

	/**
	 * Returns whether or not subtitles are currently enabled.
	 * @return true if subtitles are enabled.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Subtitles")
	static ENGINE_API bool AreSubtitlesEnabled();

	// --- Audio Functions ----------------------------
	/** Set the sound mix of the audio system for special EQing */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject"))
	static ENGINE_API void SetBaseSoundMix(const UObject* WorldContextObject, class USoundMix* InSoundMix);

	/** Primes the sound, caching the first chunk of streamed audio. */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	static ENGINE_API void PrimeSound(USoundBase* InSound);

	/** Get list of available Audio Spatialization Plugin names */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API TArray<FName> GetAvailableSpatialPluginNames(const UObject* WorldContextObject);

	/** Get currently active Audio Spatialization Plugin name */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API FName GetActiveSpatialPluginName(const UObject* WorldContextObject);

	/** Get list of available Audio Spatialization Plugins */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool SetActiveSpatialPluginByName(const UObject* WorldContextObject, FName InPluginName);

	/** Primes the sound waves in the given USoundClass, caching the first chunk of streamed audio. */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	static ENGINE_API void PrimeAllSoundsInSoundClass(class USoundClass* InSoundClass);

	/** Iterate through all sound waves and releases handles to retained chunks. (If the chunk is not being played it will be up for eviction) */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	static ENGINE_API void UnRetainAllSoundsInSoundClass(class USoundClass* InSoundClass);

	/** Overrides the sound class adjuster in the given sound mix. If the sound class does not exist in the input sound mix, 
	 *	the sound class adjuster will be added to the list of active sound mix modifiers. 
	 * @param InSoundMixModifier The sound mix to modify.
	 * @param InSoundClass The sound class to override (or add) in the sound mix.
	 * @param Volume The volume scale to set the sound class adjuster to.
	 * @param Pitch The pitch scale to set the sound class adjuster to.
	 * @param FadeInTime The interpolation time to use to go from the current sound class adjuster values to the new values.
	 * @param bApplyToChildren Whether or not to apply this override to the sound class' children or to just the specified sound class.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetSoundMixClassOverride(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier, class USoundClass* InSoundClass, float Volume = 1.0f, float Pitch = 1.0f, float FadeInTime = 1.0f, bool bApplyToChildren = true);

	/** Clears any existing override of the Sound Class Adjuster in the given Sound Mix
	 * @param InSoundMixModifier The sound mix to modify.
	 * @param InSoundClass The sound class in the sound mix to clear overrides from.
	 * @param FadeOutTime The interpolation time to use to go from the current sound class adjuster override values to the non-override values.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void ClearSoundMixClassOverride(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier, class USoundClass* InSoundClass, float FadeOutTime = 1.0f);

	/** Push a sound mix modifier onto the audio system 
	 * @param InSoundMixModifier The Sound Mix Modifier to add to the system
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject"))
	static ENGINE_API void PushSoundMixModifier(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier);

	/** Pop a sound mix modifier from the audio system 
	 *	@param InSoundMixModifier The Sound Mix Modifier to remove from the system
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject"))
	static ENGINE_API void PopSoundMixModifier(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier);

	/** Clear all sound mix modifiers from the audio system */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject"))
	static ENGINE_API void ClearSoundMixModifiers(const UObject* WorldContextObject);

	/** Activates a Reverb Effect without the need for an Audio Volume
	 * @param ReverbEffect Reverb Effect to use
	 * @param TagName Tag to associate with Reverb Effect
	 * @param Priority Priority of the Reverb Effect
	 * @param Volume Volume level of Reverb Effect
	 * @param FadeTime Time before Reverb Effect is fully active
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject", AdvancedDisplay = "2"))
	static ENGINE_API void ActivateReverbEffect(const UObject* WorldContextObject, class UReverbEffect* ReverbEffect, FName TagName, float Priority = 0.f, float Volume = 0.5f, float FadeTime = 2.f);

	/**
	 * Deactivates a Reverb Effect that was applied outside of an Audio Volume
	 *
	 * @param TagName Tag associated with Reverb Effect to remove
	 */
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(WorldContext = "WorldContextObject"))
	static ENGINE_API void DeactivateReverbEffect(const UObject* WorldContextObject, FName TagName);

	/** 
	 * Returns the highest priority reverb settings currently active from any source (Audio Volumes or manual settings). 
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API class UReverbEffect* GetCurrentReverbEffect(const UObject* WorldContextObject);

	/**
	 * Sets the max number of voices (also known as "channels") dynamically by percentage. E.g. if you want to temporarily
	 * reduce voice count by 50%, use 0.50. Later, you can return to the original max voice count by using 1.0.
	 * @param MaxChannelCountScale The percentage of the original voice count to set the max number of voices to
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API void SetMaxAudioChannelsScaled(const UObject* WorldContextObject, float MaxChannelCountScale);

	/**
	 * Retrieves the max voice count currently used by the audio engine.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API int32 GetMaxAudioChannelCount(const UObject* WorldContextObject);


	// --- Decal functions ------------------------------

	/** Spawns a decal at the given location and rotation, fire and forget. Does not replicate.
	 * @param DecalMaterial - decal's material
	 * @param DecalSize - size of decal
	 * @param Location - location to place the decal in world space
	 * @param Rotation - rotation to place the decal in world space	
	 * @param LifeSpan - destroy decal component after time runs out (0 = infinite)
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Rendering|Decal", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UDecalComponent* SpawnDecalAtLocation(const UObject* WorldContextObject, class UMaterialInterface* DecalMaterial, FVector DecalSize, FVector Location, FRotator Rotation = FRotator(-90, 0, 0), float LifeSpan = 0);

	/** Spawns a decal attached to and following the specified component. Does not replicate.
	 * @param DecalMaterial - decal's material
	 * @param DecalSize - size of decal
	 * @param AttachComponent - Component to attach to.
	 * @param AttachPointName - Optional named point within the AttachComponent to spawn the emitter at
	 * @param Location - Depending on the value of Location Type this is either a relative offset from the attach component/point or an absolute world position that will be translated to a relative offset
	 * @param Rotation - Depending on the value of LocationType this is either a relative offset from the attach component/point or an absolute world rotation that will be translated to a realative offset
	 * @param LocationType - Specifies whether Location is a relative offset or an absolute world position
	 * @param LifeSpan - destroy decal component after time runs out (0 = infinite)
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Rendering|Decal", meta=(UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UDecalComponent* SpawnDecalAttached(class UMaterialInterface* DecalMaterial, FVector DecalSize, class USceneComponent* AttachToComponent, FName AttachPointName = NAME_None, FVector Location = FVector(ForceInit), FRotator Rotation = FRotator::ZeroRotator, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset, float LifeSpan = 0);

	/** Extracts data from a HitResult. 
	 * @param Hit			The source HitResult.
	 * @param bBlockingHit	True if there was a blocking hit, false otherwise.
	 * @param bInitialOverlap True if the hit started in an initial overlap. In this case some other values should be interpreted differently. Time will be 0, ImpactPoint will equal Location, and normals will be equal and indicate a depenetration vector.
	 * @param Time			'Time' of impact along trace direction ranging from [0.0 to 1.0) if there is a hit, indicating time between start and end. Equals 1.0 if there is no hit.
	 * @param Distance		The distance from the TraceStart to the Location in world space. This value is 0 if there was an initial overlap (trace started inside another colliding object).
	 * @param Location		Location of the hit in world space. If this was a swept shape test, this is the location where we can place the shape in the world where it will not penetrate.
	 * @param Normal		Normal of the hit in world space, for the object that was swept (e.g. for a sphere trace this points towards the sphere's center). Equal to ImpactNormal for line tests.
	 * @param ImpactPoint	Location of the actual contact point of the trace shape with the surface of the hit object. Equal to Location in the case of an initial overlap.
	 * @param ImpactNormal	Normal of the hit in world space, for the object that was hit by the sweep.
	 * @param PhysMat		Physical material that was hit. Must set bReturnPhysicalMaterial to true in the query params for this to be returned.
	 * @param HitActor		Actor hit by the trace.
	 * @param HitComponent	PrimitiveComponent hit by the trace.
	 * @param HitBoneName	Name of the bone hit (valid only if we hit a skeletal mesh).
	 * @param BoneName		Name of the trace bone hit (valid only if we hit a skeletal mesh).
	 * @param HitItem		Primitive-specific data recording which item in the primitive was hit
	 * @param ElementIndex	If colliding with a primitive with multiple parts, index of the part that was hit.
	 * @param FaceIndex		If colliding with trimesh or landscape, index of face that was hit.
	 */
	UFUNCTION(BlueprintPure, Category = "Collision", meta=(NativeBreakFunc, AdvancedDisplay="3"))
	static ENGINE_API void BreakHitResult(const struct FHitResult& Hit, bool& bBlockingHit, bool& bInitialOverlap, float& Time, float& Distance, FVector& Location, FVector& ImpactPoint, FVector& Normal, FVector& ImpactNormal, class UPhysicalMaterial*& PhysMat, class AActor*& HitActor, class UPrimitiveComponent*& HitComponent, FName& HitBoneName, FName& BoneName, int32& HitItem, int32& ElementIndex, int32& FaceIndex, FVector& TraceStart, FVector& TraceEnd);

	/** 
	 *	Create a HitResult struct
	 * @param Hit			The source HitResult.
	 * @param bBlockingHit	True if there was a blocking hit, false otherwise.
	 * @param bInitialOverlap True if the hit started in an initial overlap. In this case some other values should be interpreted differently. Time will be 0, ImpactPoint will equal Location, and normals will be equal and indicate a depenetration vector.
	 * @param Time			'Time' of impact along trace direction ranging from [0.0 to 1.0) if there is a hit, indicating time between start and end. Equals 1.0 if there is no hit.
	 * @param Distance		The distance from the TraceStart to the Location in world space. This value is 0 if there was an initial overlap (trace started inside another colliding object).
	 * @param Location		Location of the hit in world space. If this was a swept shape test, this is the location where we can place the shape in the world where it will not penetrate.
	 * @param Normal		Normal of the hit in world space, for the object that was swept (e.g. for a sphere trace this points towards the sphere's center). Equal to ImpactNormal for line tests.
	 * @param ImpactPoint	Location of the actual contact point of the trace shape with the surface of the hit object. Equal to Location in the case of an initial overlap.
	 * @param ImpactNormal	Normal of the hit in world space, for the object that was hit by the sweep.
	 * @param PhysMat		Physical material that was hit. Must set bReturnPhysicalMaterial to true in the query params for this to be returned.
	 * @param HitActor		Actor hit by the trace.
	 * @param HitComponent	PrimitiveComponent hit by the trace.
	 * @param HitBoneName	Name of the bone hit (valid only if we hit a skeletal mesh).
	 * @param BoneName		Name of the trace bone hit (valid only if we hit a skeletal mesh).
	 * @param HitItem		Primitive-specific data recording which item in the primitive was hit
	 * @param ElementIndex	If colliding with a primitive with multiple parts, index of the part that was hit.
	 * @param FaceIndex		If colliding with trimesh or landscape, index of face that was hit.
	 */
	UFUNCTION(BlueprintPure, Category = "Collision", meta = (NativeMakeFunc, AdvancedDisplay="2", Normal="0,0,1", ImpactNormal="0,0,1"))
	static ENGINE_API FHitResult MakeHitResult(bool bBlockingHit, bool bInitialOverlap, float Time, float Distance, FVector Location, FVector ImpactPoint, FVector Normal, FVector ImpactNormal, class UPhysicalMaterial* PhysMat, class AActor* HitActor, class UPrimitiveComponent* HitComponent, FName HitBoneName, FName BoneName, int32 HitItem, int32 ElementIndex, int32 FaceIndex, FVector TraceStart, FVector TraceEnd);


	/** Returns the EPhysicalSurface type of the given Hit. 
	 * To edit surface type for your project, use ProjectSettings/Physics/PhysicalSurface section
	 */
	UFUNCTION(BlueprintPure, Category="Collision")
	static ENGINE_API EPhysicalSurface GetSurfaceType(const struct FHitResult& Hit);

	/** 
	 *	Try and find the UV for a collision impact. Note this ONLY works if 'Support UV From Hit Results' is enabled in Physics Settings.
	 */
	UFUNCTION(BlueprintPure, Category = "Collision")
	static ENGINE_API bool FindCollisionUV(const struct FHitResult& Hit, int32 UVChannel, FVector2D& UV);

	// --- Save Game functions ------------------------------

	/** 
	 * Create a new, empty SaveGame object to set data on and then pass to SaveGameToSlot.
	 * @param	SaveGameClass	Class of SaveGame to create
	 * @return					New SaveGame object to write data to
	 */
	UFUNCTION(BlueprintCallable, Category="SaveGame", meta=(DeterminesOutputType="SaveGameClass"))
	static ENGINE_API USaveGame* CreateSaveGameObject(TSubclassOf<USaveGame> SaveGameClass);

	/** 
	 * Serialize our USaveGame object into a given array of bytes
	 * @note This will write out all non-transient properties, the SaveGame property flag is not checked
	 * 
	 * @param SaveGameObject	Object that contains data about the save game that we want to write out
	 * @param OutSaveData		Byte array that is written into
	 * @return					Whether we successfully wrote data
	 */
	static ENGINE_API bool SaveGameToMemory(USaveGame* SaveGameObject, TArray<uint8>& OutSaveData);

	/** 
	 * Save the contents of the buffer to a platform-specific save slot/file 
	 * @param InSaveData		Data to save
	 * @param SlotName			Name of save game slot to save to.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @return					Whether we successfully saved this information
	 */
	static ENGINE_API bool SaveDataToSlot(const TArray<uint8>& InSaveData, const FString& SlotName, const int32 UserIndex);

	/** 
	 * Schedule an async save to a specific slot. UAsyncActionHandleSaveGame::AsyncSaveGameToSlot is the blueprint version of this.
	 * This will do the serialize on the game thread, the platform-specific write on a worker thread, then call the complete delegate on the game thread.
	 * The passed in delegate will be copied to a worker thread so make sure any payload is thread safe to copy by value.
	 *
	 * @param SaveGameObject	Object that contains data about the save game that we want to write out.
	 * @param SlotName			Name of the save game slot to load from.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @param SavedDelegate		Delegate that will be called on game thread when save succeeds or fails.
	 */
	static ENGINE_API void AsyncSaveGameToSlot(USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex, FAsyncSaveGameToSlotDelegate SavedDelegate = FAsyncSaveGameToSlotDelegate());

	/** 
	 * Save the contents of the SaveGameObject to a platform-specific save slot/file.
	 * @note This will write out all non-transient properties, the SaveGame property flag is not checked
	 *
	 * @param SaveGameObject	Object that contains data about the save game that we want to write out
	 * @param SlotName			Name of save game slot to save to.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @return					Whether we successfully saved this information
	 */
	UFUNCTION(BlueprintCallable, Category="SaveGame")
	static ENGINE_API bool SaveGameToSlot(USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex);

	/**
	 * See if a save game exists with the specified name.
	 * @param SlotName			Name of save game slot.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 */
	UFUNCTION(BlueprintCallable, Category="SaveGame")
	static ENGINE_API bool DoesSaveGameExist(const FString& SlotName, const int32 UserIndex);

	/**
	 * Tries to load a SaveGame object from a given array of bytes.
	 * @param InSaveData		The array containing the serialized USaveGame data to load
	 * @return					Object containing loaded game state (nullptr if load fails)
	 */
	static ENGINE_API USaveGame* LoadGameFromMemory(const TArray<uint8>& InSaveData);

	/**
	 * Load contents from a slot/file into a buffer of save data.
	 * @param OutSaveData		Data buffer to load into
	 * @param SlotName			Name of save game slot to save to.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @return					Whether valid save data was found and loaded.
	 */
	static ENGINE_API bool LoadDataFromSlot(TArray<uint8>& OutSaveData, const FString& SlotName, const int32 UserIndex);

	/** 
	 * Schedule an async load of a specific slot. UAsyncActionHandleSaveGame::AsyncLoadGameFromSlot is the blueprint version of this.
	 * This will do the platform-specific read on a worker thread, the serialize and creation on the game thread, and then will call the passed in delegate
	 * The passed in delegate will be copied to a worker thread so make sure any payload is thread safe to copy by value
	 *
	 * @param SlotName			Name of the save game slot to load from.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @param LoadedDelegate	Delegate that will be called on game thread when load succeeds or fails.
	 */
	static ENGINE_API void AsyncLoadGameFromSlot(const FString& SlotName, const int32 UserIndex, FAsyncLoadGameFromSlotDelegate LoadedDelegate);

	/** 
	 * Load the contents from a given slot.
	 * @param SlotName			Name of the save game slot to load from.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @return					Object containing loaded game state (nullptr if load fails)
	 */
	UFUNCTION(BlueprintCallable, Category="SaveGame")
	static ENGINE_API USaveGame* LoadGameFromSlot(const FString& SlotName, const int32 UserIndex);

	/** 
	 * Takes the provided buffer and consumes it, parsing past the internal header data, returning a MemoryReader
	 * that has: 1) been set up with all the related header information, and 2) offset to where tagged USaveGame object 
	 * serialization begins. NOTE: that the returned object has a reference to the supplied data - scope them 
	 * accordingly.
	 *
	 * @param SaveData			A byte array, presumably produced by one of the SaveGame functions here.
	 * @return					A memory reader, wrapping SaveData, offset to the point past the header data.
	 */
	static ENGINE_API FMemoryReader StripSaveGameHeader(const TArray<uint8>& SaveData);

	/**
	 * Delete a save game in a particular slot.
	 * @param SlotName			Name of save game slot to delete.
	 * @param UserIndex			The platform user index that identifies the user doing the saving, ignored on some platforms.
	 * @return					True if a file was actually able to be deleted. use DoesSaveGameExist to distinguish between delete failures and failure due to file not existing.
	 */
	UFUNCTION(BlueprintCallable, Category = "SaveGame")
	static ENGINE_API bool DeleteGameInSlot(const FString& SlotName, const int32 UserIndex);

	/** Returns the frame delta time in seconds, adjusted by time dilation. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time", meta = (WorldContext="WorldContextObject"))
	static ENGINE_API double GetWorldDeltaSeconds(const UObject* WorldContextObject);

	/** Returns time in seconds since world was brought up for play, adjusted by time dilation and IS stopped when game pauses */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API double GetTimeSeconds(const UObject* WorldContextObject);

	/** Returns time in seconds since world was brought up for play, adjusted by time dilation and IS NOT stopped when game pauses */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API double GetUnpausedTimeSeconds(const UObject* WorldContextObject);

	/** Returns time in seconds since world was brought up for play, does NOT stop when game pauses, NOT dilated/clamped */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API double GetRealTimeSeconds(const UObject* WorldContextObject);
	
	/** Returns time in seconds since world was brought up for play, IS stopped when game pauses, NOT dilated/clamped. */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API double GetAudioTimeSeconds(const UObject* WorldContextObject);

	/** Returns time in seconds since the application was started. Unlike the other time functions this is accurate to the exact time this function is called instead of set once per frame. */
	UFUNCTION(BlueprintPure, Category="Utilities|Time")
	static ENGINE_API void GetAccurateRealTime(int32& Seconds, double& PartialSeconds);

	/*~ DVRStreaming API */
	
	/**
	 * Toggle live DVR streaming.
	 * @param Enable			If true enable streaming, otherwise disable.
	 */
	UFUNCTION(BlueprintCallable, Category="Game", meta=(Enable="true"))
	static ENGINE_API void EnableLiveStreaming(bool Enable);

	/**
	 * Returns the string name of the current platform, to perform different behavior based on platform. 
	 * (Platform names include Windows, Mac, Linux, IOS, Android, consoles, etc.). */
	UFUNCTION(BlueprintPure, Category="Game")
	static ENGINE_API FString GetPlatformName();

	/**
	 * Calculates an launch velocity for a projectile to hit a specified point.
	 * @param TossVelocity					(output) Result launch velocity.
	 * @param StartLocation					Intended launch location
	 * @param EndLocation					Desired landing location
	 * @param LaunchSpeed					Desired launch speed
	 * @param OverrideGravityZ				Optional gravity override.  0 means "do not override".
	 * @param TraceOption					Controls whether or not to validate a clear path by tracing along the calculated arc
	 * @param CollisionRadius				Radius of the projectile (assumed spherical), used when tracing
	 * @param bFavorHighArc					If true and there are 2 valid solutions, will return the higher arc.  If false, will favor the lower arc.
	 * @param bDrawDebug					When true, a debug arc is drawn (red for an invalid arc, green for a valid arc)
	 * @param bAcceptClosestOnNoSolutions	If target is unreachable and no solutions are possible, provide a velocity that gets as close to the target as possible given this launch speed
	 * @return								Returns false if there is no valid solution or the valid solutions are blocked.  Returns true otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Game", DisplayName="Suggest Projectile Velocity", meta=(WorldContext="WorldContextObject"))
	static bool BlueprintSuggestProjectileVelocity(const UObject* WorldContextObject, FVector& TossVelocity, FVector StartLocation, FVector EndLocation, float LaunchSpeed, float OverrideGravityZ, ESuggestProjVelocityTraceOption::Type TraceOption, float CollisionRadius, bool bFavorHighArc, bool bDrawDebug, bool bAcceptClosestOnNoSolutions = false);

	UE_DEPRECATED(5.4, "Deprecated parameter list. Please use FSuggestProjectileVelocityParameters instead.")
	static ENGINE_API bool SuggestProjectileVelocity(const UObject* WorldContextObject, FVector& TossVelocity, FVector StartLocation, FVector EndLocation, float TossSpeed, bool bHighArc = false, float CollisionRadius = 0.f, float OverrideGravityZ = 0, ESuggestProjVelocityTraceOption::Type TraceOption = ESuggestProjVelocityTraceOption::TraceFullPath, FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam, TArray<AActor*> ActorsToIgnore = TArray<AActor*>(), bool bDrawDebug = false, bool bAcceptClosestOnNoSolutions = false);

	/** Native version, has more options than the Blueprint version. */
	static ENGINE_API bool SuggestProjectileVelocity(const FSuggestProjectileVelocityParameters& ProjectileParams, FVector& OutTossVelocity);


	/** Stepwise trace along the path of a given velocity*/
	static ENGINE_API bool IsProjectileTrajectoryBlocked(const UWorld* World, FVector StartLocation, FVector& ProjectileVelocity, float TargetDeltaXY, float GravityZ, float CollisionRadius = 0.f, ESuggestProjVelocityTraceOption::Type TraceOption = ESuggestProjVelocityTraceOption::TraceFullPath, const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam, const TArray<AActor*>& ActorsToIgnore = TArray<AActor*>(), bool bDrawDebug = false);

	/**
	* Predict the arc of a virtual projectile affected by gravity with collision checks along the arc. Returns a list of positions of the simulated arc and the destination reached by the simulation.
	* Returns true if it hit something.
	*
	* @param OutPathPositions			Predicted projectile path. Ordered series of positions from StartPos to the end. Includes location at point of impact if it hit something.
	* @param OutHit						Predicted hit result, if the projectile will hit something
	* @param OutLastTraceDestination	Goal position of the final trace it did. Will not be in the path if there is a hit.
	* @param StartPos					First start trace location
	* @param LaunchVelocity				Velocity the "virtual projectile" is launched at
	* @param bTracePath					Trace along the entire path to look for blocking hits
	* @param ProjectileRadius			Radius of the virtual projectile to sweep against the environment
	* @param ObjectTypes				ObjectTypes to trace against, if bTracePath is true.
	* @param bTraceComplex				Use TraceComplex (trace against triangles not primitives)
	* @param ActorsToIgnore				Actors to exclude from the traces
	* @param DrawDebugType				Debug type (one-frame, duration, persistent)
	* @param DrawDebugTime				Duration of debug lines (only relevant for DrawDebugType::Duration)
	* @param SimFrequency				Determines size of each sub-step in the simulation (chopping up MaxSimTime)
	* @param MaxSimTime					Maximum simulation time for the virtual projectile.
	* @param OverrideGravityZ			Optional override of Gravity (if 0, uses WorldGravityZ)
	* @return							True if hit something along the path if tracing for collision.
	*/
	UFUNCTION(BlueprintCallable, Category = "Game", DisplayName="Predict Projectile Path By ObjectType", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", AdvancedDisplay = "DrawDebugTime, DrawDebugType, SimFrequency, MaxSimTime, OverrideGravityZ", bTracePath = true))
	static ENGINE_API bool Blueprint_PredictProjectilePath_ByObjectType(const UObject* WorldContextObject, FHitResult& OutHit, TArray<FVector>& OutPathPositions, FVector& OutLastTraceDestination, FVector StartPos, FVector LaunchVelocity, bool bTracePath, float ProjectileRadius, const TArray<TEnumAsByte<EObjectTypeQuery> >& ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, float DrawDebugTime, float SimFrequency = 15.f, float MaxSimTime = 2.f, float OverrideGravityZ = 0);

	/**
	* Predict the arc of a virtual projectile affected by gravity with collision checks along the arc. Returns a list of positions of the simulated arc and the destination reached by the simulation.
	* Returns true if it hit something (if tracing with collision).
	*
	* @param OutPathPositions			Predicted projectile path. Ordered series of positions from StartPos to the end. Includes location at point of impact if it hit something.
	* @param OutHit						Predicted hit result, if the projectile will hit something
	* @param OutLastTraceDestination	Goal position of the final trace it did. Will not be in the path if there is a hit.
	* @param StartPos					First start trace location
	* @param LaunchVelocity				Velocity the "virtual projectile" is launched at
	* @param bTracePath					Trace along the entire path to look for blocking hits
	* @param ProjectileRadius			Radius of the virtual projectile to sweep against the environment
	* @param TraceChannel				TraceChannel to trace against, if bTracePath is true.
	* @param bTraceComplex				Use TraceComplex (trace against triangles not primitives)
	* @param ActorsToIgnore				Actors to exclude from the traces
	* @param DrawDebugType				Debug type (one-frame, duration, persistent)
	* @param DrawDebugTime				Duration of debug lines (only relevant for DrawDebugType::Duration)
	* @param SimFrequency				Determines size of each sub-step in the simulation (chopping up MaxSimTime)
	* @param MaxSimTime					Maximum simulation time for the virtual projectile.
	* @param OverrideGravityZ			Optional override of Gravity (if 0, uses WorldGravityZ)
	* @return							True if hit something along the path (if tracing with collision).
	*/
	UFUNCTION(BlueprintCallable, Category = "Game", DisplayName="Predict Projectile Path By TraceChannel", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", AdvancedDisplay = "DrawDebugTime, DrawDebugType, SimFrequency, MaxSimTime, OverrideGravityZ", TraceChannel = ECC_WorldDynamic, bTracePath = true))
	static ENGINE_API bool Blueprint_PredictProjectilePath_ByTraceChannel(const UObject* WorldContextObject, FHitResult& OutHit, TArray<FVector>& OutPathPositions, FVector& OutLastTraceDestination, FVector StartPos, FVector LaunchVelocity, bool bTracePath, float ProjectileRadius, TEnumAsByte<ECollisionChannel> TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, float DrawDebugTime, float SimFrequency = 15.f, float MaxSimTime = 2.f, float OverrideGravityZ = 0);

	/**
	* Predict the arc of a virtual projectile affected by gravity with collision checks along the arc.
	* Returns true if it hit something.
	*
	* @param PredictParams				Input params to the trace (start location, velocity, time to simulate, etc).
	* @param PredictResult				Output result of the trace (Hit result, array of location/velocity/times for each trace step, etc).
	* @return							True if hit something along the path (if tracing with collision).
	*/
	static ENGINE_API bool PredictProjectilePath(const UObject* WorldContextObject, const FPredictProjectilePathParams& PredictParams, FPredictProjectilePathResult& PredictResult);

	/**
	* Predict the arc of a virtual projectile affected by gravity with collision checks along the arc.
	* Returns true if it hit something.
	*
	* @param PredictParams				Input params to the trace (start location, velocity, time to simulate, etc).
	* @param PredictResult				Output result of the trace (Hit result, array of location/velocity/times for each trace step, etc).
	* @return							True if hit something along the path (if tracing with collision).
	*/
	UFUNCTION(BlueprintCallable, Category = "Game", DisplayName="Predict Projectile Path (Advanced)", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool Blueprint_PredictProjectilePath_Advanced(const UObject* WorldContextObject, const FPredictProjectilePathParams& PredictParams, FPredictProjectilePathResult& PredictResult);

	/**
	* Returns the launch velocity needed for a projectile at rest at StartPos to land on EndPos.
	* Assumes a medium arc (e.g. 45 deg on level ground). Projectile velocity is variable and unconstrained.
	* Does no tracing.
	*
	* @param OutLaunchVelocity			Returns the launch velocity required to reach the EndPos
	* @param StartPos					Start position of the simulation
	* @param EndPos						Desired end location for the simulation
	* @param OverrideGravityZ			Optional override of WorldGravityZ
	* @param ArcParam					Change height of arc between 0.0-1.0 where 0.5 is the default medium arc, 0 is up, and 1 is directly toward EndPos.
	*/
	UFUNCTION(BlueprintCallable, Category = "Game", DisplayName = "Suggest Projectile Velocity Custom Arc", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "OverrideGravityZ, ArcParam"))
	static ENGINE_API bool SuggestProjectileVelocity_CustomArc(const UObject* WorldContextObject, FVector& OutLaunchVelocity, FVector StartPos, FVector EndPos, float OverrideGravityZ = 0, float ArcParam = 0.5f);

	/**
	 * Returns a launch velocity need for a projectile to hit the TargetActor in TimeToTarget seconds based on the TargetActor's current velocity.
	 * This assumes the projectile is only accelerated by gravity (i.e. no outside forces), and that the TargetActor is moving at a constant velocity.
	 * 
	 * @param OutLaunchVelocity			The launch velocity returned from this calculation
	 * @param ProjectileStartLocation	Location the projectile is launched from
	 * @param TargetActor				Actor that the projectile should hit in TimeToTarget seconds
	 * @param TargetLocationOffset		Offset to apply to the location the projectile is aiming for
	 * @param GravityZOverride			Optional override of WorldGravityZ
	 * @param TimeToTarget				Time (in seconds) between the projectile being launched and the projectile hitting the TargetActor - clamped to be at least 0.1
	 */
	UFUNCTION(BlueprintCallable, Category = "Game", DisplayName = "Suggest Projectile Velocity Moving Target", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "DrawDebugType, DrawDebugTime, DrawDebugColor"))
	static ENGINE_API bool SuggestProjectileVelocity_MovingTarget(const UObject* WorldContextObject, FVector& OutLaunchVelocity, FVector ProjectileStartLocation, AActor* TargetActor, FVector TargetLocationOffset = FVector::ZeroVector, double GravityZOverride = 0.f, double TimeToTarget = 1.f, EDrawDebugTrace::Type DrawDebugType = EDrawDebugTrace::Type::None, float DrawDebugTime = 3.f, FLinearColor DrawDebugColor = FLinearColor::Red);

	/** Returns world origin current location. */
	UFUNCTION(BlueprintPure, Category="Game", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API FIntVector GetWorldOriginLocation(const UObject* WorldContextObject);
	
	/** Requests a new location for a world origin. */
	UFUNCTION(BlueprintCallable, Category="Game", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API void SetWorldOriginLocation(const UObject* WorldContextObject, FIntVector NewLocation);

	/** Returns origin based position for local world location. */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API FVector RebaseLocalOriginOntoZero(UObject* WorldContextObject, FVector WorldLocation);

	/** Returns local location for origin based position. */
	UFUNCTION(BlueprintPure, Category = "Game", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API FVector RebaseZeroOriginOntoLocal(UObject* WorldContextObject, FVector WorldLocation);

	/**
	* Counts how many grass foliage instances overlap a given sphere.
	*
	* @param	Mesh			The static mesh we are interested in counting.
	* @param	CenterPosition	The center position of the sphere.
	* @param	Radius			The radius of the sphere.
	*
	* @return Number of foliage instances with their mesh set to Mesh that overlap the sphere.
	*/
	UFUNCTION(BlueprintCallable, Category = "Foliage", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GrassOverlappingSphereCount(const UObject* WorldContextObject, const UStaticMesh* StaticMesh, FVector CenterPosition, float Radius);

	/** 
	 * Transforms the given 2D screen space coordinate into a 3D world-space point and direction.
	 * @param Player			Deproject using this player's view.
	 * @param ScreenPosition	2D screen space to deproject.
	 * @param WorldPosition		(out) Corresponding 3D position in world space.
	 * @param WorldDirection	(out) World space direction vector away from the camera at the given 2d point.
	 */
	UFUNCTION(BlueprintPure, Category = "Camera", meta = (Keywords = "unproject"))
	static ENGINE_API bool DeprojectScreenToWorld(APlayerController const* Player, const FVector2D& ScreenPosition, FVector& WorldPosition, FVector& WorldDirection);

	/** 
	 * Transforms the given 2D UV coordinate into a 3D world-space point and direction.
	 * @param SceneCapture2D	Deproject using this scene capture's view.
	 * @param ScreenPosition	UV in scene capture render target to deproject.
	 * @param WorldPosition		(out) Corresponding 3D position on camera near plane, in world space.
	 * @param WorldDirection	(out) World space direction vector away from the camera at the given 2d point.
	 */
	UFUNCTION(BlueprintPure, Category = "Camera", meta = (Keywords = "unproject"))
	static ENGINE_API bool DeprojectSceneCaptureToWorld(ASceneCapture2D const* SceneCapture2D, const FVector2D& TargetUV, FVector& WorldPosition, FVector& WorldDirection);

	/**
	 * Transforms the given 2D UV coordinate into a 3D world-space point and direction.
	 * @param SceneCaptureComponent2D	Deproject using this scene capture component's view.
	 * @param ScreenPosition			UV in scene capture render target to deproject.
	 * @param WorldPosition				(out) Corresponding 3D position on camera near plane, in world space.
	 * @param WorldDirection			(out) World space direction vector away from the camera at the given 2d point.
	 */
	UFUNCTION(BlueprintPure, Category = "Camera", meta = (Keywords = "unproject"))
	static ENGINE_API bool DeprojectSceneCaptureComponentToWorld(USceneCaptureComponent2D* SceneCaptureComponent2D, const FVector2D& TargetUV, FVector& WorldPosition, FVector& WorldDirection);

	/** 
	 * Transforms the given 3D world-space point into a its 2D screen space coordinate. 
	 * @param Player			Project using this player's view.
	 * @param WorldPosition		World position to project.
	 * @param ScreenPosition	(out) Corresponding 2D position in screen space
	 * @param bPlayerViewportRelative	Should this be relative to the player viewport subregion (useful when using player attached widgets in split screen)
	 */
	UFUNCTION(BlueprintPure, Category = "Camera")
	static ENGINE_API bool ProjectWorldToScreen(APlayerController const* Player, const FVector& WorldPosition, FVector2D& ScreenPosition, bool bPlayerViewportRelative = false);

	/**
	 * Transforms a world space location into "first person space". This function mirrors the morphing that is applied to first person primitives
	 * when they are rendered on the GPU, so it can be used for spawning objects (e.g. projectiles or ejected shell casings) relative to the morphed
	 * first person geometry on screen.
	 * @param ViewInfo					FMinimalViewInfo struct holding the first person camera parameters.
	 * @param WorldPosition				World space position to transform.
	 * @param bIgnoreFirstPersonScale	Ignores the scaling that is applied to first person primitives which can be useful when spawning full size world space projectiles relative to a first person weapon.
	 */
	UFUNCTION(BlueprintPure, Category = "Camera")
	static ENGINE_API FVector TransformWorldToFirstPerson(const FMinimalViewInfo& ViewInfo, const FVector& WorldPosition, bool bIgnoreFirstPersonScale);

	/**
	 * Transforms a first person space location back into world space. This function is the inverse of TransformWorldToFirstPerson.
	 * @param ViewInfo					FMinimalViewInfo struct holding the first person camera parameters.
	 * @param FirstPersonPosition		First person space position to transform.
	 * @param bScaleWasIgnoredOnInput	Should be set to true if FirstPersonPosition was computed with TransformWorldToFirstPerson and the bIgnoreFirstPersonScale parameter set to true.
	 */
	UFUNCTION(BlueprintPure, Category = "Camera")
	static ENGINE_API FVector TransformFirstPersonToWorld(const FMinimalViewInfo& ViewInfo, const FVector& FirstPersonPosition, bool bScaleWasIgnoredOnInput);

	/**
	 * Returns the View Matrix, Projection Matrix and the View x Projection Matrix for a given view
	 * @param DesiredView			FMinimalViewInfo struct for a camera.
	 * @param ViewMatrix			(out) Corresponding View Matrix
	 * @param ProjectionMatrix		(out) Corresponding Projection Matrix
	 * @param ViewProjectionMatrix	(out) Corresponding View x Projection Matrix
	 */
	UFUNCTION(BlueprintPure, Category = "Camera")
	static ENGINE_API void GetViewProjectionMatrix(FMinimalViewInfo DesiredView, FMatrix &ViewMatrix, FMatrix &ProjectionMatrix, FMatrix &ViewProjectionMatrix);

	/**
	 * Calculate view-projection matrices from a specified view target
	 */
	static ENGINE_API void CalculateViewProjectionMatricesFromViewTarget(AActor* InViewTarget, FMatrix& OutViewMatrix, FMatrix& OutProjectionMatrix, FMatrix& OutViewProjectionMatrix);

	/**
	 * Calculate view-projection matrices from a specified MinimalViewInfo and optional custom projection matrix
	 */
	static ENGINE_API void CalculateViewProjectionMatricesFromMinimalView(const FMinimalViewInfo& MinimalViewInfo, const TOptional<FMatrix>& CustomProjectionMatrix, FMatrix& OutViewMatrix, FMatrix& OutProjectionMatrix, FMatrix& OutViewProjectionMatrix);

	//~ Utility functions for interacting with Options strings

	//~=========================================================================
	//~ URL Parsing

	static ENGINE_API bool GrabOption( FString& Options, FString& ResultString );

	/** 
	 * Break up a key=value pair into its key and value. 
	 * @param Pair			The string containing a pair to split apart.
	 * @param Key			(out) Key portion of Pair. If no = in string will be the same as Pair.
	 * @param Value			(out) Value portion of Pair. If no = in string will be empty.
	 */
	UFUNCTION(BlueprintPure, Category="Game Options", meta=(BlueprintThreadSafe))
	static ENGINE_API void GetKeyValue( const FString& Pair, FString& Key, FString& Value );

	/** 
	 * Find an option in the options string and return it.
	 * @param Options		The string containing the options.
	 * @param Key			The key to find the value of in Options.
	 * @return				The value associated with Key if Key found in Options string.
	 */
	UFUNCTION(BlueprintPure, Category="Game Options", meta=(BlueprintThreadSafe))
	static ENGINE_API FString ParseOption( FString Options, const FString& Key );

	/** 
	 * Returns whether a key exists in an options string.
	 * @param Options		The string containing the options.
	 * @param Key			The key to determine if it exists in Options.
	 * @return				Whether Key was found in Options.
	 */
	UFUNCTION(BlueprintPure, Category="Game Options", meta=(BlueprintThreadSafe))
	static ENGINE_API bool HasOption( FString Options, const FString& InKey );

	/** 
	 * Find an option in the options string and return it as an integer.
	 * @param Options		The string containing the options.
	 * @param Key			The key to find the value of in Options.
	 * @return				The value associated with Key as an integer if Key found in Options string, otherwise DefaultValue.
	 */
	UFUNCTION(BlueprintPure, Category="Game Options", meta=(BlueprintThreadSafe))
	static ENGINE_API int32 GetIntOption( const FString& Options, const FString& Key, int32 DefaultValue);

	//~=========================================================================
	//~ Launch Options Parsing

	/**
	* Checks the commandline to see if the desired option was specified on the commandline (e.g. -demobuild)
	* @return				True if the launch option was specified on the commandline, false otherwise
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool HasLaunchOption(const FString& OptionToCheck);

	/**
	 * If accessibility is enabled, have the platform announce a string to the player.
	 * These announcements can be interrupted by system accessibiliity announcements or other accessibility announcement requests.
	 * This should be used judiciously as flooding a player with announcements can be overrwhelming and confusing.
	 * Try to make announcements concise and clear.
	 * NOTE: Currently only supported on Win10, Mac, iOS
	*/
	UFUNCTION(BlueprintCallable, Category = "Accessibility")
	static ENGINE_API void AnnounceAccessibleString(const FString& AnnouncementString);
};

