// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "KismetSystemLibrary.h"
#include "Engine/HitResult.h"
#include "GameplayStaticsTypes.generated.h"

// Trace option to SuggestProjectileVelocity functions.
UENUM()
namespace ESuggestProjVelocityTraceOption
{
	enum Type : int
	{
		DoNotTrace,
		TraceFullPath,
		OnlyTraceWhileAscending,
	};
}

// Input parameters to PredictProjectilePath functions.
USTRUCT(BlueprintType)
struct FPredictProjectilePathParams
{
	GENERATED_USTRUCT_BODY();

	// Location of the start of the trace.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams)
	FVector StartLocation;

	// Initial launch velocity at the start of the trace.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams)
	FVector LaunchVelocity;

	// Whether to trace along the path looking for blocking collision and stopping at the first hit.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams)
	bool bTraceWithCollision;

	// Projectile radius, used when tracing for collision. If <= 0, a line trace is used instead.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams)
	float ProjectileRadius;

	// Maximum simulation time for the virtual projectile.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams)
	float MaxSimTime;

	// Whether or not to use TraceChannel, if tracing with collision.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PredictProjectilePathParams)
	bool bTraceWithChannel;

	// Trace channel to use, if tracing with collision.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PredictProjectilePathParams)
	TEnumAsByte<ECollisionChannel> TraceChannel;

	// Object type to use, if tracing with collision.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	TArray<TEnumAsByte<EObjectTypeQuery>> ObjectTypes;

	// Actors to ignore when tracing with collision.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	TArray<TObjectPtr<AActor>> ActorsToIgnore;

	// Determines size of each sub-step in the simulation (chopping up MaxSimTime). Recommended between 10 to 30 depending on desired quality versus performance.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	float SimFrequency;

	// Optional override of Gravity (if 0, uses WorldGravityZ).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	float OverrideGravityZ;

	// Debug drawing duration option.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	TEnumAsByte<EDrawDebugTrace::Type> DrawDebugType;
	
	// Duration of debug lines (only relevant for DrawDebugType::Duration)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	float DrawDebugTime;

	// Trace against complex collision (triangles rather than simple primitives) if tracing with collision.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PredictProjectilePathParams, AdvancedDisplay)
	bool bTraceComplex;

	// Empty constructor. You typically want to use another one that enforces thought about reasonable values for the most important parameters.
	FPredictProjectilePathParams()
	{
		Init(0.f, FVector::ZeroVector, FVector::ForwardVector, 1.f, false);
	}

	// Constructor defaulting to no collision.
	FPredictProjectilePathParams(float InProjectileRadius, FVector InStartLocation, FVector InLaunchVelocity, float InMaxSimTime)
	{
		Init(InProjectileRadius, InStartLocation, InLaunchVelocity, InMaxSimTime, false);
	}

	// Constructor to use collision with an ObjectType. Additional types can be added to the array. TraceWithCollision is set to true automatically.
	FPredictProjectilePathParams(float InProjectileRadius, FVector InStartLocation, FVector InLaunchVelocity, float InMaxSimTime, EObjectTypeQuery ObjectType, AActor* ActorToIgnore = nullptr)
	{
		Init(InProjectileRadius, InStartLocation, InLaunchVelocity, InMaxSimTime, true);
		ObjectTypes.Add(ObjectType);
		bTraceWithChannel = false;
		if (ActorToIgnore)
		{
			ActorsToIgnore.Add(ActorToIgnore);
		}
	}

	// Constructor to use collision with a trace channel. TraceWithCollision is set to true automatically.
	FPredictProjectilePathParams(float InProjectileRadius, FVector InStartLocation, FVector InLaunchVelocity, float InMaxSimTime, ECollisionChannel InTraceChannel, AActor* ActorToIgnore = nullptr)
	{
		Init(InProjectileRadius, InStartLocation, InLaunchVelocity, InMaxSimTime, true);
		TraceChannel = InTraceChannel;
		if (ActorToIgnore)
		{
			ActorsToIgnore.Add(ActorToIgnore);
		}
	}

private:

	void Init(float InProjectileRadius, FVector InStartLocation, FVector InLaunchVelocity, float InMaxSimTime, bool InTraceWithCollision)
	{
		StartLocation = InStartLocation;
		LaunchVelocity = InLaunchVelocity;
		bTraceWithCollision = InTraceWithCollision;
		ProjectileRadius = InProjectileRadius;
		MaxSimTime = InMaxSimTime;
		SimFrequency = 20.f;
		OverrideGravityZ = 0.f;
		DrawDebugTime = 1.f;
		bTraceWithChannel = true;
		TraceChannel = ECC_WorldStatic;
		DrawDebugType = EDrawDebugTrace::None;
		bTraceComplex = false;
	}
};


// Data about a single point in a projectile path trace.
USTRUCT(BlueprintType)
struct FPredictProjectilePathPointData
{
	GENERATED_USTRUCT_BODY();

	// Location of this point
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathPointData)
	FVector Location;

	// Velocity at this point
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathPointData)
	FVector Velocity;

	// Elapsed time at this point from the start of the trace.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathPointData)
	float Time;

	FPredictProjectilePathPointData()
		: Location(ForceInitToZero)
		, Velocity(ForceInitToZero)
		, Time(0.f)
	{
	}

	FPredictProjectilePathPointData(FVector InLocation, FVector InVelocity, float InTime)
		: Location(InLocation)
		, Velocity(InVelocity)
		, Time(InTime)
	{
	}

	void Reset()
	{
		Set(FVector(ForceInitToZero), FVector(ForceInitToZero), 0.f);
	}

	void Set(FVector InLocation, FVector InVelocity, float InTime)
	{
		Location = InLocation;
		Velocity = InVelocity;
		Time = InTime;
	}
};


// Container for the result of a projectile path trace (using PredictProjectilePath).
USTRUCT(BlueprintType)
struct FPredictProjectilePathResult
{
	GENERATED_USTRUCT_BODY();

	// Info for each point on the path.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathResult)
	TArray<FPredictProjectilePathPointData> PathData;

	// Info on the last point we tried to trace to, which may have been beyond the final hit.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathResult)
	FPredictProjectilePathPointData LastTraceDestination;

	// Hit along the trace, if tracing with collision was enabled.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=PredictProjectilePathResult)
	FHitResult HitResult;

	void Reset(int32 ReserveSize = 0)
	{
		PathData.Reset(ReserveSize);
		LastTraceDestination = FPredictProjectilePathPointData();
		HitResult.Init();
	}

	void AddPoint(FVector Location, FVector Velocity, float Time)
	{
		PathData.Emplace(FPredictProjectilePathPointData(Location, Velocity, Time));
	}
};

