// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Engine/Texture2D.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "ImportanceSamplingLibrary.generated.h"

/** Provides different weighting functions for texture importance sampling */
UENUM(BlueprintType)
namespace EImportanceWeight
{
	enum Type : int
	{
		/** Importance from color luminance. */
		Luminance,

		/** Importance from red channel of texture. */
		Red,
		
		/** Importance from green channel of texture. */
		Green,
		
		/** Importance from blue channel of texture. */
		Blue,

		/** Importance from alpha channel of texture. */
		Alpha,
	};
}

/**
* Texture processed for importance sampling
* Holds marginal PDF of the rows, as well as the PDF of each row
*/
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/Engine.ImportanceSamplingLibrary.MakeImportanceTexture", HasNativeBreak = "/Script/Engine.ImportanceSamplingLibrary.BreakImportanceTexture"))
struct FImportanceTexture
{
	GENERATED_BODY()

	// active texture dimensions, capped to 1024 x 1024
	UPROPERTY()
	FIntPoint Size;

	// active number of MIP levels
	UPROPERTY()
	int NumMips;

	// Unnormalized cumulative density of the image by rows (Size.Y+1)
	// First entry is zero, final entry is the CDF normalization factor
	UPROPERTY()
	TArray<float> MarginalCDF;

	// Unnormalized cumulative probability of each pixel in a row (Size.Y row CDFs of Size.X+1) 
	// First entry of each row is zero, final entry in each row is the CDF normalization factor for that row
	UPROPERTY()
	TArray<float> ConditionalCDF;

	UE_DEPRECATED(5.6, "FColor TextureData has been deprecated, use LinearTextureData")
	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Please use LinearTextureData instead."))
	TArray<FColor> TextureData_DEPRECATED;
	
	// packed copy of MIP level data for filtered sampling (capped to 1024x1024)
	// local copy seems better than allocating and copying the same data temporarily for each sample
	UPROPERTY(Transient)
	TArray<FLinearColor> LinearTextureData;

	// Original texture object for Break function
	UPROPERTY()
	TWeakObjectPtr<UTexture2D> Texture;

	// Original importance weight for Break function
	UPROPERTY()
	TEnumAsByte<EImportanceWeight::Type> Weighting;

public:
	/* Default constructor, must Initialize before use */
	FImportanceTexture();

	/* Constructor with initialization */
	FImportanceTexture(UTexture2D *SourceTexture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc);

	FImportanceTexture(const FImportanceTexture& Other);
	FImportanceTexture(FImportanceTexture&& Other);
	FImportanceTexture& operator=(const FImportanceTexture& Other);
	FImportanceTexture& operator=(FImportanceTexture&& Other);


	/* Allocate and compute PDF arrays for a texture */
	void Initialize(UTexture2D *SourceTexture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc);

	/**
	* Distribute sample points proportional to Texture2D luminance.
	* @param Rand - Random 2D point with components evenly distributed between 0 and 1
	* @param Samples - Total number of samples that will be used
	* @param Intensity - Overall target intensity scale
	* @outparam SamplePosition - Importance sampled 2D output texture coordinate (0-1)
	* @outparam SampleColor - Representative color near Position from MIP level for SampleSize
	* @outparam SampleIntensity - Intensity of individual point
	* @outparam SampleSize - Local density of points near Position (scaled for 1x1 texture space)
	*/
	void ImportanceSample(const FVector2D &Rand, int Samples, float Intensity,
			FVector2D &SamplePosition, FLinearColor &SampleColor, float &SampleIntensity, float &SampleSize) const;

	/* return color of texel at given MIP level, clamped to available Mip levels */
	FLinearColor GetColorBilinear(FVector2D Position, int32 Mip) const;

	/* return color interpolated between MIP levels */
	FLinearColor GetColorTrilinear(FVector2D Position, float Mip) const;

	/* importance probability weight for given texel */
	float ImportanceWeight(FColor Texel, TEnumAsByte<EImportanceWeight::Type> WeightingFunc) const;
};


UCLASS(MinimalAPI)
class UImportanceSamplingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	//
	// Sobol quasi-random generator functions
	//

	/**
	* @param Index - Which sequential point
	* @param Dimension - Which Sobol dimension (0 to 15)
	* @param Seed - Random seed (in the range 0-1) to randomize across multiple sequences
	* @return Sobol-distributed random number between 0 and 1
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API float RandomSobolFloat(int32 Index, int32 Dimension, float Seed);

	/**
	* @param Index - Which sequential point
	* @param Dimension - Which Sobol dimension (0 to 15)
	* @param PreviousValue - The Sobol value for Index-1
	* @return Sobol-distributed random number between 0 and 1
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API float NextSobolFloat(int32 Index, int32 Dimension, float PreviousValue);

	/**
	* @param Index - Which sequential point in the cell (starting at 0)
	* @param NumCells - Size of cell grid, 1 to 32768. Rounded up to the next power of two
	* @param Cell - Give a point from this integer grid cell
	* @param Seed - Random 2D seed (components in the range 0-1) to randomize across multiple sequences
	* @return Sobol-distributed random 2D position in the given grid cell
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API FVector2D RandomSobolCell2D(int32 Index, int32 NumCells = 1, FVector2D Cell = FVector2D(0,0), FVector2D Seed = FVector2D(0,0));

	/**
	* @param Index - Which sequential point
	* @param NumCells - Size of cell grid, 1 to 32768. Rounded up to the next power of two
	* @param PreviousValue - The Sobol value for Index-1
	* @return Sobol-distributed random 2D position in the same grid cell
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API FVector2D NextSobolCell2D(int32 Index, int32 NumCells = 1, FVector2D PreviousValue = FVector2D(0,0));

	/**
	* @param Index - Which sequential point in the cell (starting at 0)
	* @param NumCells - Size of cell grid, 1 to 1024. Rounded up to the next power of two
	* @param Cell - Give a point from this integer grid cell
	* @param Seed - Random 3D seed (components in the range 0-1) to randomize across multiple sequences
	* @return Sobol-distributed random 3D vector in the given grid cell
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API FVector RandomSobolCell3D(int32 Index, int32 NumCells = 1, FVector Cell = FVector(0,0,0), FVector Seed = FVector(0,0,0));

	/**
	* @param Index - Which sequential point
	* @param NumCells - Size of cell grid, 1 to 1024. Rounded up to the next power of two
	* @param PreviousValue - The Sobol value for Index-1
	* @return Sobol-distributed random 3D position in the same grid cell
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API FVector NextSobolCell3D(int32 Index, int32 NumCells = 1, FVector PreviousValue = FVector(0,0,0));

	/**
	* Create an FImportanceTexture object for texture-driven importance sampling from a 2D RGBA8 texture
	* @param Texture - Texture object to use. Must be RGBA8 format.
	* @param WeightingFunc - How to turn the texture data into probability weights
	* @return new ImportanceTexture object for use with ImportanceSample
	*/
	UFUNCTION(BlueprintPure, meta = (NativeMakeFunc), Category = "Math|Random")
	static ENGINE_API FImportanceTexture MakeImportanceTexture(UTexture2D *Texture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc);

	/**
	* Get texture used to create an ImportanceTexture object
	* @param ImportanceTexture - The source ImportanceTexture object
	* @outparam Texture - Texture object for this ImportanceTexture.
	* @param WeightingFunc - How to turn the texture data into probability weights
	* @return new ImportanceTexture object for use with ImportanceSample
	*/
	UFUNCTION(BlueprintPure, meta = (NativeBreakFunc), Category = "Math|Random")
	static ENGINE_API void BreakImportanceTexture(const FImportanceTexture &ImportanceTexture, UTexture2D *&Texture, TEnumAsByte<EImportanceWeight::Type> &WeightingFunc);

	/**
	* Distribute sample points proportional to Texture2D luminance.
	* @param Rand - Random 2D point with components evenly distributed between 0 and 1
	* @param Samples - Total number of samples that will be used
	* @param Intensity - Total intensity for light
	* @outparam SamplePosition - Importance sampled 2D output texture coordinate (0-1)
	* @outparam SampleColor - Representative color near Position from MIP level for SampleSize
	* @outparam SampleIntensity - Intensity of individual points, scaled by probability and number of samples
	* @outparam SampleSize - Local density of points near Position (scaled for 1x1 texture space)
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random")
	static ENGINE_API void ImportanceSample(const FImportanceTexture &Texture, const FVector2D &Rand, int Samples, float Intensity,
		FVector2D &SamplePosition, FLinearColor &SampleColor, float &SampleIntensity, float &SampleSize);
};
