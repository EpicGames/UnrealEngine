// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Net/Core/PushModel/PushModel.h"
#include "KismetArrayLibrary.generated.h"

class AActor;

UENUM(BlueprintType)
enum class EArraySortOrder : uint8
{
	Ascending,
	Descending
};

UCLASS(meta=(BlueprintThreadSafe), MinimalAPI)
class UKismetArrayLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	/** 
	 *Add item to array
	 *
	 *@param	TargetArray		The array to add item to
	 *@param	NewItem			The item to add to the array
	 *@return	The index of the newly added item
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Add", CompactNodeTitle = "ADD", ArrayParm = "TargetArray", ArrayTypeDependentParams = "NewItem", AutoCreateRefTerm = "NewItem"), Category="Utilities|Array")
	static ENGINE_API int32 Array_Add(UPARAM(ref) TArray<int32>& TargetArray, const int32& NewItem);

	/**
	*Add item to array (unique)
	*
	*@param		TargetArray		The array to add item to
	*@param		NewItem			The item to add to the array
	*@return	The index of the newly added item, or INDEX_NONE if the item is already present in the array
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (DisplayName = "Add Unique", CompactNodeTitle = "ADDUNIQUE", ArrayParm = "TargetArray", ArrayTypeDependentParams = "NewItem", AutoCreateRefTerm = "NewItem"), Category = "Utilities|Array")
	static ENGINE_API int32 Array_AddUnique(UPARAM(ref) TArray<int32>& TargetArray, const int32& NewItem);

	/** 
	 * Shuffle (randomize) the elements of an array
	 *
	 *@param	TargetArray		The array to shuffle
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Shuffle", CompactNodeTitle = "SHUFFLE", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Shuffle(UPARAM(ref) TArray<int32>& TargetArray);
	
	/** 
	 * Shuffle (randomize) the elements of an array from a specific stream of random data, useful for achieving determinism
	 *
	 * @param	TargetArray		The array to shuffle
	 * @param	RandomStream	The random stream
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Shuffle from Stream", CompactNodeTitle = "SHUFFLE", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_ShuffleFromStream(UPARAM(ref) TArray<int32>& TargetArray, UPARAM(Ref) FRandomStream& RandomStream);

	/** 
	 * Checks if two arrays are memberwise identical
	 *
	 * @param	ArrayA		One of the arrays to compare
	 * @param	ArrayB		The other array to compare
	 * @return Whether the two arrays are identical
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Identical", CompactNodeTitle = "==", ArrayParm = "ArrayA,ArrayB", ArrayTypeDependentParams = "ArrayB"), Category="Utilities|Array")
	static ENGINE_API bool Array_Identical(const TArray<int32>& ArrayA, const TArray<int32>& ArrayB);

	/** 
	 *Append an array to another array
	 *
	 *@param	TargetArray		The array to add the source array to
	 *@param	SourceArray		The array to add to the target array
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Append Array", CompactNodeTitle = "APPEND", ArrayParm = "TargetArray,SourceArray", ArrayTypeDependentParams = "SourceArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Append(UPARAM(ref) TArray<int32>& TargetArray, const TArray<int32>& SourceArray);

	/* 
	 *Insert item at the given index into the array.
	 *	
	 *@param	TargetArray		The array to insert into
	 *@param	NewItem			The item to insert into the array
	 *@param	Index			The index at which to insert the item into the array
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Insert", CompactNodeTitle = "INSERT", ArrayParm = "TargetArray", ArrayTypeDependentParams = "NewItem", AutoCreateRefTerm = "NewItem"), Category="Utilities|Array")
	static ENGINE_API void Array_Insert(UPARAM(ref) TArray<int32>& TargetArray, const int32& NewItem, int32 Index);


	/* 
	 *Remove item at the given index from the array.
	 *
	 *@param	TargetArray		The array to remove from
	 *@param	IndexToRemove	The index into the array to remove from
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Remove Index", CompactNodeTitle = "REMOVE INDEX", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Remove(UPARAM(ref) TArray<int32>& TargetArray, int32 IndexToRemove);

	/* 
	 *Remove all instances of item from array.
	 *
	 *@param	TargetArray		The array to remove from
	 *@param	Item			The item to remove from the array
	 *@return	True if one or more items were removed
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Remove Item", CompactNodeTitle = "REMOVE", ArrayParm = "TargetArray", ArrayTypeDependentParams = "Item", AutoCreateRefTerm = "Item"), Category="Utilities|Array")
	static ENGINE_API bool Array_RemoveItem(UPARAM(ref) TArray<int32>& TargetArray, const int32 &Item);

	/* 
	 *Clear an array, removes all content
	 *
	 *@param	TargetArray		The array to clear
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Clear", CompactNodeTitle = "CLEAR", Keywords = "empty", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Clear(UPARAM(ref) TArray<int32>& TargetArray);

	/* 
	 *Resize Array to specified size. 
	 *	
	 *@param	TargetArray		The array to resize
	 *@param	Size			The new size of the array
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Resize", CompactNodeTitle = "RESIZE", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Resize(UPARAM(ref) TArray<int32>& TargetArray, int32 Size);
	
	/**
	 * Reverse the elements of an array
	 *
	 *@param	TargetArray		The array to reverse
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (DisplayName = "Reverse", CompactNodeTitle = "REVERSE", ArrayParm = "TargetArray"), Category = "Utilities|Array")
	static ENGINE_API void Array_Reverse(UPARAM(ref) TArray<int32>& TargetArray);

	/* 
	 *Get the number of items in an array
	 *
	 *@param	TargetArray		The array to get the length of
	 *@return	The length of the array
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Length", CompactNodeTitle = "LENGTH", ArrayParm = "TargetArray", Keywords = "num size count", BlueprintThreadSafe), Category="Utilities|Array")
	static ENGINE_API int32 Array_Length(const TArray<int32>& TargetArray);

	/*
	 *Check if the array is empty
	 *
	 *@param	TargetArray		The array to check
	 *@return	A boolean indicating if the array is empty
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Empty", CompactNodeTitle = "IS EMPTY", ArrayParm = "TargetArray", BlueprintThreadSafe), Category = "Utilities|Array")
	static ENGINE_API bool Array_IsEmpty(const TArray<int32>& TargetArray);

	/*
	 *Check if the array has any elements
	 *
	 *@param	TargetArray		The array to check
	 *@return	A boolean indicating if the array has any elements
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Not Empty", CompactNodeTitle = "IS NOT EMPTY", ArrayParm = "TargetArray", BlueprintThreadSafe), Category = "Utilities|Array")
	static ENGINE_API bool Array_IsNotEmpty(const TArray<int32>& TargetArray);


	/* 
	 *Get the last valid index into an array
	 *	
	 *@param	TargetArray		The array to perform the operation on
	 *@return	The last valid index of the array
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Last Index", CompactNodeTitle = "LAST INDEX", ArrayParm = "TargetArray", BlueprintThreadSafe), Category="Utilities|Array")
	static ENGINE_API int32 Array_LastIndex(const TArray<int32>& TargetArray);

	/*
	 *Given an array and an index, returns a copy of the item found at that index
	 *
	 *@param	TargetArray		The array to get an item from
	 *@param	Index			The index in the array to get an item from
	 *@return	A copy of the item stored at the index
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta=(BlueprintInternalUseOnly = "true", DisplayName = "Get", CompactNodeTitle = "GET", ArrayParm = "TargetArray", ArrayTypeDependentParams = "Item", BlueprintThreadSafe), Category="Utilities|Array")
	static ENGINE_API void Array_Get(const TArray<int32>& TargetArray, int32 Index, int32& Item);

	/* 
	 *Given an array and an index, assigns the item to that array element
	 *
	 *@param	TargetArray		The array to perform the operation on
	 *@param	Index			The index to assign the item to
	 *@param	Item			The item to assign to the index of the array
	 *@param	bSizeToFit		If true, the array will expand if Index is greater than the current size of the array
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Set Array Elem", ArrayParm = "TargetArray", ArrayTypeDependentParams = "Item", AutoCreateRefTerm = "Item"), Category="Utilities|Array")
	static ENGINE_API void Array_Set(UPARAM(ref) TArray<int32>& TargetArray, int32 Index, const int32& Item, bool bSizeToFit);

	/*
	 *Swaps the elements at the specified positions in the specified array
	 *If the specified positions are equal, invoking this method leaves the array unchanged
	 *
	 *@param	TargetArray		The array to perform the operation on
	 *@param    FirstIndex      The index of one element to be swapped
	 *@param    SecondIndex     The index of the other element to be swapped
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(DisplayName = "Swap Array Elements", CompactNodeTitle = "SWAP", ArrayParm = "TargetArray"), Category="Utilities|Array")
	static ENGINE_API void Array_Swap(UPARAM(ref) TArray<int32>& TargetArray, int32 FirstIndex, int32 SecondIndex);

	/*  
	 *Finds the index of the first instance of the item within the array
	 *	
	 *@param	TargetArray		The array to search for the item
	 *@param	ItemToFind		The item to look for
	 *@return	The index the item was found at, or -1 if not found
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Find Item", CompactNodeTitle = "FIND", ArrayParm = "TargetArray", ArrayTypeDependentParams = "ItemToFind", AutoCreateRefTerm = "ItemToFind", BlueprintThreadSafe), Category="Utilities|Array")
	static ENGINE_API int32 Array_Find(const TArray<int32>& TargetArray, const int32& ItemToFind);

	/*  
	 *Returns true if the array contains the given item
	 *
	 *@param	TargetArray		The array to search for the item
	 *@param	ItemToFind		The item to look for
	 *@return	True if the item was found within the array
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Contains Item", CompactNodeTitle = "CONTAINS", ArrayParm = "TargetArray", ArrayTypeDependentParams = "ItemToFind", AutoCreateRefTerm = "ItemToFind", BlueprintThreadSafe), Category="Utilities|Array")
	static ENGINE_API bool Array_Contains(const TArray<int32>& TargetArray, const int32& ItemToFind);

	/*  
	 *Filter an array based on a Class derived from Actor.  
	 *	
	 *@param	TargetArray		The array to filter from
	 *@param	FilterClass		The Actor sub-class type that acts as the filter, only objects derived from it will be returned.
	 *@return	An array containing only those objects which are derived from the class specified.
	*/
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Filter Array", DeterminesOutputType = "FilterClass", DynamicOutputParam = "FilteredArray"), Category = "Utilities|Array")
	static ENGINE_API void FilterArray(const TArray<AActor*>& TargetArray, TSubclassOf<class AActor> FilterClass, TArray<AActor*>& FilteredArray);

	/** 
	 * Not exposed to users. Supports setting an array property on an object by name.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, meta=(BlueprintInternalUseOnly = "true", ArrayParm = "Value", ArrayTypeDependentParams="Value"))
	static ENGINE_API void SetArrayPropertyByName(UObject* Object, FName PropertyName, const TArray<int32>& Value);

	/*
	 *Tests if IndexToTest is valid, i.e. greater than or equal to zero, and less than the number of elements in TargetArray.
	 *
	 *@param	TargetArray		Array to use for the IsValidIndex test
	 *@param	IndexToTest		The Index, that we want to test for being valid
	 *@return	True if the Index is Valid, i.e. greater than or equal to zero, and less than the number of elements in TargetArray.
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "Is Valid Index", CompactNodeTitle = "IS VALID INDEX", ArrayParm = "TargetArray", BlueprintThreadSafe), Category = "Utilities|Array")
	static ENGINE_API bool Array_IsValidIndex(const TArray<int32>& TargetArray, int32 IndexToTest);
	
	/**
	 * Gets a random item from specified array
	 * 
	 * @param	TargetArray		The array
	 * @param	OutItem			The random item from this array
	 * @param	OutIndex		The index of random item (will be -1 if array is empty)
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Random Array Item", CompactNodeTitle = "RANDOM", ArrayParm = "TargetArray", ArrayTypeDependentParams = "OutItem"), Category="Utilities|Array")
	static ENGINE_API void Array_Random(const TArray<int32>& TargetArray, int32& OutItem, int32& OutIndex);

	/** 
	 * Gets a random item from specified array (using random stream)
	 * 
	 * @param	TargetArray		The array
	 * @param	RandomStream	The random stream
	 * @param	OutItem			The random item from this array
	 * @param	OutIndex		The index of random item (will be -1 if array is empty)
	 */
	UFUNCTION(BlueprintPure, CustomThunk, meta=(DisplayName = "Random Array Item from Stream", ArrayParm = "TargetArray", ArrayTypeDependentParams = "OutItem"), Category="Utilities|Array")
	static ENGINE_API void Array_RandomFromStream(const TArray<int32>& TargetArray, UPARAM(Ref) FRandomStream& RandomStream, int32& OutItem, int32& OutIndex);

	/**
	 * Sorts an array of strings alphabetically.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (AdvancedDisplay = "bStableSort,SortOrder"))
	static ENGINE_API void SortStringArray(UPARAM(Ref) TArray<FString>& TargetArray, bool bStableSort = false, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sorts an array of FNames.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	bLexicalSort	If the names should be sorted based on the string value of the name rather than the comparison index. This is slower when enabled.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (AdvancedDisplay = "bStableSort,bLexicalSort,SortOrder"))
	static ENGINE_API void SortNameArray(UPARAM(Ref) TArray<FName>& TargetArray, bool bStableSort = false, bool bLexicalSort = true, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sorts an array of bytes.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (AdvancedDisplay = "bStableSort,SortOrder"))
	static ENGINE_API void SortByteArray(UPARAM(Ref) TArray<uint8>& TargetArray, bool bStableSort = false, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sorts an array of integers.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (DisplayName = "Sort Integer Array", AdvancedDisplay = "bStableSort,SortOrder"))
	static ENGINE_API void SortIntArray(UPARAM(Ref) TArray<int32>& TargetArray, bool bStableSort = false, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sorts an array of 64-bit integers.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (DisplayName = "Sort Integer64 Array", AdvancedDisplay = "bStableSort,SortOrder"))
	static ENGINE_API void SortInt64Array(UPARAM(Ref) TArray<int64>& TargetArray, bool bStableSort = false, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sorts an array of doubles.
	 *
	 * @param	TargetArray		The array to sort.
	 * @param	bStableSort		If a stable sort should be used. This preserves the order of identical elements, but is slower.
	 * @param	SortOrder		If the array should be sorted in ascending or descending order.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Array|Sort", meta = (AdvancedDisplay = "bStableSort,SortOrder"))
	static ENGINE_API void SortFloatArray(UPARAM(Ref) TArray<double>& TargetArray, bool bStableSort = false, EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	// Native functions that will be called by the below custom thunk layers, which read off the property address, and call the appropriate native handler
	static ENGINE_API int32 GenericArray_Add(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem);
	static ENGINE_API int32 GenericArray_AddUnique(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem);
	static ENGINE_API void GenericArray_Shuffle(void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API void GenericArray_ShuffleFromStream(void* TargetArray, const FArrayProperty* ArrayProp, FRandomStream* RandomStream);
	static ENGINE_API bool GenericArray_Identical(void* ArrayA, const FArrayProperty* ArrayAProp, void* ArrayB, const FArrayProperty* ArrayBProperty);
	static ENGINE_API void GenericArray_Append(void* TargetArray, const FArrayProperty* TargetArrayProp, void* SourceArray, const FArrayProperty* SourceArrayProperty);
	static ENGINE_API void GenericArray_Insert(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem, int32 Index);
	static ENGINE_API void GenericArray_Remove(void* TargetArray, const FArrayProperty* ArrayProp, int32 IndexToRemove);
	static ENGINE_API bool GenericArray_RemoveItem(void* TargetArray, const FArrayProperty* ArrayProp, const void* Item);
	static ENGINE_API void GenericArray_Clear(void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API void GenericArray_Resize(void* TargetArray, const FArrayProperty* ArrayProp, int32 Size);
	static ENGINE_API void GenericArray_Reverse(void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API int32 GenericArray_Length(const void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API bool GenericArray_IsEmpty(const void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API bool GenericArray_IsNotEmpty(const void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API int32 GenericArray_LastIndex(const void* TargetArray, const FArrayProperty* ArrayProp);
	static ENGINE_API void GenericArray_Get(void* TargetArray, const FArrayProperty* ArrayProp, int32 Index, void* Item);
	static ENGINE_API void GenericArray_Set(void* TargetArray, const FArrayProperty* ArrayProp, int32 Index, const void* NewItem, bool bSizeToFit);
	static ENGINE_API void GenericArray_Swap(const void* TargetArray, const FArrayProperty* ArrayProp, int32 First, int32 Second);
	static ENGINE_API int32 GenericArray_Find(const void* TargetArray, const FArrayProperty* ArrayProperty, const void* ItemToFind);
	static ENGINE_API void GenericArray_SetArrayPropertyByName(UObject* OwnerObject, FName ArrayPropertyName, const void* SrcArrayAddr);
	static ENGINE_API void GenericArray_Random(void* TargetArray, const FArrayProperty* ArrayProp, void* OutItem, int32* OutIndex);
	static ENGINE_API void GenericArray_RandomFromStream(void* TargetArray, const FArrayProperty* ArrayProp, FRandomStream* RandomStream, void* OutItem, int32* OutIndex);
	static ENGINE_API bool GenericArray_IsValidIndex(const void* TargetArray, const FArrayProperty* ArrayProp, int32 IndexToTest);
	
private:
	static constexpr int32 MaxSupportedArraySize = TNumericLimits<int32>::Max();

	static ENGINE_API void GenericArray_HandleBool(const FProperty* Property, void* ItemPtr);

public:
	// Shared warning IDs for use by other container libraries.
	static ENGINE_API const FName ReachedMaximumContainerSizeWarning;

	// Helper function to get the last valid index of the array for error reporting, or 0 if the array is empty
	static int32 GetLastIndex(const FScriptArrayHelper& ArrayHelper)
	{
		const int32 ArraySize = ArrayHelper.Num();
		return (ArraySize > 0) ? ArraySize-1 : 0;
	}

	DECLARE_FUNCTION(execArray_Add)
	{
		Stack.MostRecentProperty = nullptr;
 		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
 
 		// Since NewItem isn't really an int, step the stack manually
 		const FProperty* InnerProp = ArrayProperty->Inner;
 		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
 		void* StorageSpace = FMemory_Alloca(PropertySize);
 		InnerProp->InitializeValue(StorageSpace);
 
 		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
 		Stack.StepCompiledIn<FProperty>(StorageSpace);
 
 		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericArray_Add(ArrayAddr, ArrayProperty, StorageSpace);
		P_NATIVE_END;
		InnerProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execArray_AddUnique)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since NewItem isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);

		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericArray_AddUnique(ArrayAddr, ArrayProperty, StorageSpace);
		P_NATIVE_END;
		InnerProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execArray_Shuffle)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Shuffle(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_ShuffleFromStream)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		FRandomStream* RandomStream = (FRandomStream*)Stack.MostRecentPropertyAddress;

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_ShuffleFromStream(ArrayAddr, ArrayProperty, RandomStream);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Identical)
	{
		// Retrieve the first array
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayAProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayAProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		// Retrieve the second array
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayBAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayBProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayBProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_Identical(ArrayAAddr, ArrayAProperty, ArrayBAddr, ArrayBProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Append)
	{
		// Retrieve the target array
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* TargetArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!TargetArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		// Retrieve the source array
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* SourceArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* SourceArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!SourceArrayProperty )
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Append(TargetArrayAddr, TargetArrayProperty, SourceArrayAddr, SourceArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Insert)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		// Since NewItem isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);

		P_GET_PROPERTY(FIntProperty, Index);
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Insert(ArrayAddr, ArrayProperty, StorageSpace, Index);
		P_NATIVE_END;
		InnerProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execArray_Remove)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_GET_PROPERTY(FIntProperty, Index);
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Remove(ArrayAddr, ArrayProperty, Index);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_RemoveItem)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		// Since Item isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemPtr = StorageSpace;

		P_FINISH;

		// Bools need to be processed internally by the property so that C++ bool value is properly set.
		GenericArray_HandleBool(InnerProp, ItemPtr);
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_RemoveItem(ArrayAddr, ArrayProperty, ItemPtr);
		P_NATIVE_END;

		InnerProp->DestroyValue(StorageSpace);
	}
	
	DECLARE_FUNCTION(execArray_Clear)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Clear(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Resize)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_GET_PROPERTY(FIntProperty, Size);
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Resize(ArrayAddr, ArrayProperty, Size);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_Reverse)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Reverse(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Length)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericArray_Length(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_IsEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(nullptr);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_IsEmpty(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_IsNotEmpty)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(nullptr);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_IsNotEmpty(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_LastIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_FINISH;
		P_NATIVE_BEGIN;
		*(int32*)RESULT_PARAM = GenericArray_LastIndex(ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Get)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_GET_PROPERTY(FIntProperty, Index);

		// Since Item isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		const FFieldClass* InnerPropClass = InnerProp->GetClass();
		const FFieldClass* MostRecentPropClass = Stack.MostRecentProperty->GetClass();
		void* ItemPtr;
		// If the destination and the inner type are identical in size and their field classes derive from one another, then permit the writing out of the array element to the destination memory
		if (Stack.MostRecentPropertyAddress != NULL && (PropertySize == Stack.MostRecentProperty->GetElementSize()*Stack.MostRecentProperty->ArrayDim) &&
			(MostRecentPropClass->IsChildOf(InnerPropClass) || InnerPropClass->IsChildOf(MostRecentPropClass)))
		{
			ItemPtr = Stack.MostRecentPropertyAddress;
		}
		else
		{
			ItemPtr = StorageSpace;
		}

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Get(ArrayAddr, ArrayProperty, Index, ItemPtr);
		P_NATIVE_END;
		InnerProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execArray_Set)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		P_GET_PROPERTY(FIntProperty, Index);

		// Since NewItem isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);

		P_GET_UBOOL(bSizeToFit);

		P_FINISH;

		P_NATIVE_BEGIN;
		GenericArray_Set(ArrayAddr, ArrayProperty, Index, StorageSpace, bSizeToFit);
		P_NATIVE_END;
		InnerProp->DestroyValue(StorageSpace);
	}


	DECLARE_FUNCTION(execArray_Swap)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(nullptr);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_GET_PROPERTY(FIntProperty, First);
		P_GET_PROPERTY(FIntProperty, Second);

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Swap(ArrayAddr, ArrayProperty, First, Second);
		P_NATIVE_END;
	}

	DECLARE_FUNCTION(execArray_Find)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		// Since ItemToFind isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemToFindPtr = StorageSpace;

		P_FINISH;

		// Bools need to be processed internally by the property so that C++ bool value is properly set.
		GenericArray_HandleBool(InnerProp, ItemToFindPtr);

		P_NATIVE_BEGIN;
		// Perform the search
		*(int32*)RESULT_PARAM = GenericArray_Find(ArrayAddr, ArrayProperty, ItemToFindPtr);
		P_NATIVE_END;

		InnerProp->DestroyValue(StorageSpace);
	}
	
	DECLARE_FUNCTION(execArray_Contains)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}
		// Since ItemToFind isn't really an int, step the stack manually
		const FProperty* InnerProp = ArrayProperty->Inner;
		const int32 PropertySize = InnerProp->GetElementSize() * InnerProp->ArrayDim;
		void* StorageSpace = FMemory_Alloca(PropertySize);
		InnerProp->InitializeValue(StorageSpace);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.StepCompiledIn<FProperty>(StorageSpace);
		void* ItemToFindPtr = StorageSpace;

		P_FINISH;

		// Bools need to be processed internally by the property so that C++ bool value is properly set.
		GenericArray_HandleBool(InnerProp, ItemToFindPtr);

		// Perform the search
		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_Find(ArrayAddr, ArrayProperty, ItemToFindPtr) >= 0;
		P_NATIVE_END;

		InnerProp->DestroyValue(StorageSpace);
	}

	DECLARE_FUNCTION(execSetArrayPropertyByName)
	{
		P_GET_OBJECT(UObject, OwnerObject);
		P_GET_PROPERTY(FNameProperty, ArrayPropertyName);

		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* SrcArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_FINISH;

		P_NATIVE_BEGIN;
		MARK_PROPERTY_DIRTY(Stack.Object, Stack.MostRecentProperty);
		GenericArray_SetArrayPropertyByName(OwnerObject, ArrayPropertyName, SrcArrayAddr);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_IsValidIndex)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(NULL);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		P_GET_PROPERTY(FIntProperty, IndexToTest);

		P_FINISH;

		P_NATIVE_BEGIN;
		*(bool*)RESULT_PARAM = GenericArray_IsValidIndex(ArrayAddr, ArrayProperty, IndexToTest);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_Random)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(nullptr);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		void* Result = Stack.MostRecentPropertyAddress;

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		int32* OutIndex = (int32*)Stack.MostRecentPropertyAddress;

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_Random(ArrayAddr, ArrayProperty, Result, OutIndex);
		P_NATIVE_END;
	}
	
	DECLARE_FUNCTION(execArray_RandomFromStream)
	{
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FArrayProperty>(nullptr);
		void* ArrayAddr = Stack.MostRecentPropertyAddress;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);
		if (!ArrayProperty)
		{
			Stack.bArrayContextFailed = true;
			return;
		}

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		FRandomStream* RandomStream = (FRandomStream*)Stack.MostRecentPropertyAddress;

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		void* Result = Stack.MostRecentPropertyAddress;

		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
		int32* OutIndex = (int32*)Stack.MostRecentPropertyAddress;

		P_FINISH;
		P_NATIVE_BEGIN;
		GenericArray_RandomFromStream(ArrayAddr, ArrayProperty, RandomStream, Result, OutIndex);
		P_NATIVE_END;
	}
};
