// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetGuidLibrary.generated.h"


UCLASS(meta=(ScriptName="GuidLibrary"), MinimalAPI)
class UKismetGuidLibrary
	: public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Equal (Guid)", CompactNodeTitle="==", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API bool EqualEqual_GuidGuid( const FGuid& A, const FGuid& B );
	
	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Not Equal (Guid)", CompactNodeTitle="!=", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API bool NotEqual_GuidGuid( const FGuid& A, const FGuid& B );

	/** Checks whether the given GUID is valid */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Valid Guid", CompactNodeTitle="isValid?", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API bool IsValid_Guid( const FGuid& InGuid );

	/** Invalidates the given GUID */
	UFUNCTION(BlueprintCallable, meta=(DisplayName="Invalidate Guid", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API void Invalidate_Guid( UPARAM(ref) FGuid& InGuid );

	/** Returns a new unique GUID */
	UFUNCTION(BlueprintPure, Category="Guid")
	static ENGINE_API FGuid NewGuid();
	
	/** Converts a GUID value to a string, in the form 'A-B-C-D' */
	UFUNCTION(BlueprintPure, meta=(DisplayName="To String (Guid)", CompactNodeTitle="->", ScriptMethod="ToString", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API FString Conv_GuidToString( const FGuid& InGuid );

	/** Converts a String of format EGuidFormats to a Guid. Returns Guid OutGuid, Returns bool Success */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Parse String to Guid", BlueprintThreadSafe), Category="Guid")
	static ENGINE_API void Parse_StringToGuid( const FString& GuidString, FGuid& OutGuid, bool& Success );
};
