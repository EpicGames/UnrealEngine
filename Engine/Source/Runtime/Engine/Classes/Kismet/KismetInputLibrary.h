// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Framework/Commands/InputChord.h"
#include "Input/Events.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetInputLibrary.generated.h"

class FModifierKeysState;

UENUM(BlueprintType)
enum class ESlateGesture : uint8
{
	None,
	Scroll,
	Magnify,
	Swipe,
	Rotate,
	LongPress,
	Pan,
	Tap,
	Flick
};

/** A structure which captures the application's modifier key states (shift, alt, ctrl, etc.) */
USTRUCT(BlueprintType, DisplayName="Modifier Keys State")
struct FSlateModifierKeysState
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint8 ModifierKeysStateMask = 0;

	FSlateModifierKeysState() {}
	FSlateModifierKeysState(const FModifierKeysState& InModifierKeysState);	
};

UCLASS(meta=(ScriptName="InputLibrary"), MinimalAPI)
class UKismetInputLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Calibrate the tilt for the input device */
	UFUNCTION(BlueprintCallable, Category="Input|MotionTracking")
	static ENGINE_API void CalibrateTilt();

	/**
	 * Test if the input key are equal (A == B)
	 * @param A - The key to compare against
	 * @param B - The key to compare
	 * Returns true if the key are equal, false otherwise
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Key)", CompactNodeTitle = "=="), Category="Input|Key")
	static ENGINE_API bool EqualEqual_KeyKey(FKey A, FKey B);

	/**
	* Test if the input chords are equal (A == B)
	* @param A - The chord to compare against
	* @param B - The chord to compare
	* Returns true if the chords are equal, false otherwise
	*/
	UFUNCTION( BlueprintPure, meta = ( DisplayName = "Equal (InputChord)", CompactNodeTitle = "==" ), Category = "Input|Key" )
	static ENGINE_API bool EqualEqual_InputChordInputChord( FInputChord A, FInputChord B );

	/**
	 * Returns true if the key is a modifier key: Ctrl, Command, Alt, Shift
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Modifier Key"), Category="Input|Key")
	static ENGINE_API bool Key_IsModifierKey(const FKey& Key);

	/**
	 * Returns true if the key is a gamepad button
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Gamepad Key"), Category="Input|Key")
	static ENGINE_API bool Key_IsGamepadKey(const FKey& Key);

	/**
	 * Returns true if the key is a mouse button
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Mouse Button"), Category="Input|Key")
	static ENGINE_API bool Key_IsMouseButton(const FKey& Key);

	/**
	 * Returns true if the key is a keyboard button
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Keyboard Key"), Category="Input|Key")
	static ENGINE_API bool Key_IsKeyboardKey(const FKey& Key);

	/**
	 * Returns true if the key is a 1D (float) axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Axis 1D"), Category="Input|Key")
	static ENGINE_API bool Key_IsAxis1D(const FKey& Key);

	/**
	 * Returns true if the key is a 2D (vector) axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Axis 2D"), Category="Input|Key")
	static ENGINE_API bool Key_IsAxis2D(const FKey& Key);

	/**
	 * Returns true if the key is a 3D (vector) axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Axis 3D"), Category="Input|Key")
	static ENGINE_API bool Key_IsAxis3D(const FKey& Key);

	/**
	 * Returns true if the key is a 1D axis emulating a digital button press.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Button Axis"), Category = "Input|Key")
	static ENGINE_API bool Key_IsButtonAxis(const FKey& Key);

	/**
	 * Returns true if the key is an analog axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Analog"), Category = "Input|Key")
	static ENGINE_API bool Key_IsAnalog(const FKey& Key);

	/**
	 * Returns true if the key is a digital button press
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Digital"), Category = "Input|Key")
	static ENGINE_API bool Key_IsDigital(const FKey& Key);

	/**
	 * Returns true if this is a valid key.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Valid Key"), Category = "Input|Key")
	static ENGINE_API bool Key_IsValid(const FKey& Key);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Key Navigation Action", DeprecatedFunction, DeprecationMessage = "Use Get Key Event Navigation Action instead"), Category = "Input|Key")
	static ENGINE_API EUINavigationAction Key_GetNavigationAction(const FKey& InKey);

	/** Returns the navigation action corresponding to this key, or Invalid if not found */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Key Event Navigation Action"), Category = "Input|KeyEvent")
	static ENGINE_API EUINavigationAction Key_GetNavigationActionFromKey(const FKeyEvent& InKeyEvent);

	/** Returns the navigation action corresponding to this key, or Invalid if not found */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Key Event Navigation Direction"), Category = "Input|KeyEvent")
	static ENGINE_API EUINavigation Key_GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent);

	/** Returns the navigation action corresponding to this key, or Invalid if not found */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Analog Event Navigation Direction"), Category = "Input|AnalogEvent")
	static ENGINE_API EUINavigation Key_GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent);

	/**
	 * Returns the display name of the key.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Key Display Name", BlueprintAutocast), Category="Input|Key")
	static ENGINE_API FText Key_GetDisplayName(const FKey& Key, bool bLongDisplayName = true);

	/**
	 * Returns whether or not this character is an auto-repeated keystroke
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Repeat" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsRepeat(const FInputEvent& Input);

	/**
	 * Returns true if either shift key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Shift Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsShiftDown(const FInputEvent& Input);

	/**
	 * Returns true if left shift key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Left Shift Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsLeftShiftDown(const FInputEvent& Input);

	/**
	 * Returns true if right shift key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Right Shift Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsRightShiftDown(const FInputEvent& Input);

	/**
	 * Returns true if either control key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Control Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsControlDown(const FInputEvent& Input);

	/**
	 * Returns true if left control key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Left Control Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsLeftControlDown(const FInputEvent& Input);

	/**
	 * Returns true if left control key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Right Control Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsRightControlDown(const FInputEvent& Input);

	/**
	 * Returns true if either alt key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Alt Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsAltDown(const FInputEvent& Input);

	/**
	 * Returns true if left alt key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Left Alt Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsLeftAltDown(const FInputEvent& Input);

	/**
	 * Returns true if right alt key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Right Alt Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsRightAltDown(const FInputEvent& Input);

	/**
	 * Returns true if either command key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Command Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsCommandDown(const FInputEvent& Input);

	/**
	 * Returns true if left command key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Left Command Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsLeftCommandDown(const FInputEvent& Input);

	/**
	 * Returns true if right command key was down when this event occurred
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Right Command Down" ), Category="Input|InputEvent")
	static ENGINE_API bool InputEvent_IsRightCommandDown(const FInputEvent& Input);

	/**
	 * Returns true if either shift key was down when the key state was captured 
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Shift Down" ), Category="Input|ModifierKeys")
	static ENGINE_API bool ModifierKeysState_IsShiftDown(const FSlateModifierKeysState& KeysState);

	/**
	 * Returns true if either control key was down when the key state was captured 
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Control Down" ), Category="Input|ModifierKeys")
	static ENGINE_API bool ModifierKeysState_IsControlDown(const FSlateModifierKeysState& KeysState);

	/**
	 * Returns true if either alt key was down when the key state was captured 
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Alt Down" ), Category="Input|ModifierKeys")
	static ENGINE_API bool ModifierKeysState_IsAltDown(const FSlateModifierKeysState& KeysState);

	/**
	 * Returns true if either command key was down when the key state was captured 
	 */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Command Down" ), Category="Input|ModifierKeys")
	static ENGINE_API bool ModifierKeysState_IsCommandDown(const FSlateModifierKeysState& KeysState);

	/** Returns a snapshot of the cached modifier-keys state for the application. */
	UFUNCTION(BlueprintPure, Category = "Input|ModifierKeys")
	static ENGINE_API FSlateModifierKeysState GetModifierKeysState();

	/** @return The display name of the input chord */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Input Chord Display Name"), Category = "Input|Key")
	static ENGINE_API FText InputChord_GetDisplayName(const FInputChord& Key);

	/**
	 * Returns the key for this event.
	 *
	 * @return  Key name
	 */
	UFUNCTION(BlueprintPure, Category="Input|KeyEvent")
	static ENGINE_API FKey GetKey(const FKeyEvent& Input);

	UFUNCTION(BlueprintPure, Category = "Input|KeyEvent")
	static ENGINE_API int32 GetUserIndex(const FKeyEvent& Input);

	UFUNCTION(BlueprintPure, Category = "Input|InputEvent")
	static ENGINE_API float GetAnalogValue(const FAnalogInputEvent& Input);

	/** Returns The position of the cursor in screen space */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Screen Space Position" ), Category="Input|PointerEvent")
	static ENGINE_API FVector2D PointerEvent_GetScreenSpacePosition(const FPointerEvent& Input);

	/** Returns the position of the cursor in screen space last time we handled an input event */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Last Screen Space Position" ), Category="Input|PointerEvent")
	static ENGINE_API FVector2D PointerEvent_GetLastScreenSpacePosition(const FPointerEvent& Input);

	/** Returns the distance the mouse traveled since the last event was handled. */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Cursor Delta" ), Category="Input|PointerEvent")
	static ENGINE_API FVector2D PointerEvent_GetCursorDelta(const FPointerEvent& Input);

	/** Mouse buttons that are currently pressed */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Mouse Button Down" ), Category="Input|PointerEvent")
	static ENGINE_API bool PointerEvent_IsMouseButtonDown(const FPointerEvent& Input, FKey MouseButton);

	/** Mouse button that caused this event to be raised (possibly FKey::Invalid) */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Effecting Button" ), Category="Input|PointerEvent")
	static ENGINE_API FKey PointerEvent_GetEffectingButton(const FPointerEvent& Input);

	/** How much did the mouse wheel turn since the last mouse event */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Wheel Delta" ), Category="Input|PointerEvent")
	static ENGINE_API float PointerEvent_GetWheelDelta(const FPointerEvent& Input);

	/** Returns the index of the user that caused the event */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get User Index" ), Category="Input|PointerEvent")
	static ENGINE_API int32 PointerEvent_GetUserIndex(const FPointerEvent& Input);

	/** Returns the unique identifier of the pointer (e.g., finger index) */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Pointer Index" ), Category="Input|PointerEvent")
	static ENGINE_API int32 PointerEvent_GetPointerIndex(const FPointerEvent& Input);

	/** Returns the index of the touch pad that generated this event (for platforms with multiple touch pads per user) */
	UE_DEPRECATED(5.8, "TouchpadIndex is was never fully implemented and has been retired.")
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Touchpad Index" ), Category="Input|PointerEvent")
	static ENGINE_API int32 PointerEvent_GetTouchpadIndex(const FPointerEvent& Input);

	/** Returns true if this event a result from a touch (as opposed to a mouse) */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Is Touch Event" ), Category="Input|PointerEvent")
	static ENGINE_API bool PointerEvent_IsTouchEvent(const FPointerEvent& Input);

	/** Returns the type of touch gesture */
	UFUNCTION(BlueprintPure, Category="Input|PointerEvent")
	static ENGINE_API ESlateGesture PointerEvent_GetGestureType(const FPointerEvent& Input);

	/** Returns the change in gesture value since the last gesture event of the same type. */
	UFUNCTION(BlueprintPure, meta=( DisplayName = "Get Gesture Delta" ), Category="Input|PointerEvent")
	static ENGINE_API FVector2D PointerEvent_GetGestureDelta(const FPointerEvent& Input);
};
