// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetInternationalizationLibrary.generated.h"

enum class ELocalizationLoadFlags : uint8;
enum class ELocalizedTextSourceCategory : uint8;

UCLASS(meta=(BlueprintThreadSafe, ScriptName="InternationalizationLibrary"), MinimalAPI)
class UKismetInternationalizationLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Set the current culture.
	 * @note This function is a sledgehammer, and will set both the language and locale, as well as clear out any asset group cultures that may be set.
	 * @param Culture The culture to set, as an IETF language tag (eg, "zh-Hans-CN").
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 * @return True if the culture was set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API bool SetCurrentCulture(const FString& Culture, const bool SaveToConfig = false);

	/**
	 * Get the current culture as an IETF language tag:
	 *  - A two-letter ISO 639-1 language code (eg, "zh").
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans").
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN").
	 * @note This function exists for legacy API parity with SetCurrentCulture and is equivalent to GetCurrentLanguage.
	 * @return The culture as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetCurrentCulture();

	/**
	 * Set *only* the current language (for localization).
	 * @note Unless you're doing something advanced, you likely want SetCurrentLanguageAndLocale or SetCurrentCulture instead.
	 * @param Culture The language to set, as an IETF language tag (eg, "zh-Hans-CN").
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 * @return True if the language was set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API bool SetCurrentLanguage(const FString& Culture, const bool SaveToConfig = false);

	/**
	 * Get the current language (for localization) as an IETF language tag:
	 *  - A two-letter ISO 639-1 language code (eg, "zh").
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans").
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN").
	 * @return The language as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetCurrentLanguage();

	/**
	 * Set *only* the current locale (for internationalization).
	 * @note Unless you're doing something advanced, you likely want SetCurrentLanguageAndLocale or SetCurrentCulture instead.
	 * @param Culture The locale to set, as an IETF language tag (eg, "zh-Hans-CN").
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 * @return True if the locale was set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API bool SetCurrentLocale(const FString& Culture, const bool SaveToConfig = false);

	/**
	 * Get the current locale (for internationalization) as an IETF language tag:
	 *  - A two-letter ISO 639-1 language code (eg, "zh").
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans").
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN").
	 * @return The locale as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetCurrentLocale();

	/**
	 * Set the current language (for localization) and locale (for internationalization).
	 * @param Culture The language and locale to set, as an IETF language tag (eg, "zh-Hans-CN").
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 * @return True if the language and locale were set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API bool SetCurrentLanguageAndLocale(const FString& Culture, const bool SaveToConfig = false);

	/**
	 * Set the given asset group category culture from an IETF language tag (eg, "zh-Hans-CN").
	 * @param AssetGroup The asset group to set the culture for.
	 * @param Culture The culture to set, as an IETF language tag (eg, "zh-Hans-CN").
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 * @return True if the culture was set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="2"))
	static ENGINE_API bool SetCurrentAssetGroupCulture(const FName AssetGroup, const FString& Culture, const bool SaveToConfig = false);

	/**
	 * Get the given asset group category culture.
	 * @note Returns the current language if the group category doesn't have a culture override.
	 * @param AssetGroup The asset group to get the culture for.
	 * @return The culture as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetCurrentAssetGroupCulture(const FName AssetGroup);

	/**
	 * Clear the given asset group category culture.
	 * @param AssetGroup The asset group to clear the culture for.
	 * @param SaveToConfig If true, save the new setting to the users' "GameUserSettings" config so that it persists after a reload.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API void ClearCurrentAssetGroupCulture(const FName AssetGroup, const bool SaveToConfig = false);

	/**
	 * Get the native culture for the given localization category.
	 * @param Category The localization category to query.
	 * @return The culture as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetNativeCulture(const ELocalizedTextSourceCategory TextCategory);

	/**
	 * Get the list of cultures that have localization data available for them.
	 * @param IncludeGame Check for localized game resources.
	 * @param IncludeEngine Check for localized engine resources.
	 * @param IncludeEditor Check for localized editor resources.
	 * @param IncludeAdditional Check for localized additional (eg, plugin) resources.
	 * @return The list of cultures as IETF language tags (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API TArray<FString> GetLocalizedCultures(const bool IncludeGame = true, const bool IncludeEngine = false, const bool IncludeEditor = false, const bool IncludeAdditional = false);

	/**
	 * Get the list of cultures that have localization data available for them.
	 * @param LoadFlags Controls which resource groups should be checked.
	 * @return The list of cultures as IETF language tags (eg, "zh-Hans-CN").
	 */
	static ENGINE_API TArray<FString> GetLocalizedCultures(const ELocalizationLoadFlags LoadFlags);

	/**
	 * Get the language that we last loaded localization data for.
	 * eg) If the available languages were [en, fr, de] and the current language was "en-US", this function would return "en".
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetLocalizedLanguage();

	/**
	 * Given a list of available cultures, try and find the most suitable culture from the list based on culture prioritization.
	 *   eg) If your list was [en, fr, de] and the given culture was "en-US", this function would return "en".
	 *   eg) If your list was [zh, ar, pl] and the given culture was "en-US", this function would return the fallback culture.
	 * @param AvailableCultures List of available cultures to filter against (see GetLocalizedCultures).
	 * @param CultureToMatch Culture to try and match (see GetCurrentLanguage).
	 * @param FallbackCulture The culture to return if there is no suitable match in the list (typically your native culture, see GetNativeCulture).
	 * @return The culture as an IETF language tag (eg, "zh-Hans-CN").
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API FString GetSuitableCulture(const TArray<FString>& AvailableCultures, const FString& CultureToMatch, const FString& FallbackCulture = TEXT("en"));

	/**
	 * Get the display name of the given culture.
	 * @param Culture The culture to get the display name of, as an IETF language tag (eg, "zh-Hans-CN")
	 * @param Localized True to get the localized display name (the name of the culture in its own language), or False to get the display name in the current language.
	 * @return The display name of the culture, or the given culture code on failure.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization", meta=(AdvancedDisplay="1"))
	static ENGINE_API FString GetCultureDisplayName(const FString& Culture, const bool Localized = true);

	/**
	 * Returns if the given culture reads left to right
	 * @param Culture The culture to get the display name of, as an IETF language tag (eg, "zh-Hans-CN")
	 * @return True if the given culture reads left to right.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Internationalization")
	static ENGINE_API bool IsCultureRightToLeft(const FString& Culture);
};
