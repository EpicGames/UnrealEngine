// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetMaterialLibrary.generated.h"

class UMaterialParameterCollection;

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMIDCreationFlags : uint8
{
	None					UMETA(DisplayName = "None"),
	Transient = 1 << 0		UMETA(DisplayName = "Transient")
};
ENUM_CLASS_FLAGS(EMIDCreationFlags)

UCLASS(MinimalAPI, meta=(ScriptName="MaterialLibrary"))
class UKismetMaterialLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Sets a scalar parameter value on the material collection instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="SetFloatParameterValue", WorldContext="WorldContextObject", MaterialParameterCollectionFunction = "true"))
	static ENGINE_API void SetScalarParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName, float ParameterValue);

	/** Sets a vector parameter value on the material collection instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="SetColorParameterValue", WorldContext="WorldContextObject", MaterialParameterCollectionFunction = "true"))
	static ENGINE_API void SetVectorParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName, const FLinearColor& ParameterValue);

	/** Gets a scalar parameter value from the material collection instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetFloatParameterValue", WorldContext="WorldContextObject", MaterialParameterCollectionFunction = "true"))
	static ENGINE_API float GetScalarParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName);

	/** Gets a vector parameter value from the material collection instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetColorParameterValue", WorldContext="WorldContextObject", MaterialParameterCollectionFunction = "true"))
	static ENGINE_API FLinearColor GetVectorParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName);

	/** Creates a Dynamic Material Instance which you can modify during gameplay. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(WorldContext="WorldContextObject", MaterialParameterCollectionFunction = "true"))
	static ENGINE_API class UMaterialInstanceDynamic* CreateDynamicMaterialInstance(UObject* WorldContextObject, class UMaterialInterface* Parent, FName OptionalName = NAME_None, EMIDCreationFlags CreationFlags = EMIDCreationFlags::None);
};
