// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "Math/RandomStream.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "UObject/Stack.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/QualifiedFrameTime.h"
#include "Engine/NetSerialization.h"

#include "KismetMathLibrary.generated.h"

// Whether to inline functions at all
#define KISMET_MATH_INLINE_ENABLED	(!UE_BUILD_DEBUG)

/** Provides different easing functions that can be used in blueprints */
UENUM(BlueprintType)
namespace EEasingFunc
{
	enum Type : int
	{
		/** Simple linear interpolation. */
		Linear,

		/** Simple step interpolation. */
		Step,

		/** Sinusoidal in interpolation. */
		SinusoidalIn,

		/** Sinusoidal out interpolation. */
		SinusoidalOut,

		/** Sinusoidal in/out interpolation. */
		SinusoidalInOut,

		/** Smoothly accelerates, but does not decelerate into the target.  Ease amount controlled by BlendExp. */
		EaseIn,

		/** Immediately accelerates, but smoothly decelerates into the target.  Ease amount controlled by BlendExp. */
		EaseOut,

		/** Smoothly accelerates and decelerates.  Ease amount controlled by BlendExp. */
		EaseInOut,

		/** Easing in using an exponential */
		ExpoIn,

		/** Easing out using an exponential */
		ExpoOut,

		/** Easing in/out using an exponential method */
		ExpoInOut,

		/** Easing is based on a half circle. */
		CircularIn,

		/** Easing is based on an inverted half circle. */
		CircularOut,

		/** Easing is based on two half circles. */
		CircularInOut,

	};
}

/** Different methods for interpolating rotation between transforms */
UENUM(BlueprintType)
namespace ELerpInterpolationMode
{
	enum Type : int
	{
		/** Shortest Path or Quaternion interpolation for the rotation. */
		QuatInterp,

		/** Rotor or Euler Angle interpolation. */
		EulerInterp,

		/** Dual quaternion interpolation, follows helix or screw-motion path between keyframes.   */
		DualQuatInterp
	};
}

/** Possible columns for an FMatrix */
UENUM(BlueprintType)
namespace EMatrixColumns
{
	enum Type : int
	{
		/** First Column. */
		First,

		/** Second Column. */
		Second,

		/** Third Column. */
		Third,

		/** Fourth Column. */
		Fourth
	};
}

USTRUCT(BlueprintType)
struct FFloatSpringState
{
	GENERATED_BODY()

	float PrevTarget;
	float Velocity;
	bool  bPrevTargetValid;

	FFloatSpringState()
	: PrevTarget(0.f)
	, Velocity(0.f)
	, bPrevTargetValid(false)
	{
	}

	void Reset()
	{
		PrevTarget = Velocity = 0.f;
		bPrevTargetValid = false;
	}
};

USTRUCT(BlueprintType)
struct FVectorSpringState
{
	GENERATED_BODY()

	FVector PrevTarget;
	FVector Velocity;
	bool    bPrevTargetValid;

	FVectorSpringState()
	: PrevTarget(FVector::ZeroVector)
	, Velocity(FVector::ZeroVector)
	, bPrevTargetValid(false)
	{
	}

	void Reset()
	{
		PrevTarget = Velocity = FVector::ZeroVector;
		bPrevTargetValid = false;
	}
};

USTRUCT(BlueprintType)
struct FQuaternionSpringState
{
	GENERATED_BODY()

	FQuat   PrevTarget;
	FVector AngularVelocity;
	bool    bPrevTargetValid;

	FQuaternionSpringState()
	: PrevTarget(FQuat::Identity)
	, AngularVelocity(FVector::ZeroVector)
	, bPrevTargetValid(false)
	{
	}

	void Reset()
	{
		PrevTarget = FQuat::Identity;
		AngularVelocity = FVector::ZeroVector;
		bPrevTargetValid = false;
	}
};

#if KISMET_MATH_INLINE_ENABLED
#define UE_INL_API
#else
#define UE_INL_API ENGINE_API
#endif

struct FRuntimeFloatCurve;

UCLASS(meta=(BlueprintThreadSafe, ScriptName = "MathLibrary"), MinimalAPI)
class UKismetMathLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	//
	// Boolean functions.
	//
	
	/** Returns a uniformly distributed random bool*/
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API bool RandomBool();

	/** 
	 * Get a random chance with the specified weight. Range of weight is 0.0 - 1.0 E.g.,
	 *		Weight = .6 return value = True 60% of the time
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(Weight = "0.5", NotBlueprintThreadSafe))
	static ENGINE_API bool RandomBoolWithWeight(float Weight);

	/** 
	 * Get a random chance with the specified weight. Range of weight is 0.0 - 1.0 E.g.,
	*		Weight = .6 return value = True 60% of the time
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(Weight = "0.5", ScriptMethod = "RandomBoolWithWeight", ScriptMethodMutable))
	static ENGINE_API bool RandomBoolWithWeightFromStream(const FRandomStream& RandomStream, float Weight);

	/** Returns the logical complement of the Boolean value (NOT A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "NOT Boolean", CompactNodeTitle = "NOT", Keywords = "! not negate"), Category="Math|Boolean")
	static UE_INL_API bool Not_PreBool(bool A);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Boolean)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Math|Boolean")
	static UE_INL_API bool EqualEqual_BoolBool(bool A, bool B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Boolean)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Math|Boolean")
	static UE_INL_API bool NotEqual_BoolBool(bool A, bool B);

	/** Returns the logical AND of two values (A AND B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "AND Boolean", CompactNodeTitle = "AND", Keywords = "& and", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Boolean")
	static UE_INL_API bool BooleanAND(bool A, bool B);

	/** Returns the logical NAND of two values (A AND B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "NAND Boolean", CompactNodeTitle = "NAND", Keywords = "!& nand", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Boolean")
	static UE_INL_API bool BooleanNAND(bool A, bool B);

	/** Returns the logical OR of two values (A OR B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "OR Boolean", CompactNodeTitle = "OR", Keywords = "| or", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Boolean")
	static UE_INL_API bool BooleanOR(bool A, bool B);
		
	/** Returns the logical eXclusive OR of two values (A XOR B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "XOR Boolean", CompactNodeTitle = "XOR", Keywords = "^ xor"), Category="Math|Boolean")
	static UE_INL_API bool BooleanXOR(bool A, bool B);

	/** Returns the logical Not OR of two values (A NOR B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "NOR Boolean", CompactNodeTitle = "NOR", Keywords = "!^ nor"), Category="Math|Boolean")
	static UE_INL_API bool BooleanNOR(bool A, bool B);

	//
	// Byte functions.
	//

	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte * Byte", CompactNodeTitle = "*", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Byte")
	static UE_INL_API uint8 Multiply_ByteByte(uint8 A, uint8 B);

	/** Division (A / B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte / Byte", CompactNodeTitle = "/", Keywords = "/ divide division"), Category="Math|Byte")
	static UE_INL_API uint8 Divide_ByteByte(uint8 A, uint8 B = 1);

	/** Modulo (A % B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "% (Byte)", CompactNodeTitle = "%", Keywords = "% modulus"), Category="Math|Byte")
	static UE_INL_API uint8 Percent_ByteByte(uint8 A, uint8 B = 1);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte + Byte", CompactNodeTitle = "+", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Byte")
	static UE_INL_API uint8 Add_ByteByte(uint8 A, uint8 B = 1);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte - Byte", CompactNodeTitle = "-", Keywords = "- subtract minus"), Category="Math|Byte")
	static UE_INL_API uint8 Subtract_ByteByte(uint8 A, uint8 B = 1);

	/** Returns the minimum value of A and B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Min (Byte)", CompactNodeTitle = "MIN", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Byte")
	static UE_INL_API uint8 BMin(uint8 A, uint8 B);

	/** Returns the maximum value of A and B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Max (Byte)", CompactNodeTitle = "MAX", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Byte")
	static UE_INL_API uint8 BMax(uint8 A, uint8 B);
	
	/** Returns true if A is less than B (A < B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte < Byte", CompactNodeTitle = "<", Keywords = "< less"), Category="Math|Byte")
	static UE_INL_API bool Less_ByteByte(uint8 A, uint8 B);

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte > Byte", CompactNodeTitle = ">", Keywords = "> greater"), Category="Math|Byte")
	static UE_INL_API bool Greater_ByteByte(uint8 A, uint8 B);

	/** Returns true if A is less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte <= Byte", CompactNodeTitle = "<=", Keywords = "<= less"), Category="Math|Byte")
	static UE_INL_API bool LessEqual_ByteByte(uint8 A, uint8 B);

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Byte >= Byte", CompactNodeTitle = ">=", Keywords = ">= greater"), Category="Math|Byte")
	static UE_INL_API bool GreaterEqual_ByteByte(uint8 A, uint8 B);

	/** Returns true if A is equal to B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Byte)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Math|Byte")
	static UE_INL_API bool EqualEqual_ByteByte(uint8 A, uint8 B);

	/** Returns true if A is not equal to B (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Byte)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Math|Byte")
	static UE_INL_API bool NotEqual_ByteByte(uint8 A, uint8 B);

	//
	// Integer functions.
	//

	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "int * int", CompactNodeTitle = "*", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 Multiply_IntInt(int32 A, int32 B);

	/** Division (A / B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "int / int", CompactNodeTitle = "/", Keywords = "/ divide division"), Category="Math|Integer")
	static UE_INL_API int32 Divide_IntInt(int32 A, int32 B = 1);

	/** Modulo (A % B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "% (Integer)", CompactNodeTitle = "%", Keywords = "% modulus"), Category="Math|Integer")
	static UE_INL_API int32 Percent_IntInt(int32 A, int32 B = 1);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "int + int", CompactNodeTitle = "+", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 Add_IntInt(int32 A, int32 B = 1);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "int - int", CompactNodeTitle = "-", Keywords = "- subtract minus"), Category="Math|Integer")
	static UE_INL_API int32 Subtract_IntInt(int32 A, int32 B = 1);

	/** Returns true if A is less than B (A < B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer < integer", CompactNodeTitle = "<", Keywords = "< less"), Category="Math|Integer")
	static UE_INL_API bool Less_IntInt(int32 A, int32 B);

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer > integer", CompactNodeTitle = ">", Keywords = "> greater"), Category="Math|Integer")
	static UE_INL_API bool Greater_IntInt(int32 A, int32 B);

	/** Returns true if A is less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer <= integer", CompactNodeTitle = "<=", Keywords = "<= less"), Category="Math|Integer")
	static UE_INL_API bool LessEqual_IntInt(int32 A, int32 B);

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer >= integer", CompactNodeTitle = ">=", Keywords = ">= greater"), Category="Math|Integer")
	static UE_INL_API bool GreaterEqual_IntInt(int32 A, int32 B);

	/** Returns true if A is equal to B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Integer)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Math|Integer")
	static UE_INL_API bool EqualEqual_IntInt(int32 A, int32 B);

	/** Returns true if A is not equal to B (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Integer)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Math|Integer")
	static UE_INL_API bool NotEqual_IntInt(int32 A, int32 B);

	/** Returns true if value is between Min and Max (V >= Min && V <= Max)
	 * If InclusiveMin is true, value needs to be equal or larger than Min, else it needs to be larger
	 * If InclusiveMax is true, value needs to be smaller or equal than Max, else it needs to be smaller
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "In Range (Integer)", Min = "0", Max = "10"), Category = "Math|Integer")
	static ENGINE_API bool InRange_IntInt(int32 Value, int32 Min, int32 Max, bool InclusiveMin = true, bool InclusiveMax = true);

	/** Bitwise AND (A & B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise AND", CompactNodeTitle = "&", Keywords = "& and", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 And_IntInt(int32 A, int32 B);

	/** Bitwise XOR (A ^ B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise XOR", CompactNodeTitle = "^", Keywords = "^ xor"), Category="Math|Integer")
	static UE_INL_API int32 Xor_IntInt(int32 A, int32 B);

	/** Bitwise OR (A | B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise OR", CompactNodeTitle = "|", Keywords = "| or", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 Or_IntInt(int32 A, int32 B);

	/** Bitwise NOT (~A) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Bitwise NOT", CompactNodeTitle = "~", Keywords = "~ not"), Category = "Math|Integer")
	static UE_INL_API int32 Not_Int(int32 A);

	/** Sign (integer, returns -1 if A < 0, 0 if A is zero, and +1 if A > 0) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Sign (Integer)"), Category="Math|Integer")
	static UE_INL_API int32 SignOfInteger(int32 A);

	/** Returns a uniformly distributed random number between 0 and Max - 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API int32 RandomInteger(UPARAM(DisplayName = "Exclusive Max") int32 Max);

	/** Return a random integer between Min and Max (>= Min and <= Max) */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta = (NotBlueprintThreadSafe))
	static UE_INL_API int32 RandomIntegerInRange(int32 Min, int32 Max);

	/** Returns the minimum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Min (Integer)", CompactNodeTitle = "MIN", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 Min(int32 A, int32 B);

	/** Returns the maximum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Max (Integer)", CompactNodeTitle = "MAX", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer")
	static UE_INL_API int32 Max(int32 A, int32 B);

	/** Returns Value clamped to be between A and B (inclusive) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Clamp (Integer)"), Category="Math|Integer")
	static UE_INL_API int32 Clamp(int32 Value, int32 Min, int32 Max);

	/** Returns Value between A and B (inclusive) that wraps around */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Wrap (Integer)", Min = "0", Max = "100"), Category = "Math|Integer")
	static UE_INL_API int32 Wrap(int32 Value, int32 Min, int32 Max);

	/** Returns the absolute (positive) value of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Absolute (Integer)", CompactNodeTitle = "ABS"), Category="Math|Integer")
	static UE_INL_API int32 Abs_Int(int32 A);

	//
	// Integer64 functions.
	//
	
	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 * integer64", CompactNodeTitle = "*", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 Multiply_Int64Int64(int64 A, int64 B);

	/** Division (A / B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 / integer64", CompactNodeTitle = "/", Keywords = "/ divide division"), Category="Math|Integer64")
	static UE_INL_API int64 Divide_Int64Int64(int64 A, int64 B = 1);
	
	/** Modulo (A % B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "% (integer64)", CompactNodeTitle = "%", Keywords = "% modulus"), Category = "Math|Integer64")
	static UE_INL_API int64 Percent_Int64Int64(int64 A, int64 B = 1);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 + integer64", CompactNodeTitle = "+", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 Add_Int64Int64(int64 A, int64 B = 1);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 - integer64", CompactNodeTitle = "-", Keywords = "- subtract minus"), Category="Math|Integer64")
	static UE_INL_API int64 Subtract_Int64Int64(int64 A, int64 B = 1);

	/** Returns true if A is less than B (A < B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 < integer64", CompactNodeTitle = "<", Keywords = "< less"), Category="Math|Integer64")
	static UE_INL_API bool Less_Int64Int64(int64 A, int64 B);

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 > integer64", CompactNodeTitle = ">", Keywords = "> greater"), Category="Math|Integer64")
	static UE_INL_API bool Greater_Int64Int64(int64 A, int64 B);

	/** Returns true if A is less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 <= integer64", CompactNodeTitle = "<=", Keywords = "<= less"), Category="Math|Integer64")
	static UE_INL_API bool LessEqual_Int64Int64(int64 A, int64 B);

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "integer64 >= integer64", CompactNodeTitle = ">=", Keywords = ">= greater"), Category="Math|Integer64")
	static UE_INL_API bool GreaterEqual_Int64Int64(int64 A, int64 B);

	/** Returns true if A is equal to B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Integer64)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Math|Integer64")
	static UE_INL_API bool EqualEqual_Int64Int64(int64 A, int64 B);

	/** Returns true if A is not equal to B (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Integer64)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Math|Integer64")
	static UE_INL_API bool NotEqual_Int64Int64(int64 A, int64 B);
	
	/** Returns true if value is between Min and Max (V >= Min && V <= Max)
	 * If InclusiveMin is true, value needs to be equal or larger than Min, else it needs to be larger
	 * If InclusiveMax is true, value needs to be smaller or equal than Max, else it needs to be smaller
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "In Range (Integer64)", Min = "0", Max = "10"), Category = "Math|Integer64")
	static ENGINE_API bool InRange_Int64Int64(int64 Value, int64 Min, int64 Max, bool InclusiveMin = true, bool InclusiveMax = true);

	/** Bitwise AND (A & B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise AND", CompactNodeTitle = "&", Keywords = "& and", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 And_Int64Int64(int64 A, int64 B);

	/** Bitwise XOR (A ^ B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise XOR", CompactNodeTitle = "^", Keywords = "^ xor"), Category="Math|Integer64")
	static UE_INL_API int64 Xor_Int64Int64(int64 A, int64 B);

	/** Bitwise OR (A | B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Bitwise OR", CompactNodeTitle = "|", Keywords = "| or", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 Or_Int64Int64(int64 A, int64 B);

	/** Bitwise NOT (~A) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Bitwise NOT", CompactNodeTitle = "~", Keywords = "~ not"), Category = "Math|Integer64")
	static UE_INL_API int64 Not_Int64(int64 A);

	/** Sign (integer64, returns -1 if A < 0, 0 if A is zero, and +1 if A > 0) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Sign (Integer64)"), Category="Math|Integer64")
	static UE_INL_API int64 SignOfInteger64(int64 A);

	/** Returns a uniformly distributed random number between 0 and Max - 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API int64 RandomInteger64(UPARAM(DisplayName = "Exclusive Max") int64 Max);

	/** Return a random integer64 between Min and Max (>= Min and <= Max) */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta = (NotBlueprintThreadSafe))
	static UE_INL_API int64 RandomInteger64InRange(int64 Min, int64 Max);

	/** Returns the minimum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Min (Integer64)", CompactNodeTitle = "MIN", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 MinInt64(int64 A, int64 B);

	/** Returns the maximum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Max (Integer64)", CompactNodeTitle = "MAX", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Integer64")
	static UE_INL_API int64 MaxInt64(int64 A, int64 B);

	/** Returns Value clamped to be between A and B (inclusive) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Clamp (Integer64)"), Category="Math|Integer64")
	static UE_INL_API int64 ClampInt64(int64 Value, int64 Min, int64 Max);

	/** Returns the absolute (positive) value of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Absolute (Integer64)", CompactNodeTitle = "ABS"), Category="Math|Integer64")
	static UE_INL_API int64 Abs_Int64(int64 A);

	//
	// Float functions.
	//

	/** Power (Base to the Exp-th power) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Power"), Category = "Math|Float")
	static UE_INL_API double MultiplyMultiply_FloatFloat(double Base, double Exp);

	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "int * float", CompactNodeTitle = "*", Keywords = "* multiply"), Category = "Math|Float")
	static UE_INL_API double Multiply_IntFloat(int32 A, double B);

	/** Modulo (A % B) */
	UFUNCTION(BlueprintPure, CustomThunk, meta = (DisplayName = "% (Float)", CompactNodeTitle = "%", Keywords = "% modulus"), Category = "Math|Float")
	static ENGINE_API double Percent_FloatFloat(double A, double B = 1.f);

	static ENGINE_API double GenericPercent_FloatFloat(double A, double B);

	/** Custom thunk to allow script stack trace in case of modulo by zero */
	DECLARE_FUNCTION(execPercent_FloatFloat)
	{
		P_GET_PROPERTY(FDoubleProperty, A);
		P_GET_PROPERTY(FDoubleProperty, B);

		P_FINISH;

		if (B == 0.f)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Modulo by zero detected: %f %% 0\n%s"), A, *Stack.GetStackTrace()), ELogVerbosity::Warning);
			*(double*)RESULT_PARAM = 0;
			return;
		}

		*(double*)RESULT_PARAM = GenericPercent_FloatFloat(A, B);
	}

	/** Returns the fractional part of a float. */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API double Fraction(double A);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float + float", CompactNodeTitle = "+", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Float")
	static UE_INL_API double Add_DoubleDouble(double A, double B = 1.0);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float - float", CompactNodeTitle = "-", Keywords = "- subtract minus"), Category = "Math|Float")
	static UE_INL_API double Subtract_DoubleDouble(double A, double B = 1.0);

	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float * float", CompactNodeTitle = "*", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Float")
	static UE_INL_API double Multiply_DoubleDouble(double A, double B);

	/** Division (A / B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float / float", CompactNodeTitle = "/", Keywords = "/ divide division"), Category = "Math|Float")
	static UE_INL_API double Divide_DoubleDouble(double A, double B = 1.0);

	/** Returns true if A is Less than B (A < B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float < float", CompactNodeTitle = "<", Keywords = "< less"), Category = "Math|Float")
	static UE_INL_API bool Less_DoubleDouble(double A, double B);

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float > float", CompactNodeTitle = ">", Keywords = "> greater"), Category = "Math|Float")
	static UE_INL_API bool Greater_DoubleDouble(double A, double B);

	/** Returns true if A is Less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float <= float", CompactNodeTitle = "<=", Keywords = "<= less"), Category = "Math|Float")
	static UE_INL_API bool LessEqual_DoubleDouble(double A, double B);

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "float >= float", CompactNodeTitle = ">=", Keywords = ">= greater"), Category = "Math|Float")
	static UE_INL_API bool GreaterEqual_DoubleDouble(double A, double B);

	/** Returns true if A is exactly equal to B (A == B)*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Float)", CompactNodeTitle = "==", Keywords = "== equal"), Category = "Math|Float")
	static UE_INL_API bool EqualEqual_DoubleDouble(double A, double B);

	/** Returns true if A is nearly equal to B (|A - B| < ErrorTolerance) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Nearly Equal (Float)", Keywords = "== equal"), Category = "Math|Float")
	static UE_INL_API bool NearlyEqual_FloatFloat(double A, double B, double ErrorTolerance = 1.e-6);

	/** Returns true if A does not equal B (A != B)*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (Float)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category = "Math|Float")
	static UE_INL_API bool NotEqual_DoubleDouble(double A, double B);

	/** Returns true if value is between Min and Max (V >= Min && V <= Max)
	 * If InclusiveMin is true, value needs to be equal or larger than Min, else it needs to be larger
	 * If InclusiveMax is true, value needs to be smaller or equal than Max, else it needs to be smaller
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "In Range (Float)", Min = "0.0", Max = "1.0"), Category = "Math|Float")
	static ENGINE_API bool InRange_FloatFloat(double Value, double Min, double Max, bool InclusiveMin = true, bool InclusiveMax = true);

	/** Returns the hypotenuse of a right-angled triangle given the width and height. */
	UFUNCTION(BlueprintPure, meta=(Keywords = "pythagorean theorem"), Category = "Math|Float")
	static ENGINE_API double Hypotenuse(double Width, double Height);
	
	/** Snaps a value to the nearest grid multiple. E.g.,
	 *		Location = 5.1, GridSize = 10.0 : return value = 10.0
	 * If GridSize is 0 Location is returned
	 * if GridSize is very small precision issues may occur.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Snap To Grid (Float)"), Category = "Math|Float")
	static UE_INL_API double GridSnap_Float(double Location, double GridSize);

	/** Returns the absolute (positive) value of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Absolute (Float)", CompactNodeTitle = "ABS"), Category="Math|Float")
	static UE_INL_API double Abs(double A);

	/** Returns the sine of A (expects Radians)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Sin (Radians)", CompactNodeTitle = "SIN", Keywords = "sine"), Category="Math|Trig")
	static UE_INL_API double Sin(double A);

	/** Returns the inverse sine (arcsin) of A (result is in Radians) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Asin (Radians)", CompactNodeTitle = "ASIN", Keywords = "sine"), Category="Math|Trig")
	static UE_INL_API double Asin(double A);

	/** Returns the cosine of A (expects Radians)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Cos (Radians)", CompactNodeTitle = "COS"), Category="Math|Trig")
	static UE_INL_API double Cos(double A);

	/** Returns the inverse cosine (arccos) of A (result is in Radians) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Acos (Radians)", CompactNodeTitle = "ACOS"), Category="Math|Trig")
	static UE_INL_API double Acos(double A);

	/** Returns the tan of A (expects Radians)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Tan (Radians)", CompactNodeTitle = "TAN"), Category="Math|Trig")
	static UE_INL_API double Tan(double A);

	/** Returns the inverse tan (atan) (result is in Radians)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Atan (Radians)"), Category="Math|Trig")
	static UE_INL_API double Atan(double A);

	/** Returns the inverse tan (atan2) of A/B (result is in Radians)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Atan2 (Radians)"), Category="Math|Trig")
	static UE_INL_API double Atan2(double Y, double X);

	/** Returns exponential(e) to the power A (e^A)*/
	UFUNCTION(BlueprintPure, Category="Math|Float", meta=(CompactNodeTitle = "e"))
	static UE_INL_API double Exp(double A);

	/** Returns log of A base B (if B^R == A, returns R)*/
	UFUNCTION(BlueprintPure, Category = "Math|Float")
	static ENGINE_API double Log(double A, double Base = 1.0);

	/** Returns natural log of A (if e^R == A, returns R)*/
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API double Loge(double A);

	/** Returns square root of A*/
	UFUNCTION(BlueprintPure, Category="Math|Float", meta=(Keywords = "square root", CompactNodeTitle = "SQRT"))
	static UE_INL_API double Sqrt(double A);

	/** Returns square of A (A*A)*/
	UFUNCTION(BlueprintPure, Category="Math|Float", meta=(CompactNodeTitle = "^2"))
	static UE_INL_API double Square(double A);

	/** Returns a random float between 0 and 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API double RandomFloat();

	/** Generate a random number between Min and Max */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API double RandomFloatInRange(double Min, double Max);

	/** Returns the value of PI */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get PI", CompactNodeTitle = "PI"), Category="Math|Trig")
	static UE_INL_API double GetPI();

	/** Returns the value of TAU (= 2 * PI) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get TAU", CompactNodeTitle = "TAU"), Category="Math|Trig")
	static UE_INL_API double GetTAU();

	/** Returns radians value based on the input degrees */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Degrees To Radians", CompactNodeTitle = "D2R"), Category="Math|Trig")
	static UE_INL_API double DegreesToRadians(double A);

	/** Returns degrees value based on the input radians */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Radians To Degrees", CompactNodeTitle = "R2D"), Category="Math|Trig")
	static UE_INL_API double RadiansToDegrees(double A);

	/** Returns the sin of A (expects Degrees)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Sin (Degrees)", CompactNodeTitle = "SINd", Keywords = "sine"), Category="Math|Trig")
	static UE_INL_API double DegSin(double A);

	/** Returns the inverse sin (arcsin) of A (result is in Degrees) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Asin (Degrees)", CompactNodeTitle = "ASINd", Keywords = "sine"), Category="Math|Trig")
	static UE_INL_API double DegAsin(double A);

	/** Returns the cos of A (expects Degrees)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Cos (Degrees)", CompactNodeTitle = "COSd"), Category="Math|Trig")
	static UE_INL_API double DegCos(double A);

	/** Returns the inverse cos (arccos) of A (result is in Degrees) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Acos (Degrees)", CompactNodeTitle = "ACOSd"), Category="Math|Trig")
	static UE_INL_API double DegAcos(double A);

	/** Returns the tan of A (expects Degrees)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Tan (Degrees)", CompactNodeTitle = "TANd"), Category="Math|Trig")
	static UE_INL_API double DegTan(double A);

	/** Returns the inverse tan (atan) (result is in Degrees)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Atan (Degrees)"), Category="Math|Trig")
	static UE_INL_API double DegAtan(double A);

	/** Returns the inverse tan (atan2) of A/B (result is in Degrees)*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Atan2 (Degrees)"), Category="Math|Trig")
	static UE_INL_API double DegAtan2(double Y, double X);

	/** 
	 * Clamps an arbitrary angle to be between the given angles.  Will clamp to nearest boundary.
	 * 
	 * @param MinAngleDegrees	"from" angle that defines the beginning of the range of valid angles (sweeping clockwise)
	 * @param MaxAngleDegrees	"to" angle that defines the end of the range of valid angles
	 * @return Returns clamped angle in the range -180..180.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Clamp Angle"), Category="Math|Float")
	static UE_INL_API double ClampAngle(double AngleDegrees, double MinAngleDegrees, double MaxAngleDegrees);

	/** Returns the minimum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Min (Float)", CompactNodeTitle = "MIN", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Float")
	static UE_INL_API double FMin(double A, double B);

	/** Returns the maximum value of A and B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Max (Float)", CompactNodeTitle = "MAX", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Float")
	static UE_INL_API double FMax(double A, double B);

	/** Returns Value clamped between A and B (inclusive) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Clamp (Float)", Min="0.0", Max="1.0"), Category="Math|Float")
	static UE_INL_API double FClamp(double Value, double Min, double Max);

	/** Returns Value wrapped from A and B (inclusive) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Wrap (Float)", Min = "0.0", Max = "1.0"), Category = "Math|Float")
	static UE_INL_API double FWrap(double Value, double Min, double Max);

	/** This functions returns 0 if B (the denominator) is zero */
	UFUNCTION(BlueprintPure, Category = "Math|Float", meta = (Keywords = "percent"))
	static ENGINE_API double SafeDivide(double A, double B);

	/** Returns max of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Integer")
	static ENGINE_API void MaxOfIntArray(const TArray<int32>& IntArray, int32& IndexOfMaxValue, int32& MaxValue);

	/** Returns min of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Integer")
	static ENGINE_API void MinOfIntArray(const TArray<int32>& IntArray, int32& IndexOfMinValue, int32& MinValue);

	/** Returns median of all array entries. Returns value of 0 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category = "Math|Integer")
	static ENGINE_API void MedianOfIntArray(TArray<int32> IntArray, float& MedianValue);

	/** Returns average of all array entries. Returns value of 0 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category = "Math|Integer")
	static ENGINE_API void AverageOfIntArray(const TArray<int32>& IntArray, float& AverageValue);

	/** Returns max of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static ENGINE_API void MaxOfFloatArray(const TArray<float>& FloatArray, int32& IndexOfMaxValue, float& MaxValue);

	/** Returns min of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static ENGINE_API void MinOfFloatArray(const TArray<float>& FloatArray, int32& IndexOfMinValue, float& MinValue);

	/** Returns max of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Byte")
	static ENGINE_API void MaxOfByteArray(const TArray<uint8>& ByteArray, int32& IndexOfMaxValue, uint8& MaxValue);

	/** Returns min of all array entries and the index at which it was found. Returns value of 0 and index of -1 if the supplied array is empty. */
	UFUNCTION(BlueprintPure, Category="Math|Byte")
	static ENGINE_API void MinOfByteArray(const TArray<uint8>& ByteArray, int32& IndexOfMinValue, uint8& MinValue);

	/** Linearly interpolates between A and B based on Alpha (100% of A when Alpha=0 and 100% of B when Alpha=1) */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API double Lerp(double A, double B, double Alpha);
	
	/** Easing between A and B using a specified easing function */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Ease", BlueprintInternalUseOnly = "true"), Category = "Math|Interpolation")
	static ENGINE_API double Ease(double A, double B, double Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, double BlendExp = 2, int32 Steps = 2);

	/** Rounds A to the nearest integer (e.g., -1.6 becomes -2 and 1.6 becomes 2) */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API int32 Round(double A);

	/** Rounds A down towards negative infinity / down to the previous integer (e.g., -1.6 becomes -2 and 1.6 becomes 1) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Floor"), Category="Math|Float")
	static UE_INL_API int32 FFloor(double A);
	
	/** Rounds A towards zero, truncating the fractional part (e.g., -1.6 becomes -1 and 1.6 becomes 1) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Truncate", BlueprintAutocast), Category="Math|Float")
	static UE_INL_API int32 FTrunc(double A); // TODO: ok to change with BP Autocast?

	/** Rounds A up towards positive infinity / up to the next integer (e.g., -1.6 becomes -1 and 1.6 becomes 2) */
	UFUNCTION(BlueprintPure, Category = "Math|Float", meta=(DisplayName="Ceil"))
	static UE_INL_API int32 FCeil(double A);

	/** Rounds A to the nearest integer (e.g., -1.6 becomes -2 and 1.6 becomes 2) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Round to Integer64"), Category = "Math|Float")
	static UE_INL_API int64 Round64(double A);

	/** Rounds A down towards negative infinity / down to the previous integer (e.g., -1.6 becomes -2 and 1.6 becomes 1) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Floor to Integer64"), Category = "Math|Float")
	static UE_INL_API int64 FFloor64(double A);

	/** Rounds A towards zero, truncating the fractional part (e.g., -1.6 becomes -1 and 1.6 becomes 1) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Truncate to Integer64", BlueprintAutocast), Category = "Math|Float")
	static UE_INL_API int64 FTrunc64(double A);

	/** Rounds A up towards positive infinity / up to the next integer (e.g., -1.6 becomes -1 and 1.6 becomes 2) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Ceil to Integer64"), Category = "Math|Float")
	static UE_INL_API int64 FCeil64(double A);

	/** Returns the number of times Divisor will go into Dividend (i.e., Dividend divided by Divisor), as well as the remainder */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Division (Whole and Remainder)"), Category="Math|Float")
	static ENGINE_API int32 FMod(double Dividend, double Divisor, double& Remainder);

	/** Returns the number of times Divisor will go into Dividend (i.e., Dividend divided by Divisor), as well as the remainder */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Division (Whole and Remainder) to Integer64"), Category = "Math|Float")
	static ENGINE_API int64 FMod64(double Dividend, double Divisor, double& Remainder);

	/** Sign (float, returns -1 if A < 0, 0 if A is zero, and +1 if A > 0) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Sign (Float)"), Category="Math|Float")
	static UE_INL_API double SignOfFloat(double A);

	/** Returns Value normalized to the given range.  (e.g. 20 normalized to the range 10->50 would result in 0.25) */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static ENGINE_API double NormalizeToRange(double Value, double RangeMin, double RangeMax);

	/** Returns Value mapped from one range into another.  (e.g. 20 normalized from the range 10->50 to 20->40 would result in 25) */
	UFUNCTION(BlueprintPure, Category="Math|Float", meta=(Keywords = "get mapped value"))
	static ENGINE_API double MapRangeUnclamped(double Value, double InRangeA, double InRangeB, double OutRangeA, double OutRangeB);

	/** Returns Value mapped from one range into another where the Value is clamped to the Input Range.  (e.g. 0.5 normalized from the range 0->1 to 0->50 would result in 25) */
	UFUNCTION(BlueprintPure, Category="Math|Float", meta=(Keywords = "get mapped value"))
	static ENGINE_API double MapRangeClamped(double Value, double InRangeA, double InRangeB, double OutRangeA, double OutRangeB);
	
	/** Multiplies the input value by pi. */
	UFUNCTION(BlueprintPure, meta=(Keywords = "* multiply"), Category="Math|Float")
	static UE_INL_API double MultiplyByPi(double Value);

	/** Interpolate between A and B, applying an ease in/out function.  Exp controls the degree of the curve. */
	UFUNCTION(BlueprintPure, Category = "Math|Float")
	static ENGINE_API double FInterpEaseInOut(double A, double B, double Alpha, double Exponent);

	/**
	* Evaluate this runtime float curve at the specified time 
	* 
	* @param Curve				The runtime float curve to evaluate
	* @param InTime				The time at which to evaluate the curve
	* @param InDefaultValue		The default value which should be used if the curve cannot be evaluated at the given time.
	* 
	* @return					The curve's value at the given time.
	* 
	* @see FRichCurve::Eval
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Curves")
	static ENGINE_API float GetRuntimeFloatCurveValue(const FRuntimeFloatCurve& Curve, const float InTime, const float InDefaultValue = 0.0f);

	/**
	* Simple function to create a pulsating scalar value
	*
	* @param  InCurrentTime  Current absolute time
	* @param  InPulsesPerSecond  How many full pulses per second?
	* @param  InPhase  Optional phase amount, between 0.0 and 1.0 (to synchronize pulses)
	*
	* @return  Pulsating value (0.0-1.0)
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Float")
	static ENGINE_API float MakePulsatingValue(float InCurrentTime, float InPulsesPerSecond = 1.0f, float InPhase = 0.0f);

	/** 
	 * Returns a new rotation component value
	 *
	 * @param InCurrent is the current rotation value
	 * @param InDesired is the desired rotation value
	 * @param InDeltaRate is the rotation amount to apply
	 *
	 * @return a new rotation component value clamped in the range (-360,360)
	 */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API float FixedTurn(float InCurrent, float InDesired, float InDeltaRate);


	//
	// IntPoint constants
	//
	
	/** Zero Int Point (0, 0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Zero", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_Zero();
	
	/** One Int Point (1, 1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "One", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_One();
	
	/** Up Int Point (0, -1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Up", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_Up();
	
	/** Left Int Point (-1, 0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Left", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_Left();
	
	/** Right Int Point (1, 0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Right", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_Right();
	
	/** Down Int Point (0, 1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Down", ScriptConstantHost = "/Script/CoreUObject.IntPoint"), Category = "Math|IntPoint|Constants")
	static UE_INL_API FIntPoint IntPoint_Down();

	//
	// IntPoint functions
	//

	/** Converts an IntPoint to a Vector2D */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector2D (IntPoint)", CompactNodeTitle = "->", ScriptMethod = "Vector2D", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector2D Conv_IntPointToVector2D(FIntPoint InIntPoint);

	/** Returns IntPoint A added by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint + IntPoint", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Add_IntPointIntPoint(FIntPoint A, FIntPoint B);

	/** Addition (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint + Integer", CompactNodeTitle = "+", ScriptMethod = "AddInt", ScriptOperator = "+;+=", Keywords = "+ add plus"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Add_IntPointInt(FIntPoint A, int32 B);

	/** Returns IntPoint A subtracted by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint - IntPoint", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Subtract_IntPointIntPoint(FIntPoint A, FIntPoint B);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint - Integer", CompactNodeTitle = "-", ScriptMethod = "SubtractInt", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Subtract_IntPointInt(FIntPoint A, int32 B);

	/** Returns IntPoint A multiplied by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint * IntPoint", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Multiply_IntPointIntPoint(FIntPoint A, FIntPoint B);

	/** Multiplication (A * B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint * Integer", CompactNodeTitle = "*", ScriptMethod = "MultiplyInt", ScriptOperator = "*;*=", Keywords = "* multiply"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Multiply_IntPointInt(FIntPoint A, int32 B);

	/** Returns IntPoint A divided by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint / IntPoint", CompactNodeTitle = "/", ScriptMethod = "Divide", ScriptOperator = "/;/=", Keywords = "/ divide"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Divide_IntPointIntPoint(FIntPoint A, FIntPoint B);

	/** Division (A * B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "IntPoint / Integer", CompactNodeTitle = "/", ScriptMethod = "DivideInt", ScriptOperator = "/;/=", Keywords = "/ divide"), Category = "Math|IntPoint")
	static UE_INL_API FIntPoint Divide_IntPointInt(FIntPoint A, int32 B);

	/** Returns true if IntPoint A is equal to IntPoint B (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (IntPoint)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|IntPoint")
	static UE_INL_API bool Equal_IntPointIntPoint(FIntPoint A, FIntPoint B);

	/** Returns true if IntPoint A is NOT equal to IntPoint B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (IntPoint)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|IntPoint")
	static UE_INL_API bool NotEqual_IntPointIntPoint(FIntPoint A, FIntPoint B);

	//
	// Vector2D constants - exposed for scripting
	//

	/** 2D one vector constant (1,1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "One", ScriptConstantHost = "/Script/CoreUObject.Vector2D"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Vector2D_One();

	/** 2D unit vector constant along the 45 degree angle or symmetrical positive axes (sqrt(.5),sqrt(.5)) or (.707,.707). https://en.wikipedia.org/wiki/Unit_vector */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Unit45Deg", ScriptConstantHost = "/Script/CoreUObject.Vector2D"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Vector2D_Unit45Deg();

	/** 2D zero vector constant (0,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Zero", ScriptConstantHost = "/Script/CoreUObject.Vector2D"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Vector2D_Zero();


	//
	// Vector2D functions
	//

	/** Makes a 2d vector {X, Y} */
	UFUNCTION(BlueprintPure, Category = "Math|Vector2D", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_INL_API FVector2D MakeVector2D(double X, double Y);

	/** Breaks a 2D vector apart into X, Y. */
	UFUNCTION(BlueprintPure, Category = "Math|Vector2D", meta = (NativeBreakFunc))
	static UE_INL_API void BreakVector2D(FVector2D InVec, double& X, double& Y);

	/** Converts a Vector2D to a Vector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (Vector2D)", CompactNodeTitle = "->", ScriptMethod = "Vector", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_Vector2DToVector(FVector2D InVector2D, float Z = 0);

	/** Converts a Vector2D to an IntPoint */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To IntPoint (Vector2D)", CompactNodeTitle = "->", ScriptMethod = "IntPoint", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FIntPoint Conv_Vector2DToIntPoint(FVector2D InVector2D);

	/** Returns addition of Vector A and Vector B (A + B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d + vector2d", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Add_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns Vector A added by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d + float", CompactNodeTitle = "+", ScriptMethod = "AddFloat", ScriptOperator = "+;+=", Keywords = "+ add plus"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Add_Vector2DFloat(FVector2D A, double B);

	/** Returns subtraction of Vector B from Vector A (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d - vector2d", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Subtract_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns Vector A subtracted by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d - float", CompactNodeTitle = "-", ScriptMethod = "SubtractFloat", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Subtract_Vector2DFloat(FVector2D A, double B);

	/** Element-wise Vector multiplication (Result = {A.x*B.x, A.y*B.y}) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d * vector2d", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Multiply_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns Vector A scaled by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d * float", CompactNodeTitle = "*", ScriptMethod = "MultiplyFloat", ScriptOperator = "*;*=", Keywords = "* multiply"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Multiply_Vector2DFloat(FVector2D A, double B);

	/** Element-wise Vector divide (Result = {A.x/B.x, A.y/B.y}) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d / vector2d", CompactNodeTitle = "/", ScriptMethod = "Divide", ScriptOperator = "/;/=", Keywords = "/ divide division"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Divide_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns Vector A divided by B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "vector2d / float", CompactNodeTitle = "/", ScriptMethod = "DivideFloat", ScriptOperator = "/;/=", Keywords = "/ divide division"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Divide_Vector2DFloat(FVector2D A, double B = 1.f);

	/** Returns true if vector A is equal to vector B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal Exactly (Vector2D)", CompactNodeTitle = "===", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category="Math|Vector2D")
	static UE_INL_API bool EqualExactly_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns true if vector2D A is equal to vector2D B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Vector2D)", CompactNodeTitle = "==", ScriptMethod = "IsNearEqual", Keywords = "== equal"), Category = "Math|Vector2D")
	static UE_INL_API bool EqualEqual_Vector2DVector2D(FVector2D A, FVector2D B, float ErrorTolerance = 1.e-4f);

	/** Returns true if vector2D A is not equal to vector2D B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal Exactly (Vector2D)", CompactNodeTitle = "!==", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Vector2D")
	static UE_INL_API bool NotEqualExactly_Vector2DVector2D(FVector2D A, FVector2D B);

	/** Returns true if vector2D A is not equal to vector2D B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (Vector2D)", CompactNodeTitle = "!=", ScriptMethod = "IsNotNearEqual", Keywords = "!= not equal"), Category = "Math|Vector2D")
	static UE_INL_API bool NotEqual_Vector2DVector2D(FVector2D A, FVector2D B, float ErrorTolerance = 1.e-4f);

	/** Gets a negated copy of the vector. */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "Negated", ScriptOperator = "neg"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Negated2D(const FVector2D& A);

	/**
	 * Set the values of the vector directly.
	 *
	 * @param InX New X coordinate.
	 * @param InY New Y coordinate.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Set"), Category = "Math|Vector2D")
	static UE_INL_API void Set2D(UPARAM(ref) FVector2D& A, double X, double Y);

	/**
	 * Creates a copy of this vector with both axes clamped to the given range.
	 * @return New vector with clamped axes.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "ClampedAxes"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D ClampAxes2D(FVector2D A, double MinAxisVal, double MaxAxisVal);

	/** Returns the cross product of two 2d vectors - see  http://mathworld.wolfram.com/CrossProduct.html */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Cross Product (2D)", CompactNodeTitle = "cross", ScriptMethod = "Cross", ScriptOperator = "^"), Category = "Math|Vector2D")
	static UE_INL_API double CrossProduct2D(FVector2D A, FVector2D B);

	/**
	 * Distance between two 2D points.
	 *
	 * @param V1 The first point.
	 * @param V2 The second point.
	 * @return The distance between two 2D points.
	 */
	UFUNCTION(BlueprintPure, meta = (Keywords = "magnitude", ScriptMethod = "Distance"), Category = "Math|Vector2D")
	static UE_INL_API double Distance2D(FVector2D V1, FVector2D V2);

	/**
	 * Squared distance between two 2D points.
	 *
	 * @param V1 The first point.
	 * @param V2 The second point.
	 * @return The squared distance between two 2D points.
	 */
	UFUNCTION(BlueprintPure, meta = (Keywords = "magnitude", ScriptMethod = "DistanceSquared"), Category = "Math|Vector2D")
	static UE_INL_API double DistanceSquared2D(FVector2D V1, FVector2D V2);

	/** Returns the dot product of two 2d vectors - see http://mathworld.wolfram.com/DotProduct.html */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Dot Product (2D)", CompactNodeTitle = "dot", ScriptMethod = "Dot", ScriptOperator = "|"), Category = "Math|Vector2D")
	static UE_INL_API double DotProduct2D(FVector2D A, FVector2D B);

	/**
	* Get a copy of this vector with absolute value of each component.
	*
	* @return A copy of this vector with absolute value of each component.
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "GetAbs"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D GetAbs2D(FVector2D A);

	/**
	 * Get the maximum absolute value of the vector's components.
	 *
	 * @return The maximum absolute value of the vector's components.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "GetAbsMax"), Category = "Math|Vector2D")
	static UE_INL_API double GetAbsMax2D(FVector2D A);

	/**
	 * Get the maximum value of the vector's components.
	 *
	 * @return The maximum value of the vector's components.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "GetMax"), Category = "Math|Vector2D")
	static UE_INL_API double GetMax2D(FVector2D A);

	/**
	 * Get the minimum value of the vector's components.
	 *
	 * @return The minimum value of the vector's components.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "GetMin"), Category = "Math|Vector2D")
	static UE_INL_API double GetMin2D(FVector2D A);

	/**
	 * Rotates around axis (0,0,1)
	 *
	 * @param AngleDeg Angle to rotate (in degrees)
	 * @return Rotated Vector
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "GetRotated"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D GetRotated2D(FVector2D A, float AngleDeg);

	/**
	 * Checks whether vector is near to zero within a specified tolerance.
	 *
	 * @param Tolerance Error tolerance.
	 * @return true if vector is in tolerance to zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsNearlyZero"), Category = "Math|Vector2D")
	static UE_INL_API bool IsNearlyZero2D(const FVector2D& A, float Tolerance = 1.e-4f);

	/**
	 * Checks whether all components of the vector are exactly zero.
	 *
	 * @return true if vector is exactly zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsZero"), Category = "Math|Vector2D")
	static UE_INL_API bool IsZero2D(const FVector2D& A);

	/**
	 * Tries to reach Target based on distance from Current position, giving a nice smooth feeling when tracking a position.
	 *
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation", meta=(ScriptMethod="InterpTo", Keywords="position"))
	static UE_INL_API FVector2D Vector2DInterpTo(FVector2D Current, FVector2D Target, float DeltaTime, float InterpSpeed);
	
	/**
	 * Tries to reach Target at a constant rate.
	 *
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation", meta=(ScriptMethod="InterpToConstant", Keywords="position"))
	static UE_INL_API FVector2D Vector2DInterpTo_Constant(FVector2D Current, FVector2D Target, float DeltaTime, float InterpSpeed);

	/**
	 * Gets a normalized copy of the vector, checking it is safe to do so based on the length.
	 * Returns zero vector if vector length is too small to safely normalize.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 * @return A normalized copy of the vector if safe, (0,0) otherwise.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Normal Safe (Vector2D)", Keywords = "Unit Vector", ScriptMethod = "Normal"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D NormalSafe2D(FVector2D A, float Tolerance = 1.e-8f);

	/** Returns a unit normal version of the 2D vector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Normalize 2D", Keywords = "Unit Vector", ScriptMethod = "NormalUnsafe"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D Normal2D(FVector2D A);

	/**
	 * Normalize this vector in-place if it is large enough, set it to (0,0) otherwise.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 * @see NormalSafe2D()
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Normalize In Place (Vector2D)", Keywords = "Unit Vector", ScriptMethod = "Normalize"), Category = "Math|Vector2D")
	static UE_INL_API void Normalize2D(UPARAM(ref) FVector2D& A, float Tolerance = 1.e-8f);

	/** Converts spherical coordinates on the unit sphere into a Cartesian unit length vector. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Spherical2D To Unit Cartesian", Keywords = "Unit Vector", ScriptMethod = "SphericalToUnitCartesian"), Category = "Math|Vector2D")
	static UE_INL_API FVector Spherical2DToUnitCartesian(FVector2D A);

	/**
	 * Util to convert this vector into a unit direction vector and its original length.
	 *
	 * @param OutDir Reference passed in to store unit direction vector.
	 * @param OutLength Reference passed in to store length of the vector.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Direction And Length", ScriptMethod = "ToDirectionAndLength"), Category = "Math|Vector2D")
	static UE_INL_API void ToDirectionAndLength2D(FVector2D A, FVector2D &OutDir, double &OutLength);

	/**
	 * Get this vector as a vector where each component has been rounded to the nearest int.
	 *
	 * @return New FVector2D from this vector that is rounded.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Rounded (Vector2D)", ScriptMethod = "ToRounded"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D ToRounded2D(FVector2D A);

	/**
	* Get a copy of the vector as sign only.
	* Each component is set to +1 or -1, with the sign of zero treated as +1.
	*
	* @return A copy of the vector with each component set to +1 or -1
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Sign (+1/-1) 2D", ScriptMethod = "ToSign"), Category = "Math|Vector2D")
	static UE_INL_API FVector2D ToSign2D(FVector2D A);

	/** Returns the length of a 2D Vector. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector2D Length", Keywords = "magnitude", ScriptMethod = "Length"), Category = "Math|Vector2D")
	static UE_INL_API double VSize2D(FVector2D A);

	/** Returns the squared length of a 2D Vector. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector2D Length Squared", Keywords = "magnitude", ScriptMethod = "LengthSquared"), Category = "Math|Vector2D")
	static UE_INL_API double VSize2DSquared(FVector2D A);


	//
	// Vector (3D) constants - exposed for scripting
	//

	/** 3D vector zero constant (0,0,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Zero", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Zero();

	/** 3D vector one constant (1,1,1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "One", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_One();

	/** 3D vector Unreal forward direction constant (1,0,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Forward", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Forward();

	/** 3D vector Unreal backward direction constant (-1,0,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Backward", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Backward();

	/** 3D vector Unreal up direction constant (0,0,1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Up", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Up();

	/** 3D vector Unreal down direction constant (0,0,-1) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Down", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Down();

	/** 3D vector Unreal right direction constant (0,1,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Right", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Right();

	/** 3D vector Unreal left direction constant (0,-1,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Left", ScriptConstantHost = "/Script/CoreUObject.Vector"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_Left();

	//
	// Vector (3D) functions.
	//

	/** Makes a vector {X, Y, Z} */
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_INL_API FVector MakeVector(double X, double Y, double Z);

	/** Creates a directional vector from rotation values {Pitch, Yaw} supplied in degrees with specified Length*/	
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (Keywords = "rotation rotate"))
	static ENGINE_API FVector CreateVectorFromYawPitch(float Yaw, float Pitch, float Length = 1.0f );

	/**
	 * Assign the values of the supplied vector.
	 *
	 * @param InVector Vector to copy values from.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Assign"), Category = "Math|Vector")
	static UE_INL_API void Vector_Assign(UPARAM(ref) FVector& A, const FVector& InVector);

	/**
	 * Set the values of the vector directly.
	 *
	 * @param InX New X coordinate.
	 * @param InY New Y coordinate.
	 * @param InZ New Z coordinate.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Set"), Category = "Math|Vector")
	static UE_INL_API void Vector_Set(UPARAM(ref) FVector& A, double X, double Y, double Z);

	/** Breaks a vector apart into X, Y, Z */
	UFUNCTION(BlueprintPure, Category="Math|Vector", meta=(NativeBreakFunc))
	static UE_INL_API void BreakVector(FVector InVec, double& X, double& Y, double& Z);

	/** Converts a vector to LinearColor */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To LinearColor (Vector)", CompactNodeTitle = "->", ScriptMethod = "LinearColor", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API FLinearColor Conv_VectorToLinearColor(FVector InVec);

	/** Converts a vector to a transform. Uses vector as location */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Transform (Vector)", CompactNodeTitle = "->", ScriptMethod = "Transform", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static ENGINE_API FTransform Conv_VectorToTransform(FVector InLocation);
	
	/** Converts a Vector to a Vector2D using the Vector's (X, Y) coordinates */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Vector2D (Vector)", CompactNodeTitle = "->", ScriptMethod = "Vector2D", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static ENGINE_API FVector2D Conv_VectorToVector2D(FVector InVector);

	/**
	 * Return the FRotator orientation corresponding to the direction in which the vector points.
	 * Sets Yaw and Pitch to the proper numbers, and sets Roll to zero because the roll can't be determined from a vector.
	 *
	 * @return FRotator from the Vector's direction, without any roll.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Rotation From X Vector", ScriptMethod = "Rotator", Keywords="rotation rotate cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API FRotator Conv_VectorToRotator(FVector InVec);

	/** Create a rotation from an axis and supplied angle (in degrees) */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "RotatorFromAxisAndAngle", Keywords="make construct build rotate rotation"), Category="Math|Vector")
	static ENGINE_API FRotator RotatorFromAxisAndAngle(FVector Axis, float Angle);

	/**
	 * Return the Quaternion orientation corresponding to the direction in which the vector points.
	 * Similar to the FRotator version, returns a result without roll such that it preserves the up vector.
	 *
	 * @note If you don't care about preserving the up vector and just want the most direct rotation, you can use the faster
	 * 'FindBetweenVectors(ForwardVector, YourVector)' or 'FindBetweenNormals(...)' if you know the vector is of unit length.
	 *
	 * @return Quaternion from the Vector's direction, without any roll.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Quaternion (Vector)", ScriptMethod = "Quaternion", Keywords="rotation rotate cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API FQuat Conv_VectorToQuaternion(FVector InVec);

	/**
	 * Interpolate from a vector to the direction of another vector along a spherical path.
	 *
	 * @param Vector Vector we interpolate from
	 * @param Direction Target direction we interpolate to
	 * @param Alpha interpolation amount, usually between 0-1
	 * @return Vector after interpolating between Vector and Direction along a spherical path. The magnitude will remain the length of the starting vector.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "SlerpVectorToDirection", ScriptMethod = "SlerpVectors", Keywords = "spherical lerpvector vectorslerp interpolate"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_SlerpVectorToDirection(FVector Vector, FVector Direction, double Alpha);

	/**
	 * Interpolate from normalized vector A to normalized vector B along a spherical path.
	 *
	 * @param NormalA Start direction of interpolation, must be normalized.
	 * @param NormalB End target direction of interpolation, must be normalized.
	 * @param Alpha interpolation amount, usually between 0-1
	 * @return Vector after interpolating between A and B along a spherical path.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "SlerpNormals", ScriptMethod = "SlerpNormals", Keywords = "spherical lerpnormal normalslerp interpolate"), Category = "Math|Vector")
	static UE_INL_API FVector Vector_SlerpNormals(FVector NormalA, FVector NormalB, double Alpha);

	/** Vector addition */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector + vector", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Vector")
	static UE_INL_API FVector Add_VectorVector(FVector A, FVector B);

	/** Adds a float to each component of a vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector + float", CompactNodeTitle = "+", ScriptMethod = "AddFloat", Keywords = "+ add plus"), Category="Math|Vector")
	static UE_INL_API FVector Add_VectorFloat(FVector A, double B);
	
	/** Adds an integer to each component of a vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector + integer", CompactNodeTitle = "+", ScriptMethod = "AddInt", Keywords = "+ add plus"), Category="Math|Vector")
	static UE_INL_API FVector Add_VectorInt(FVector A, int32 B);

	/** Vector subtraction */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector - vector", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category="Math|Vector")
	static UE_INL_API FVector Subtract_VectorVector(FVector A, FVector B);

	/** Subtracts a float from each component of a vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector - float", CompactNodeTitle = "-", ScriptMethod = "SubtractFloat", Keywords = "- subtract minus"), Category="Math|Vector")
	static UE_INL_API FVector Subtract_VectorFloat(FVector A, double B);

	/** Subtracts an integer from each component of a vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector - integer", CompactNodeTitle = "-", ScriptMethod = "SubtractInt", Keywords = "- subtract minus"), Category="Math|Vector")
	static UE_INL_API FVector Subtract_VectorInt(FVector A, int32 B);

	/** Element-wise Vector multiplication (Result = {A.x*B.x, A.y*B.y, A.z*B.z}) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector * vector", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category="Math|Vector")
	static UE_INL_API FVector Multiply_VectorVector(FVector A, FVector B);

	/** Scales Vector A by B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector * float", CompactNodeTitle = "*", ScriptMethod = "MultiplyFloat", ScriptOperator = "*;*=", Keywords = "* multiply"), Category="Math|Vector")
	static UE_INL_API FVector Multiply_VectorFloat(FVector A, double B);
	
	/** Scales Vector A by B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector * integer", CompactNodeTitle = "*", ScriptMethod = "MultiplyInt", Keywords = "* multiply"), Category="Math|Vector")
	static UE_INL_API FVector Multiply_VectorInt(FVector A, int32 B);

	/** Element-wise Vector division (Result = {A.x/B.x, A.y/B.y, A.z/B.z}) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector / vector", CompactNodeTitle = "/", ScriptMethod = "Divide", ScriptOperator = "/;/=", Keywords = "/ divide division"), Category="Math|Vector")
	static UE_INL_API FVector Divide_VectorVector(FVector A, FVector B = FVector(1.f,1.f,1.f));

	/** Vector divide by a float */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector / float", CompactNodeTitle = "/", ScriptMethod = "DivideFloat", ScriptOperator = "/;/=", Keywords = "/ divide division"), Category="Math|Vector")
	static UE_INL_API FVector Divide_VectorFloat(FVector A, double B = 1.f);

	/** Vector divide by an integer */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "vector / integer", CompactNodeTitle = "/", ScriptMethod = "DivideInt", Keywords = "/ divide division"), Category="Math|Vector")
	static UE_INL_API FVector Divide_VectorInt(FVector A, int32 B = 1);

	/** Negate a vector. */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "Negated", ScriptOperator = "neg"), Category="Math|Vector")
	static UE_INL_API FVector NegateVector(FVector A);

	/** Returns true if vector A is equal to vector B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal Exactly (Vector)", CompactNodeTitle = "===", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category="Math|Vector")
	static UE_INL_API bool EqualExactly_VectorVector(FVector A, FVector B);

	/** Returns true if vector A is equal to vector B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Vector)", CompactNodeTitle = "==", ScriptMethod = "IsNearEqual", Keywords = "== equal"), Category="Math|Vector")
	static UE_INL_API bool EqualEqual_VectorVector(FVector A, FVector B, float ErrorTolerance = 1.e-4f);

	/** Returns true if vector A is not equal to vector B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal Exactly (Vector)", CompactNodeTitle = "!==", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Vector")
	static UE_INL_API bool NotEqualExactly_VectorVector(FVector A, FVector B);

	/** Returns true if vector A is not equal to vector B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Vector)", CompactNodeTitle = "!=", ScriptMethod = "IsNotNearEqual"), Category="Math|Vector")
	static UE_INL_API bool NotEqual_VectorVector(FVector A, FVector B, float ErrorTolerance = 1.e-4f);

	/** Returns the dot product of two 3d vectors - see http://mathworld.wolfram.com/DotProduct.html */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Dot Product", CompactNodeTitle = "dot", ScriptMethod = "Dot", ScriptOperator = "|"), Category="Math|Vector" )
	static UE_INL_API double Dot_VectorVector(FVector A, FVector B);

	/** Returns the cross product of two 3d vectors - see http://mathworld.wolfram.com/CrossProduct.html */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Cross Product", CompactNodeTitle = "cross", ScriptMethod = "Cross", ScriptOperator = "^"), Category="Math|Vector" )
	static UE_INL_API FVector Cross_VectorVector(FVector A, FVector B);

	/** Returns result of vector A rotated by Rotator B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Rotate Vector", ScriptMethod = "Rotate"), Category="Math|Vector")
	static UE_INL_API FVector GreaterGreater_VectorRotator(FVector A, FRotator B);

	/** Returns result of vector A rotated by AngleDeg around Axis */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Rotate Vector Around Axis", ScriptMethod = "RotateAngleAxis"), Category="Math|Vector")
	static ENGINE_API FVector RotateAngleAxis(FVector InVect, float AngleDeg, FVector Axis);

	/** Returns result of vector A rotated by the inverse of Rotator B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Unrotate Vector", ScriptMethod = "Unrotate"), Category="Math|Vector")
	static UE_INL_API FVector LessLess_VectorRotator(FVector A, FRotator B);

	/** When this vector contains Euler angles (degrees), ensure that angles are between +/-180 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "UnwindEuler"), Category = "Math|Vector")
	static UE_INL_API void Vector_UnwindEuler(UPARAM(ref) FVector& A);

	/** Create a copy of this vector, with its magnitude/size/length clamped between Min and Max. */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ClampedSize"), Category="Math|Vector")
	static UE_INL_API FVector ClampVectorSize(FVector A, double Min, double Max);

	/** Create a copy of this vector, with the 2D magnitude/size/length clamped between Min and Max. Z is unchanged. */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ClampedSize2D"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ClampSize2D(FVector A, double Min, double Max);

	/** Create a copy of this vector, with its maximum magnitude/size/length clamped to MaxSize. */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ClampedSizeMax"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ClampSizeMax(FVector A, double Max);

	/** Create a copy of this vector, with the maximum 2D magnitude/size/length clamped to MaxSize. Z is unchanged. */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ClampedSizeMax2D"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ClampSizeMax2D(FVector A, double Max);

	/** Find the minimum element (X, Y or Z) of a vector */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetMinElement"), Category="Math|Vector")
	static UE_INL_API double GetMinElement(FVector A);

	/** Find the maximum element (X, Y or Z) of a vector */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetMaxElement"), Category="Math|Vector")
	static UE_INL_API double GetMaxElement(FVector A);

	/** Find the maximum absolute element (abs(X), abs(Y) or abs(Z)) of a vector */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetAbsMax"), Category="Math|Vector")
	static UE_INL_API double Vector_GetAbsMax(FVector A);

	/** Find the minimum absolute element (abs(X), abs(Y) or abs(Z)) of a vector */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetAbsMin"), Category="Math|Vector")
	static UE_INL_API double Vector_GetAbsMin(FVector A);

	/**
	 * Get a copy of this vector with absolute value of each component.
	 *
	 * @return A copy of this vector with absolute value of each component.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetAbs"), Category="Math|Vector")
	static UE_INL_API FVector Vector_GetAbs(FVector A);

	/** Find the minimum elements (X, Y and Z) between the two vector's components */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetMin"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ComponentMin(FVector A, FVector B);

	/** Find the maximum elements (X, Y and Z) between the two vector's components */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetMax"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ComponentMax(FVector A, FVector B);

	/**
	 * Get a copy of the vector as sign only.
	 * Each component is set to +1 or -1, with the sign of zero treated as +1.
	 *
	 * @param A copy of the vector with each component set to +1 or -1
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetSignVector"), Category="Math|Vector")
	static UE_INL_API FVector Vector_GetSignVector(FVector A);

	/**
	 * Projects 2D components of vector based on Z.
	 *
	 * @return Projected version of vector based on Z.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetProjection"), Category="Math|Vector")
	static UE_INL_API FVector Vector_GetProjection(FVector A);

	/**
	 * Convert a direction vector into a 'heading' angle.
	 *
	 * @return 'Heading' angle between +/-PI radians. 0 is pointing down +X.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "HeadingAngle"), Category="Math|Vector")
	static UE_INL_API double Vector_HeadingAngle(FVector A);

	/**
	 * Returns the cosine of the angle between this vector and another projected onto the XY plane (no Z).
	 *
	 * @param B the other vector to find the 2D cosine of the angle with.
	 * @return The cosine.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "CosineAngle2D"), Category="Math|Vector")
	static UE_INL_API double Vector_CosineAngle2D(FVector A, FVector B);

	/**
	 * Converts a vector containing degree values to a vector containing radian values.
	 *
	 * @return Vector containing radian values
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ToRadians"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ToRadians(FVector A);

	/**
	 * Converts a vector containing radian values to a vector containing degree values.
	 *
	 * @return Vector  containing degree values
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ToDegrees"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ToDegrees(FVector A);

	/** 
	 * Converts a Cartesian unit vector into spherical coordinates on the unit sphere.
	 * @return Output Theta will be in the range [0, PI], and output Phi will be in the range [-PI, PI]. 
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "UnitCartesianToSpherical"), Category="Math|Vector")
	static UE_INL_API FVector2D Vector_UnitCartesianToSpherical(FVector A);

	/** Find the unit direction vector from one position to another or (0,0,0) if positions are the same. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Unit Direction (Vector)", ScriptMethod = "DirectionUnitTo", Keywords = "Unit Vector"), Category="Math|Vector")
	static UE_INL_API FVector GetDirectionUnitVector(FVector From, FVector To);

	/** Breaks a vector apart into Yaw, Pitch rotation values given in degrees. (non-clamped) */
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (ScriptMethod = "GetYawPitch", NativeBreakFunc))
	static ENGINE_API void GetYawPitchFromVector(FVector InVec, float& Yaw, float& Pitch);

	/** Breaks a direction vector apart into Azimuth (Yaw) and Elevation (Pitch) rotation values given in degrees. (non-clamped)
	 Relative to the provided reference frame (an Actor's WorldTransform for example) */
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (ScriptMethod = "GetAzimuthElevation", NativeBreakFunc))
	static ENGINE_API void GetAzimuthAndElevation(FVector InDirection, const FTransform& ReferenceFrame, float& Azimuth, float& Elevation);

	/** Find the average of an array of vectors */
	UFUNCTION(BlueprintPure, Category="Math|Vector")
	static ENGINE_API FVector GetVectorArrayAverage(const TArray<FVector>& Vectors);

	/** Rounds A to an integer with truncation towards zero for each element in a vector.  (e.g. -1.7 truncated to -1, 2.8 truncated to 2) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Truncate (Vector)", ScriptMethod = "Truncated", BlueprintAutocast), Category = "Math|Float")
	static UE_INL_API FIntVector FTruncVector(const FVector& InVector);

	/** Rounds A to an integer with truncation towards zero for each element in a vector2D.  (e.g. -1.7 truncated to -1, 2.8 truncated to 2) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Truncate (Vector2D)", ScriptMethod = "Truncated", BlueprintAutocast), Category = "Math|Float")
	static UE_INL_API FIntVector2 FTruncVector2D(const FVector2D& InVector2D);

	/**
	 * Distance between two points.
	 *
	 * @param V1 The first point.
	 * @param V2 The second point.
	 * @return The distance between two points.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Distance (Vector)", ScriptMethod = "Distance", Keywords = "magnitude"), Category = "Math|Vector")
	static UE_INL_API double Vector_Distance(FVector V1, FVector V2);

	/**
	 * Squared distance between two points.
	 *
	 * @param V1 The first point.
	 * @param V2 The second point.
	 * @return The squared distance between two points.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Distance Squared (Vector)", ScriptMethod = "DistanceSquared", Keywords = "magnitude"), Category = "Math|Vector")
	static UE_INL_API double Vector_DistanceSquared(FVector V1, FVector V2);

	/**
	* Euclidean distance between two points in the XY plane (ignoring Z).
	*
	* @param V1 The first point.
	* @param V2 The second point.
	* @return The distance between two points in the XY plane.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Distance2D (Vector)", ScriptMethod = "Distance2D", Keywords = "magnitude"), Category = "Math|Vector")
	static UE_INL_API double Vector_Distance2D(FVector V1, FVector V2);

	/**
	* Squared euclidean distance between two points in the XY plane (ignoring Z).
	*
	* @param V1 The first point.
	* @param V2 The second point.
	* @return The distance between two points in the XY plane.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Distance2D Squared (Vector)", ScriptMethod = "Distance2DSquared", Keywords = "magnitude"), Category = "Math|Vector")
	static UE_INL_API double Vector_Distance2DSquared(FVector V1, FVector V2);

	/** Returns the length of the vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Vector Length", ScriptMethod = "Length", Keywords="magnitude"), Category="Math|Vector")
	static UE_INL_API double VSize(FVector A);

	/** Returns the squared length of the vector */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Vector Length Squared", ScriptMethod = "LengthSquared", Keywords="magnitude"), Category="Math|Vector")
	static UE_INL_API double VSizeSquared(FVector A);

	/** Returns the length of the vector's XY components. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Vector Length XY", ScriptMethod = "Length2D", Keywords="magnitude"), Category="Math|Vector")
	static UE_INL_API double VSizeXY(FVector A);

	/** Returns the squared length of the vector's XY components. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Vector Length XY Squared", ScriptMethod = "Length2DSquared", Keywords="magnitude"), Category="Math|Vector")
	static UE_INL_API double VSizeXYSquared(FVector A);

	/**
	 * Checks whether vector is near to zero within a specified tolerance.
	 *
	 * @param Tolerance Error tolerance.
	 * @return true if vector is in tolerance to zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsNearlyZero"), Category = "Math|Vector")
	static UE_INL_API bool Vector_IsNearlyZero(const FVector& A, float Tolerance = 1.e-4f);

	/**
	 * Checks whether all components of the vector are exactly zero.
	 *
	 * @return true if vector is exactly zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsZero"), Category = "Math|Vector")
	static UE_INL_API bool Vector_IsZero(const FVector& A);

	/**
	 * Determines if any component is not a number (NAN)
	 *
	 * @return true if one or more components is NAN, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsNAN"), Category = "Math|Vector")
	static UE_INL_API bool Vector_IsNAN(const FVector& A);

	/**
	 * Checks whether all components of this vector are the same, within a tolerance.
	 *
	 * @param Tolerance Error tolerance.
	 * @return true if the vectors are equal within tolerance limits, false otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Uniform (Vector)", ScriptMethod = "IsUniform"), Category="Math|Vector")
	static UE_INL_API bool Vector_IsUniform(const FVector& A, float Tolerance = 1.e-4f);

	/**
	 * Determines if vector is normalized / unit (length 1) within specified squared tolerance.
	 *
	 * @return true if unit, false otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Unit (Vector)", ScriptMethod = "IsUnit", Keywords="Unit Vector"), Category="Math|Vector")
	static UE_INL_API bool Vector_IsUnit(const FVector& A, float SquaredLenthTolerance = 1.e-4f);

	/**
	 * Determines if vector is normalized / unit (length 1).
	 *
	 * @return true if normalized, false otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Normal (Vector)", ScriptMethod = "IsNormal", Keywords="Unit Vector"), Category="Math|Vector")
	static UE_INL_API bool Vector_IsNormal(const FVector& A);

	/**
	 * Gets a normalized unit copy of the vector, ensuring it is safe to do so based on the length.
	 * Returns zero vector if vector length is too small to safely normalize.
	 *
	 * @param Tolerance Minimum squared vector length.
	 * @return A normalized copy if safe, (0,0,0) otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Normalize", ScriptMethod = "Normal", Keywords="Unit Vector"), Category="Math|Vector")
	static UE_INL_API FVector Normal(FVector A, float Tolerance = 1.e-4f);

	/**
	 * Gets a normalized unit copy of the 2D components of the vector, ensuring it is safe to do so. Z is set to zero. 
	 * Returns zero vector if vector length is too small to normalize.
	 *
	 * @param Tolerance Minimum squared vector length.
	 * @return Normalized copy if safe, (0,0,0) otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Normalize 2D (Vector)", ScriptMethod = "Normal2D", Keywords="Unit Vector"), Category="Math|Vector")
	static UE_INL_API FVector Vector_Normal2D(FVector A, float Tolerance = 1.e-4f);

	/**
	 * Calculates normalized unit version of vector without checking for zero length.
	 *
	 * @return Normalized version of vector.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Normal Unsafe (Vector)", ScriptMethod = "NormalUnsafe", Keywords="Unit Vector"), Category="Math|Vector")
	static UE_INL_API FVector Vector_NormalUnsafe(const FVector& A);

	/**
	 * Normalize this vector in-place if it is large enough or set it to (0,0,0) otherwise.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Normalize In Place (Vector)", ScriptMethod = "Normalize", Keywords = "Unit Vector"), Category = "Math|Vector")
	static UE_INL_API void Vector_Normalize(UPARAM(ref) FVector& A, float Tolerance = 1.e-8f);

	/** Linearly interpolates between A and B based on Alpha (100% of A when Alpha=0 and 100% of B when Alpha=1) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Lerp (Vector)", ScriptMethod = "LerpTo"), Category="Math|Vector")
	static UE_INL_API FVector VLerp(FVector A, FVector B, float Alpha);

	/** Easing between A and B using a specified easing function */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Ease (Vector)", BlueprintInternalUseOnly = "true"), Category = "Math|Interpolation")
	static ENGINE_API FVector VEase(FVector A, FVector B, float Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp = 2, int32 Steps = 2);

	/**
	 * Tries to reach Target based on distance from Current position, giving a nice smooth feeling when tracking a position.
	 *
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation", meta=(ScriptMethod = "InterpTo", Keywords="position"))
	static UE_INL_API FVector VInterpTo(FVector Current, FVector Target, float DeltaTime, float InterpSpeed);

	/**
	 * Tries to reach Target at a constant rate.
	 *
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Interpolation", meta = (ScriptMethod = "InterpToConstant", Keywords = "position"))
	static UE_INL_API FVector VInterpTo_Constant(FVector Current, FVector Target, float DeltaTime, float InterpSpeed);

	/**
	 * Uses a simple spring model to interpolate a vector from Current to Target.
	 *
	 * @param Current               Current value
	 * @param Target                Target value
	 * @param SpringState           Data related to spring model (velocity, error, etc..) - Create a unique variable per spring
	 * @param Stiffness             How stiff the spring model is (more stiffness means more oscillation around the target value)
	 * @param CriticalDampingFactor How much damping to apply to the spring (0 means no damping, 1 means critically damped which means no oscillation)
	 * @param DeltaTime             Time difference since the last update
	 * @param Mass                  Multiplier that acts like mass on a spring
	 * @param TargetVelocityAmount  If 1 then the target velocity will be calculated and used, which results following the target more closely/without lag. Values down to zero (recommended when using this to smooth data) will progressively disable this effect.
	 * @param bClamp                Whether to use the Min/Max values to clamp the motion
	 * @param MinValue              Clamps the minimum output value and cancels the velocity if it reaches this limit
	 * @param MaxValue              Clamps the maximum output value and cancels the velocity if it reaches this limit
	 * @param bInitializeFromTarget If set then the current value will be set from the target on the first update
	 */
	UFUNCTION(BlueprintCallable,  meta = (ScriptMethod = "InterpSpringTo", Keywords = "position", AdvancedDisplay = "8"), Category = "Math|Interpolation")
	static ENGINE_API FVector VectorSpringInterp(FVector Current, FVector Target, UPARAM(ref) FVectorSpringState& SpringState,
	                                  float Stiffness, float CriticalDampingFactor, float DeltaTime,
	                                  float Mass = 1.f, float TargetVelocityAmount = 1.f, 
	                                  bool bClamp = false, FVector MinValue = FVector(-1.f), FVector MaxValue = FVector(1.f),
	                                  bool bInitializeFromTarget = false);

	/**
	* Uses a simple spring model to interpolate a quaternion from Current to Target.
	*
	* @param Current               Current value
	* @param Target                Target value
	* @param SpringState           Data related to spring model (velocity, error, etc..) - Create a unique variable per spring
	* @param Stiffness             How stiff the spring model is (more stiffness means more oscillation around the target value)
	* @param CriticalDampingFactor How much damping to apply to the spring (0 means no damping, 1 means critically damped which means no oscillation)
	* @param DeltaTime             Time difference since the last update
	* @param Mass                  Multiplier that acts like mass on a spring
	* @param TargetVelocityAmount  If 1 then the target velocity will be calculated and used, which results following the target more closely/without lag. Values down to zero (recommended when using this to smooth data) will progressively disable this effect.
	* @param bInitializeFromTarget If set then the current value will be set from the target on the first update
	*/
	UFUNCTION(BlueprintCallable,  meta = (ScriptMethod = "InterpSpringTo", Keywords = "quaternion", AdvancedDisplay = "8"), Category = "Math|Interpolation")
	static ENGINE_API FQuat QuaternionSpringInterp(FQuat Current, FQuat Target, UPARAM(ref) FQuaternionSpringState& SpringState,
	                                    float Stiffness, float CriticalDampingFactor, float DeltaTime,
	                                    float Mass = 1.f, float TargetVelocityAmount = 1.f, 
                                        bool bInitializeFromTarget = false);

	/**
	 * Gets the reciprocal of this vector, avoiding division by zero.
	 * Zero components are set to BIG_NUMBER.
	 *
	 * @return Reciprocal of this vector.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Reciprocal (Vector)", ScriptMethod = "Reciprocal"), Category="Math|Vector")
	static UE_INL_API FVector Vector_Reciprocal(const FVector& A);

	/** 
	 * Given a direction vector and a surface normal, returns the vector reflected across the surface normal.
	 * Produces a result like shining a laser at a mirror!
	 *
	 * @param Direction Direction vector the ray is coming from.
	 * @param SurfaceNormal A normal of the surface the ray should be reflected on.
	 *
	 * @returns Reflected vector.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "MirrorByVector", Keywords = "Reflection"), Category="Math|Vector")
	static UE_INL_API FVector GetReflectionVector(FVector Direction, FVector SurfaceNormal);

	/** 
	 * Given a direction vector and a surface normal, returns the vector reflected across the surface normal.
	 * Produces a result like shining a laser at a mirror!
	 *
	 * @param InVect Direction vector the ray is coming from.
	 * @param InNormal A normal of the surface the ray should be reflected on.
	 *
	 * @returns Reflected vector.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Vector")
	static UE_INL_API FVector MirrorVectorByNormal(FVector InVect, FVector InNormal);

	/**
	 * Mirrors a vector about a plane.
	 *
	 * @param Plane Plane to mirror about.
	 * @return Mirrored vector.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "MirrorByPlane", Keywords = "Reflection"), Category="Math|Vector")
	static UE_INL_API FVector Vector_MirrorByPlane(FVector A, const FPlane& InPlane);

	/**
	 * Gets a copy of this vector snapped to a grid.
	 *
	 * @param InGridSize Grid dimension / step.
	 * @return A copy of this vector snapped to a grid.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "SnappedToGrid", Keywords = "Bounding"), Category="Math|Vector")
	static UE_INL_API FVector Vector_SnappedToGrid(FVector InVect, float InGridSize);

	/**
	 * Get a copy of this vector, clamped inside of an axis aligned cube centered at the origin.
	 *
	 * @param InRadius Half size of the cube (or radius of sphere circumscribed in the cube).
	 * @return A copy of this vector, bound by cube.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "BoundedToCube", Keywords = "Bounding"), Category="Math|Vector")
	static UE_INL_API FVector Vector_BoundedToCube(FVector InVect, float InRadius);

	/**
	 * Add a vector to this and clamp the result to an axis aligned cube centered at the origin.
	 *
	 * @param InAddVect Vector to add.
	 * @param InRadius Half size of the cube.
	 */
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod = "AddBounded", Keywords = "Bounding"), Category="Math|Vector")
	static ENGINE_API void Vector_AddBounded(UPARAM(ref) FVector& A, FVector InAddVect, float InRadius);

	/** Get a copy of this vector, clamped inside of the specified axis aligned cube. */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "BoundedToBox", Keywords = "Bounding"), Category="Math|Vector")
	static UE_INL_API FVector Vector_BoundedToBox(FVector InVect, FVector InBoxMin, FVector InBoxMax);

	/**
	 * Gets a copy of this vector projected onto the input vector, which is assumed to be unit length.
	 * 
	 * @param  InNormal Vector to project onto (assumed to be unit length).
	 * @return Projected vector.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ProjectOnToNormal", Keywords = "Project"), Category="Math|Vector")
	static UE_INL_API FVector Vector_ProjectOnToNormal(FVector V, FVector InNormal);

	/**
	* Projects one vector (V) onto another (Target) and returns the projected vector.
	* If Target is nearly zero in length, returns the zero vector.
	*
	* @param  V Vector to project.
	* @param  Target Vector on which we are projecting.
	* @return V projected on to Target.
	*/
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ProjectOnTo", Keywords = "Project"), Category="Math|Vector")
	static UE_INL_API FVector ProjectVectorOnToVector(FVector V, FVector Target);

	/**
	 * Projects/snaps a point onto a plane defined by a point on the plane and a plane normal.
	 *
	 * @param  Point Point to project onto the plane.
	 * @param  PlaneBase A point on the plane.
	 * @param  PlaneNormal Normal of the plane.
	 * @return Point projected onto the plane.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ProjectPointOnToPlane", Keywords = "Project"), Category = "Math|Vector")
	static UE_INL_API FVector ProjectPointOnToPlane(FVector Point, FVector PlaneBase, FVector PlaneNormal);

	/**
	* Projects a vector onto a plane defined by a normalized vector (PlaneNormal).
	*
	* @param  V Vector to project onto the plane.
	* @param  PlaneNormal Normal of the plane.
	* @return Vector projected onto the plane.
	*/
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ProjectOnToPlane", Keywords = "Project"), Category="Math|Vector")
	static UE_INL_API FVector ProjectVectorOnToPlane(FVector V, FVector PlaneNormal);

	/**
	 * Find closest points between 2 segments.
	 *
	 * @param	Segment1Start	Start of the 1st segment.
	 * @param	Segment1End		End of the 1st segment.
	 * @param	Segment2Start	Start of the 2nd segment.
	 * @param	Segment2End		End of the 2nd segment.
	 * @param	Segment1Point	Closest point on segment 1 to segment 2.
	 * @param	Segment2Point	Closest point on segment 2 to segment 1.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Vector")
	static UE_INL_API void FindNearestPointsOnLineSegments(FVector Segment1Start, FVector Segment1End, FVector Segment2Start, FVector Segment2End, FVector& Segment1Point, FVector& Segment2Point);
	
	/**
	 * Find the closest point on a segment to a given point.
	 *
	 * @param Point			Point for which we find the closest point on the segment.
	 * @param SegmentStart	Start of the segment.
	 * @param SegmentEnd	End of the segment.
	 * @return The closest point on the segment to the given point.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Vector")
	static UE_INL_API FVector FindClosestPointOnSegment(FVector Point, FVector SegmentStart, FVector SegmentEnd);

	/**
	 * Find the closest point on an infinite line to a given point.
	 *
	 * @param Point			Point for which we find the closest point on the line.
	 * @param LineOrigin	Point of reference on the line.
	 * @param LineDirection Direction of the line. Not required to be normalized.
	 * @return The closest point on the line to the given point.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Vector")
	static ENGINE_API FVector FindClosestPointOnLine(FVector Point, FVector LineOrigin, FVector LineDirection);

	/**
	* Find the distance from a point to the closest point on a segment.
	*
	* @param Point			Point for which we find the distance to the closest point on the segment.
	* @param SegmentStart	Start of the segment.
	* @param SegmentEnd		End of the segment.
	* @return The distance from the given point to the closest point on the segment.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Vector")
	static UE_INL_API float GetPointDistanceToSegment(FVector Point, FVector SegmentStart, FVector SegmentEnd);

	/**
	* Find the distance from a point to the closest point on an infinite line.
	*
	* @param Point			Point for which we find the distance to the closest point on the line.
	* @param LineOrigin		Point of reference on the line.
	* @param LineDirection	Direction of the line. Not required to be normalized.
	* @return The distance from the given point to the closest point on the line.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Vector")
	static UE_INL_API float GetPointDistanceToLine(FVector Point, FVector LineOrigin, FVector LineDirection);

	/** Returns a random vector with length of 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(NotBlueprintThreadSafe))
	static ENGINE_API FVector RandomUnitVector();

	/** Returns a random point within the specified bounding box using the first vector as an origin and the second as the box extents. */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(ScriptMethod = "RandomPointInBoxExtents", NotBlueprintThreadSafe))
	static UE_INL_API FVector RandomPointInBoundingBox(const FVector Center, const FVector HalfSize);

	/** Returns a random point within the specified bounding box. */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(DisplayName = "Random Point In Bounding Box (Box)", ScriptMethod = "RandomPointInBoxExtents", NotBlueprintThreadSafe))
	static UE_INL_API FVector RandomPointInBoundingBox_Box(const FBox Box);

	/** 
	 * Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	 * @param ConeDir					The base "center" direction of the cone.
	 * @param ConeHalfAngleInRadians	The half-angle of the cone (from ConeDir to edge), in radians.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(NotBlueprintThreadSafe))
	static UE_INL_API FVector RandomUnitVectorInConeInRadians(FVector ConeDir, float ConeHalfAngleInRadians);

	/** 
	 * Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	 * @param ConeDir					The base "center" direction of the cone.
	 * @param ConeHalfAngleInDegrees	The half-angle of the cone (from ConeDir to edge), in degrees.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta=(NotBlueprintThreadSafe))
	static FVector RandomUnitVectorInConeInDegrees(FVector ConeDir, float ConeHalfAngleInDegrees)
	{
		return RandomUnitVectorInConeInRadians(ConeDir, FMath::DegreesToRadians(ConeHalfAngleInDegrees));
	}

	/**
	* Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	* The shape of the cone can be modified according to the yaw and pitch angles.
	*
	* @param MaxYawInRadians	The yaw angle of the cone (from ConeDir to horizontal edge), in radians.
	* @param MaxPitchInRadians	The pitch angle of the cone (from ConeDir to vertical edge), in radians.	
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (Keywords = "RandomVector Pitch Yaw", NotBlueprintThreadSafe))
	static ENGINE_API FVector RandomUnitVectorInEllipticalConeInRadians(FVector ConeDir, float MaxYawInRadians, float MaxPitchInRadians);

	/**
	* Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	* The shape of the cone can be modified according to the yaw and pitch angles.
	*
	* @param MaxYawInDegrees	The yaw angle of the cone (from ConeDir to horizontal edge), in degrees.
	* @param MaxPitchInDegrees	The pitch angle of the cone (from ConeDir to vertical edge), in degrees.	
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (Keywords = "RandomVector Pitch Yaw", NotBlueprintThreadSafe))
	static FVector RandomUnitVectorInEllipticalConeInDegrees(FVector ConeDir, float MaxYawInDegrees, float MaxPitchInDegrees)
	{
		return RandomUnitVectorInEllipticalConeInRadians(ConeDir, FMath::DegreesToRadians(MaxYawInDegrees), FMath::DegreesToRadians(MaxPitchInDegrees));
	}


	//
	// Vector4 constants - exposed for scripting
	//

	/** 4D vector zero constant (0,0,0) */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Zero", ScriptConstantHost = "/Script/CoreUObject.Vector4"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_Zero();

	//
	// Vector4 functions
	//

	/** Makes a 4D vector {X, Y, Z, W} */
	UFUNCTION(BlueprintPure, meta = (Keywords = "construct build", NativeMakeFunc), Category = "Math|Vector4")
	static UE_INL_API FVector4 MakeVector4(double X, double Y, double Z, double W);

	/** Breaks a 4D vector apart into X, Y, Z, W. */
	UFUNCTION(BlueprintPure, meta = (NativeBreakFunc), Category = "Math|Vector4")
	static UE_INL_API void BreakVector4(const FVector4& InVec, double& X, double& Y, double& Z, double& W);

	/** Converts a Vector4 to a Vector (dropping the W element) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (Vector4)", CompactNodeTitle = "->", ScriptMethod = "Vector", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_Vector4ToVector(const FVector4& InVector4);

	/**
	 * Return the FRotator orientation corresponding to the direction in which the vector points.
	 * Sets Yaw and Pitch to the proper numbers, and sets Roll to zero because the roll can't be determined from a vector.
	 *
	 * @return FRotator from the Vector's direction, without any roll.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Rotation (Vector4)", ScriptMethod = "Rotator", Keywords = "rotation rotate cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FRotator Conv_Vector4ToRotator(const FVector4& InVec);
	
	/**
	 * Return the Quaternion orientation corresponding to the direction in which the vector points.
	 * Similar to the FRotator version, returns a result without roll such that it preserves the up vector.
	 *
	 * @note If you don't care about preserving the up vector and just want the most direct rotation, you can use the faster
	 * 'FindBetweenVectors(ForwardVector, YourVector)' or 'FindBetweenNormals(...)' if you know the vector is of unit length.
	 *
	 * @return Quaternion from the Vector's direction, without any roll.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Quaternion (Vector4)", ScriptMethod = "Quaternion", Keywords = "rotation rotate cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FQuat Conv_Vector4ToQuaternion(const FVector4& InVec);

	/** Returns addition of Vector A and Vector B (A + B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector4 + Vector4", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Add_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Returns subtraction of Vector B from Vector A (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector4 - Vector4", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Subtract_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Element-wise Vector multiplication (Result = {A.x*B.x, A.y*B.y, A.z*B.z, A.w*B.w}) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector4 * Vector4", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Multiply_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Element-wise Vector divide (Result = {A.x/B.x, A.y/B.y, A.z/B.z, A.w/B.w}) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector4 / Vector4", CompactNodeTitle = "/", ScriptMethod = "Divide", ScriptOperator = "/;/=", Keywords = "/ divide division"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Divide_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Returns true if vector A is equal to vector B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal Exactly (Vector4)", CompactNodeTitle = "===", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|Vector4")
	static UE_INL_API bool EqualExactly_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Returns true if vector A is equal to vector B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Vector4)", CompactNodeTitle = "==", ScriptMethod = "IsNearEqual", Keywords = "== equal"), Category = "Math|Vector4")
	static UE_INL_API bool EqualEqual_Vector4Vector4(const FVector4& A, const FVector4& B, float ErrorTolerance = 1.e-4f);

	/** Returns true if vector A is not equal to vector B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal Exactly (Vector4)", CompactNodeTitle = "!==", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Vector4")
	static UE_INL_API bool NotEqualExactly_Vector4Vector4(const FVector4& A, const FVector4& B);

	/** Returns true if vector A is not equal to vector B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (Vector4)", CompactNodeTitle = "!=", ScriptMethod = "IsNotNearEqual", Keywords = "!= not equal"), Category = "Math|Vector4")
	static UE_INL_API bool NotEqual_Vector4Vector4(const FVector4& A, const FVector4& B, float ErrorTolerance = 1.e-4f);

	/** Gets a negated copy of the vector. Equivalent to -Vector for scripts. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Negated (Vector4)", ScriptMethod = "Negated", ScriptOperator = "neg"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_Negated(const FVector4& A);

	/**
	 * Assign the values of the supplied vector.
	 *
	 * @param InVector Vector to copy values from.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Assign"), Category = "Math|Vector4")
	static UE_INL_API void Vector4_Assign(UPARAM(ref) FVector4& A, const FVector4& InVector);

	/**
	 * Set the values of the vector directly.
	 *
	 * @param InX New X coordinate.
	 * @param InY New Y coordinate.
	 * @param InZ New Z coordinate.
	 * @param InW New W coordinate.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Set"), Category = "Math|Vector4")
	static UE_INL_API void Vector4_Set(UPARAM(ref) FVector4& A, double X, double Y, double Z, double W);

	/** Returns the cross product of two vectors - see  http://mathworld.wolfram.com/CrossProduct.html */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Cross Product XYZ (Vector4)", CompactNodeTitle = "cross3", ScriptMethod = "Cross3"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_CrossProduct3(const FVector4& A, const FVector4& B);

	/** Returns the dot product of two vectors - see http://mathworld.wolfram.com/DotProduct.html */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Dot Product (Vector4)", CompactNodeTitle = "dot", ScriptMethod = "Dot", ScriptOperator = "|"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_DotProduct(const FVector4& A, const FVector4& B);

	/** Returns the dot product of two vectors - see http://mathworld.wolfram.com/DotProduct.html The W element is ignored.*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Dot Product XYZ (Vector4)", CompactNodeTitle = "dot3", ScriptMethod = "Dot3"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_DotProduct3(const FVector4& A, const FVector4& B);

	/**
	 * Determines if any component is not a number (NAN)
	 *
	 * @return true if one or more components is NAN, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsNAN"), Category = "Math|Vector4")
	static UE_INL_API bool Vector4_IsNAN(const FVector4& A);

	/**
	 * Checks whether vector is near to zero within a specified tolerance. The W element is ignored.
	 *
	 * @param Tolerance Error tolerance.
	 * @return true if vector is in tolerance to zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsNearlyZero3"), Category = "Math|Vector4")
	static UE_INL_API bool Vector4_IsNearlyZero3(const FVector4& A, float Tolerance = 1.e-4f);

	/**
	 * Checks whether all components of the vector are exactly zero.
	 *
	 * @return true if vector is exactly zero, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "IsZero"), Category = "Math|Vector4")
	static UE_INL_API bool Vector4_IsZero(const FVector4& A);

	/** Returns the length of the vector. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Length (Vector4)", ScriptMethod = "Length", Keywords = "magnitude"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_Size(const FVector4& A);

	/** Returns the squared length of the vector. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Length Squared (Vector4)", ScriptMethod = "LengthSquared", Keywords = "magnitude"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_SizeSquared(const FVector4& A);

	/** Returns the length of the vector. The W element is ignored. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Length XYZ (Vector4)", ScriptMethod = "Length3", Keywords = "magnitude"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_Size3(const FVector4& A);

	/** Returns the squared length of the vector. The W element is ignored. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Length XYZ Squared (Vector4)", ScriptMethod = "LengthSquared3", Keywords = "magnitude"), Category = "Math|Vector4")
	static UE_INL_API double Vector4_SizeSquared3(const FVector4& A);

	/**
	 * Determines if vector is normalized / unit (length 1) within specified squared tolerance. The W element is ignored.
	 *
	 * @return true if unit, false otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Unit XYZ (Vector4)", ScriptMethod = "IsUnit3", Keywords = "Unit Vector"), Category = "Math|Vector4")
	static UE_INL_API bool Vector4_IsUnit3(const FVector4& A, float SquaredLenthTolerance = 1.e-4f);

	/**
	 * Determines if vector is normalized / unit (length 1). The W element is ignored.
	 *
	 * @return true if normalized, false otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Normal XYZ (Vector4)", ScriptMethod = "IsNormal3", Keywords = "Unit Vector"), Category = "Math|Vector4")
	static UE_INL_API bool Vector4_IsNormal3(const FVector4& A);

	/**
	 * Gets a normalized unit copy of the vector, ensuring it is safe to do so based on the length. The W element is ignored and the returned vector has W=0.
	 * Returns zero vector if vector length is too small to safely normalize.
	 *
	 * @param Tolerance Minimum squared vector length.
	 * @return A normalized copy if safe, (0,0,0) otherwise.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Normalize XYZ (Vector 4)", ScriptMethod = "Normal3", Keywords = "Unit Vector"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_Normal3(const FVector4& A, float Tolerance = 1.e-4f);

	/**
	 * Calculates normalized unit version of vector without checking for zero length. The W element is ignored and the returned vector has W=0.
	 *
	 * @return Normalized version of vector.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Normal Unsafe XYZ (Vector4)", ScriptMethod = "NormalUnsafe3", Keywords = "Unit Vector"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_NormalUnsafe3(const FVector4& A);

	/**
	 * Normalize this vector in-place if it is large enough or set it to (0,0,0,0) otherwise. The W element is ignored and the returned vector has W=0.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Normalize In Place XYZ (Vector4)", ScriptMethod = "Normalize3", Keywords = "Unit Vector"), Category = "Math|Vector4")
	static UE_INL_API void Vector4_Normalize3(UPARAM(ref) FVector4& A, float Tolerance = 1.e-8f);

	/** 
	 * Given a direction vector and a surface normal, returns the vector reflected across the surface normal.
	 * Produces a result like shining a laser at a mirror!
	 * The W element is ignored.
	 *
	 * @param Direction Direction vector the ray is coming from.
	 * @param SurfaceNormal A normal of the surface the ray should be reflected on.
	 *
	 * @returns Reflected vector.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "MirrorByVector3", Keywords = "Reflection"), Category = "Math|Vector4")
	static UE_INL_API FVector4 Vector4_MirrorByVector3(const FVector4& Direction, const FVector4& SurfaceNormal);

	/**
	 * Transform the input vector4 by a provided matrix4x4 and returns the resulting vector4.
	 *
	 * @return Transformed vector4.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Transform Vector4 by Matrix"), Category = "Math|Vector4")
	static UE_INL_API FVector4 TransformVector4(const FMatrix& Matrix, const FVector4& Vec4);

	//
	// Rotator functions.
	//

	/** Makes a rotator {Roll, Pitch, Yaw} from rotation values supplied in degrees */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator", NativeMakeFunc))
	static UE_INL_API FRotator MakeRotator(
		UPARAM(DisplayName="X (Roll)") float Roll,	
		UPARAM(DisplayName="Y (Pitch)") float Pitch,
		UPARAM(DisplayName="Z (Yaw)") float Yaw);

	/** Builds a rotator given only a XAxis. Y and Z are unspecified but will be orthonormal. XAxis need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromX(const FVector& X);

	/** Builds a rotation matrix given only a YAxis. X and Z are unspecified but will be orthonormal. YAxis need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromY(const FVector& Y);

	/** Builds a rotation matrix given only a ZAxis. X and Y are unspecified but will be orthonormal. ZAxis need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromZ(const FVector& Z);

	/** Builds a matrix with given X and Y axes. X will remain fixed, Y may be changed minimally to enforce orthogonality. Z will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromXY(const FVector& X, const FVector& Y);

	/** Builds a matrix with given X and Z axes. X will remain fixed, Z may be changed minimally to enforce orthogonality. Y will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromXZ(const FVector& X, const FVector& Z);

	/** Builds a matrix with given Y and X axes. Y will remain fixed, X may be changed minimally to enforce orthogonality. Z will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromYX(const FVector& Y, const FVector& X);

	/** Builds a matrix with given Y and Z axes. Y will remain fixed, Z may be changed minimally to enforce orthogonality. X will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromYZ(const FVector& Y, const FVector& Z);

	/** Builds a matrix with given Z and X axes. Z will remain fixed, X may be changed minimally to enforce orthogonality. Y will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromZX(const FVector& Z, const FVector& X);

	/** Builds a matrix with given Z and Y axes. Z will remain fixed, Y may be changed minimally to enforce orthogonality. X will be computed. Inputs need not be normalized. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate rotator makerotator"))
	static UE_INL_API FRotator MakeRotFromZY(const FVector& Z, const FVector& Y);

	// Build a reference frame from three axes
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="construct build rotation rotate"))
	static ENGINE_API FRotator MakeRotationFromAxes(FVector Forward, FVector Right, FVector Up);

	/** Find a rotation for an object at Start location to point at Target location. */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="rotation rotate"))
	static UE_INL_API FRotator FindLookAtRotation(const FVector& Start, const FVector& Target);

	/** 
	 * Find a local rotation (range of [-180, 180]) for an object with StartTransform to point at TargetLocation. 
	 * Useful for getting LookAt Azimuth or Pawn Aim Offset.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Rotator", meta = (Keywords = "rotation rotate local azimuth"))
	static ENGINE_API FRotator FindRelativeLookAtRotation(const FTransform& StartTransform, const FVector& TargetLocation);

	/** Breaks apart a rotator into {Roll, Pitch, Yaw} angles in degrees */
	UFUNCTION(BlueprintPure, Category = "Math|Rotator", meta = (Keywords = "rotation rotate rotator breakrotator", NativeBreakFunc))
	static UE_INL_API void BreakRotator(
		UPARAM(DisplayName="Rotation") FRotator InRot,
		UPARAM(DisplayName="X (Roll)") float& Roll,
		UPARAM(DisplayName="Y (Pitch)") float& Pitch,
		UPARAM(DisplayName="Z (Yaw)") float& Yaw);

	/** Breaks apart a rotator into its component axes */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="rotation rotate rotator breakrotator"))
	static ENGINE_API void BreakRotIntoAxes(const FRotator& InRot, FVector& X, FVector& Y, FVector& Z);

	/** Returns true if rotator A is equal to rotator B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Rotator)", CompactNodeTitle = "==", ScriptMethod = "IsNearEqual", ScriptOperator = "==", Keywords = "== equal"), Category="Math|Rotator")
	static UE_INL_API bool EqualEqual_RotatorRotator(FRotator A, FRotator B, float ErrorTolerance = 1.e-4f);

	/** Returns true if rotator A is not equal to rotator B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Rotator)", CompactNodeTitle = "!=", ScriptMethod = "IsNotNearEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category="Math|Rotator")
	static UE_INL_API bool NotEqual_RotatorRotator(FRotator A, FRotator B, float ErrorTolerance = 1.e-4f);

	/** Returns rotator representing rotator A scaled by B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Scale Rotator (Float)", CompactNodeTitle = "*", ScriptMethod = "Scale", Keywords = "* multiply rotate rotation"), Category="Math|Rotator")
	static UE_INL_API FRotator Multiply_RotatorFloat(FRotator A, float B);
	
	/** Returns rotator representing rotator A scaled by B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Scale Rotator (Integer)", CompactNodeTitle = "*", ScriptMethod = "ScaleInteger", Keywords = "* multiply rotate rotation"), Category="Math|Rotator")
	static UE_INL_API FRotator Multiply_RotatorInt(FRotator A, int32 B);

	/** Combine 2 rotations to give you the resulting rotation of first applying A, then B. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Combine Rotators", ScriptMethod = "Combine", Keywords="rotate rotation add"), Category="Math|Rotator")
	static ENGINE_API FRotator ComposeRotators(FRotator A, FRotator B);

	/** Negate a rotator*/
	UFUNCTION(BlueprintPure, meta=(DisplayName="Invert Rotator", ScriptMethod = "Inversed", ScriptOperator = "neg", Keywords="rotate rotation"), Category="Math|Rotator")
	static UE_INL_API FRotator NegateRotator(FRotator A);

	/** Rotate the world forward vector by the given rotation */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetForwardVector", Keywords="rotation rotate"), Category="Math|Vector")
	static ENGINE_API FVector GetForwardVector(FRotator InRot);

	/** Rotate the world right vector by the given rotation */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetRightVector", Keywords="rotation rotate"), Category="Math|Vector")
	static ENGINE_API FVector GetRightVector(FRotator InRot);

	/** Rotate the world up vector by the given rotation */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetUpVector", Keywords="rotation rotate"), Category="Math|Vector")
	static ENGINE_API FVector GetUpVector(FRotator InRot);

	/** Get the X direction vector after this rotation */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "ToVector", DisplayName = "Get Rotation X Vector", Keywords="rotation rotate cast convert", BlueprintAutocast), Category="Math|Rotator")
	static UE_INL_API FVector Conv_RotatorToVector(FRotator InRot);

	/** Converts Rotator to Transform */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Transform (Rotator)", CompactNodeTitle = "->", ScriptMethod = "Transform", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FTransform Conv_RotatorToTransform(const FRotator& InRotator);

	/** Get the reference frame direction vectors (axes) described by this rotation */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "GetAxes", Keywords="rotate rotation"), Category="Math|Rotator")
	static ENGINE_API void GetAxes(FRotator A, FVector& X, FVector& Y, FVector& Z);

	/** Generates a random rotation, with optional random roll. */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(Keywords="rotate rotation", NotBlueprintThreadSafe))
	static ENGINE_API FRotator RandomRotator(bool bRoll = false);

	/** Linearly interpolates between A and B based on Alpha (100% of A when Alpha=0 and 100% of B when Alpha=1) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Lerp (Rotator)", ScriptMethod = "Lerp"), Category="Math|Rotator")
	static ENGINE_API FRotator RLerp(FRotator A, FRotator B, float Alpha, bool bShortestPath);

	/** Easing between A and B using a specified easing function */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Ease (Rotator)", BlueprintInternalUseOnly = "true", ScriptMethod = "Ease"), Category = "Math|Interpolation")
	static ENGINE_API FRotator REase(FRotator A, FRotator B, float Alpha, bool bShortestPath, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp = 2, int32 Steps = 2);

	/** Normalized A-B */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Delta (Rotator)", ScriptMethod = "Delta"), Category="Math|Rotator")
	static ENGINE_API FRotator NormalizedDeltaRotator(FRotator A, FRotator B);

	/**
	* Clamps an angle to the range of [0, 360].
	*
	* @param Angle The angle to clamp.
	* @return The clamped angle.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Rotator")
	static ENGINE_API float ClampAxis(float Angle);

	/**
	* Clamps an angle to the range of [-180, 180].
	*
	* @param Angle The Angle to clamp.
	* @return The clamped angle.
	*/
	UFUNCTION(BlueprintPure, Category="Math|Rotator")
	static ENGINE_API float NormalizeAxis(float Angle);


	//
	// Matrix functions
	//

	/** Converts a Matrix to a Transform 
	* (Assumes Matrix represents a transform) 
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Transform (Matrix)", CompactNodeTitle = "->", ScriptMethod = "Transform", Keywords = "cast convert"), Category = "Math|Conversions")
	static UE_INL_API FTransform Conv_MatrixToTransform(const FMatrix& InMatrix);

	/** Converts a Matrix to a Rotator 
	* (Assumes Matrix represents a transform) 
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Rotator (Matrix)", CompactNodeTitle = "->", ScriptMethod = "Rotator", Keywords = "cast convert"), Category = "Math|Conversions")
	static UE_INL_API FRotator Conv_MatrixToRotator(const FMatrix& InMatrix);

	/**
	 * Get the origin of the co-ordinate system
	 * (Assumes Matrix represents a transform)
	 *
	 * @return co-ordinate system origin
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Origin (Matrix)", ScriptMethod = "GetOrigin"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_GetOrigin(const FMatrix& InMatrix);

	// Identity matrix
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Identity (Matrix)", ScriptConstant = "Identity", ScriptConstantHost = "/Script/CoreUObject.Matrix"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_Identity();

	/**
	 * Gets the result of multiplying a Matrix to this.
	 *
	 * @param Other The matrix to multiply this by.
	 * @return The result of multiplication.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Matrix * Matrix", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Multiply_MatrixMatrix (const FMatrix& A, const FMatrix& B);

	/**
	 * Gets the result of adding a matrix to this.
	 *
	 * @param Other The Matrix to add.
	 * @return The result of addition.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Matrix + Matrix", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Add_MatrixMatrix (const FMatrix& A, const FMatrix& B);

	/**
	  * Multiplies all values of the matrix by a float.
	  * If your Matrix represents a Transform that you wish to scale you should use Apply Scale instead
	  */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Matrix * Float", CompactNodeTitle = "*", ScriptMethod = "MultiplyFloat", ScriptOperator = "*;*=", Keywords = "* multiply"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Multiply_MatrixFloat (const FMatrix& A, double B);

	/**
	 * Checks whether another Matrix is equal to this, within specified tolerance.
	 *
	 * @param Other The other Matrix.
	 * @param Tolerance Error Tolerance.
	 * @return true if two Matrix are equal, within specified tolerance, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Matrix)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|Matrix")
	static UE_INL_API bool EqualEqual_MatrixMatrix(const FMatrix& A, const FMatrix& B, float Tolerance = 1.e-4f);

	/**
	 * Checks whether another Matrix is not equal to this, within specified tolerance.
	 *
	 * @param Other The other Matrix.
	 * @return true if two Matrix are not equal, within specified tolerance, otherwise false.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (Matrix)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Matrix")
	static UE_INL_API bool NotEqual_MatrixMatrix(const FMatrix& A, const FMatrix& B, float Tolerance = 1.e-4f);

	/**
	 * Transform a vector by the matrix.
	 * (Assumes Matrix represents a transform) 
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Transform Vector4 (Matrix)", ScriptMethod = "TransformVector4"), Category = "Math|Matrix")
	static UE_INL_API FVector4 Matrix_TransformVector4(const FMatrix& M, FVector4 V);

	/** Transform a location - will take into account translation part of the FMatrix.
	 * (Assumes Matrix represents a transform)
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Transform Position (Matrix)", ScriptMethod = "TransformPosition"), Category = "Math|Matrix")
	static UE_INL_API FVector4 Matrix_TransformPosition(const FMatrix& M, FVector V);

	/** Inverts the matrix and then transforms V - correctly handles scaling in this matrix.
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Inverse Transform Position (Matrix)", ScriptMethod = "InverseTransformPosition"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_InverseTransformPosition(const FMatrix& M, FVector V);

	/**
	 *	Transform a direction vector - will not take into account translation part of the FMatrix.
	 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT.
	 * (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Transform Vector (Matrix)", ScriptMethod = "TransformVector"), Category = "Math|Matrix")
	static UE_INL_API FVector4 Matrix_TransformVector(const FMatrix& M, FVector V);

	/**
	 *	Transform a direction vector by the inverse of this matrix - will not take into account translation part.
	 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT with adjoint of matrix inverse.
	 * (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Inverse Transform Vector (Matrix)", ScriptMethod = "InverseTransformVector"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_InverseTransformVector(const FMatrix& M, FVector V);

	// Transpose.
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Transposed (Matrix)", ScriptMethod = "GetTransposed"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_GetTransposed(const FMatrix& M);

	// @return determinant of this matrix.
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Determinant (Matrix)", ScriptMethod = "GetDeterminant"), Category = "Math|Matrix")
	static UE_INL_API float Matrix_GetDeterminant(const FMatrix& M);

	/** @return the determinant of rotation 3x3 matrix 
	* (Assumes Top Left 3x3 Submatrix represents a Rotation)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Rotation Determinant (Matrix)", ScriptMethod = "GetRotDeterminant"), Category = "Math|Matrix")
	static UE_INL_API float Matrix_GetRotDeterminant(const FMatrix& M);

	/** Get the inverse of the Matrix. Handles nil matrices. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "GetInverse (Matrix)", ScriptMethod = "GetInverse"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_GetInverse(const FMatrix& M);

	/** Get the Transose Adjoint of the Matrix. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Transpose Adjoint (Matrix)", ScriptMethod = "GetTransposeAdjoint"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_GetTransposeAdjoint(const FMatrix& M);

	/** Remove any scaling from this matrix (ie magnitude of each row is 1) with error Tolerance
	* (Assumes Matrix represents a transform) 
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Remove Scaling (Matrix)", ScriptMethod = "RemoveScaling"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_RemoveScaling(UPARAM(Ref) FMatrix& M, float Tolerance = 1.e-8f);

	/** Returns matrix after RemoveScaling with error Tolerance
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Matrix Without Scale (Matrix)", ScriptMethod = "GetMatrixWithoutScale"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_GetMatrixWithoutScale(const FMatrix& M, float Tolerance = 1.e-8f);

	/** return a 3D scale vector calculated from this matrix (where each component is the magnitude of a row vector) with error Tolerance.
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Scale Vector (Matrix)", ScriptMethod = "GetScaleVector"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_GetScaleVector(const FMatrix& M, float Tolerance = 1.e-8f);

	/** Remove any translation from this matrix
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Remove Translation (Matrix)", ScriptMethod = "RemoveTranslation"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_RemoveTranslation(const FMatrix& M);

	/** Returns a matrix with an additional translation concatenated.
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Concatenate Translation (Matrix)", ScriptMethod = "ConcatenateTranslation"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_ConcatenateTranslation(const FMatrix& M, FVector Translation);

	/** Returns true if any element of this matrix is NaN */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Contains NaN (Matrix)", ScriptMethod = "ContainsNaN"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_ContainsNaN(const FMatrix& M);

	/** Scale the translation part of the matrix by the supplied vector.
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Scale Translation (Matrix)", ScriptMethod = "ScaleTranslation"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_ScaleTranslation(const FMatrix& M, FVector Scale3D);

	/** @return the maximum magnitude of any row of the matrix.
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Maximum Axis Scale (Matrix)", ScriptMethod = "GetMaximumAxisScale"), Category = "Math|Matrix")
	static UE_INL_API float Matrix_GetMaximumAxisScale(const FMatrix& M);

	/** Apply Scale to this matrix
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Apply Scale (Matrix)", ScriptMethod = "ApplyScale"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_ApplyScale(const FMatrix& M, float Scale);

	/**
	 * get axis of this matrix scaled by the scale of the matrix
	 * (Assumes Matrix represents a transform)
	 *
	 * @param i index into the axis of the matrix
	 * @ return vector of the axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Scaled Axis (Matrix)", ScriptMethod = "GetScaledAxis"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_GetScaledAxis(const FMatrix& M, TEnumAsByte<EAxis::Type> Axis);

	/**
	 * get axes of this matrix scaled by the scale of the matrix
	 * (Assumes Matrix represents a transform)
	 *
	 * @param X axes returned to this param
	 * @param Y axes returned to this param
	 * @param Z axes returned to this param
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Scaled Axes (Matrix)", ScriptMethod = "GetScaledAxes"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_GetScaledAxes(const FMatrix& M, FVector &X, FVector &Y, FVector &Z);

	/**
	 * get unit length axis of this matrix
	 * (Assumes Matrix represents a transform)
	 *
	 * @param i index into the axis of the matrix
	 * @return vector of the axis
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Unit Axis (Matrix)", ScriptMethod = "GetUnitAxis"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_GetUnitAxis(const FMatrix& M, TEnumAsByte<EAxis::Type> Axis);

	/**
	 * get unit length axes of this matrix
	 * (Assumes Matrix represents a transform)
	 *
	 * @param X axes returned to this param
	 * @param Y axes returned to this param
	 * @param Z axes returned to this param
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Unit Axes (Matrix)", ScriptMethod = "GetUnitAxes"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_GetUnitAxes(const FMatrix& M, FVector &X, FVector &Y, FVector &Z);

	/**
	 * set an axis of this matrix
	 * (Assumes Matrix represents a transform)
	 *
	 * @param i index into the axis of the matrix
	 * @param Axis vector of the axis
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Axis (Matrix)", ScriptMethod = "SetAxis"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_SetAxis(UPARAM(Ref) FMatrix& M, TEnumAsByte<EAxis::Type> Axis, FVector AxisVector);

	/** Set the origin of the coordinate system to the given vector
	* (Assumes Matrix represents a transform)
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Origin (Matrix)", ScriptMethod = "SetOrigin"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_SetOrigin(UPARAM(Ref) FMatrix& M, FVector NewOrigin);

	/**
	 * get a column of this matrix
	 *
	 * @param i index into the column of the matrix
	 * @return vector of the column
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Column (Matrix)", ScriptMethod = "GetColumn"), Category = "Math|Matrix")
	static UE_INL_API FVector Matrix_GetColumn(const FMatrix& M, TEnumAsByte<EMatrixColumns::Type> Column);

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Column (Matrix)", ScriptMethod = "SetColumn"), Category = "Math|Matrix")
	static UE_INL_API void Matrix_SetColumn(UPARAM(Ref) FMatrix& M, TEnumAsByte<EMatrixColumns::Type> Column, FVector Value);

	/** 
	* Get the rotator representation of this matrix
	* (Assumes Matrix represents a transform)
	*@return rotator representation of this matrix
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Rotator (Matrix)", ScriptMethod = "GetRotator"), Category = "Math|Matrix")
	static UE_INL_API FRotator Matrix_GetRotator(const FMatrix& M);

	/**
	 * Transform a rotation matrix into a quaternion.
	 * (Assumes Matrix represents a transform)
	 *
	 * @warning rotation part will need to be unit length for this to be right!
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Quat (Matrix)", ScriptMethod = "ToQuat"), Category = "Math|Matrix")
	static UE_INL_API FQuat Matrix_ToQuat(const FMatrix& M);

	// Frustum plane extraction.

	/** Get the near plane of the Frustum of this matrix 
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the near plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Near Plane (Matrix)", ScriptMethod = "GetFrustumNearPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumNearPlane(const FMatrix& M, FPlane& OutPlane);

	/** Get the far plane of the Frustum of this matrix
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the far plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Far Plane (Matrix)", ScriptMethod = "GetFrustumFarPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumFarPlane(const FMatrix& M, FPlane& OutPlane);

	/** Get the left plane of the Frustum of this matrix
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the left plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Left Plane (Matrix)", ScriptMethod = "GetFrustumLeftPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumLeftPlane(const FMatrix& M, FPlane& OutPlane);

	/** Get the right plane of the Frustum of this matrix
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the right plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Right Plane (Matrix)", ScriptMethod = "GetFrustumRightPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumRightPlane(const FMatrix& M, FPlane& OutPlane);

	/** Get the top plane of the Frustum of this matrix
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the top plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Top Plane (Matrix)", ScriptMethod = "GetFrustumTopPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumTopPlane(const FMatrix& M, FPlane& OutPlane);

	/** Get the bottom plane of the Frustum of this matrix
	 * (Assumes Matrix represents a View Projection Matrix)
	 * @param OutPlane the bottom plane of the Frustum of this matrix 
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Frustum Bottom Plane (Matrix)", ScriptMethod = "GetFrustumBottomPlane"), Category = "Math|Matrix")
	static UE_INL_API bool Matrix_GetFrustumBottomPlane(const FMatrix& M, FPlane& OutPlane);

	/**
	 * Utility for mirroring this transform across a certain plane, and flipping one of the axis as well.
	 * (Assumes Matrix represents a transform)
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Mirror (Matrix)", ScriptMethod = "Mirror"), Category = "Math|Matrix")
	static UE_INL_API FMatrix Matrix_Mirror(const FMatrix& M, TEnumAsByte<EAxis::Type> MirrorAxis, TEnumAsByte<EAxis::Type> FlipAxis);

	//
	// Quat constants - exposed for scripting
	//

	/** Identity quaternion constant */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Identity", ScriptConstantHost = "/Script/CoreUObject.Quat"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Identity();

	//
	// Quat functions
	//

	/** Returns true if Quaternion A is equal to Quaternion B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Quat)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|Quat")
	static UE_INL_API bool EqualEqual_QuatQuat(const FQuat& A, const FQuat& B, float Tolerance = 1.e-4f);

	/** Returns true if Quat A is not equal to Quat B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (Quat)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Quat")
	static UE_INL_API bool NotEqual_QuatQuat(const FQuat& A, const FQuat& B, float ErrorTolerance = 1.e-4f);

	/** Returns addition of Vector A and Vector B (A + B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Quat + Quat", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Quat")
	static UE_INL_API FQuat Add_QuatQuat(const FQuat& A, const FQuat& B);

	/** Returns subtraction of Vector B from Vector A (A - B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Quat - Quat", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category = "Math|Quat")
	static UE_INL_API FQuat Subtract_QuatQuat(const FQuat& A, const FQuat& B);

	/** Makes a quat {X, Y, Z, W} */
	UFUNCTION(BlueprintPure, Category = "Math|Quat", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_INL_API FQuat MakeQuat(float X, float Y, float Z, float W);

	/** Breaks a quat apart into X, Y, Z, W */
	UFUNCTION(BlueprintPure, Category = "Math|Quat", meta = (NativeBreakFunc))
	static UE_INL_API void BreakQuat(const FQuat& InQuat, float& X, float& Y, float& Z, float& W);

	/**
	 * Gets the result of multiplying two quaternions (A * B).
	 *
	 * Order matters when composing quaternions: C = A * B will yield a quaternion C that logically
	 * first applies B then A to any subsequent transformation (right first, then left).
	 *
	 * @param B The Quaternion to multiply by.
	 * @return The result of multiplication (A * B).
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Quat * Quat", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply", CommutativeAssociativeBinaryOperator = "true"), Category = "Math|Quat")
	static UE_INL_API FQuat Multiply_QuatQuat(const FQuat& A, const FQuat& B);

	/**
	 * Checks whether this Quaternion is an Identity Quaternion.
	 * Assumes Quaternion tested is normalized.
	 *
	 * @param Tolerance Error tolerance for comparison with Identity Quaternion.
	 * @return true if Quaternion is a normalized Identity Quaternion.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Identity (Quat)", ScriptMethod = "IsIdentity"), Category = "Math|Quat")
	static UE_INL_API bool Quat_IsIdentity(const FQuat& Q, float Tolerance = 1.e-4f);

	/**	Return true if this quaternion is normalized */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Normalized (Quat)", ScriptMethod = "IsNormalized"), Category = "Math|Quat")
	static UE_INL_API bool Quat_IsNormalized(const FQuat& Q);

	/** Determine if all the values  are finite (not NaN nor Inf) in this Quat.	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Finite (Quat)", ScriptMethod = "IsFinite"), Category = "Math|Quat")
	static UE_INL_API bool Quat_IsFinite(const FQuat& Q);

	/** Determine if there are any non-finite values (NaN or Inf) in this Quat.	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Non-Finite (Quat)", ScriptMethod = "IsNonFinite"), Category = "Math|Quat")
	static UE_INL_API bool Quat_IsNonFinite(const FQuat& Q);

	/**
	 * Find the angular distance/difference between two rotation quaternions.
	 *
	 * @param B Quaternion to find angle distance to
	 * @return angular distance in radians
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Angular Distance (Quat)", ScriptMethod = "AngularDistance"), Category = "Math|Quat")
	static UE_INL_API float Quat_AngularDistance(const FQuat& A, const FQuat& B);

	/** Modify the quaternion to ensure that the delta between it and B represents the shortest possible rotation angle. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Ensure Shortest Arc To (Quat)", ScriptMethod = "EnsureShortestArcTo"), Category = "Math|Quat")
	static UE_INL_API void Quat_EnforceShortestArcWith(UPARAM(ref) FQuat& A, const FQuat& B);

	/** Returns the version of this quaternion that represents the shortest rotation angle to the other quaternion */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Shortest Arc To (Quat)", ScriptMethod = "GetShortestArcTo"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_GetShortestArcWith(const FQuat& A, const FQuat& B);

	/**	Convert a Quaternion into floating-point Euler angles (in degrees). */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Euler (Quat)", ScriptMethod = "Euler"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_Euler(const FQuat& Q);

	/**
	 * Used in combination with Log().
	 * Assumes a quaternion with W=0 and V=theta*v (where |v| = 1).
	 * Exp(q) = (sin(theta)*v, cos(theta))
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Exp (Quat)", ScriptMethod = "Exp"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Exp(const FQuat& Q);

	/** Get the angle of this quaternion in radians */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Angle (Quat)", ScriptMethod = "GetAngle"), Category = "Math|Quat")
	static UE_INL_API float Quat_GetAngle(const FQuat& Q);

	/** Get the forward direction (X axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Axis X (Quat)", ScriptMethod = "GetAxisX"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_GetAxisX(const FQuat& Q);

	/** Get the right direction (Y axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Axis Y (Quat)", ScriptMethod = "GetAxisY"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_GetAxisY(const FQuat& Q);

	/** Get the up direction (Z axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Axis Z (Quat)", ScriptMethod = "GetAxisZ"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_GetAxisZ(const FQuat& Q);

	/** Get the forward direction (X axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector Forward (Quat)", ScriptMethod = "VectorForward"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_VectorForward(const FQuat& Q);

	/** Get the right direction (Y axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector Right (Quat)", ScriptMethod = "VectorRight"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_VectorRight(const FQuat& Q);

	/** Get the up direction (Z axis) after it has been rotated by this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector Up (Quat)", ScriptMethod = "VectorUp"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_VectorUp(const FQuat& Q);

	/**
	 * Normalize this quaternion if it is large enough as compared to the supplied tolerance.
	 * If it is too small then set it to the identity quaternion.
	 *
	 * @param Tolerance Minimum squared length of quaternion for normalization.
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Normalize (Quat)", ScriptMethod = "Normalize"), Category = "Math|Quat")
	static UE_INL_API void Quat_Normalize(UPARAM(ref) FQuat& Q, float Tolerance = 1.e-4f);

	/**
	 * Get a normalized copy of this quaternion.
	 * If it is too small, returns an identity quaternion.
	 *
	 * @param Tolerance Minimum squared length of quaternion for normalization.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Normalized (Quat)", ScriptMethod = "Normalized"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Normalized(const FQuat& Q, float Tolerance = 1.e-4f);

	/**
	 * Get the axis of rotation of the Quaternion.
	 * This is the axis around which rotation occurs to transform the canonical coordinate system to the target orientation.
	 * For the identity Quaternion which has no such rotation, FVector(1,0,0) is returned.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Rotation Axis (Quat)", ScriptMethod = "GetRotationAxis"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_GetRotationAxis(const FQuat& Q);

	/**	Return an inversed copy of this quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Inversed (Quat)", ScriptMethod = "Inversed"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Inversed(const FQuat& Q);

	/**	Quaternion with W=0 and V=theta*v. Used in combination with Exp(). */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Log (Quat)", ScriptMethod = "Log"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Log(const FQuat& Q);

	/** Set X, Y, Z, W components of Quaternion. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Components (Quat)", ScriptMethod = "SetComponents"), Category = "Math|Quat")
	static UE_INL_API void Quat_SetComponents(UPARAM(ref) FQuat& Q, float X, float Y, float Z, float W);

	/**
	 * Convert a vector of floating-point Euler angles (in degrees) into a Quaternion.
	 * 
	 * @param Q Quaternion to update
	 * @param Euler the Euler angles
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set from Euler (Quat)", ScriptMethod = "SetFromEuler"), Category = "Math|Quat")
	static UE_INL_API void Quat_SetFromEuler(UPARAM(ref) FQuat& Q, const FVector& Euler);

	/**
	 * Convert a vector of floating-point Euler angles (in degrees) into a Quaternion.
	 * 
	 * @param Euler the Euler angles
	 * @return constructed Quat
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Make from Euler (Quat)"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_MakeFromEuler(const FVector& Euler);

	/** Converts to Rotator representation of this Quaternion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToRotator (Quat)", CompactNodeTitle = "->", ScriptMethod = "Rotator", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FRotator Quat_Rotator(const FQuat& Q);

	/** Converts to Quaternion representation of this Rotator. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "ToQuaternion (Rotator)", CompactNodeTitle = "->", ScriptMethod = "Quaternion", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API FQuat Conv_RotatorToQuaternion(FRotator InRot);

	/**
	 * Get the rotation vector corresponding to this quaternion. 
	 * The direction of the vector represents the rotation axis, 
	 * and the magnitude the angle in degrees.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Rotation Vector (Quat)", ScriptMethod = "ToRotationVector"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_ToRotationVector(const FQuat& Q);

	/**
	 * Constructs a quaternion corresponding to the rotation vector. 
	 * The direction of the vector represents the rotation axis, 
	 * and the magnitude the angle in degrees.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Make from Rotation Vector (Quat)", ScriptMethod = "MakeFromRotationVector"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_MakeFromRotationVector(const FVector& V);

	/**
	 * Get the length of the quaternion.
	 *
	 * @return The length of the quaternion.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Size (Quat)", ScriptMethod = "Size"), Category = "Math|Quat")
	static UE_INL_API float Quat_Size(const FQuat& Q);

	/**
	 * Get the squared length of the quaternion.
	 *
	 * @return The squared length of the quaternion.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Size Squared (Quat)", ScriptMethod = "SizeSquared"), Category = "Math|Quat")
	static UE_INL_API float Quat_SizeSquared(const FQuat& Q);

	/**
	 * Rotate a vector by this quaternion.
	 *
	 * @param V the vector to be rotated
	 * @return vector after rotation
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Rotate Vector (Quat)", ScriptMethod = "RotateVector"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_RotateVector(const FQuat& Q, const FVector& V);

	/**
	 * Rotate a vector by the inverse of this quaternion.
	 *
	 * @param V the vector to be rotated
	 * @return vector after rotation by the inverse of this quaternion.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Unrotate Vector (Quat)", ScriptMethod = "UnrotateVector"), Category = "Math|Quat")
	static UE_INL_API FVector Quat_UnrotateVector(const FQuat& Q, const FVector& V);

	/**
	 * Spherical interpolation between Quaternions. Result is normalized.
	 * 
	 * @param A The starting quat we interp from
	 * @param B The target quat we interp to
	 * @param Alpha The interpolation amount, usually 0 to 1.
	 * @return Quat after spherical interpolation
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Slerp (Quat)", ScriptMethod = "SlerpQuat", Keywords = "spherical interpolate"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_Slerp(const FQuat& A, const FQuat& B, double Alpha);

	/**
	 * Generates the 'smallest' (geodesic) rotation around a sphere between two vectors of arbitrary length.
	 * 
	 * @param Start Vector the rotation starts from
	 * @param End Vector the rotation ends at
	 * @return Quat that will rotate from Start to End
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Find Quat Between Vectors", ScriptMethod = "FindQuatBetweenVectors", Keywords = "FindQuat FindBetween"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_FindBetweenVectors(FVector Start, FVector End);

	/**
	 * Generates the 'smallest' (geodesic) rotation around a sphere between two normals (assumed to be unit length).
	 * 
	 * @param Start Normalized vector the rotation starts from
	 * @param End Normalized vector the rotation ends at
	 * @return Quat that will rotate from Start to End
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Find Quat Between Normals", ScriptMethod = "FindQuatBetweenNormals", Keywords = "FindQuat FindBetween"), Category = "Math|Quat")
	static UE_INL_API FQuat Quat_FindBetweenNormals(FVector StartNormal, FVector EndNormal);

	//
	// LinearColor constants - exposed for scripting
	//

	/** White linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "White", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_White();

	/** Grey linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Gray", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Gray();

	/** Black linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Black", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Black();

	/** Red linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Red", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Red();

	/** Green linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Green", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Green();

	/** Blue linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Blue", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Blue();

	/** Yellow linear color */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Yellow", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Yellow();

	/** Transparent linear color - black with 0 opacity/alpha */
	UFUNCTION(BlueprintPure, meta = (ScriptConstant = "Transparent", ScriptConstantHost = "/Script/CoreUObject.LinearColor"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Transparent();


	//
	//	LinearColor functions
	//

	/** Make a color from individual color components (RGB space) */
	UFUNCTION(BlueprintPure, Category = "Math|Color", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_INL_API FLinearColor MakeColor(float R, float G, float B, float A = 1.0f);

	/** Breaks apart a color into individual RGB components (as well as alpha) */
	UFUNCTION(BlueprintPure, Category = "Math|Color")
	static UE_INL_API void BreakColor(FLinearColor InColor, float& R, float& G, float& B, float& A);

	/** Assign contents of InColor */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "Set"), Category = "Math|Color")
	static UE_INL_API void LinearColor_Set(UPARAM(ref) FLinearColor& InOutColor, FLinearColor InColor);

	/** Assign individual linear RGBA components. */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetRGBA"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetRGBA(UPARAM(ref) FLinearColor& InOutColor, float R, float G, float B, float A = 1.0f);

	/** Assigns an HSV color to a linear space RGB color */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetFromHSV"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetFromHSV(UPARAM(ref) FLinearColor& InOutColor, float H, float S, float V, float A = 1.0f);

	/**
	 * Assigns an FColor coming from an observed sRGB output, into a linear color.
	 * @param InSRGB The sRGB color that needs to be converted into linear space.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetFromSRGB"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetFromSRGB(UPARAM(ref) FLinearColor& InOutColor, const FColor& InSRGB);

	/**
	 * Assigns an FColor coming from an observed Pow(1/2.2) output, into a linear color.
	 * @param InColor The Pow(1/2.2) color that needs to be converted into linear space.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetFromPow22"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetFromPow22(UPARAM(ref) FLinearColor& InOutColor, const FColor& InColor);

	/** Converts temperature in Kelvins of a black body radiator to RGB chromaticity. */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetTemperature"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetTemperature(UPARAM(ref) FLinearColor& InOutColor, float InTemperature);

	/** Sets to a random color. Choses a quite nice color based on a random hue. */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod = "SetRandomHue"), Category = "Math|Color")
	static UE_INL_API void LinearColor_SetRandomHue(UPARAM(ref) FLinearColor& InOutColor);

	/** Converts a float into a LinearColor, where each RGB element is that float */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To LinearColor (Float)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FLinearColor Conv_DoubleToLinearColor(double InDouble);

	/** Make a color from individual color components (HSV space; Hue is [0..360) while Saturation and Value are 0..1) */
	UFUNCTION(BlueprintPure, Category = "Math|Color", meta = (DisplayName = "HSV to RGB"))
	static UE_INL_API FLinearColor HSVToRGB(float H, float S, float V, float A = 1.0f);

	/** Converts a HSV linear color (where H is in R (0..360), S is in G (0..1), and V is in B (0..1)) to RGB */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "HSV to RGB (Vector)", ScriptMethod = "HSVIntoRGB", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API void HSVToRGB_Vector(FLinearColor HSV, FLinearColor& RGB);

	/** Converts a HSV linear color (where H is in R, S is in G, and V is in B) to linear RGB */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "HSV to RGB linear color", ScriptMethod = "HSVToRGB", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API FLinearColor HSVToRGBLinear(FLinearColor HSV);

	/** Breaks apart a color into individual HSV components (as well as alpha) (Hue is [0..360) while Saturation and Value are 0..1) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "RGB to HSV", ScriptMethod = "RGBIntoHSVComponents"), Category = "Math|Color")
	static UE_INL_API void RGBToHSV(FLinearColor InColor, float& H, float& S, float& V, float& A);

	/** Converts a RGB linear color to HSV (where H is in R (0..360), S is in G (0..1), and V is in B (0..1)) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "RGB to HSV (Vector)", ScriptMethod = "RGBIntoHSV", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API void RGBToHSV_Vector(FLinearColor RGB, FLinearColor& HSV);

	/** Converts a RGB linear color to HSV (where H is in R, S is in G, and V is in B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "RGB to HSV (LinearColor)", ScriptMethod = "RGBToHSV", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API FLinearColor RGBLinearToHSV(FLinearColor RGB);

	/** Converts a LinearColor to a vector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (LinearColor)", ScriptMethod = "ToRGBVector", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_LinearColorToVector(FLinearColor InLinearColor);

	/** Converts from linear to 8-bit RGBE as outlined in Gregory Ward's Real Pixels article, Graphics Gems II, page 80. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To RGBE (LinearColor)", ScriptMethod = "ToRGBE", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Color")
	static UE_INL_API FColor LinearColor_ToRGBE(FLinearColor InLinearColor);

	/** Quantizes the linear color and returns the result as a FColor with optional sRGB conversion and quality as goal. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Color (LinearColor)", ScriptMethod = "ToColor", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FColor Conv_LinearColorToColor(FLinearColor InLinearColor, bool InUseSRGB = true);

	/** Quantizes the linear color and returns the result as an 8-bit color.  This bypasses the SRGB conversion. */
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "Use LinearColor_QuantizeRound instead for correct color conversion.", DisplayName = "Quantize to 8-bit (LinearColor)", ScriptMethod = "Quantize", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API FColor LinearColor_Quantize(FLinearColor InColor);

	/** Quantizes the linear color with rounding and returns the result as an 8-bit color.  This bypasses the SRGB conversion. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Quantize With Rounding To 8-bit (LinearColor)", ScriptMethod = "QuantizeRound", Keywords = "cast convert"), Category = "Math|Color")
	static UE_INL_API FColor LinearColor_QuantizeRound(FLinearColor InColor);

	/**
	 * Returns a desaturated color, with 0 meaning no desaturation and 1 == full desaturation
	 *
	 * @param	Desaturation	Desaturation factor in range [0..1]
	 * @return	Desaturated color
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Desaturate (LinearColor)", ScriptMethod = "Desaturated"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_Desaturated(FLinearColor InColor, float InDesaturation);

	/** Euclidean distance between two color points. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Distance (LinearColor)", ScriptMethod = "Distance", Keywords = "magnitude"), Category = "Math|Color")
	static UE_INL_API float LinearColor_Distance(FLinearColor C1, FLinearColor C2);
	
	/** Returns a copy of this color using the specified opacity/alpha.	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "New Opacity (LinearColor)", ScriptMethod = "ToNewOpacity"), Category = "Math|Color")
	static UE_INL_API FLinearColor LinearColor_ToNewOpacity(FLinearColor InColor, float InOpacity);

	/**	Returns the perceived brightness of a color on a display taking into account the impact on the human eye per color channel: green > red > blue. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Luminance (LinearColor)", ScriptMethod = "GetLuminance"), Category = "Math|Color")
	static UE_INL_API float LinearColor_GetLuminance(FLinearColor InColor);

	/**
	 * Returns the maximum color channel value in this color structure
	 *
	 * @return The maximum color channel value
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Max (LinearColor)", ScriptMethod = "GetMax"), Category = "Math|Color")
	static UE_INL_API float LinearColor_GetMax(FLinearColor InColor);

	/**
	 * Returns the minimum color channel value in this color structure
	 *
	 * @return The minimum color channel value
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Min (LinearColor)", ScriptMethod = "GetMin"), Category = "Math|Color")
	static UE_INL_API float LinearColor_GetMin(FLinearColor InColor);

	/**
	 * Interpolate Linear Color from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out.
	 *
	 * @param		Current			Current Color
	 * @param		Target			Target Color
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated Color
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Interpolate (LinearColor)", ScriptMethod = "InterpolateTo", Keywords = "color"), Category = "Math|Interpolation")
	static UE_INL_API FLinearColor CInterpTo(FLinearColor Current, FLinearColor Target, float DeltaTime, float InterpSpeed);

	/** Linearly interpolates between A and B based on Alpha (100% of A when Alpha=0 and 100% of B when Alpha=1) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Lerp (LinearColor)", ScriptMethod = "LerpTo"), Category="Math|Color")
	static UE_INL_API FLinearColor LinearColorLerp(FLinearColor A, FLinearColor B, float Alpha);

	/**
	 * Linearly interpolates between two colors by the specified Alpha amount (100% of A when Alpha=0 and 100% of B when Alpha=1).  The interpolation is performed in HSV color space taking the shortest path to the new color's hue.  This can give better results than a normal lerp, but is much more expensive.  The incoming colors are in RGB space, and the output color will be RGB.  The alpha value will also be interpolated.
	 * 
	 * @param	A		The color and alpha to interpolate from as linear RGBA
	 * @param	B		The color and alpha to interpolate to as linear RGBA
	 * @param	Alpha	Scalar interpolation amount (usually between 0.0 and 1.0 inclusive)
	 * 
	 * @return	The interpolated color in linear RGB space along with the interpolated alpha value
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Lerp Using HSV (LinearColor)", ScriptMethod = "LerpUsingHSVTo"), Category="Math|Color")
	static UE_INL_API FLinearColor LinearColorLerpUsingHSV(FLinearColor A, FLinearColor B, float Alpha);

	/** Returns true if linear color A is equal to linear color B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Near Equal (LinearColor)", CompactNodeTitle = "~==", ScriptMethod = "IsNearEqual", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|Color")
	static UE_INL_API bool LinearColor_IsNearEqual(FLinearColor A, FLinearColor B, float Tolerance = 1.e-4f);

	/** Returns true if linear color A is equal to linear color B (A == B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (LinearColor)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category = "Math|Color")
	static UE_INL_API bool EqualEqual_LinearColorLinearColor(FLinearColor A, FLinearColor B);

	/** Returns true if linear color A is not equal to linear color B (A != B) within a specified error tolerance */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (LinearColor)", CompactNodeTitle = "!=", ScriptMethod = "NotEqual", ScriptOperator = "!=", Keywords = "!= not equal"), Category = "Math|Color")
	static UE_INL_API bool NotEqual_LinearColorLinearColor(FLinearColor A, FLinearColor B);

	/** Element-wise addition of two linear colors (R+R, G+G, B+B, A+A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "LinearColor + LinearColor", CompactNodeTitle = "+", ScriptMethod = "Add", ScriptOperator = "+;+=", Keywords = "+ add plus"), Category="Math|Color")
	static UE_INL_API FLinearColor Add_LinearColorLinearColor(FLinearColor A, FLinearColor B);

	/** Element-wise subtraction of two linear colors (R-R, G-G, B-B, A-A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "LinearColor - LinearColor", CompactNodeTitle = "-", ScriptMethod = "Subtract", ScriptOperator = "-;-=", Keywords = "- subtract minus"), Category="Math|Color")
	static UE_INL_API FLinearColor Subtract_LinearColorLinearColor(FLinearColor A, FLinearColor B);

	/** Element-wise multiplication of two linear colors (R*R, G*G, B*B, A*A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "LinearColor * LinearColor", CompactNodeTitle = "*", ScriptMethod = "Multiply", ScriptOperator = "*;*=", Keywords = "* multiply"), Category="Math|Color")
	static UE_INL_API FLinearColor Multiply_LinearColorLinearColor(FLinearColor A, FLinearColor B);

	/** Element-wise multiplication of a linear color by a float (F*R, F*G, F*B, F*A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "LinearColor * Float", CompactNodeTitle = "*", ScriptMethod = "MultiplyFloat", Keywords = "* multiply"), Category="Math|Color")
	static UE_INL_API FLinearColor Multiply_LinearColorFloat(FLinearColor A, float B);

	/** Element-wise multiplication of two linear colors (R/R, G/G, B/B, A/A) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "LinearColor / LinearColor", CompactNodeTitle = "/", ScriptMethod = "Divide", ScriptOperator = "/;/=", Keywords = "/ divide"), Category="Math|Color")
	static UE_INL_API FLinearColor Divide_LinearColorLinearColor(FLinearColor A, FLinearColor B);
	
	/** Converts this color value to a hexadecimal string. The format of the string is RRGGBBAA. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToHex"), Category = "Math|Color")
	static UE_INL_API FString ToHex_LinearColor(FLinearColor InColor);

	//
	// Plane functions.
	//
	
	/** 
	* Creates a plane with a facing direction of Normal at the given Point
	* 
	* @param Point	A point on the plane
	* @param Normal  The Normal of the plane at Point
	* @return Plane instance
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Plane", meta=(Keywords="make plane"))
	static ENGINE_API FPlane MakePlaneFromPointAndNormal(FVector Point, FVector Normal);
	

	//
	// DateTime functions.
	//

	/** Makes a DateTime struct */
	UFUNCTION(BlueprintPure, Category="Math|DateTime", meta=(IgnoreTypePromotion, NativeMakeFunc, AdvancedDisplay = "3"))
	static ENGINE_API FDateTime MakeDateTime(int32 Year, int32 Month, int32 Day, int32 Hour = 0, int32 Minute = 0, int32 Second = 0, int32 Millisecond = 0);

	/** Breaks a DateTime into its components */
	UFUNCTION(BlueprintPure, Category="Math|DateTime", meta=(IgnoreTypePromotion, NativeBreakFunc, AdvancedDisplay = "4"))
	static ENGINE_API void BreakDateTime(FDateTime InDateTime, int32& Year, int32& Month, int32& Day, int32& Hour, int32& Minute, int32& Second, int32& Millisecond);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime + Timespan", CompactNodeTitle="+", Keywords="+ add plus"), Category="Math|DateTime")
	static UE_INL_API FDateTime Add_DateTimeTimespan( FDateTime A, FTimespan B );

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime - Timespan", CompactNodeTitle="-", Keywords="- subtract minus"), Category="Math|DateTime")
	static UE_INL_API FDateTime Subtract_DateTimeTimespan(FDateTime A, FTimespan B);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta = (IgnoreTypePromotion, DisplayName = "DateTime + DateTime", CompactNodeTitle = "+", Keywords = "+ add plus"), Category = "Math|DateTime")
	static UE_INL_API FDateTime Add_DateTimeDateTime(FDateTime A, FDateTime B);

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta = (IgnoreTypePromotion, DisplayName = "DateTime - DateTime", CompactNodeTitle = "-", Keywords = "- subtract minus"), Category = "Math|DateTime")
	static UE_INL_API FTimespan Subtract_DateTimeDateTime(FDateTime A, FDateTime B);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="Equal (DateTime)", CompactNodeTitle="==", Keywords="== equal"), Category="Math|DateTime")
	static UE_INL_API bool EqualEqual_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="Not Equal (DateTime)", CompactNodeTitle="!=", Keywords="!= not equal"), Category="Math|DateTime")
	static UE_INL_API bool NotEqual_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime > DateTime", CompactNodeTitle=">", Keywords="> greater"), Category="Math|DateTime")
	static UE_INL_API bool Greater_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime >= DateTime", CompactNodeTitle=">=", Keywords=">= greater"), Category="Math|DateTime")
	static UE_INL_API bool GreaterEqual_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns true if A is less than B (A < B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime < DateTime", CompactNodeTitle="<", Keywords="< less"), Category="Math|DateTime")
	static UE_INL_API bool Less_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns true if A is less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta=(IgnoreTypePromotion, DisplayName="DateTime <= DateTime", CompactNodeTitle="<=", Keywords="<= less"), Category="Math|DateTime")
	static UE_INL_API bool LessEqual_DateTimeDateTime( FDateTime A, FDateTime B );

	/** Returns the date component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Date"), Category="Math|DateTime")
	static UE_INL_API FDateTime GetDate( FDateTime A );

	/** Returns the day component of A (1 to 31) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Day"), Category="Math|DateTime")
	static UE_INL_API int32 GetDay( FDateTime A );

	/** Returns the day of year of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Day Of Year"), Category="Math|DateTime")
	static UE_INL_API int32 GetDayOfYear( FDateTime A );

	/** Returns the hour component of A (24h format) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Hour"), Category="Math|DateTime")
	static UE_INL_API int32 GetHour( FDateTime A );

	/** Returns the hour component of A (12h format) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Hour 12"), Category="Math|DateTime")
	static UE_INL_API int32 GetHour12( FDateTime A );

	/** Returns the millisecond component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Millisecond"), Category="Math|DateTime")
	static UE_INL_API int32 GetMillisecond( FDateTime A );

	/** Returns the minute component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Minute"), Category="Math|DateTime")
	static UE_INL_API int32 GetMinute( FDateTime A );

	/** Returns the month component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Month"), Category="Math|DateTime")
	static UE_INL_API int32 GetMonth( FDateTime A );

	/** Returns the second component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Second"), Category="Math|DateTime")
	static UE_INL_API int32 GetSecond( FDateTime A );

	/** Returns the time elapsed since midnight of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Time Of Day"), Category="Math|DateTime")
	static UE_INL_API FTimespan GetTimeOfDay( FDateTime A );

	/** Returns the year component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Year"), Category="Math|DateTime")
	static UE_INL_API int32 GetYear( FDateTime A );

	/** Returns whether A's time is in the afternoon */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Afternoon"), Category="Math|DateTime")
	static UE_INL_API bool IsAfternoon( FDateTime A );

	/** Returns whether A's time is in the morning */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Morning"), Category="Math|DateTime")
	static UE_INL_API bool IsMorning( FDateTime A );

	/** Returns the number of days in the given year and month */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Days In Month"), Category="Math|DateTime")
	static UE_INL_API int32 DaysInMonth( int32 Year, int32 Month );

	/** Returns the number of days in the given year */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Days In Year"), Category="Math|DateTime")
	static UE_INL_API int32 DaysInYear( int32 Year );

	/** Returns whether given year is a leap year */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Is Leap Year"), Category="Math|DateTime")
	static UE_INL_API bool IsLeapYear( int32 Year );

	/** Returns the maximum date and time value */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Max Value (DateTime)"), Category="Math|DateTime")
	static UE_INL_API FDateTime DateTimeMaxValue( );

	/** Returns the minimum date and time value */
	UFUNCTION(BlueprintPure, meta=(DisplayName="MinValue (DateTime)"), Category="Math|DateTime")
	static UE_INL_API FDateTime DateTimeMinValue( );

	/** Returns the local date and time on this computer */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Now"), Category="Math|DateTime")
	static UE_INL_API FDateTime Now( );

	/** Returns the local date on this computer */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Today"), Category="Math|DateTime")
	static UE_INL_API FDateTime Today( );

	/** Returns the UTC date and time on this computer */
	UFUNCTION(BlueprintPure, meta=(DisplayName="UTC Now"), Category="Math|DateTime")
	static UE_INL_API FDateTime UtcNow( );

	/** Converts a date string in ISO-8601 format to a DateTime object */
	UFUNCTION(BlueprintPure, Category="Math|DateTime")
	static UE_INL_API bool DateTimeFromIsoString(FString IsoString, FDateTime& Result);

	/** Converts a date string to a DateTime object */
	UFUNCTION(BlueprintPure, Category="Math|DateTime")
	static UE_INL_API bool DateTimeFromString(FString DateTimeString, FDateTime& Result);

	/**
	 * Returns this date as the number of seconds since the Unix Epoch (January 1st of 1970).
	 *
	 * @return Time of day.
	 * @see FromUnixTimestamp
	 */
	UFUNCTION(BlueprintPure, Category="Math|DateTime")
	static UE_INL_API int64 ToUnixTimestamp(const FDateTime& Time);

	/**
	 * Returns this date as the number of seconds since the Unix Epoch (January 1st of 1970).
	 *
	 * @return Time of day.
	 * @see FromUnixTimestamp
	 */
	UFUNCTION(BlueprintPure, Category="Math|DateTime")
	static UE_INL_API double ToUnixTimestampDouble(const FDateTime& Time);
	
	/**
	 * Returns the date from Unix time (seconds from midnight 1970-01-01)
	 *
	 * @param UnixTime Unix time (seconds from midnight 1970-01-01)
	 * @return Gregorian date and time.
	 * @see ToUnixTimestamp
	 */
	UFUNCTION(BlueprintPure, Category="Math|DateTime")
	static UE_INL_API FDateTime FromUnixTimestamp(const int64 UnixTime);

	//
	// Timespan constants
	//

	/** Returns the maximum time span value */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Max Value (Timespan)", ScriptConstant = "MaxValue", ScriptConstantHost = "/Script/CoreUObject.Timespan"), Category="Math|Timespan")
	static UE_INL_API FTimespan TimespanMaxValue( );

	/** Returns the minimum time span value */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Min Value (Timespan)", ScriptConstant = "MinValue", ScriptConstantHost = "/Script/CoreUObject.Timespan"), Category="Math|Timespan")
	static UE_INL_API FTimespan TimespanMinValue( );

	/** Returns a zero time span value */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Zero Value (Timespan)", ScriptConstant = "Zero", ScriptConstantHost = "/Script/CoreUObject.Timespan"), Category="Math|Timespan")
	static UE_INL_API FTimespan TimespanZeroValue( );

	//
	// Timespan functions.
	//

	/** Makes a Timespan struct */
	UFUNCTION(BlueprintPure, Category="Math|Timespan", meta=(NativeMakeFunc))
	static ENGINE_API FTimespan MakeTimespan(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 Milliseconds);

	/** Makes a Timespan struct */
	UFUNCTION(BlueprintPure, Category="Math|Timespan", meta=(NativeMakeFunc))
	static ENGINE_API FTimespan MakeTimespan2(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 FractionNano);

	/** Breaks a Timespan into its components */
	UFUNCTION(BlueprintPure, Category="Math|Timespan", meta=(NativeBreakFunc))
	static ENGINE_API void BreakTimespan(FTimespan InTimespan, int32& Days, int32& Hours, int32& Minutes, int32& Seconds, int32& Milliseconds);

	/** Breaks a Timespan into its components */
	UFUNCTION(BlueprintPure, Category="Math|Timespan", meta=(NativeBreakFunc))
	static ENGINE_API void BreakTimespan2(FTimespan InTimespan, int32& Days, int32& Hours, int32& Minutes, int32& Seconds, int32& FractionNano);

	/** Addition (A + B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan + Timespan", CompactNodeTitle="+", Keywords="+ add plus"), Category="Math|Timespan")
	static UE_INL_API FTimespan Add_TimespanTimespan( FTimespan A, FTimespan B );

	/** Subtraction (A - B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan - Timespan", CompactNodeTitle="-", Keywords="- subtract minus"), Category="Math|Timespan")
	static UE_INL_API FTimespan Subtract_TimespanTimespan( FTimespan A, FTimespan B );

	/** Scalar multiplication (A * s) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan * float", CompactNodeTitle="*", Keywords="* multiply"), Category="Math|Timespan")
	static UE_INL_API FTimespan Multiply_TimespanFloat( FTimespan A, float Scalar );

	/** Scalar division (A / s) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan / float", CompactNodeTitle="/", Keywords="/ divide"), Category="Math|Timespan")
	static UE_INL_API FTimespan Divide_TimespanFloat( FTimespan A, float Scalar );

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Equal (Timespan)", CompactNodeTitle="==", Keywords="== equal"), Category="Math|Timespan")
	static UE_INL_API bool EqualEqual_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Not Equal (Timespan)", CompactNodeTitle="!=", Keywords="!= not equal"), Category="Math|Timespan")
	static UE_INL_API bool NotEqual_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns true if A is greater than B (A > B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan > Timespan", CompactNodeTitle=">", Keywords="> greater"), Category="Math|Timespan")
	static UE_INL_API bool Greater_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns true if A is greater than or equal to B (A >= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan >= Timespan", CompactNodeTitle=">=", Keywords=">= greater"), Category="Math|Timespan")
	static UE_INL_API bool GreaterEqual_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns true if A is less than B (A < B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan < Timespan", CompactNodeTitle="<", Keywords="< less"), Category="Math|Timespan")
	static UE_INL_API bool Less_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns true if A is less than or equal to B (A <= B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan <= Timespan", CompactNodeTitle="<=", Keywords="<= less"), Category="Math|Timespan")
	static UE_INL_API bool LessEqual_TimespanTimespan( FTimespan A, FTimespan B );

	/** Returns the days component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Days"), Category="Math|Timespan")
	static UE_INL_API int32 GetDays( FTimespan A );

	/** Returns the absolute value of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Duration"), Category="Math|Timespan")
	static UE_INL_API FTimespan GetDuration( FTimespan A );

	/** Returns the hours component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Hours"), Category="Math|Timespan")
	static UE_INL_API int32 GetHours( FTimespan A );

	/** Returns the milliseconds component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Milliseconds"), Category="Math|Timespan")
	static UE_INL_API int32 GetMilliseconds( FTimespan A );

	/** Returns the minutes component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Minutes"), Category="Math|Timespan")
	static UE_INL_API int32 GetMinutes( FTimespan A );

	/** Returns the seconds component of A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Seconds"), Category="Math|Timespan")
	static UE_INL_API int32 GetSeconds( FTimespan A );

	/** Returns the total number of days in A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Total Days"), Category="Math|Timespan")
	static UE_INL_API double GetTotalDays( FTimespan A );

	/** Returns the total number of hours in A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Total Hours"), Category="Math|Timespan")
	static UE_INL_API double GetTotalHours( FTimespan A );

	/** Returns the total number of milliseconds in A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Total Milliseconds"), Category="Math|Timespan")
	static UE_INL_API double GetTotalMilliseconds( FTimespan A );

	/** Returns the total number of minutes in A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Total Minutes"), Category="Math|Timespan")
	static UE_INL_API double GetTotalMinutes( FTimespan A );

	/** Returns the total number of seconds in A */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Get Total Seconds"), Category="Math|Timespan")
	static UE_INL_API double GetTotalSeconds( FTimespan A );

	/** Returns a time span that represents the specified number of days */
	UFUNCTION(BlueprintPure, meta=(DisplayName="From Days"), Category="Math|Timespan")
	static ENGINE_API FTimespan FromDays( double Days );

	/** Returns a time span that represents the specified number of hours */
	UFUNCTION(BlueprintPure, meta=(DisplayName="From Hours"), Category="Math|Timespan")
	static ENGINE_API FTimespan FromHours(double Hours );

	/** Returns a time span that represents the specified number of milliseconds */
	UFUNCTION(BlueprintPure, meta=(DisplayName="From Milliseconds"), Category="Math|Timespan")
	static ENGINE_API FTimespan FromMilliseconds(double Milliseconds );

	/** Returns a time span that represents the specified number of minutes */
	UFUNCTION(BlueprintPure, meta=(DisplayName="From Minutes"), Category="Math|Timespan")
	static ENGINE_API FTimespan FromMinutes(double Minutes );

	/** Returns a time span that represents the specified number of seconds */
	UFUNCTION(BlueprintPure, meta=(DisplayName="From Seconds"), Category="Math|Timespan")
	static ENGINE_API FTimespan FromSeconds(double Seconds );

	/** Returns the ratio between two time spans (A / B), handles zero values */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Timespan Ratio"), Category="Math|Timespan")
	static UE_INL_API float TimespanRatio( FTimespan A, FTimespan B );

	/** Converts a time span string to a Timespan object */
	UFUNCTION(BlueprintPure, Category="Math|Timespan")
	static UE_INL_API bool TimespanFromString(FString TimespanString, FTimespan& Result);

	//
	// Frame Time and Frame Rate Functions
	//

	/** Creates a FQualifiedFrameTime out of a frame number, frame rate, and optional 0-1 clamped subframe. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management", meta = (NativeMakeFunc))
	static ENGINE_API FQualifiedFrameTime MakeQualifiedFrameTime(FFrameNumber Frame, FFrameRate FrameRate, float SubFrame = 0.f);

	/** Breaks a FQualifiedFrameTime into its component parts again. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management", meta = (NativeBreakFunc))
	static ENGINE_API void BreakQualifiedFrameTime(const FQualifiedFrameTime& InFrameTime, FFrameNumber& Frame, FFrameRate& FrameRate, float& SubFrame);

	/** Creates a FFrameRate from a Numerator and a Denominator. Enforces that the Denominator is at least one. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management", meta = (NativeMakeFunc))
	static ENGINE_API FFrameRate MakeFrameRate(int32 Numerator, int32 Denominator = 1);

	/** Breaks a FFrameRate into a numerator and denominator. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management", meta = (NativeBreakFunc))
	static ENGINE_API void BreakFrameRate(const FFrameRate& InFrameRate, int32& Numerator, int32& Denominator);
	
	// -- Begin K2 utilities

	/** Converts a byte to a float */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Float (Byte)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API double Conv_ByteToDouble(uint8 InByte);

	/** Converts an integer to a float */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Float (Integer)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API double Conv_IntToDouble(int32 InInt);

	/** Converts an integer to a 64 bit integer */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Integer64 (Integer)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API int64 Conv_IntToInt64(int32 InInt);

	/** Converts an integer to a byte (if the integer is too large, returns the low 8 bits) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Byte (Integer)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API uint8 Conv_IntToByte(int32 InInt);

	/** Converts a 64 bit integer to a 32 bit integer (if the integer is too large, returns the low 32 bits) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Integer (Integer64)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API int32 Conv_Int64ToInt(int64 InInt);

	/** Converts a 64 bit floating point to a 32 bit floating point (if the float is too large, returns the low 32 bits) */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", DeprecatedFunction, DeprecationMessage = "Explicit conversions between floats and doubles are not necessary. Please remove node."))
	static UE_INL_API float Conv_DoubleToFloat(double InDouble);

	/** Converts a 32 bit floating point to a 64 bit floating point */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", DeprecatedFunction, DeprecationMessage = "Explicit conversions between floats and doubles are not necessary. Please remove node."))
	static UE_INL_API double Conv_FloatToDouble(float InFloat);

	/** Converts a 64 bit integer to a byte (if the integer is too large, returns the low 8 bits) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Byte (Integer64)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API uint8 Conv_Int64ToByte(int64 InInt);

	/** Converts a float to a 64 bit integer */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Integer64 (Float)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API int64 Conv_DoubleToInt64(double InDouble);

	/** Converts a 64 bit integer to a float */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Float (Integer64)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API double Conv_Int64ToDouble(int64 InInt);

	/** Converts an integer to an IntVector*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To IntVector (Integer)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FIntVector Conv_IntToIntVector(int32 InInt);

	/** Converts an integer to an IntVector2*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To IntVector2 (Integer)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FIntVector2 Conv_IntToIntVector2(int32 InInt);

	/** Converts an integer to a FVector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (Integer)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_IntToVector(int32 InInt);

	/** Converts a int to a bool*/
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Boolean (Integer)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API bool Conv_IntToBool(int32 InInt);

	/** Converts a bool to an int */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Integer (Boolean)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API int32 Conv_BoolToInt(bool InBool);

	/** Converts a bool to a float (0.0 or 1.0) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Float (Boolean)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API double Conv_BoolToDouble(bool InBool);

	/** Converts a bool to a byte */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Byte (Boolean)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API uint8 Conv_BoolToByte(bool InBool);
	
	/** Converts a byte to an integer */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Integer (Byte)", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API int32 Conv_ByteToInt(uint8 InByte);

	/** Converts a byte to an integer */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Integer64 (Byte)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API int64 Conv_ByteToInt64(uint8 InByte);

	/** Converts a color to LinearColor */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To LinearColor (Color)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FLinearColor Conv_ColorToLinearColor(FColor InColor);

	/** Converts an IntVector to a vector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (IntVector)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_IntVectorToVector(const FIntVector& InIntVector);

	/** Converts an IntVector2 to a vector2D */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector2D (IntVector2)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector2D Conv_IntVector2ToVector2D(const FIntVector2& InIntVector2);

	/** Converts a double into a vector, where each element is that float */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Vector (Float)", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast), Category = "Math|Conversions")
	static UE_INL_API FVector Conv_DoubleToVector(double InDouble);

	/** Convert a float into a vector, where each element is that float */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Vector2D", CompactNodeTitle = "->", Keywords="cast convert", BlueprintAutocast), Category="Math|Conversions")
	static UE_INL_API FVector2D Conv_DoubleToVector2D(double InDouble);
	
	//
	// Box functions
	//

	/** Makes an FBox from Min and Max and sets IsValid to true */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(Keywords="construct build", NativeMakeFunc))
	static UE_INL_API FBox MakeBox(FVector Min, FVector Max);

	/** 
	 * Utility function to build an box from an Origin and Extent 
	 *
	 * @param Origin The location of the bounding box.
	 * @param Extent Half size of the bounding box.
	 * @return A new axis-aligned bounding box.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(Keywords="construct build"))
	static ENGINE_API FBox MakeBoxWithOrigin(const FVector& Origin, const FVector& Extent);

	/** 
	 * Makes an FOrientedBox from Center, Orientation and Extent 
	 * 
	 * @param Origin The center of the box
	 * @param Orientation The rotation of the box
	 * @param Extent The half-size of the box
	 * @return A new oriented bounding box
	 */
	UFUNCTION(BlueprintPure, Category = "Math|OrientedBox", meta = (Keywords = "construct build", NativeMakeFunc))
	static ENGINE_API FOrientedBox MakeOrientedBox(FVector Origin, FRotator Orientation, FVector Extent);

	/**
	 * Returns true if the InnerTest Box is is completely inside of the OuterTest Box
	 * 
	 * @param InnerTest		The box to check if it is on the inside
	 * @param OuterTest		The box to check if InnerTest is within.
	 * 
	 * @return True if InnerTest Box is is completely inside of OuterTest Box
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Is Inside (Box)"))
	static ENGINE_API bool Box_IsInside(const FBox& InnerTest, const FBox& OuterTest);

	/**
	 * Returns true if the InnerTest Box is is completely inside or on OuterTest Box
	 * 
	 * @param InnerTest		The box to check if it is on the inside
	 * @param OuterTest		The box to check if InnerTest is within or on.
	 * 
	 * @return True if InnerTest Box is is completely inside of or on OuterTest Box
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Is Inside Or On (Box)"))
	static ENGINE_API bool Box_IsInsideOrOn(const FBox& InnerTest, const FBox& OuterTest);

	/** 
	 * Checks whether the given location is inside this box.
	 *
	 * @param Box	The box to test
	 * @param Point The location to test for inside the bounding volume.
	 * @return true if location is inside this volume.
	 *
	 * @note  This function assumes boxes have open bounds, i.e. points lying on the border of the box are not inside.
	 *        Use IsPointInBox_Box to include borders in the test.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Is Vector Inside (Box)"))
	static ENGINE_API bool Box_IsPointInside(const FBox& Box, const FVector& Point);
	
	/**
	 * Checks whether the given bounding box A intersects this bounding box B.
	 *
	 * @param A The bounding box to check intersection against
	 * @param B The bounding box to intersect with.
	 * 
	 * @return true if the boxes intersect, false otherwise.
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. boxes with
	 *        coincident borders on any edge will overlap.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Intersects (Box)"))
	static ENGINE_API bool Box_Intersects(const FBox& A, const FBox& B);

	/**
	* Returns a box of increased size.
	*
	* @param Negative The size to increase the volume by in the negative direction (positive values move the bounds outwards)
	* @param Positive The size to increase the volume by in the positive direction (positive values move the bounds outwards)
	* @return A new bounding box.
	*/
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Expand By (Box)"))
	static ENGINE_API FBox Box_ExpandBy(const FBox& Box, const FVector& Negative, const FVector& Positive);

	/**
	 * Returns the overlap TBox<T> of two boxes
	 *
	 * @param A		The bounding box to test
	 * @param B		The bounding box to test overlap against
	 * 
	 * @return the overlap box. It can be 0 if they don't overlap
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="Overlap (Box)"))
	static ENGINE_API FBox Box_Overlap(const FBox& A, const FBox& B);

	/**
	 * Calculates the closest point on or inside the box to a given point in space.
	 *
	 * @param Box	The box to check if the point is inside of
	 * @param Point	The point in space
	 * @return The closest point on or inside the box.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Box", meta=(DisplayName="GetClosestPointTo (Box)"))
	static ENGINE_API FVector Box_GetClosestPointTo(const FBox& Box, const FVector& Point);
	
	//
	// Box2D functions
	//

	/** Makes an FBox2D from Min and Max and sets IsValid to true */
	UFUNCTION(BlueprintPure, Category = "Math|Box2D", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_INL_API FBox2D MakeBox2D(FVector2D Min, FVector2D Max);

	
	//
	// BoxSphereBounds functions
	//
	
	/** Makes an FBoxSphereBounds given an origin, extent, and radius */
	UFUNCTION(BlueprintPure, meta = (Keywords = "construct build", NativeMakeFunc), Category = "Math|BoxSphereBounds")
	static UE_INL_API FBoxSphereBounds MakeBoxSphereBounds(FVector Origin, FVector BoxExtent, float SphereRadius);

	/** Breaks an FBoxSphereBounds into origin, extent, and radius */
	UFUNCTION(BlueprintPure, meta = (NativeBreakFunc), Category = "Math|BoxSphereBounds")
	static UE_INL_API void BreakBoxSphereBounds(const FBoxSphereBounds& InBoxSphereBounds, FVector& Origin, FVector& BoxExtent, float& SphereRadius);


	//
	// Misc functions
	//

	/** Makes a SRand-based random number generator */
	UFUNCTION(BlueprintPure, meta = (Keywords = "construct build", NativeMakeFunc), Category = "Math|Random")
	static ENGINE_API FRandomStream MakeRandomStream(int32 InitialSeed);

	/**
	 * Makes a random stream that will generate deterministic values, optionally grouping outputs within cells on a grid
	 * 
	 * @param Location			The input location to return the stream for
	 * @param DistanceInterval	If non-zero, the grid size for which all locations within a grid cell will return the same random stream
	 * @param bIncludeZ			Whether to factor in the Z-coordinate of the input location
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (NativeMakeFunc))
	static ENGINE_API FRandomStream MakeRandomStreamFromLocation(const FVector& Location, float DistanceInterval = 200.f, bool bIncludeZ = false);

	/** Breaks apart a random number generator */
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (NativeBreakFunc))
	static ENGINE_API void BreakRandomStream(const FRandomStream& InRandomStream, int32& InitialSeed);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static UE_INL_API FString SelectString(const FString& A, const FString& B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static UE_INL_API FText SelectText(const FText A, const FText B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static UE_INL_API FName SelectName(const FName A, const FName B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Integer")
	static UE_INL_API int32 SelectInt(int32 A, int32 B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Float")
	static UE_INL_API double SelectFloat(double A, double B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Vector")
	static UE_INL_API FVector SelectVector(FVector A, FVector B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Rotator", meta=(Keywords="rotation rotate"))
	static UE_INL_API FRotator SelectRotator(FRotator A, FRotator B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Color")
	static UE_INL_API FLinearColor SelectColor(FLinearColor A, FLinearColor B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Math|Transform")
	static UE_INL_API FTransform SelectTransform(const FTransform& A, const FTransform& B, bool bPickA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category="Utilities")
	static UE_INL_API UObject* SelectObject(UObject* A, UObject* B, bool bSelectA);

	/** If bPickA is true, A is returned, otherwise B is */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static UE_INL_API UClass* SelectClass(UClass* A, UClass* B, bool bSelectA);


	//
	// Object operators and functions.
	//
	
	/** Returns true if A and B are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Object)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Utilities")
	static UE_INL_API bool EqualEqual_ObjectObject(class UObject* A, class UObject* B);

	/** Returns true if A and B are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Object)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Utilities")
	static UE_INL_API bool NotEqual_ObjectObject(class UObject* A, class UObject* B);

	//
	// Class operators and functions.
	//

	/** Returns true if A and B are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Class)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Utilities")
	static UE_INL_API bool EqualEqual_ClassClass(class UClass* A, class UClass* B);

	/** Returns true if A and B are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Class)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Utilities")
	static UE_INL_API bool NotEqual_ClassClass(class UClass* A, class UClass* B);

	/**
	 * Determine if a class is a child of another class.
	 *
	 * @return	true if TestClass == ParentClass, or if TestClass is a child of ParentClass; false otherwise, or if either
	 *			the value for either parameter is 'None'.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities")
	static ENGINE_API bool ClassIsChildOf(TSubclassOf<class UObject> TestClass, TSubclassOf<class UObject> ParentClass);


	//
	// Name operators.
	//
	
	/** Returns true if A and B are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (Name)", CompactNodeTitle = "==", Keywords = "== equal"), Category="Utilities|Name")
	static UE_INL_API bool EqualEqual_NameName(FName A, FName B);

	/** Returns true if A and B are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal (Name)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category="Utilities|Name")
	static UE_INL_API bool NotEqual_NameName(FName A, FName B);


	//
	// Transform functions
	//
	
	/** Make a transform from location, rotation and scale */
	UFUNCTION(BlueprintPure, meta = (Keywords = "construct build", NativeMakeFunc), Category = "Math|Transform")
	static UE_INL_API FTransform MakeTransform(FVector Location, FRotator Rotation, FVector Scale = FVector(1,1,1));

	/** Breaks apart a transform into location, rotation and scale */
	UFUNCTION(BlueprintPure, Category = "Math|Transform", meta = (NativeBreakFunc))
	static UE_INL_API void BreakTransform(const FTransform& InTransform, FVector& Location, FRotator& Rotation, FVector& Scale);

	/** Returns true if transform A is equal to transform B */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (Transform)", CompactNodeTitle = "==", ScriptMethod = "Equals", ScriptOperator = "==", Keywords = "== equal"), Category="Math|Transform")
	static UE_INL_API bool EqualEqual_TransformTransform(const FTransform& A, const FTransform& B);

	/** 
	 *	Returns true if transform A is nearly equal to B 
	 *	@param LocationTolerance	How close position of transforms need to be to be considered equal
	 *	@param RotationTolerance	How close rotations of transforms need to be to be considered equal
	 *	@param Scale3DTolerance		How close scale of transforms need to be to be considered equal
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Nearly Equal (Transform)", ScriptMethod = "IsNearEqual", Keywords = "== equal"), Category = "Math|Transform")
	static ENGINE_API bool NearlyEqual_TransformTransform(const FTransform& A, const FTransform& B, float LocationTolerance = 1.e-4f, float RotationTolerance = 1.e-4f, float Scale3DTolerance = 1.e-4f);

	/**
	 * Compose two transforms in order: A * B.
	 *
	 * Order matters when composing transforms:
	 * A * B will yield a transform that logically first applies A then B to any subsequent transformation.
	 *
	 * Example: LocalToWorld = ComposeTransforms(DeltaRotation, LocalToWorld) will change rotation in local space by DeltaRotation.
	 * Example: LocalToWorld = ComposeTransforms(LocalToWorld, DeltaRotation) will change rotation in world space by DeltaRotation.
	 *
	 * @return New transform: A * B
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "Multiply", ScriptOperator = "*;*=", CompactNodeTitle = "*", Keywords="multiply *"), Category="Math|Transform")
	static UE_INL_API FTransform ComposeTransforms(const FTransform& A, const FTransform& B);

	/** 
	 *	Transform a position by the supplied transform.
	 *	For example, if T was an object's transform, this would transform a position from local space to world space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="TransformLocation", Keywords="location"), Category="Math|Transform")
	static UE_INL_API FVector TransformLocation(const FTransform& T, FVector Location);

	/** 
	 *	Transform a direction vector by the supplied transform - will not change its length. 
	 *	For example, if T was an object's transform, this would transform a direction from local space to world space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="TransformDirection"), Category="Math|Transform")
	static UE_INL_API FVector TransformDirection(const FTransform& T, FVector Direction);

	/** 
	 *	Transform a rotator by the supplied transform. 
	 *	For example, if T was an object's transform, this would transform a rotation from local space to world space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="TransformRotation"), Category="Math|Transform")
	static ENGINE_API FRotator TransformRotation(const FTransform& T, FRotator Rotation);

	/** 
	 *	Transform a position by the inverse of the supplied transform.
	 *	For example, if T was an object's transform, this would transform a position from world space to local space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="InverseTransformLocation", Keywords="location"), Category="Math|Transform")
	static UE_INL_API FVector InverseTransformLocation(const FTransform& T, FVector Location);

	/** 
	 *	Transform a direction vector by the inverse of the supplied transform - will not change its length.
	 *	For example, if T was an object's transform, this would transform a direction from world space to local space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="InverseTransformDirection"), Category="Math|Transform")
	static UE_INL_API FVector InverseTransformDirection(const FTransform& T, FVector Direction);

	/** 
	 *	Transform a rotator by the inverse of the supplied transform. 
	 *	For example, if T was an object's transform, this would transform a rotation from world space to local space.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod="InverseTransformRotation"), Category="Math|Transform")
	static ENGINE_API FRotator InverseTransformRotation(const FTransform& T, FRotator Rotation);

	/** 
	 * Computes a relative transform of one transform compared to another.
	 *
	 * Example: ChildOffset = MakeRelativeTransform(Child.GetActorTransform(), Parent.GetActorTransform())
	 * This computes the relative transform of the Child from the Parent.
	 *
	 * @param		A				The object's transform
	 * @param		RelativeTo		The transform the result is relative to (in the same space as A)
	 * @return		The new relative transform
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod = "MakeRelative", Keywords="convert torelative"), Category="Math|Transform")
	static UE_INL_API FTransform MakeRelativeTransform(const FTransform& A, const FTransform& RelativeTo);

	/** 
	 * Returns the inverse of the given transform T.
	 * 
	 * Example: Given a LocalToWorld transform, WorldToLocal will be returned.
	 *
	 * @param	T	The transform you wish to invert
	 * @return	The inverse of T.
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "Inverse", Keywords = "inverse"), Category = "Math|Transform")
	static UE_INL_API FTransform InvertTransform(const FTransform& T);

	/** Linearly interpolates between A and B based on Alpha (100% of A when Alpha=0 and 100% of B when Alpha=1). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Lerp (Transform)", AdvancedDisplay = "3", ScriptMethod = "Lerp"), Category="Math|Transform")
	static ENGINE_API FTransform TLerp(const FTransform& A, const FTransform& B, float Alpha, TEnumAsByte<ELerpInterpolationMode::Type> InterpMode = ELerpInterpolationMode::QuatInterp);

	/** Ease between A and B using a specified easing function. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Ease (Transform)", BlueprintInternalUseOnly = "true", ScriptMethod = "Ease"), Category = "Math|Interpolation")
	static ENGINE_API FTransform TEase(const FTransform& A, const FTransform& B, float Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp = 2, int32 Steps = 2);

	/**
	 * Tries to reach Target transform based on distance from Current position, giving a nice smooth feeling when tracking a position.
	 *
	 * @param		Current			Actual transform
	 * @param		Target			Target transform
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated transform
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod = "InterpTo"), Category="Math|Interpolation")
	static ENGINE_API FTransform TInterpTo(const FTransform& Current, const FTransform& Target, float DeltaTime, float InterpSpeed);

	/** Calculates the determinant of the transform (converts to FMatrix internally) */
	UFUNCTION(BlueprintPure, Category="Math|Transform", meta = (DisplayName = "Determinant", ScriptMethod = "Determinant"))
	static ENGINE_API float Transform_Determinant(const FTransform& Transform);

	/** Converts a Transform to a Matrix with scale */
	UFUNCTION(BlueprintPure, Category="Math|Transform", meta = (DisplayName = "To Matrix (Transform)", ScriptMethod = "ToMatrix", CompactNodeTitle = "->", Keywords = "cast convert", BlueprintAutocast))
	static ENGINE_API FMatrix Conv_TransformToMatrix(const FTransform& Transform);


	//
	// Interpolation functions
	//

	/**
	 * Tries to reach Target based on distance from Current position, giving a nice smooth feeling when tracking a position.
	 * 
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation")
	static UE_INL_API double FInterpTo(double Current, double Target, double DeltaTime, double InterpSpeed);

	/**
	 * Tries to reach Target at a constant rate.
	 *
	 * @param		Current			Actual position
	 * @param		Target			Target position
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation")
	static UE_INL_API double FInterpTo_Constant(double Current, double Target, double DeltaTime, double InterpSpeed);

	/**
	 * Tries to reach Target rotation based on Current rotation, giving a nice smooth feeling when rotating to Target rotation.
	 *
	 * @param		Current			Actual rotation
	 * @param		Target			Target rotation
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed, if the speed given is 0, then jump to the target.
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation", meta=(Keywords="rotation rotate"))
	static UE_INL_API FRotator RInterpTo(FRotator Current, FRotator Target, float DeltaTime, float InterpSpeed);

	/**
	 * Tries to reach Target rotation at a constant rate.
	 *
	 * @param		Current			Actual rotation
	 * @param		Target			Target rotation
	 * @param		DeltaTime		Time since last tick
	 * @param		InterpSpeed		Interpolation speed
	 * @return		New interpolated position
	 */
	UFUNCTION(BlueprintPure, Category="Math|Interpolation", meta=(Keywords="rotation rotate"))
	static UE_INL_API FRotator RInterpTo_Constant(FRotator Current, FRotator Target, float DeltaTime, float InterpSpeed);

	/**
	 * Uses a simple spring model to interpolate a float from Current to Target.
	 *
	 * @param Current               Current value
	 * @param Target                Target value
	 * @param SpringState           Data related to spring model (velocity, error, etc..) - Create a unique variable per spring
	 * @param Stiffness             How stiff the spring model is (more stiffness means more oscillation around the target value)
	 * @param CriticalDampingFactor How much damping to apply to the spring (0 means no damping, 1 means critically damped which means no oscillation)
	 * @param DeltaTime             Time difference since the last update
	 * @param Mass                  Multiplier that acts like mass on a spring
	 * @param TargetVelocityAmount  If 1 then the target velocity will be calculated and used, which results following the target more closely/without lag. Values down to zero (recommended when using this to smooth data) will progressively disable this effect.
	 * @param bClamp                Whether to use the Min/Max values to clamp the motion
	 * @param MinValue              Clamps the minimum output value and cancels the velocity if it reaches this limit
	 * @param MaxValue              Clamps the maximum output value and cancels the velocity if it reaches this limit
	 * @param bInitializeFromTarget If set then the current value will be set from the target on the first update
	 */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation", meta=(AdvancedDisplay = "8"))
	static ENGINE_API float FloatSpringInterp(float Current, float Target, UPARAM(ref) FFloatSpringState& SpringState,
	                               float Stiffness, float CriticalDampingFactor, float DeltaTime,
	                               float Mass = 1.f, float TargetVelocityAmount = 1.f, 
	                               bool bClamp = false, float MinValue = -1.f, float MaxValue = 1.f,
	                               bool bInitializeFromTarget = false);

	/** Resets the state of a float spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void ResetFloatSpringState(UPARAM(ref) FFloatSpringState& SpringState);

	/** Resets the state of a vector spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void ResetVectorSpringState(UPARAM(ref) FVectorSpringState& SpringState);

	/** Resets the state of a quaternion spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void ResetQuaternionSpringState(UPARAM(ref) FQuaternionSpringState& SpringState);

	/** Sets the state velocity of a float spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void SetFloatSpringStateVelocity(UPARAM(ref) FFloatSpringState& SpringState, float Velocity);

	/** Sets the state velocity of a vector spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void SetVectorSpringStateVelocity(UPARAM(ref) FVectorSpringState& SpringState, FVector Velocity);

	/** Sets the state angular velocity of a quaternion spring */
	UFUNCTION(BlueprintCallable, Category = "Math|Interpolation")
	static ENGINE_API void SetQuaternionSpringStateAngularVelocity(UPARAM(ref) FQuaternionSpringState& SpringState, FVector AngularVelocity);

	//
	// Random stream functions
	//

	/** Returns a uniformly distributed random number between 0 and Max - 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomInt", ScriptMethodMutable))
	static ENGINE_API int32 RandomIntegerFromStream(const FRandomStream& Stream, int32 Max);

	/** Return a random integer between Min and Max (>= Min and <= Max) */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomIntInRange", ScriptMethodMutable))
	static ENGINE_API int32 RandomIntegerInRangeFromStream(const FRandomStream& Stream, int32 Min, int32 Max);

	/** Returns a random bool */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomBool", ScriptMethodMutable))
	static ENGINE_API bool RandomBoolFromStream(const FRandomStream& Stream);

	/** Returns a random float between 0 and 1 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomFloat", ScriptMethodMutable))
	static ENGINE_API float RandomFloatFromStream(const FRandomStream& Stream);

	/** Generate a random number between Min and Max */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomFloatInRange", ScriptMethodMutable))
	static ENGINE_API float RandomFloatInRangeFromStream(const FRandomStream& Stream, float Min, float Max);

	/** Returns a random vector with length of 1.0 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomUnitVector", ScriptMethodMutable))
	static ENGINE_API FVector RandomUnitVectorFromStream(const FRandomStream& Stream);

	/** Returns a random point within the specified bounding box using the first vector as an origin and the second as the half size of the AABB. */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomPointInBoundedBox", ScriptMethodMutable))
	static ENGINE_API FVector RandomPointInBoundingBoxFromStream(const FRandomStream& Stream, const FVector Center, const FVector HalfSize);

	/** Returns a random point within the specified bounding box. */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(DisplayName="Random Point In Bounding Box From Stream (Box)"), meta=(ScriptMethod="RandomPointInBox", ScriptMethodMutable))
	static ENGINE_API FVector RandomPointInBoundingBoxFromStream_Box(const FRandomStream& Stream, const FBox Box);

	/** Create a random rotation */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta=(ScriptMethod="RandomRotator", ScriptMethodMutable))
	static ENGINE_API FRotator RandomRotatorFromStream(const FRandomStream& Stream, bool bRoll);

	/** Reset a random stream */
	UFUNCTION(BlueprintCallable, Category="Math|Random", meta=(ScriptMethod="Reset", ScriptMethodMutable))
	static ENGINE_API void ResetRandomStream(const FRandomStream& Stream);

	/** Create a new random seed for a random stream */
	UFUNCTION(BlueprintCallable, Category="Math|Random", meta=(ScriptMethod="GenerateNewSeed"))
	static ENGINE_API void SeedRandomStream(UPARAM(ref) FRandomStream& Stream);

	/** Set the seed of a random stream to a specific number */
	UFUNCTION(BlueprintCallable, Category="Math|Random", meta=(ScriptMethod="SetSeed"))
	static ENGINE_API void SetRandomStreamSeed(UPARAM(ref) FRandomStream& Stream, int32 NewSeed);

	/** 
	 * Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	 * @param ConeDir					The base "center" direction of the cone.
	 * @param ConeHalfAngleInRadians	The half-angle of the cone (from ConeDir to edge), in radians.
	 * @param Stream					The random stream from which to obtain the vector.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta = (Keywords = "RandomVector", ScriptMethod = "RandomUnitVectorInConeInRadians", ScriptMethodMutable))
	static ENGINE_API FVector RandomUnitVectorInConeInRadiansFromStream(const FRandomStream& Stream, const FVector& ConeDir, float ConeHalfAngleInRadians);

	/** 
	 * Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	 * @param ConeDir					The base "center" direction of the cone.
	 * @param ConeHalfAngleInDegrees	The half-angle of the cone (from ConeDir to edge), in degrees.
	 * @param Stream					The random stream from which to obtain the vector.
	 */
	UFUNCTION(BlueprintPure, Category="Math|Random", meta = (Keywords = "RandomVector", ScriptMethod = "RandomUnitVectorInConeInDegrees", ScriptMethodMutable))
	static FVector RandomUnitVectorInConeInDegreesFromStream(const FRandomStream& Stream, const FVector& ConeDir, float ConeHalfAngleInDegrees)
	{
		return RandomUnitVectorInConeInRadiansFromStream(Stream, ConeDir, FMath::DegreesToRadians(ConeHalfAngleInDegrees));
	}

	/**
	* Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	* The shape of the cone can be modified according to the yaw and pitch angles.
	*
	* @param MaxYawInRadians	The yaw angle of the cone (from ConeDir to horizontal edge), in radians.
	* @param MaxPitchInRadians	The pitch angle of the cone (from ConeDir to vertical edge), in radians.
	* @param Stream				The random stream from which to obtain the vector.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (Keywords = "RandomVector", ScriptMethod = "RandomUnitVectorInEllipticalConeInRadians", ScriptMethodMutable))
	static ENGINE_API FVector RandomUnitVectorInEllipticalConeInRadiansFromStream(const FRandomStream& Stream, const FVector& ConeDir, float MaxYawInRadians, float MaxPitchInRadians);

	/**
	* Returns a random vector with length of 1, within the specified cone, with uniform random distribution.
	* The shape of the cone can be modified according to the yaw and pitch angles.
	*
	* @param MaxYawInDegrees	The yaw angle of the cone (from ConeDir to horizontal edge), in degrees.
	* @param MaxPitchInDegrees	The pitch angle of the cone (from ConeDir to vertical edge), in degrees.
	* @param Stream				The random stream from which to obtain the vector.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Random", meta = (Keywords = "RandomVector", ScriptMethod = "RandomUnitVectorInEllipticalConeInDegrees", ScriptMethodMutable))
	static FVector RandomUnitVectorInEllipticalConeInDegreesFromStream(const FRandomStream& Stream, const FVector& ConeDir, float MaxYawInDegrees, float MaxPitchInDegrees)
	{
		return RandomUnitVectorInEllipticalConeInRadiansFromStream(Stream, ConeDir, FMath::DegreesToRadians(MaxYawInDegrees), FMath::DegreesToRadians(MaxPitchInDegrees));
	}

	/**
	 * Generates a 1D Perlin noise from the given value.  Returns a continuous random value between -1.0 and 1.0.
	 *
	 * @param	Value	The input value that Perlin noise will be generated from.  This is usually a steadily incrementing time value.
	 *
	 * @return	Perlin noise in the range of -1.0 to 1.0
	 */
	UFUNCTION(BlueprintPure, Category="Math|Random")
	static ENGINE_API float PerlinNoise1D(const float Value);

	//
	// Geometry
	//

	/**
	 * Finds the minimum area rectangle that encloses a set of coplanar points.
	 * Uses the exhaustive search algorithm in http://www.geometrictools.com/Documentation/MinimumAreaRectangle.pdf
	 *
	 * @param	WorldContextObject - Pointer to world context; only used when debug draw is enabled
	 * @param	InPoints - Points to enclose in the rectangle; need to be within the same plane for correct results
	 * @param	SampleSurfaceNormal - Normal indicating the surface direction for the points
	 * @param	OutRectCenter - Translation for the output rectangle from the origin
	 * @param	OutRectRotation - Rotation for the output rectangle from the XY plane
	 * @param	OutRectLengthX - Length of the output rectangle along the X axis before rotation
	 * @param	OutRectLengthY - Length of the output rectangle along the Y axis before rotation
	 * @param	bDebugDraw - Draws the output rectangle for debugging purposes provided the world context is set as well
	*/
	UFUNCTION(BlueprintCallable, Category="Math|Geometry", meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext))
	static ENGINE_API void MinAreaRectangle(UObject* WorldContextObject, const TArray<FVector>& InPoints, const FVector& SampleSurfaceNormal, FVector& OutRectCenter,
	                             FRotator& OutRectRotation, float& OutRectLengthX, float& OutRectLengthY, bool bDebugDraw = false);

	/**
	 * Determines whether a given set of points are coplanar, with a tolerance. Any three points or less are always coplanar.
	 *
	 * @param Points - The set of points to determine coplanarity for.
	 * @param Tolerance - Larger numbers means more variance is allowed.
	 *
	 * @return Whether the points are relatively coplanar, based on the tolerance
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Geometry")
	static UE_INL_API bool PointsAreCoplanar(const TArray<FVector>& Points, float Tolerance = 0.1f);

	/**
	 * Determines whether the given point is in a box. Includes points on the box.
	 *
	 * @param Point			Point to test
	 * @param BoxOrigin		Origin of the box
	 * @param BoxExtent		Extents of the box (distance in each axis from origin)
	 * @return Whether the point is in the box.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Geometry")
	static ENGINE_API bool IsPointInBox(FVector Point, FVector BoxOrigin, FVector BoxExtent);

	/**
     * Determines whether the given point is in a box. Includes points on the box.
     *
     * @param Point			Point to test
     * @param Box			Box to test against
     * @return Whether the point is in the box.
     */
    UFUNCTION(BlueprintPure, Category = "Math|Geometry", meta=(DisplayName = "Is Point In Box (Box)"))
    static ENGINE_API bool IsPointInBox_Box(FVector Point, FBox Box);

	/**
	 * Gets the volume of this box.
	 *
	 * @return The box volume.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Geometry", meta=(DisplayName = "Get Volume (Box)"))
	static ENGINE_API double GetBoxVolume(const FBox& InBox);

	/**
	 * Gets the size of this box.
	 *
	 * @return The box size.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Geometry", meta=(DisplayName = "Get Size (Box)"))
	static ENGINE_API FVector GetBoxSize(const FBox& InBox);

	/**
	 * Gets the center point of this box.
	 *
	 * @return The center point.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Geometry", meta=(DisplayName = "Get Center (Box)"))
	static ENGINE_API FVector GetBoxCenter(const FBox& InBox);

	/**
	* Determines whether a given point is in a box with a given transform. Includes points on the box.
	*
	* @param Point				Point to test
	* @param BoxWorldTransform	Component-to-World transform of the box.
	* @param BoxExtent			Extents of the box (distance in each axis from origin), in component space.
	* @return Whether the point is in the box.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Geometry")
	static ENGINE_API bool IsPointInBoxWithTransform(FVector Point, const FTransform& BoxWorldTransform, FVector BoxExtent);

	/**
	* Determines whether a given point is in a box with a given transform. Includes points on the box.
	*
	* @param Point				Point to test
	* @param BoxWorldTransform	Component-to-World transform of the box.
	* @param Box				Box to test agains in component space.
	* @return Whether the point is in the box.
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Geometry", meta=(DisplayName = "Is Point In Box With Transform (Box)"))
	static ENGINE_API bool IsPointInBoxWithTransform_Box(FVector Point, const FTransform& BoxWorldTransform, FBox BoxExtent);

	/**
	* Returns Slope Pitch and Roll angles in degrees based on the following information: 
	*
	* @param	MyRightYAxis				Right (Y) direction unit vector of Actor standing on Slope.
	* @param	FloorNormal					Floor Normal (unit) vector.
	* @param	UpVector					UpVector of reference frame.
	* @outparam OutSlopePitchDegreeAngle	Slope Pitch angle (degrees)
	* @outparam OutSlopeRollDegreeAngle		Slope Roll angle (degrees)
	*/
	UFUNCTION(BlueprintPure, Category = "Math|Geometry")
	static ENGINE_API void GetSlopeDegreeAngles(const FVector& MyRightYAxis, const FVector& FloorNormal, const FVector& UpVector, float& OutSlopePitchDegreeAngle, float& OutSlopeRollDegreeAngle);

	//
	// Intersection
	//

	/**
	 * Computes the intersection point between a line and a plane.
	 * @param		T - The t of the intersection between the line and the plane
	 * @param		Intersection - The point of intersection between the line and the plane
	 * @return		True if the intersection test was successful.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Intersection")
	static ENGINE_API bool LinePlaneIntersection(const FVector& LineStart, const FVector& LineEnd, const FPlane& APlane, float& T, FVector& Intersection);

	/**
	 * Computes the intersection point between a line and a plane.
	 * @param		T - The t of the intersection between the line and the plane
	 * @param		Intersection - The point of intersection between the line and the plane
	 * @return		True if the intersection test was successful.
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Intersection", meta = (DisplayName = "Line Plane Intersection (Origin & Normal)"))
	static ENGINE_API bool LinePlaneIntersection_OriginNormal(const FVector& LineStart, const FVector& LineEnd, FVector PlaneOrigin, FVector PlaneNormal, float& T, FVector& Intersection);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and the weight
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param Weight - The weight to blend with
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Weighted Moving Average Float"))
	static ENGINE_API float WeightedMovingAverage_Float(float CurrentSample, float PreviousSample, float Weight);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and the weight
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param Weight - The weight to blend with
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Weighted Moving Average Vector"))
	static ENGINE_API FVector WeightedMovingAverage_FVector(FVector CurrentSample, FVector PreviousSample, float Weight);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and the weight
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param Weight - The weight to blend with
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Weighted Moving Average Rotator"))
	static ENGINE_API FRotator WeightedMovingAverage_FRotator(FRotator CurrentSample, FRotator PreviousSample, float Weight);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and a weight range.
	 * The weight range is used to dynamically adjust based upon distance between the samples
	 * This allows you to smooth a value more aggressively for small noise and let large movements be smoothed less (or vice versa)
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param MaxDistance - Distance to use as the blend between min weight or max weight
	 * @param MinWeight - The weight use when the distance is small
	 * @param MaxWeight - The weight use when the distance is large
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Dynamic Weighted Moving Average Float"))
	static ENGINE_API float DynamicWeightedMovingAverage_Float(float CurrentSample, float PreviousSample, float MaxDistance, float MinWeight, float MaxWeight);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and a weight range.
	 * The weight range is used to dynamically adjust based upon distance between the samples
	 * This allows you to smooth a value more aggressively for small noise and let large movements be smoothed less (or vice versa)
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param MaxDistance - Distance to use as the blend between min weight or max weight
	 * @param MinWeight - The weight use when the distance is small
	 * @param MaxWeight - The weight use when the distance is large
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Dynamic Weighted Moving Average Vector"))
	static ENGINE_API FVector DynamicWeightedMovingAverage_FVector(FVector CurrentSample, FVector PreviousSample, float MaxDistance, float MinWeight, float MaxWeight);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and a weight range.
	 * The weight range is used to dynamically adjust based upon distance between the samples
	 * This allows you to smooth a value more aggressively for small noise and let large movements be smoothed less (or vice versa)
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param MaxDistance - Distance to use as the blend between min weight or max weight
	 * @param MinWeight - The weight use when the distance is small
	 * @param MaxWeight - The weight use when the distance is large
	 *
	 * @return the next value in the series
	 */
	UFUNCTION(BlueprintPure, Category="Math|Smoothing", meta=(DisplayName="Dynamic Weighted Moving Average Rotator"))
	static ENGINE_API FRotator DynamicWeightedMovingAverage_FRotator(FRotator CurrentSample, FRotator PreviousSample, float MaxDistance, float MinWeight, float MaxWeight);
	
	//
	// Multidimensional index conversions
	// 

	/**
	 *
	 * Maps a 1D array index to a 2D array index.
	 * 
	 * @param Index1D - The 1D array index
	 * @param XSize - X dimension of the 2D array
	 * 
	 * @return The equivalent 2D index of the array
	 */
	UFUNCTION(BlueprintPure, Category="Math|Conversions|Indices", meta=(DisplayName="Convert a 1D Index to a 2D Index"))
	static ENGINE_API FIntPoint Convert1DTo2D(int32 Index1D, int32 XSize);

	/**
	 *
	 * Maps a 1D array index to a 3D array index.
	 *
	 * @param Index1D - The 1D array index
	 * @param XSize - X dimension of the 3D array
	 * @param YSize - Y dimension of the 3D array
	 *
	 * @return The equivalent 3D index of the array
	 */
	UFUNCTION(BlueprintPure, Category="Math|Conversions|Indices", meta=(DisplayName="Convert a 1D Index to a 3D Index"))
	static ENGINE_API FIntVector Convert1DTo3D(int32 Index1D, int32 XSize, int32 YSize);

	/**
	 *
	 * Maps a 2D array index to a 1D array index.
	 *
	 * @param Index2D - The 2D array index
	 * @param XSize - X dimension of the 2D array
	 *
	 * @return The equivalent 1D index of the array
	 */
	UFUNCTION(BlueprintPure, Category="Math|Conversions|Indices", meta=(DisplayName="Convert a 2D Index to a 1D Index"))
	static ENGINE_API int32 Convert2DTo1D(const FIntPoint& Index2D, int32 XSize);

	/**
	 *
	 * Maps a 3D array index to a 1D array index.
	 *
	 * @param Index3D - The 3D array index
	 * @param XSize - X dimension of the 3D array
	 * @param YSize - Y dimension of the 3D array
	 *
	 * @return The equivalent 1D index of the array
	 */
	UFUNCTION(BlueprintPure, Category="Math|Conversions|Indices", meta=(DisplayName="Convert a 3D Index to a 1D Index"))
	static ENGINE_API int32 Convert3DTo1D(const FIntVector& Index3D, int32 XSize, int32 YSize);

	// NetQuantized vector make/breaks
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeMakeFunc))
	static FVector_NetQuantize MakeVector_NetQuantize(double X, double Y, double Z) { return FVector_NetQuantize(X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeMakeFunc))
	static FVector_NetQuantize10 MakeVector_NetQuantize10(double X, double Y, double Z) { return FVector_NetQuantize10(X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeMakeFunc))
	static FVector_NetQuantize100 MakeVector_NetQuantize100(double X, double Y, double Z) { return FVector_NetQuantize100(X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeMakeFunc))
	static FVector_NetQuantizeNormal MakeVector_NetQuantizeNormal(double X, double Y, double Z) { return FVector_NetQuantizeNormal(X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeBreakFunc))
	static void BreakVector_NetQuantize(FVector_NetQuantize InVec, double& X, double& Y, double& Z) { BreakVector((FVector)InVec, X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeBreakFunc))
	static void BreakVector_NetQuantize10(FVector_NetQuantize10 InVec, double& X, double& Y, double& Z) { BreakVector((FVector)InVec, X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeBreakFunc))
	static void BreakVector_NetQuantize100(FVector_NetQuantize100 InVec, double& X, double& Y, double& Z) { BreakVector((FVector)InVec, X, Y, Z); }
	UFUNCTION(BlueprintPure, Category = "Math|Vector", meta = (NativeBreakFunc))
	static void BreakVector_NetQuantizeNormal(FVector_NetQuantizeNormal InVec, double& X, double& Y, double& Z) { BreakVector((FVector)InVec, X, Y, Z); }

private:

	static ENGINE_API void ReportError_Divide_ByteByte();
	static ENGINE_API void ReportError_Percent_ByteByte();
	static ENGINE_API void ReportError_Divide_IntInt();
	static ENGINE_API void ReportError_Divide_FloatFloat();
	static ENGINE_API void ReportError_Divide_DoubleDouble();
	static ENGINE_API void ReportError_Divide_Int64Int64();
	static ENGINE_API void ReportError_Percent_IntInt();
	static ENGINE_API void ReportError_Percent_Int64Int64();
	static ENGINE_API void ReportError_Sqrt();
	static ENGINE_API void ReportError_Divide_VectorFloat();
	static ENGINE_API void ReportError_Divide_VectorInt();
	static ENGINE_API void ReportError_Divide_VectorVector();
	static ENGINE_API void ReportError_ProjectVectorOnToVector();
	static ENGINE_API void ReportError_Divide_IntPointOnInt();
	static ENGINE_API void ReportError_Divide_IntPointOnIntPoint();
	static ENGINE_API void ReportError_Divide_Vector2DFloat();
	static ENGINE_API void ReportError_Divide_Vector2DVector2D();
	static ENGINE_API void ReportError_DaysInMonth();
};

#undef UE_INL_API

// Conditionally inlined
#if KISMET_MATH_INLINE_ENABLED
#include "KismetMathLibrary.inl" // IWYU pragma: export
#endif
