// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// When KISMET_MATH_INLINE_ENABLED is true this file is included in the header
// so we need to mark these functions as inline.
#if KISMET_MATH_INLINE_ENABLED
	#define KISMET_MATH_INLINE			inline
#else
	#define KISMET_MATH_INLINE			// nothing
#endif

KISMET_MATH_INLINE
bool UKismetMathLibrary::RandomBool()
{
	return FMath::RandBool();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Not_PreBool(bool A)
{
	return !A;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_BoolBool(bool A, bool B)
{
	return ((!A) == (!B));
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_BoolBool(bool A, bool B)
{
	return ((!A) != (!B));
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::BooleanAND(bool A, bool B)
{
	return A && B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::BooleanNAND(bool A, bool B)
{
	return !(A && B);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::BooleanOR(bool A, bool B)
{
	return A || B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::BooleanXOR(bool A, bool B)
{
	return A ^ B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::BooleanNOR(bool A, bool B)
{
	return !(A || B);
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Multiply_ByteByte(uint8 A, uint8 B)
{
	return A * B;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Add_ByteByte(uint8 A, uint8 B)
{
	return A + B;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Subtract_ByteByte(uint8 A, uint8 B)
{
	return A - B;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Divide_ByteByte(uint8 A, uint8 B)
{
	if (B == 0)
	{
		ReportError_Divide_ByteByte();
		return 0;
	}

	return (A / B);
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Percent_ByteByte(uint8 A, uint8 B)
{
	if (B == 0)
	{
		ReportError_Percent_ByteByte();
		return 0;
	}

	return (A % B);
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::BMin(uint8 A, uint8 B)
{
	return FMath::Min<uint8>(A, B);
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::BMax(uint8 A, uint8 B)
{
	return FMath::Max<uint8>(A, B);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_ByteByte(uint8 A, uint8 B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_ByteByte(uint8 A, uint8 B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_ByteByte(uint8 A, uint8 B)
{
	return A <= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_ByteByte(uint8 A, uint8 B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_ByteByte(uint8 A, uint8 B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_ByteByte(uint8 A, uint8 B)
{
	return A != B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Multiply_IntInt(int32 A, int32 B)
{
	return A * B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Add_IntInt(int32 A, int32 B)
{
	return A + B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Subtract_IntInt(int32 A, int32 B)
{
	return A - B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_IntInt(int32 A, int32 B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_IntInt(int32 A, int32 B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_IntInt(int32 A, int32 B)
{
	return A <= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_IntInt(int32 A, int32 B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_IntInt(int32 A, int32 B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_IntInt(int32 A, int32 B)
{
	return A != B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::And_IntInt(int32 A, int32 B)
{
	return A & B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Xor_IntInt(int32 A, int32 B)
{
	return A ^ B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Or_IntInt(int32 A, int32 B)
{
	return A | B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Not_Int(int32 A)
{
	return ~A;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::SignOfInteger(int32 A)
{
	return FMath::Sign<int32>(A);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::RandomInteger(int32 A)
{
	return FMath::RandHelper(A);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::RandomIntegerInRange(int32 Min, int32 Max)
{
	return FMath::RandRange(Min, Max);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Min(int32 A, int32 B)
{
	return FMath::Min(A, B);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Max(int32 A, int32 B)
{
	return FMath::Max(A, B);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Clamp(int32 V, int32 A, int32 B)
{
	return FMath::Clamp(V, A, B);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Wrap(int32 Value, int32 Min, int32 Max)
{
	// FMath::Wrap only takes floating point types, so operate in doubles, which can hold int32 losslessly.
	return (int32)FMath::Wrap((double)Value, (double)Min, (double)Max);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Abs_Int(int32 A)
{
	return FMath::Abs(A);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Multiply_Int64Int64(int64 A, int64 B)
{
	return A * B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Add_Int64Int64(int64 A, int64 B)
{
	return A + B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Subtract_Int64Int64(int64 A, int64 B)
{
	return A - B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_Int64Int64(int64 A, int64 B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_Int64Int64(int64 A, int64 B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_Int64Int64(int64 A, int64 B)
{
	return A <= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_Int64Int64(int64 A, int64 B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_Int64Int64(int64 A, int64 B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_Int64Int64(int64 A, int64 B)
{
	return A != B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::And_Int64Int64(int64 A, int64 B)
{
	return A & B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Xor_Int64Int64(int64 A, int64 B)
{
	return A ^ B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Or_Int64Int64(int64 A, int64 B)
{
	return A | B;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Not_Int64(int64 A)
{
	return ~A;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::SignOfInteger64(int64 A)
{
	return FMath::Sign<int64>(A);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::RandomInteger64(int64 A)
{
	return FMath::RandHelper64(A);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::RandomInteger64InRange(int64 Min, int64 Max)
{
	return FMath::RandRange(Min, Max);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::MinInt64(int64 A, int64 B)
{
	return FMath::Min(A, B);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::MaxInt64(int64 A, int64 B)
{
	return FMath::Max(A, B);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::ClampInt64(int64 V, int64 A, int64 B)
{
	return FMath::Clamp(V, A, B);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Abs_Int64(int64 A)
{
	return FMath::Abs(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::MultiplyMultiply_FloatFloat(double Base, double Exp)
{
	return FMath::Pow(Base, Exp);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Multiply_IntFloat(int32 A, double B)
{
	return A * B;
}	

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Divide_IntInt(int32 A, int32 B)
{
	if (B == 0)
	{
		ReportError_Divide_IntInt();
		return 0;
	}

	return (A / B);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Divide_Int64Int64(int64 A, int64 B)
{
	if (B == 0)
	{
		ReportError_Divide_Int64Int64();
		return 0;
	}

	return (A / B);
}


KISMET_MATH_INLINE
int32 UKismetMathLibrary::Percent_IntInt(int32 A, int32 B)
{
	if (B == 0)
	{
		ReportError_Percent_IntInt();
		return 0;
	}

	return (A % B);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Percent_Int64Int64(int64 A, int64 B)
{
	if (B == 0)
	{
		ReportError_Percent_Int64Int64();
		return 0;
	}

	return (A % B);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Fraction(double A)
{
	return FMath::Fractional(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Add_DoubleDouble(double A, double B)
{
	return A + B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Subtract_DoubleDouble(double A, double B)
{
	return A - B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Multiply_DoubleDouble(double A, double B)
{
	return A * B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Divide_DoubleDouble(double A, double B)
{
	if (B == 0.0)
	{
		ReportError_Divide_DoubleDouble();
		return 0.0;
	}
	return A / B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_DoubleDouble(double A, double B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_DoubleDouble(double A, double B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_DoubleDouble(double A, double B)
{
	return A <= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_DoubleDouble(double A, double B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_DoubleDouble(double A, double B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NearlyEqual_FloatFloat(double A, double B, double ErrorTolerance)
{
	return FMath::IsNearlyEqual(A, B, ErrorTolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_DoubleDouble(double A, double B)
{
	return A != B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GridSnap_Float(double Location, double GridSize)
{
	return FMath::GridSnap(Location, GridSize);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetPI()
{
	return UE_DOUBLE_PI;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTAU()
{
	return 2.0 * UE_DOUBLE_PI;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegreesToRadians(double A)
{
	return FMath::DegreesToRadians(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::RadiansToDegrees(double A)
{
	return FMath::RadiansToDegrees(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Abs(double A)
{
	return FMath::Abs(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Sin(double A)
{
	return FMath::Sin(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Asin(double A)
{
	return FMath::Asin(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Cos(double A)
{
	return FMath::Cos(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Acos(double A)
{
	return FMath::Acos(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Tan(double A)
{
	return FMath::Tan(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Atan(double A)
{
	return FMath::Atan(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Atan2(double Y, double X)
{
	return FMath::Atan2(Y, X);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::DegSin(double A)
{
	return FMath::Sin(UE_DOUBLE_PI/(180.0) * A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegAsin(double A)
{
	return (180.0)/UE_DOUBLE_PI * FMath::Asin(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegCos(double A)
{
	return FMath::Cos(UE_DOUBLE_PI/(180.0) * A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegAcos(double A)
{
	return (180.0)/UE_DOUBLE_PI * FMath::Acos(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegTan(double A)
{
	return FMath::Tan(UE_DOUBLE_PI/(180.0) * A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegAtan(double A)
{
	return (180.0)/UE_DOUBLE_PI * FMath::Atan(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DegAtan2(double Y, double X)
{
	return (180.0)/UE_DOUBLE_PI * FMath::Atan2(Y, X);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::ClampAngle(double AngleDegrees, double MinAngleDegrees, double MaxAngleDegrees)
{
	return FMath::ClampAngle(AngleDegrees, MinAngleDegrees, MaxAngleDegrees);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Exp(double A)
{
	return FMath::Exp(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Loge(double A)
{
	return FMath::Loge(A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::Sqrt(double A)
{
	if (A >= 0.0)
	{
		return FMath::Sqrt(A);
	}
	else
	{
		ReportError_Sqrt();
		return 0.0;
	}
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Square(double A)
{
	return FMath::Square(A);
}	

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Round(double A)
{
	return static_cast<int32>(FMath::RoundToInt(A));
}	

KISMET_MATH_INLINE
int32 UKismetMathLibrary::FFloor(double A)
{
	return static_cast<int32>(FMath::FloorToInt(A));
}	

KISMET_MATH_INLINE
int32 UKismetMathLibrary::FTrunc(double A)
{
	return static_cast<int32>(FMath::TruncToInt(A));
}	

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Round64(double A)
{
	return (int64)FMath::RoundToDouble(A);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::FFloor64(double A)
{
	return (int64)FMath::FloorToDouble(A);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::FTrunc64(double A)
{
	return (int64)A;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::FCeil64(double A)
{
	return (int64)FMath::CeilToDouble(A);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::FCeil(double A)
{
	return static_cast<int32>(FMath::CeilToInt(A));
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::SignOfFloat(double A)
{
	return FMath::Sign<double>(A);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::MultiplyByPi(double Value)
{
	return Value * UE_DOUBLE_PI;
}

KISMET_MATH_INLINE
float UKismetMathLibrary::FixedTurn(float InCurrent, float InDesired, float InDeltaRate)
{
	return FMath::FixedTurn(InCurrent, InDesired, InDeltaRate);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::RandomFloat()
{
	return FMath::FRand(); // LWC_todo: add and switch to double version.
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::RandomFloatInRange(double Min, double Max)
{
	return FMath::FRandRange(Min, Max);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::FMin(double A, double B)
{
	return FMath::Min(A, B);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::FMax(double A, double B)
{
	return FMath::Max(A, B);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::FClamp(double V, double A, double B)
{
	return FMath::Clamp(V, A, B);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::FWrap(double Value, double Min, double Max)
{
	return FMath::Wrap(Value, Min, Max);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Lerp(double A, double B, double V)
{
	return A + V*(B-A);
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::FInterpTo(double Current, double Target, double DeltaTime, double InterpSpeed)
{
	return FMath::FInterpTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::FInterpTo_Constant(double Current, double Target, double DeltaTime, double InterpSpeed)
{
	return FMath::FInterpConstantTo(Current, Target, DeltaTime, InterpSpeed);
}


/* FRotator
*****************************************************************************/

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::RInterpTo_Constant(FRotator Current, FRotator Target, float DeltaTime, float InterpSpeed)
{
	return FMath::RInterpConstantTo(Current, Target, DeltaTime, InterpSpeed);
}


KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_RotatorRotator(FRotator A, FRotator B, float ErrorTolerance)
{
	return A.Equals(B, ErrorTolerance);
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_RotatorRotator(FRotator A, FRotator B, float ErrorTolerance)
{
	return !A.Equals(B, ErrorTolerance);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Multiply_RotatorFloat(FRotator A, float B)
{
	return A * B;
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Multiply_RotatorInt(FRotator A, int32 B)
{
	return A * (float)B;
}	

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::NegateRotator(FRotator A)
{
	return A.GetInverse();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::RInterpTo(FRotator Current, FRotator Target, float DeltaTime, float InterpSpeed)
{
	return FMath::RInterpTo(Current, Target, DeltaTime, InterpSpeed);
}


/* FTransform
*****************************************************************************/

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::MakeTransform(FVector Translation, FRotator Rotation, FVector Scale)
{
	return FTransform(Rotation, Translation, Scale);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakTransform(const FTransform& InTransform, FVector& Translation, FRotator& Rotation, FVector& Scale)
{
	Translation = InTransform.GetLocation();
	Rotation = InTransform.Rotator();
	Scale = InTransform.GetScale3D();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::TransformLocation(const FTransform& T, FVector Location)
{
	return T.TransformPosition(Location);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::TransformDirection(const FTransform& T, FVector Direction)
{
	return T.TransformVectorNoScale(Direction);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::InverseTransformLocation(const FTransform& T, FVector Location)
{
	return T.InverseTransformPosition(Location);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::InverseTransformDirection(const FTransform& T, FVector Direction)
{
	return T.InverseTransformVectorNoScale(Direction);
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::ComposeTransforms(const FTransform& A, const FTransform& B)
{
	return A * B;
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::MakeRelativeTransform(const FTransform& A, const FTransform& RelativeTo)
{
	return A.GetRelativeTransform(RelativeTo);
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::InvertTransform(const FTransform& T)
{
	return T.Inverse();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_TransformTransform(const FTransform& A, const FTransform& B)
{
	return NearlyEqual_TransformTransform(A, B);
}


/* FIntPoint
*****************************************************************************/

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_Zero()
{
	return FIntPoint::ZeroValue;
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_One()
{
	return FIntPoint(1);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_Up()
{
	return FIntPoint(0, -1);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_Left()
{
	return FIntPoint(-1, 0);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_Right()
{
	return FIntPoint(1, 0);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::IntPoint_Down()
{
	return FIntPoint(0, 1);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Conv_IntPointToVector2D(FIntPoint InIntPoint)
{
	return FVector2D(InIntPoint.X, InIntPoint.Y);
}


KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Add_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	return A + B;
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Add_IntPointInt(FIntPoint A, int32 B)
{
	return A + FIntPoint(B);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Subtract_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	return A - B;
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Subtract_IntPointInt(FIntPoint A, int32 B)
{
	return A - FIntPoint(B);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Multiply_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	return A * B;
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Multiply_IntPointInt(FIntPoint A, int32 B)
{
	return A * FIntPoint(B);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Divide_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	if (B.X == 0 || B.Y == 0)
	{
		ReportError_Divide_IntPointOnIntPoint();
		return FIntPoint::ZeroValue;
	}
	return A / B;
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Divide_IntPointInt(FIntPoint A, int32 B)
{
	if (B == 0)
	{
		ReportError_Divide_IntPointOnInt();
		return FIntPoint::ZeroValue;
	}

	return A / FIntPoint(B);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Equal_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_IntPointIntPoint(FIntPoint A, FIntPoint B)
{
	return A != B;
}


/* FVector2D
*****************************************************************************/

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector2D_One()
{
	return FVector2D::UnitVector;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector2D_Unit45Deg()
{
	return FVector2D::Unit45Deg;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector2D_Zero()
{
	return FVector2D::ZeroVector;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::MakeVector2D(double X, double Y)
{
	return FVector2D(X, Y);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakVector2D(FVector2D InVec, double& X, double& Y)
{
	X = InVec.X;
	Y = InVec.Y;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_Vector2DToVector(FVector2D InVec2D, float Z)
{
	return FVector(InVec2D, Z);
}

KISMET_MATH_INLINE
FIntPoint UKismetMathLibrary::Conv_Vector2DToIntPoint(FVector2D InVec2D)
{
	return InVec2D.IntPoint();
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Add_Vector2DVector2D(FVector2D A, FVector2D B)
{
	return A + B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Subtract_Vector2DVector2D(FVector2D A, FVector2D B)
{
	return A - B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Multiply_Vector2DFloat(FVector2D A, double B)
{
	return A * B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Multiply_Vector2DVector2D(FVector2D A, FVector2D B)
{
	return A * B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Divide_Vector2DFloat(FVector2D A, double B)
{
	if (B == 0.f)
	{
		ReportError_Divide_Vector2DFloat();
		return FVector2D::ZeroVector;
	}

	return A / B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Divide_Vector2DVector2D(FVector2D A, FVector2D B)
{
	if (B.X == 0.f || B.Y == 0.f)
	{
		ReportError_Divide_Vector2DVector2D();
		return FVector2D::ZeroVector;
	}

	return A / B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Add_Vector2DFloat(FVector2D A, double B)
{
	return A+B;
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Subtract_Vector2DFloat(FVector2D A, double B)
{
	return A-B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualExactly_Vector2DVector2D(FVector2D A, FVector2D B)
{ 
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_Vector2DVector2D(FVector2D A, FVector2D B, float ErrorTolerance)
{ 
	return A.Equals(B,ErrorTolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqualExactly_Vector2DVector2D(FVector2D A, FVector2D B)
{
	return A != B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_Vector2DVector2D(FVector2D A, FVector2D B, float ErrorTolerance)
{
	return !A.Equals(B,ErrorTolerance);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Negated2D(const FVector2D& A)
{
	return FVector2D(-A.X, -A.Y);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Set2D(FVector2D& A, double X, double Y)
{
	A.Set(X, Y);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::ClampAxes2D(FVector2D A, double MinAxisVal, double MaxAxisVal)
{
	return A.ClampAxes(MinAxisVal, MaxAxisVal);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::CrossProduct2D(FVector2D A, FVector2D B)
{
	return FVector2D::CrossProduct(A, B);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Distance2D(FVector2D V1, FVector2D V2)
{
	return FVector2D::Distance(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DistanceSquared2D(FVector2D V1, FVector2D V2)
{
	return FVector2D::DistSquared(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::DotProduct2D(FVector2D A, FVector2D B)
{
	return FVector2D::DotProduct(A, B);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::GetAbs2D(FVector2D A)
{
	return A.GetAbs();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetAbsMax2D(FVector2D A)
{
	return A.GetAbsMax();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetMax2D(FVector2D A)
{
	return A.GetMax();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetMin2D(FVector2D A)
{
	return A.GetMin();
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::GetRotated2D(FVector2D A, float AngleDeg)
{
	return A.GetRotated(AngleDeg);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::IsNearlyZero2D(const FVector2D& A, float Tolerance)
{
	return A.IsNearlyZero(Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::IsZero2D(const FVector2D& A)
{
	return A.IsZero();
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector2DInterpTo(FVector2D Current, FVector2D Target, float DeltaTime, float InterpSpeed)
{
	return FMath::Vector2DInterpTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector2DInterpTo_Constant(FVector2D Current, FVector2D Target, float DeltaTime, float InterpSpeed)
{
	return FMath::Vector2DInterpConstantTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Normal2D(FVector2D A)
{
	return A.GetSafeNormal();
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::NormalSafe2D(FVector2D A, float Tolerance)
{
	return A.GetSafeNormal(Tolerance);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Normalize2D(FVector2D& A, float Tolerance)
{
	A.Normalize(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Spherical2DToUnitCartesian(FVector2D A)
{
	return A.SphericalToUnitCartesian();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::ToDirectionAndLength2D(FVector2D A, FVector2D& OutDir, double& OutLength)
{
	A.ToDirectionAndLength(OutDir, OutLength);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::ToRounded2D(FVector2D A)
{
	return A.RoundToVector();
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::ToSign2D(FVector2D A)
{
	return A.GetSignVector();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::VSize2D(FVector2D A)
{
	return A.Size();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::VSize2DSquared(FVector2D A)
{
	return A.SizeSquared();
}


/* FVector
*****************************************************************************/

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Zero()
{
	return FVector::ZeroVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_One()
{
	return FVector::OneVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Forward()
{
	return FVector::ForwardVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Backward()
{
	return FVector::BackwardVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Up()
{
	return FVector::UpVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Down()
{
	return FVector::DownVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Right()
{
	return FVector::RightVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Left()
{
	return FVector::LeftVector;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::MakeVector(double X, double Y, double Z)
{
	return FVector(X, Y, Z);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector_Assign(FVector& A, const FVector& InVector)
{
	A = InVector;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector_Set(FVector& A, double X, double Y, double Z)
{
	A.Set(X, Y, Z);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakVector(FVector InVec, double& X, double& Y, double& Z)
{
	X = InVec.X;
	Y = InVec.Y;
	Z = InVec.Z;
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Conv_VectorToRotator(FVector InVec)
{
	return InVec.ToOrientationRotator();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Conv_VectorToQuaternion(FVector InVec)
{
	return InVec.ToOrientationQuat();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_SlerpVectorToDirection(FVector Vector, FVector Direction, double Alpha)
{
	return FVector::SlerpVectorToDirection(Vector, Direction, Alpha);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_SlerpNormals(FVector NormalA, FVector NormalB, double Alpha)
{
	return FVector::SlerpNormals(NormalA, NormalB, Alpha);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Conv_VectorToLinearColor(FVector InVec)
{
	return FLinearColor(InVec);	
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Conv_VectorToVector2D(FVector InVec)
{
	return FVector2D(InVec);
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::Conv_VectorToTransform(FVector InTranslation)
{
	return FTransform(InTranslation);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Multiply_VectorFloat(FVector A, double B)
{
	return A * B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Multiply_VectorInt(FVector A, int32 B)
{
	return A * (float)B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Multiply_VectorVector(FVector A, FVector B)
{
	return A * B;
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Divide_VectorFloat(FVector A, double B)
{
	if (B == 0.f)
	{
		ReportError_Divide_VectorFloat();
		return FVector::ZeroVector;
	}

	return A / B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Divide_VectorInt(FVector A, int32 B)
{
	if (B == 0)
	{
		ReportError_Divide_VectorInt();
		return FVector::ZeroVector;
	}

	return A / (float)B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Divide_VectorVector(FVector A, FVector B)
{
	if (B.X == 0.f || B.Y == 0.f || B.Z == 0.f)
	{
		ReportError_Divide_VectorVector();
		return FVector::ZeroVector;
	}

	return A / B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Add_VectorVector(FVector A, FVector B)
{
	return A + B;
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Add_VectorFloat(FVector A, double B)
{
	return A + B;
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Add_VectorInt(FVector A, int32 B)
{
	return A + (float)B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Subtract_VectorVector(FVector A, FVector B)
{
	return A - B;
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Subtract_VectorFloat(FVector A, double B)
{
	return A - B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Subtract_VectorInt(FVector A, int32 B)
{
	return A - (float)B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::LessLess_VectorRotator(FVector A, FRotator B)
{
	return B.UnrotateVector(A);
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::GreaterGreater_VectorRotator(FVector A, FRotator B)
{
	return B.RotateVector(A);
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualExactly_VectorVector(FVector A, FVector B)
{
	return A == B;
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_VectorVector(FVector A, FVector B, float ErrorTolerance)
{
	return A.Equals(B, ErrorTolerance);
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqualExactly_VectorVector(FVector A, FVector B)
{
	return A != B;
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_VectorVector(FVector A, FVector B, float ErrorTolerance)
{
	return !A.Equals(B, ErrorTolerance);
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Cross_VectorVector(FVector A, FVector B)
{
	return FVector::CrossProduct(A, B);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Dot_VectorVector(FVector A, FVector B)
{
	return FVector::DotProduct(A, B);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_Distance(FVector V1, FVector V2)
{
	return FVector::Distance(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_DistanceSquared(FVector V1, FVector V2)
{
	return FVector::DistSquared(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_Distance2D(FVector V1, FVector V2)
{
	return FVector::DistXY(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_Distance2DSquared(FVector V1, FVector V2)
{
	return FVector::DistSquaredXY(V1, V2);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::VSize(FVector A)
{
	return A.Size();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::VSizeSquared(FVector A)
{
	return A.SizeSquared();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::VSizeXY(FVector A)
{
	return A.Size2D();
}	

KISMET_MATH_INLINE
double UKismetMathLibrary::VSizeXYSquared(FVector A)
{
	return A.SizeSquared2D();
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsNearlyZero(const FVector& A, float Tolerance)
{
	return A.IsNearlyZero(Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsZero(const FVector& A)
{
	return (A.X == 0.0f) && (A.Y == 0.0f) && (A.Z == 0.0f);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsNAN(const FVector& A)
{
	return A.ContainsNaN();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsUniform(const FVector& A, float Tolerance)
{
	return A.AllComponentsEqual(Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsUnit(const FVector& A, float SquaredLenthTolerance)
{
	return A.IsUnit(SquaredLenthTolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector_IsNormal(const FVector& A)
{
	return A.IsNormalized();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Normal(FVector A, float Tolerance)
{
	return A.GetSafeNormal(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Normal2D(FVector A, float Tolerance)
{
	return A.GetSafeNormal2D(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_NormalUnsafe(const FVector& A)
{
	return A.GetUnsafeNormal();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector_Normalize(FVector& A, float Tolerance)
{
	A.Normalize(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::VLerp(FVector A, FVector B, float V)
{
	return A + V * (B - A);
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::VInterpTo(FVector Current, FVector Target, float DeltaTime, float InterpSpeed)
{
	return FMath::VInterpTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::VInterpTo_Constant(FVector Current, FVector Target, float DeltaTime, float InterpSpeed)
{
	return FMath::VInterpConstantTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_Reciprocal(const FVector& A)
{
	return A.Reciprocal();
}	

KISMET_MATH_INLINE
FVector UKismetMathLibrary::RandomPointInBoundingBox(const FVector Center, const FVector HalfSize)
{
	const FVector BoxMin = Center - HalfSize;
	const FVector BoxMax = Center + HalfSize;
	return FMath::RandPointInBox(FBox(BoxMin, BoxMax));
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::RandomPointInBoundingBox_Box(const FBox Box)
{
	return FMath::RandPointInBox(Box);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::RandomUnitVectorInConeInRadians(FVector ConeDir, float ConeHalfAngleInRadians)
{
	return FMath::VRandCone(ConeDir, ConeHalfAngleInRadians);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::GetReflectionVector(FVector Direction, FVector SurfaceNormal)
{
	// Same as MirrorVectorByNormal() below - though they are bound to different names so keeping both for now
	return FMath::GetReflectionVector(Direction, SurfaceNormal);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::MirrorVectorByNormal(FVector InVect, FVector InNormal)
{
	// Same as GetReflectionVector() above - though they are bound to cdifferent names so keeping both for now
	return FMath::GetReflectionVector(InVect, InNormal);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_MirrorByPlane(FVector A, const FPlane& InPlane)
{
	return A.MirrorByPlane(InPlane);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_SnappedToGrid(FVector InVect, float InGridSize)
{
	return InVect.GridSnap(InGridSize);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_BoundedToCube(FVector InVect, float InRadius)
{
	return InVect.BoundToCube(InRadius);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector_AddBounded(FVector& A, FVector InAddVect, float InRadius)
{
	return A.AddBounded(InAddVect, InRadius);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_BoundedToBox(FVector InVect, FVector InBoxMin, FVector InBoxMax)
{
	return InVect.BoundToBox(InBoxMin, InBoxMax);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ProjectOnToNormal(FVector V, FVector InNormal)
{
	return V.ProjectOnToNormal(InNormal);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::ProjectVectorOnToVector(FVector V, FVector Target)
{
	if (Target.SizeSquared() > UE_SMALL_NUMBER)
	{
		return V.ProjectOnTo(Target);
	}
	else
	{
		ReportError_ProjectVectorOnToVector();
		return FVector::ZeroVector;
	}
}

KISMET_MATH_INLINE
void UKismetMathLibrary::FindNearestPointsOnLineSegments(FVector Segment1Start, FVector Segment1End, FVector Segment2Start, FVector Segment2End, FVector& Segment1Point, FVector& Segment2Point)
{
	FMath::SegmentDistToSegmentSafe(Segment1Start, Segment1End, Segment2Start, Segment2End, Segment1Point, Segment2Point);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::FindClosestPointOnSegment(FVector Point, FVector SegmentStart, FVector SegmentEnd)
{
	return FMath::ClosestPointOnSegment(Point, SegmentStart, SegmentEnd);
}

KISMET_MATH_INLINE
float UKismetMathLibrary::GetPointDistanceToSegment(FVector Point, FVector SegmentStart, FVector SegmentEnd)
{
	return FMath::PointDistToSegment(Point, SegmentStart, SegmentEnd);
}

KISMET_MATH_INLINE
float UKismetMathLibrary::GetPointDistanceToLine(FVector Point, FVector LineOrigin, FVector LineDirection)
{
	return FMath::PointDistToLine(Point, LineDirection, LineOrigin);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::ProjectPointOnToPlane(FVector Point, FVector PlaneBase, FVector PlaneNormal)
{
	return FVector::PointPlaneProject(Point, PlaneBase, PlaneNormal);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::ProjectVectorOnToPlane(FVector V, FVector PlaneNormal)
{
	return FVector::VectorPlaneProject(V, PlaneNormal);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::NegateVector(FVector A)
{
	return -A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector_UnwindEuler(FVector& A)
{
	A.UnwindEuler();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::ClampVectorSize(FVector A, double Min, double Max)
{
	return A.GetClampedToSize(Min, Max);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ClampSize2D(FVector A, double Min, double Max)
{
	return A.GetClampedToSize2D(Min, Max);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ClampSizeMax(FVector A, double Max)
{
	return A.GetClampedToMaxSize(Max);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ClampSizeMax2D(FVector A, double Max)
{
	return A.GetClampedToMaxSize2D(Max);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetMinElement(FVector A)
{
	return A.GetMin();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetMaxElement(FVector A)
{
	return A.GetMax();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_GetAbsMax(FVector A)
{
	return A.GetAbsMax();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_GetAbsMin(FVector A)
{
	return A.GetAbsMin();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_GetAbs(FVector A)
{
	return A.GetAbs();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ComponentMin(FVector A, FVector B)
{
	return A.ComponentMin(B);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ComponentMax(FVector A, FVector B)
{
	return A.ComponentMax(B);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_GetSignVector(FVector A)
{
	return A.GetSignVector();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_GetProjection(FVector A)
{
	return A.Projection();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_HeadingAngle(FVector A)
{
	return A.HeadingAngle();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector_CosineAngle2D(FVector A, FVector B)
{
	return A.CosineAngle2D(B);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ToRadians(FVector A)
{
	return FVector::DegreesToRadians(A);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Vector_ToDegrees(FVector A)
{
	return FVector::RadiansToDegrees(A);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Vector_UnitCartesianToSpherical(FVector A)
{
	return A.UnitCartesianToSpherical();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::GetDirectionUnitVector(FVector From, FVector To)
{
	return (To - From).GetSafeNormal();
}

KISMET_MATH_INLINE
FIntVector UKismetMathLibrary::FTruncVector(const FVector& InVector)
{
	return FIntVector(static_cast<int32>(FMath::TruncToInt(InVector.X)),
		static_cast<int32>(FMath::TruncToInt(InVector.Y)),
		static_cast<int32>(FMath::TruncToInt(InVector.Z)));
}

KISMET_MATH_INLINE
FIntVector2 UKismetMathLibrary::FTruncVector2D(const FVector2D& InVector2D)
{
	return FIntVector2(static_cast<int32>(FMath::TruncToInt(InVector2D.X)),
		static_cast<int32>(FMath::TruncToInt(InVector2D.Y)));
}

/* FVector4
*****************************************************************************/

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_Zero()
{
	return FVector4(ForceInitToZero);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::MakeVector4(double X, double Y, double Z, double W)
{
	return FVector4(X, Y, Z, W);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakVector4(const FVector4& InVec, double& X, double& Y, double& Z, double& W)
{
	X = InVec.X;
	Y = InVec.Y;
	Z = InVec.Z;
	W = InVec.W;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_Vector4ToVector(const FVector4& InVector4)
{
	return FVector(InVector4.X, InVector4.Y, InVector4.Z);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Conv_Vector4ToRotator(const FVector4& InVec)
{
	return InVec.ToOrientationRotator();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Conv_Vector4ToQuaternion(const FVector4& InVec)
{
	return InVec.ToOrientationQuat();
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Add_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A + B;
}	

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Subtract_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A - B;
}	

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Multiply_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A * B;
}	

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Divide_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A / B;
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualExactly_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A == B;
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_Vector4Vector4(const FVector4& A, const FVector4& B, float ErrorTolerance)
{
	return A.Equals(B, ErrorTolerance);
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqualExactly_Vector4Vector4(const FVector4& A, const FVector4& B)
{
	return A != B;
}	

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_Vector4Vector4(const FVector4& A, const FVector4& B, float ErrorTolerance)
{
	return !A.Equals(B, ErrorTolerance);
}	

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_Negated(const FVector4& A)
{
	return -A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector4_Assign(FVector4& A, const FVector4& InVector)
{
	A = InVector;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector4_Set(FVector4& A, double X, double Y, double Z, double W)
{
	A.Set(X, Y, Z, W);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_CrossProduct3(const FVector4& A, const FVector4& B)
{
	return A ^ B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_DotProduct(const FVector4& A, const FVector4& B)
{
	return Dot4(A, B);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_DotProduct3(const FVector4& A, const FVector4& B)
{
	return Dot3(A, B);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector4_IsNAN(const FVector4& A)
{
	return A.ContainsNaN();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector4_IsNearlyZero3(const FVector4& A, float Tolerance)
{
	return A.IsNearlyZero3(Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector4_IsZero(const FVector4& A)
{
	return (A.X == 0.0f) && (A.Y == 0.0f) && (A.Z == 0.0f) && (A.W == 0.0f);
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_Size(const FVector4& A)
{
	return A.Size();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_SizeSquared(const FVector4& A)
{
	return A.SizeSquared();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_Size3(const FVector4& A)
{
	return A.Size3();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Vector4_SizeSquared3(const FVector4& A)
{
	return A.SizeSquared3();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector4_IsUnit3(const FVector4& A, float SquaredLenthTolerance)
{
	return A.IsUnit3(SquaredLenthTolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Vector4_IsNormal3(const FVector4& A)
{
	return A.IsUnit3(UE_THRESH_VECTOR_NORMALIZED);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_Normal3(const FVector4& A, float Tolerance)
{
	return A.GetSafeNormal(Tolerance);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_NormalUnsafe3(const FVector4& A)
{
	return A.GetUnsafeNormal3();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Vector4_Normalize3(FVector4& A, float Tolerance)
{
	A = A.GetSafeNormal(Tolerance);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Vector4_MirrorByVector3(const FVector4& Direction, const FVector4& SurfaceNormal)
{
	return Direction.Reflect3(SurfaceNormal);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::TransformVector4(const FMatrix& Matrix, const FVector4& Vec4)
{
	return Matrix.TransformFVector4(Vec4);
}


/* FName
*****************************************************************************/

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_NameName(FName A, FName B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_NameName(FName A, FName B)
{
	return A != B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_ObjectObject(class UObject* A, class UObject* B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_ObjectObject(class UObject* A, class UObject* B)
{
	return A != B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_ClassClass(class UClass* A, class UClass* B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_ClassClass(class UClass* A, class UClass* B)
{
	return A != B;
}

/* DateTime functions
*****************************************************************************/

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::Add_DateTimeTimespan(FDateTime A, FTimespan B)
{
	return A + B;
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::Subtract_DateTimeTimespan(FDateTime A, FTimespan B)
{
	return A - B;
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::Subtract_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A - B;
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::Add_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A + FTimespan(B.GetTicks());
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A != B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_DateTimeDateTime(FDateTime A, FDateTime B)
{
	return A <= B;
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::GetDate(FDateTime A)
{
	return A.GetDate();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetDay(FDateTime A)
{
	return A.GetDay();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetDayOfYear(FDateTime A)
{
	return A.GetDayOfYear();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetHour(FDateTime A)
{
	return A.GetHour();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetHour12(FDateTime A)
{
	return A.GetHour12();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetMillisecond(FDateTime A)
{
	return A.GetMillisecond();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetMinute(FDateTime A)
{
	return A.GetMinute();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetMonth(FDateTime A)
{
	return A.GetMonth();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetSecond(FDateTime A)
{
	return A.GetSecond();
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::GetTimeOfDay(FDateTime A)
{
	return A.GetTimeOfDay();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetYear(FDateTime A)
{
	return A.GetYear();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::IsAfternoon(FDateTime A)
{
	return A.IsAfternoon();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::IsMorning(FDateTime A)
{
	return A.IsMorning();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::DaysInMonth(int32 Year, int32 Month)
{
	if ((Month < 1) || (Month > 12))
	{
		ReportError_DaysInMonth();
		return 0;
	}

	return FDateTime::DaysInMonth(Year, Month);
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::DaysInYear(int32 Year)
{
	return FDateTime::DaysInYear(Year);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::IsLeapYear(int32 Year)
{
	return FDateTime::IsLeapYear(Year);
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::DateTimeMaxValue()
{
	return FDateTime::MaxValue();
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::DateTimeMinValue()
{
	return FDateTime::MinValue();
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::Now()
{
	return FDateTime::Now();
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::Today()
{
	return FDateTime::Today();
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::UtcNow()
{
	return FDateTime::UtcNow();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::DateTimeFromIsoString(FString IsoString, FDateTime& Result)
{
	return FDateTime::ParseIso8601(*IsoString, Result);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::DateTimeFromString(FString DateTimeString, FDateTime& Result)
{
	return FDateTime::Parse(DateTimeString, Result);
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::ToUnixTimestamp(const FDateTime& Time)
{
	return Time.ToUnixTimestamp();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::ToUnixTimestampDouble(const FDateTime& Time)
{
	return Time.ToUnixTimestampDecimal();
}

KISMET_MATH_INLINE
FDateTime UKismetMathLibrary::FromUnixTimestamp(const int64 UnixTime)
{
	return FDateTime::FromUnixTimestamp(UnixTime);
}

/* Timespan functions
*****************************************************************************/

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::Add_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A + B;
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::Subtract_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A - B;
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::Multiply_TimespanFloat(FTimespan A, float Scalar)
{
	return A * Scalar;
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::Divide_TimespanFloat(FTimespan A, float Scalar)
{
	return A / Scalar;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A != B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Greater_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A > B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::GreaterEqual_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A >= B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Less_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A < B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LessEqual_TimespanTimespan(FTimespan A, FTimespan B)
{
	return A <= B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetDays(FTimespan A)
{
	return A.GetDays();
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::GetDuration(FTimespan A)
{
	return A.GetDuration();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetHours(FTimespan A)
{
	return A.GetHours();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetMilliseconds(FTimespan A)
{
	return A.GetFractionMilli();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetMinutes(FTimespan A)
{
	return A.GetMinutes();
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::GetSeconds(FTimespan A)
{
	return A.GetSeconds();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTotalDays(FTimespan A)
{
	return A.GetTotalDays();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTotalHours(FTimespan A)
{
	return A.GetTotalHours();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTotalMilliseconds(FTimespan A)
{
	return A.GetTotalMilliseconds();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTotalMinutes(FTimespan A)
{
	return A.GetTotalMinutes();
}

KISMET_MATH_INLINE
double UKismetMathLibrary::GetTotalSeconds(FTimespan A)
{
	return A.GetTotalSeconds();
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::TimespanMaxValue()
{
	return FTimespan::MaxValue();
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::TimespanMinValue()
{
	return FTimespan::MinValue();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::TimespanRatio(FTimespan A, FTimespan B)
{
	return static_cast<float>(FTimespan::Ratio(A, B));
}

KISMET_MATH_INLINE
FTimespan UKismetMathLibrary::TimespanZeroValue()
{
	return FTimespan::Zero();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::TimespanFromString(FString TimespanString, FTimespan& Result)
{
	return FTimespan::Parse(TimespanString, Result);
}


/* K2 Utilities
*****************************************************************************/

KISMET_MATH_INLINE
double UKismetMathLibrary::Conv_ByteToDouble(uint8 InByte)
{
	return (double)InByte;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Conv_IntToDouble(int32 InInt)
{
	return (double)InInt;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Conv_IntToInt64(int32 InInt)
{
	return (int64)InInt;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Conv_IntToByte(int32 InInt)
{
	return (uint8)InInt;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Conv_Int64ToInt(int64 InInt)
{
	return (int32)InInt;
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Conv_DoubleToFloat(double InDouble)
{
	return (float)InDouble;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Conv_FloatToDouble(float InFloat)
{
	return (double)InFloat;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Conv_Int64ToByte(int64 InInt)
{
	return (uint8)InInt;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Conv_DoubleToInt64(double InDouble)
{
	return (int64)InDouble;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Conv_Int64ToDouble(int64 InInt)
{
	return (double)InInt;
}

KISMET_MATH_INLINE
FIntVector UKismetMathLibrary::Conv_IntToIntVector(int32 InInt)
{
	return FIntVector(InInt, InInt, InInt);
}

KISMET_MATH_INLINE
FIntVector2 UKismetMathLibrary::Conv_IntToIntVector2(int32 InInt)
{
	return FIntVector2(InInt, InInt);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_IntToVector(int32 InInt)
{
	return FVector((float)InInt);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Conv_IntToBool(int32 InInt)
{
	return InInt == 0 ? false : true;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Conv_BoolToInt(bool InBool)
{
	return InBool ? 1 : 0;
}

KISMET_MATH_INLINE
uint8 UKismetMathLibrary::Conv_BoolToByte(bool InBool)
{
	return InBool ? 1 : 0;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::Conv_BoolToDouble(bool InBool)
{
	return InBool ? 1.0 : 0.0;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::Conv_ByteToInt(uint8 InByte)
{
	return (int32)InByte;
}

KISMET_MATH_INLINE
int64 UKismetMathLibrary::Conv_ByteToInt64(uint8 InByte)
{
	return (int64)InByte;
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::Conv_RotatorToTransform(const FRotator& InRotator)
{
	return FTransform(InRotator);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_RotatorToVector(FRotator InRot)
{
	return InRot.Vector();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_IntVectorToVector(const FIntVector& InIntVector)
{
	return FVector(InIntVector);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Conv_IntVector2ToVector2D(const FIntVector2& InIntVector2)
{
	return FVector2D(InIntVector2.X, InIntVector2.Y);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Conv_ColorToLinearColor(FColor InColor)
{
	return FLinearColor(InColor);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_DoubleToVector(double InDouble)
{
	return FVector(InDouble);
}

KISMET_MATH_INLINE
FVector2D UKismetMathLibrary::Conv_DoubleToVector2D(double InDouble)
{
	return FVector2D(InDouble);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Conv_DoubleToLinearColor(double InDouble)
{
	// potential precision loss by float conversion
	float Value = static_cast<float>(InDouble);
	return FLinearColor(Value, Value, Value);
}

KISMET_MATH_INLINE
FBox UKismetMathLibrary::MakeBox(FVector Min, FVector Max)
{
	return FBox(Min, Max);
}

KISMET_MATH_INLINE
FBox2D UKismetMathLibrary::MakeBox2D(FVector2D Min, FVector2D Max)
{
	return FBox2D(Min, Max);
}

KISMET_MATH_INLINE
FBoxSphereBounds UKismetMathLibrary::MakeBoxSphereBounds(FVector Origin, FVector BoxExtent, float SphereRadius)
{
	return FBoxSphereBounds(Origin, BoxExtent, SphereRadius);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakBoxSphereBounds(const FBoxSphereBounds& InBoxSphereBounds, FVector& Origin, FVector& BoxExtent, float& SphereRadius)
{
	Origin = InBoxSphereBounds.Origin;
	BoxExtent = InBoxSphereBounds.BoxExtent;
	SphereRadius = (float)InBoxSphereBounds.SphereRadius;
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotator(float Roll, float Pitch, float Yaw)
{
	return FRotator(Pitch,Yaw,Roll);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::FindLookAtRotation(const FVector& Start, const FVector& Target)
{
	return MakeRotFromX(Target - Start);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromX(const FVector& X)
{
	return FRotationMatrix::MakeFromX(X).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromY(const FVector& Y)
{
	return FRotationMatrix::MakeFromY(Y).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromZ(const FVector& Z)
{
	return FRotationMatrix::MakeFromZ(Z).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromXY(const FVector& X, const FVector& Y)
{
	return FRotationMatrix::MakeFromXY(X, Y).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromXZ(const FVector& X, const FVector& Z)
{
	return FRotationMatrix::MakeFromXZ(X, Z).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromYX(const FVector& Y, const FVector& X)
{
	return FRotationMatrix::MakeFromYX(Y, X).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromYZ(const FVector& Y, const FVector& Z)
{
	return FRotationMatrix::MakeFromYZ(Y, Z).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromZX(const FVector& Z, const FVector& X)
{
	return FRotationMatrix::MakeFromZX(Z, X).Rotator();
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::MakeRotFromZY(const FVector& Z, const FVector& Y)
{
	return FRotationMatrix::MakeFromZY(Z, Y).Rotator();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakRotator(FRotator InRot, float& Roll, float& Pitch, float& Yaw)
{
	Pitch = static_cast<float>(InRot.Pitch);
	Yaw = static_cast<float>(InRot.Yaw);
	Roll = static_cast<float>(InRot.Roll);
}


/* FMatrix
*****************************************************************************/

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::Conv_MatrixToTransform(const FMatrix& InMatrix)
{
	return FTransform(InMatrix);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Conv_MatrixToRotator(const FMatrix& InMatrix)
{
	return InMatrix.Rotator();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_GetOrigin(const FMatrix& InMatrix)
{
	return InMatrix.GetOrigin();
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_Identity()
{
	return FMatrix::Identity;
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Multiply_MatrixMatrix(const FMatrix& A, const FMatrix& B)
{
	return A * B;
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Add_MatrixMatrix(const FMatrix& A, const FMatrix& B)
{
	return A + B;
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Multiply_MatrixFloat(const FMatrix& A, double B)
{
	return A * B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_MatrixMatrix(const FMatrix& A, const FMatrix& B, float Tolerance /*= KINDA_SMALL_NUMBER*/)
{
	return A.Equals(B, Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_MatrixMatrix(const FMatrix& A, const FMatrix& B, float Tolerance /*= KINDA_SMALL_NUMBER*/)
{
	return !(A.Equals(B, Tolerance));
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Matrix_TransformVector4(const FMatrix& M, FVector4 V)
{
	return M.TransformFVector4(V);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Matrix_TransformPosition(const FMatrix& M, FVector V)
{
	return M.TransformPosition(V);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_InverseTransformPosition(const FMatrix& M, FVector V)
{
	return M.InverseTransformPosition(V);
}

KISMET_MATH_INLINE
FVector4 UKismetMathLibrary::Matrix_TransformVector(const FMatrix& M, FVector V)
{
	return M.TransformVector(V);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_InverseTransformVector(const FMatrix& M, FVector V)
{
	return M.InverseTransformVector(V);
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_GetTransposed(const FMatrix& M)
{
	return M.GetTransposed();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Matrix_GetDeterminant(const FMatrix& M)
{
	return static_cast<float>(M.Determinant());
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Matrix_GetRotDeterminant(const FMatrix& M)
{
	return static_cast<float>(M.RotDeterminant());
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_GetInverse(const FMatrix& M)
{
	return M.Inverse();
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_GetTransposeAdjoint(const FMatrix& M)
{
	return M.TransposeAdjoint();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_RemoveScaling(FMatrix& M, float Tolerance /*= SMALL_NUMBER*/)
{
	M.RemoveScaling(Tolerance);
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_GetMatrixWithoutScale(const FMatrix& M, float Tolerance /*= SMALL_NUMBER*/)
{
	return M.GetMatrixWithoutScale(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_GetScaleVector(const FMatrix& M, float Tolerance /*= SMALL_NUMBER*/)
{
	return M.GetScaleVector(Tolerance);
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_RemoveTranslation(const FMatrix& M)
{
	return M.RemoveTranslation();
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_ConcatenateTranslation(const FMatrix& M, FVector Translation)
{
	return M.ConcatTranslation(Translation);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_ContainsNaN(const FMatrix& M)
{
	return M.ContainsNaN();
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_ScaleTranslation(const FMatrix& M, FVector Scale3D)
{
	FMatrix ScaledMatrix = M;
	ScaledMatrix.ScaleTranslation(Scale3D);
	return ScaledMatrix;
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Matrix_GetMaximumAxisScale(const FMatrix& M)
{
	return static_cast<float>(M.GetMaximumAxisScale());
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_ApplyScale(const FMatrix& M, float Scale)
{
	return M.ApplyScale(Scale);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_GetScaledAxis(const FMatrix& M, TEnumAsByte<EAxis::Type> Axis)
{
	return M.GetScaledAxis(Axis);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_GetScaledAxes(const FMatrix& M, FVector &X, FVector &Y, FVector &Z)
{
	M.GetScaledAxes(X, Y, Z);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_GetUnitAxis(const FMatrix& M, TEnumAsByte<EAxis::Type> Axis)
{
	return M.GetUnitAxis(Axis);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_GetUnitAxes(const FMatrix& M, FVector &X, FVector &Y, FVector &Z)
{
	M.GetUnitAxes(X, Y, Z);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_SetAxis(FMatrix& M, TEnumAsByte<EAxis::Type> Axis, FVector AxisVector)
{
	M.SetAxis(StaticCast<int32>(Axis), AxisVector);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_SetOrigin(FMatrix& M, FVector NewOrigin)
{
	M.SetOrigin(NewOrigin);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Matrix_GetColumn(const FMatrix& M, TEnumAsByte<EMatrixColumns::Type> Column)
{
	return M.GetColumn(StaticCast<int32>(Column));
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Matrix_SetColumn(FMatrix& M, TEnumAsByte<EMatrixColumns::Type> Column, FVector Value)
{
	M.SetColumn(StaticCast<int32>(Column), Value);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Matrix_GetRotator(const FMatrix& M)
{
	return M.Rotator();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Matrix_ToQuat(const FMatrix& M)
{
	return M.ToQuat();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumNearPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumNearPlane(OutPlane);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumFarPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumFarPlane(OutPlane);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumLeftPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumLeftPlane(OutPlane);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumRightPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumRightPlane(OutPlane);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumTopPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumTopPlane(OutPlane);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Matrix_GetFrustumBottomPlane(const FMatrix& M, FPlane& OutPlane)
{
	return M.GetFrustumBottomPlane(OutPlane);
}

KISMET_MATH_INLINE
FMatrix UKismetMathLibrary::Matrix_Mirror(const FMatrix& M, TEnumAsByte<EAxis::Type> MirrorAxis, TEnumAsByte<EAxis::Type> FlipAxis)
{
	FMatrix MirrorMatrix = M;
	MirrorMatrix.Mirror(MirrorAxis, FlipAxis);
	return MirrorMatrix;
}

/* FQuat
*****************************************************************************/

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Identity()
{
	return FQuat::Identity;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_QuatQuat(const FQuat& A, const FQuat& B, float Tolerance)
{
	return A.Equals(B, Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_QuatQuat(const FQuat& A, const FQuat& B, float Tolerance)
{
	return !A.Equals(B, Tolerance);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Add_QuatQuat(const FQuat& A, const FQuat& B)
{
	return A + B;
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Subtract_QuatQuat(const FQuat& A, const FQuat& B)
{
	return A - B;
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Multiply_QuatQuat(const FQuat& A, const FQuat& B)
{
	return A * B;
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::MakeQuat(float X, float Y, float Z, float W)
{
	return FQuat(X, Y, Z, W);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakQuat(const FQuat& InQuat, float& X, float& Y, float& Z, float& W)
{
	X = static_cast<float>(InQuat.X);
	Y = static_cast<float>(InQuat.Y);
	Z = static_cast<float>(InQuat.Z);
	W = static_cast<float>(InQuat.W);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Quat_IsIdentity(const FQuat& Q, float Tolerance)
{
	return Q.IsIdentity(Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Quat_IsNormalized(const FQuat& Q)
{
	return Q.IsNormalized();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Quat_IsFinite(const FQuat& Q)
{
	return !Q.ContainsNaN();
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::Quat_IsNonFinite(const FQuat& Q)
{
	return Q.ContainsNaN();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Quat_AngularDistance(const FQuat& A, const FQuat& B)
{
	return static_cast<float>(A.AngularDistance(B));
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Quat_EnforceShortestArcWith(FQuat& A, const FQuat& B)
{
	A.EnforceShortestArcWith(B);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_GetShortestArcWith(const FQuat& A, const FQuat& B)
{
	return A.GetShortestArcWith(B);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_Euler(const FQuat& Q)
{
	return Q.Euler();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Exp(const FQuat& Q)
{
	return Q.Exp();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Quat_GetAngle(const FQuat& Q)
{
	return static_cast<float>(Q.GetAngle());
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_GetAxisX(const FQuat& Q)
{
	return Q.GetAxisX();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_GetAxisY(const FQuat& Q)
{
	return Q.GetAxisY();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_GetAxisZ(const FQuat& Q)
{
	return Q.GetAxisZ();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_VectorForward(const FQuat& Q)
{
	return Q.GetForwardVector();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_VectorRight(const FQuat& Q)
{
	return Q.GetRightVector();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_VectorUp(const FQuat& Q)
{
	return Q.GetUpVector();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Quat_Normalize(FQuat& Q, float Tolerance)
{
	Q.Normalize(Tolerance);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Normalized(const FQuat& Q, float Tolerance)
{
	return Q.GetNormalized(Tolerance);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_GetRotationAxis(const FQuat& Q)
{
	return Q.GetRotationAxis();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Inversed(const FQuat& Q)
{
	return Q.Inverse();
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Log(const FQuat& Q)
{
	return Q.Log();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Quat_SetComponents(FQuat& Q, float X, float Y, float Z, float W)
{
	Q.X = X;
	Q.Y = Y;
	Q.Z = Z;
	Q.W = W;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::Quat_SetFromEuler(FQuat& Q, const FVector& Euler)
{
	Q = FQuat::MakeFromEuler(Euler);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_MakeFromEuler(const FVector& Euler)
{
	return FQuat::MakeFromEuler(Euler);
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::Quat_Rotator(const FQuat& Q)
{
	return Q.Rotator();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_ToRotationVector(const FQuat& Q)
{
	return FMath::RadiansToDegrees(Q.ToRotationVector());
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_MakeFromRotationVector(const FVector& V)
{
	return FQuat::MakeFromRotationVector(FMath::DegreesToRadians(V));
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Conv_RotatorToQuaternion(FRotator InRot)
{
	return InRot.Quaternion();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Quat_Size(const FQuat& Q)
{
	return static_cast<float>(Q.Size());
}

KISMET_MATH_INLINE
float UKismetMathLibrary::Quat_SizeSquared(const FQuat& Q)
{
	return static_cast<float>(Q.SizeSquared());
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_RotateVector(const FQuat& Q, const FVector& V)
{
	return Q.RotateVector(V);
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Quat_UnrotateVector(const FQuat& Q, const FVector& V)
{
	return Q.UnrotateVector(V);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_Slerp(const FQuat& A, const FQuat& B, double Alpha)
{
	return FQuat::Slerp(A, B, Alpha);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_FindBetweenVectors(FVector Start, FVector End)
{
	return FQuat::FindBetweenVectors(Start, End);
}

KISMET_MATH_INLINE
FQuat UKismetMathLibrary::Quat_FindBetweenNormals(FVector StartNormal, FVector EndNormal)
{
	return FQuat::FindBetweenNormals(StartNormal, EndNormal);
}


/* FLinearColor
*****************************************************************************/

// Constants

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_White()
{
	return FLinearColor::White;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Gray()
{
	return FLinearColor::Gray;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Black()
{
	return FLinearColor::Black;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Red()
{
	return FLinearColor::Red;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Green()
{
	return FLinearColor::Green;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Blue()
{
	return FLinearColor::Blue;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Yellow()
{
	return FLinearColor::Yellow;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Transparent()
{
	return FLinearColor::Transparent;
}

// Functions

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::MakeColor(float R, float G, float B, float A)
{
	return FLinearColor(R,G,B,A);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::BreakColor(FLinearColor InColor, float& R, float& G, float& B, float& A)
{
	R = InColor.R;
	G = InColor.G;
	B = InColor.B;
	A = InColor.A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_Set(FLinearColor& InOutColor, FLinearColor InColor)
{
	InOutColor.R = InColor.R;
	InOutColor.G = InColor.G;
	InOutColor.B = InColor.B;
	InOutColor.A = InColor.A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetRGBA(FLinearColor& InOutColor, float R, float G, float B, float A)
{
	InOutColor.R = R;
	InOutColor.G = G;
	InOutColor.B = B;
	InOutColor.A = A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetFromHSV(FLinearColor& InOutColor, float H, float S, float V, float A)
{
	const FLinearColor HSV(H, S, V, A);
	InOutColor = HSV.HSVToLinearRGB();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetFromSRGB(FLinearColor& InOutColor, const FColor& InSRGB)
{
	InOutColor = FLinearColor::FromSRGBColor(InSRGB);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetFromPow22(FLinearColor& InOutColor, const FColor& InColor)
{
	InOutColor = FLinearColor::FromPow22Color(InColor);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetTemperature(FLinearColor& InOutColor, float InTemperature)
{
	InOutColor = FLinearColor::MakeFromColorTemperature(InTemperature);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::LinearColor_SetRandomHue(FLinearColor& InOutColor)
{
	InOutColor = FLinearColor::MakeRandomColor();
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::HSVToRGB(float H, float S, float V, float A)
{
	const FLinearColor HSV(H, S, V, A);
	return HSV.HSVToLinearRGB();
}

KISMET_MATH_INLINE
FColor UKismetMathLibrary::LinearColor_ToRGBE(FLinearColor InLinearColor)
{
	return InLinearColor.ToRGBE();
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::Conv_LinearColorToVector(FLinearColor InLinearColor)
{
	return FVector(InLinearColor);
}

KISMET_MATH_INLINE
FColor UKismetMathLibrary::Conv_LinearColorToColor(FLinearColor InLinearColor, bool InUseSRGB)
{
	return InLinearColor.ToFColor(InUseSRGB);
}

KISMET_MATH_INLINE
void UKismetMathLibrary::RGBToHSV(FLinearColor InColor, float& H, float& S, float& V, float& A)
{
	const FLinearColor HSV(InColor.LinearRGBToHSV());
	H = HSV.R;
	S = HSV.G;
	V = HSV.B;
	A = HSV.A;
}

KISMET_MATH_INLINE
void UKismetMathLibrary::HSVToRGB_Vector(FLinearColor HSV, FLinearColor& RGB)
{
	RGB = HSV.HSVToLinearRGB();
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::HSVToRGBLinear(FLinearColor HSV)
{
	return HSV.HSVToLinearRGB();
}

KISMET_MATH_INLINE
void UKismetMathLibrary::RGBToHSV_Vector(FLinearColor RGB, FLinearColor& HSV)
{
	HSV = RGB.LinearRGBToHSV();
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::RGBLinearToHSV(FLinearColor RGB)
{
	return RGB.LinearRGBToHSV();
}

KISMET_MATH_INLINE
FColor UKismetMathLibrary::LinearColor_Quantize(FLinearColor InColor)
{
	// @todo remove LinearColor_Quantize from Kismet?
	//	QuantizeRound should almost always be used instead
	// call QuantizeFloor to match old behavior :
	return InColor.QuantizeFloor();
}

KISMET_MATH_INLINE
FColor UKismetMathLibrary::LinearColor_QuantizeRound(FLinearColor InColor)
{
	return InColor.QuantizeRound();
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_Desaturated(FLinearColor InColor, float InDesaturation)
{
	return InColor.Desaturate(InDesaturation);
}

KISMET_MATH_INLINE
float UKismetMathLibrary::LinearColor_Distance(FLinearColor C1, FLinearColor C2)
{
	return FLinearColor::Dist(C1, C2);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColor_ToNewOpacity(FLinearColor InColor, float InOpacity)
{
	return InColor.CopyWithNewOpacity(InOpacity);
}

KISMET_MATH_INLINE
float UKismetMathLibrary::LinearColor_GetLuminance(FLinearColor InColor)
{
	return InColor.GetLuminance();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::LinearColor_GetMax(FLinearColor InColor)
{
	return InColor.GetMax();
}

KISMET_MATH_INLINE
float UKismetMathLibrary::LinearColor_GetMin(FLinearColor InColor)
{
	return InColor.GetMin();
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::CInterpTo(FLinearColor Current, FLinearColor Target, float DeltaTime, float InterpSpeed)
{
	return FMath::CInterpTo(Current, Target, DeltaTime, InterpSpeed);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColorLerp(FLinearColor A, FLinearColor B, float Alpha)
{
	return A + Alpha * (B - A);
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::LinearColorLerpUsingHSV(FLinearColor A, FLinearColor B, float Alpha)
{
	return FLinearColor::LerpUsingHSV(A, B, Alpha);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::LinearColor_IsNearEqual(FLinearColor A, FLinearColor B, float Tolerance)
{
	return A.Equals(B, Tolerance);
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::EqualEqual_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A == B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::NotEqual_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A != B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Add_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A + B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Subtract_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A - B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Multiply_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A * B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Multiply_LinearColorFloat(FLinearColor A, float B)
{
	return A * B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::Divide_LinearColorLinearColor(FLinearColor A, FLinearColor B)
{
	return A / B;
}

KISMET_MATH_INLINE
FString UKismetMathLibrary::ToHex_LinearColor(FLinearColor InColor)
{
	return InColor.ToFColor(true).ToHex();
}

KISMET_MATH_INLINE
FString UKismetMathLibrary::SelectString(const FString& A, const FString& B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FText UKismetMathLibrary::SelectText(const FText A, const FText B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FName UKismetMathLibrary::SelectName(const FName A, const FName B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
int32 UKismetMathLibrary::SelectInt(int32 A, int32 B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
double UKismetMathLibrary::SelectFloat(double A, double B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FVector UKismetMathLibrary::SelectVector(FVector A, FVector B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FRotator UKismetMathLibrary::SelectRotator(FRotator A, FRotator B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FLinearColor UKismetMathLibrary::SelectColor(FLinearColor A, FLinearColor B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
FTransform UKismetMathLibrary::SelectTransform(const FTransform& A, const FTransform& B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
UObject* UKismetMathLibrary::SelectObject(UObject* A, UObject* B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
UClass* UKismetMathLibrary::SelectClass(UClass* A, UClass* B, bool bSelectA)
{
	return bSelectA ? A : B;
}

KISMET_MATH_INLINE
bool UKismetMathLibrary::PointsAreCoplanar(const TArray<FVector>& Points, float Tolerance)
{
	return FMath::PointsAreCoplanar(Points, Tolerance);
}
