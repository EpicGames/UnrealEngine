// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This kismet library is used for helper functions primarily used in the kismet compiler
 * NOTE: Do not change the signatures for any of these functions as it can break the kismet compiler and/or the nodes referencing them
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetNodeHelperLibrary.generated.h"

UCLASS(meta=(BlueprintThreadSafe), MinimalAPI)
class UKismetNodeHelperLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	// --- Functions for handling bitmasking an integer as an array of bools ------------------------------

	/**
	 * Returns whether the bit at index "Index" is set or not in the data
	 *
	 * @param Data - The integer containing the bits that are being tested against
	 * @param Index - The bit index into the Data that we are inquiring
	 * @return  - Whether the bit at index "Index" is set or not
	 */
	 UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API bool BitIsMarked(int32 Data, int32 Index);

	/**
	 * Sets the bit at index "Index" in the data
	 *
	 * @param Data - The integer containing the bits that are being set
	 * @param Index - The bit index into the Data that we are setting
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API void MarkBit(int32& Data, int32 Index);

	/**
	 * Clears the bit at index "Index" in the data
	 *
	 * @param Data - The integer containing the bits that are being cleared
	 * @param Index - The bit index into the Data that we are clearing
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API void ClearBit(int32& Data, int32 Index);

	/**
	 * Clears all of the bit in the data
	 *
	 * @param Data - The integer containing the bits that are being cleared
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API void ClearAllBits(int32& Data);

	/**
	 * Returns whether there exists an unmarked bit in the data
	 *
	 * @param Data - The data being tested against
	 * @param NumBits - The logical number of bits we want to track
	 * @return - Whether there is a bit not marked in the data
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API bool HasUnmarkedBit(int32 Data, int32 NumBits);

	/**
	 * Returns whether there exists a marked bit in the data
	 *
	 * @param Data - The data being tested against
	 * @param NumBits - The logical number of bits we want to track
	 * @return - Whether there is a bit marked in the data
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API bool HasMarkedBit(int32 Data, int32 NumBits);

	/**
	 * Gets an already unmarked bit and returns the bit index selected
	 *
	 * @param Data - The integer containing the bits that are being set
	 * @param StartIdx - The index to start with when determining the selection'
	 * @param NumBits - The logical number of bits we want to track
	 * @param bRandom - Whether to select a random index or not
	 * @return - The index that was selected (returns INDEX_NONE if there was no unmarked bits to choose from)
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE", NotBlueprintThreadSafe))
	static ENGINE_API int32 GetUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits, bool bRandom);

	/**
	 * Gets a random not already marked bit and returns the bit index selected
	 *
	 * @param Data - The integer containing the bits that are being set
	 * @param NumBits - The logical number of bits we want to track
	 * @return - The index that was selected (returns INDEX_NONE if there was no unmarked bits to choose from)
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE", NotBlueprintThreadSafe))
	static ENGINE_API int32 GetRandomUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits);

	/**
	 * Gets the first index not already marked starting from a specific index and returns the bit index selected
	 *
	 * @param Data - The integer containing the bits that are being set
	 * @param StartIdx - The index to start looking for an available index from
	 * @param NumBits - The logical number of bits we want to track
	 * @return - The index that was selected (returns INDEX_NONE if there was no unmarked bits to choose from)
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API int32 GetFirstUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits);

	// --- Functions for handling Enumerators ------------------------------

	/**
	 * Gets enumerator name.
	 *
	 * @param Enum - Enumeration
	 * @param EnumeratorValue - Value of searched enumeration
	 * @return - name of the searched enumerator, or NAME_None
	 */
	UFUNCTION(BlueprintPure, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API FName GetEnumeratorName(const UEnum* Enum, uint8 EnumeratorValue);

	/**
	 * Gets enumerator name as FString. Use DeisplayName when possible.
	 *
	 * @param Enum - Enumeration
	 * @param EnumeratorValue - Value of searched enumeration
	 * @return - name of the searched enumerator, or NAME_None
	 */
	UFUNCTION(BlueprintPure, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API FString GetEnumeratorUserFriendlyName(const UEnum* Enum, uint8 EnumeratorValue);

	/**
	 * @param Enum - Enumeration
	 * @param EnumeratorIndex - Input value
	 * @return - if EnumeratorIndex is valid return EnumeratorIndex, otherwise return MAX value of Enum
	 */
	UFUNCTION(BlueprintPure, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API uint8 GetValidValue(const UEnum* Enum, uint8 EnumeratorValue);

	/**
	 * @param Enum - Enumeration
	 * @param EnumeratorIndex - Input index
	 * @return - The value of the enumerator, or INDEX_NONE
	 */
	UFUNCTION(BlueprintPure, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API uint8 GetEnumeratorValueFromIndex(const UEnum* Enum, uint8 EnumeratorIndex);
};
