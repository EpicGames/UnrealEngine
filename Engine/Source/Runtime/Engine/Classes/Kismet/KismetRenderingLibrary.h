// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RHI.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "Camera/CameraTypes.h"
#include "KismetRenderingLibrary.generated.h"

class UCanvas;
class UMaterialInterface;
class UTexture2D;

USTRUCT(BlueprintType)
struct FDrawToRenderTargetContext
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

#if WANTS_DRAW_MESH_EVENTS
	TOptional<FRHIBreadcrumbEventManual>* Breadcrumb = nullptr;
#endif // WANTS_DRAW_MESH_EVENTS
};

UCLASS(MinimalAPI, meta=(ScriptName="RenderingLibrary"))
class UKismetRenderingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	

	/** 
	 * Clears the specified render target with the given ClearColor.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(Keywords="ClearRenderTarget", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API void ClearRenderTarget2D(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, FLinearColor ClearColor = FLinearColor(0, 0, 0, 1));

	/**
	 * Creates a new render target and initializes it to the specified dimensions
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API UTextureRenderTarget2D* CreateRenderTarget2D(UObject* WorldContextObject, int32 Width = 256, int32 Height = 256, ETextureRenderTargetFormat Format = RTF_RGBA16f, FLinearColor ClearColor = FLinearColor::Black, bool bAutoGenerateMipMaps = false, bool bSupportUAVs = false);

	/**
	 * Creates a new render target array and initializes it to the specified dimensions
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API UTextureRenderTarget2DArray* CreateRenderTarget2DArray(UObject* WorldContextObject, int32 Width = 256, int32 Height = 256, int32 Slices = 1, ETextureRenderTargetFormat Format = RTF_RGBA16f, FLinearColor ClearColor = FLinearColor::Black, bool bAutoGenerateMipMaps = false, bool bSupportUAVs = false);

	/**
	 * Creates a new volume render target and initializes it to the specified dimensions
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API UTextureRenderTargetVolume* CreateRenderTargetVolume(UObject* WorldContextObject, int32 Width = 16, int32 Height = 16, int32 Depth = 16, ETextureRenderTargetFormat Format = RTF_RGBA16f, FLinearColor ClearColor = FLinearColor::Black, bool bAutoGenerateMipMaps = false, bool bSupportUAVs = false);

	/**
	 * Manually releases GPU resources of a render target. This is useful for blueprint creating a lot of render target that would
	 * normally be released too late by the garbage collector that can be problematic on platforms that have tight GPU memory constrains.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static ENGINE_API void ReleaseRenderTarget2D(UTextureRenderTarget2D* TextureRenderTarget);

	/**
	 * Changes the resolution of a render target. This is useful for when you need to resize the game viewport or change the in-game resolution during runtime
	 * and thus need to update the sizes of all the render targets in the game accordingly.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static ENGINE_API void ResizeRenderTarget2D(UTextureRenderTarget2D* TextureRenderTarget, int32 Width = 256, int32 Height = 256);

	/** 
	 * Renders a quad with the material applied to the specified render target.   
	 * This sets the render target even if it is already set, which is an expensive operation. 
	 * Use BeginDrawCanvasToRenderTarget / EndDrawCanvasToRenderTarget instead if rendering multiple primitives to the same render target.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(Keywords="DrawMaterialToRenderTarget", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API void DrawMaterialToRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, UMaterialInterface* Material);

	/**
	* Creates a new Static Texture from a Render Target 2D.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (DisplayName = "Render Target 2D Create Static Texture 2D Editor Only", Keywords = "Create Static Texture from Render Target", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UTexture2D* RenderTargetCreateStaticTexture2DEditorOnly(UTextureRenderTarget2D* RenderTarget, FString Name = "Texture", enum TextureCompressionSettings CompressionSettings = TC_Default, enum TextureMipGenSettings MipSettings = TMGS_FromTextureGroup);

	/**
	* Creates a new Static Texture 2D Array from a Render Target 2D Array.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (DisplayName = "Render Target 2D Array Create Static Texture 2D Array Editor Only", Keywords = "Create Static Texture from Render Target", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UTexture2DArray* RenderTargetCreateStaticTexture2DArrayEditorOnly(UTextureRenderTarget2DArray* RenderTarget, FString Name = "Texture", enum TextureCompressionSettings CompressionSettings = TC_Default, enum TextureMipGenSettings MipSettings = TMGS_FromTextureGroup);

	/**
	* Creates a new Static Texture Cube from a Render Target Cube.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (DisplayName = "Render Target Cube Create Static Texture Cube Editor Only", Keywords = "Create Static Texture from Render Target", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UTextureCube* RenderTargetCreateStaticTextureCubeEditorOnly(UTextureRenderTargetCube* RenderTarget, FString Name = "Texture", enum TextureCompressionSettings CompressionSettings = TC_Default, enum TextureMipGenSettings MipSettings = TMGS_FromTextureGroup);

	/**
	* Creates a new Static Volume Texture from a Render Target Volume.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (DisplayName = "Render Target Volume Create Static Volume Texture Editor Only", Keywords = "Create Static Texture from Render Target", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API UVolumeTexture* RenderTargetCreateStaticVolumeTextureEditorOnly(UTextureRenderTargetVolume* RenderTarget, FString Name = "Texture", enum TextureCompressionSettings CompressionSettings = TC_Default, enum TextureMipGenSettings MipSettings = TMGS_FromTextureGroup);

	/**
	 * Copies the contents of a UTextureRenderTarget2D to a UTexture2D
	 * Only works in the editor
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (Keywords = "Convert Render Target", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void ConvertRenderTargetToTexture2DEditorOnly(UObject* WorldContextObject, UTextureRenderTarget2D* RenderTarget, UTexture2D* Texture);

	/**
	 * Copies the contents of a UTextureRenderTarget2DArray to a UTexture2DArray
	 * Only works in the editor
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (Keywords = "Convert Render Target", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void ConvertRenderTargetToTexture2DArrayEditorOnly(UObject* WorldContextObject, UTextureRenderTarget2DArray* RenderTarget, UTexture2DArray* Texture);

	/**
	 * Copies the contents of a UTextureRenderTargetCube to a UTextureCube
	 * Only works in the editor
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (Keywords = "Convert Render Target", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void ConvertRenderTargetToTextureCubeEditorOnly(UObject* WorldContextObject, UTextureRenderTargetCube* RenderTarget, UTextureCube* Texture);

	/**
	 * Copies the contents of a UTextureRenderTargetVolume to a UVolumeTexture
	 * Only works in the editor
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Rendering", meta = (Keywords = "Convert Render Target", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static ENGINE_API void ConvertRenderTargetToTextureVolumeEditorOnly(UObject* WorldContextObject, UTextureRenderTargetVolume* RenderTarget, UVolumeTexture* Texture);

	/**
	 * Exports a render target as a HDR or PNG image onto the disk (depending on the format of the render target)
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ExportRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API void ExportRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, const FString& FilePath, const FString& FileName);

	/**
	* Incredibly inefficient and slow operation! Read a value as sRGB color from a render target using integer pixel coordinates.
	* LDR render targets are assumed to be in sRGB space. HDR ones are assumed to be in linear space.
	* Result is 8-bit per channel [0,255] BGRA in sRGB space.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API FColor ReadRenderTargetPixel(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, int32 X, int32 Y);

	/**
	* Incredibly inefficient and slow operation! Read a value as sRGB color from a render target using UV [0,1]x[0,1] coordinates.
	* LDR render targets are assumed to be in sRGB space. HDR ones are assumed to be in linear space.
	* Result is 8-bit per channel [0,255] BGRA in sRGB space.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API FColor ReadRenderTargetUV(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, float U, float V);

	/**
	* Incredibly inefficient and slow operation! Reads entire render target as sRGB color and returns a linear array of sRGB colors.
	* LDR render targets are assumed to be in sRGB space. HDR ones are assumed to be in linear space.
	* Result whether the operation succeeded.  If successful, OutSamples will an entry per pixel, where each is 8-bit per channel [0,255] BGRA in sRGB space.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API bool ReadRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, TArray<FColor>& OutSamples, bool bNormalize = true);

	/**
	* Incredibly inefficient and slow operation! Read a value as-is from a render target using integer pixel coordinates.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API FLinearColor ReadRenderTargetRawPixel(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, int32 X, int32 Y, bool bNormalize = true);

	/**
    * Incredibly inefficient and slow operation! Read an area of values as-is from a render target using a rectangle defined by integer pixel coordinates.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API TArray<FLinearColor> ReadRenderTargetRawPixelArea(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, int32 MinX, int32 MinY, int32 MaxX, int32 MaxY, bool bNormalize = true);

	/**
	* Incredibly inefficient and slow operation! Read a value as-is from a render target using UV [0,1]x[0,1] coordinates.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API FLinearColor ReadRenderTargetRawUV(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, float U, float V, bool bNormalize = true);

	/**
	* Incredibly inefficient and slow operation! Read entire texture as-is from a render target.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API bool ReadRenderTargetRaw(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, TArray<FLinearColor>& OutLinearSamples, bool bNormalize = true);

	/**
	* Incredibly inefficient and slow operation! Read an area of values as-is from a render target using a rectangle defined by UV [0,1]x[0,1] coordinates.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ReadRenderTarget", WorldContext = "WorldContextObject"))
	static ENGINE_API TArray<FLinearColor> ReadRenderTargetRawUVArea(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, FBox2D Area, bool bNormalize = true);

	/**
	 * Exports a Texture2D as a HDR image onto the disk.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "ExportTexture2D", WorldContext = "WorldContextObject"))
	static ENGINE_API void ExportTexture2D(UObject* WorldContextObject, UTexture2D* Texture, const FString& FilePath, const FString& FileName);

	/**
	 * Imports a texture file from disk and creates Texture2D from it. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API UTexture2D* ImportFileAsTexture2D(UObject* WorldContextObject, const FString& Filename);

	/**
	 * Imports a texture from a buffer and creates Texture2D from it.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API UTexture2D* ImportBufferAsTexture2D(UObject* WorldContextObject, const TArray<uint8>& Buffer);
	
	/**
	 * Returns a Canvas object that can be used to draw to the specified render target.
	 * Canvas has functions like DrawMaterial with size parameters that can be used to draw to a specific area of a render target.
	 * Be sure to call EndDrawCanvasToRenderTarget to complete the rendering!
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(Keywords="BeginDrawCanvasToRenderTarget", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API void BeginDrawCanvasToRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, UCanvas*& Canvas, FVector2D& Size, FDrawToRenderTargetContext& Context);

	/**  
	 * Must be paired with a BeginDrawCanvasToRenderTarget to complete rendering to a render target.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(Keywords="EndDrawCanvasToRenderTarget", WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true"))
	static ENGINE_API void EndDrawCanvasToRenderTarget(UObject* WorldContextObject, const FDrawToRenderTargetContext& Context);

	/** Create FSkelMeshSkinWeightInfo */
	UFUNCTION(BlueprintPure, Category = "Rendering", meta=(NativeMakeFunc))
	static ENGINE_API FSkelMeshSkinWeightInfo MakeSkinWeightInfo(int32 Bone0, uint8 Weight0, int32 Bone1, uint8 Weight1, int32 Bone2, uint8 Weight2, int32 Bone3, uint8 Weight3);

	/** Break FSkelMeshSkinWeightInfo */
	UFUNCTION(BlueprintPure, Category = "Rendering", meta=(NativeBreakFunc))
	static ENGINE_API void BreakSkinWeightInfo(FSkelMeshSkinWeightInfo InWeight, int32& Bone0, uint8& Weight0, int32& Bone1, uint8& Weight1, int32& Bone2, uint8& Weight2, int32& Bone3, uint8& Weight3);

	/** Set the inset shadow casting state of the given component and all its child attachments. 
	 *	Also choose if all attachments should be grouped for the inset shadow rendering. If enabled, one depth target will be shared for all attachments.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering", meta=(Keywords="SetCastShadowForAllAttachments", UnsafeDuringActorConstruction="true"))
	static ENGINE_API void SetCastInsetShadowForAllAttachments(UPrimitiveComponent* PrimitiveComponent, bool bCastInsetShadow, bool bLightAttachmentsAsGroup);

	/** Calculates the projection matrix using this view info's aspect ratio (regardless of bConstrainAspectRatio) */
	UFUNCTION(BlueprintPure, Category="Rendering", meta = (DisplayName = "Calculate Projection Matrix (Minimal View Info)", ScriptMethod = "CalculateProjectionMatrix"))
	static ENGINE_API FMatrix CalculateProjectionMatrix(const FMinimalViewInfo& MinimalViewInfo);

	/** Enables or disables the path tracer for the current Game Viewport.
	 * This command is equivalent to setting ShowFlag.PathTracing, but is accessible even from shipping builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords="EnablePathTracing"))
	static ENGINE_API void EnablePathTracing(bool bEnablePathTracer);

	/** Forces the path tracer to restart sample accumulation.
	 * This can be used to force the path tracer to compute a new frame in situations where it can not detect a change in the scene automatically.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (Keywords = "PathTracing"))
	static ENGINE_API void RefreshPathTracingOutput();

	/** Returns the number of active PSO precompilation tasks which are still running - can be used to extend loading screens to wait for PSO precaching to be finished */
	UFUNCTION(BlueprintCallable, Category = "Rendering | Pipeline Cache")
	static ENGINE_API int32 NumPrecompilingPSOsRemaining();
};
