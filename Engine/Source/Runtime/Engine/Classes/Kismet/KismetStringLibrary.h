// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetStringLibrary.generated.h"

UCLASS(meta=(BlueprintThreadSafe, ScriptName="StringLibrary"), MinimalAPI)
class UKismetStringLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Converts a double value to a string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Float)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_DoubleToString(double InDouble);

	/** Converts an integer value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Integer)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_IntToString(int32 InInt);

	/** Converts an 64-bit integer value to a string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Integer64)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_Int64ToString(int64 InInt);
	
	/** Converts a byte value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Byte)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_ByteToString(uint8 InByte);

	/** Converts a boolean value to a string, either 'true' or 'false' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Boolean)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_BoolToString(bool InBool);

	/** Converts a vector value to a string, in the form 'X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Vector)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_VectorToString(FVector InVec);

	/** Converts a float vector value to a string, in the form 'X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Float Vector)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_Vector3fToString(FVector3f InVec);

	/** Converts an IntVector value to a string, in the form 'X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (IntVector)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_IntVectorToString(FIntVector InIntVec);

	/** Converts an IntVector2 value to a string, in the form 'X= Y=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (IntVector2)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_IntVector2ToString(FIntVector2 InIntVec2);

	/** Converts an IntPoint value to a string, in the form 'X= Y=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (IntPoint)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_IntPointToString(FIntPoint InIntPoint);

	/** Converts a vector2d value to a string, in the form 'X= Y=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Vector2d)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_Vector2dToString(FVector2D InVec);

	/** Converts a rotator value to a string, in the form 'P= Y= R=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Rotator)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_RotatorToString(FRotator InRot);

	/** Converts a transform value to a string, in the form 'Translation: X= Y= Z= Rotation: P= Y= R= Scale: X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Transform)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_TransformToString(const FTransform& InTrans);

	/** Converts a UObject value to a string by calling the object's GetName method */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Object)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_ObjectToString(class UObject* InObj);

	/** Converts a FBox value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Box)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_BoxToString(const FBox& Box);

	/** Converts a FBox value to a string of its Center and Extents values. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String Center and Extents (Box)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_BoxCenterAndExtentsToString(const FBox& Box);

	/** Converts a InputDeviceId value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (InputDeviceId)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_InputDeviceIdToString(FInputDeviceId InDeviceId);

	/** Converts a PlatformUserId value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (PlatformUserId)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_PlatformUserIdToString(FPlatformUserId InPlatformUserId);

	/** Converts a linear color value to a string, in the form '(R=,G=,B=,A=)' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (LinearColor)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_ColorToString(FLinearColor InColor);

	/** Converts a name value to a string */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To String (Name)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FString Conv_NameToString(FName InName);

	/** Converts a name value to a string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Matrix)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_MatrixToString(const FMatrix& InMatrix);

	/** Converts a string to a name value */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Name", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API FName Conv_StringToName(const FString& InString);

	/** Converts a string to a int value */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Integer", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|String")
	static ENGINE_API int32 Conv_StringToInt(const FString& InString);

	/** Converts a string to a int value */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "String To Integer64", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API int64 Conv_StringToInt64(const FString& InString);

	/** Converts a string to a double value */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "String To Float", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API double Conv_StringToDouble(const FString& InString);

	/** Convert String Back To Vector. IsValid indicates whether or not the string could be successfully converted. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Vector", CompactNodeTitle = "->"), Category="Utilities|String")
	static ENGINE_API void Conv_StringToVector(const FString& InString, FVector& OutConvertedVector, bool& OutIsValid);

	/** Convert String Back To Float Vector. IsValid indicates whether or not the string could be successfully converted. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "String To Float Vector", CompactNodeTitle = "->"), Category = "Utilities|String")
	static ENGINE_API void Conv_StringToVector3f(const FString& InString, FVector3f& OutConvertedVector, bool& OutIsValid);

	/** Convert String Back To Vector2D. IsValid indicates whether or not the string could be successfully converted. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Vector2D", CompactNodeTitle = "->"), Category="Utilities|String")
	static ENGINE_API void Conv_StringToVector2D(const FString& InString, FVector2D& OutConvertedVector2D, bool& OutIsValid);

	/** Convert String Back To Rotator. IsValid indicates whether or not the string could be successfully converted. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Rotator", CompactNodeTitle = "->"), Category="Utilities|String")
	static ENGINE_API void Conv_StringToRotator(const FString& InString, FRotator& OutConvertedRotator, bool& OutIsValid);

	/** Convert String Back To Color. IsValid indicates whether or not the string could be successfully converted. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "String To Color", CompactNodeTitle = "->"), Category="Utilities|String")
	static ENGINE_API void Conv_StringToColor(const FString& InString, FLinearColor& OutConvertedColor, bool& OutIsValid);

	/**
	 * Converts a double->string, create a new string in the form AppendTo+Prefix+InDouble+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InDouble - The double value to convert
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Build String (Float)"), Category = "Utilities|String")
	static ENGINE_API FString BuildString_Double(const FString& AppendTo, const FString& Prefix, double InDouble, const FString& Suffix);

	/** 
	 * Converts a int->string, creating a new string in the form AppendTo+Prefix+InInt+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InInt - The int value to convert
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Integer)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Int(const FString& AppendTo, const FString& Prefix, int32 InInt, const FString& Suffix);

	/** 
	 * Converts a boolean->string, creating a new string in the form AppendTo+Prefix+InBool+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InBool - The bool value to convert. Will add "true" or "false" to the conversion string
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Boolean)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Bool(const FString& AppendTo, const FString& Prefix, bool InBool, const FString& Suffix);

	/** 
	 * Converts a vector->string, creating a new string in the form AppendTo+Prefix+InVector+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InVector - The vector value to convert. Uses the standard FVector::ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Vector)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Vector(const FString& AppendTo, const FString& Prefix, FVector InVector, const FString& Suffix);

	/**
	 * Converts an IntVector->string, creating a new string in the form AppendTo+Prefix+InIntVector+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InIntVector - The intVector value to convert. Uses the standard FVector::ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Build String (IntVector)"), Category = "Utilities|String")
	static ENGINE_API FString BuildString_IntVector(const FString& AppendTo, const FString& Prefix, FIntVector InIntVector2, const FString& Suffix);

	/**
	 * Converts an IntVector2->string, creating a new string in the form AppendTo+Prefix+InIntVector+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InIntVector - The intVector2 value to convert. Uses the standard FVector::ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Build String (IntVector2)"), Category = "Utilities|String")
	static ENGINE_API FString BuildString_IntVector2(const FString& AppendTo, const FString& Prefix, FIntVector2 InIntVector, const FString& Suffix);

	/** 
	 * Converts a vector2d->string, creating a new string in the form AppendTo+Prefix+InVector2d+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InVector2d - The vector2d value to convert. Uses the standard FVector2D::ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Vector2D)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Vector2d(const FString& AppendTo, const FString& Prefix, FVector2D InVector2d, const FString& Suffix);

	/** 
	 * Converts a rotator->string, creating a new string in the form AppendTo+Prefix+InRot+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InRot	- The rotator value to convert. Uses the standard ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Rotator)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Rotator(const FString& AppendTo, const FString& Prefix, FRotator InRot, const FString& Suffix);

	/** 
	 * Converts a object->string, creating a new string in the form AppendTo+Prefix+object name+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InObj - The object to convert. Will insert the name of the object into the conversion string
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Object)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Object(const FString& AppendTo, const FString& Prefix, class UObject* InObj, const FString& Suffix);

	/** 
	 * Converts a color->string, creating a new string in the form AppendTo+Prefix+InColor+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InColor - The linear color value to convert. Uses the standard ToString conversion
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (LinearColor)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Color(const FString& AppendTo, const FString& Prefix, FLinearColor InColor, const FString& Suffix);

	/** 
	 * Converts a color->string, creating a new string in the form AppendTo+Prefix+InName+Suffix
	 * @param AppendTo - An existing string to use as the start of the conversion string
	 * @param Prefix - A string to use as a prefix, after the AppendTo string
	 * @param InName - The name value to convert
	 * @param Suffix - A suffix to append to the end of the conversion string
	 * @return A new string built from the passed parameters
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Build String (Name)"), Category="Utilities|String")
	static ENGINE_API FString BuildString_Name(const FString& AppendTo, const FString& Prefix, FName InName, const FString& Suffix);


	//
	// String functions.
	//
	
	/**
	 * Concatenates two strings together to make a new string
	 * @param A - The original string
	 * @param B - The string to append to A
	 * @returns A new string which is the concatenation of A+B
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Append", CommutativeAssociativeBinaryOperator = "true"), Category="Utilities|String")
	static ENGINE_API FString Concat_StrStr(const FString& A, const FString& B);

	/**
	 * Test if the input strings are equal (A == B)
	 * @param A - The string to compare against
	 * @param B - The string to compare
	 * @returns True if the strings are equal, false otherwise
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal Exactly (String)", CompactNodeTitle = "==="), Category="Utilities|String")
	static ENGINE_API bool EqualEqual_StrStr(const FString& A, const FString& B);

	/**
	 * Test if the input strings are equal (A == B), ignoring case
	 * @param A - The string to compare against
	 * @param B - The string to compare
	 * @returns True if the strings are equal, false otherwise
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal, Case Insensitive (String)", CompactNodeTitle = "=="), Category="Utilities|String")
	static ENGINE_API bool EqualEqual_StriStri(const FString& A, const FString& B);

	/** 
	 * Test if the input string are not equal (A != B)
	 * @param A - The string to compare against
	 * @param B - The string to compare
	 * @return Returns true if the input strings are not equal, false if they are equal
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal Exactly (String)", CompactNodeTitle = "!=="), Category="Utilities|String")
	static ENGINE_API bool NotEqual_StrStr(const FString& A, const FString& B);

	/** Test if the input string are not equal (A != B), ignoring case differences
	 * @param A - The string to compare against
	 * @param B - The string to compare
	 * @return Returns true if the input strings are not equal, false if they are equal
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal, Case Insenstive (String)", CompactNodeTitle = "!="), Category="Utilities|String")
	static ENGINE_API bool NotEqual_StriStri(const FString& A, const FString& B);

	/** 
	 * Returns the number of characters in the string
	 * @param SourceString - The string to measure
	 * @return The number of chars in the string
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String", meta=(CompactNodeTitle = "LEN", Keywords = "length"))
	static ENGINE_API int32 Len(const FString& S);
	
	/**
	 *	Returns true if the string is empty
	 *	@param InString - The string to check
	 *	@return Whether or not the string is empty
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String", meta=(Keywords = "empty"))
	static ENGINE_API bool IsEmpty(const FString& InString);

	/** 
	 * Returns a substring from the string starting at the specified position
	 * @param SourceString - The string to get the substring from
	 * @param StartIndex - The location in SourceString to use as the start of the substring
	 * @param Length The length of the requested substring
	 *
	 * @return The requested substring
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static ENGINE_API FString GetSubstring(const FString& SourceString, int32 StartIndex = 0, int32 Length = 1);

	/**
	 * Finds the starting index of a substring in the a specified string
	 * @param SearchIn The string to search within
	 * @param Substring The string to look for in the SearchIn string
	 * @param bUseCase Whether or not to be case-sensitive
	 * @param bSearchFromEnd Whether or not to start the search from the end of the string instead of the beginning
	 * @param StartPosition The position to start the search from
	 * @return The index (starting from 0 if bSearchFromEnd is false) of the first occurence of the substring
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API int32 FindSubstring(const FString& SearchIn, const FString& Substring, bool bUseCase = false, bool bSearchFromEnd = false, int32 StartPosition = -1);

	/**
	* Returns whether this string contains the specified substring.
	*
	* @param SubStr			Find to search for
	* @param SearchCase		Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	* @param SearchDir			Indicates whether the search starts at the begining or at the end ( defaults to ESearchDir::FromStart )
	* @return					Returns whether the string contains the substring
	**/
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool Contains(const FString& SearchIn, const FString& Substring, bool bUseCase = false, bool bSearchFromEnd = false);

	/** 
	 * Gets a single character from the string (as an integer)
	 * @param SourceString - The string to convert
	 * @param Index - Location of the character whose value is required
	 * @return The integer value of the character or 0 if index is out of range
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static ENGINE_API int32 GetCharacterAsNumber(const FString& SourceString, int32 Index = 0);

	/** 
	 * Gets an array of strings from a source string divided up by a separator and empty strings can optionally be culled.
	 * @param SourceString - The string to chop up
	 * @param Delimiter - The string to delimit on
	 * @param CullEmptyStrings = true - Cull (true) empty strings or add them to the array (false)
	 * @return The array of string that have been separated
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static ENGINE_API TArray<FString> ParseIntoArray(const FString& SourceString, const FString& Delimiter = FString(" "), const bool CullEmptyStrings = true);
	
	/**
	 * Concatenates an array of strings into a single string.
	 * @param SourceArray - The array of strings to concatenate.
	 * @param Separator - The string used to separate each element.
	 * @return The final, joined, separated string.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static ENGINE_API FString JoinStringArray(const TArray<FString>& SourceArray, const FString&  Separator = FString(" "));

	/**
	* Returns an array that contains one entry for each character in SourceString
	* @param	SourceString	The string to break apart into characters
	* @return	An array containing one entry for each character in SourceString
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API TArray<FString> GetCharacterArrayFromString(const FString& SourceString);

	/**
	 * Returns a string converted to Upper case
	 * @param	SourceString	The string to convert
	 * @return	The string in upper case
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String")
	static ENGINE_API FString ToUpper(const FString& SourceString);

	/**
	* Returns a string converted to Lower case
	* @param	SourceString	The string to convert
	* @return	The string in lower case
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString ToLower(const FString& SourceString);

	/*
	 * Pad the left of this string for a specified number of characters 
	 * @param	SourceString	The string to pad
	 * @param	ChCount			Amount of padding required
	 * @return	The padded string
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString LeftPad(const FString& SourceString, int32 ChCount);

	/*
	 * Pad the right of this string for a specified number of characters
	 * @param	SourceString	The string to pad
	 * @param	ChCount			Amount of padding required
	 * @return	The padded string
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString RightPad(const FString& SourceString, int32 ChCount);

	/*
	 * Checks if a string contains only numeric characters
	 * @param	SourceString	The string to check
	 * @return true if the string only contains numeric characters 
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool IsNumeric(const FString& SourceString);

	/**
	 * Test whether this string starts with given string.
	 *
	 * @param SearchCase		Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	 * @return true if this string begins with specified text, false otherwise
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool StartsWith(const FString& SourceString, const FString& InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Test whether this string ends with given string.
	 *
	 * @param SearchCase		Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	 * @return true if this string ends with specified text, false otherwise
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool EndsWith(const FString& SourceString, const FString& InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Searches this string for a given wild card
	 *
	 * @param Wildcard		*?-type wildcard
	 * @param SearchCase	Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	 * @return true if this string matches the *?-type wildcard given.
	 * @warning This is a simple, SLOW routine. Use with caution
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool MatchesWildcard(const FString& SourceString, const FString& Wildcard, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Removes whitespace characters from the front of this string.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Trim(const FString& SourceString);

	/**
	 * Removes trailing whitespace characters
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString TrimTrailing(const FString& SourceString);

	/**
	 * Takes an array of strings and removes any zero length entries.
	 *
	 * @param	InArray	The array to cull
	 *
	 * @return	The number of elements left in InArray
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API int32 CullArray(const FString& SourceString, TArray<FString>& InArray);

	/**
	* Returns a copy of this string, with the characters in reverse order
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Reverse(const FString& SourceString);

	/**
	 * Replace all occurrences of a substring in this string
	 *
	 * @param From substring to replace
	 * @param To substring to replace From with
	 * @param SearchCase	Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	 * @return a copy of this string with the replacement made
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Replace(const FString& SourceString, const FString& From, const FString& To, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Replace all occurrences of SearchText with ReplacementText in this string.
	 *
	 * @param	SearchText	the text that should be removed from this string
	 * @param	ReplacementText		the text to insert in its place
	 * @param SearchCase	Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	 *
	 * @return	the number of occurrences of SearchText that were replaced.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|String")
	static ENGINE_API int32 ReplaceInline(UPARAM(ref) FString& SourceString, const FString& SearchText, const FString& ReplacementText, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	* Splits this string at given string position case sensitive.
	*
	* @param InStr The string to search and split at
	* @param LeftS out the string to the left of InStr, not updated if return is false
	* @param RightS out the string to the right of InStr, not updated if return is false
	* @param SearchCase		Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
	* @param SearchDir			Indicates whether the search starts at the begining or at the end ( defaults to ESearchDir::FromStart )
	* @return true if string is split, otherwise false
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API bool Split(const FString& SourceString, const FString& InStr, FString& LeftS, FString& RightS, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase, ESearchDir::Type SearchDir = ESearchDir::FromStart);
	
	/** Returns the left most given number of characters */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Left(const FString& SourceString, int32 Count);
	
	/** Returns the left most characters from the string chopping the given number of characters from the end */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString LeftChop(const FString& SourceString, int32 Count);
	
	/** Returns the string to the right of the specified location, counting back from the right (end of the word). */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Right(const FString& SourceString, int32 Count);

	/** Returns the string to the right of the specified location, counting forward from the left (from the beginning of the word). */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString RightChop(const FString& SourceString, int32 Count);

	/** Returns the substring from Start position for Count characters. */
	UFUNCTION(BlueprintPure, Category = "Utilities|String")
	static ENGINE_API FString Mid(const FString& SourceString, int32 Start, int32 Count);

#if WITH_EDITOR
	/** 
	 * Returns a string describing the LCS difference between two strings, First and Second.
	 * This function is editor only because it is expensive for large strings. Examples:
	 * 
	 * ("A", "") -> "- [ 0001 |      ] A"
	 * ("", "") -> ""
	 * ("The quick brown fox", "The quick red fox") -> "~ [ 0001 | 0001 ]
     *                   The quick brown fox
     *                             ~~~~~    
	 *                   The quick red   fox"
	 * 
	 * The numbers between the [], separated by |, are the line numbers in the First and Second string at
	 * which the difference occurs. Preceding the line number information is a symbol identifying the type 
	 * of difference: A tilde (~) indicates a modification, a minus (-) indicates a removal from First, and
	 * a plus (+) indicates an addition to second.
	 * 
	 * @param First - The string to treat as basis
	 * @param Second - The string to treat as change
     * @return the LCS difference between the First and Second string
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|String")
	static ENGINE_API FString DiffString(const FString& First, const FString& Second);
#endif	// WITH_EDITOR

	/**
	 * Convert a number of seconds into minutes:seconds.milliseconds format string (including leading zeroes)
	 *
	 * @return A new string built from the seconds parameter
	 */
	UFUNCTION(BlueprintPure,  Category = "Utilities|String")
	static ENGINE_API FString TimeSecondsToString(float InSeconds);
};
