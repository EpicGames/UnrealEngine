// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetStringTableLibrary.generated.h"

class UStringTable;

UCLASS(meta=(BlueprintThreadSafe, ScriptName="StringTableLibrary"), MinimalAPI)
class UKismetStringTableLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get String Table ID"))
	static ENGINE_API FName GetTableId(const UStringTable* StringTable);

	/** Returns true if the given table ID corresponds to a registered string table. */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Is String Table Registered"))
	static ENGINE_API bool IsRegisteredTableId(const FName TableId);

	/** Returns true if the given table ID corresponds to a registered string table, and that table has. */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Is String Table Entry Registered"))
	static ENGINE_API bool IsRegisteredTableEntry(const FName TableId, const FString& Key);

	/** Returns the namespace of the given string table. */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get String Table Namespace"))
	static ENGINE_API FString GetTableNamespace(const FName TableId);

	/** Returns the source string of the given string table entry (or an empty string). */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get String Table Entry Source String"))
	static ENGINE_API FString GetTableEntrySourceString(const FName TableId, const FString& Key);

	/** Returns the specified meta-data of the given string table entry (or an empty string). */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get String Table Entry Meta-Data"))
	static ENGINE_API FString GetTableEntryMetaData(const FName TableId, const FString& Key, const FName MetaDataId);

	/** Returns an array of all registered string table IDs */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get Registered String Tables"))
	static ENGINE_API TArray<FName> GetRegisteredStringTables();

	/** Returns an array of all keys within the given string table */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get Keys from String Table"))
	static ENGINE_API TArray<FString> GetKeysFromStringTable(const FName TableId);

	/** Returns an array of all meta-data IDs within the given string table entry */
	UFUNCTION(BlueprintPure, Category="Utilities|String Table", meta=(DisplayName="Get Meta-Data IDs from String Table Entry"))
	static ENGINE_API TArray<FName> GetMetaDataIdsFromStringTableEntry(const FName TableId, const FString& Key);

	/** Export the key, string, and meta-data information in the given string table to a CSV string (does not export the namespace) */
	UFUNCTION(BlueprintCallable, Category="Utilities|String Table", meta=(DisplayName="Export String Table to CSV String"))
	static ENGINE_API bool ExportTableToCSVString(const FName TableId, FString& OutCSVString);

	/** Export the key, string, and meta-data information in the given string table to a CSV file (does not export the namespace) */
	UFUNCTION(BlueprintCallable, Category="Utilities|String Table", meta=(DisplayName="Export String Table to CSV File"))
	static ENGINE_API bool ExportTableToCSVFile(const FName TableId, const FString& Filename);

	/** Import key, string, and meta-data information from a CSV string to the given string table (does not import the namespace) */
	UFUNCTION(BlueprintCallable, Category="Utilities|String Table", meta=(DisplayName="Import String Table from CSV String"))
	static ENGINE_API bool ImportTableFromCSVString(const FName TableId, const FString& CSVString);

	/** Import key, string, and meta-data information from a CSV file to the given string table (does not import the namespace) */
	UFUNCTION(BlueprintCallable, Category="Utilities|String Table", meta=(DisplayName="Import String Table from CSV File"))
	static ENGINE_API bool ImportTableFromCSVFile(const FName TableId, const FString& Filename);
};
