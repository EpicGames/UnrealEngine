// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"
#include "UObject/TextProperty.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/TopLevelAssetPath.h"
#include "Engine/LatentActionManager.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/CollisionProfile.h"
#include "AssetRegistry/ARFilter.h"
#include "KismetSystemLibrary.generated.h"

class AActor;
class ACameraActor;
class APlayerController;
class UPrimitiveComponent;
class USceneComponent;
class UTexture2D;
struct FHitResult;

UENUM(BlueprintType)
namespace EDrawDebugTrace
{
	enum Type : int
	{
		None, 
		ForOneFrame, 
		ForDuration, 
		Persistent
	};
}

/** Enum used to indicate the scene depth priority -- whether to draw with depth testing vs the world, or to draw in the foreground */
UENUM(BlueprintType)
enum class EDrawDebugSceneDepthPriorityGroup : uint8
{
	// Draw in world with depth testing, allowing occlusion by other surfaces
	World,
	// Draw in foreground, so drawn shapes will always be visible
	Foreground
};

/** Enum used to indicate desired behavior for MoveComponentTo latent function. */
UENUM()
namespace EMoveComponentAction
{
	enum Type : int
	{
		/** Move to target over specified time. */
		Move, 
		/** If currently moving, stop. */
		Stop,
		/** If currently moving, return to where you started, over the time elapsed so far. */
		Return
	};
}

UENUM()
namespace EQuitPreference
{
	enum Type : int
	{
		/** Exit the game completely. */
		Quit,
		/** Move the application to the background. */
		Background,
	};
}

/** States a property value queried by IsEditorPropertyOverridden can be in */
UENUM()
enum class EEditorPropertyValueState : uint8
{
	/** This property is in a default state */
	Default,
	/** This property is in an overridden state */
	Overridden,
	/** The property was not found on the object or its archetype; its state is unknown */
	NotFound,
	/** The property could not be accessed to query its state; its state is unknown */
	AccessDenied,
};

USTRUCT(BlueprintInternalUseOnly)
struct FGenericStruct
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 Data = 0;
};

UCLASS(meta=(ScriptName="SystemLibrary"), MinimalAPI)
class UKismetSystemLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	// --- Globally useful functions ------------------------------
	/** Prints a stack trace to the log, so you can see how a blueprint got to this node */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Development|Editor", meta=(Keywords = "ScriptTrace"))
	static ENGINE_API void StackTrace();
	static ENGINE_API void StackTraceImpl(const FFrame& StackFrame);
	DECLARE_FUNCTION(execStackTrace)
	{
		P_FINISH;
		StackTraceImpl(Stack);
	}

	/**
	 * Raises an error in the form of a Blueprint exception.
	 * Logs the error to the Output and Message Log with information pertaining to where the error was raised.
	 *
	 * @param ErrorMessage		The error message to display
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Debugging", meta=(Keywords = "raise error", DevelopmentOnly))
	static ENGINE_API void RaiseScriptError(const FString& ErrorMessage = FString("An error occurred"));

	// Return true if the object is usable : non-null and not pending kill
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static bool IsValid(const UObject* Object);

	// Return true if the class is usable : non-null and not pending kill
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static bool IsValidClass(UClass* Class);

	// Returns the actual object name.
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (BlueprintThreadSafe))
	static ENGINE_API FString GetObjectName(const UObject* Object);

	// Returns the full path to the specified object as a string
	UFUNCTION(BlueprintPure, Category="Utilities", meta = (DisplayName = "Get Object Path String", BlueprintThreadSafe))
	static ENGINE_API FString GetPathName(const UObject* Object);

	// Returns the full path to the specified object as a Soft Object Path
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API FSoftObjectPath GetSoftObjectPath(const UObject* Object);

#if WITH_EDITOR
	// Gets the UObject associated with a SoftPath, loading if necessary.
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static UObject* GetObjectFromSoftPath(const FSoftObjectPath& Path);

	// Gets the UClass associated with a SoftPath, loading if necessary.
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static UClass* GetClassFromSoftPath(const FSoftClassPath& Path);
#endif

	// Returns the full file system path to a UObject
	// If given a non-asset UObject, it will return an empty string
	UFUNCTION(BlueprintPure, Category = "Utilities|Paths")
	static ENGINE_API FString GetSystemPath(const UObject* Object);

	// Returns the display name (or actor label), for displaying as a debugging aid.
	// Note: In editor builds, this is the actor label.  In non-editor builds, this is the actual object name.  This function should not be used to uniquely identify actors!
	// It is not localized and should not be used for display to an end user of a game.
	UFUNCTION(BlueprintPure, Category="Utilities")
	static ENGINE_API FString GetDisplayName(const UObject* Object);

	// Returns the display name of a class
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (DisplayName = "Get Class Display Name"))
	static ENGINE_API FString GetClassDisplayName(const UClass* Class);

	// Returns the full path to the specified class as a Soft Class Path (that can be used as a Soft Object Path)
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API FSoftClassPath GetSoftClassPath(const UClass* Class);

	// Returns the full path to the specified class as a Top Level Asset Path used by asset utilities
	UFUNCTION(BlueprintPure, Category = "Utilities", DisplayName="Get Class Path Name (Top Level Asset Path)", meta=(ScriptMethod="GetClassPathName"))
	static ENGINE_API FTopLevelAssetPath GetClassTopLevelAssetPath(const UClass* Class);

	// Returns the full path to the specified struct as a Top Level Asset Path used by asset utilities
	UFUNCTION(BlueprintPure, Category = "Utilities", DisplayName = "Get Struct Path Name (Top Level Asset Path)", meta=(ScriptMethod="GetStructPathName"))
	static FTopLevelAssetPath GetStructTopLevelAssetPath(const UScriptStruct* Struct);

	// Returns the full path to the specified enum as a Top Level Asset Path used by asset utilities
	UFUNCTION(BlueprintPure, Category = "Utilities", DisplayName = "Get Enum Path Name (Top Level Asset Path)", meta=(ScriptMethod="GetEnumPathName"))
	static FTopLevelAssetPath GetEnumTopLevelAssetPath(const UEnum* Enum);

	// Returns the outer object of an object.
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API UObject* GetOuterObject(const UObject* Object);

#if WITH_EDITOR
	/** Returns true if the object is cooked */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool IsObjectCooked(const UObject* Object);

	/** Returns true if the object has its editor-only data */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool ObjectHasEditorOnlyData(const UObject* Object);

	/**
	 * Duplicate an object to a new instance
	 *
	 * @param SourceObject	The object being duplicated
	 * @param Outer			The outer to use for the new instance
	 * @param Name			The optional name of the new instance
	 *
	 * @return The duplicated object or null if it failed for some reason
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities")
	static ENGINE_API UObject* DuplicateObject(const UObject* SourceObject, UObject* Outer, const FName Name = NAME_None);
#endif	// WITH_EDITOR

	// Engine build number, for displaying to end users.
	UFUNCTION(BlueprintPure, Category="Development", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetEngineVersion();

	// Build version, for displaying to end users in diagnostics.
	UFUNCTION(BlueprintPure, Category="Development", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetBuildVersion();

	// Build configuration, for displaying to end users in diagnostics.
	UFUNCTION(BlueprintPure, Category = "Development", meta = (BlueprintThreadSafe))
	static ENGINE_API FString GetBuildConfiguration();

	/** Get the name of the current game  */
	UFUNCTION(BlueprintPure, Category="Game", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetGameName();

	/** Get the directory of the current project */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetProjectDirectory();

	/** Get the content directory of the current project */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetProjectContentDirectory();

	/** Get the saved directory of the current project */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths", meta=(BlueprintThreadSafe))
	static ENGINE_API FString GetProjectSavedDirectory();

	/* Converts passed in filename to use a relative path */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths")
	static ENGINE_API FString ConvertToRelativePath(const FString& Filename);

	/* Converts passed in filename to use a absolute path */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths")
	static ENGINE_API FString ConvertToAbsolutePath(const FString& Filename);

	/* Convert all / and \ to TEXT("/") */
	UFUNCTION(BlueprintPure, Category="Utilities|Paths", meta=(BlueprintThreadSafe))
	static ENGINE_API FString NormalizeFilename(const FString& InFilename);

	/**
	 * Retrieves the game's platform-specific bundle identifier or package name of the game
	 *
	 * @return The game's bundle identifier or package name.
	 */
	UFUNCTION(BlueprintPure, Category="Game", meta=(Keywords = "bundle id package name"))
	static ENGINE_API FString GetGameBundleId();

	/** Get the current user name from the OS */
	UFUNCTION(BlueprintPure, Category="Utilities|Platform")
	static ENGINE_API FString GetPlatformUserName();

	/** Get the current user dir from the OS */
	UFUNCTION(BlueprintPure, Category = "Utilities|Platform")
	static ENGINE_API FString GetPlatformUserDir();

	/** Checks if the given object implements a specific interface, works for both native and blueprint interfacse */
	UFUNCTION(BlueprintPure, Category="Utilities", meta=(DisplayName = "Does Object Implement Interface"))
	static ENGINE_API bool DoesImplementInterface(const UObject* TestObject, TSubclassOf<UInterface> Interface);

	/** Checks if the given class implements a specific interface, works for both native and blueprint interfacse */
	UFUNCTION(BlueprintPure, Category="Utilities")
	static ENGINE_API bool DoesClassImplementInterface(const UClass* TestClass, TSubclassOf<UInterface> Interface);

	/** 
	 * Get the current game time, in seconds. This stops when the game is paused and is affected by slomo. 
	 * 
	 * @param WorldContextObject	World context
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Time", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API double GetGameTimeInSeconds(const UObject* WorldContextObject);

	/** Returns the value of GFrameCounter, a running count of the number of frames that have occurred. */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API int64 GetFrameCount();

#if WITH_EDITOR
	/** 
	 * Returns the current platform time in seconds. Not coupled to any gameplay or other containerization logic - this
	 * function is useful for timing execution time or timestamping data. Marked as callable rather than pure because 
	 * implicit evaluation may be confusing, both for blueprint authors and blueprint readers. For implicit execution 
	 * simply wrap it in a blueprint pure function.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Time")
	static ENGINE_API double GetPlatformTime_Seconds();
#endif // WITH_EDITOR

	/** Returns whether the world this object is in is the host or not */
	UFUNCTION(BlueprintPure, Category="Networking", meta=(WorldContext="WorldContextObject") )
	static ENGINE_API bool IsServer(const UObject* WorldContextObject);

	/** Returns whether this is running on a dedicated server */
	UFUNCTION(BlueprintPure, Category="Networking", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API bool IsDedicatedServer(const UObject* WorldContextObject);

	/** Returns whether this game instance is stand alone (no networking). */
	UFUNCTION(BlueprintPure, Category="Networking", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API bool IsStandalone(const UObject* WorldContextObject);
	
	/** Returns whether we currently have more than one local player. */
	UE_DEPRECATED(5.0, "IsSplitScreen was only ever checking if there are more than one local player. Use HasMultipleLocalPlayers instead.")
	UFUNCTION(BlueprintPure, Category = "Viewport", meta = (WorldContext = "WorldContextObject", DeprecatedFunction, DeprecationMessage = "Use HasMultipleLocalPlayers instead"))
	static ENGINE_API bool IsSplitScreen(const UObject* WorldContextObject);

	/** Returns whether there are currently multiple local players in the given world */
	UFUNCTION(BlueprintPure, Category = "Viewport", meta = (WorldContext = "WorldContextObject"))
	static ENGINE_API bool HasMultipleLocalPlayers(const UObject* WorldContextObject);

	/** Returns whether this is a build that is packaged for distribution */
	UFUNCTION(BlueprintPure, Category="Development", meta=(BlueprintThreadSafe))
	static ENGINE_API bool IsPackagedForDistribution();

	/** Returns the platform specific unique device id */
	UFUNCTION(BlueprintPure, Category="Utilities|Platform", meta = (DeprecatedFunction, DeprecationMessage = "Use GetDeviceId instead"))
	static ENGINE_API FString GetUniqueDeviceId();

	/** Returns the platform specific unique device id */
	UFUNCTION(BlueprintPure, Category="Utilities|Platform")
	static ENGINE_API FString GetDeviceId();

	/** Casts from an object to a class, this will only work if the object is already a class */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Cast To Class", DeterminesOutputType = "Class"), Category="Utilities")
	static ENGINE_API UClass* Conv_ObjectToClass(UObject* Object, TSubclassOf<UObject> Class);

	/** Converts an interface instance into an object */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Object (Interface)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities")
	static ENGINE_API UObject* Conv_InterfaceToObject(const FScriptInterface& Interface); 

	/** 
	 * Checks if the interface instance has a valid object for blueprint interface functions.
	 * This will return true for both natively implemented and blueprint implemented interfaces.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities")
	static bool IsValidInterface(const FScriptInterface& Interface); 

	/** Builds a Soft Object Path struct from a string that contains a full /folder/packagename.object path */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (Keywords = "construct build", NativeMakeFunc, BlueprintThreadSafe, BlueprintAutocast))
	static ENGINE_API FSoftObjectPath MakeSoftObjectPath(const FString& PathString);

	/** Gets the path string out of a Soft Object Path */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (NativeBreakFunc, BlueprintThreadSafe, BlueprintAutocast))
	static ENGINE_API void BreakSoftObjectPath(FSoftObjectPath InSoftObjectPath, FString& PathString);

	/** Converts a Soft Object Path into a base Soft Object Reference, this is not guaranteed to be resolvable */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Soft Object Reference", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API TSoftObjectPtr<UObject> Conv_SoftObjPathToSoftObjRef(const FSoftObjectPath& SoftObjectPath);

	/** Converts a Soft Object Reference into a Soft Object Path */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Soft Object Path", CompactNodeTitle = "->", BlueprintThreadSafe, BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FSoftObjectPath Conv_SoftObjRefToSoftObjPath(TSoftObjectPtr<UObject> SoftObjectReference);

	/** Builds a TopLevelAssetPath struct from single Path string or from PackageName and AssetName string. */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (Keywords = "construct build", NativeMakeFunc, BlueprintThreadSafe, BlueprintAutocast))
	static ENGINE_API FTopLevelAssetPath MakeTopLevelAssetPath(UPARAM(DisplayName="FullPathOrPackageName") const FString& PackageName, const FString& AssetName);

	/** Gets the path string out of a TopLevelAssetPath */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (NativeBreakFunc, BlueprintThreadSafe, BlueprintAutocast))
	static ENGINE_API void BreakTopLevelAssetPath(const FTopLevelAssetPath& TopLevelAssetPath, FString& PathString);

	/**
	 * Builds a Soft Class Path struct from a string that contains a full /folder/packagename.class path.
	 * For blueprint classes, this needs to point to the actual class (often with _C) and not the blueprint editor asset
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (Keywords = "construct build", NativeMakeFunc, BlueprintThreadSafe))
	static ENGINE_API FSoftClassPath MakeSoftClassPath(const FString& PathString);

	/** Gets the path string out of a Soft Class Path */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (NativeBreakFunc, BlueprintThreadSafe))
	static ENGINE_API void BreakSoftClassPath(FSoftClassPath InSoftClassPath, FString& PathString);

	/** Converts a Soft Class Path into a base Soft Class Reference, this is not guaranteed to be resolvable */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Soft Class Reference", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API TSoftClassPtr<UObject> Conv_SoftClassPathToSoftClassRef(const FSoftClassPath& SoftClassPath);

	/** Converts a Soft Class Reference into a Soft Class Path (which can be used like a Soft Object Path) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Soft Class Path", CompactNodeTitle = "->", BlueprintThreadSafe, BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FSoftClassPath Conv_SoftObjRefToSoftClassPath(TSoftClassPtr<UObject> SoftClassReference);

	/** Returns true if the Soft Object Reference is not null */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (BlueprintThreadSafe))
	static ENGINE_API bool IsValidSoftObjectReference(const TSoftObjectPtr<UObject>& SoftObjectReference);

	/** Converts a Soft Object Reference to a path string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (SoftObjectReference)", CompactNodeTitle = "->", BlueprintThreadSafe, BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FString Conv_SoftObjectReferenceToString(const TSoftObjectPtr<UObject>& SoftObjectReference);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (SoftObjectReference)", CompactNodeTitle = "==", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API bool EqualEqual_SoftObjectReference(const TSoftObjectPtr<UObject>& A, const TSoftObjectPtr<UObject>& B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (SoftObjectReference)", CompactNodeTitle = "!=", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API bool NotEqual_SoftObjectReference(const TSoftObjectPtr<UObject>& A, const TSoftObjectPtr<UObject>& B);

	/** Resolves or loads a Soft Object Reference immediately, this will cause hitches and Async Load Asset should be used if possible */
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (DeterminesOutputType = "Asset"))
	static ENGINE_API UObject* LoadAsset_Blocking(TSoftObjectPtr<UObject> Asset);

	/** Returns true if the Soft Class Reference is not null */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (BlueprintThreadSafe))
	static ENGINE_API bool IsValidSoftClassReference(const TSoftClassPtr<UObject>& SoftClassReference);

	/** Converts a Soft Class Reference to a Top Level Asset Path used by asset utilities */
	UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe, BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FTopLevelAssetPath GetSoftClassTopLevelAssetPath(TSoftClassPtr<UObject> SoftClassReference);

	/** Converts a Soft Class Reference to a path string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (SoftClassReference)", CompactNodeTitle = "->", BlueprintThreadSafe, BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FString Conv_SoftClassReferenceToString(const TSoftClassPtr<UObject>& SoftClassReference);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (SoftClassReference)", CompactNodeTitle = "==", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API bool EqualEqual_SoftClassReference(const TSoftClassPtr<UObject>& A, const TSoftClassPtr<UObject>& B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (SoftClassReference)", CompactNodeTitle = "!=", BlueprintThreadSafe), Category = "Utilities")
	static ENGINE_API bool NotEqual_SoftClassReference(const TSoftClassPtr<UObject>& A, const TSoftClassPtr<UObject>& B);

	/** Resolves or loads a Soft Class Reference immediately, this will cause hitches and Async Load Class Asset should be used if possible */
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (DeterminesOutputType = "AssetClass"))
	static ENGINE_API UClass* LoadClassAsset_Blocking(TSoftClassPtr<UObject> AssetClass);

	/** Returns true if Object is of type SoftClass - either an instance of the class or child class, or implements the interface. Alternative to Cast - slower but without adding a hard reference. */
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (ExpandEnumAsExecs = ReturnValue, DisplayName = "IsA ( soft )"))
	static bool IsObjectOfSoftClass(const UObject* Object, TSoftClassPtr<UObject> SoftClass);

	// Internal functions used by K2Node_LoadAsset and K2Node_ConvertAsset

	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API UObject* Conv_SoftObjectReferenceToObject(const TSoftObjectPtr<UObject>& SoftObject);

	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API TSubclassOf<UObject> Conv_SoftClassReferenceToClass(const TSoftClassPtr<UObject>& SoftClass);

	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API TSoftObjectPtr<UObject> Conv_ObjectToSoftObjectReference(UObject* Object);

	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API TSoftClassPtr<UObject> Conv_ClassToSoftClassReference(const TSubclassOf<UObject>& Class);

	UFUNCTION(BlueprintPure, meta = (BlueprintAutocast), Category = "Utilities")
	static ENGINE_API FSoftComponentReference Conv_ComponentReferenceToSoftComponentReference(const FComponentReference& ComponentReference);

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAssetLoaded, class UObject*, Loaded);

	UFUNCTION(BlueprintCallable, meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject", BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API void LoadAsset(const UObject* WorldContextObject, TSoftObjectPtr<UObject> Asset, FOnAssetLoaded OnLoaded, FLatentActionInfo LatentInfo);

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAssetClassLoaded, TSubclassOf<UObject>, Loaded);

	UFUNCTION(BlueprintCallable, meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject", BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API void LoadAssetClass(const UObject* WorldContextObject, TSoftClassPtr<UObject> AssetClass, FOnAssetClassLoaded OnLoaded, FLatentActionInfo LatentInfo);

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAssetsLoaded, const TArray<class UObject*>&, Loaded);

	UFUNCTION(BlueprintCallable, meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject", BlueprintInternalUseOnly = "true"), Category = "Utilities")
	static ENGINE_API void LoadAssets(const UObject* WorldContextObject, TArray<TSoftObjectPtr<UObject>> Assets, FOnAssetsLoaded OnLoaded, FLatentActionInfo LatentInfo);

	/**
	 * Creates a literal integer
	 * @param	Value	value to set the integer to
	 * @return	The literal integer
	 */
	UFUNCTION(BlueprintPure, Category="Math|Integer", meta=(BlueprintThreadSafe))
	static int32 MakeLiteralInt(int32 Value);

	/**
	 * Creates a literal 64-bit integer
	 * @param	Value	value to set the 64-bit integer to
	 * @return	The literal 64-bit integer
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Integer", meta = (BlueprintThreadSafe))
	static int64 MakeLiteralInt64(int64 Value);

	/**
	 * Creates a literal float
	 * @param	Value	value to set the float to
	 * @return	The literal float
	 */
	UE_DEPRECATED(5.1, "This method has been deprecated and will be removed.")
	static float MakeLiteralFloat(float Value);

	/**
	 * Creates a literal float (double-precision)
	 * @param	Value	value to set the float (double-precision) to
	 * @return	The literal float (double-precision)
	 */
	UFUNCTION(BlueprintPure, Category = "Math|Float", meta = (BlueprintThreadSafe, DisplayName = "Make Literal Float"))
	static double MakeLiteralDouble(double Value);

	/**
	 * Creates a literal bool
	 * @param	Value	value to set the bool to
	 * @return	The literal bool
	 */
	UFUNCTION(BlueprintPure, Category="Math|Boolean", meta=(BlueprintThreadSafe))
	static bool MakeLiteralBool(bool Value);

	/**
	 * Creates a literal name
	 * @param	Value	value to set the name to
	 * @return	The literal name
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Name", meta=(BlueprintThreadSafe))
	static FName MakeLiteralName(FName Value);

	/**
	 * Creates a literal byte
	 * @param	Value	value to set the byte to
	 * @return	The literal byte
	 */
	UFUNCTION(BlueprintPure, Category="Math|Byte", meta=(BlueprintThreadSafe))
	static uint8 MakeLiteralByte(uint8 Value);

	/**
	 * Creates a literal string
	 * @param	Value	value to set the string to
	 * @return	The literal string
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|String", meta=(BlueprintThreadSafe))
	static FString MakeLiteralString(FString Value);

	/**
	 * Creates a literal FText
	 * @param	Value	value to set the FText to
	 * @return	The literal FText
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(BlueprintThreadSafe))
	static FText MakeLiteralText(FText Value);

	/**
	 * Prints a string to the log
	 * If Print To Log is true, it will be visible in the Output Log window.  Otherwise it will be logged only as 'Verbose', so it generally won't show up.
	 *
	 * @param	InString		The string to log out
	 * @param	bPrintToLog		Whether or not to print the output to the log
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|String", meta=(BlueprintThreadSafe, Keywords = "log print", DevelopmentOnly))
	static ENGINE_API void LogString(const FString& InString = FString("Hello"), bool bPrintToLog = true);
	
	/**
	 * Prints a string to the log, and optionally, to the screen
	 * If Print To Log is true, it will be visible in the Output Log window.  Otherwise it will be logged only as 'Verbose', so it generally won't show up.
	 *
	 * @param	InString		The string to log out
	 * @param	bPrintToScreen	Whether or not to print the output to the screen
	 * @param	bPrintToLog		Whether or not to print the output to the log
	 * @param	bPrintToConsole	Whether or not to print the output to the console
	 * @param	TextColor		The color of the text to display
	 * @param	Duration		The display duration (if Print to Screen is True). Using negative number will result in loading the duration time from the config.
	 * @param	Key				If a non-empty key is provided, the message will replace any existing on-screen messages with the same key.
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext, Keywords = "log print", AdvancedDisplay = "2", DevelopmentOnly), Category="Development")
	static ENGINE_API void PrintString(const UObject* WorldContextObject, const FString& InString = FString("Hello"), bool bPrintToScreen = true, bool bPrintToLog = true, FLinearColor TextColor = FLinearColor(0.0f, 0.66f, 1.0f), float Duration = 2.f, const FName Key = NAME_None);

	/**
	 * Prints text to the log, and optionally, to the screen
	 * If Print To Log is true, it will be visible in the Output Log window.  Otherwise it will be logged only as 'Verbose', so it generally won't show up.
	 *
	 * @param	InText			The text to log out
	 * @param	bPrintToScreen	Whether or not to print the output to the screen
	 * @param	bPrintToLog		Whether or not to print the output to the log
	 * @param	bPrintToConsole	Whether or not to print the output to the console
	 * @param	TextColor		The color of the text to display
	 * @param	Duration		The display duration (if Print to Screen is True). Using negative number will result in loading the duration time from the config.
	 * @param	Key				If a non-empty key is provided, the message will replace any existing on-screen messages with the same key.
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext, Keywords = "log", AdvancedDisplay = "2", DevelopmentOnly), Category="Development")
	static ENGINE_API void PrintText(const UObject* WorldContextObject, const FText InText = INVTEXT("Hello"), bool bPrintToScreen = true, bool bPrintToLog = true, FLinearColor TextColor = FLinearColor(0.0f, 0.66f, 1.0f), float Duration = 2.f, const FName Key = NAME_None);

	/**
	 * Prints a warning string to the log and the screen. Meant to be used as a way to inform the user that they misused the node.
	 *
	 * WARNING!! Don't change the signature of this function without fixing up all nodes using it in the compiler
	 *
	 * @param	InString		The string to log out
	 */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "TRUE"))
	static ENGINE_API void PrintWarning(const FString& InString);

	/** Sets the game window title */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void SetWindowTitle(const FText& Title);

	/**
	 * Executes a console command, optionally on a specific controller
	 * 
	 * @param	Command			Command to send to the console
	 * @param	SpecificPlayer	If specified, the console command will be routed through the specified player
	 */
	UFUNCTION(BlueprintCallable, Category="Development",meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext))
	static ENGINE_API void ExecuteConsoleCommand(const UObject* WorldContextObject, const FString& Command, class APlayerController* SpecificPlayer = NULL );

	/**
	 * Attempts to retrieve the value of the specified string console variable, if it exists.
	 * 
	 * @param	VariableName	Name of the console variable to find.
	 * @return	The value if found, empty string otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Development")
	static ENGINE_API FString GetConsoleVariableStringValue(const FString& VariableName);

	/**
	 * Attempts to retrieve the value of the specified float console variable, if it exists.
	 * 
	 * @param	VariableName	Name of the console variable to find.
	 * @return	The value if found, 0 otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Development")
	static ENGINE_API float GetConsoleVariableFloatValue(const FString& VariableName);

	/**
	 * Attempts to retrieve the value of the specified integer console variable, if it exists.
	 * 
	 * @param	VariableName	Name of the console variable to find.
	 * @return	The value if found, 0 otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Development")
	static ENGINE_API int32 GetConsoleVariableIntValue(const FString& VariableName);

	/**
	 * Evaluates, if it exists, whether the specified integer console variable has a non-zero value (true) or not (false).
	 *
	 * @param	VariableName	Name of the console variable to find.
	 * @return	True if found and has a non-zero value, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Development")
	static ENGINE_API bool GetConsoleVariableBoolValue(const FString& VariableName);

	/** 
	 *	Exit the current game 
	 * @param	SpecificPlayer	The specific player to quit the game. If not specified, player 0 will quit.
	 * @param	QuitPreference	Form of quitting.
	 * @param	bIgnorePlatformRestrictions	Ignores and best-practices based on platform (e.g on some consoles, games should never quit). Non-shipping only
	 */
	UFUNCTION(BlueprintCallable, Category="Game",meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext))
	static ENGINE_API void QuitGame(const UObject* WorldContextObject, class APlayerController* SpecificPlayer, TEnumAsByte<EQuitPreference::Type> QuitPreference, bool bIgnorePlatformRestrictions);
	
#if WITH_EDITOR
	/**
	 *	Exit the editor
	 */
	UFUNCTION(BlueprintCallable, Category="Development")
	static ENGINE_API void QuitEditor();
#endif	// WITH_EDITOR

	//=============================================================================
	// Latent Actions

	/** 
	 * Perform a latent action with a delay (specified in seconds).  Calling again while it is counting down will be ignored.
	 * 
	 * @param WorldContext	World context.
	 * @param Duration 		length of delay (in seconds).
	 * @param LatentInfo 	The latent action.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|FlowControl", meta=(Latent, WorldContext="WorldContextObject", LatentInfo="LatentInfo", Duration="0.2", Keywords="sleep"))
	static ENGINE_API void	Delay(const UObject* WorldContextObject, float Duration, struct FLatentActionInfo LatentInfo );

	/**
	 * Perform a latent action with a delay of one tick.  Calling again while it is pending will be ignored.
	 * Note that this delay is until the object's next tick, which may occur subsequently during the current engine frame.
	 * @see DelayUntilNextFrame for a guaranteed wait until next engine frame's tick.
	 * Enabling the CVar 'LatentActions.GuaranteeNextTickDelay' will guarantee this delay waits until next frame's tick.
	 *
	 * @param WorldContext	World context.
	 * @param LatentInfo 	The latent action.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|FlowControl", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", Keywords = "sleep"))
	static ENGINE_API void	DelayUntilNextTick(const UObject* WorldContextObject, FLatentActionInfo LatentInfo);

	/**
	 * Perform a latent action with a delay until the next engine frame.  Calling again while it is pending will be ignored.
	 * @param WorldContext	World context.
	 * @param LatentInfo 	The latent action.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|FlowControl", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", Keywords = "sleep"))
	static ENGINE_API void	DelayUntilNextFrame(const UObject* WorldContextObject, FLatentActionInfo LatentInfo);


	/** 
	 * Perform a latent action with a retriggerable delay (specified in seconds).  Calling again while it is counting down will reset the countdown to Duration.
	 * 
	 * @param WorldContext	World context.
	 * @param Duration 		length of delay (in seconds).
	 * @param LatentInfo 	The latent action.
	 */
	UFUNCTION(BlueprintCallable, meta=(Latent, LatentInfo="LatentInfo", WorldContext="WorldContextObject", Duration="0.2", Keywords="sleep"), Category="Utilities|FlowControl")
	static ENGINE_API void RetriggerableDelay(const UObject* WorldContextObject, float Duration, FLatentActionInfo LatentInfo);

	/*
	 * Interpolate a component to the specified relative location and rotation over the course of OverTime seconds. 
	 * @param Component						Component to interpolate
	 * @param TargetRelativeLocation		Relative target location
	 * @param TargetRelativeRotation		Relative target rotation
	 * @param bEaseOut						if true we will ease out (ie end slowly) during interpolation
	 * @param bEaseIn						if true we will ease in (ie start slowly) during interpolation
	 * @param OverTime						duration of interpolation
	 * @param bForceShortestRotationPath	if true we will always use the shortest path for rotation
	 * @param MoveAction					required movement behavior @see EMoveComponentAction
	 * @param LatentInfo					The latent action
	 */
	UFUNCTION(BlueprintCallable, meta=(Latent, LatentInfo="LatentInfo", WorldContext="WorldContextObject", ExpandEnumAsExecs="MoveAction", OverTime="0.2"), Category="Components")
	static ENGINE_API void MoveComponentTo(USceneComponent* Component, FVector TargetRelativeLocation, FRotator TargetRelativeRotation, bool bEaseOut, bool bEaseIn, float OverTime, bool bForceShortestRotationPath, TEnumAsByte<EMoveComponentAction::Type> MoveAction, FLatentActionInfo LatentInfo);

	// --- Timer functions with delegate input ----------

	/**
	 * Set a timer to execute delegate. Setting an existing timer will reset that timer with updated parameters.
	 * @param Event						Event. Can be a K2 function or a Custom Event.
	 * @param Time						How long to wait before executing the delegate, in seconds. Setting a timer to <= 0 seconds will clear it if it is set.
	 * @param bLooping					True to keep executing the delegate every Time seconds, false to execute delegate only once.
	 * @param bMaxOncePerFrame			For looping timers, whether to execute only once when the timer would otherwise expires multiple times in the current frame.
	 * @param InitialStartDelay			Initial delay passed to the timer manager, in seconds.
	 * @param InitialStartDelayVariance	Use this to add some variance to when the timer starts in lieu of doing a random range on the InitialStartDelay input, in seconds. 
	 * @return							The timer handle to pass to other timer functions to manipulate this timer.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Set Timer by Event", ScriptName = "SetTimerDelegate", AdvancedDisplay="InitialStartDelay, InitialStartDelayVariance"), Category="Utilities|Time")
	static ENGINE_API FTimerHandle K2_SetTimerDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate, float Time, bool bLooping, bool bMaxOncePerFrame = false, float InitialStartDelay = 0.f, float InitialStartDelayVariance = 0.f);

	/**
	 * Set a timer to execute a delegate next tick.
	 * @param Event						Event. Can be a K2 function or a Custom Event.
	 * @return							The timer handle to pass to other timer functions to manipulate this timer.
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Timer for Next Tick by Event", ScriptName = "SetTimerForNextTickDelegate"), Category = "Utilities|Time")
	static ENGINE_API FTimerHandle K2_SetTimerForNextTickDelegate(UPARAM(DisplayName = "Event") FTimerDynamicDelegate Delegate);

	/**
	 * Clears a set timer.
	 * @param Event  Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage = "Use Clear Timer by Handle", DisplayName = "Clear Timer by Event", ScriptName = "ClearTimerDelegate"), Category="Utilities|Time")
	static ENGINE_API void K2_ClearTimerDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Pauses a set timer at its current elapsed time.
	 * @param Event  Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage = "Use Pause Timer by Handle", DisplayName = "Pause Timer by Event", ScriptName = "PauseTimerDelegate"), Category="Utilities|Time")
	static ENGINE_API void K2_PauseTimerDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Resumes a paused timer from its current elapsed time.
	 * @param Event  Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DeprecatedFunction, DeprecationMessage = "Use Unpause Timer by Handle", DisplayName = "Unpause Timer by Event", ScriptName = "UnPauseTimerDelegate"), Category="Utilities|Time")
	static ENGINE_API void K2_UnPauseTimerDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Returns true if a timer exists and is active for the given delegate, false otherwise.
	 * @param Event  Can be a K2 function or a Custom Event.
	 * @return				True if the timer exists and is active.
	 */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "Use Is Timer Active by Handle", DisplayName = "Is Timer Active by Event", ScriptName = "IsTimerActiveDelegate"), Category="Utilities|Time")
	static ENGINE_API bool K2_IsTimerActiveDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Returns true if a timer exists and is paused for the given delegate, false otherwise.
	 * @param Event  Can be a K2 function or a Custom Event.
	 * @return				True if the timer exists and is paused.
	 */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "Use Is Timer Paused by Handle", DisplayName = "Is Timer Paused by Event", ScriptName = "IsTimerPausedDelegate"), Category = "Utilities|Time")
	static ENGINE_API bool K2_IsTimerPausedDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Returns true is a timer for the given delegate exists, false otherwise.
	 * @param Event  Can be a K2 function or a Custom Event.
	 * @return				True if the timer exists.
	 */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "Use Does Timer Exist by Handle", DisplayName = "Does Timer Exist by Event", ScriptName = "TimerExistsDelegate"), Category = "Utilities|Time")
	static ENGINE_API bool K2_TimerExistsDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);
	
	/**
	 * Returns elapsed time for the given delegate (time since current countdown iteration began).
	 * @param Event  Can be a K2 function or a Custom Event.
	 * @return				How long has elapsed since the current iteration of the timer began.
	 */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "Use Get Timer Elapsed Time by Handle", DisplayName = "Get Timer Elapsed Time by Event", ScriptName = "GetTimerElapsedTimeDelegate"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerElapsedTimeDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	/**
	 * Returns time until the timer will next execute its delegate.
	 * @param Event  Can be a K2 function or a Custom Event.
	 * @return				How long is remaining in the current iteration of the timer.
	 */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "Use Get Timer Remaining Time by Handle", DisplayName = "Get Timer Remaining Time by Event", ScriptName = "GetTimerRemainingTimeDelegate"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerRemainingTimeDelegate(UPARAM(DisplayName="Event") FTimerDynamicDelegate Delegate);

	// --- Timer functions with handle input ----------

	/**
	 * Returns whether the timer handle is valid. This does not indicate that there is an active timer that this handle references, but rather that it once referenced a valid timer.
	 * @param Handle		The handle of the timer to check validity of.
	 * @return				Whether the timer handle is valid.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Valid Timer Handle", ScriptName = "IsValidTimerHandle"), Category="Utilities|Time")
	static ENGINE_API bool K2_IsValidTimerHandle(FTimerHandle Handle);

	/**
	 * Invalidate the supplied TimerHandle and return it.
	 * @param Handle		The handle of the timer to invalidate.
	 * @return				Return the invalidated timer handle for convenience.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Invalidate Timer Handle", ScriptName = "InvalidateTimerHandle"), Category="Utilities|Time")
	static ENGINE_API FTimerHandle K2_InvalidateTimerHandle(UPARAM(ref) FTimerHandle& Handle);

	/**
	 * Clears a set timer.
	 * @param Handle		The handle of the timer to clear.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Clear Timer by Handle", ScriptName = "ClearTimerHandle", WorldContext="WorldContextObject", DeprecatedFunction, DeprecationMessage = "Use Clear and Invalidate Timer by Handle. Note: you no longer need to reset your handle yourself after switching to the new function."), Category="Utilities|Time")
	static ENGINE_API void K2_ClearTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Clears a set timer.
	 * @param Handle		The handle of the timer to clear.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Clear and Invalidate Timer by Handle", ScriptName = "ClearAndInvalidateTimerHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API void K2_ClearAndInvalidateTimerHandle(const UObject* WorldContextObject, UPARAM(ref) FTimerHandle& Handle);

	/**
	 * Pauses a set timer at its current elapsed time.
	 * @param Handle		The handle of the timer to pause.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Pause Timer by Handle", ScriptName = "PauseTimerHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API void K2_PauseTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Resumes a paused timer from its current elapsed time.
	 * @param Handle		The handle of the timer to unpause.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Unpause Timer by Handle", ScriptName = "UnPauseTimerHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API void K2_UnPauseTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Returns true if a timer exists and is active for the given handle, false otherwise.
	 * @param Handle		The handle of the timer to check whether it is active.
	 * @return				True if the timer exists and is active.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Timer Active by Handle", ScriptName = "IsTimerActiveHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API bool K2_IsTimerActiveHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Returns true if a timer exists and is paused for the given handle, false otherwise.
	 * @param Handle		The handle of the timer to check whether it is paused.
	 * @return				True if the timer exists and is paused.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Timer Paused by Handle", ScriptName = "IsTimerPausedHandle", WorldContext="WorldContextObject"), Category = "Utilities|Time")
	static ENGINE_API bool K2_IsTimerPausedHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Returns true is a timer for the given handle exists, false otherwise.
	 * @param Handle		The handle to check whether it exists.
 	 * @return				True if the timer exists.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Does Timer Exist by Handle", ScriptName = "TimerExistsHandle", WorldContext="WorldContextObject"), Category = "Utilities|Time")
	static ENGINE_API bool K2_TimerExistsHandle(const UObject* WorldContextObject, FTimerHandle Handle);
	
	/**
	 * Returns elapsed time for the given handle (time since current countdown iteration began).
	 * @param Handle		The handle of the timer to get the elapsed time of.
	 * @return				How long has elapsed since the current iteration of the timer began.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Timer Elapsed Time by Handle", ScriptName = "GetTimerElapsedTimeHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerElapsedTimeHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	/**
	 * Returns time until the timer will next execute its handle.
	 * @param Handle		The handle of the timer to time remaining of.
	 * @return				How long is remaining in the current iteration of the timer.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Timer Remaining Time by Handle", ScriptName = "GetTimerRemainingTimeHandle", WorldContext="WorldContextObject"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerRemainingTimeHandle(const UObject* WorldContextObject, FTimerHandle Handle);

	// --- Timer functions ------------------------------

	/**
	 * Set a timer to execute delegate. Setting an existing timer will reset that timer with updated parameters.
	 * @param Object					Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName				Delegate function name. Can be a K2 function or a Custom Event.
	 * @param Time						How long to wait before executing the delegate, in seconds. Setting a timer to <= 0 seconds will clear it if it is set.
	 * @param bLooping					True to keep executing the delegate every Time seconds, false to execute delegate only once.
	 * @param bMaxOncePerFrame			For looping timers, whether to execute only once when the timer would otherwise expires multiple times in the current frame.
	 * @param InitialStartDelay			Initial delay passed to the timer manager to allow some variance in when the timer starts, in seconds.
	 * @param InitialStartDelayVariance	Use this to add some variance to when the timer starts in lieu of doing a random range on the InitialStartDelay input, in seconds.
	 * @return							The timer handle to pass to other timer functions to manipulate this timer.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Set Timer by Function Name", ScriptName = "SetTimer", DefaultToSelf = "Object", AdvancedDisplay="InitialStartDelay, InitialStartDelayVariance"), Category="Utilities|Time")
	static ENGINE_API FTimerHandle K2_SetTimer(UObject* Object, FString FunctionName, float Time, bool bLooping, bool bMaxOncePerFrame = false, float InitialStartDelay = 0.f, float InitialStartDelayVariance = 0.f);

	/**
	 * Set a timer to execute a delegate on the next tick.
	 * @param Object					Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName				Delegate function name. Can be a K2 function or a Custom Event.
	 * @return							The timer handle to pass to other timer functions to manipulate this timer.
	 */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Timer for Next Tick by Function Name", ScriptName = "SetTimerForNextTick", DefaultToSelf = "Object"), Category = "Utilities|Time")
	static ENGINE_API FTimerHandle K2_SetTimerForNextTick(UObject* Object, FString FunctionName);

	/**
	 * Clears a set timer.
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Clear Timer by Function Name", ScriptName = "ClearTimer", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API void K2_ClearTimer(UObject* Object, FString FunctionName);

	/**
	 * Pauses a set timer at its current elapsed time.
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Pause Timer by Function Name", ScriptName = "PauseTimer", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API void K2_PauseTimer(UObject* Object, FString FunctionName);

	/**
	 * Resumes a paused timer from its current elapsed time.
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Unpause Timer by Function Name", ScriptName = "UnPauseTimer", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API void K2_UnPauseTimer(UObject* Object, FString FunctionName);

	/**
	 * Returns true if a timer exists and is active for the given delegate, false otherwise.
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 * @return				True if the timer exists and is active.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Is Timer Active by Function Name", ScriptName = "IsTimerActive", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API bool K2_IsTimerActive(UObject* Object, FString FunctionName);

	/**
	* Returns true if a timer exists and is paused for the given delegate, false otherwise.
	* @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	* @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	* @return				True if the timer exists and is paused.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Timer Paused by Function Name", ScriptName = "IsTimerPaused", DefaultToSelf = "Object"), Category = "Utilities|Time")
	static ENGINE_API bool K2_IsTimerPaused(UObject* Object, FString FunctionName);

	/**
	* Returns true is a timer for the given delegate exists, false otherwise.
	* @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	* @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	* @return				True if the timer exists.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Does Timer Exist by Function Name", ScriptName = "TimerExists", DefaultToSelf = "Object"), Category = "Utilities|Time")
	static ENGINE_API bool K2_TimerExists(UObject* Object, FString FunctionName);
	
	/**
	 * Returns elapsed time for the given delegate (time since current countdown iteration began).
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 * @return				How long has elapsed since the current iteration of the timer began.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Timer Elapsed Time by Function Name", ScriptName = "GetTimerElapsedTime", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerElapsedTime(UObject* Object, FString FunctionName);

	/**
	 * Returns time until the timer will next execute its delegate.
	 * @param Object		Object that implements the delegate function. Defaults to self (this blueprint)
	 * @param FunctionName	Delegate function name. Can be a K2 function or a Custom Event.
	 * @return				How long is remaining in the current iteration of the timer.
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Get Timer Remaining Time by Function Name", ScriptName = "GetTimerRemainingTime", DefaultToSelf = "Object"), Category="Utilities|Time")
	static ENGINE_API float K2_GetTimerRemainingTime(UObject* Object, FString FunctionName);


	// --- 'Set property by name' functions ------------------------------

	/** Set an int32 property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetIntPropertyByName(UObject* Object, FName PropertyName, int32 Value);
	
	/** Set an int64 property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetInt64PropertyByName(UObject* Object, FName PropertyName, int64 Value);

	/** Set an uint8 or enum property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetBytePropertyByName(UObject* Object, FName PropertyName, uint8 Value);

	/** Set a double property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetDoublePropertyByName(UObject* Object, FName PropertyName, double Value);

	/** Set a bool property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetBoolPropertyByName(UObject* Object, FName PropertyName, bool Value);

	/** Set an OBJECT property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetObjectPropertyByName(UObject* Object, FName PropertyName, UObject* Value);

	/** Set a CLASS property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetClassPropertyByName(UObject* Object, FName PropertyName, TSubclassOf<UObject> Value);

	/** Set an INTERFACE property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API void SetInterfacePropertyByName(UObject* Object, FName PropertyName, const FScriptInterface& Value);

	/** Set a NAME property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetNamePropertyByName(UObject* Object, FName PropertyName, const FName& Value);

	/** Set a SOFTOBJECT property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetSoftObjectPropertyByName(UObject* Object, FName PropertyName, const TSoftObjectPtr<UObject>& Value);

	/** Set a SOFTCLASS property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetSoftClassPropertyByName(UObject* Object, FName PropertyName, const TSoftClassPtr<UObject>& Value);

	/** Set a STRING property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetStringPropertyByName(UObject* Object, FName PropertyName, const FString& Value);

	/** Set a TEXT property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetTextPropertyByName(UObject* Object, FName PropertyName, const FText& Value);

	/** Set a VECTOR property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetVectorPropertyByName(UObject* Object, FName PropertyName, const FVector& Value);

	/** Set a VECTOR3F property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetVector3fPropertyByName(UObject* Object, FName PropertyName, const FVector3f& Value);

	/** Set a ROTATOR property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetRotatorPropertyByName(UObject* Object, FName PropertyName, const FRotator& Value);

	/** Set a LINEAR COLOR property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetLinearColorPropertyByName(UObject* Object, FName PropertyName, const FLinearColor& Value);

	/** Set a COLOR property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetColorPropertyByName(UObject* Object, FName PropertyName, const FColor& Value);

	/** Set a TRANSFORM property by name */
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value" ))
	static ENGINE_API void SetTransformPropertyByName(UObject* Object, FName PropertyName, const FTransform& Value);

	/** Set a CollisionProfileName property by name */
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetCollisionProfileNameProperty(UObject* Object, FName PropertyName, const FCollisionProfileName& Value);

	/** Set a SOFTOBJECT property by name */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetFieldPathPropertyByName(UObject* Object, FName PropertyName, const TFieldPath<FField>& Value);

	DECLARE_FUNCTION(execSetCollisionProfileNameProperty);

	/** Set a custom structure property by name */
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (BlueprintInternalUseOnly = "true", CustomStructureParam = "Value", AutoCreateRefTerm = "Value"))
	static ENGINE_API void SetStructurePropertyByName(UObject* Object, FName PropertyName, const FGenericStruct& Value);

	/** Based on UKismetArrayLibrary::execSetArrayPropertyByName */
	DECLARE_FUNCTION(execSetStructurePropertyByName);

	// --- Collision functions ------------------------------

	/**
	 * Returns an array of actors that overlap the given sphere.
	 * @param WorldContext	World context
	 * @param SpherePos		Center of sphere.
	 * @param SphereRadius	Size of sphere.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Sphere Overlap Actors"))
	static ENGINE_API bool SphereOverlapActors(const UObject* WorldContextObject, const FVector SpherePos, float SphereRadius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of components that overlap the given sphere.
	 * @param WorldContext	World context
	 * @param SpherePos		Center of sphere.
	 * @param SphereRadius	Size of sphere.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Sphere Overlap Components"))
	static ENGINE_API bool SphereOverlapComponents(const UObject* WorldContextObject, const FVector SpherePos, float SphereRadius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);
	

	/**
	 * Returns an array of actors that overlap the given axis-aligned box.
	 * @param WorldContext	World context
	 * @param BoxPos		Center of box.
	 * @param BoxExtent		Extents of box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Box Overlap Actors"))
	static ENGINE_API bool BoxOverlapActors(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of actors that overlap the given axis-aligned box.
	 * @param WorldContext	World context
	 * @param BoxPos		Center of box.
	 * @param BoxExtent		Extents of box.
	 * @param Orientation	Orientation of the box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Box Overlap Actors With Orientation"))
	static ENGINE_API bool BoxOverlapActorsWithOrientation(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of components that overlap the given axis-aligned box.
	 * @param WorldContext	World context
	 * @param BoxPos		Center of box.
	 * @param BoxExtent		Extents of box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Box Overlap Components"))
	static ENGINE_API bool BoxOverlapComponents(const UObject* WorldContextObject, const FVector BoxPos, FVector Extent, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);

	/**
	 * Returns an array of components that overlap the given box.
	 * @param WorldContext	World context
	 * @param BoxPos		Center of box.
	 * @param Extent		Extents of box.
	 * @param Orientation	Orientation of the box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Box Overlap Components With Orientation"))
	static ENGINE_API bool BoxOverlapComponentsWithOrientation(const UObject* WorldContextObject, const FVector BoxPos, FVector Extent, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);


	/**
	 * Returns an array of actors that overlap the given capsule.
	 * @param WorldContext	World context
	 * @param CapsulePos	Center of the capsule.
	 * @param Radius		Radius of capsule hemispheres and radius of center cylinder portion.
	 * @param HalfHeight	Half-height of the capsule (from center of capsule to tip of hemisphere.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Capsule Overlap Actors"))
	static ENGINE_API bool CapsuleOverlapActors(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of actors that overlap the given capsule.
	 * @param WorldContext	World context
	 * @param CapsulePos	Center of the capsule.
	 * @param Radius		Radius of capsule hemispheres and radius of center cylinder portion.
	 * @param HalfHeight	Half-height of the capsule (from center of capsule to tip of hemisphere.
	 * @param Orientation	Orientation of the box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Capsule Overlap Actors With Orientation"))
	static ENGINE_API bool CapsuleOverlapActorsWithOrientation(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of components that overlap the given capsule.
	 * @param WorldContext	World context
	 * @param CapsulePos	Center of the capsule.
	 * @param Radius		Radius of capsule hemispheres and radius of center cylinder portion.
	 * @param HalfHeight	Half-height of the capsule (from center of capsule to tip of hemisphere.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Capsule Overlap Components") )
	static ENGINE_API bool CapsuleOverlapComponents(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);

	/**
	 * Returns an array of components that overlap the given capsule.
	 * @param WorldContext	World context
	 * @param CapsulePos	Center of the capsule.
	 * @param Radius		Radius of capsule hemispheres and radius of center cylinder portion.
	 * @param HalfHeight	Half-height of the capsule (from center of capsule to tip of hemisphere.
	 * @param Orientation	Orientation of the box.
	 * @param Filter		Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter	If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors		Returned array of actors. Unsorted.
	 * @return				true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Capsule Overlap Components With Orientation") )
	static ENGINE_API bool CapsuleOverlapComponentsWithOrientation(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);

	/**
	 * Returns an array of actors that overlap the given component.
	 * @param Component				Component to test with.
	 * @param ComponentTransform	Defines where to place the component for overlap testing.
	 * @param Filter				Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter			If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors				Returned array of actors. Unsorted.
	 * @return						true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(AutoCreateRefTerm="ActorsToIgnore", DisplayName="Component Overlap Actors"))
	static ENGINE_API bool ComponentOverlapActors(UPrimitiveComponent* Component, const FTransform& ComponentTransform, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class AActor*>& OutActors);

	/**
	 * Returns an array of components that overlap the given component.
	 * @param Component				Component to test with.
	 * @param ComponentTransform	Defines where to place the component for overlap testing.
	 * @param Filter				Option to restrict results to only static or only dynamic.  For efficiency.
	 * @param ClassFilter			If set, will only return results of this class or subclasses of it.
	 * @param ActorsToIgnore		Ignore these actors in the list
	 * @param OutActors				Returned array of actors. Unsorted.
	 * @return						true if there was an overlap that passed the filters, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(AutoCreateRefTerm="ActorsToIgnore", DisplayName="Component Overlap Components"))
	static ENGINE_API bool ComponentOverlapComponents(UPrimitiveComponent* Component, const FTransform& ComponentTransform, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<class UPrimitiveComponent*>& OutComponents);


	/**
	 * Does a collision trace along the given line and returns the first blocking hit encountered.
	 * This trace finds the objects that RESPONDS to the given TraceChannel
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param TraceChannel	
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName="Line Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="raycast"))
	static ENGINE_API bool LineTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);
	
	/**
	 * Does a collision trace along the given line and returns all hits encountered up to and including the first blocking hit.
	 * This trace finds the objects that RESPOND to the given TraceChannel
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param TraceChannel	The channel to trace
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a blocking hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Line Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="raycast"))
	static ENGINE_API bool LineTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a sphere along the given line and returns the first blocking hit encountered.
	 * This trace finds the objects that RESPONDS to the given TraceChannel
	 * 
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the sphere to sweep
	 * @param TraceChannel	
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Sphere Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool SphereTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a sphere along the given line and returns all hits encountered up to and including the first blocking hit.
	 * This trace finds the objects that RESPOND to the given TraceChannel
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the sphere to sweep
	 * @param TraceChannel	
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	 * @return				True if there was a blocking hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Sphere Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool SphereTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	* Sweeps a box along the given line and returns the first blocking hit encountered.
	* This trace finds the objects that RESPONDS to the given TraceChannel
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param HalfSize	    Distance from the center of box along each axis
	* @param Orientation	Orientation of the box
	* @param TraceChannel
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext="WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Box Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool BoxTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	* Sweeps a box along the given line and returns all hits encountered.
	* This trace finds the objects that RESPONDS to the given TraceChannel
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param HalfSize	    Distance from the center of box along each axis
	* @param Orientation	Orientation of the box
	* @param TraceChannel
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHits		A list of hits, sorted along the trace from start to finish. The blocking hit will be the last hit, if there was one.
	* @return				True if there was a blocking hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext="WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Box Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool BoxTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, FVector HalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	
	/**
	 * Sweeps a capsule along the given line and returns the first blocking hit encountered.
	 * This trace finds the objects that RESPOND to the given TraceChannel
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the capsule to sweep
	 * @param HalfHeight	Distance from center of capsule to tip of hemisphere endcap.
	 * @param TraceChannel	
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Capsule Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool CapsuleTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a capsule along the given line and returns all hits encountered up to and including the first blocking hit.
	 * This trace finds the objects that RESPOND to the given TraceChannel
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the capsule to sweep
	 * @param HalfHeight	Distance from center of capsule to tip of hemisphere endcap.
	 * @param TraceChannel	
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	 * @return				True if there was a blocking hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Capsule Trace By Channel", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool CapsuleTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Does a collision trace along the given line and returns the first hit encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Line Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="raycast"))
	static ENGINE_API bool LineTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f );
	
	/**
	 * Does a collision trace along the given line and returns all hits encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Line Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="raycast"))
	static ENGINE_API bool LineTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a sphere along the given line and returns the first hit encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the sphere to sweep
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Sphere Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool SphereTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a sphere along the given line and returns all hits encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the sphere to sweep
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Sphere Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool SphereTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);
	

	/**
	* Sweeps a box along the given line and returns the first hit encountered.
	* This only finds objects that are of a type specified by ObjectTypes.
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Orientation	
	* @param HalfSize		Radius of the sphere to sweep
	* @param ObjectTypes	Array of Object Types to trace
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext="WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Box Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool BoxTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);


	/**
	* Sweeps a box along the given line and returns all hits encountered.
	* This only finds objects that are of a type specified by ObjectTypes.
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Orientation
	* @param HalfSize		Radius of the sphere to sweep
	* @param ObjectTypes	Array of Object Types to trace
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext="WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Box Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool BoxTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a capsule along the given line and returns the first hit encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the capsule to sweep
	 * @param HalfHeight	Distance from center of capsule to tip of hemisphere endcap.
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHit		Properties of the trace hit.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Capsule Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool CapsuleTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	 * Sweeps a capsule along the given line and returns all hits encountered.
	 * This only finds objects that are of a type specified by ObjectTypes.
	 * 
	 * @param WorldContext	World context
	 * @param Start			Start of line segment.
	 * @param End			End of line segment.
	 * @param Radius		Radius of the capsule to sweep
	 * @param HalfHeight	Distance from center of capsule to tip of hemisphere endcap.
	 * @param ObjectTypes	Array of Object Types to trace 
	 * @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	 * @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	 * @return				True if there was a hit, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Collision", meta=(bIgnoreSelf="true", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", DisplayName = "Multi Capsule Trace For Objects", AdvancedDisplay="TraceColor,TraceHitColor,DrawTime", Keywords="sweep"))
	static ENGINE_API bool CapsuleTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	// BY PROFILE

	/**
	* Trace a ray against the world using a specific profile and return the first blocking hit
	*
	* @param WorldContext	World context
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Line Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "raycast"))
	static ENGINE_API bool LineTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, UPARAM(Meta=(GetOptions="Engine.KismetSystemLibrary.GetCollisionProfileNames")) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Trace a ray against the world using a specific profile and return overlapping hits and then first blocking hit
	*  Results are sorted, so a blocking hit (if found) will be the last element of the array
	*  Only the single closest blocking result will be generated, no tests will be done after that
	*
	* @param WorldContext	World context
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit		Properties of the trace hit.
	* @return				True if there was a blocking hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Line Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "raycast"))
	static ENGINE_API bool LineTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Sweep a sphere against the world and return the first blocking hit using a specific profile
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Radius			Radius of the sphere to sweep
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Sphere Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool SphereTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Sweep a sphere against the world and return all initial overlaps using a specific profile, then overlapping hits and then first blocking hit
	*  Results are sorted, so a blocking hit (if found) will be the last element of the array
	*  Only the single closest blocking result will be generated, no tests will be done after that
	*
	* @param WorldContext	World context
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Radius		Radius of the sphere to sweep
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	* @return				True if there was a blocking hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Sphere Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool SphereTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Sweep a box against the world and return the first blocking hit using a specific profile
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param HalfSize	    Distance from the center of box along each axis
	* @param Orientation	Orientation of the box
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Box Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool BoxTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Sweep a box against the world and return all initial overlaps using a specific profile, then overlapping hits and then first blocking hit
	*  Results are sorted, so a blocking hit (if found) will be the last element of the array
	*  Only the single closest blocking result will be generated, no tests will be done after that
	*
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param HalfSize	    Distance from the center of box along each axis
	* @param Orientation	Orientation of the box
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHits		A list of hits, sorted along the trace from start to finish. The blocking hit will be the last hit, if there was one.
	* @return				True if there was a blocking hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Box Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool BoxTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, FVector HalfSize, const FRotator Orientation, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);


	/**
	*  Sweep a capsule against the world and return the first blocking hit using a specific profile
	*
	* @param WorldContext	World context
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Radius			Radius of the capsule to sweep
	* @param HalfHeight		Distance from center of capsule to tip of hemisphere endcap.
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHit			Properties of the trace hit.
	* @return				True if there was a hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Capsule Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool CapsuleTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	/**
	*  Sweep a capsule against the world and return all initial overlaps using a specific profile, then overlapping hits and then first blocking hit
	*  Results are sorted, so a blocking hit (if found) will be the last element of the array
	*  Only the single closest blocking result will be generated, no tests will be done after that
	*
	* @param WorldContext	World context
	* @param Start			Start of line segment.
	* @param End			End of line segment.
	* @param Radius			Radius of the capsule to sweep
	* @param HalfHeight		Distance from center of capsule to tip of hemisphere endcap.
	* @param ProfileName	The 'profile' used to determine which components to hit
	* @param bTraceComplex	True to test against complex collision, false to test against simplified collision.
	* @param OutHits		A list of hits, sorted along the trace from start to finish.  The blocking hit will be the last hit, if there was one.
	* @return				True if there was a blocking hit, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", DisplayName = "Multi Capsule Trace By Profile", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static ENGINE_API bool CapsuleTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, UPARAM(Meta=(GetOptions=GetCollisionProfileNames)) FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

#if WITH_EDITOR
	UFUNCTION()
	static TArray<FName> GetCollisionProfileNames();
#endif


	/**
	 * Returns an array of unique actors represented by the given list of components.
	 * @param ComponentList		List of components.
	 * @param ClassFilter		If set, will only return results of this class or subclasses of it.
	 * @param OutActorList		Start of line segment.
	 */
	UFUNCTION(BlueprintCallable, Category="Actor")
	static ENGINE_API void GetActorListFromComponentList(const TArray<class UPrimitiveComponent*>& ComponentList, UClass* ActorClassFilter, TArray<class AActor*>& OutActorList);


	// --- Debug drawing functions ------------------------------

	/** Draw a debug line */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugLine(const UObject* WorldContextObject, const FVector LineStart, const FVector LineEnd, FLinearColor LineColor, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug circle! */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugCircle(const UObject* WorldContextObject, FVector Center, float Radius, int32 NumSegments=12, FLinearColor LineColor = FLinearColor::White, float Duration=0.f, float Thickness=0.f, FVector YAxis=FVector(0.f,1.f,0.f),FVector ZAxis=FVector(0.f,0.f,1.f), bool bDrawAxis=false, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);
	
	/** Draw a debug point */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugPoint(const UObject* WorldContextObject, const FVector Position, float Size, FLinearColor PointColor, float Duration=0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw directional arrow, pointing from LineStart to LineEnd. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugArrow(const UObject* WorldContextObject, const FVector LineStart, const FVector LineEnd, float ArrowSize, FLinearColor LineColor, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug box */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugBox(const UObject* WorldContextObject, const FVector Center, FVector Extent, FLinearColor LineColor, const FRotator Rotation=FRotator::ZeroRotator, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug coordinate system. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugCoordinateSystem(const UObject* WorldContextObject, const FVector AxisLoc, const FRotator AxisRot, float Scale=1.f, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug sphere */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugSphere(const UObject* WorldContextObject, const FVector Center, float Radius=100.f, int32 Segments=12, FLinearColor LineColor = FLinearColor::White, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug cylinder */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugCylinder(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius=100.f, int32 Segments=12, FLinearColor LineColor = FLinearColor::White, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);
	
	/** Draw a debug cone */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DeprecatedFunction, DeprecationMessage="DrawDebugCone has been changed to use degrees for angles instead of radians. Place a new DrawDebugCone node and pass your angles as degrees.", DevelopmentOnly))
	static ENGINE_API void DrawDebugCone(const UObject* WorldContextObject, const FVector Origin, const FVector Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FLinearColor LineColor, float Duration = 0.f, float Thickness = 0.f);

	/** 
	 * Draw a debug cone 
	 * Angles are specified in degrees
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DisplayName="Draw Debug Cone", DevelopmentOnly))
	static ENGINE_API void DrawDebugConeInDegrees(const UObject* WorldContextObject, const FVector Origin, const FVector Direction, float Length=100.f, float AngleWidth=45.f, float AngleHeight=45.f, int32 NumSides = 12, FLinearColor LineColor = FLinearColor::White, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug capsule */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugCapsule(const UObject* WorldContextObject, const FVector Center, float HalfHeight, float Radius, const FRotator Rotation, FLinearColor LineColor = FLinearColor::White, float Duration=0.f, float Thickness = 0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** Draw a debug string at a 3d world location. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugString(const UObject* WorldContextObject, const FVector TextLocation, const FString& Text, class AActor* TestBaseActor = NULL, FLinearColor TextColor = FLinearColor::White, float Duration=0.f);
	/** 
	 * Removes all debug strings. 
	 *
	 * @param WorldContext	World context
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void FlushDebugStrings(const UObject* WorldContextObject);

	/** Draws a debug plane. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugPlane(const UObject* WorldContextObject, const FPlane& PlaneCoordinates, const FVector Location, float Size, FLinearColor PlaneColor = FLinearColor::White, float Duration=0.f, EDrawDebugSceneDepthPriorityGroup DepthPriority = EDrawDebugSceneDepthPriorityGroup::World);

	/** 
	 * Flush all persistent debug lines and shapes.
	 *
	 * @param WorldContext	World context
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void FlushPersistentDebugLines(const UObject* WorldContextObject);

	/** Draws a debug frustum. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugFrustum(const UObject* WorldContextObject, const FTransform& FrustumTransform, FLinearColor FrustumColor = FLinearColor::White, float Duration=0.f, float Thickness = 0.f);

	/** Draw a debug camera shape. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Debug", meta=(DevelopmentOnly))
	static ENGINE_API void DrawDebugCamera(const ACameraActor* CameraActor, FLinearColor CameraColor = FLinearColor::White, float Duration=0.f);

	/* Draws a 2D Histogram of size 'DrawSize' based FDebugFloatHistory struct, using DrawTransform for the position in the world. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugFloatHistoryTransform(const UObject* WorldContextObject, const FDebugFloatHistory& FloatHistory, const FTransform& DrawTransform, FVector2D DrawSize, FLinearColor DrawColor = FLinearColor::White, float Duration = 0.f);

	/* Draws a 2D Histogram of size 'DrawSize' based FDebugFloatHistory struct, using DrawLocation for the location in the world, rotation will face camera of first player. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Debug", meta=(WorldContext="WorldContextObject", DevelopmentOnly))
	static ENGINE_API void DrawDebugFloatHistoryLocation(const UObject* WorldContextObject, const FDebugFloatHistory& FloatHistory, FVector DrawLocation, FVector2D DrawSize, FLinearColor DrawColor = FLinearColor::White, float Duration = 0.f);

	UFUNCTION(BlueprintCallable, Category = "Rendering|Debug", meta=(DevelopmentOnly))
	static ENGINE_API FDebugFloatHistory AddFloatHistorySample(float Value, const FDebugFloatHistory& FloatHistory);
	
	/** Mark as modified. */
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	static ENGINE_API void CreateCopyForUndoBuffer(UObject* ObjectToModify);

	/** Get bounds */
	UFUNCTION(BlueprintPure, Category="Collision")
	static ENGINE_API void GetComponentBounds(const USceneComponent* Component, FVector& Origin, FVector& BoxExtent, float& SphereRadius);

	UFUNCTION(BlueprintPure, Category="Collision", meta=(DeprecatedFunction))
	static ENGINE_API void GetActorBounds(const AActor* Actor, FVector& Origin, FVector& BoxExtent);

	/**
	 * Get the clamped state of r.DetailMode, see console variable help (allows for scalability, cannot be used in construction scripts)
	 * 0: low, show objects with DetailMode low
	 * 1: medium, show objects with DetailMode medium or below
	 * 2: high, show objects with DetailMode high or below
	 * 3: epic, show all objects
	 */
	UFUNCTION(BlueprintPure, Category="Rendering", meta=(UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetRenderingDetailMode();

	/**
	 * Get the clamped state of r.MaterialQualityLevel, see console variable help (allows for scalability, cannot be used in construction scripts)
	 * 0: low
	 * 1: high
	 * 2: medium
	 */
	UFUNCTION(BlueprintPure, Category="Rendering|Material", meta=(UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetRenderingMaterialQualityLevel();

	/**
	 * Gets the list of support fullscreen resolutions.
	 * @return true if successfully queried the device for available resolutions.
	 */
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static ENGINE_API bool GetSupportedFullscreenResolutions(TArray<FIntPoint>& Resolutions);

	/**
	* Gets the list of windowed resolutions which are convenient for the current primary display size.
	* @return true if successfully queried the device for available resolutions.
	*/
	UFUNCTION(BlueprintCallable, Category="Rendering")
	static ENGINE_API bool GetConvenientWindowedResolutions(TArray<FIntPoint>& Resolutions);

	/**
	 * Gets the smallest Y resolution we want to support in the UI, clamped within reasons
	 * @return value in pixels
	 */
	UFUNCTION(BlueprintPure, Category="Rendering", meta=(UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetMinYResolutionForUI();

	/**
	* Gets the smallest Y resolution we want to support in the 3D view, clamped within reasons
	* @return value in pixels
	*/
	UFUNCTION(BlueprintPure, Category = "Rendering", meta = (UnsafeDuringActorConstruction = "true"))
	static ENGINE_API int32 GetMinYResolutionFor3DView();

	// Opens the specified URL in the platform's web browser of choice
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void LaunchURL(const FString& URL);

	// Opens an external URL in the platform's web browser of choice if it meets the allowlist of passed in domains
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void LaunchExternalUrl(const TArray<FString>& InDomainStrings, const FString& URL);

	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API bool CanLaunchURL(const FString& URL);

	// Deletes all unreferenced objects, keeping only referenced objects (this command will be queued and happen at the end of the frame)
	// Note: This can be a slow operation, and should only be performed where a hitch would be acceptable
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void CollectGarbage();

	/**
	 * Will show an ad banner (iAd on iOS, or AdMob on Android) on the top or bottom of screen, on top of the GL view (doesn't resize the view)
	 * (iOS and Android only)
	 *
	 * @param AdIdIndex The index of the ID to select for the ad to show
	 * @param bShowOnBottomOfScreen If true, the iAd will be shown at the bottom of the screen, top otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ShowAdBanner(int32 AdIdIndex, bool bShowOnBottomOfScreen);

	/**
	* Retrieves the total number of Ad IDs that can be selected between
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Platform", meta = (DisplayName = "Get Ad ID Count"))
	static ENGINE_API int32 GetAdIDCount();

	/**
	 * Hides the ad banner (iAd on iOS, or AdMob on Android). Will force close the ad if it's open
	 * (iOS and Android only)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void HideAdBanner();

	/**
	 * Forces closed any displayed ad. Can lead to loss of revenue
	 * (iOS and Android only)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ForceCloseAdBanner();

	/**
	* Will load a fullscreen interstitial AdMob ad. Call this before using ShowInterstitialAd
	* (Android only)
	*
	* @param AdIdIndex The index of the ID to select for the ad to show
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void LoadInterstitialAd(int32 AdIdIndex);

	/**
	* Returns true if the requested interstitial ad is loaded and ready
	* (Android only)
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API bool IsInterstitialAdAvailable();

	/**
	* Returns true if the requested interstitial ad has been successfully requested (false if load request fails)
	* (Android only)
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API bool IsInterstitialAdRequested();

	/**
	* Shows the loaded interstitial ad (loaded with LoadInterstitialAd)
	* (Android only)
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ShowInterstitialAd();

	/**
	 * Displays the built-in leaderboard GUI (iOS and Android only; this function may be renamed or moved in a future release)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ShowPlatformSpecificLeaderboardScreen(const FString& CategoryName);

	/**
	 * Displays the built-in achievements GUI (iOS and Android only; this function may be renamed or moved in a future release)
	 *
	 * @param SpecificPlayer Specific player's achievements to show. May not be supported on all platforms. If null, defaults to the player with ControllerId 0
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ShowPlatformSpecificAchievementsScreen(const class APlayerController* SpecificPlayer);

	/**
	 * Returns whether the player is logged in to the currently active online subsystem.
	 *
	 * @param Player Specific player's login status to get. May not be supported on all platforms. If null, defaults to the player with ControllerId 0.
	 */
	UFUNCTION(BlueprintPure, Category = "Online")
	static ENGINE_API bool IsLoggedIn(const APlayerController* SpecificPlayer);

	/**
	 * Returns true if screen saver is enabled.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API bool IsScreensaverEnabled();

	/**
	 * Allows or inhibits screensaver
	 * @param	bAllowScreenSaver		If false, don't allow screensaver if possible, otherwise allow default behavior
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ControlScreensaver(bool bAllowScreenSaver);

	/**
	 * Allows or inhibits system default handling of volume up and volume down buttons (Android only)
	 * @param	bEnabled				If true, allow Android to handle volume up and down events
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void SetVolumeButtonsHandledBySystem(bool bEnabled);

	/**
	 * Returns true if system default handling of volume up and volume down buttons enabled (Android only)
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Platform")
	static ENGINE_API bool GetVolumeButtonsHandledBySystem();

	/**
	 * Sets whether attached gamepads will block feedback from the device itself (Mobile only).
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void SetGamepadsBlockDeviceFeedback(bool bBlock);

	/**
	 * Resets the gamepad to player controller id assignments (Android and iOS only)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ResetGamepadAssignments();

	/*
	 * Resets the gamepad assignment to player controller id (Android and iOS only)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void ResetGamepadAssignmentToController(int32 ControllerId);

	/**
	 * Returns true if controller id assigned to a gamepad (Android and iOS only)
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Platform")
	static ENGINE_API bool IsControllerAssignedToGamepad(int32 ControllerId);

	/**
	* Returns name of controller if assigned to a gamepad (or None if not assigned) (Android and iOS only)
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Platform")
    static ENGINE_API FString GetGamepadControllerName(int32 ControllerId);
    
    /**
    * Returns glyph assigned to a gamepad button (or a null ptr if not assigned) (iOS and tvOS only)
    */
    UFUNCTION(BlueprintPure, Category = "Utilities|Platform")
    static ENGINE_API UTexture2D* GetGamepadButtonGlyph(const FString& ButtonKey, int32 ControllerIndex);

	/**
	 * Sets the state of the transition message rendered by the viewport. (The blue text displayed when the game is paused and so forth.)
	 *
	 * @param WorldContextObject	World context
	 * @param State					set true to suppress transition message
	 */
	UFUNCTION(BlueprintCallable, Category = "Viewport", meta = (WorldContext="WorldContextObject"))
	static ENGINE_API void SetSuppressViewportTransitionMessage(const UObject* WorldContextObject, bool bState);

	/**
	 * Returns an array of the user's preferred languages in order of preference
	 * @return An array of language IDs ordered from most preferred to least
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Internationalization")
	static ENGINE_API TArray<FString> GetPreferredLanguages();
	
	/**
	 * Get the default language (for localization) used by this platform
	 * @note This is typically the same as GetDefaultLocale unless the platform distinguishes between the two
	 * @note This should be returned in IETF language tag form:
	 *  - A two-letter ISO 639-1 language code (eg, "zh")
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans")
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN")
	 * @return The language as an IETF language tag (eg, "zh-Hans-CN")
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Internationalization")
	static ENGINE_API FString GetDefaultLanguage();

	/**
	 * Get the default locale (for internationalization) used by this platform
	 * @note This should be returned in IETF language tag form:
	 *  - A two-letter ISO 639-1 language code (eg, "zh")
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans")
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN")
	 * @return The locale as an IETF language tag (eg, "zh-Hans-CN")
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Internationalization")
	static ENGINE_API FString GetDefaultLocale();

	/**
	* Returns the currency code associated with the device's locale
	* @return the currency code associated with the device's locale
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Internationalization")
	static ENGINE_API FString GetLocalCurrencyCode();

	/**
	* Returns the currency symbol associated with the device's locale
	* @return the currency symbol associated with the device's locale
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Internationalization")
	static ENGINE_API FString GetLocalCurrencySymbol();

	/**
	 * Requests permission to send remote notifications to the user's device.
	 * (Android and iOS only)
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void RegisterForRemoteNotifications();

	/**
	* Requests Requests unregistering from receiving remote notifications to the user's device.
	* (Android only)
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities|Platform")
	static ENGINE_API void UnregisterForRemoteNotifications();

	/**
	 * Tells the engine what the user is doing for debug, analytics, etc.
	 */
	UFUNCTION(BlueprintCallable, Category = "Development")
	static ENGINE_API void SetUserActivity(const FUserActivity& UserActivity);

	/**
	 * Returns the command line that the process was launched with.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities")
	static ENGINE_API FString GetCommandLine();

	/*
	* Parses the given string into loose tokens, switches (arguments that begin with - or /) and parameters (-mySwitch=myVar)
	*
	* @param	InCmdLine			The the string to parse (ie '-foo -bar=/game/baz testtoken' )
	* @param	OutTokens[out]		Filled with all loose tokens found in the string (ie: testToken in above example)
	* @param	OutSwitches[out]	Filled with all switches found in the string (ie -foo)
	* @param	OutParams[out]		Filled with all switches found in the string with the format key = value (ie: -bar, /game/baz)
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities")
	static ENGINE_API void ParseCommandLine(const FString& InCmdLine, TArray<FString>& OutTokens, TArray<FString>& OutSwitches, TMap<FString, FString>& OutParams);

	
	/**
	 * Returns true if the string has -param in it (do not specify the leading -)
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool ParseParam(const FString& InString, const FString& InParam);

	/**
	 * Returns 'value' if -option=value is in the string
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool ParseParamValue(const FString& InString, const FString& InParam, FString& OutValue);

	/**
	 * Returns true if running unattended (-unattended is on the command line)
	 *
	 * @return	Unattended state
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities")
	static ENGINE_API bool IsUnattended();

	// --- Property Access ---------------------------

#if WITH_EDITOR
	/**
	 * Attempts to retrieve the value of a named property from the given object.
	 *
	 * @param Object The object you want to retrieve a property value from.
	 * @param PropertyName The name of the object property to retrieve the value from.
	 * @param PropertyValue The retrieved property value, if found.
	 *
	 * @return Whether the property value was found and correctly retrieved.
	 */
    UFUNCTION(BlueprintCallable, CustomThunk, Category = "Utilities", meta=(CustomStructureParam="PropertyValue", BlueprintInternalUseOnly="true"))
    static ENGINE_API bool GetEditorProperty(UObject* Object, const FName PropertyName, int32& PropertyValue);
	static ENGINE_API bool Generic_GetEditorProperty(const UObject* Object, const FName PropertyName, void* ValuePtr, const FProperty* ValueProp);
	DECLARE_FUNCTION(execGetEditorProperty);

	/**
	 * Attempts to set the value of a named property on the given object.
	 *
	 * @param Object The object you want to set a property value on.
	 * @param PropertyName The name of the object property to set the value of.
	 * @param PropertyValue The property value to set.
	 * @param ChangeNotifyMode When to emit property change notifications.
	 *
	 * @return Whether the property value was found and correctly set.
	 */
    UFUNCTION(BlueprintCallable, CustomThunk, Category = "Utilities", meta=(CustomStructureParam="PropertyValue", AdvancedDisplay="ChangeNotifyMode", BlueprintInternalUseOnly="true"))
    static ENGINE_API bool SetEditorProperty(UObject* Object, const FName PropertyName, const int32& PropertyValue, const EPropertyAccessChangeNotifyMode ChangeNotifyMode);
	static ENGINE_API bool Generic_SetEditorProperty(UObject* Object, const FName PropertyName, const void* ValuePtr, const FProperty* ValueProp, const EPropertyAccessChangeNotifyMode ChangeNotifyMode);
	DECLARE_FUNCTION(execSetEditorProperty);

	/**
	 * Attempts to reset the value of a named property on the given object so that it matches the value of the archetype.
	 *
	 * @param Object The object you want to reset a property value on.
	 * @param PropertyName The name of the object property to reset the value of.
	 * @param ChangeNotifyMode When to emit property change notifications.
	 *
	 * @return Whether the property value was found and correctly reset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta=(ScriptMethod))
	static ENGINE_API bool ResetEditorProperty(UObject* Object, const FName PropertyName, const EPropertyAccessChangeNotifyMode ChangeNotifyMode = EPropertyAccessChangeNotifyMode::Default);

	/**
	 * Attempts to query whether the value of a named property on the given object overrides the value of its archetype (ie, would ResetEditorProperty do anything?).
	 *
	 * @param Object The object you want to query a property value on.
	 * @param PropertyName The name of the object property to query the value of.
	 *
	 * @return What state the requested property is in.
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta=(ScriptMethod))
	static ENGINE_API EEditorPropertyValueState IsEditorPropertyOverridden(UObject* Object, const FName PropertyName);
#endif

	// --- Transactions ------------------------------

	/**
	 * Begin a new undo transaction. An undo transaction is defined as all actions which take place when the user selects "undo" a single time.
	 * @note If there is already an active transaction in progress, then this increments that transaction's action counter instead of beginning a new transaction.
	 * @note You must call TransactObject before modifying each object that should be included in this undo transaction.
	 * @note Only available in the editor.
	 * 
	 * @param	Context			The context for the undo session. Typically the tool/editor that caused the undo operation.
	 * @param	Description		The description for the undo session. This is the text that will appear in the "Edit" menu next to the Undo item.
	 * @param	PrimaryObject	The primary object that the undo session operators on (can be null, and mostly is).
	 *
	 * @return	The number of active actions when BeginTransaction was called (values greater than 0 indicate that there was already an existing undo transaction in progress), or -1 on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "Transactions")
	static ENGINE_API int32 BeginTransaction(const FString& Context, FText Description, UObject* PrimaryObject);

	/**
	 * Attempt to end the current undo transaction. Only successful if the transaction's action counter is 1.
	 * @note Only available in the editor.
	 * 
	 * @return	The number of active actions when EndTransaction was called (a value of 1 indicates that the transaction was successfully closed), or -1 on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "Transactions")
	static ENGINE_API int32 EndTransaction();

	/**
	 * Cancel the current transaction, and no longer capture actions to be placed in the undo buffer.
	 * @note Only available in the editor.
	 *
	 * @param	Index		The action counter to cancel transactions from (as returned by a call to BeginTransaction).
	 */
	UFUNCTION(BlueprintCallable, Category = "Transactions")
	static ENGINE_API void CancelTransaction(const int32 Index);

	/**
	 * Notify the current transaction (if any) that this object is about to be modified and should be placed into the undo buffer.
	 * @note Internally this calls Modify on the given object, so will also mark the owner package dirty.
	 * @note Only available in the editor.
	 *
	 * @param	Object		The object that is about to be modified.
	 */
	UFUNCTION(BlueprintCallable, Category = "Transactions")
	static ENGINE_API void TransactObject(UObject* Object);

	/**
	 * Notify the current transaction (if any) that this object is about to be modified and should be snapshot for intermediate update.
	 * @note Internally this calls SnapshotTransactionBuffer on the given object.
	 * @note Only available in the editor.
	 *
	 * @param	Object		The object that is about to be modified.
	 */
	UFUNCTION(BlueprintCallable, Category = "Transactions")
	static ENGINE_API void SnapshotObject(UObject* Object);

	// --- Asset Manager ------------------------------

	/** Returns the Object associated with a Primary Asset Id, this will only return a valid object if it is in memory, it will not load it */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="GetObject"))
	static ENGINE_API UObject* GetObjectFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId);

	/** Returns the Blueprint Class associated with a Primary Asset Id, this will only return a valid object if it is in memory, it will not load it */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="GetClass"))
	static ENGINE_API TSubclassOf<UObject> GetClassFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId);

	/** Returns the Object Id associated with a Primary Asset Id, this works even if the asset is not loaded */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="GetSoftObjectReference"))
	static ENGINE_API TSoftObjectPtr<UObject> GetSoftObjectReferenceFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId);

	/** Returns the Blueprint Class Id associated with a Primary Asset Id, this works even if the asset is not loaded */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="GetSoftClassReference"))
	static ENGINE_API TSoftClassPtr<UObject> GetSoftClassReferenceFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId);

	/** Returns the Primary Asset Id for an Object, this can return an invalid one if not registered */
	UFUNCTION(BlueprintPure, Category = "AssetManager")
	static ENGINE_API FPrimaryAssetId GetPrimaryAssetIdFromObject(UObject* Object);

	/** Returns the Primary Asset Id for a Class, this can return an invalid one if not registered */
	UFUNCTION(BlueprintPure, Category = "AssetManager")
	static ENGINE_API FPrimaryAssetId GetPrimaryAssetIdFromClass(TSubclassOf<UObject> Class);

	/** Returns the Primary Asset Id for a Soft Object Reference, this can return an invalid one if not registered */
	UFUNCTION(BlueprintPure, Category = "AssetManager")
	static ENGINE_API FPrimaryAssetId GetPrimaryAssetIdFromSoftObjectReference(TSoftObjectPtr<UObject> SoftObjectReference);

	/** Returns the Primary Asset Id for a Soft Class Reference, this can return an invalid one if not registered */
	UFUNCTION(BlueprintPure, Category = "AssetManager")
	static ENGINE_API FPrimaryAssetId GetPrimaryAssetIdFromSoftClassReference(TSoftClassPtr<UObject> SoftClassReference);

	/** Returns list of PrimaryAssetIds for a PrimaryAssetType */
	UFUNCTION(BlueprintCallable, Category = "AssetManager", meta=(ScriptMethod))
	static ENGINE_API void GetPrimaryAssetIdList(FPrimaryAssetType PrimaryAssetType, TArray<FPrimaryAssetId>& OutPrimaryAssetIdList);

	/** Returns true if the Primary Asset Id is valid */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="IsValid", ScriptOperator="bool", BlueprintThreadSafe))
	static ENGINE_API bool IsValidPrimaryAssetId(FPrimaryAssetId PrimaryAssetId);

	/** Converts a Primary Asset Id to a string. The other direction is not provided because it cannot be validated */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (PrimaryAssetId)", CompactNodeTitle = "->", ScriptMethod="ToString", BlueprintThreadSafe, BlueprintAutocast), Category = "AssetManager")
	static ENGINE_API FString Conv_PrimaryAssetIdToString(FPrimaryAssetId PrimaryAssetId);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (PrimaryAssetId)", CompactNodeTitle = "==", ScriptOperator="==", BlueprintThreadSafe), Category = "AssetManager")
	static ENGINE_API bool EqualEqual_PrimaryAssetId(FPrimaryAssetId A, FPrimaryAssetId B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (PrimaryAssetId)", CompactNodeTitle = "!=", ScriptOperator="!=", BlueprintThreadSafe), Category = "AssetManager")
	static ENGINE_API bool NotEqual_PrimaryAssetId(FPrimaryAssetId A, FPrimaryAssetId B);

	/** Returns list of Primary Asset Ids for a PrimaryAssetType */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="IsValid", ScriptOperator="bool", BlueprintThreadSafe))
	static ENGINE_API bool IsValidPrimaryAssetType(FPrimaryAssetType PrimaryAssetType);

	/** Converts a Primary Asset Type to a string. The other direction is not provided because it cannot be validated */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (PrimaryAssetType)", CompactNodeTitle = "->", ScriptMethod="ToString", BlueprintThreadSafe, BlueprintAutocast), Category = "AssetManager")
	static ENGINE_API FString Conv_PrimaryAssetTypeToString(FPrimaryAssetType PrimaryAssetType);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (PrimaryAssetType)", CompactNodeTitle = "==", ScriptOperator="==", BlueprintThreadSafe), Category = "AssetManager")
	static ENGINE_API bool EqualEqual_PrimaryAssetType(FPrimaryAssetType A, FPrimaryAssetType B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (PrimaryAssetType)", CompactNodeTitle = "!=", ScriptOperator="!=", BlueprintThreadSafe), Category = "AssetManager")
	static ENGINE_API bool NotEqual_PrimaryAssetType(FPrimaryAssetType A, FPrimaryAssetType B);

	/** Unloads a primary asset, which allows it to be garbage collected if nothing else is referencing it */
	UFUNCTION(BlueprintCallable, Category = "AssetManager", meta=(ScriptMethod="Unload"))
	static ENGINE_API void UnloadPrimaryAsset(FPrimaryAssetId PrimaryAssetId);

	/** Unloads a primary asset, which allows it to be garbage collected if nothing else is referencing it */
	UFUNCTION(BlueprintCallable, Category = "AssetManager")
	static ENGINE_API void UnloadPrimaryAssetList(const TArray<FPrimaryAssetId>& PrimaryAssetIdList);

	/** 
	 * Returns the list of loaded bundles for a given Primary Asset. This will return false if the asset is not loaded at all.
	 * If ForceCurrentState is true it will return the current state even if a load is in process
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetManager", meta=(ScriptMethod))
	static ENGINE_API bool GetCurrentBundleState(FPrimaryAssetId PrimaryAssetId, bool bForceCurrentState, TArray<FName>& OutBundles);

	/** 
	 * Returns the list of assets that are in a given bundle state. Required Bundles must be specified
	 * If ExcludedBundles is not empty, it will not return any assets in those bundle states
	 * If ValidTypes is not empty, it will only return assets of those types
	 * If ForceCurrentState is true it will use the current state even if a load is in process
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetManager", meta=(AutoCreateRefTerm = "ExcludedBundles, ValidTypes"))
	static ENGINE_API void GetPrimaryAssetsWithBundleState(const TArray<FName>& RequiredBundles, const TArray<FName>& ExcludedBundles, const TArray<FPrimaryAssetType>& ValidTypes, bool bForceCurrentState, TArray<FPrimaryAssetId>& OutPrimaryAssetIdList);

	/**
	 * Builds an ARFilter struct. You should be using ClassPaths and RecursiveClassPathsExclusionSet, ClassNames and RecursiveClassesExclusionSet are deprecated.
	 * 
	 * @param ClassNames [DEPRECATED] - Class names are now represented by path names. If non-empty, this input will result in a runtime warning. Please use the ClassPaths input instead.
	 * @param RecursiveClassesExclusionSet [DEPRECATED] - Class names are now represented by path names. If non-empty, this input will result in a runtime warning. Please use the RecursiveClassPathsExclusionSet input instead.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities", 
		meta = (Keywords = "construct build", NativeMakeFunc, BlueprintThreadSafe, AdvancedDisplay = "5", 
		AutoCreateRefTerm = "PackageNames, PackagePaths, SoftObjectPaths, ClassNames, ClassPaths, RecursiveClassPathsExclusionSet, RecursiveClassesExclusionSet"))
	static ENGINE_API FARFilter MakeARFilter(
		const TArray<FName>& PackageNames, 
		const TArray<FName>& PackagePaths, 
		const TArray<FSoftObjectPath>& SoftObjectPaths, 
		const TArray<FTopLevelAssetPath>& ClassPaths,
		const TSet<FTopLevelAssetPath>& RecursiveClassPathsExclusionSet, 
		const TArray<FName>& ClassNames, 
		const TSet<FName>& RecursiveClassesExclusionSet, 
		const bool bRecursivePaths = false, 
		const bool bRecursiveClasses = false, 
		const bool bIncludeOnlyOnDiskAssets = false
		);

	/**
	 * Breaks an ARFilter struct into its component pieces. You should be using ClassPaths and RecursiveClassPathsExclusionSet from this node, ClassNames and RecursiveClassesExclusionSet are deprecated.
	 *
	 * @param ClassNames [DEPRECATED] - Class names are now represented by path names. Please use the ClassPaths output instead.
	 * @param RecursiveClassesExclusionSet [DEPRECATED] - Class names are now represented by path names. Please use the RecursiveClassPathsExclusionSet output instead.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities", meta = (NativeBreakFunc, BlueprintThreadSafe, AdvancedDisplay = "6"))
	static ENGINE_API void BreakARFilter(
		FARFilter InARFilter,
		TArray<FName>& PackageNames,
		TArray<FName>& PackagePaths,
		TArray<FSoftObjectPath>& SoftObjectPaths,
		TArray<FTopLevelAssetPath>& ClassPaths,
		TSet<FTopLevelAssetPath>& RecursiveClassPathsExclusionSet,
		TArray<FName>& ClassNames,
		TSet<FName>& RecursiveClassesExclusionSet,
		bool& bRecursivePaths,
		bool& bRecursiveClasses,
		bool& bIncludeOnlyOnDiskAssets
		);

};



//////////////////////////////////////////////////////////////////////////
// UKismetSystemLibrary inlines


inline bool UKismetSystemLibrary::IsValid(const UObject* Object)
{
	return ::IsValid(Object);
}

inline bool UKismetSystemLibrary::IsValidClass(UClass* Class)
{
	return ::IsValid(Class);
}

inline int32 UKismetSystemLibrary::MakeLiteralInt(int32 Value)
{
	return Value;
}

inline int64 UKismetSystemLibrary::MakeLiteralInt64(int64 Value)
{
	return Value;
}

inline float UKismetSystemLibrary::MakeLiteralFloat(float Value)
{
	return Value;
}

inline double UKismetSystemLibrary::MakeLiteralDouble(double Value)
{
	return Value;
}

inline bool UKismetSystemLibrary::MakeLiteralBool(bool Value)
{
	return Value;
}

inline FName UKismetSystemLibrary::MakeLiteralName(FName Value)
{
	return Value;
}

inline uint8 UKismetSystemLibrary::MakeLiteralByte(uint8 Value)
{
	return Value;
}

inline FString UKismetSystemLibrary::MakeLiteralString(FString Value)
{
	return Value;
}

inline FText UKismetSystemLibrary::MakeLiteralText(FText Value)
{
	return Value;
}
