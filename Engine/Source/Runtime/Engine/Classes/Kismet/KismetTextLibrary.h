// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Internationalization/PolyglotTextData.h"
#include "KismetTextLibrary.generated.h"

class FTextProperty;

#if !CPP
/** This code is only used for meta-data extraction and the main declaration is with FText, be sure to update there as well */

/** Provides rounding modes for converting numbers into localized text */
UENUM(BlueprintType)
enum ERoundingMode : int
{
	/** Rounds to the nearest place, equidistant ties go to the value which is closest to an even value: 1.5 becomes 2, 0.5 becomes 0 */
	HalfToEven,
	/** Rounds to nearest place, equidistant ties go to the value which is further from zero: -0.5 becomes -1.0, 0.5 becomes 1.0 */
	HalfFromZero,
	/** Rounds to nearest place, equidistant ties go to the value which is closer to zero: -0.5 becomes 0, 0.5 becomes 0. */
	HalfToZero,
	/** Rounds to the value which is further from zero, "larger" in absolute value: 0.1 becomes 1, -0.1 becomes -1 */
	FromZero,
	/** Rounds to the value which is closer to zero, "smaller" in absolute value: 0.1 becomes 0, -0.1 becomes 0 */
	ToZero,
	/** Rounds to the value which is more negative: 0.1 becomes 0, -0.1 becomes -1 */
	ToNegativeInfinity,
	/** Rounds to the value which is more positive: 0.1 becomes 1, -0.1 becomes 0 */
	ToPositiveInfinity,
};

UENUM(BlueprintType)
enum EMemoryUnitStandard : int
{
	/* International Electrotechnical Commission (MiB) 1024-based */
	IEC,
	/* International System of Units 1000-based */
	SI
};

UENUM(BlueprintType)
enum class EMemoryUnit : uint8
{
	Unit_Bytes,

	/* Auto-select based on magnitude using IEC (1024-based) */
	Auto_IEC,

	/* IEC (1024-based) */
	Unit_KiB, Unit_MiB, Unit_GiB, Unit_TiB, Unit_PiB, Unit_EiB,

	/* Auto-select based on magnitude using SI (1000-based) */
	Auto_SI,

	/* SI (1000-based) */
	Unit_KB, Unit_MB, Unit_GB, Unit_TB, Unit_PB, Unit_EB
};

UENUM(BlueprintType)
enum class ETextGender : uint8
{
	Masculine,
	Feminine,
	Neuter,
};

UENUM(BlueprintType)
namespace EDateTimeStyle
{
	enum Type : int
	{
		Default,
		Short,
		Medium,
		Long,
		Full,
		Custom UMETA(Hidden), // Internal use only
	};
}

UENUM(BlueprintType)
namespace EFormatArgumentType
{
	enum Type : int
	{
		Int,
		UInt,
		Float,
		Double,
		Text,
		Gender,
	};
}

/**
 * Used to pass argument/value pairs into FText::Format.
 * The full C++ struct is located here: Engine\Source\Runtime\Core\Public\Internationalization\Text.h
 */
USTRUCT(noexport, BlueprintInternalUseOnly)
struct FFormatArgumentData
{
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentName)
	FString ArgumentName;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	TEnumAsByte<EFormatArgumentType::Type> ArgumentValueType;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	FText ArgumentValue;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	int64 ArgumentValueInt;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	float ArgumentValueFloat;
	
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	double ArgumentValueDouble;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=ArgumentValue)
	ETextGender ArgumentValueGender;
};
#endif

UCLASS(meta=(BlueprintThreadSafe, ScriptName="TextLibrary"), MinimalAPI)
class UKismetTextLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Converts a vector value to localized formatted text, in the form 'X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Vector)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_VectorToText(FVector InVec);

	/** Converts a vector2d value to localized formatted text, in the form 'X= Y=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Vector2d)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_Vector2dToText(FVector2D InVec);

	/** Converts a rotator value to localized formatted text, in the form 'P= Y= R=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Rotator)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_RotatorToText(FRotator InRot);

	/** Converts a transform value to localized formatted text, in the form 'Translation: X= Y= Z= Rotation: P= Y= R= Scale: X= Y= Z=' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Transform)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_TransformToText(const FTransform& InTrans);

	/** Converts a UObject value to culture invariant text by calling the object's GetName method */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Object)", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_ObjectToText(class UObject* InObj);

	/** Converts a linear color value to localized formatted text, in the form '(R=,G=,B=,A=)' */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (LinearColor)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_ColorToText(FLinearColor InColor);

	/** Converts localizable text to the string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Text)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static ENGINE_API FString Conv_TextToString(const FText& InText);

	/** Converts string to culture invariant text. Use 'Make Literal Text' to create localizable text, or 'Format' if concatenating localized text */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Text (String)", BlueprintAutocast), Category="Utilities|Text")
	static ENGINE_API FText Conv_StringToText(const FString& InString);

	/** Converts Name to culture invariant text */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Text (Name)", BlueprintAutocast), Category="Utilities|Text")
	static ENGINE_API FText Conv_NameToText(FName InName);

	/** Converts string to culture invariant text. Use 'Make Literal Text' to create localizable text, or 'Format' if concatenating localized text */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText MakeInvariantText(const FString& InString);

	/** Returns true if text is empty. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API bool TextIsEmpty(const FText& InText);

	/** Returns true if text is transient. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API bool TextIsTransient(const FText& InText);

	/** Returns true if text is culture invariant. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API bool TextIsCultureInvariant(const FText& InText);

	/**
	 * Transforms the text to lowercase in a culture correct way.
	 * @note The returned instance is linked to the original and will be rebuilt if the active culture is changed.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText TextToLower(const FText& InText);

	/**
	 * Transforms the text to uppercase in a culture correct way.
	 * @note The returned instance is linked to the original and will be rebuilt if the active culture is changed.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText TextToUpper(const FText& InText);

	/** Removes whitespace characters from the front of the text. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText TextTrimPreceding(const FText& InText);

	/** Removes trailing whitespace characters. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText TextTrimTrailing(const FText& InText);

	/** Removes whitespace characters from the front and end of the text. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText TextTrimPrecedingAndTrailing(const FText& InText);

	/** Returns an empty piece of text. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText GetEmptyText();

	/**
	  * === !! This is an ADVANCED function. USE WITH CAUTION !! ===
	  *
	  * Attempt to dynamically reference an EXISTING Text via its active display string in the live table.
	  * Note: This can ONLY find text that is currently localized (gathered, translated, and has an active display string in TextLocalizationManager). If you need to find a localizable but untranslated text, see 'Make Literal Text'.
	  * Note: Direct dynamic references to Text are EXTREMELY FRAGILE, and you may want to use a string table instead!
	  *
	  * @param Namespace The namespace of the text to find (if any).
	  * @param Key The key of the text to find.
	  * @param SourceString If set (not empty) then the found text must also have been created from this source string.
	  */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", DisplayName="Find Text in Live Table (Advanced)", meta = (AdvancedDisplay = "SourceString", ScriptName="FindTextInLiveTable_Advanced;FindTextInLocalizationTable"))
	static ENGINE_API bool FindTextInLocalizationTable(const FString& Namespace, const FString& Key, FText& OutText, const FString& SourceString = TEXT(""));

	/** Returns true if A and B are linguistically equal (A == B). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal Exactly (Text)", CompactNodeTitle = "==="), Category="Utilities|Text")
	static ENGINE_API bool EqualEqual_TextText(const FText& A, const FText& B);

	/** Returns true if A and B are linguistically equal (A == B), ignoring case. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal, Case Insensitive (Text)", CompactNodeTitle = "=="), Category="Utilities|Text")
	static ENGINE_API bool EqualEqual_IgnoreCase_TextText(const FText& A, const FText& B);
				
	/** Returns true if A and B are linguistically not equal (A != B). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal Exactly (Text)", CompactNodeTitle = "!=="), Category="Utilities|Text")
	static ENGINE_API bool NotEqual_TextText(const FText& A, const FText& B);

	/** Returns true if A and B are linguistically not equal (A != B), ignoring case. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Not Equal, Case Insensitive (Text)", CompactNodeTitle = "!="), Category="Utilities|Text")
	static ENGINE_API bool NotEqual_IgnoreCase_TextText(const FText& A, const FText& B);

	/** Converts a boolean value to formatted text, either 'true' or 'false' */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Text (Boolean)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|Text")
	static ENGINE_API FText Conv_BoolToText(bool InBool);

	/** Converts a byte value to formatted text */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Text (Byte)", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|Text")
	static ENGINE_API FText Conv_ByteToText(uint8 Value);

	// Default values are duplicated from FNumberFormattingOptions and should be replicated in all functions and in the struct when changed!
	/** Converts a passed in integer to text based on formatting options */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To Text (Integer)", AdvancedDisplay = "1", BlueprintAutocast), Category="Utilities|Text")
	static ENGINE_API FText Conv_IntToText(int32 Value, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324);

	/** Converts a passed in integer to text based on formatting options */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Integer64)", AdvancedDisplay = "1", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_Int64ToText(int64 Value, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324);

	/** Converts a passed in double to text based on formatting options */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To Text (Float)", AdvancedDisplay = "1", BlueprintAutocast), Category = "Utilities|Text")
	static ENGINE_API FText Conv_DoubleToText(double Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3);
	
	/**
	 * Generate an FText that represents the passed number as currency in the current culture.
	 * BaseVal is specified in the smallest fractional value of the currency and will be converted for formatting according to the selected culture.
	 * Keep in mind the CurrencyCode is completely independent of the culture it's displayed in (and they do not imply one another).
	 * For example: FText::AsCurrencyBase(650, TEXT("EUR")); would return an FText of "<EUR>6.50" in most English cultures (en_US/en_UK) and "6,50<EUR>" in Spanish (es_ES) (where <EUR> is U+20AC)
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "As Currency"), Category = "Utilities|Text")
	static ENGINE_API FText AsCurrencyBase(int32 BaseValue, const FString& CurrencyCode);

	// Default values are duplicated from FNumberFormattingOptions and should be replicated in all functions and in the struct when changed!
	/** Converts a passed in integer to a text formatted as a currency */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "AsCurrency (integer) - DEPRECATED (use AsCurrency)"), Category="Utilities|Text")
	static ENGINE_API FText AsCurrency_Integer(int32 Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3, const FString& CurrencyCode = TEXT(""));

	// Default values are duplicated from FNumberFormattingOptions and should be replicated in all functions and in the struct when changed!
	/** Converts a passed in float to a text formatted as a currency */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "AsCurrency (float) - DEPRECATED (use AsCurrency)"), Category="Utilities|Text")
	static ENGINE_API FText AsCurrency_Float(float Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3, const FString& CurrencyCode = TEXT(""));

	// Default values are duplicated from FNumberFormattingOptions and should be replicated in all functions and in the struct when changed!
	/** Converts a passed in float to a text, formatted as a percent */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Percent", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsPercent_Float(float Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign = false, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3);

	/** Converts a passed in date & time to a text, formatted as a date using an invariant timezone. This will use the given date & time as-is, so it's assumed to already be in the correct timezone. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Date", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsDate_DateTime(const FDateTime& InDateTime, TEnumAsByte<EDateTimeStyle::Type> InDateStyle = EDateTimeStyle::Default);

	/** Converts a passed in date & time to a text, formatted as a date using the given timezone (default is the local timezone). This will convert the given date & time from UTC to the given timezone (taking into account DST). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Date (from UTC)", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsTimeZoneDate_DateTime(const FDateTime& InDateTime, const FString& InTimeZone = TEXT(""), TEnumAsByte<EDateTimeStyle::Type> InDateStyle = EDateTimeStyle::Default);

	/** Converts a passed in date & time to a text, formatted as a date & time using an invariant timezone. This will use the given date & time as-is, so it's assumed to already be in the correct timezone. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As DateTime", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsDateTime_DateTime(const FDateTime& In, TEnumAsByte<EDateTimeStyle::Type> InDateStyle = EDateTimeStyle::Default, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle = EDateTimeStyle::Default);

	/** Converts a passed in date & time to a text, formatted as a date & time using the given timezone (default is the local timezone). This will convert the given date & time from UTC to the given timezone (taking into account DST). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As DateTime (from UTC)", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsTimeZoneDateTime_DateTime(const FDateTime& InDateTime, const FString& InTimeZone = TEXT(""), TEnumAsByte<EDateTimeStyle::Type> InDateStyle = EDateTimeStyle::Default, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle = EDateTimeStyle::Default);

	/** Converts a passed in date & time to a text, formatted as a time using an invariant timezone. This will use the given date & time as-is, so it's assumed to already be in the correct timezone. */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Time", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsTime_DateTime(const FDateTime& In, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle = EDateTimeStyle::Default);

	/** Converts a passed in date & time to a text, formatted as a time using the given timezone (default is the local timezone). This will convert the given date & time from UTC to the given timezone (taking into account DST). */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Time (from UTC)", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsTimeZoneTime_DateTime(const FDateTime& InDateTime, const FString& InTimeZone = TEXT(""), TEnumAsByte<EDateTimeStyle::Type> InTimeStyle = EDateTimeStyle::Default);

	/** Converts a passed in time span to a text, formatted as a time span */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "As Timespan", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsTimespan_Timespan(const FTimespan& InTimespan);

	/** Generate an FText that represents the passed number as a memory size in the current culture */
	UFUNCTION(BlueprintPure, meta=(AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsMemory(int64 NumBytes, TEnumAsByte<EMemoryUnitStandard> UnitStandard = EMemoryUnitStandard::IEC, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3);

	/** Generate an FText that represents the passed number as a memory size in a specific unit. Use Auto_IEC or Auto_SI to select automatically by magnitude. */
	UFUNCTION(BlueprintPure, meta=(DisplayName="As Memory (Specific Unit)", AdvancedDisplay = "1"), Category="Utilities|Text")
	static ENGINE_API FText AsMemoryUnit(int64 NumBytes, EMemoryUnit Unit = EMemoryUnit::Auto_IEC, bool bUseGrouping = true, int32 MinimumIntegralDigits = 1, int32 MaximumIntegralDigits = 324, int32 MinimumFractionalDigits = 0, int32 MaximumFractionalDigits = 3);

	/** Used for formatting text using the FText::Format function and utilized by the UK2Node_FormatText */
	UFUNCTION(BlueprintPure, meta=(BlueprintInternalUseOnly = "true"))
	static ENGINE_API FText Format(FText InPattern, TArray<FFormatArgumentData> InArgs);

	/** Returns true if the given text is referencing a string table. */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(DisplayName="Is Text from String Table"))
	static ENGINE_API bool TextIsFromStringTable(const FText& Text);

	/**
	 * Attempts to create a text instance from a string table ID and key.
	 * @note This exists to allow programmatic look-up of a string table entry from dynamic content - you should favor setting your string table reference on a text property or pin wherever possible as it is significantly more robust (see "Make Literal Text").
	 * @return The found text, or a dummy text if the entry could not be found.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(DisplayName="Make Text from String Table (Advanced)"))
	static ENGINE_API FText TextFromStringTable(const FName TableId, const FString& Key);

	/**
	 * Attempts to get the String Table ID and key used by the given text.
	 * @return True if the String Table ID and key were found, false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(DisplayName="Get String Table ID and Key from Text"))
	static ENGINE_API bool StringTableIdAndKeyFromText(FText Text, FName& OutTableId, FString& OutKey);

	/**
	 * Attempts to get the ID (namespace and key) used by the given text.
	 * @return True if the namespace (which may be empty) and key were found, false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(DisplayName="Get ID from Text"))
	static ENGINE_API bool GetTextId(FText Text, FString& OutNamespace, FString& OutKey);

	/**
	 * Get the (non-localized) source string of the given text.
	 * @note For a generated text (eg, the result of a Format), this will deep build the source string as if the generation had run for the native language.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text", meta=(DisplayName="Get Source String from Text"))
	static ENGINE_API FString GetTextSourceString(FText Text);

	/**
	 * Check whether the given polyglot data is valid.
	 * @return True if the polyglot data is valid, false otherwise. ErrorMessage will be filled in if the the data is invalid.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API void IsPolyglotDataValid(const FPolyglotTextData& PolyglotData, bool& IsValid, FText& ErrorMessage);

	/**
	 * Get the text instance created from this polyglot data.
	 * @return The text instance, or an empty text if the data is invalid.
	 */
	UFUNCTION(BlueprintPure, Category="Utilities|Text")
	static ENGINE_API FText PolyglotDataToText(const FPolyglotTextData& PolyglotData);

	/**
	 * Edit the source string of the given text property, akin to what happens when editing a text property in a details panel.
	 * This will attempt to preserve the existing ID of the text property being edited, or failing that will attempt to build a deterministic ID based on the object and property info.
	 * 
	 * @note This is an ADVANCED function that is ONLY safe to be used in environments where the modified text property will be gathered for localization (eg, in the editor, or a game mode that collects text properties to be localized).
	 * 
	 * @param TextOwner			The object that owns the given Text to be edited.
	 * @param Text				The text property to edit. This must be a property that exists on TextOwner.
	 * @param SourceString		The source string that the edited text property should use.
	 * @param bEmitChangeNotify	True to emit a pre/post edit change notify (in the editor). You should set this to false if editing text within a construction script.
	 * 
	 * @return True if edit was possible, or false if not.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Utilities|Text", meta=(DefaultToSelf="TextOwner"))
	static ENGINE_API bool EditTextSourceString(UObject* TextOwner, UPARAM(ref) FText& Text, const FString& SourceString, const bool bEmitChangeNotify = true);
	DECLARE_FUNCTION(execEditTextSourceString);

	/**
	 * Edit the source string of the given text property, akin to what happens when editing a text property in a details panel.
	 * This will attempt to preserve the existing ID of the text property being edited, or failing that will attempt to build a deterministic ID based on the object and property info.
	 * 
	 * @note This is an ADVANCED function that is ONLY safe to be used in environments where the modified text property will be gathered for localization (eg, in the editor, or a game mode that collects text properties to be localized).
	 * 
	 * @param TextOwner			The object that owns the given Text to be edited.
	 * @param PropertyName		The name of the text property to edit. This must be a property that exists on TextOwner.
	 * @param SourceString		The source string that the edited text property should use.
	 * @param bEmitChangeNotify	True to emit a pre/post edit change notify (in the editor). You should set this to false if editing text within a construction script.
	 * 
	 * @return True if edit was possible, or false if not.
	 */
	UFUNCTION(BlueprintCallable, Category="Utilities|Text", meta=(DefaultToSelf="TextOwner"))
	static ENGINE_API bool EditTextPropertySourceString(UObject* TextOwner, const FName PropertyName, const FString& SourceString, const bool bEmitChangeNotify = true);
	static bool Generic_EditTextPropertySourceString(UObject* TextOwner, const FTextProperty* TextProperty, const FString& SourceString, const bool bEmitChangeNotify);

	/** Converts a generic value to localized formatted text using the user culture. */
	UFUNCTION(BlueprintPure, CustomThunk, DisplayName = "To Text", Category = "Utilities|Text", meta = (CustomStructureParam = "Value", BlueprintInternalUseOnly = "true"))
	static ENGINE_API FText Conv_NumericPropertyToText(UPARAM(ref) const int32& Value);
	DECLARE_FUNCTION(execConv_NumericPropertyToText);
};
