// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Layer.generated.h"

USTRUCT()
struct FLayerActorStats
{
	GENERATED_USTRUCT_BODY()

	/** A Type of Actor currently associated with the Layer */
	UPROPERTY()
	TObjectPtr<UClass> Type;

	/** The total number of Actors of Type assigned to the Layer */
	UPROPERTY()
	int32 Total;

	FLayerActorStats()
		: Type(NULL)
		, Total(0)
	{
	}
};
 
UCLASS(MinimalAPI)
class ULayer : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API void SetLayerName(FName InName);	
	ENGINE_API FName GetLayerName() const;

	ENGINE_API void SetVisible(bool bIsVisible);
	ENGINE_API bool IsVisible() const;

	ENGINE_API const TArray<FLayerActorStats>& GetActorStats() const;
	ENGINE_API void ClearActorStats();
	ENGINE_API void AddToStats(AActor* Actor);
	ENGINE_API bool RemoveFromStats(AActor* Actor);

private:
	/** The display name of the layer */
	UPROPERTY()
	FName LayerName;

	/** Whether actors associated with the layer are visible in the viewport */
	UPROPERTY()
	uint32 bIsVisible:1;

	/** 
	 * Basic stats regarding the number of Actors and their types currently assigned to the Layer 
	 */
	UPROPERTY(Transient)
	TArray<FLayerActorStats> ActorStats;
};
