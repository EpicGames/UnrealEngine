// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Collection of surfaces in a single static lighting mapping.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "LightmappedSurfaceCollection.generated.h"

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI)
class ULightmappedSurfaceCollection : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The UModel these surfaces come from. */
	UPROPERTY(EditAnywhere, Category=LightmappedSurfaceCollection)
	TObjectPtr<class UModel> SourceModel;

	/** An array of the surface indices grouped into a single static lighting mapping. */
	UPROPERTY(EditAnywhere, Category=LightmappedSurfaceCollection)
	TArray<int32> Surfaces;

};

