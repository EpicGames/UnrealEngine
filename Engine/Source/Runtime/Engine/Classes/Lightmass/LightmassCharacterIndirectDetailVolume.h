// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// LightmassCharacterIndirectDetailVolume:  Defines areas where Lightmass should place more indirect lighting samples than normal.
//=============================================================================
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "LightmassCharacterIndirectDetailVolume.generated.h"

UCLASS(hidecategories=(Collision, Brush, Attachment, Physics, Volume), MinimalAPI)
class ALightmassCharacterIndirectDetailVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

};



