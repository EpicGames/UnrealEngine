// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// LightmassImportanceVolume:  a bounding volume outside of which Lightmass
// photon emissions are decreased
//=============================================================================
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "LightmassImportanceVolume.generated.h"

UCLASS(hidecategories=(Collision, Brush, Attachment, Physics, Volume), MinimalAPI)
class ALightmassImportanceVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
#endif // WITH_EDITOR
};



