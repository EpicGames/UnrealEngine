// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LightmassPortal.generated.h"

class UBillboardComponent;

UCLASS(hidecategories=(Collision, Attachment, Actor), MinimalAPI)
class ALightmassPortal : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = Portal, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class ULightmassPortalComponent> PortalComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif // WITH_EDITORONLY_DATA

public:	

	virtual bool IsLevelBoundsRelevant() const override { return false; }

	class ULightmassPortalComponent* GetPortalComponent() const { return PortalComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif

#if WITH_EDITOR
	virtual void PostEditMove(bool bFinished) override;
#endif // WITH_EDITOR
};



