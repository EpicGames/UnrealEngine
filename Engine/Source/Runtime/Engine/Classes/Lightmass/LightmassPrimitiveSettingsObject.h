// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 *	Primitive settings for Lightmass
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"

#include "LightmassPrimitiveSettingsObject.generated.h"

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI,collapsecategories)
class ULightmassPrimitiveSettingsObject : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Lightmass)
	struct FLightmassPrimitiveSettings LightmassSettings;

};

