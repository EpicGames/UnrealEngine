// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// PrecomputedVisibilityOverrideVolume:  Overrides visibility for a set of actors
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "PrecomputedVisibilityOverrideVolume.generated.h"

UCLASS(hidecategories=(Collision, Brush, Attachment, Physics, Volume), MinimalAPI)
class APrecomputedVisibilityOverrideVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	/** Array of actors that will always be considered visible by Precomputed Visibility when viewed from inside this volume. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PrecomputedVisibilityOverrideVolume)
	TArray<TObjectPtr<class AActor>> OverrideVisibleActors;

	/** Array of actors that will always be considered invisible by Precomputed Visibility when viewed from inside this volume. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PrecomputedVisibilityOverrideVolume)
	TArray<TObjectPtr<class AActor>> OverrideInvisibleActors;

	/** Array of level names whose actors will always be considered invisible by Precomputed Visibility when viewed from inside this volume. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PrecomputedVisibilityOverrideVolume)
	TArray<FName> OverrideInvisibleLevels;
};



