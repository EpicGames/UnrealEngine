// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// PrecomputedVisibilityVolume:  Indicates where visibility should be precomputed
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "PrecomputedVisibilityVolume.generated.h"

UCLASS(hidecategories=(Collision, Brush, Attachment, Physics, Volume), MinimalAPI)
class APrecomputedVisibilityVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

};



