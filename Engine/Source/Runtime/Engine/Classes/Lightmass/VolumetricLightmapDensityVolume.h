// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// VolumetricLightmapDensityVolume
//=============================================================================
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "VolumetricLightmapDensityVolume.generated.h"

/** Provides local control over volumetric lightmap density.  Only convex shapes supported. */
UCLASS(hidecategories=(Collision, Brush, Attachment, Physics, Volume), MinimalAPI)
class AVolumetricLightmapDensityVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	/**
	 * The Volumetric Lightmap has 3 mipmaps, where the highest density mipmap (mip0) corresponds to VolumetricLightmapDetailCellSize.
	 * mip0: DetailCellSize
	 * mip1: DetailCellSize * 4
	 * mip2: DetailCellSize * 16
	 * 
	 * By default, highest density will be placed around static geometry and static lights, but AllowedMipLevelRange can be used to override this behavior. 
	 * Examples:
	 * [0, 3] = Volume does nothing
	 * [1, 3] = Volume removes highest density mip, even near static geometry, which can be useful to save memory ('stat MapBuildData')
	 * [0, 0] = Volume forces highest density.  Warning: using this on a large area can greatly increase memory and build times!
	 * When multiple volumes overlap, the smallest (highest density) values will be used.
	 */
	UPROPERTY(EditAnywhere, Category = VolumetricLightmapDensityVolume)
	FInt32Interval AllowedMipLevelRange;
};



