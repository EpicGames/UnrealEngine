// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MeshVertexPainter.generated.h"

class UStaticMeshComponent;

UENUM()
enum class EVertexPaintAxis : uint8
{
	X,
	Y,
	Z
};


class FMeshVertexPainter
{
public:
	/**
	 * Paints a single vertex color to the given mesh component.
	 *
	 * @param	StaticMeshComponent		The static mesh component to be painted
	 * @param	FillColor				The color to be applied.
	 * @param	bConvertToSRGB			Whether the vertex color should be converted to sRGB.
	 */
	static ENGINE_API void PaintVerticesSingleColor(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& FillColor, bool bConvertToSRGB = true);

	/**
	 * Paints vertex colors on a mesh component lerping from the start to the end color along the specified axis.
	 *
	 * @param	StaticMeshComponent		The static mesh component to be painted
	 * @param	StartColor				The color to paint at the minimum bound
	 * @param	EndColor				The color to paint at the maximum bound
	 * @param	Axis					The mesh local space axis along which to lerp
	 * @param	bConvertToSRGB			Whether the vertex color should be converted to sRGB.
	 */
	static ENGINE_API void PaintVerticesLerpAlongAxis(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& StartColor, const FLinearColor& EndColor, EVertexPaintAxis Axis, bool bConvertToSRGB = true);

	/**
	 * Removes painted vertex colors from the given mesh component.
	 *
	 * @param	StaticMeshComponent		The static mesh component to have its vertex colors removed
	 */
	static ENGINE_API void RemovePaintedVertices(UStaticMeshComponent* StaticMeshComponent);
};
