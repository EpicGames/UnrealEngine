// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MeshVertexPainterKismetLibrary.generated.h"

class UStaticMeshComponent;
enum class EVertexPaintAxis : uint8;

UCLASS(MinimalAPI, meta=(ScriptName="MeshVertexPainterLibrary"))
class UMeshVertexPainterKismetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Paints vertex colors on a mesh component in a specified color. */
	UFUNCTION(BlueprintCallable, Category = "VertexPaint")
	static void PaintVerticesSingleColor(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& FillColor, bool bConvertToSRGB = true);

	/** Paints vertex colors on a mesh component lerping from the start to the end color along the specified axis. */
	UFUNCTION(BlueprintCallable, Category = "VertexPaint")
	static void PaintVerticesLerpAlongAxis(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& StartColor, const FLinearColor& EndColor, EVertexPaintAxis Axis, bool bConvertToSRGB = true);

	/** Removes vertex colors on a mesh component */
	UFUNCTION(BlueprintCallable, Category = "VertexPaint")
	static void RemovePaintedVertices(UStaticMeshComponent* StaticMeshComponent);
};
