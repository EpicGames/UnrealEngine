// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "HandlerComponentFactory.h"

#include "EngineHandlerComponentFactory.generated.h"

class HandlerComponent;

/**
 * Factory class for loading HandlerComponent's contained within Engine
 */
UCLASS()
class UEngineHandlerComponentFactory : public UHandlerComponentFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};
