// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "ParticleModuleAcceleration.generated.h"

class UParticleLODLevel;
class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Acceleration"), MinimalAPI)
class UParticleModuleAcceleration : public UParticleModuleAccelerationBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The initial acceleration of the particle.
	 *	Value is obtained using the EmitterTime at particle spawn.
	 *	Each frame, the current and base velocity of the particle 
	 *	is then updated using the formula 
	 *		velocity += acceleration * DeltaTime
	 *	where DeltaTime is the time passed since the last frame.
	 */
	UPROPERTY(EditAnywhere, Category=Acceleration)
	struct FRawDistributionVector Acceleration;

	/**
	 *	If true, then apply the particle system components scale 
	 *	to the acceleration value.
	 */
	UPROPERTY(EditAnywhere, Category=Acceleration)
	uint32 bApplyOwnerScale:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//End UParticleModule Interface

protected:
	friend class FParticleModuleAccelerationDetails;
};



