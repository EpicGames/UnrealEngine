// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleAccelerationBase.generated.h"

class UParticleEmitter;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Acceleration"), abstract)
class UParticleModuleAccelerationBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

	/**
	 *	If true, then treat the acceleration as world-space
	 */
	UPROPERTY(EditAnywhere, Category=Acceleration)
	uint8 bAlwaysInWorldSpace : 1;

	//~ Begin UParticleModule Interface
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//~ End UParticleModule Interface
};

