// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleAccelerationConstant: Constant particle acceleration.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "ParticleModuleAccelerationConstant.generated.h"

struct FParticleEmitterInstance;

UCLASS(MinimalAPI, editinlinenew, hidecategories=(Object, Acceleration), meta=(DisplayName = "Const Acceleration"))
class UParticleModuleAccelerationConstant : public UParticleModuleAccelerationBase
{
	GENERATED_UCLASS_BODY()

	/** Constant acceleration for particles in this system. */
	UPROPERTY(EditAnywhere, Category=ParticleModuleAccelerationConstant)
	FVector Acceleration;


	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	//End UParticleModule Interface
};



