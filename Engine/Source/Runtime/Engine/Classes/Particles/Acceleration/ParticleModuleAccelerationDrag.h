// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleAccelerationDrag: Drag coefficient.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "ParticleModuleAccelerationDrag.generated.h"

class UParticleLODLevel;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=(Object, Acceleration), MinimalAPI, meta=(DisplayName = "Drag"))
class UParticleModuleAccelerationDrag : public UParticleModuleAccelerationBase
{
	GENERATED_UCLASS_BODY()

	/** Per-particle drag coefficient. Evaluted using emitter time. */
	UPROPERTY()
	TObjectPtr<class UDistributionFloat> DragCoefficient_DEPRECATED;

	/** Per-particle drag coefficient. Evaluted using emitter time. */
	UPROPERTY(EditAnywhere, Category = Drag)
	FRawDistributionFloat DragCoefficientRaw;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Update(const FUpdateContext& Context) override;
	//End UParticleModule Interface

	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

protected:
	friend class FParticleModuleAccelerationDragDetails;
};



