// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleAccelerationDragScaleOverLife: Drag scale over lifetime.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "ParticleModuleAccelerationDragScaleOverLife.generated.h"

class UParticleLODLevel;

UCLASS(editinlinenew, hidecategories=(UObject, Acceleration), MinimalAPI, meta=(DisplayName = "Drag Scale/Life"))
class UParticleModuleAccelerationDragScaleOverLife : public UParticleModuleAccelerationBase
{
	GENERATED_UCLASS_BODY()

	/** Per-particle drag scale. Evaluted using particle relative time. */
	UPROPERTY()
	TObjectPtr<class UDistributionFloat> DragScale_DEPRECATED;

	/** Per-particle drag scale. Evaluted using particle relative time. */
	UPROPERTY(EditAnywhere, Category = Drag)
	FRawDistributionFloat DragScaleRaw;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	//End UParticleModule Interface

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

protected:
	friend class FParticleModuleAccelerationDragScaleOverLifeDetails;
};



