// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "ParticleModuleAccelerationOverLifetime.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Acceleration/Life"), MinimalAPI)
class UParticleModuleAccelerationOverLifetime : public UParticleModuleAccelerationBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The acceleration of the particle over its lifetime.
	 *	Value is obtained using the RelativeTime of the partice.
	 *	The current and base velocity values of the particle 
	 *	are then updated using the formula 
	 *		velocity += acceleration* DeltaTime
	 *	where DeltaTime is the time passed since the last frame.
	 */
	UPROPERTY(EditAnywhere, Category=Acceleration)
	struct FRawDistributionVector AccelOverLife;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Update(const FUpdateContext& Context) override;
	//End UParticleModule Interface
};



