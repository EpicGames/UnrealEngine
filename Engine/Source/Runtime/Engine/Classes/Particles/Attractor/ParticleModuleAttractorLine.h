// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Attractor/ParticleModuleAttractorBase.h"
#include "ParticleModuleAttractorLine.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Line Attractor"))
class UParticleModuleAttractorLine : public UParticleModuleAttractorBase
{
	GENERATED_UCLASS_BODY()

	/** The first endpoint of the line. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	FVector EndPoint0;

	/** The second endpoint of the line. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	FVector EndPoint1;

	/** The range of the line attractor. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Range;

	/** The strength of the line attractor. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Strength;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Update(const FUpdateContext& Context) override;
	virtual void Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



