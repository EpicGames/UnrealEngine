// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Attractor/ParticleModuleAttractorBase.h"
#include "ParticleModuleAttractorParticle.generated.h"

class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

UENUM()
enum EAttractorParticleSelectionMethod : int
{
	EAPSM_Random UMETA(DisplayName="Random"),
	EAPSM_Sequential UMETA(DisplayName="Sequential"),
	EAPSM_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Particle Attractor"))
class UParticleModuleAttractorParticle : public UParticleModuleAttractorBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The source emitter for attractors
	 */
	UPROPERTY(EditAnywhere, export, noclear, Category=Attractor)
	FName EmitterName;

	/**
	 *	The radial range of the attraction around the source particle.
	 *	Particle-life relative.
	 */
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Range;

	/**
	 *	The strength curve is a function of distance or of time.
	 */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bStrengthByDistance:1;

	/**
	 *	The strength of the attraction (negative values repel).
	 *	Particle-life relative if StrengthByDistance is false.
	 */
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Strength;

	/**	If true, the velocity adjustment will be applied to the base velocity.	*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bAffectBaseVelocity:1;

	/**
	 *	The method to use when selecting an attractor target particle from the emitter.
	 *	One of the following:
	 *	Random		- Randomly select a particle from the source emitter.  
	 *	Sequential  - Select a particle using a sequential order. 
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	TEnumAsByte<enum EAttractorParticleSelectionMethod> SelectionMethod;

	/**
	 *	Whether the particle should grab a new particle if it's source expires.
	 */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bRenewSource:1;

	/**
	 *	Whether the particle should inherit the source veloctiy if it expires.
	 */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bInheritSourceVel:1;

	UPROPERTY()
	int32 LastSelIndex;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	//End UParticleModule Interface
};



