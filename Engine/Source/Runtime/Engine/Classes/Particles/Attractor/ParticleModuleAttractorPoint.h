// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Attractor/ParticleModuleAttractorBase.h"
#include "ParticleModuleAttractorPoint.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Point Attractor"))
class UParticleModuleAttractorPoint : public UParticleModuleAttractorBase
{
	GENERATED_UCLASS_BODY()

	/**	The position of the point attractor from the source of the emitter.		*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionVector Position;

	/**	The radial range of the attractor.										*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Range;

	/**	The strength of the point attractor.									*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	struct FRawDistributionFloat Strength;

	/**	The strength curve is a function of distance or of time.				*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 StrengthByDistance:1;

	/**	If true, the velocity adjustment will be applied to the base velocity.	*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bAffectBaseVelocity:1;

	/**	If true, set the velocity.												*/
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bOverrideVelocity:1;

	/**	If true, treat the position as world space.  So don't transform the the point to localspace. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 bUseWorldSpacePosition:1;

	/** Whether particles can move along the positive X axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Positive_X:1;

	/** Whether particles can move along the positive Y axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Positive_Y:1;

	/** Whether particles can move along the positive Z axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Positive_Z:1;

	/** Whether particles can move along the negative X axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Negative_X:1;

	/** Whether particles can move along the negative Y axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Negative_Y:1;

	/** Whether particles can move along the negative Z axis. */
	UPROPERTY(EditAnywhere, Category=Attractor)
	uint32 Negative_Z:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Update(const FUpdateContext& Context) override;
	virtual void Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



