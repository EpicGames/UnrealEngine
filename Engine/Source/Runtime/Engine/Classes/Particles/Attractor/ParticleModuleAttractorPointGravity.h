// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleAttractorPointGravity: Causes particles to accelerate towards
		a point in space.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Attractor/ParticleModuleAttractorBase.h"
#include "ParticleModuleAttractorPointGravity.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Point Gravity"))
class UParticleModuleAttractorPointGravity : public UParticleModuleAttractorBase
{
	GENERATED_UCLASS_BODY()

	/** The position of the point gravity source. */
	UPROPERTY(EditAnywhere, Category=PointGravitySource)
	FVector Position;

	/** The distance at which the influence of the point begins to falloff. */
	UPROPERTY(EditAnywhere, Category=PointGravitySource)
	float Radius;

	/** The strength of the point source. */
	UPROPERTY()
	TObjectPtr<class UDistributionFloat> Strength_DEPRECATED;

	/** The strength of the point source. */
	UPROPERTY(EditAnywhere, noclear, Category=PointGravitySource)
	FRawDistributionFloat StrengthRaw;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	virtual void Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



