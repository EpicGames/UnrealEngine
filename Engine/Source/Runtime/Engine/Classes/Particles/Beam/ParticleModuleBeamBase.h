// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleBeamBase
 *
 *	The base class for all Beam modules.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleBeamBase.generated.h"

/** The method to use in determining the source/target. */
UENUM()
enum Beam2SourceTargetMethod : int
{
	/** Default	- use the distribution. 
	 *	This is the fallback for when other modes can't be resolved.
	 */
	PEB2STM_Default UMETA(DisplayName="Default"),
	/** UserSet	- use the user set value. 
	 *	Primarily intended for weapon effects.
	 */
	PEB2STM_UserSet UMETA(DisplayName="User Set"),
	/** Emitter	- use the emitter position as the source/target.
	 */
	PEB2STM_Emitter UMETA(DisplayName="Emitter"),
	/** Particle	- use the particles from a given emitter in the system.		
	 *	The name of the emitter should be set in <Source/Target>Name.
	 */
	PEB2STM_Particle UMETA(DisplayName="Particle"),
	/** Actor		- use the actor as the source/target.
	 *	The name of the actor should be set in <Source/Target>Name.
	 */
	PEB2STM_Actor UMETA(DisplayName="Actor"),
	PEB2STM_MAX,
};

/** The method to use in determining the source/target tangent. */
UENUM()
enum Beam2SourceTargetTangentMethod : int
{
	/** Direct - a direct line between source and target.				 */
	PEB2STTM_Direct UMETA(DisplayName="Direct"),
	/** UserSet	- use the user set value.								 */
	PEB2STTM_UserSet UMETA(DisplayName="User Set"),
	/** Distribution - use the distribution.							 */
	PEB2STTM_Distribution UMETA(DisplayName="Distribution"),
	/** Emitter	- use the emitter direction.							 */
	PEB2STTM_Emitter UMETA(DisplayName="Emitter"),
	PEB2STTM_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Beam"))
class UParticleModuleBeamBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()


	//Begin UParticleModule Interface
	virtual EModuleType	GetModuleType() const override {	return EPMT_Beam;	}

	virtual bool CanTickInAnyThread() override
	{
		return false;
	}
	//End UParticleModule Interface
};



