// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleBeamModifier
 *
 *	This module implements a single modifier for a beam emitter.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/ParticleModule.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "ParticleModuleBeamModifier.generated.h"

class UInterpCurveEdSetup;
class UParticleModuleTypeDataBase;
class UParticleSystemComponent;
struct FCurveEdEntry;
struct FParticleEmitterInstance;

/**	What to modify. */
UENUM()
enum BeamModifierType : int
{
	/** Modify the source of the beam.				*/
	PEB2MT_Source UMETA(DisplayName="Source"),
	/** Modify the target of the beam.				*/
	PEB2MT_Target UMETA(DisplayName="Target"),
	PEB2MT_MAX,
};

USTRUCT()
struct FBeamModifierOptions
{
	GENERATED_USTRUCT_BODY()

	/** If true, modify the value associated with this grouping.	*/
	UPROPERTY(EditAnywhere, Category=BeamModifierOptions)
	uint32 bModify:1;

	/** If true, scale the associated value by the given value.		*/
	UPROPERTY(EditAnywhere, Category=BeamModifierOptions)
	uint32 bScale:1;

	/** If true, lock the modifier to the life of the particle.		*/
	UPROPERTY(EditAnywhere, Category=BeamModifierOptions)
	uint32 bLock:1;


	FBeamModifierOptions()
		: bModify(false)
		, bScale(false)
		, bLock(false)
	{
	}

};

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Beam Modifier"))
class UParticleModuleBeamModifier : public UParticleModuleBeamBase
{
	GENERATED_UCLASS_BODY()

	/**	Whether this module modifies the Source or the Target. */
	UPROPERTY(EditAnywhere, Category=Modifier)
	TEnumAsByte<enum BeamModifierType> ModifierType;

	/** The options associated with the position.								*/
	UPROPERTY(EditAnywhere, Category=Position)
	struct FBeamModifierOptions PositionOptions;

	/** The value to use when modifying the position.							*/
	UPROPERTY(EditAnywhere, Category=Position)
	struct FRawDistributionVector Position;

	/** The options associated with the Tangent.								*/
	UPROPERTY(EditAnywhere, Category=Tangent)
	struct FBeamModifierOptions TangentOptions;

	/** The value to use when modifying the Tangent.							*/
	UPROPERTY(EditAnywhere, Category=Tangent)
	struct FRawDistributionVector Tangent;

	/** If true, don't transform the tangent modifier into the tangent basis.	*/
	UPROPERTY(EditAnywhere, Category=Tangent)
	uint32 bAbsoluteTangent:1;

	/** The options associated with the Strength.								*/
	UPROPERTY(EditAnywhere, Category=Strength)
	struct FBeamModifierOptions StrengthOptions;

	/** The value to use when modifying the Strength.							*/
	UPROPERTY(EditAnywhere, Category=Strength)
	struct FRawDistributionFloat Strength;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End Uobject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual void AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual void GetCurveObjects(TArray<FParticleCurvePair>& OutCurves) override;
	virtual	bool AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries) override;
	virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList) override;
	//End UParticleModule Interface


	// @ todo document
	void	GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
		int32& CurrentOffset, FBeam2TypeDataPayload*& BeamDataPayload, 
		FBeamParticleModifierPayloadData*& SourceModifierPayload,
		FBeamParticleModifierPayloadData*& TargetModifierPayload);

	// @ todo document
	void	GetDataPointerOffsets(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
		int32& CurrentOffset, int32& BeamDataOffset, int32& SourceModifierOffset, 
		int32& TargetModifierOffset);
};



