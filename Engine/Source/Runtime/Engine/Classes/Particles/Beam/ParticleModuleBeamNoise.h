// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleBeamNoise
 *
 *	This module implements noise for a beam emitter.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "ParticleModuleBeamNoise.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Noise"))
class UParticleModuleBeamNoise : public UParticleModuleBeamBase
{
	GENERATED_UCLASS_BODY()

	static const uint32 MaxNoiseTessellation = 500;

	/** Is low frequency noise enabled. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bLowFreq_Enabled:1;

	/** The frequency of noise points. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	int32 Frequency;

	/** 
	 *	If not 0, then the frequency will select a random value in the range
	 *		[Frequency_LowRange..Frequency]
	 */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	int32 Frequency_LowRange;

	/** The noise point ranges. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	struct FRawDistributionVector NoiseRange;

	/** A scale factor that will be applied to the noise range. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	struct FRawDistributionFloat NoiseRangeScale;

	/** 
	 *	If true,  the NoiseRangeScale will be grabbed based on the emitter time.
	 *	If false, the NoiseRangeScale will be grabbed based on the particle time.
	 */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bNRScaleEmitterTime:1;

	/** The speed with which to move each noise point. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	struct FRawDistributionVector NoiseSpeed;

	/** Whether the noise movement should be smooth or 'jerky'. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bSmooth:1;

	/** Default target-point information to use if the beam method is endpoint. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	float NoiseLockRadius;

	/** INTERNAL - Whether the noise points should be locked. */
	UPROPERTY()
	uint32 bNoiseLock:1;

	/** Whether the noise points should be oscillate. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bOscillate:1;

	/** How long the  noise points should be locked - 0.0 indicates forever. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	float NoiseLockTime;

	/** The tension to apply to the tessellated noise line. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	float NoiseTension;

	/** If true, calculate tangents at each noise point. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bUseNoiseTangents:1;

	/** The strength of noise tangents, if enabled. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	struct FRawDistributionFloat NoiseTangentStrength;

	/** The amount of tessellation between noise points. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	int32 NoiseTessellation;

	/** 
	 *	Whether to apply noise to the target point (or end of line in distance mode...)
	 *	If true, the beam could potentially 'leave' the target...
	 */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bTargetNoise:1;

	/** 
	 *	The distance at which to deposit noise points.
	 *	If 0.0, then use the static frequency value.
	 *	If not, distribute noise points at the given distance, up to the static Frequency value.
	 *	At that point, evenly distribute them along the beam.
	 */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	float FrequencyDistance;

	/** If true, apply the noise scale to the beam. */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	uint32 bApplyNoiseScale:1;

	/** 
	 *	The scale factor to apply to noise range.
	 *	The lookup value is determined by dividing the number of noise points present by the 
	 *	maximum number of noise points (Frequency).
	 */
	UPROPERTY(EditAnywhere, Category=LowFreq)
	struct FRawDistributionFloat NoiseScale;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//End UParticleModule Interface

	// @todo document
	void GetNoiseRange(FVector& NoiseMin, FVector& NoiseMax);
};



