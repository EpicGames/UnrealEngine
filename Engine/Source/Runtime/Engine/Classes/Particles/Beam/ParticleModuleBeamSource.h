// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleBeamSource
 *
 *	This module implements a single source for a beam emitter.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "ParticleModuleBeamSource.generated.h"

class UParticleModuleTypeDataBase;
class UParticleSystemComponent;
struct FParticleBeam2EmitterInstance;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Source"))
class UParticleModuleBeamSource : public UParticleModuleBeamBase
{
	GENERATED_UCLASS_BODY()

	/** The method flag. */
	UPROPERTY(EditAnywhere, Category=Source)
	TEnumAsByte<enum Beam2SourceTargetMethod> SourceMethod;

	/** The strength of the tangent from the source point for each beam. */
	UPROPERTY(EditAnywhere, Category=Source)
	FName SourceName;

	/** Whether to treat the as an absolute position in world space. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bSourceAbsolute:1;

	/** Default source-point to use. */
	UPROPERTY(EditAnywhere, Category=Source)
	struct FRawDistributionVector Source;

	/** Whether to lock the source to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bLockSource:1;

	/** The method to use for the source tangent. */
	UPROPERTY(EditAnywhere, Category=Source)
	TEnumAsByte<enum Beam2SourceTargetTangentMethod> SourceTangentMethod;

	/** The tangent for the source point for each beam. */
	UPROPERTY(EditAnywhere, Category=Source)
	struct FRawDistributionVector SourceTangent;

	/** Whether to lock the source to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bLockSourceTangent:1;

	/** The strength of the tangent from the source point for each beam. */
	UPROPERTY(EditAnywhere, Category=Source)
	struct FRawDistributionFloat SourceStrength;

	/** Whether to lock the source to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bLockSourceStength:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual void AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList) override;
	//End UParticleModule Interface


	// @todo document
	void GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
				int32& CurrentOffset, 
				FBeamParticleSourceTargetPayloadData*& ParticleSource,
				FBeamParticleSourceBranchPayloadData*& BranchSource);

	// @todo document
	void GetDataPointerOffsets(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
				int32& CurrentOffset, int32& ParticleSourceOffset, int32& BranchSourceOffset);
						

	// @todo document
	bool ResolveSourceData(const FContext& Context, FParticleBeam2EmitterInstance* BeamInst, 
				FBeam2TypeDataPayload* BeamData, const uint8* ParticleBase, 
				int32& CurrentOffset, int32	ParticleIndex, bool bSpawning,
				FBeamParticleModifierPayloadData* ModifierData);

	int32 LastSelectedParticleIndex;
};



