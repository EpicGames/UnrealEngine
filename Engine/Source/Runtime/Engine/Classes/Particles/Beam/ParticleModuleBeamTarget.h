// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleBeamTarget
 *
 *	This module implements a single target for a beam emitter.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "ParticleModuleBeamTarget.generated.h"

class UParticleSystemComponent;
struct FParticleBeam2EmitterInstance;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Target"))
class UParticleModuleBeamTarget : public UParticleModuleBeamBase
{
	GENERATED_UCLASS_BODY()

	/** The method flag. */
	UPROPERTY(EditAnywhere, Category=Target)
	TEnumAsByte<enum Beam2SourceTargetMethod> TargetMethod;

	/** The target point sources of each beam, when using the end point method. */
	UPROPERTY(EditAnywhere, Category=Target)
	FName TargetName;

	/** Default target-point information to use if the beam method is endpoint. */
	UPROPERTY(EditAnywhere, Category=Target)
	struct FRawDistributionVector Target;

	/** Whether to treat the as an absolute position in world space. */
	UPROPERTY(EditAnywhere, Category=Target)
	uint32 bTargetAbsolute:1;

	/** Whether to lock the Target to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Target)
	uint32 bLockTarget:1;

	/** The method to use for the Target tangent. */
	UPROPERTY(EditAnywhere, Category=Target)
	TEnumAsByte<enum Beam2SourceTargetTangentMethod> TargetTangentMethod;

	/** The tangent for the Target point for each beam. */
	UPROPERTY(EditAnywhere, Category=Target)
	struct FRawDistributionVector TargetTangent;

	/** Whether to lock the Target to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Target)
	uint32 bLockTargetTangent:1;

	/** The strength of the tangent from the Target point for each beam. */
	UPROPERTY(EditAnywhere, Category=Target)
	struct FRawDistributionFloat TargetStrength;

	/** Whether to lock the Target to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Target)
	uint32 bLockTargetStength:1;

	/** Default target-point information to use if the beam method is endpoint. */
	UPROPERTY(EditAnywhere, Category=Target)
	float LockRadius;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList) override;
	//End UParticleModule Interface

	// @todo document
	void	GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
				int32& CurrentOffset, 
				FBeamParticleSourceTargetPayloadData*& ParticleSource);
						
	// @todo document
	bool	ResolveTargetData(const FContext& Context, FParticleBeam2EmitterInstance* BeamInst, 
				FBeam2TypeDataPayload* BeamData, const uint8* ParticleBase, 
				int32& CurrentOffset, int32	ParticleIndex, bool bSpawning,
				FBeamParticleModifierPayloadData* ModifierData);

	int32 LastSelectedParticleIndex;
};



