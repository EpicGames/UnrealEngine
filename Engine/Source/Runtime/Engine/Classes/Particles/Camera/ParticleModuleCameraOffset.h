// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Camera/ParticleModuleCameraBase.h"
#include "ParticleModuleCameraOffset.generated.h"

class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

/**
 *	The update method for the offset
 */
UENUM()
enum EParticleCameraOffsetUpdateMethod : int
{
	EPCOUM_DirectSet UMETA(DisplayName="Direct Set"),
	EPCOUM_Additive UMETA(DisplayName="Additive"),
	EPCOUM_Scalar UMETA(DisplayName="Scalar"),
	EPCOUM_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Camera Offset"), MinimalAPI)
class UParticleModuleCameraOffset : public UParticleModuleCameraBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The camera-relative offset to apply to sprite location
	 */
	UPROPERTY(EditAnywhere, Category=Camera)
	struct FRawDistributionFloat CameraOffset;

	/** If true, the offset will only be processed at spawn time */
	UPROPERTY(EditAnywhere, Category=Camera)
	uint32 bSpawnTimeOnly:1;

	/**
	 * How to update the offset for this module.
	 * DirectSet - Set the value directly (overwrite any previous setting)
	 * Additive  - Add the offset of this module to the existing offset
	 * Scalar    - Scale the existing offset by the value of this module
	 */
	UPROPERTY(EditAnywhere, Category=Camera)
	TEnumAsByte<enum EParticleCameraOffsetUpdateMethod> UpdateMethod;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	//End UParticleModule Interface

	virtual bool CanTickInAnyThread() override;
};

