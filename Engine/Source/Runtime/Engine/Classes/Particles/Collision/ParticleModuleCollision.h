// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "CollisionQueryParams.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Collision/ParticleModuleCollisionBase.h"
#include "ParticleModuleCollision.generated.h"

class UParticleEmitter;
class UParticleLODLevel;
class UParticleModuleTypeDataBase;
struct FHitResult;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Actor Collision"))
class UParticleModuleCollision : public UParticleModuleCollisionBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	How much to `slow' the velocity of the particle after a collision.
	 *	Value is obtained using the EmitterTime at particle spawn.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	struct FRawDistributionVector DampingFactor;

	/**
	 *	How much to `slow' the rotation of the particle after a collision.
	 *	Value is obtained using the EmitterTime at particle spawn.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	struct FRawDistributionVector DampingFactorRotation;

	/**
	 *	The maximum number of collisions a particle can have. 
	 *  Value is obtained using the EmitterTime at particle spawn. 
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	struct FRawDistributionFloat MaxCollisions;

	/**
	 *	What to do once a particles MaxCollisions is reached.
	 *	One of the following:
	 *	EPCC_Kill
	 *		Kill the particle when MaxCollisions is reached
	 *	EPCC_Freeze
	 *		Freeze in place, NO MORE UPDATES
	 *	EPCC_HaltCollisions,
	 *		Stop collision checks, keep updating everything
	 *	EPCC_FreezeTranslation,
	 *		Stop translations, keep updating everything else
	 *	EPCC_FreezeRotation,
	 *		Stop rotations, keep updating everything else
	 *	EPCC_FreezeMovement
	 *		Stop all movement, keep updating
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	TEnumAsByte<enum EParticleCollisionComplete> CollisionCompletionOption;

	/**
	 * Which ObjectTypes to collide with
	 */
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray< TEnumAsByte<enum EObjectTypeQuery> > CollisionTypes;

	/**
	 * ObjectParams created for array of CollisionTypes
	 */
	FCollisionObjectQueryParams ObjectParams;
	

	/** 
	 *	If true, physic will be applied between a particle and the 
	 *	object it collides with. 
	 *	This is one-way - particle --> object. The particle does 
	 *	not have physics applied to it - it just generates an 
	 *	impulse applied to the object it collides with. 
	 * NOTE: having this on prevents the code from running off the game thread.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	uint32 bApplyPhysics:1;

	/** Any trigger volumes that are hit will be ignored. NOTE: This can be turned off if the TrigerVolume physics object type is not in the CollisionTypes array.
	* Turning this off is strongly recommended as having it on prevents the code from running off the game thread.*/
	UPROPERTY(EditAnywhere, Category=Collision)
	uint32 bIgnoreTriggerVolumes:1;

	/** 
	 *	The mass of the particle - for use when bApplyPhysics is true. 
	 *	Value is obtained using the EmitterTime at particle spawn. 
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	struct FRawDistributionFloat ParticleMass;

	/**
	 *	The directional scalar value - used to scale the bounds to 
	 *	'assist' in avoiding inter-penetration or large gaps.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	float DirScalar;

	/**
	 *	If true, then collisions with Pawns will still react, but 
	 *	the UsedMaxCollisions count will not be decremented. 
	 *	(ie., They don't 'count' as collisions)
	 * NOTE: Having this on prevents the code from running in parallel.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	uint32 bPawnsDoNotDecrementCount:1;

	/**
	 *	If true, then collisions that do not have a vertical hit 
	 *	normal will still react, but UsedMaxCollisions count will 
	 *	not be decremented. (ie., They don't 'count' as collisions)
	 *	Useful for having particles come to rest on floors.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	uint32 bOnlyVerticalNormalsDecrementCount:1;

	/**
	 *	The fudge factor to use to determine vertical.
	 *	True vertical will have a Hit.Normal.Z == 1.0
	 *	This will allow for Z components in the range of
	 *	[1.0-VerticalFudgeFactor..1.0]
	 *	to count as vertical collisions.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	float VerticalFudgeFactor;

	/**
	 *	How long to delay before checking a particle for collisions.
	 *	Value is retrieved using the EmitterTime.
	 *	During update, the particle flag IgnoreCollisions will be 
	 *	set until the particle RelativeTime has surpassed the 
	 *	DelayAmount.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	struct FRawDistributionFloat DelayAmount;

	/** If true, Particle collision only if particle system is currently being rendered. */
	UPROPERTY(EditAnywhere, Category=Performance)
	uint32 bCollideOnlyIfVisible:1;	
	
	/**
	 *	If true, then the source actor is ignored in collision checks.
	 *	Defaults to true.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	uint32 bIgnoreSourceActor:1;

	/** Max distance at which particle collision will occur. */
	UPROPERTY(EditAnywhere, Category=Performance)
	float MaxCollisionDistance;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual uint32 RequiredBytesPerInstance() override;
	virtual uint32 PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual bool GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel) override;
	virtual bool CanTickInAnyThread() override;
	//End UParticleModule Interface

	/**
	 *	Perform the desired collision check for this module.
	 *	
	 *	@param	Owner			The emitter instance that owns the particle being checked
	 *	@param	InParticle		The particle being checked for a collision
	 *	@param	Hit				The hit results to fill in for a collision
	 *	@param	SourceActor		The source actor for the check
	 *	@param	End				The end position for the check
	 *	@param	Start			The start position for the check
	 *	@param	Extent			The extent to use for the check
	 *	
	 *	@return bool			true if a collision occurred.
	 */
	virtual bool PerformCollisionCheck(FParticleEmitterInstance* Owner, FBaseParticle* InParticle, 
		FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& Extent);
};



