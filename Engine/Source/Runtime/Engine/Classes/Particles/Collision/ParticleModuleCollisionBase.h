// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleCollisionBase.generated.h"

/**
 *	Flags indicating what to do with the particle when MaxCollisions is reached
 */
UENUM()
enum EParticleCollisionComplete : int
{
	/**	Kill the particle when MaxCollisions is reached		*/
	EPCC_Kill UMETA(DisplayName="Kill"),
	/**	Freeze the particle in place						*/
	EPCC_Freeze UMETA(DisplayName="Freeze"),
	/**	Stop collision checks, but keep updating			*/
	EPCC_HaltCollisions UMETA(DisplayName="Halt Collisions"),
	/**	Stop translations of the particle					*/
	EPCC_FreezeTranslation UMETA(DisplayName="Freeze Translation"),
	/**	Stop rotations of the particle						*/
	EPCC_FreezeRotation UMETA(DisplayName="Freeze Rotation"),
	/**	Stop all movement of the particle					*/
	EPCC_FreezeMovement UMETA(DisplayName="Freeze Movement"),
	EPCC_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Collision"))
class UParticleModuleCollisionBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

};

