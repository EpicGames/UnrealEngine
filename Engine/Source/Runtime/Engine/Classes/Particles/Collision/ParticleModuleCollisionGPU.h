// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Collision/ParticleModuleCollisionBase.h"
#include "ParticleModuleCollisionGPU.generated.h"

class UParticleEmitter;
class UParticleLODLevel;
namespace EParticleCollisionMode { enum Type : int; }

/**
 * How particles respond to collision events.
 */
UENUM()
namespace EParticleCollisionResponse
{
	enum Type : int
	{
		/** The particle will bounce off of the surface. */
		Bounce,
		/** The particle will stop on the surface. */
		Stop,
		/** The particle will be killed. */
		Kill
	};
}

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Collision"))
class UParticleModuleCollisionGPU : public UParticleModuleCollisionBase
{
	GENERATED_UCLASS_BODY()

	/**
	 * Dampens the velocity of a particle in the direction normal to the
	 * collision plane.
	 */
	UPROPERTY(EditAnywhere, Category=Collision, meta=(ToolTip="The bounciness of the particle."))
	struct FRawDistributionFloat Resilience;

	/**
	 * Modulates the resilience of the particle over its lifetime.
	 */
	UPROPERTY(EditAnywhere, Category=Collision, meta=(ToolTip="Scales the bounciness of the particle over its life."))
	struct FRawDistributionFloat ResilienceScaleOverLife;

	/** 
	 * Friction applied to all particles during a collision or while moving
	 * along a surface.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	float Friction;
	
	/** 
	 * Controls how wide the bouncing particles are distributed (0 = disabled).
	 */
	UPROPERTY(EditAnywhere, Category=Collision, meta=(UIMin=0, ClampMin=0, UIMax=1, ClampMax=1))
	float RandomSpread;
	
	/** 
	 * Controls bouncing particles distribution (1 = uniform distribution; 2 = squared distribution).
	 */
	UPROPERTY(EditAnywhere, Category=Collision, meta=(UIMin=1, ClampMin=1))
	float RandomDistribution;

	/**
	 * Scale applied to the size of the particle to obtain the collision radius.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	float RadiusScale;

	/**
	 * Bias applied to the collision radius.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	float RadiusBias;

	/**
	 * How particles respond to a collision event.
	 */
	UPROPERTY(EditAnywhere, Category=Collision)
	TEnumAsByte<EParticleCollisionResponse::Type> Response;

	UPROPERTY(EditAnywhere, Category=Collision)
	TEnumAsByte<EParticleCollisionMode::Type> CollisionMode;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//End UParticleModule Interface

protected:
	friend class FParticleModuleCollisionGPUDetails;
};



