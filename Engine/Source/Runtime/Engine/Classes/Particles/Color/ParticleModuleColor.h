// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Color/ParticleModuleColorBase.h"
#include "ParticleModuleColor.generated.h"

class UInterpCurveEdSetup;
class UParticleEmitter;
struct FCurveEdEntry;
struct FParticleEmitterInstance;
struct FRawDistributionFloat;
struct FRawDistributionVector;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Initial Color"), MinimalAPI)
class UParticleModuleColor : public UParticleModuleColorBase
{
	GENERATED_UCLASS_BODY()

	/** Initial color for a particle as a function of Emitter time. */
	UPROPERTY(EditAnywhere, Category = Color, meta = (TreatAsColor))
	FRawDistributionVector StartColor;

	/** Initial alpha for a particle as a function of Emitter time. */
	UPROPERTY(EditAnywhere, Category=Color)
	FRawDistributionFloat StartAlpha;

	/** If true, the alpha value will be clamped to the [0..1] range. */
	UPROPERTY(EditAnywhere, Category=Color)
	uint8 bClampAlpha:1;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//~ End UObject Interface


	//Begin UParticleModule Interface
	ENGINE_API virtual	bool AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries) override;
	ENGINE_API virtual void Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	ENGINE_API virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	ENGINE_API void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);

};



