// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Color/ParticleModuleColorBase.h"
#include "ParticleModuleColorOverLife.generated.h"

class UInterpCurveEdSetup;
class UParticleEmitter;
struct FCurveEdEntry;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Color Over Life"))
class UParticleModuleColorOverLife : public UParticleModuleColorBase
{
	GENERATED_UCLASS_BODY()

	/** The color to apply to the particle, as a function of the particle RelativeTime. */
	UPROPERTY(EditAnywhere, Category=Color, meta=(TreatAsColor))
	struct FRawDistributionVector ColorOverLife;

	/** The alpha to apply to the particle, as a function of the particle RelativeTime. */
	UPROPERTY(EditAnywhere, Category=Color)
	struct FRawDistributionFloat AlphaOverLife;

	/** If true, the alpha value will be clamped to the [0..1] range. */
	UPROPERTY(EditAnywhere, Category=Color)
	uint32 bClampAlpha:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface


	//Begin UParticleModule Interface
	virtual	bool AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//End UParticleModule Interface

};



