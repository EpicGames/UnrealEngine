// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleColorScaleOverLife
 *
 *	The base class for all Beam modules.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Color/ParticleModuleColorBase.h"
#include "ParticleModuleColorScaleOverLife.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Scale Color / Life"))
class UParticleModuleColorScaleOverLife : public UParticleModuleColorBase
{
	GENERATED_UCLASS_BODY()

	/** The scale factor for the color.													*/
	UPROPERTY(EditAnywhere, Category = Color, meta = (TreatAsColor))
	struct FRawDistributionVector ColorScaleOverLife;

	/** The scale factor for the alpha.													*/
	UPROPERTY(EditAnywhere, Category=Color)
	struct FRawDistributionFloat AlphaScaleOverLife;

	/** Whether it is EmitterTime or ParticleTime related.								*/
	UPROPERTY(EditAnywhere, Category=Color)
	uint32 bEmitterTime:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
#if WITH_EDITOR
	virtual int32 GetNumberOfCustomMenuOptions() const override;
	virtual bool GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const override;
	virtual bool PerformCustomMenuEntry(int32 InEntryIndex) override;
#endif
	//End UParticleModule Interface
};



