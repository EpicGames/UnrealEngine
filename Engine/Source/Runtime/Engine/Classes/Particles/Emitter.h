// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// Emitter actor class.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Emitter.generated.h"

class UPhysicalMaterial;

/** Fires when a particle is spawned
	* @param EventName - Custom event name for the Spawn Event.
	* @param EmitterTime - The emitter time when the event occured.
	* @param Location - Location at which the particle was spawned.
	* @param Velocity - Initial velocity of the spawned particle.
	*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams( FParticleSpawnSignature, FName, EventName, float, EmitterTime, FVector, Location, FVector, Velocity);

/** Fires when a particle system bursts
	* @param EventName - Custom event name for the Burst Event
	* @param EmitterTime - The emitter time when the event occured.
	* @param ParticleCount - The number of particles spawned in the burst.
	*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams( FParticleBurstSignature, FName, EventName, float, EmitterTime, int32, ParticleCount);

/** Fires when a particle dies
	* @param EventName - Custom event name for the Death Event.
	* @param EmitterTime - The emitter time when the event occured.
	* @param ParticleTime - How long the particle had been alive at the time of the event.
	* @param Location - Location the particle was at when it died.
	* @param Velocity - Velocity of the particle when it died.
	* @param Direction - Direction of the particle when it died.
	*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SixParams( FParticleDeathSignature, FName, EventName, float, EmitterTime, int32, ParticleTime, FVector, Location, FVector, Velocity, FVector, Direction);

/** Fires when a particle dies
	* @param EventName - Custom event name for the Collision Event.
	* @param EmitterTime - The emitter time when the event occured.
	* @param ParticleTime - How long the particle had been alive at the time of the event.
	* @param Location - Location of the collision.
	* @param Velocity - Velocity of the particle at the point of collision.
	* @param Direction - Direction of the particle at the point of collision.
	* @param Normal - Normal to the surface with which collision occurred.
	* @param BoneName- Name of the bone that the particle collided with. (Only valid if collision was with a Skeletal Mesh)
	* @param PhysMat - Physical Material for this collision.
	*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_NineParams( FParticleCollisionSignature, FName, EventName, float, EmitterTime, int32, ParticleTime, FVector, Location, FVector, Velocity, FVector, Direction, FVector, Normal, FName, BoneName, UPhysicalMaterial*, PhysMat);

UCLASS(hideCategories=(Activation,"Components|Activation",Input,Collision,"Game|Damage"), ComponentWrapperClass, MinimalAPI)
class AEmitter : public AActor
{
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const override;

private:
	UPROPERTY(Category = Emitter, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Particles|Beam,Particles|Parameters,Particles,Effects|Components|ParticleSystem,Rendering,Activation,Components|Activation", AllowPrivateAccess = "true"))
	TObjectPtr<class UParticleSystemComponent> ParticleSystemComponent;
public:

	UPROPERTY()
	uint32 bDestroyOnSystemFinish:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Emitter)
	uint32 bPostUpdateTickGroup:1;

	/** used to update status of toggleable level placed emitters on clients */
	UPROPERTY(replicatedUsing=OnRep_bCurrentlyActive)
	uint32 bCurrentlyActive:1;

	UPROPERTY(BlueprintAssignable)
	FParticleSpawnSignature OnParticleSpawn;

	UPROPERTY(BlueprintAssignable)
	FParticleBurstSignature OnParticleBurst;

	UPROPERTY(BlueprintAssignable)
	FParticleDeathSignature OnParticleDeath;

	UPROPERTY(BlueprintAssignable)
	FParticleCollisionSignature OnParticleCollide;

#if WITH_EDITORONLY_DATA

private:
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;

	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
public:

#endif

	UFUNCTION()
	ENGINE_API virtual void OnParticleSystemFinished(class UParticleSystemComponent* FinishedComponent);

	/** Replication Notification Callbacks */
	UFUNCTION()
	ENGINE_API virtual void OnRep_bCurrentlyActive();

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category=Particles, meta=(DeprecatedFunction))
	ENGINE_API void Activate();
	UFUNCTION(BlueprintCallable, Category=Particles, meta=(DeprecatedFunction))
	ENGINE_API void Deactivate();
	UFUNCTION(BlueprintCallable, Category=Particles, meta=(DeprecatedFunction))
	ENGINE_API void ToggleActive();
	UFUNCTION(BlueprintCallable, Category=Particles, meta=(DeprecatedFunction))
	ENGINE_API bool IsActive() const;
	UFUNCTION(BlueprintCallable, Category=Particles, meta=(DeprecatedFunction))
	ENGINE_API virtual void SetTemplate(class UParticleSystem* NewTemplate);
	UFUNCTION(BlueprintCallable, Category="Particles|Parameters", meta=(DeprecatedFunction))
	ENGINE_API void SetFloatParameter(FName ParameterName, float Param);
	UFUNCTION(BlueprintCallable, Category="Particles|Parameters", meta=(DeprecatedFunction))
	ENGINE_API void SetVectorParameter(FName ParameterName, FVector Param);
	UFUNCTION(BlueprintCallable, Category="Particles|Parameters", meta=(DeprecatedFunction))
	ENGINE_API void SetColorParameter(FName ParameterName, FLinearColor Param);
	UFUNCTION(BlueprintCallable, Category="Particles|Parameters", meta=(DeprecatedFunction))
	ENGINE_API void SetActorParameter(FName ParameterName, class AActor* Param);
	UFUNCTION(BlueprintCallable, Category="Particles|Parameters", meta=(DeprecatedFunction))
	ENGINE_API void SetMaterialParameter(FName ParameterName, class UMaterialInterface* Param);
	// END DEPRECATED


	ENGINE_API void AutoPopulateInstanceProperties();

	//~ Begin UObject Interface
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	//~ End UObject Interface


	//~ Begin AActor Interface
	ENGINE_API virtual void PostActorCreated() override;
	ENGINE_API virtual void PostInitializeComponents() override;
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const override;
	//~ End AActor Interface

	/**
	 *	Called to reset the emitter actor in the level.
	 *	Intended for use in editor only
	 */
	ENGINE_API void ResetInLevel();
#endif

public:
	/** Returns ParticleSystemComponent subobject **/
	class UParticleSystemComponent* GetParticleSystemComponent() { return ParticleSystemComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	class UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif
};



