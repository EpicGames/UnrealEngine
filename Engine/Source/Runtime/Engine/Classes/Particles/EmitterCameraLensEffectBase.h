// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for Camera Lens Effects.  Needed so we can have AnimNotifies be able to show camera effects
 * in a nice drop down.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Particles/Emitter.h"
#include "Camera/CameraLensEffectInterface.h"
#include "EmitterCameraLensEffectBase.generated.h"

class APlayerCameraManager;

UCLASS(abstract, Blueprintable, MinimalAPI)
class AEmitterCameraLensEffectBase : public AEmitter, public ICameraLensEffectInterface
{
	GENERATED_UCLASS_BODY()

protected:
	/** Particle System to use */
	UPROPERTY(EditDefaultsOnly, Category = EmitterCameraLensEffectBase)
	TObjectPtr<class UParticleSystem> PS_CameraEffect;

	/** Camera this emitter is attached to, will be notified when emitter is destroyed */
	UPROPERTY(transient)
	TObjectPtr<class APlayerCameraManager> BaseCamera;

	/** 
	 * Effect-to-camera transform to allow arbitrary placement of the particle system .
	 * Note the X component of the location will be scaled with camera fov to keep the lens effect the same apparent size.
	 */
	UPROPERTY(EditDefaultsOnly, Category = EmitterCameraLensEffectBase)
	FTransform RelativeTransform;

public:
	/** This is the assumed FOV for which the effect was authored. The code will make automatic adjustments to make it look the same at different FOVs */
	UPROPERTY(EditDefaultsOnly, Category = EmitterCameraLensEffectBase)
	float BaseFOV;

	/** true if multiple instances of this emitter can exist simultaneously, false otherwise.  */
	UPROPERTY(EditAnywhere, Category = EmitterCameraLensEffectBase)
	uint8 bAllowMultipleInstances:1;

	/** If bAllowMultipleInstances is true and this effect is retriggered, the particle system will be reset if this is true */
	UPROPERTY(EditAnywhere, Category = EmitterCameraLensEffectBase)
	uint8 bResetWhenRetriggered:1;

	/** 
	 *  If an emitter class in this array is currently playing, do not play this effect.
	 *  Useful for preventing multiple similar or expensive camera effects from playing simultaneously.
	 */
	UPROPERTY(EditDefaultsOnly, Category = EmitterCameraLensEffectBase, meta = (MustImplement = "/Script/Engine.CameraLensEffectInterface"))
	TArray<TSubclassOf<AActor>> EmittersToTreatAsSame;
public:
	//~ Begin AActor Interface
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void PostInitializeComponents() override;
	ENGINE_API virtual void PostLoad() override;
	//~ End AActor Interface

	//~ Begin ICameraLensEffectInterface interface
	ENGINE_API virtual const FTransform& GetRelativeTransform() const override;
	ENGINE_API virtual float GetBaseFOV() const override;
	ENGINE_API virtual bool ShouldAllowMultipleInstances() const override;
	ENGINE_API virtual bool ResetWhenTriggered() const override;
	ENGINE_API virtual bool ShouldTreatEmitterAsSame(TSubclassOf<AActor> OtherEmitter) const override;
	ENGINE_API virtual bool IsLooping() const override;

	ENGINE_API virtual void AdjustBaseFOV(float NewFOV);

	/** Tell the emitter what camera it is attached to. */
	ENGINE_API virtual void RegisterCamera(APlayerCameraManager* C) override;

	ENGINE_API virtual void NotifyWillBePooled() override;
	
	/** Called when this emitter is re-triggered, for bAllowMultipleInstances=false emitters. */
	ENGINE_API virtual void NotifyRetriggered() override;

	/** This will actually activate the lens Effect.  We want this separated from PostInitializeComponents so we can cache these emitters **/
	ENGINE_API virtual void ActivateLensEffect() override;
	
	/** Deactivtes the particle system. If bDestroyOnSystemFinish is true, actor will die after particles are all dead. */
	ENGINE_API virtual void DeactivateLensEffect() override;
	
	/** Given updated camera information, adjust this effect to display appropriately. */
	ENGINE_API virtual void UpdateLocation(const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg) override;
	//~ End ICameraLensEffectInterface interface

	UE_DEPRECATED(5.0, "ICameraLensEffectInterface::GetAttachedEmitterTransform is favored now")
	static ENGINE_API FTransform GetAttachedEmitterTransform(AEmitterCameraLensEffectBase const* Emitter, const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg);

private:
	/** UE_DEPRECATED(4.11) */
	UPROPERTY()
	float DistFromCamera_DEPRECATED;
};
