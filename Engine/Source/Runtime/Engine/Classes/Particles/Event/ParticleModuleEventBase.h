// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleEventBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Event"))
class UParticleModuleEventBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()


	//~ Begin UParticleModule Interface
	virtual EModuleType	GetModuleType() const override {	return EPMT_Event;	}
	//~ End UParticleModule Interface
};

