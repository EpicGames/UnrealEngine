// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/Event/ParticleModuleEventBase.h"
#include "ParticleModuleEventGenerator.generated.h"

class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

/**
 */
USTRUCT()
struct FParticleEvent_GenerateInfo
{
	GENERATED_USTRUCT_BODY()

	/** The type of event. */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	TEnumAsByte<EParticleEventType> Type;

	/** How often to trigger the event (<= 1 means EVERY time). */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	int32 Frequency;

	/** Only fire the first time (collision only). */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	int32 ParticleFrequency;

	/** Only fire the first time (collision only). */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	uint32 FirstTimeOnly:1;

	/** Only fire the last time (collision only). */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	uint32 LastTimeOnly:1;

	/** Use the impact FVector not the hit normal (collision only). */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	uint32 UseReflectedImpactVector:1;

	/** Use the orbit offset when computing the position at which the event occurred. */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	uint32 bUseOrbitOffset:1;

	/** Should the event tag with a custom name? Leave blank for the default. */
	UPROPERTY(EditAnywhere, Category=ParticleEvent_GenerateInfo)
	FName CustomName;

	/** The events we want to fire off when this event has been generated */
	UPROPERTY(EditAnywhere, Instanced, Category = ParticleEvent_GenerateInfo)
	TArray<TObjectPtr<class UParticleModuleEventSendToGame>> ParticleModuleEventsToSendToGame;

	FParticleEvent_GenerateInfo()
	: Type(0)
	, Frequency(0)
	, ParticleFrequency(0)
	, FirstTimeOnly(false)
	, LastTimeOnly(false)
	, UseReflectedImpactVector(false)
	, bUseOrbitOffset(false)
	{
	}
};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Event Generator"))
class UParticleModuleEventGenerator : public UParticleModuleEventBase
{
	GENERATED_UCLASS_BODY()

	/**
	 */
	UPROPERTY(EditAnywhere, export, noclear, Category=Events)
	TArray<struct FParticleEvent_GenerateInfo> Events;


	//Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual uint32	RequiredBytesPerInstance() override;
	virtual uint32	PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData) override;
	//End UParticleModule Interface


	/**
	 *	Called when a particle is spawned and an event payload is present.
	 *
	 *	@param	Owner			Pointer to the owning FParticleEmitterInstance.
	 *	@param	EventPayload	Pointer to the event instance payload data.
	 *	@param	NewParticle		Pointer to the particle that was spawned.
	 *
	 *	@return	bool			true if processed, false if not.
	 */
	virtual bool	HandleParticleSpawned(FParticleEmitterInstance* Owner, FParticleEventInstancePayload* EventPayload, FBaseParticle* NewParticle);

	/**
	 *	Called when a particle is killed and an event payload is present.
	 *
	 *	@param	Owner			Pointer to the owning FParticleEmitterInstance.
	 *	@param	EventPayload	Pointer to the event instance payload data.
	 *	@param	DeadParticle	Pointer to the particle that is being killed.
	 *
	 *	@return	bool			true if processed, false if not.
	 */
	virtual bool	HandleParticleKilled(FParticleEmitterInstance* Owner, FParticleEventInstancePayload* EventPayload, FBaseParticle* DeadParticle);

	/**
	 *	Called when a particle collides and an event payload is present.
	 *
	 *	@param	Owner				Pointer to the owning FParticleEmitterInstance.
	 *	@param	EventPayload		Pointer to the event instance payload data.
	 *	@param	CollidePayload		Pointer to the collision payload data.
	 *	@param	Hit					The CheckResult for the collision.
	 *	@param	CollideParticle		Pointer to the particle that has collided.
	 *	@param	CollideDirection	The direction the particle was traveling when the collision occurred.
	 *
	 *	@return	bool				true if processed, false if not.
	 */
	virtual bool	HandleParticleCollision(FParticleEmitterInstance* Owner, FParticleEventInstancePayload* EventPayload, 
		FParticleCollisionPayload* CollidePayload, FHitResult* Hit, FBaseParticle* CollideParticle, FVector& CollideDirection);

	/**
	 *	Called when a particle bursts and an event payload is present.
	 *
	 *	@param	Owner				Pointer to the owning FParticleEmitterInstance.
	 *	@param	EventPayload		Pointer to the event instance payload data.
	 *	@param	ParticleCount		The count of particles that are being spawned
	 *
	 *	@return	bool			true if processed, false if not.
	 */
	virtual bool	HandleParticleBurst(FParticleEmitterInstance* Owner, FParticleEventInstancePayload* EventPayload, const int32 ParticleCount);
};



