// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Event/ParticleModuleEventBase.h"
#include "Particles/ParticleSystemComponent.h"
#include "ParticleModuleEventReceiverBase.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, abstract)
class UParticleModuleEventReceiverBase : public UParticleModuleEventBase
{
	GENERATED_UCLASS_BODY()

	/** The type of event that will generate the kill. */
	UPROPERTY(EditAnywhere, Category=Source)
	TEnumAsByte<EParticleEventType> EventGeneratorType;

	/** The name of the emitter of interest for generating the event. */
	UPROPERTY(EditAnywhere, Category=Source)
	FName EventName;


	/**
	 *	Is the module interested in events of the given type?
	 *
	 *	@param	InEventType		The event type to check
	 *
	 *	@return	bool			true if interested.
	 */
	virtual bool WillProcessParticleEvent(EParticleEventType InEventType)
	{
		if ((EventGeneratorType == EPET_Any) || (InEventType == EventGeneratorType))
		{
			return true;
		}

		return false;
	}

	/**
	 *	Process the event...
	 *
	 *	@param	Owner		The FParticleEmitterInstance this module is contained in.
	 *	@param	InEvent		The FParticleEventData that occurred.
	 *	@param	DeltaTime	The time slice of this frame.
	 *
	 *	@return	bool		true if the event was processed; false if not.
	 */
	virtual bool ProcessParticleEvent(FParticleEmitterInstance* Owner, FParticleEventData& InEvent, float DeltaTime)
	{
		return false;
	}
};

