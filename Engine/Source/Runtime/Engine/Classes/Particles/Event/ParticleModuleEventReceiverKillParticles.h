// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Event/ParticleModuleEventReceiverBase.h"
#include "ParticleModuleEventReceiverKillParticles.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "EventReceiver Kill All"))
class UParticleModuleEventReceiverKillParticles : public UParticleModuleEventReceiverBase
{
	GENERATED_UCLASS_BODY()

	/** If true, stop this emitter from spawning as well. */
	UPROPERTY(EditAnywhere, Category=ParticleModuleEventReceiverKillParticles)
	uint32 bStopSpawning:1;


	//~ Begin UParticleModuleEventBase Interface
	virtual bool ProcessParticleEvent(FParticleEmitterInstance* Owner, FParticleEventData& InEvent, float DeltaTime) override;
	//~ End UParticleModuleEventBase Interface
};



