// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Event/ParticleModuleEventReceiverBase.h"
#include "ParticleModuleEventReceiverSpawn.generated.h"

class UPhysicalMaterial;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "EventReceiver Spawn"))
class UParticleModuleEventReceiverSpawn : public UParticleModuleEventReceiverBase
{
	GENERATED_UCLASS_BODY()

	/** The number of particles to spawn. */
	UPROPERTY(EditAnywhere, Category=Spawn)
	struct FRawDistributionFloat SpawnCount;

	/** 
	 *	For Death-based event receiving, if this is true, it indicates that the 
	 *	ParticleTime of the event should be used to look-up the SpawnCount.
	 *	Otherwise (and in all other events received), use the emitter time of 
	 *	the event.
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint32 bUseParticleTime:1;

	/**
	 *	If true, use the location of the particle system component for spawning.
	 *	if false (default), use the location of the particle event.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 bUsePSysLocation:1;

	/**
	 *	If true, use the velocity of the dying particle as the start velocity of 
	 *	the spawned particle.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	uint32 bInheritVelocity:1;

	/**
	 *	If bInheritVelocity is true, scale the velocity with this.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	struct FRawDistributionVector InheritVelocityScale;

	/**
	*	Array of physical materials that can be used to allow or ban a specific set of materials when receiving collision events.
	*/
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<TObjectPtr<UPhysicalMaterial>> PhysicalMaterials;

	/**
	*	When true, the PhysicalMaterials list is used to ban specified materials for collision events but allow all others.
	*	When false, the PhysicalMaterials list is used to allow only specified materials for collision events and ban all others.
	*/
	UPROPERTY(EditAnywhere, Category = Collision)
	uint32 bBanPhysicalMaterials : 1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModuleEventBase Interface
	virtual bool ProcessParticleEvent(FParticleEmitterInstance* Owner, FParticleEventData& InEvent, float DeltaTime) override;
	//~ End UParticleModuleEventBase Interface
};



