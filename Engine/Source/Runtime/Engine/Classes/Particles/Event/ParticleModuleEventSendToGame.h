// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ParticleModuleEventSendToGame.generated.h"

UCLASS(abstract, editinlinenew, hidecategories=Object)
class UParticleModuleEventSendToGame : public UObject
{
	GENERATED_UCLASS_BODY()


	/** This is our function to allow subclasses to "do the event action" **/
	virtual void DoEvent( const FVector& InCollideDirection, const FVector& InHitLocation, const FVector& InHitNormal, const FName& InBoneName ) {}
};

