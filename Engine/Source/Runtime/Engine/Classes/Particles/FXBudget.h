// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ParticlePerfStatsManager.h"

class FParticlePerfStatsListener_FXBudget;

/** Timing data for various parts of FX work. Typically holds direct timing data in ms but can occasionally hold related data like usage ratios etc. */
struct FFXTimeData
{
	FFXTimeData(): GT(0.0f), GTConcurrent(0.0f), RT(0.0f) {}
	FFXTimeData(float InGT, float InConcurrent, float InRT)	: GT(InGT), GTConcurrent(InConcurrent), RT(InRT) {}

	/** Total time of work that must run on the game thread. */
	float GT;
	/** Total time of *potentially* concurrent work spawned from the game thread. This may run on the Gamethread but can run concurrently. */
	float GTConcurrent;
	/** Total render thread time. */
	float RT;
};

#if WITH_GLOBAL_RUNTIME_FX_BUDGET
class FFXBudget
{
public:
	/** Returns the global FX time in ms. */
	static ENGINE_API FFXTimeData GetTime();
	/** Returns the global FX budgets in ms. */
	static ENGINE_API FFXTimeData GetBudget();
	/** Returns the global FX time / budget ratio. */
	static ENGINE_API FFXTimeData GetUsage();
	/** 
	 * Returns the global FX time / budget ratio but adjusted in various ways better drive FX scaling. 
	 * e.g. Usage goes up in line with the real usage but can fall only at a set rate. Useful to avoid FX flipping on/off if their cost is tipping the usage over the budget.
* 	 * Other adjustments may be made in future.
	 **/
	static ENGINE_API FFXTimeData GetAdjustedUsage();
	/** Returns the highest single adjusted usage value. */
	inline static float GetWorstAdjustedUsage() { return WorstAdjustedUsage; }
	inline static void SetWorstAdjustedUsage(float NewAdjustedUsage){ WorstAdjustedUsage = NewAdjustedUsage; }

	static ENGINE_API void Reset();

	static ENGINE_API TSharedPtr<FParticlePerfStatsListener_FXBudget, ESPMode::ThreadSafe> StatsListener;

	static ENGINE_API void OnEnabledCVarChanged(IConsoleVariable* CVar);
	inline static bool Enabled(){ return bEnabled; }
	static ENGINE_API void SetEnabled(bool bInEnabled);

	static ENGINE_API bool bEnabled;

	static ENGINE_API FFXTimeData AdjustedUsage;
	static ENGINE_API float WorstAdjustedUsage;

private:
	static ENGINE_API void OnEnabledChangedInternal();
};
#else
class FFXBudget
{
public:
	inline static FFXTimeData GetTime(){ return FFXTimeData(); }
	inline static FFXTimeData GetBudget() { return FFXTimeData(); }
	inline static FFXTimeData GetUsage() { return FFXTimeData(); }
	inline static FFXTimeData GetAdjustedUsage() { return FFXTimeData(); }
	inline static float GetWorstAdjustedUsage() { return 0.0f; }
	inline static bool Enabled() { return false; }
	inline static void SetEnabled(bool bInEnabled) { }

	inline static void Reset(){}
};
#endif
