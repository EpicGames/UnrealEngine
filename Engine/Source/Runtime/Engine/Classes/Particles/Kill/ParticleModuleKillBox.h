// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Kill/ParticleModuleKillBase.h"
#include "ParticleModuleKillBox.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Kill Box"))
class UParticleModuleKillBox : public UParticleModuleKillBase
{
	GENERATED_UCLASS_BODY()

	/** The lower left corner of the box. */
	UPROPERTY(EditAnywhere, Category=Kill)
	struct FRawDistributionVector LowerLeftCorner;

	/** The upper right corner of the box. */
	UPROPERTY(EditAnywhere, Category=Kill)
	struct FRawDistributionVector UpperRightCorner;

	/** If true, the box coordinates are in world space./ */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bAbsolute:1;

	/**
	 *	If true, particles INSIDE the box will be killed. 
	 *	If false (the default), particles OUTSIDE the box will be killed.
	 */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bKillInside:1;

	/** If true, the box will always be axis aligned and non-scalable. */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bAxisAlignedAndFixedSize:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void	Update(const FUpdateContext& Context) override;
	virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



