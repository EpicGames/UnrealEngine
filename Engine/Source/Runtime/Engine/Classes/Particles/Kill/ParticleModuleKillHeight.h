// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Kill/ParticleModuleKillBase.h"
#include "ParticleModuleKillHeight.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Kill Height"))
class UParticleModuleKillHeight : public UParticleModuleKillBase
{
	GENERATED_UCLASS_BODY()

	/** The height at which to kill the particle. */
	UPROPERTY(EditAnywhere, Category=Kill)
	struct FRawDistributionFloat Height;

	/** If true, the height should be treated as a world-space position. */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bAbsolute:1;

	/**
	 *	If true, the plane should be considered a floor - ie kill anything BELOW it.
	 *	If false, if is a ceiling - ie kill anything ABOVE it.
	 */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bFloor:1;

	/** If true, take the particle systems scale into account */
	UPROPERTY(EditAnywhere, Category=Kill)
	uint32 bApplyPSysScale:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void	Update(const FUpdateContext& Context) override;
	virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



