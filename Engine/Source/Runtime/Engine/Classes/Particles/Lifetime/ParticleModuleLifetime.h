// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Lifetime/ParticleModuleLifetimeBase.h"
#include "ParticleModuleLifetime.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Lifetime"))
class UParticleModuleLifetime : public UParticleModuleLifetimeBase
{
	GENERATED_UCLASS_BODY()

	/** The lifetime of the particle, in seconds. Retrieved using the EmitterTime at the spawn of the particle. */
	UPROPERTY(EditAnywhere, Category=Lifetime)
	struct FRawDistributionFloat Lifetime;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//End UParticleModule Interface

	//~ Begin UParticleModuleLifetimeBase Interface
	virtual float GetMaxLifetime() override;
	virtual float GetLifetimeValue(const FContext& Context, float InTime, UObject* Data = NULL) override;
	//~ End UParticleModuleLifetimeBase Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



