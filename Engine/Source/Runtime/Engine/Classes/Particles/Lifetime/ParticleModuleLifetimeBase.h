// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleLifetimeBase.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Lifetime"))
class UParticleModuleLifetimeBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()


	/** Return the maximum lifetime this module would return. */
	virtual float	GetMaxLifetime()
	{
		return 0.0f;
	}

	/**
	 *	Return the lifetime value at the given time.
	 *
	 *	@param	Owner		The emitter instance that owns this module
	 *	@param	InTime		The time input for retrieving the lifetime value
	 *	@param	Data		The data associated with the distribution
	 *
	 *	@return	float		The Lifetime value
	 */
	virtual float	GetLifetimeValue(const FContext& Context, float InTime, UObject* Data = NULL)
		PURE_VIRTUAL(UParticleModuleLifetimeBase::GetLifetimeValue,return 0.0f;);
};

