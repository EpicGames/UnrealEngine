// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Light/ParticleModuleLightBase.h"
#include "ParticleModuleLight.generated.h"

class UParticleEmitter;
class UParticleModuleTypeDataBase;
class UPointLightComponent;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Light"))
class UParticleModuleLight : public UParticleModuleLightBase
{
	GENERATED_UCLASS_BODY()

	/** Whether to use physically based inverse squared falloff from the light.  If unchecked, the LightExponent distribution will be used instead. */
	UPROPERTY(EditAnywhere, Category=Light)
	bool bUseInverseSquaredFalloff;

	/** 
	 * Whether lights from this module should affect translucency.
	 * Use with caution.  Modules enabling this should only make a few particle lights at most, and the smaller they are, the less they will cost.
	 */
	UPROPERTY(EditAnywhere, Category=Light)
	bool bAffectsTranslucency;

	/** When enabled we will override the project default setting with our local setting. */
	UPROPERTY(EditAnywhere, Category = Light, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverrideInverseExposureBlend : 1;

	/** 
	 * Will draw wireframe spheres to preview the light radius if enabled.
	 * Note: this is intended for previewing and the value will not be saved, it will always revert to disabled.
	 */
	UPROPERTY(EditAnywhere, Transient, Category=Light)
	bool bPreviewLightRadius;

	/** Fraction of particles in this emitter to create lights on. */
	UPROPERTY(EditAnywhere, Category=Light)
	float SpawnFraction;

	/** Scale that is applied to the particle's color to calculate the light's color, and can be setup as a curve over the particle's lifetime. */
	UPROPERTY(EditAnywhere, Category=Light)
	struct FRawDistributionVector ColorScaleOverLife;

	/** Brightness scale for the light, which can be setup as a curve over the particle's lifetime. */
	UPROPERTY(EditAnywhere, Category=Light)
	struct FRawDistributionFloat BrightnessOverLife;

	/** Scales the particle's radius, to calculate the light's radius. */
	UPROPERTY(EditAnywhere, Category=Light)
	struct FRawDistributionFloat RadiusScale;

	/** Provides the light's exponent when inverse squared falloff is disabled. */
	UPROPERTY(EditAnywhere, Category=Light)
	struct FRawDistributionFloat LightExponent;

	/**
	* Blend Factor used to blend between Intensity and Intensity/Exposure.
	* This is useful for gameplay lights that should have constant brighness on screen independent of current exposure.
	* This feature can cause issues with exposure particularly when used on the primary light on a scene, as such it's usage should be limited.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Light, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "1.0", EditCondition = "bOverrideInverseExposureBlend"))
	float InverseExposureBlend = 0.0f;

	/**
	* Channels that this light should affect.
	* Only affect high quality lights
	* These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Light)
	FLightingChannels LightingChannels;

	/** Intensity of the volumetric scattering from this light.  This scales Intensity and LightColor. */
	UPROPERTY(BlueprintReadOnly, interp, Category=Light, meta=(UIMin = "0.25", UIMax = "4.0"))
	float VolumetricScatteringIntensity;

	/** Converts the particle lights into high quality lights as if they came from a PointLightComponent.  High quality lights cost significantly more on both CPU and GPU. */
	UPROPERTY(EditAnywhere, Category = Light)
	bool bHighQualityLights;

	/** 
	 * Whether to cast shadows from the particle lights.  Requires High Quality Lights to be enabled.
	 * Warning: This can be incredibly expensive on the GPU - use with caution.
	 */
	UPROPERTY(EditAnywhere, Category = Light)
	bool bShadowCastingLights;	

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface


	//Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual EModuleType	GetModuleType() const override { return EPMT_Light; }
	virtual void Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface

	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);

	virtual bool CanTickInAnyThread() override;

private:
		
	uint64 SpawnHQLight(const FLightParticlePayload& Payload, const FBaseParticle& Particle, FParticleEmitterInstance* Owner);	
	void UpdateHQLight(UPointLightComponent* PointLightComponent, const FLightParticlePayload& Payload, const FBaseParticle& Particle, int32 ScreenAlignment, FVector ComponentScale, bool bLocalSpace, FSceneInterface* OwnerScene, bool bDoRTUpdate);
};



