// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocation.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Initial Location"), MinimalAPI)
class UParticleModuleLocation : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The location the particle should be emitted.
	 *	Relative in local space to the emitter by default.
	 *	Relative in world space as a WorldOffset module or when the emitter's UseLocalSpace is off.
	 *	Retrieved using the EmitterTime at the spawn of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector StartLocation;

	/**
	 *  When set to a non-zero value this will force the particles to only spawn on evenly distributed
	 *  positions between the two points specified.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	float DistributeOverNPoints;

	/**
	 *  When DistributeOverNPoints is set to a non-zero value, this specifies the ratio of particles spawned
	 *  that should use the distribution.  (For example setting this to 1 will cause all the particles to
	 *  be distributed evenly whereas .75 would cause 1/4 of the particles to be randomly placed).
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	float DistributeThreshold;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

protected:
	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	ENGINE_API virtual void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);

public:
	//Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	ENGINE_API virtual void Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void Render3DPreview(const FPreviewContext& Context) override;
	//End UParticleModule Interface
};



