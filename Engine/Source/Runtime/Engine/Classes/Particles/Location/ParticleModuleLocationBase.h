// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleLocationBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Location"), MinimalAPI)
class UParticleModuleLocationBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

};

