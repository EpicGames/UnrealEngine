// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ParticleHelper.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationBoneSocket.generated.h"

class UParticleModuleTypeDataBase;
class UParticleSystemComponent;
class USkeletalMeshComponent;
class USkeletalMeshSocket;
struct FModuleLocationBoneSocketInstancePayload;
struct FParticleEmitterInstance;

UENUM()
enum ELocationBoneSocketSource : int
{
	BONESOCKETSOURCE_Bones,
	BONESOCKETSOURCE_Sockets,
	BONESOCKETSOURCE_MAX,
};

UENUM()
enum ELocationBoneSocketSelectionMethod : int
{
	BONESOCKETSEL_Sequential,
	BONESOCKETSEL_Random,
	BONESOCKETSEL_MAX,
};

USTRUCT()
struct FLocationBoneSocketInfo
{
	GENERATED_USTRUCT_BODY()

	/** The name of the bone/socket on the skeletal mesh */
	UPROPERTY(EditAnywhere, Category=LocationBoneSocketInfo)
	FName BoneSocketName;

	/** The offset from the bone/socket to use */
	UPROPERTY(EditAnywhere, Category=LocationBoneSocketInfo)
	FVector Offset;


	FLocationBoneSocketInfo()
		: Offset(ForceInit)
	{
	}

};

enum class EBoneSocketSourceIndexMode : uint8
{
	SourceLocations,//Module has source locations so SourceIndex is an index into these.
	PreSelectedIndices,//Module has no source locations but requires tracking of bone velocities so SourceIndex is an index into an array of preselected indices. These indices are Direct into the bone/sockets of the source mesh.
	Direct,//Module has no source locations and no bone tracking requirement so can simply access the mesh via direct indices to the bones/sockets.
};

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Bone/Socket Location"), MinimalAPI)
class UParticleModuleLocationBoneSocket : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Whether the module uses Bones or Sockets for locations.
	 *
	 *	BONESOCKETSOURCE_Bones		- Use Bones as the source locations.
	 *	BONESOCKETSOURCE_Sockets	- Use Sockets as the source locations.
	 */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	TEnumAsByte<enum ELocationBoneSocketSource> SourceType;

	/** An offset to apply to each bone/socket */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	FVector UniversalOffset;

	/** The name(s) of the bone/socket(s) to position at. If this is empty, the module will attempt to spawn from all bones or sockets. */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	TArray<struct FLocationBoneSocketInfo> SourceLocations;

	/**
	 *	The method by which to select the bone/socket to spawn at.
	 *
	 *	SEL_Sequential			- loop through the bone/socket array in order
	 *	SEL_Random				- randomly select a bone/socket from the array
	 */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	TEnumAsByte<enum ELocationBoneSocketSelectionMethod> SelectionMethod;

	/** If true, update the particle locations each frame with that of the bone/socket */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	uint32 bUpdatePositionEachFrame:1;

	/** If true, rotate mesh emitter meshes to orient w/ the socket. Currently does nothing. */
	UPROPERTY()
	uint32 bOrientMeshEmitters:1;

	/** If true, particles inherit the associated bone velocity when spawned */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	uint32 bInheritBoneVelocity:1;

	/** A scale on how much of the bone's velocity a particle will inherit. */
	UPROPERTY(EditAnywhere, Category = BoneSocket)
	float InheritVelocityScale;

	/**
	 *	The parameter name of the skeletal mesh actor that supplies the SkelMeshComponent for in-game.
	 */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	FName SkelMeshActorParamName;
	
	/** 
	When we have no source locations and we need to track bone velocities due to bInheritBoneVelocity, we pre select a set of bones to use each frame. This property determines how big the list is.
	Too low and the randomness of selection may suffer, too high and memory will be wasted.
	*/
	UPROPERTY(EditAnywhere, Category = BoneSocket, meta = (UIMin = "1"))
	int32 NumPreSelectedIndices;

#if WITH_EDITORONLY_DATA
	/** The name of the skeletal mesh to use in the editor */
	UPROPERTY(EditAnywhere, Category=BoneSocket)
	TObjectPtr<class USkeletalMesh> EditorSkelMesh;

#endif // WITH_EDITORONLY_DATA

	/** How particle SourceIndex should be interpreted. */
	EBoneSocketSourceIndexMode SourceIndexMode;

	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	PostLoad() override;
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Update(const FUpdateContext& Context) override;
	ENGINE_API virtual void	FinalUpdate(const FUpdateContext& Context) override;
	ENGINE_API virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	ENGINE_API virtual uint32	RequiredBytesPerInstance() override;
	ENGINE_API virtual uint32	PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData) override;
	virtual bool	TouchesMeshRotation() const override { return true; }
	ENGINE_API virtual void	AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual bool CanTickInAnyThread() override
	{
		return true;
	}
#if WITH_EDITOR
	ENGINE_API virtual int32 GetNumberOfCustomMenuOptions() const override;
	ENGINE_API virtual bool GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const override;
	ENGINE_API virtual bool PerformCustomMenuEntry(int32 InEntryIndex) override; 
	ENGINE_API virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)override;
#endif
	//~ End UParticleModule Interface

	/**
	 *	Retrieve the skeletal mesh component source to use for the current emitter instance.
	 *
	 *	@param	Owner						The particle emitter instance that is being setup
	 *
	 *	@return	USkeletalMeshComponent*		The skeletal mesh component to use as the source
	 */
	ENGINE_API void GetSkeletalMeshComponentSource(const FContext& Context, FModuleLocationBoneSocketInstancePayload* InstancePayload);

	/**
	 *	Retrieve the position for the given socket index.
	 *
	 *	@param	Owner					The particle emitter instance that is being setup
	 *	@param	InSkelMeshComponent		The skeletal mesh component to use as the source
	 *	@param	InBoneSocketIndex		The index of the bone/socket of interest
	 *	@param	OutPosition				The position for the particle location
	 *	@param	OutRotation				Optional orientation for the particle (mesh emitters)
	 *	
	 *	@return	bool					true if successful, false if not
	 */
	ENGINE_API bool GetParticleLocation(FModuleLocationBoneSocketInstancePayload* InstancePayload, FParticleEmitterInstance* Owner, USkeletalMeshComponent* InSkelMeshComponent, int32 InBoneSocketIndex, FVector& OutPosition, FQuat* OutRotation);

	ENGINE_API int32 GetMaxSourceIndex(FModuleLocationBoneSocketInstancePayload* Payload, USkeletalMeshComponent* SourceComponent)const;
	ENGINE_API bool GetSocketInfoForSourceIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, int32 SourceIndex, USkeletalMeshSocket*& OutSocket, FVector& OutOffset)const;
	ENGINE_API bool GetBoneInfoForSourceIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, int32 SourceIndex, FMatrix& OutBoneMatrix, FVector& OutOffset)const;
	
	/** Selects the next socket or bone index to spawn from. */
	ENGINE_API int32 SelectNextSpawnIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, FRandomStream& InRandomStream);
	ENGINE_API void RegeneratePreSelectedIndices(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, FRandomStream& InRandomStream);

	ENGINE_API void UpdatePrevBoneLocationsAndVelocities(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, float DeltaTime);

	//If we're updating our position each frame, there's no point in inheriting bone velocity
	inline bool InheritingBoneVelocity()const { return bInheritBoneVelocity && !bUpdatePositionEachFrame; }

	ENGINE_API void SetSourceIndexMode();

	/** Validate that all LOD levels will not have issues with indexing into SourceLocations */
	static ENGINE_API void ValidateLODLevels(UParticleEmitter* Emitter, int32 iModule);
};

/** ModuleLocationBoneSocket instance payload */
struct FModuleLocationBoneSocketInstancePayload
{
	/** The skeletal mesh component used as the source of the sockets */
	TWeakObjectPtr<USkeletalMeshComponent> SourceComponent;
	/** Actor pointer we took the skel mesh comp from. */
	TWeakObjectPtr<AActor> CachedActor;
	/** The last selected index into the socket array */
	int32 LastSelectedIndex;
	/** The position of each bone/socket from the previous tick. Used to calculate the inherited bone velocity when spawning particles. */
	TPreallocatedArrayProxy<FVector3f> PrevFrameBoneSocketPositions;
	/** The velocity of each bone/socket. Used to calculate the inherited bone velocity when spawning particles. */
	TPreallocatedArrayProxy<FVector3f> BoneSocketVelocities;
	/** The pre selected bone socket indices. */
	TPreallocatedArrayProxy<int32> PreSelectedBoneSocketIndices;

	/** Initialize array proxies and map to memory that has been allocated in the emitter's instance data buffer */
	void InitArrayProxies(int32 FixedArraySize)
	{
		// Calculate offsets into instance data buffer for the arrays and initialize the buffer proxies. The allocation 
		// size for these arrays is calculated in RequiredBytesPerInstance.
		const uint32 StructSize = sizeof(FModuleLocationBoneSocketInstancePayload);
		PrevFrameBoneSocketPositions = TPreallocatedArrayProxy<FVector3f>((uint8*)this + StructSize, FixedArraySize);

		uint32 StructOffset = StructSize + (FixedArraySize*sizeof(FVector3f));
		BoneSocketVelocities = TPreallocatedArrayProxy<FVector3f>((uint8*)this + StructOffset, FixedArraySize);
		StructOffset += (FixedArraySize*sizeof(FVector3f));

		PreSelectedBoneSocketIndices = TPreallocatedArrayProxy<int32>((uint8*)this + StructOffset, FixedArraySize);
		StructOffset += (FixedArraySize*sizeof(int32));
	}
};
