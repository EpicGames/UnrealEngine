// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleLocationDirect
 *
 *	Sets the location of particles directly.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationDirect.generated.h"

class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Direct Location"), MinimalAPI)
class UParticleModuleLocationDirect : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The location of the particle at a give time. Retrieved using the particle RelativeTime. 
	 *	IMPORTANT: the particle location is set to this value, thereby over-writing any previous module impacts.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector Location;

	/**
	 *	An offset to apply to the position retrieved from the Location calculation. 
	 *	The offset is retrieved using the EmitterTime. 
	 *	The offset will remain constant over the life of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector LocationOffset;

	/**
	 *	Scales the velocity of the object at a given point in the time-line.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector ScaleFactor;

	/** Currently unused. */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector Direction;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void	PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Update(const FUpdateContext& Context) override;
	ENGINE_API virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	//~ End UParticleModule Interface
};



