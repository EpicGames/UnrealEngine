// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleModuleLocationEmitter
//
// A location module that uses particles from another emitters particles as
// spawn points for its particles.
//
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationEmitter.generated.h"

struct FParticleEmitterInstance;

UENUM()
enum ELocationEmitterSelectionMethod : int
{
	ELESM_Random,
	ELESM_Sequential,
	ELESM_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Emitter Initial Location"), MinimalAPI)
class UParticleModuleLocationEmitter : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	//~=============================================================================
	// Variables
	//~=============================================================================
	// LocationEmitter
	
	/** The name of the emitter to use that the source location for particle. */
	UPROPERTY(EditAnywhere, export, noclear, Category=Location)
	FName EmitterName;

	/** 
	 *	The method to use when selecting a spawn target particle from the emitter.
	 *	Can be one of the following:
	 *		ELESM_Random		Randomly select a particle from the source emitter.
	 *		ELESM_Sequential	Step through each particle from the source emitter in order.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	TEnumAsByte<enum ELocationEmitterSelectionMethod> SelectionMethod;

	/** If true, the spawned particle should inherit the velocity of the source particle. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 InheritSourceVelocity:1;

	/** Amount to scale the source velocity by when inheriting it. */
	UPROPERTY(EditAnywhere, Category=Location)
	float InheritSourceVelocityScale;

	/** If true, the spawned particle should inherit the rotation of the source particle. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 bInheritSourceRotation:1;

	/** Amount to scale the source rotation by when inheriting it. */
	UPROPERTY(EditAnywhere, Category=Location)
	float InheritSourceRotationScale;


	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual uint32	RequiredBytesPerInstance() override;
	//~ End UParticleModule Interface
};



