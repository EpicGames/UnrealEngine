// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleModuleLocationEmitterDirect
//
// A location module that uses particles from another emitters particles as
// position for its particles.
//
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationEmitterDirect.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Emitter Direct Location"), MinimalAPI)
class UParticleModuleLocationEmitterDirect : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** The name of the emitter to use as a source for the location of the particles. */
	UPROPERTY(EditAnywhere, export, noclear, Category=Location)
	FName EmitterName;


	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Update(const FUpdateContext& Context) override;
	//~ End UParticleModule Interface

};



