// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// ParticleModuleLocationPrimitiveBase
// Base class for setting particle spawn locations based on primitives.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationPrimitiveBase.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI)
class UParticleModuleLocationPrimitiveBase : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** Whether the positive X axis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Positive_X:1;

	/** Whether the positive Y axis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Positive_Y:1;

	/** Whether the positive Z axis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Positive_Z:1;

	/** Whether the negative X axis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Negative_X:1;

	/** Whether the negative Y axis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Negative_Y:1;

	/** Whether the negative Zaxis is valid for spawning. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Negative_Z:1;

	/** Whether particles will only spawn on the surface of the primitive. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 SurfaceOnly:1;

	/** Whether the particle should get its velocity from the position within the primitive. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 Velocity:1;

	/** The scale applied to the velocity. (Only used if 'Velocity' is checked). */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat VelocityScale;

	/** The location of the bounding primitive relative to the position of the emitter. */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector StartLocation;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//End UObject Interface

	//@todo document
	ENGINE_API virtual void	DetermineUnitDirection(FParticleEmitterInstance* Owner, FVector& vUnitDir, struct FRandomStream* InRandomStream);
};



