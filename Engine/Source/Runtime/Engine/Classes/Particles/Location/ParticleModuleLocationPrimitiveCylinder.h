// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleModuleLocationPrimitiveCylinder
// Location primitive spawning within a cylinder.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveBase.h"
#include "ParticleModuleLocationPrimitiveCylinder.generated.h"

struct FParticleEmitterInstance;

UENUM()
enum CylinderHeightAxis : int
{
	PMLPC_HEIGHTAXIS_X UMETA(DisplayName="X"),
	PMLPC_HEIGHTAXIS_Y UMETA(DisplayName="Y"),
	PMLPC_HEIGHTAXIS_Z UMETA(DisplayName="Z"),
	PMLPC_HEIGHTAXIS_MAX,
};

UCLASS(editinlinenew, hidecategories = Object, meta = (DisplayName = "Cylinder"), MinimalAPI)
class UParticleModuleLocationPrimitiveCylinder : public UParticleModuleLocationPrimitiveBase
{
	GENERATED_UCLASS_BODY()

	/** If true, get the particle velocity form the radial distance inside the primitive. */
	UPROPERTY(EditAnywhere, Category=Location)
	uint32 RadialVelocity:1;

	/** The radius of the cylinder. */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat StartRadius;

	/** The height of the cylinder, centered about the location. */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat StartHeight;

	/**
	 * Determine the particle system axis that should represent the height of the cylinder.
	 * Can be one of the following:
	 *   PMLPC_HEIGHTAXIS_X - Orient the height along the particle system X-axis.
	 *   PMLPC_HEIGHTAXIS_Y - Orient the height along the particle system Y-axis.
	 *   PMLPC_HEIGHTAXIS_Z - Orient the height along the particle system Z-axis.
	 */
	UPROPERTY(EditAnywhere, Category=Location)
	TEnumAsByte<enum CylinderHeightAxis> HeightAxis;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//~ End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	ENGINE_API void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



