// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveCylinder.h"
#include "ParticleModuleLocationPrimitiveCylinder_Seeded.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Cylinder (Seed)"), MinimalAPI)
class UParticleModuleLocationPrimitiveCylinder_Seeded : public UParticleModuleLocationPrimitiveCylinder
{
	GENERATED_UCLASS_BODY()

	/** The random seed(s) to use for looking up values in StartLocation */
	UPROPERTY(EditAnywhere, Category=RandomSeed)
	struct FParticleRandomSeedInfo RandomSeedInfo;


	//Begin UParticleModule Interface
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo() override
	{
		return &RandomSeedInfo;
	}
	ENGINE_API virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner) override;
	//End UParticleModule Interface
};



