// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleModuleLocationPrimitiveSphere
// Location primitive spawning within a Sphere.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveBase.h"
#include "ParticleModuleLocationPrimitiveSphere.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Sphere"), MinimalAPI)
class UParticleModuleLocationPrimitiveSphere : public UParticleModuleLocationPrimitiveBase
{
	GENERATED_UCLASS_BODY()

	/** The radius of the sphere. Retrieved using EmitterTime. */
	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat StartRadius;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//~ End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	ENGINE_API void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



