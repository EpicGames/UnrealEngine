// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveSphere.h"
#include "ParticleModuleLocationPrimitiveSphere_Seeded.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Sphere (Seed)"), MinimalAPI)
class UParticleModuleLocationPrimitiveSphere_Seeded : public UParticleModuleLocationPrimitiveSphere
{
	GENERATED_UCLASS_BODY()

	/** The random seed(s) to use for looking up values in StartLocation */
	UPROPERTY(EditAnywhere, Category=RandomSeed)
	struct FParticleRandomSeedInfo RandomSeedInfo;


	//Begin UParticleModule Interface
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo() override
	{
		return &RandomSeedInfo;
	}
	ENGINE_API virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner) override;
	//End UParticleModule Interface

};



