// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleModuleLocationPrimitiveTriangle
// Location primitive spawning within a triangle.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationPrimitiveTriangle.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Triangle"), MinimalAPI)
class UParticleModuleLocationPrimitiveTriangle : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()


	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionVector StartOffset;

	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat Height;

	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat Angle;

	UPROPERTY(EditAnywhere, Category=Location)
	struct FRawDistributionFloat Thickness;
	
	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//~ End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	ENGINE_API void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



