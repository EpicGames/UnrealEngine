// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleLocationSkelVertSurface.generated.h"

class UParticleLODLevel;
class UParticleModuleTypeDataBase;
class UParticleSystemComponent;
class USkeletalMeshComponent;
struct FParticleEmitterInstance;
class FSkeletalMeshLODRenderData;
struct FSkelMeshRenderSection;

UENUM()
enum ELocationSkelVertSurfaceSource : int
{
	VERTSURFACESOURCE_Vert UMETA(DisplayName="Vertices"),
	VERTSURFACESOURCE_Surface UMETA(DisplayName="Surfaces"),
	VERTSURFACESOURCE_MAX,
};

/** ModuleLocationVertSurface instance payload */
struct FModuleLocationVertSurfaceInstancePayload
{
	/** The skeletal mesh component used as the source of the sockets */
	TWeakObjectPtr<USkeletalMeshComponent> SourceComponent;
	/** Actor that owns the skel mesh component we're using. */
	TWeakObjectPtr<AActor> CachedActor;
	/** The index of the vertice this particle system spawns from */
	int32 VertIndex;
	/** The number of valid bone indices that which can be used for . */
	int32 NumValidAssociatedBoneIndices;
	/** Bone indices for the associated bone names. */
	TPreallocatedArrayProxy<int32> ValidAssociatedBoneIndices;
	/** The position of each bone from the previous tick. Used to calculate the inherited bone velocity when spawning particles. */
	TPreallocatedArrayProxy<FVector3f> PrevFrameBonePositions;
	/** The velocity of each bone. Used to calculate the inherited bone velocity when spawning particles. */
	TPreallocatedArrayProxy<FVector3f> BoneVelocities;

	/** Min LOD for the current mesh. INDEX_NONE if the mesh is not valid. */
	int32 MeshMinLOD;

	/** Initialize array proxies and map to memory that has been allocated in the emitter's instance data buffer */
	void InitArrayProxies(int32 FixedArraySize)
	{
		// Calculate offsets into instance data buffer for the arrays and initialize the buffer proxies. The allocation 
		// size for these arrays is calculated in RequiredBytesPerInstance.
		const uint32 StructSize = sizeof(FModuleLocationVertSurfaceInstancePayload);
		ValidAssociatedBoneIndices = TPreallocatedArrayProxy<int32>((uint8*)this + StructSize, FixedArraySize);

		uint32 StructOffset = StructSize + (FixedArraySize * sizeof(int32));
		PrevFrameBonePositions = TPreallocatedArrayProxy<FVector3f>((uint8*)this + StructOffset, FixedArraySize);

		StructOffset = StructSize + (FixedArraySize * sizeof(int32)) + (FixedArraySize * sizeof(FVector3f));
		BoneVelocities = TPreallocatedArrayProxy<FVector3f>((uint8*)this + StructOffset, FixedArraySize);
	}

	inline bool MeshIsValid() { return SourceComponent.IsValid() && MeshMinLOD != INDEX_NONE; }
};

/** ModuleLocationVertSurface per-particle payload - only used if updating each frame */
struct FModuleLocationVertSurfaceParticlePayload
{
	/** The index of the socket this particle is 'attached' to */
	int32 SourceIndex;
};


UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Skel Vert/Surf Location"), MinimalAPI)
class UParticleModuleLocationSkelVertSurface : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Whether the module uses Verts or Surfaces for locations.
	 *
	 *	VERTSURFACESOURCE_Vert		- Use Verts as the source locations.
	 *	VERTSURFACESOURCE_Surface	- Use Surfaces as the source locations.
	 */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	TEnumAsByte<enum ELocationSkelVertSurfaceSource> SourceType;

	/** An offset to apply to each vert/surface */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	FVector UniversalOffset;

	/** If true, update the particle locations each frame with that of the vert/surface */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	uint32 bUpdatePositionEachFrame:1;

	/** If true, rotate mesh emitter meshes to orient w/ the vert/surface */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	uint32 bOrientMeshEmitters:1;

	/** If true, particles inherit the associated bone velocity when spawned */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	uint32 bInheritBoneVelocity:1;

	/** A scale on how much of the bone's velocity a particle will inherit. */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	float InheritVelocityScale;

	/**
	 *	The parameter name of the skeletal mesh actor that supplies the SkelMeshComponent for in-game.
	 */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	FName SkelMeshActorParamName;

#if WITH_EDITORONLY_DATA
	/** The name of the skeletal mesh to use in the editor */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	TObjectPtr<class USkeletalMesh> EditorSkelMesh;

#endif // WITH_EDITORONLY_DATA
	/** This module will only spawn from verts or surfaces associated with the bones in this list */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	TArray<FName> ValidAssociatedBones;

	/** When true use the RestrictToNormal and NormalTolerance values to check surface normals */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	uint32 bEnforceNormalCheck:1;

	/** Use this normal to restrict spawning locations */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	FVector NormalToCompare;

	/** Normal tolerance.  0 degrees means it must be an exact match, 180 degrees means it can be any angle. */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	float NormalCheckToleranceDegrees;

	/** Normal tolerance.  Value between 1.0 and -1.0 with 1.0 being exact match, 0.0 being everything up to
		perpendicular and -1.0 being any direction or don't restrict at all. */
	UPROPERTY()
	float NormalCheckTolerance;

	/**
	 *	Array of material indices that are valid materials to spawn from.
	 *	If empty, any material will be considered valid
	 */
	UPROPERTY(EditAnywhere, Category=VertSurface)
	TArray<int32> ValidMaterialIndices;

	/** If true, particles inherit the associated vertex color on spawn. This feature is not supported for GPU particles. */
	UPROPERTY(EditAnywhere, Category = VertSurface)
	uint32 bInheritVertexColor : 1;

	/** If true, particles inherit the associated UV data on spawn. Accessed through dynamic parameter module X and Y, must be a "Spawn Time Only" parameter on "AutoSet" mode. This feature is not supported for GPU particles. */
	UPROPERTY(EditAnywhere, Category = VertSurface)
	uint32 bInheritUV : 1;

	/** UV channel to inherit from the spawn mesh, internally clamped to those available.  */
	UPROPERTY(EditAnywhere, Category = VertSurface)
	uint32 InheritUVChannel;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//Begin UParticleModule Interface
	ENGINE_API virtual void Spawn(const FSpawnContext& Context) override;
	ENGINE_API virtual void Update(const FUpdateContext& Context) override;
	ENGINE_API virtual void FinalUpdate(const FUpdateContext& Context) override;
	ENGINE_API virtual uint32	PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData) override;
	ENGINE_API virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	ENGINE_API virtual uint32	RequiredBytesPerInstance() override;
	virtual bool	TouchesMeshRotation() const override { return true; }
	ENGINE_API virtual void	AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual bool CanTickInAnyThread() override
	{
		return false;
	}
#if WITH_EDITOR
	ENGINE_API virtual int32 GetNumberOfCustomMenuOptions() const override;
	ENGINE_API virtual bool GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const override;
	ENGINE_API virtual bool PerformCustomMenuEntry(int32 InEntryIndex) override;
	ENGINE_API virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//End UParticleModule Interface

	/**
	 *	Retrieve the skeletal mesh component source to use for the current emitter instance.
	 *
	 *	@param	Owner						The particle emitter instance that is being setup
	 *	@param	Owner						The payload data that is being setup.
	 *
	 */
	ENGINE_API void GetSkeletalMeshComponentSource(const FContext& Context, FModuleLocationVertSurfaceInstancePayload* InstancePayload);

	/**
	 *	Retrieve the position for the given socket index.
	 *
	 *	@param	Owner					The particle emitter instance that is being setup
	 *	@param	InSkelMeshComponent		The skeletal mesh component to use as the source
	 *	@param	InPrimaryVertexIndex	The index of the only vertex (vert mode) or the first vertex (surface mode)
	 *	@param	OutPosition				The position for the particle location
	 *	@param	OutRotation				Optional orientation for the particle (mesh emitters)
	 *  @param  bSpawning				When true and when using normal check on surfaces, will return false if the check fails.
	 *	
	 *	@return	bool					true if successful, false if not
	 */
	ENGINE_API bool GetParticleLocation(const FContext& Context, FModuleLocationVertSurfaceInstancePayload* InstPayload, USkeletalMeshComponent* InSkelMeshComponent, int32 InPrimaryVertexIndex, FVector& OutPosition, FQuat& OutRotation, bool bSpawning = false);

	/**
	 *  Check to see if the vert is influenced by a bone on our approved list.
	 *
	 *	@param	Owner					The particle emitter instance that is being setup
	 *	@param	InSkelMeshComponent		The skeletal mesh component to use as the source
	 *  @param  InVertexIndex			The vertex index of the vert to check.
	 *  @param	OutBoneIndex			Optional return of matching bone index
	 *
	 *  @return bool					true if it is influenced by an approved bone, false otherwise.
	 */
	ENGINE_API bool VertInfluencedByActiveBone(FParticleEmitterInstance* Owner, FModuleLocationVertSurfaceInstancePayload* InstPayload, USkeletalMeshComponent* InSkelMeshComponent, int32 InVertexIndex, int32* OutBoneIndex = NULL);

	/**
	 *	Updates the indices list with the bone index for each named bone in the editor exposed values.
	 *	
	 *	@param	Owner		The FParticleEmitterInstance that 'owns' the particle.
	 */
	ENGINE_API void UpdateBoneIndicesList(FParticleEmitterInstance* Owner);

private:
	/** Helper function for concrete types. */
	ENGINE_API bool VertInfluencedByActiveBoneTyped(FSkeletalMeshLODRenderData& LODData, int32 LODIndex, const FSkelMeshRenderSection& Section, int32 VertIndex, USkeletalMeshComponent* InSkelMeshComponent, FModuleLocationVertSurfaceInstancePayload* InstancePayload, int32* OutBoneIndex);
};
