// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocation.h"
#include "ParticleModuleLocationWorldOffset.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, meta=(DisplayName = "World Offset"), MinimalAPI)
class UParticleModuleLocationWorldOffset : public UParticleModuleLocation
{
	GENERATED_UCLASS_BODY()


protected:
	//Begin UParticleModuleLocation Interface
	ENGINE_API virtual void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream) override;
	//End UParticleModuleLocation Interface
};

