// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationWorldOffset.h"
#include "ParticleModuleLocationWorldOffset_Seeded.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, meta=(DisplayName = "World Offset (Seed)"), MinimalAPI)
class UParticleModuleLocationWorldOffset_Seeded : public UParticleModuleLocationWorldOffset
{
	GENERATED_UCLASS_BODY()

	/** The random seed(s) to use for looking up values in StartLocation */
	UPROPERTY(EditAnywhere, Category=RandomSeed)
	struct FParticleRandomSeedInfo RandomSeedInfo;


	//Begin UParticleModule Interface
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo() override
	{
		return &RandomSeedInfo;
	}
	ENGINE_API virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner) override;
	//End UParticleModule Interface
};



