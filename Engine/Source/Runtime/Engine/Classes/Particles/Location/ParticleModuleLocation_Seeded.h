// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocation.h"
#include "ParticleModuleLocation_Seeded.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Initial Location (Seed)"), MinimalAPI)
class UParticleModuleLocation_Seeded : public UParticleModuleLocation
{
	GENERATED_UCLASS_BODY()

	/** The random seed(s) to use for looking up values in StartLocation */
	UPROPERTY(EditAnywhere, Category=RandomSeed)
	struct FParticleRandomSeedInfo RandomSeedInfo;


	//Begin UParticleModule Interface
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo() override
	{
		return &RandomSeedInfo;
	}
	ENGINE_API virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner) override;
	//End UParticleModule Interface
};



