// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModuleSourceMovement.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Source Movement"), MinimalAPI)
class UParticleModuleSourceMovement : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The scale factor to apply to the source movement before adding to the particle location.
	 *	The value is looked up using the particles RELATIVE time [0..1].
	 */
	UPROPERTY(EditAnywhere, Category=SourceMovement)
	struct FRawDistributionVector SourceMovementScale;

	//Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	ENGINE_API virtual void FinalUpdate(const FUpdateContext& Context) override;
	ENGINE_API virtual bool CanTickInAnyThread() override;
	//End UParticleModule Interface

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();
};



