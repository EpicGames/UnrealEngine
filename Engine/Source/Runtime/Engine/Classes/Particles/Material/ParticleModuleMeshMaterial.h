// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Material/ParticleModuleMaterialBase.h"
#include "ParticleModuleMeshMaterial.generated.h"

class UParticleLODLevel;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Mesh Material"))
class UParticleModuleMeshMaterial : public UParticleModuleMaterialBase
{
	GENERATED_UCLASS_BODY()

	/** The array of materials to apply to the mesh particles. */
	UPROPERTY(EditAnywhere, Category=MeshMaterials)
	TArray<TObjectPtr<class UMaterialInterface>> MeshMaterials;


	//Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual uint32	RequiredBytesPerInstance() override;

#if WITH_EDITOR
	virtual bool	IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//End UParticleModule Interface
};
