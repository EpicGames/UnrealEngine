// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "ParticleModulePivotOffset.generated.h"

class UParticleLODLevel;

UCLASS(editinlinenew, hidecategories=Object, DisplayName="Pivot Offset", MinimalAPI)
class UParticleModulePivotOffset : public UParticleModuleLocationBase
{
	GENERATED_UCLASS_BODY()

	/** Offset applied in UV space to the particle vertex positions. Defaults to (0.5,0.5) putting the pivot in the centre of the partilce. */
	UPROPERTY(EditAnywhere, Category=PivotOffset)
	FVector2D PivotOffset;

	/** Initializes the default values for this property */
	ENGINE_API void InitializeDefaults();

	//Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	ENGINE_API virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	//End UParticleModule Interface

#if WITH_EDITOR
	ENGINE_API virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

};



