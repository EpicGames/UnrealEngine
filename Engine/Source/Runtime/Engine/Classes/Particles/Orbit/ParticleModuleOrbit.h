// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Orbit/ParticleModuleOrbitBase.h"
#include "ParticleModuleOrbit.generated.h"

class UParticleLODLevel;
class UParticleModuleTypeDataBase;
struct FParticleEmitterInstance;

UENUM()
enum EOrbitChainMode : int
{
	/** Add the module values to the previous results						*/
	EOChainMode_Add UMETA(DisplayName="Add"),
	/**	Multiply the module values by the previous results					*/
	EOChainMode_Scale UMETA(DisplayName="Scale"),
	/**	'Break' the chain and apply the values from the	previous results	*/
	EOChainMode_Link UMETA(DisplayName="Link"),
	EOChainMode_MAX,
};

/**
 *	Container struct for holding options on the data updating for the module.
 */
USTRUCT()
struct FOrbitOptions
{
	GENERATED_USTRUCT_BODY()

	/**
	 *	Whether to process the data during spawning.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	uint32 bProcessDuringSpawn:1;

	/**
	 *	Whether to process the data during updating.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	uint32 bProcessDuringUpdate:1;

	/**
	 *	Whether to use emitter time during data retrieval.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	uint32 bUseEmitterTime:1;



		FOrbitOptions()
		: bProcessDuringSpawn(true)
		, bProcessDuringUpdate(false)
		, bUseEmitterTime(false)
		{
		}
	
};

UCLASS(editinlinenew, hidecategories=(Object, Orbit), MinimalAPI, meta=(DisplayName = "Orbit"))
class UParticleModuleOrbit : public UParticleModuleOrbitBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Orbit modules will chain together in the order they appear in the module stack.
	 *	The combination of a module with the one prior to it is defined by using one
	 *	of the following enumerations:
	 *		EOChainMode_Add		Add the values to the previous results
	 *		EOChainMode_Scale	Multiply the values by the previous results
	 *		EOChainMode_Link	'Break' the chain and apply the values from the	previous results
	 */
	UPROPERTY(EditAnywhere, Category=Chaining)
	TEnumAsByte<enum EOrbitChainMode> ChainMode;

	/** The amount to offset the sprite from the particle position. */
	UPROPERTY(EditAnywhere, Category=Offset)
	struct FRawDistributionVector OffsetAmount;

	/** The options associated with the OffsetAmount look-up. */
	UPROPERTY(EditAnywhere, Category=Offset)
	struct FOrbitOptions OffsetOptions;

	/**
	 *	The amount (in 'turns') to rotate the offset about the particle position.
	 *		0.0 = no rotation
	 *		0.5	= 180 degree rotation
	 *		1.0 = 360 degree rotation
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionVector RotationAmount;

	/** The options associated with the RotationAmount look-up. */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FOrbitOptions RotationOptions;

	/**
	 *	The rate (in 'turns') at which to rotate the offset about the particle positon.
	 *		0.0 = no rotation
	 *		0.5	= 180 degree rotation
	 *		1.0 = 360 degree rotation
	 */
	UPROPERTY(EditAnywhere, Category=RotationRate)
	struct FRawDistributionVector RotationRateAmount;

	/** The options associated with the RotationRateAmount look-up. */
	UPROPERTY(EditAnywhere, Category=RotationRate)
	struct FOrbitOptions RotationRateOptions;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	// End UObject Interface

	//Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual uint32	RequiredBytesPerInstance() override;
#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//End UParticleModule Interface

protected:
	friend class FParticleModuleOrbitDetails;
};



