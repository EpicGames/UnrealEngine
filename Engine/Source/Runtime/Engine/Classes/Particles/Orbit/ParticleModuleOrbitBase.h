// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleOrbitBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Orbit"))
class UParticleModuleOrbitBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	If true, distribution values will be retrieved using the EmitterTime.
	 *	If false (default), they will be retrieved using the Particle.RelativeTime.
	 */
	UPROPERTY(EditAnywhere, Category=Orbit)
	uint32 bUseEmitterTime:1;

};

