// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Orientation/ParticleModuleOrientationBase.h"
#include "ParticleModuleOrientationAxisLock.generated.h"

struct FParticleEmitterInstance;

// Flags indicating lock
UENUM()
enum EParticleAxisLock : int
{
	/** No locking to an axis...							*/
	EPAL_NONE UMETA(DisplayName="None"),
	/** Lock the sprite facing towards the positive X-axis	*/
	EPAL_X UMETA(DisplayName="X"),
	/** Lock the sprite facing towards the positive Y-axis	*/
	EPAL_Y UMETA(DisplayName="Y"),
	/** Lock the sprite facing towards the positive Z-axis	*/
	EPAL_Z UMETA(DisplayName="Z"),
	/** Lock the sprite facing towards the negative X-axis	*/
	EPAL_NEGATIVE_X UMETA(DisplayName="-X"),
	/** Lock the sprite facing towards the negative Y-axis	*/
	EPAL_NEGATIVE_Y UMETA(DisplayName="-Y"),
	/** Lock the sprite facing towards the negative Z-axis	*/
	EPAL_NEGATIVE_Z UMETA(DisplayName="-Z"),
	/** Lock the sprite rotation on the X-axis				*/
	EPAL_ROTATE_X UMETA(DisplayName="Rotate X"),
	/** Lock the sprite rotation on the Y-axis				*/
	EPAL_ROTATE_Y UMETA(DisplayName="Rotate Y"),
	/** Lock the sprite rotation on the Z-axis				*/
	EPAL_ROTATE_Z UMETA(DisplayName="Rotate Z"),
	EPAL_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Lock Axis"))
class UParticleModuleOrientationAxisLock : public UParticleModuleOrientationBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The lock axis flag setting.
	 *	Can be one of the following:
	 *		EPAL_NONE			No locking to an axis.
	 *		EPAL_X				Lock the sprite facing towards +X.
	 *		EPAL_Y				Lock the sprite facing towards +Y.
	 *		EPAL_Z				Lock the sprite facing towards +Z.
	 *		EPAL_NEGATIVE_X		Lock the sprite facing towards -X.
	 *		EPAL_NEGATIVE_Y		Lock the sprite facing towards -Y.
	 *		EPAL_NEGATIVE_Z		Lock the sprite facing towards -Z.
	 *		EPAL_ROTATE_X		Lock the sprite rotation on the X-axis.
	 *		EPAL_ROTATE_Y		Lock the sprite rotation on the Y-axis.
	 *		EPAL_ROTATE_Z		Lock the sprite rotation on the Z-axis.
	 */
	UPROPERTY(EditAnywhere, Category=Orientation)
	TEnumAsByte<EParticleAxisLock> LockAxisFlags;


	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	//~ End UParticleModule Interface

	//@todo document
	virtual void	SetLockAxis(EParticleAxisLock eLockFlags);
};



