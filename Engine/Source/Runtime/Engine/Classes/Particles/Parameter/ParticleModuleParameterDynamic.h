// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Particles/Parameter/ParticleModuleParameterBase.h"
#include "ParticleEmitterInstances.h"
#include "Particles/ParticleSystemComponent.h"
#include "ParticleModuleParameterDynamic.generated.h"

class UInterpCurveEdSetup;
class UParticleEmitter;
class UParticleLODLevel;
class UParticleModuleTypeDataBase;

/**
 *	EmitterDynamicParameterValue
 *	Enumeration indicating the way a dynamic parameter should be set.
 */
UENUM()
enum EEmitterDynamicParameterValue : int
{
	/** UserSet - use the user set values in the distribution (the default) */
	EDPV_UserSet,
	/** AutoSet - ignore values set in the distribution, another module will handle this data */
	EDPV_AutoSet,
	/** VelocityX - pass the particle velocity along the X-axis thru */
	EDPV_VelocityX,
	/** VelocityY - pass the particle velocity along the Y-axis thru */
	EDPV_VelocityY,
	/** VelocityZ - pass the particle velocity along the Z-axis thru */
	EDPV_VelocityZ,
	/** VelocityMag - pass the particle velocity magnitude thru */
	EDPV_VelocityMag,
	EDPV_MAX,
};

/** Helper structure for displaying the parameter. */
USTRUCT()
struct FEmitterDynamicParameter
{
	GENERATED_USTRUCT_BODY()

	/** The parameter name - from the material DynamicParameter expression. READ-ONLY */
	UPROPERTY(Category=EmitterDynamicParameter, VisibleAnywhere)
	FName ParamName;

	/** If true, use the EmitterTime to retrieve the value, otherwise use Particle RelativeTime. */
	UPROPERTY(EditAnywhere, Category=EmitterDynamicParameter)
	uint32 bUseEmitterTime:1;

	/** If true, only set the value at spawn time of the particle, otherwise update each frame. */
	UPROPERTY(EditAnywhere, Category=EmitterDynamicParameter)
	uint32 bSpawnTimeOnly:1;

	/** Where to get the parameter value from. */
	UPROPERTY(EditAnywhere, Category=EmitterDynamicParameter)
	TEnumAsByte<enum EEmitterDynamicParameterValue> ValueMethod;

	/** If true, scale the velocity value selected in ValueMethod by the evaluated ParamValue. */
	UPROPERTY(EditAnywhere, Category=EmitterDynamicParameter)
	uint32 bScaleVelocityByParamValue:1;

	/** The distriubtion for the parameter value. */
	UPROPERTY(EditAnywhere, Category=EmitterDynamicParameter)
	struct FRawDistributionFloat ParamValue;


	FEmitterDynamicParameter()
		: bUseEmitterTime(false)
		, bSpawnTimeOnly(false)
		, ValueMethod(0)
		, bScaleVelocityByParamValue(false)
	{
	}
	FEmitterDynamicParameter(FName InParamName, uint32 InUseEmitterTime, TEnumAsByte<enum EEmitterDynamicParameterValue> InValueMethod, UDistributionFloatConstant* InDistribution)
		: ParamName(InParamName)
		, bUseEmitterTime(InUseEmitterTime)
		, bSpawnTimeOnly(false)
		, ValueMethod(InValueMethod)
		, bScaleVelocityByParamValue(false)
	{
		ParamValue.Distribution = InDistribution;
	}

};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Dynamic"))
class UParticleModuleParameterDynamic : public UParticleModuleParameterBase
{
	GENERATED_UCLASS_BODY()

	/** The dynamic parameters this module uses. */
	UPROPERTY(EditAnywhere, editfixedsize, Category=ParticleModuleParameterDynamic)
	TArray<struct FEmitterDynamicParameter> DynamicParams;

	/** Flags for optimizing update */
	UPROPERTY()
	int32 UpdateFlags;

	UPROPERTY()
	uint32 bUsesVelocity:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
	virtual void	PostLoad() override;
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//~ Begin UObject Interface

	//~ Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual void	GetCurveObjects(TArray<FParticleCurvePair>& OutCurves) override;
	virtual bool WillGeneratedModuleBeIdentical(UParticleLODLevel* SourceLODLevel, UParticleLODLevel* DestLODLevel, float Percentage) override
	{
		// The assumption is that at 100%, ANY module will be identical...
		// (Although this is virtual to allow over-riding that assumption on a case-by-case basis!)
		return true;
	}
	virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList) override;
	virtual void GetParticleParametersUtilized(TArray<FString>& ParticleParameterList) override;
	virtual void RefreshModule(UInterpCurveEdSetup* EdSetup, UParticleEmitter* InEmitter, int32 InLODLevel) override;
	//~ End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
	
	/**
	 *	Update the parameter names with the given material...
	 *
	 *	@param	InMaterialInterface	Pointer to the material interface
	 *	@param	bIsMeshEmitter		true if the emitter is a mesh emitter...
	 *
	 */
	virtual void UpdateParameterNames(UMaterialInterface* InMaterialInterface);

	
	/**
	 *	Retrieve the value for the parameter at the given index.
	 *
	 *	@param	InDynParams		The FEmitterDynamicParameter to fetch the value for
	 *	@param	Particle		The particle we are getting the value for.
	 *	@param	Owner			The FParticleEmitterInstance owner of the particle.
	 *	@param	InRandomStream	The random stream to use when retrieving the value
	 *
	 *	@return	float			The value for the parameter.
	 */
	inline float GetParameterValue(const FContext& Context, FEmitterDynamicParameter& InDynParams, FBaseParticle& Particle, FParticleEmitterInstance* Owner, struct FRandomStream* InRandomStream)
	{
		float ScaleValue = 1.0f;
		float DistributionValue = 1.0f;
		switch (InDynParams.ValueMethod)
		{
		case EDPV_VelocityX:
			ScaleValue = Particle.Velocity.X;
			break;
		case EDPV_VelocityY:
			ScaleValue = Particle.Velocity.Y;
			break;
		case EDPV_VelocityZ:
			ScaleValue = Particle.Velocity.Z;
			break;
		case EDPV_VelocityMag:
			ScaleValue = Particle.Velocity.Size();
			break;
		default:
			//case EDPV_UserSet:
			//case EDPV_AutoSet:
			break;
		}

		if ((InDynParams.bScaleVelocityByParamValue == true) || (InDynParams.ValueMethod == EDPV_UserSet))
		{
			float TimeValue = InDynParams.bUseEmitterTime ? Owner->EmitterTime : Particle.RelativeTime;
			DistributionValue = InDynParams.ParamValue.GetValue(TimeValue, Context.GetDistributionData(), InRandomStream);
		}

		return DistributionValue * ScaleValue;
	}

	/**
	 *	Retrieve the value for the parameter at the given index.
	 *
	 *	@param	InDynParams		The FEmitterDynamicParameter to fetch the value for
	 *	@param	Particle		The particle we are getting the value for.
	 *	@param	Owner			The FParticleEmitterInstance owner of the particle.
	 *	@param	InRandomStream	The random stream to use when retrieving the value
	 *
	 *	@return	float			The value for the parameter.
	 */
	inline float GetParameterValue_UserSet(const FContext& Context, FEmitterDynamicParameter& InDynParams, FBaseParticle& Particle, FParticleEmitterInstance* Owner, struct FRandomStream* InRandomStream)
	{
		return InDynParams.ParamValue.GetValue(InDynParams.bUseEmitterTime ? Owner->EmitterTime : Particle.RelativeTime, Context.GetDistributionData(), InRandomStream);
	}

	/**
	 *	Set the UpdatesFlags and bUsesVelocity
	 */
	virtual	void UpdateUsageFlags();

	virtual bool CanTickInAnyThread() override;
};



