// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic.h"
#include "ParticleModuleParameterDynamic_Seeded.generated.h"

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Dynamic (Seed)"))
class UParticleModuleParameterDynamic_Seeded : public UParticleModuleParameterDynamic
{
	GENERATED_UCLASS_BODY()

	/** The random seed(s) to use for looking up values in StartLocation */
	UPROPERTY(EditAnywhere, Category=RandomSeed)
	struct FParticleRandomSeedInfo RandomSeedInfo;


	//Begin UParticleModule Interface
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo() override
	{
		return &RandomSeedInfo;
	}
	virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner) override;
	//End UParticleModule Interface
};



