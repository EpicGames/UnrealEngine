// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleEmitter
// The base class for any particle emitter objects.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitter.generated.h"

class IParticleEmitterInstanceOwner;
class UInterpCurveEdSetup;
class UParticleLODLevel;
class UParticleSystemComponent;

enum EDetailMode : int;

//~=============================================================================
//	Burst emissions
//~=============================================================================
UENUM()
enum EParticleBurstMethod : int
{
	EPBM_Instant UMETA(DisplayName="Instant"),
	EPBM_Interpolated UMETA(DisplayName="Interpolated"),
	EPBM_MAX,
};

//~=============================================================================
//	SubUV-related
//~=============================================================================
UENUM()
enum EParticleSubUVInterpMethod : int
{
	PSUVIM_None UMETA(DisplayName="None"),
	PSUVIM_Linear UMETA(DisplayName="Linear"),
	PSUVIM_Linear_Blend UMETA(DisplayName="Linear Blend"),
	PSUVIM_Random UMETA(DisplayName="Random"),
	PSUVIM_Random_Blend UMETA(DisplayName="Random Blend"),
	PSUVIM_MAX,
};

//~=============================================================================
//	Cascade-related
//~=============================================================================
UENUM()
enum EEmitterRenderMode : int
{
	ERM_Normal UMETA(DisplayName="Normal"),
	ERM_Point UMETA(DisplayName="Point"),
	ERM_Cross UMETA(DisplayName="Cross"),
	ERM_LightsOnly UMETA(DisplayName = "Lights Only"),
	ERM_None UMETA(DisplayName = "None"),
	ERM_MAX,
};

USTRUCT()
struct FParticleBurst
{
	GENERATED_USTRUCT_BODY()

	/** The number of particles to burst */
	UPROPERTY(EditAnywhere, Category=ParticleBurst)
	int32 Count;

	/** If >= 0, use as a range [CountLow..Count] */
	UPROPERTY(EditAnywhere, Category=ParticleBurst)
	int32 CountLow;

	/** The time at which to burst them (0..1: emitter lifetime) */
	UPROPERTY(EditAnywhere, Category=ParticleBurst)
	float Time;



		FParticleBurst()
		: Count(0)
		, CountLow(-1)		// Disabled by default...
		, Time(0.0f)
		{
		}
	
};

DECLARE_STATS_GROUP(TEXT("Emitters"), STATGROUP_Emitters, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Emitters"), STATGROUP_EmittersRT, STATCAT_Advanced);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("STAT_EmittersStatGroupTester"), STAT_EmittersStatGroupTester, STATGROUP_Emitters, ENGINE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("STAT_EmittersRTStatGroupTester"), STAT_EmittersRTStatGroupTester, STATGROUP_EmittersRT, ENGINE_API);

UCLASS(hidecategories=Object, editinlinenew, abstract, MinimalAPI)
class UParticleEmitter : public UObject
{
	GENERATED_UCLASS_BODY()

	//~=============================================================================
	//	General variables
	//~=============================================================================
	/** The name of the emitter. */
	UPROPERTY(EditAnywhere, Category=Particle)
	FName EmitterName;

	UPROPERTY(transient)
	int32 SubUVDataOffset;

	/**
	 *	How to render the emitter particles. Can be one of the following:
	 *		ERM_Normal	- As the intended sprite/mesh
	 *		ERM_Point	- As a 2x2 pixel block with no scaling and the color set in EmitterEditorColor
	 *		ERM_Cross	- As a cross of lines, scaled to the size of the particle in EmitterEditorColor
	 *		ERM_None	- Do not render
	 */
	UPROPERTY(EditAnywhere, Category=Cascade)
	TEnumAsByte<enum EEmitterRenderMode> EmitterRenderMode;

	/** The significance level required of this emitter's owner for this emitter to be active. */
	UPROPERTY(EditAnywhere, Category = Significance)
	EParticleSignificanceLevel SignificanceLevel;

	TEnumAsByte<EParticleAxisLock> LockAxisFlags;

	/** If true, maintains some legacy spawning behavior. */
	UPROPERTY(EditAnywhere, Category = Particle, AdvancedDisplay)
	uint8 bUseLegacySpawningBehavior : 1;

	//////////////////////////////////////////////////////////////////////////
	// Below is information udpated by calling CacheEmitterModuleInfo

	uint8 bRequiresLoopNotification : 1;
	uint8 bAxisLockEnabled : 1;
	uint8 bMeshRotationActive : 1;

	UPROPERTY()
	uint8 ConvertedModules:1;

	/** If true, then show only this emitter in the editor */
	UPROPERTY(transient)
	uint8 bIsSoloing:1;

	/** 
	 *	If true, then this emitter was 'cooked out' by the cooker. 
	 *	This means it was completely disabled, but to preserve any
	 *	indexing schemes, it is left in place.
	 */
	UPROPERTY()
	uint8 bCookedOut:1;

	/** When true, if the current LOD is disabled the emitter will be kept alive. Otherwise, the emitter will be considered complete if the current LOD is disabled. */
	UPROPERTY(EditAnywhere, Category = Particle)
	uint8 bDisabledLODsKeepEmitterAlive : 1;
	
	/** When true, emitters deemed insignificant will have their tick and render disabled Instantly. When false they will simple stop spawning new particles. */
	UPROPERTY(EditAnywhere, Category = Significance)
	uint8 bDisableWhenInsignficant : 1;

	/** Particle alignment overrides */
	uint8 bRemoveHMDRollInVR : 1;

#if WITH_EDITORONLY_DATA

	/** This value indicates the emitter should be drawn 'collapsed' in Cascade */
	UPROPERTY(EditAnywhere, Category=Cascade)
	uint8 bCollapsed:1;

	/**
	 *	The color of the emitter in the curve editor and debug rendering modes.
	 */
	UPROPERTY(EditAnywhere, Category=Cascade)
	FColor EmitterEditorColor;

#endif // WITH_EDITORONLY_DATA
	//~=============================================================================
	//	'Private' data - not required by the editor
	//~=============================================================================
	UPROPERTY(instanced)
	TArray<TObjectPtr<class UParticleLODLevel>> LODLevels;

	UPROPERTY()
	int32 PeakActiveParticles;

	//~=============================================================================
	//	Performance/LOD Data
	//~=============================================================================
	
	/**
	 *	Initial allocation count - overrides calculated peak count if > 0
	 */
	UPROPERTY(EditAnywhere, Category=Particle)
	int32 InitialAllocationCount;

	UPROPERTY(EditAnywhere, Category = Particle)
	float QualityLevelSpawnRateScale;

	/** Detail mode: Set flags reflecting which system detail mode you want the emitter to be ticked and rendered in */
	UPROPERTY(EditAnywhere, Category = Particle, meta = (Bitmask, BitmaskEnum = "/Script/Engine.EParticleDetailMode"))
	uint32 DetailModeBitmask;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, VisibleAnywhere, Category = Particle, DisplayName="Current Detailmodes")
	FString DetailModeDisplay;
#endif // WITH_EDITORONLY_DATA

	/** Map module pointers to their offset into the particle data.		*/
	TMap<UParticleModule*, uint32> ModuleOffsetMap;

	/** Map module pointers to their offset into the instance data.		*/
	TMap<UParticleModule*, uint32> ModuleInstanceOffsetMap;

	/** Map module pointers to their offset into the instance data.		*/
	TMap<UParticleModule*, uint32> ModuleRandomSeedInstanceOffsetMap;
	
	/** Materials collected from any MeshMaterial modules */
	TArray<class UMaterialInterface*> MeshMaterials;

	int32 DynamicParameterDataOffset;
	int32 LightDataOffset;
	float LightVolumetricScatteringIntensity;
	int32 CameraPayloadOffset;
	int32 ParticleSize;
	int32 ReqInstanceBytes;
	FVector2D PivotOffset;
	int32 TypeDataOffset;
	int32 TypeDataInstanceOffset;

	float MinFacingCameraBlendDistance;
	float MaxFacingCameraBlendDistance;

	// Array of modules that want emitter instance data
	TArray<UParticleModule*> ModulesNeedingInstanceData;

	// Array of modules that want emitter random seed instance data
	TArray<UParticleModule*> ModulesNeedingRandomSeedInstanceData;

	/** SubUV animation asset to use for cutout geometry. */
	class USubUVAnimation* RESTRICT SubUVAnimation;

	//////////////////////////////////////////////////////////////////////////

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif

#if WITH_EDITORONLY_DATA
	// Update the at-a-glance display for detail mode
	void UpdateDetailModeDisplayString()
	{
		DetailModeDisplay = "";
		DetailModeDisplay += DetailModeBitmask & (1 << EParticleDetailMode::PDM_Low) ? "Low, " : "";
		DetailModeDisplay += DetailModeBitmask & (1 << EParticleDetailMode::PDM_Medium) ? "Medium, " : "";
		DetailModeDisplay += DetailModeBitmask & (1 << EParticleDetailMode::PDM_High) ? "High, " : "";
		DetailModeDisplay += DetailModeBitmask & (1 << EParticleDetailMode::PDM_Epic) ? "Epic" : "";
	}
#endif // WITH_EDITORONLY_DATA
	virtual void Serialize(FArchive& Ar)override;
	virtual void PostLoad() override;
	virtual bool IsPostLoadThreadSafe() const override;
	//~ End UObject Interface

	// @todo document
	virtual FParticleEmitterInstance* CreateInstance(IParticleEmitterInstanceOwner& InComponent);

	// Sets up this emitter with sensible defaults so we can see some particles as soon as its created.
	virtual void SetToSensibleDefaults() {}

	// @todo document
	virtual void UpdateModuleLists();

	// @todo document
	ENGINE_API void SetEmitterName(FName Name);

	// @todo document
	ENGINE_API FName& GetEmitterName();

	// @todo document
	virtual	void						SetLODCount(int32 LODCount);

	// For Cascade
	// @todo document
	void	AddEmitterCurvesToEditor(UInterpCurveEdSetup* EdSetup);

	// @todo document
	ENGINE_API void	RemoveEmitterCurvesFromEditor(UInterpCurveEdSetup* EdSetup);

	// @todo document
	ENGINE_API void	ChangeEditorColor(FColor& Color, UInterpCurveEdSetup* EdSetup);

	// @todo document
	ENGINE_API void	AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp);

	/** CreateLODLevel
	 *	Creates the given LOD level.
	 *	Intended for editor-time usage.
	 *	Assumes that the given LODLevel will be in the [0..100] range.
	 *	
	 *	@return The index of the created LOD level
	 */
	ENGINE_API int32		CreateLODLevel(int32 LODLevel, bool bGenerateModuleData = true);

	/** IsLODLevelValid
	 *	Returns true if the given LODLevel is one of the array entries.
	 *	Intended for editor-time usage.
	 *	Assumes that the given LODLevel will be in the [0..(NumLODLevels - 1)] range.
	 *	
	 *	@return false if the requested LODLevel is not valid.
	 *			true if the requested LODLevel is valid.
	 */
	ENGINE_API bool	IsLODLevelValid(int32 LODLevel);

	/** GetCurrentLODLevel
	*	Returns the currently set LODLevel. Intended for game-time usage.
	*	Assumes that the given LODLevel will be in the [0..# LOD levels] range.
	*	
	*	@return NULL if the requested LODLevel is not valid.
	*			The pointer to the requested UParticleLODLevel if valid.
	*/
	ENGINE_API UParticleLODLevel* GetCurrentLODLevel(FParticleEmitterInstance* Instance);

	/**
	 * This will update the LOD of the particle in the editor.
	 *
	 * @see GetCurrentLODLevel(FParticleEmitterInstance* Instance)
	 */
	ENGINE_API void EditorUpdateCurrentLOD(FParticleEmitterInstance* Instance);

	/** GetLODLevel
	 *	Returns the given LODLevel. Intended for game-time usage.
	 *	Assumes that the given LODLevel will be in the [0..# LOD levels] range.
	 *	
	 *	@param	LODLevel - the requested LOD level in the range [0..# LOD levels].
	 *
	 *	@return NULL if the requested LODLevel is not valid.
	 *			The pointer to the requested UParticleLODLevel if valid.
	 */
	ENGINE_API UParticleLODLevel*	GetLODLevel(int32 LODLevel);
	
	/**
	 *	Autogenerate the lowest LOD level...
	 *
	 *	@param	bDuplicateHighest	If true, make the level an exact copy of the highest
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	virtual bool		AutogenerateLowestLODLevel(bool bDuplicateHighest = false);
	
	/**
	 *	CalculateMaxActiveParticleCount
	 *	Determine the maximum active particles that could occur with this emitter.
	 *	This is to avoid reallocation during the life of the emitter.
	 *
	 *	@return	true	if the number was determined
	 *			false	if the number could not be determined
	 */
	virtual bool		CalculateMaxActiveParticleCount();

	/**
	 *	Retrieve the parameters associated with this particle system.
	 *
	 *	@param	ParticleSysParamList	The list of FParticleSysParams used in the system
	 *	@param	ParticleParameterList	The list of ParticleParameter distributions used in the system
	 */
	void GetParametersUtilized(TArray<FString>& ParticleSysParamList,
							   TArray<FString>& ParticleParameterList);

	/**
	 * Builds data needed for simulation by the emitter from all modules.
	 */
	void Build();

	/** Pre-calculate data size/offset and other info from modules in this Emitter */
	void CacheEmitterModuleInfo();

	/**
	 *   Calculate spawn rate multiplier based on global effects quality level and emitter's quality scale
 	 */
	float GetQualityLevelSpawnRateMult();

	/** Returns true if the is emitter has any enabled LODs, false otherwise. */
	ENGINE_API bool HasAnyEnabledLODs() const;

	/** Stat id of this object, 0 if nobody asked for it yet */
	STAT(mutable TStatId StatID;)
	STAT(mutable TStatId StatIDRT;)
		
	/**
	* Returns the stat ID of the object...
	* We can't use the normal version of this because those names are meaningless; we need the special name in the emitter
	**/
	inline TStatId GetStatID(bool bForDeferredUse = false) const
	{
#if STATS
		// this is done to avoid even registering stats for a disabled group (unless we plan on using it later)
		if (bForDeferredUse || FThreadStats::IsCollectingData(GET_STATID(STAT_EmittersStatGroupTester)))
		{
			if (!StatID.IsValidStat())
			{
				CreateStatID();
			}
			return StatID;
		}
#endif
		return TStatId(); // not doing stats at the moment, or ever
	}

	inline TStatId GetStatIDRT(bool bForDeferredUse = false) const
	{
#if STATS
		// this is done to avoid even registering stats for a disabled group (unless we plan on using it later)
		if (bForDeferredUse || FThreadStats::IsCollectingData(GET_STATID(STAT_EmittersRTStatGroupTester)))
		{
			if (!StatIDRT.IsValidStat())
			{
				CreateStatID();
			}
			return StatIDRT;
		}
#endif
		return TStatId(); // not doing stats at the moment, or ever
	}
	/**
	* Creates this stat ID for the emitter...and handle a null this pointer
	**/
#if STATS
	void CreateStatID() const;
#endif

	/** Returns if this emitter is considered significant for the passed requirement. */
	ENGINE_API bool IsSignificant(EParticleSignificanceLevel RequiredSignificance);
};


