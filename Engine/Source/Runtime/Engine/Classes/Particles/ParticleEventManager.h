// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Particles/ParticleSystemComponent.h"

#include "ParticleEventManager.generated.h"

UCLASS(config=Game, NotBlueprintable, notplaceable)
class AParticleEventManager : public AActor
{
	GENERATED_UCLASS_BODY()
		
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	virtual void HandleParticleSpawnEvents( UParticleSystemComponent* Component, const TArray<FParticleEventSpawnData>& SpawnEvents );
	virtual void HandleParticleDeathEvents( UParticleSystemComponent* Component, const TArray<FParticleEventDeathData>& DeathEvents );
	virtual void HandleParticleCollisionEvents( UParticleSystemComponent* Component, const TArray<FParticleEventCollideData>& CollisionEvents );
	virtual void HandleParticleBurstEvents( UParticleSystemComponent* Component, const TArray<FParticleEventBurstData>& BurstEvents );
};

