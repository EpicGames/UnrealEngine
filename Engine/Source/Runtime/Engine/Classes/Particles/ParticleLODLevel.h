// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleLODLevel
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ParticleLODLevel.generated.h"

class UInterpCurveEdSetup;
class UParticleModule;
class UMaterialInterface;
struct FStreamingRenderAssetPrimitiveInfo;

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UParticleLODLevel : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The index value of the LOD level												*/
	UPROPERTY()
	int32 Level;

	/** True if the LOD level is enabled, meaning it should be updated and rendered.	*/
	UPROPERTY()
	uint32 bEnabled:1;

	/** The required module for this LOD level											*/
	UPROPERTY(instanced)
	TObjectPtr<class UParticleModuleRequired> RequiredModule;

	/** An array of particle modules that contain the adjusted data for the LOD level	*/
	UPROPERTY(instanced)
	TArray<TObjectPtr<class UParticleModule>> Modules;

	// Module<SINGULAR> used for emitter type "extension".
	UPROPERTY(export)
	TObjectPtr<class UParticleModuleTypeDataBase> TypeDataModule;

	/** The SpawnRate/Burst module - required by all emitters. */
	UPROPERTY(export)
	TObjectPtr<class UParticleModuleSpawn> SpawnModule;

	/** The optional EventGenerator module. */
	UPROPERTY(export)
	TObjectPtr<class UParticleModuleEventGenerator> EventGenerator;

	/** SpawningModules - These are called to determine how many particles to spawn.	*/
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UParticleModuleSpawnBase>> SpawningModules;

	/** SpawnModules - These are called when particles are spawned.						*/
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UParticleModule>> SpawnModules;

	/** UpdateModules - These are called when particles are updated.					*/
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UParticleModule>> UpdateModules;

	/** OrbitModules 
	 *	These are used to do offsets of the sprite from the particle location.
	 */
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UParticleModuleOrbit>> OrbitModules;

	/** Event receiver modules only! */
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UParticleModuleEventReceiverBase>> EventReceiverModules;

	UPROPERTY()
	uint32 ConvertedModules:1;

	UPROPERTY()
	int32 PeakActiveParticles;


	//~ Begin UObject Interface
	virtual void	PostLoad() override;
	virtual bool	IsPostLoadThreadSafe() const override;
	//~ End UObject Interface

	// @todo document
	virtual void	UpdateModuleLists();

	// @todo document
	virtual bool	GenerateFromLODLevel(UParticleLODLevel* SourceLODLevel, float Percentage, bool bGenerateModuleData = true);

	/**
	 *	CalculateMaxActiveParticleCount
	 *	Determine the maximum active particles that could occur with this emitter.
	 *	This is to avoid reallocation during the life of the emitter.
	 *
	 *	@return		The maximum active particle count for the LOD level.
	 */
	virtual int32	CalculateMaxActiveParticleCount();

	/** Update to the new SpawnModule method */
	void	ConvertToSpawnModule();
		
	/** @return the index of the given module if it is contained in the LOD level */
	int32		GetModuleIndex(UParticleModule* InModule);

	/** @return the module at the given index if it is contained in the LOD level */
	ENGINE_API UParticleModule* GetModuleAtIndex(int32 InIndex);

	/**
	 *	Sets the LOD 'Level' to the given value, properly updating the modules LOD validity settings.
	 *	This function assumes that any error-checking of values was done by the caller!
	 *	It also assumes that when inserting an LOD level, indices will be shifted from lowest to highest...
	 *	When removing one, the will go from highest to lowest.
	 */
	virtual void	SetLevelIndex(int32 InLevelIndex);

	// For Cascade
	void	AddCurvesToEditor(UInterpCurveEdSetup* EdSetup);
	void	RemoveCurvesFromEditor(UInterpCurveEdSetup* EdSetup);
	void	ChangeEditorColor(FColor& Color, UInterpCurveEdSetup* EdSetup);

	/**
	 *	Return true if the given module is editable for this LOD level.
	 *	
	 *	@param	InModule	The module of interest.
	 *	@return	true		If it is editable for this LOD level.
	 *			false		If it is not.
	 */
	ENGINE_API bool	IsModuleEditable(UParticleModule* InModule);

	/**
	 * Compiles all modules for this LOD level.
	 * @param EmitterBuildInfo - Where to store emitter information.
	 */
	void CompileModules( struct FParticleEmitterBuildInfo& EmitterBuildInfo );

	/**
	 * Append all used materials to the material list.
	 * @param OutMaterials - the material list.
	 * @param Slots - the material slot names
 	 * @param EmitterMaterials - the material slot materials.
	 */
	void GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, const TArray<struct FNamedEmitterMaterial>& NamedMaterialSlots, const TArray<UMaterialInterface*>& EmitterMaterials) const;

	/**
	 * Appends information on streaming meshes
	 * @param Bounds - the bounds of the parent particle system component
	 * @param OutStreamingRenderAssets - where the streaming mesh information will be added
	 */
	void GetStreamingMeshInfo(const FBoxSphereBounds& Bounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const;
};



