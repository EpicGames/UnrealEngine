// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ParticleHelper.h"
#include "ParticleModule.generated.h"

class UDistribution;
class UDistributionFloat;
class UDistributionVector;
class UInterpCurveEdSetup;
class UParticleEmitter;
class UParticleLODLevel;
class UParticleModuleTypeDataBase;
class UParticleSystemComponent;
struct FCurveEdEntry;
struct FParticleEmitterInstance;

/** ModuleType
 *	Indicates the kind of emitter the module can be applied to.
 *	ie, EPMT_Beam - only applies to beam emitters.
 *
 *	The TypeData field is present to speed up finding the TypeData module.
 */
UENUM()
enum EModuleType : int
{
	/** General - all emitter types can use it			*/
	EPMT_General UMETA(DisplayName="General"),
	/** TypeData - TypeData modules						*/
	EPMT_TypeData UMETA(DisplayName="Type Data"),
	/** Beam - only applied to beam emitters			*/
	EPMT_Beam UMETA(DisplayName="Beam"),
	/** Trail - only applied to trail emitters			*/
	EPMT_Trail UMETA(DisplayName="Trail"),
	/** Spawn - all emitter types REQUIRE it			*/
	EPMT_Spawn UMETA(DisplayName="Spawn"),
	/** Required - all emitter types REQUIRE it			*/
	EPMT_Required UMETA(DisplayName="Required"),
	/** Event - event related modules					*/
	EPMT_Event UMETA(DisplayName="Event"),
	/** Light related modules							*/
	EPMT_Light UMETA(DisplayName="Light"),
	/** SubUV related modules							*/
	EPMT_SubUV UMETA(DisplayName = "SubUV"),
	EPMT_MAX,
};

/** 
 *	Particle Selection Method, for any emitters that utilize particles
 *	as the source points.
 */
UENUM()
enum EParticleSourceSelectionMethod : int
{
	/** Random		- select a particle at random		*/
	EPSSM_Random UMETA(DisplayName="Random"),
	/** Sequential	- select a particle in order		*/
	EPSSM_Sequential UMETA(DisplayName="Sequential"),
	EPSSM_MAX,
};

USTRUCT()
struct FParticleCurvePair
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString CurveName;

	UPROPERTY()
	TObjectPtr<class UObject> CurveObject;


	FParticleCurvePair()
		: CurveObject(NULL)
	{
	}

};

USTRUCT()
struct FParticleRandomSeedInfo
{
	GENERATED_USTRUCT_BODY()

	/** The name to expose to the placed instances for setting this seed */
	UPROPERTY(EditAnywhere, Category=ParticleRandomSeedInfo)
	FName ParameterName;

	/** 
	 *	If true, the module will attempt to get the seed from the owner
	 *	instance. If that fails, it will fall back to getting it from
	 *	the RandomSeeds array.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleRandomSeedInfo)
	uint8 bGetSeedFromInstance:1;

	/** 
	 *	If true, the seed value retrieved from the instance will be an
	 *	index into the array of seeds.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleRandomSeedInfo)
	uint8 bInstanceSeedIsIndex:1;

	/** 
	 *	If true, then reset the seed upon the emitter looping.
	 *	For looping environmental effects this should likely be set to false to avoid
	 *	a repeating pattern.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleRandomSeedInfo)
	uint8 bResetSeedOnEmitterLooping:1;

	/**
	*	If true, then randomly select a seed entry from the RandomSeeds array
	*/
	UPROPERTY(EditAnywhere, Category = ParticleRandomSeedInfo)
	uint8 bRandomlySelectSeedArray:1;

	/** 
	 *	The random seed values to utilize for the module. 
	 *	More than 1 means the instance will randomly select one.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleRandomSeedInfo)
	TArray<int32> RandomSeeds;



		FParticleRandomSeedInfo()
		: bGetSeedFromInstance(false)
		, bInstanceSeedIsIndex(false)
		, bResetSeedOnEmitterLooping(true)
		, bRandomlySelectSeedArray(false)
		{
		}
		inline int32 GetInstancePayloadSize() const
		{
			return ((RandomSeeds.Num() > 0) ? sizeof(FParticleRandomSeedInstancePayload) : 0);
		}
	
};

UCLASS(editinlinenew, hidecategories=Object, abstract, Within=ParticleSystem, MinimalAPI)
class UParticleModule : public UObject
{
	GENERATED_UCLASS_BODY()

	/** If true, the module performs operations on particles during Spawning		*/
	UPROPERTY()
	uint8 bSpawnModule:1;

	/** If true, the module performs operations on particles during Updating		*/
	UPROPERTY()
	uint8 bUpdateModule:1;

	/** If true, the module performs operations on particles during final update	*/
	UPROPERTY()
	uint8 bFinalUpdateModule:1;

	/** If true, the module performs operations on particles during update and/or final update for GPU emitters*/
	UPROPERTY()
	uint8 bUpdateForGPUEmitter:1;

	/** If true, the module displays FVector curves as colors						*/
	UPROPERTY()
	uint8 bCurvesAsColor:1;

	/** If true, the module should render its 3D visualization helper				*/
	UPROPERTY(EditAnywhere, Category=Cascade)
	uint8 b3DDrawMode:1;

	/** If true, the module supports rendering a 3D visualization helper			*/
	UPROPERTY()
	uint8 bSupported3DDrawMode:1;

	/** If true, the module is enabled												*/
	UPROPERTY()
	uint8 bEnabled:1;

	/** If true, the module has had editing enabled on it							*/
	UPROPERTY()
	uint8 bEditable:1;

	/**
	*	If true, this flag indicates that auto-generation for LOD will result in
	*	an exact duplicate of the module, regardless of the percentage.
	*	If false, it will result in a module with different settings.
	*/
	UPROPERTY()
	uint8 LODDuplicate:1;

	/** If true, the module supports RandomSeed setting */
	UPROPERTY()
	uint8 bSupportsRandomSeed:1;

	/** If true, the module should be told when looping */
	UPROPERTY()
	uint8 bRequiresLoopingNotification:1;

	/**
	 *	The LOD levels this module is present in.
	 *	Bit-flags are used to indicate validity for a given LOD level.
	 *	For example, if
	 *		((1 << Level) & LODValidity) != 0
	 *	then the module is used in that LOD.
	 */
	UPROPERTY()
	uint8 LODValidity;

#if WITH_EDITORONLY_DATA
	/** The color to draw the modules curves in the curve editor. 
	 *	If bCurvesAsColor is true, it overrides this value.
	 */
	UPROPERTY(EditAnywhere, Category=Cascade)
	FColor ModuleEditorColor;

#endif // WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	ENGINE_API virtual bool IsPostLoadThreadSafe() const override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR	
	//~ End UObject Interface

	/**
	 * Called once to compile the effects of this module on runtime simulation.
	 * @param EmitterInfo - Information needed for runtime simulation.
	 */
	ENGINE_API virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo );

	struct FContext
	{
		FParticleEmitterInstance& Owner;
		const FTransform& GetTransform() const;
		UObject* GetDistributionData() const;
		FString GetTemplateName() const;
		FString GetInstanceName() const;
		FContext(FParticleEmitterInstance& Ow) : Owner(Ow) {}
	};

	/**
	 *	Called on a particle that is freshly spawned by the emitter.
	 *	
	 *	@param	Owner		The FParticleEmitterInstance that spawned the particle.
	 *	@param	Offset		The modules offset into the data payload of the particle.
	 *	@param	SpawnTime	The time of the spawn.
	 */
	struct FSpawnContext : FContext
	{
		int32 Offset;
		float SpawnTime;
		FBaseParticle* ParticleBase;
		FSpawnContext(FParticleEmitterInstance& Ow, int32 Of, float St, FBaseParticle* Pb) : FContext(Ow), Offset(Of), SpawnTime(St), ParticleBase(Pb) {}
	};
	ENGINE_API virtual void	Spawn(const FSpawnContext& Context);

	/**
	 *	Called on a particle that is being updated by its emitter.
	 *	
	 *	@param	Owner		The FParticleEmitterInstance that 'owns' the particle.
	 *	@param	Offset		The modules offset into the data payload of the particle.
	 *	@param	DeltaTime	The time since the last update.
	 */
	struct FUpdateContext : FContext
	{
		int32 Offset;
		float DeltaTime;
		FUpdateContext(FParticleEmitterInstance& Ow, int32 Of, float Dt) : FContext(Ow), Offset(Of), DeltaTime(Dt) {}
	};
	ENGINE_API virtual void	Update(const FUpdateContext& Context);
	/**
	 *	Called on an emitter when all other update operations have taken place
	 *	INCLUDING bounding box cacluations!
	 *	
	 *	@param	Owner		The FParticleEmitterInstance that 'owns' the particle.
	 *	@param	Offset		The modules offset into the data payload of the particle.
	 *	@param	DeltaTime	The time since the last update.
	 */
	ENGINE_API virtual void	FinalUpdate(const FUpdateContext& Context);

	/**
	 *	Returns the number of bytes that the module requires in the particle payload block.
	 *
	 *	@param	TypeData	The UParticleModuleTypeDataBase for the emitter that contains this module
	 *
	 *	@return	uint32		The number of bytes the module needs per particle.
	 */
	ENGINE_API virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData);
	/**
	 *	Returns the number of bytes the module requires in the emitters 'per-instance' data block.
	 *	
	 *	@return uint32		The number of bytes the module needs per emitter instance.
	 */
	ENGINE_API virtual uint32	RequiredBytesPerInstance();

	/**
	 *	Allows the module to prep its 'per-instance' data block.
	 *	
	 *	@param	Owner		The FParticleEmitterInstance that 'owns' the particle.
	 *	@param	InstData	Pointer to the data block for this module.
	 */
	ENGINE_API virtual uint32	PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData);

	// For Cascade
	/**
	 *	Called when the module is created, this function allows for setting values that make
	 *	sense for the type of emitter they are being used in.
	 *
	 *	@param	Owner			The UParticleEmitter that the module is being added to.
	 */
	ENGINE_API virtual void SetToSensibleDefaults(UParticleEmitter* Owner);
	
	/** 
	 *	Fill an array with each Object property that fulfills the FCurveEdInterface interface.
	 *
	 *	@param	OutCurve	The array that should be filled in.
	 */
	ENGINE_API virtual void	GetCurveObjects(TArray<FParticleCurvePair>& OutCurves);
	/** 
	 * Add all curve-editable Objects within this module to the curve editor.
	 *
	 * @param	EdSetup		The CurveEd setup to use for adding curved.
	 * @param	OutCurveEntries	The curves which are for this graph node
	 *
	 * @return	true, if new curves were added to the graph, otherwise they were already present
	 */
	ENGINE_API virtual	bool	AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries);
	/** 
	 *	Remove all curve-editable Objects within this module from the curve editor.
	 *
	 *	@param	EdSetup		The CurveEd setup to remove the curve from.
	 */
	ENGINE_API void	RemoveModuleCurvesFromEditor(UInterpCurveEdSetup* EdSetup);
	/** 
	 *	Does the module contain curves?
	 *
	 *	@return	bool		true if it does, false if not.
	 */
	ENGINE_API bool	ModuleHasCurves();
	/** 
	 *	Are the modules curves displayed in the curve editor?
	 *
	 *	@param	EdSetup		The CurveEd setup to check.
	 *
	 *	@return	bool		true if they are, false if not.
	 */
	ENGINE_API bool	IsDisplayedInCurveEd(UInterpCurveEdSetup* EdSetup);
	/** 
	 *	Helper function for updating the curve editor when the module editor color changes.
	 *
	 *	@param	Color		The new color the module is using.
	 *	@param	EdSetup		The CurveEd setup for the module.
	 */
	ENGINE_API void		ChangeEditorColor(FColor& Color, UInterpCurveEdSetup* EdSetup);

	/** 
	 *	Render the modules 3D visualization helper primitive.
	 *
	 *	@param	Owner		The FParticleEmitterInstance that 'owns' the module.
	 *	@param	View		The scene view that is being rendered.
	 *	@param	PDI			The FPrimitiveDrawInterface to use for rendering.
	 */
	struct FPreviewContext : FContext
	{
		const FSceneView* View;
		FPrimitiveDrawInterface* PDI;
		FPreviewContext(FParticleEmitterInstance& Ow, const FSceneView* Vi, FPrimitiveDrawInterface* Pd) : FContext(Ow), View(Vi), PDI(Pd) {}
	};
	virtual void Render3DPreview(const FPreviewContext& Context) {}

	/**
	 *	Retrieve the ModuleType of this module.
	 *
	 *	@return	EModuleType		The type of module this is.
	 */
	virtual EModuleType	GetModuleType() const	{	return EPMT_General;	}

	/**
	 *	Helper function used by the editor to auto-populate a placed AEmitter with any
	 *	instance parameters that are utilized.
	 *
	 *	@param	PSysComp		The particle system component to be populated.
	 */
	ENGINE_API virtual void	AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp);
	
	/**
	 *	Helper function used by the editor to auto-generate LOD values from a source module
	 *	and a percentage value used to scale its values.
	 *	
	 *	@param	SourceModule		The module to use as the source for values
	 *	@param	Percentage			The percentage of the source values to set
	 *	@param	LODLevel			The LOD level being generated
	 *
	 *	@return	true	if successful
	 *			false	if failed
	 */
	ENGINE_API virtual bool	GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel);

	/**
	 *	Store the given percentage of the SourceFloat distribution in the FloatDist
	 *
	 *	@param	FloatDist			The distribution to put the result into.
	 *	@param	SourceFloatDist		The distribution of use as the source.
	 *	@param	Percentage			The percentage of the source value to use [0..100]
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	ENGINE_API virtual bool	ConvertFloatDistribution(UDistributionFloat* FloatDist, UDistributionFloat* SourceFloatDist, float Percentage);
	/**
	 *	Store the given percentage of the SourceVector distribution in the VectorDist
	 *
	 *	@param	VectorDist			The distribution to put the result into.
	 *	@param	SourceVectorDist	The distribution of use as the source.
	 *	@param	Percentage			The percentage of the source value to use [0..100]
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	ENGINE_API virtual bool	ConvertVectorDistribution(UDistributionVector* VectorDist, UDistributionVector* SourceVectorDist, float Percentage);
	/**
	 *	Returns whether the module is SizeMultipleLife or not.
	 *
	 *	@return	bool	true if the module is a UParticleModuleSizeMultipleLife
	 *					false if not
	 */
	virtual bool   IsSizeMultiplyLife() { return false; };

	/**
	 *	Returns whether the module supports the RandomSeed functionality
	 *
	 *	@return	bool	true if it supports it; false if not
	 */
	bool SupportsRandomSeed() const
	{
		return bSupportsRandomSeed;
	}

	/**
	 *	Returns whether the module requires notification when an emitter loops.
	 *
	 *	@return	bool	true if the module required looping notification
	 */
	bool RequiresLoopingNotification() const
	{
		return bRequiresLoopingNotification;
	}

	/** 
	 *	Called when an emitter instance is looping...
	 *
	 *	@param	Owner					The emitter instance that owns this module
	 */
	virtual void EmitterLoopingNotify(FParticleEmitterInstance* Owner)
	{
	}

	/**
	 *	Generates a new module for LOD levels, setting the values appropriately.
	 *	Note that the module returned could simply be the module it was called on.
	 *
	 *	@param	SourceLODLevel		The source LODLevel
	 *	@param	DestLODLevel		The destination LODLevel
	 *	@param	Percentage			The percentage value that should be used when setting values
	 *
	 *	@return	UParticleModule*	The generated module, or this if percentage == 100.
	 */
	ENGINE_API virtual UParticleModule* GenerateLODModule(UParticleLODLevel* SourceLODLevel, UParticleLODLevel* DestLODLevel, float Percentage, 
		bool bGenerateModuleData, bool bForceModuleConstruction = false);

	/**
	 *	Returns true if the results of LOD generation for the given percentage will result in a 
	 *	duplicate of the module.
	 *
	 *	@param	SourceLODLevel		The source LODLevel
	 *	@param	DestLODLevel		The destination LODLevel
	 *	@param	Percentage			The percentage value that should be used when setting values
	 *
	 *	@return	bool				true if the generated module will be a duplicate.
	 *								false if not.
	 */
	virtual bool WillGeneratedModuleBeIdentical(UParticleLODLevel* SourceLODLevel, UParticleLODLevel* DestLODLevel, float Percentage)
	{
		// The assumption is that at 100%, ANY module will be identical...
		// (Although this is virtual to allow over-riding that assumption on a case-by-case basis!)

		if (Percentage != 100.0f)
		{
			return LODDuplicate;
		}

		return true;
	}

	/**
	 *	Returns true if the module validiy flags indicate this module is used in the given LOD level.
	 *
	 *	@param	SourceLODIndex		The index of the source LODLevel
	 *
	 *	@return	bool				true if the generated module is used, false if not.
	 */
	ENGINE_API virtual bool IsUsedInLODLevel(int32 SourceLODIndex) const;

	/**
	 *	Retrieve the ParticleSysParams associated with this module.
	 *
	 *	@param	ParticleSysParamList	The list of FParticleSysParams to add to
	 */
	ENGINE_API virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList);

	/**
	 *	Retrieve the distributions that use ParticleParameters in this module.
	 *
	 *	@param	ParticleParameterList	The list of ParticleParameter distributions to add to
	 */
	ENGINE_API virtual void GetParticleParametersUtilized(TArray<FString>& ParticleParameterList);
	
	/**
	 *	Refresh the module...
	 */
	virtual void RefreshModule(UInterpCurveEdSetup* EdSetup, UParticleEmitter* InEmitter, int32 InLODLevel) {}

	/**
	 *	Return true if this module impacts rotation of Mesh emitters
	 *	@return	bool		true if the module impacts mesh emitter rotation
	 */
	virtual bool	TouchesMeshRotation() const	{ return false; }

	/** 
	 *	Prepare a random seed instance payload...
	 *
	 *	@param	Owner					The emitter instance that owns this module
	 *	@param	InRandSeedPayload		The random seed instance payload to initialize
	 *	@param	InRandSeedInfo			The random seed info of the module
	 *
	 *	@return	uint32					0xffffffff is failed
	 */
	ENGINE_API virtual uint32	PrepRandomSeedInstancePayload(FParticleEmitterInstance* Owner, FParticleRandomSeedInstancePayload* InRandSeedPayload, const FParticleRandomSeedInfo& InRandSeedInfo);

	/**
	 *	Retrieve the random seed info for this module.
	 *
	 *	@return	FParticleRandomSeedInfo*	The random seed info; NULL if not supported
	 */
	virtual FParticleRandomSeedInfo* GetRandomSeedInfo()
	{
		return NULL;
	}

	/**
	 *	Set the random seed info entry at the given index to the given seed
	 *
	 *	@param	InIndex			The index of the entry to set
	 *	@param	InRandomSeed	The seed to set the entry to
	 *
	 *	@return	bool			true if successful; false if not (not found, etc.)
	 */
	ENGINE_API virtual bool SetRandomSeedEntry(int32 InIndex, int32 InRandomSeed);

	/** Return false if this emitter requires a game thread tick **/
	virtual bool CanTickInAnyThread()
	{
		return true;
	}

	/** Returns whether this module is used in any GPU emitters. */
	ENGINE_API bool IsUsedInGPUEmitter()const;

	/**
	 * Retreive the random stream that should be used for the provided instance.
	 *
	 * @param	Owner			The emitter instance that owns this module
	 *
	 * @return FRandomStream&	The random stream to use for the provided instance.
	 */
	ENGINE_API FRandomStream& GetRandomStream(const FContext& Context);

#if WITH_EDITOR
	ENGINE_API virtual void PostLoadSubobjects( FObjectInstancingGraph* OuterInstanceGraph ) override;

	/**
	 *	Custom Cascade module menu entries support
	 */
	/**
	 *	Get the number of custom entries this module has. Maximum of 3.
	 *
	 *	@return	int32		The number of custom menu entries
	 */
	virtual int32 GetNumberOfCustomMenuOptions() const { return 0; }

	/**
	 *	Get the display name of the custom menu entry.
	 *
	 *	@param	InEntryIndex		The custom entry index (0-2)
	 *	@param	OutDisplayString	The string to display for the menu
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const { return false; }

	/**
	 *	Perform the custom menu entry option.
	 *
	 *	@param	InEntryIndex		The custom entry index (0-2) to perform
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool PerformCustomMenuEntry(int32 InEntryIndex) { return false; }

	/** Returns true if the module is valid for the provided LOD level. */
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
	{
		return true;
	}

	/**
	 *	Gets a list of the names of distributions not allowed on GPU emitters.
	 */
	static ENGINE_API  void GetDistributionsRestrictedOnGPU(TArray<FString>& OutRestrictedDistributions);
		
	/**
	 *	Checks if a distribution is allowed on the GPU.
	 *
	 *	@param	Distribution		The Distribution to test.
	 *	@return	bool				true if allowed on the GPU, false if not.
	 */
	static ENGINE_API bool IsDistributionAllowedOnGPU(const UDistribution* Distribution);
	
	/**
	 *	Generates the FText to display to the user informing them that a module is using a distribution that is not allowed on GPU emitters.
	 *
	 *	@param	ModuleName		The name of the module the distribution is in.
	 *	@param	PropertyName	The name of the distribution's property.
	 *	@return	FText			The generated FText.
	 */
	static ENGINE_API FText GetDistributionNotAllowedOnGPUText(const FString& ModuleName, const FString& PropertyName);

	/**
	 *	Set the transaction flag on the module and any members which require it
	 */
	ENGINE_API void SetTransactionFlag();
#endif
};



