// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RenderCommandFence.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "Particles/SubUVAnimation.h"
#include "ParticleModuleRequired.generated.h"

class UInterpCurveEdSetup;
class UParticleLODLevel;
struct FCurveEdEntry;

UENUM()
enum class EParticleUVFlipMode : uint8
{
	/** Flips UV on all particles. */
	None,
	/** Flips UV on all particles. */
	FlipUV,
	/** Flips U only on all particles. */
	FlipUOnly,
	/** Flips V only on all particles. */
	FlipVOnly,
	/** Flips UV randomly for each particle on spawn. */
	RandomFlipUV,
	/** Flips U only randomly for each particle on spawn. */
	RandomFlipUOnly,
	/** Flips V only randomly for each particle on spawn. */
	RandomFlipVOnly,
	/** Flips U and V independently at random for each particle on spawn. */
	RandomFlipUVIndependent,
};

/** Flips the sign of a particle's base size based on it's UV flip mode. */
inline void AdjustParticleBaseSizeForUVFlipping(FVector& OutSize, EParticleUVFlipMode FlipMode, FRandomStream& InRandomStream)
{
	static const float HalfRandMax = 0.5f;

	switch (FlipMode)
	{
		case EParticleUVFlipMode::None:
			return;

		case EParticleUVFlipMode::FlipUV:
			OutSize = -OutSize;
			return;
		
		case EParticleUVFlipMode::FlipUOnly:
			OutSize.X = -OutSize.X;
			return;
		
		case EParticleUVFlipMode::FlipVOnly:
			OutSize.Y = -OutSize.Y;
			return;
		
		case EParticleUVFlipMode::RandomFlipUV:
			OutSize = InRandomStream.FRand() > HalfRandMax ? -OutSize : OutSize;
			return;

		case EParticleUVFlipMode::RandomFlipUOnly:
			OutSize.X = InRandomStream.FRand() > HalfRandMax ? -OutSize.X : OutSize.X;
			return;

		case EParticleUVFlipMode::RandomFlipVOnly:
			OutSize.Y = InRandomStream.FRand() > HalfRandMax ? -OutSize.Y : OutSize.Y;
			return;

		case EParticleUVFlipMode::RandomFlipUVIndependent:
			OutSize.X = InRandomStream.FRand() > HalfRandMax ? -OutSize.X : OutSize.X;
			OutSize.Y = InRandomStream.FRand() > HalfRandMax ? -OutSize.Y : OutSize.Y;
			return;

		default:
			checkNoEntry();
			break;
	}
}

UENUM()
enum EParticleSortMode : int
{
	PSORTMODE_None,
	PSORTMODE_ViewProjDepth,
	PSORTMODE_DistanceToView,
	PSORTMODE_Age_OldestFirst,
	PSORTMODE_Age_NewestFirst,
	PSORTMODE_MAX,
};

UENUM()
enum EEmitterNormalsMode : int
{
	/** Default mode, normals are based on the camera facing geometry. */
	ENM_CameraFacing,
	/** Normals are generated from a sphere centered at NormalsSphereCenter. */
	ENM_Spherical,
	/** Normals are generated from a cylinder going through NormalsSphereCenter, in the direction NormalsCylinderDirection. */
	ENM_Cylindrical,
	ENM_MAX,
};

struct FParticleRequiredModule
{
	uint32 NumFrames;
	uint32 NumBoundingVertices;
	uint32 NumBoundingTriangles;
	float AlphaThreshold;
	TArray<FVector2f> FrameData;
	FRHIShaderResourceView* BoundingGeometryBufferSRV;
	uint8 bCutoutTexureIsValid : 1;
	uint8 bUseVelocityForMotionBlur : 1;
};



UCLASS(editinlinenew, hidecategories=(Object, Cascade), meta=(DisplayName = "Required"), MinimalAPI)
class UParticleModuleRequired : public UParticleModule
{
	GENERATED_UCLASS_BODY()

	//
	// General.
	// 
	
	/** The material to utilize for the emitter at this LOD level.						*/
	UPROPERTY(EditAnywhere, Category=Emitter)
	TObjectPtr<class UMaterialInterface> Material;

	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_Square */
	UPROPERTY(EditAnywhere, Category = Emitter, meta = (UIMin = "0", DisplayAfter="ScreenAlignment"))
	float MinFacingCameraBlendDistance;

	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_FacingCameraPosition */
	UPROPERTY(EditAnywhere, Category = Emitter, meta = (UIMin = "0", DisplayAfter="MinFacingCameraBlendDistance"))
	float MaxFacingCameraBlendDistance;

	UPROPERTY(EditAnywhere, Category=Emitter)
	FVector EmitterOrigin;

	UPROPERTY(EditAnywhere, Category=Emitter)
	FRotator EmitterRotation;

	/** 
	 *	The screen alignment to utilize for the emitter at this LOD level.
	 *	One of the following:
	 *	PSA_FacingCameraPosition - Faces the camera position, but is not dependent on the camera rotation.  
	 *								This method produces more stable particles under camera rotation.
	 *	PSA_Square			- Uniform scale (via SizeX) facing the camera
	 *	PSA_Rectangle		- Non-uniform scale (via SizeX and SizeY) facing the camera
	 *	PSA_Velocity		- Orient the particle towards both the camera and the direction 
	 *						  the particle is moving. Non-uniform scaling is allowed.
	 *	PSA_TypeSpecific	- Use the alignment method indicated in the type data module.
	 *	PSA_FacingCameraDistanceBlend - Blends between PSA_FacingCameraPosition and PSA_Square over specified distance.
	 */
	UPROPERTY(EditAnywhere, Category=Emitter)
	TEnumAsByte<EParticleScreenAlignment> ScreenAlignment;

	/** If true, update the emitter in local space										*/
	UPROPERTY(EditAnywhere, Category=Emitter, meta = (DisplayAfter = "MaxFacingCameraBlendDistance"))
	uint8 bUseLocalSpace:1;

	/** If true, kill the emitter when the particle system is deactivated				*/
	UPROPERTY(EditAnywhere, Category=Emitter)
	uint8 bKillOnDeactivate:1;

	/** If true, kill the emitter when it completes										*/
	UPROPERTY(EditAnywhere, Category=Emitter)
	uint8 bKillOnCompleted:1;

	/**
	 *	The sorting mode to use for this emitter.
	 *	PSORTMODE_None				- No sorting required.
	 *	PSORTMODE_ViewProjDepth		- Sort by view projected depth of the particle.
	 *	PSORTMODE_DistanceToView	- Sort by distance of particle to view in world space.
	 *	PSORTMODE_Age_OldestFirst	- Sort by age, oldest drawn first.
	 *	PSORTMODE_Age_NewestFirst	- Sort by age, newest drawn first.
	 *
	 */
	UPROPERTY(EditAnywhere, Category=Emitter)
	TEnumAsByte<EParticleSortMode> SortMode;

	/**
	 *	If true, the EmitterTime for the emitter will be calculated by
	 *	modulating the SecondsSinceCreation by the EmitterDuration. As
	 *	this can lead to issues w/ looping and variable duration, a new
	 *	approach has been implemented. 
	 *	If false, this new approach is utilized, and the EmitterTime is
	 *	simply incremented by DeltaTime each tick. When the emitter 
	 *	loops, it adjusts the EmitterTime by the current EmitterDuration
	 *	resulting in proper looping/delay behavior.
	 */
	UPROPERTY(EditAnywhere, Category=Emitter)
	uint8 bUseLegacyEmitterTime:1;

	/** If true, removes the HMD view roll (e.g. in VR) */
	UPROPERTY(EditAnywhere, Category=Emitter, meta=(DisplayName = "Remove HMD Roll"))
	uint8 bRemoveHMDRoll:1;

	/** If true, gpu simulation positions are offset to support double precision vectors. Cpu sims always support large world coordinates. */
	UPROPERTY(EditAnywhere, Category=Emitter)
	uint8 bSupportLargeWorldCoordinates:1;

	/**
	 *	If true, select the emitter duration from the range
	 *		[EmitterDurationLow..EmitterDuration]
	 */
	UPROPERTY(EditAnywhere, Category = Duration)
	uint8 bEmitterDurationUseRange : 1;


	UPROPERTY(EditAnywhere, Category = Rendering)
	uint8 bOverrideUseVelocityForMotionBlur : 1;

	/**
	 * When supported by the vertex factory will use particle velocity for motion blur approximation.
	 * This will be inaccurate in some cases, i.e. sprite rotation, but may provide a reasonable result vs having this disabled
	 */
	UPROPERTY(EditAnywhere, Category = Rendering, meta=(EditCondition = "bOverrideUseVelocityForMotionBlur"))
	uint8 bUseVelocityForMotionBlur : 1;

	/** 
	 *	How long, in seconds, the emitter will run before looping.
	 */
	UPROPERTY(EditAnywhere, Category=Duration, meta = (DisplayAfter="bRemoveHMDRoll"))
	float EmitterDuration;

	//
	// Spawn-related.
	//
	
	/** The rate at which to spawn particles									*/
	UPROPERTY()
	struct FRawDistributionFloat SpawnRate;

	/** The array of burst entries.												*/
	UPROPERTY(export, noclear)
	TArray<FParticleBurst> BurstList;

	//
	// Delay-related.
	//
	
	/**
	 *	Indicates the time (in seconds) that this emitter should be delayed in the particle system.
	 */
	UPROPERTY(EditAnywhere, Category=Delay)
	float EmitterDelay;

	/** 
	 *	The low end of the emitter delay if using a range.
	 */
	UPROPERTY(EditAnywhere, Category=Delay)
	float EmitterDelayLow;

	/**
	 *	If true, the emitter will be delayed only on the first loop.
	 */
	UPROPERTY(EditAnywhere, Category=Delay)
	uint8 bDelayFirstLoopOnly:1;

	//
	// SubUV-related.
	//
	
	/** 
	 *	The interpolation method to used for the SubUV image selection.
	 *	One of the following:
	 *	PSUVIM_None			- Do not apply SubUV modules to this emitter. 
	 *	PSUVIM_Linear		- Smoothly transition between sub-images in the given order, 
	 *						  with no blending between the current and the next
	 *	PSUVIM_Linear_Blend	- Smoothly transition between sub-images in the given order, 
	 *						  blending between the current and the next 
	 *	PSUVIM_Random		- Pick the next image at random, with no blending between 
	 *						  the current and the next 
	 *	PSUVIM_Random_Blend	- Pick the next image at random, blending between the current 
	 *						  and the next 
	 */
	UPROPERTY(EditAnywhere, Category=SubUV)
	TEnumAsByte<EParticleSubUVInterpMethod> InterpolationMethod;

	/** Whether to scale the UV or not - ie, the model wasn't setup with sub uvs		*/
	UPROPERTY(EditAnywhere, Category = SubUV)
	uint8 bScaleUV : 1;

	/**
	 *	If true, select the emitter delay from the range 
	 *		[EmitterDelayLow..EmitterDelay]
	 */
	UPROPERTY(EditAnywhere, Category=Delay)
	uint8 bEmitterDelayUseRange:1;

	/** The method to utilize when burst-emitting particles						*/
	UPROPERTY()
	TEnumAsByte<EParticleBurstMethod> ParticleBurstMethod;

	/** Override the system MacroUV settings                                            */
	UPROPERTY(EditAnywhere, Category = MacroUV)
	uint8 bOverrideSystemMacroUV : 1;

	/**
	 *	If true, use the MaxDrawCount to limit the number of particles rendered.
	 *	NOTE: This does not limit the number spawned/updated, only what is drawn.
	 */
	UPROPERTY(EditAnywhere, Category = Rendering)
	uint8 bUseMaxDrawCount : 1;

	UPROPERTY(EditAnywhere, Category=ParticleCutout)
	TEnumAsByte<enum EOpacitySourceMode> OpacitySourceMode;

	/** Normal generation mode for this emitter LOD. */
	UPROPERTY(EditAnywhere, Category = Normals)
	TEnumAsByte<enum EEmitterNormalsMode> EmitterNormalsMode;

	/**
	* Ensures that movement generated from the orbit module is applied to velocity-aligned particles
	*/
	UPROPERTY(EditAnywhere, Category = Emitter)
	uint8 bOrbitModuleAffectsVelocityAlignment : 1;

	/** The number of sub-images horizontally in the texture							*/
	UPROPERTY(EditAnywhere, Category=SubUV)
	int32 SubImages_Horizontal;

	/** The number of sub-images vertically in the texture								*/
	UPROPERTY(EditAnywhere, Category=SubUV)
	int32 SubImages_Vertical;

	/**
	 *	The amount of time (particle-relative, 0.0 to 1.0) to 'lock' on a random sub image
	 *	    0.0 = change every frame
	 *      1.0 = select a random image at spawn and hold for the life of the particle
	 */
	UPROPERTY()
	float RandomImageTime;

	/** The number of times to change a random image over the life of the particle.		*/
	UPROPERTY(EditAnywhere, Category=SubUV)
	int32 RandomImageChanges;

	/** Local space position that UVs generated with the ParticleMacroUV material node will be centered on. */
	UPROPERTY(EditAnywhere, Category=MacroUV)
	FVector MacroUVPosition;

	/** World space radius that UVs generated with the ParticleMacroUV material node will tile based on. */
	UPROPERTY(EditAnywhere, Category=MacroUV)
	float MacroUVRadius;

	/**
	* Controls UV Flipping for this emitter.
	*/
	UPROPERTY(EditAnywhere, Category = Rendering)
	EParticleUVFlipMode UVFlippingMode;

	/**
	* More bounding vertices results in reduced overdraw, but adds more triangle overhead.
	* The eight vertex mode is best used when the SubUV texture has a lot of space to cut out that is not captured by the four vertex version,
	* and when the particles using the texture will be few and large.
	*/
	UPROPERTY(EditAnywhere, Category=ParticleCutout)
	TEnumAsByte<enum ESubUVBoundingVertexCount> BoundingMode;

	/**
	 *	If true, recalculate the emitter duration on each loop.
	 */
	UPROPERTY(EditAnywhere, Category = Duration)
	uint8 bDurationRecalcEachLoop : 1;

	/** 
	 * When EmitterNormalsMode is ENM_Spherical, particle normals are created to face away from NormalsSphereCenter. 
	 * NormalsSphereCenter is in local space.
	 */
	UPROPERTY(EditAnywhere, Category=Normals)
	FVector NormalsSphereCenter;

	/**
	* Alpha channel values larger than the threshold are considered occupied and will be contained in the bounding geometry.
	* Raising this threshold slightly can reduce overdraw in particles using this animation asset.
	*/
	UPROPERTY(EditAnywhere, Category = ParticleCutout, meta = (UIMin = "0", UIMax = "1"))
	float AlphaThreshold;

	/** The number of times to loop the emitter.
	 *	0 indicates loop continuously
	 */
	UPROPERTY(EditAnywhere, Category = Duration)
	int32 EmitterLoops;

	/**
	* Texture to generate bounding geometry from.
	*/
	UPROPERTY(EditAnywhere, Category = ParticleCutout)
	TObjectPtr<UTexture2D> CutoutTexture;

	/**
	 *	The maximum number of particles to DRAW for this emitter.
	 *	If set to 0, it will use whatever number are present.
	 */
	UPROPERTY(EditAnywhere, Category = Rendering)
	int32 MaxDrawCount;

	/**
	 *	The low end of the emitter duration if using a range.
	 */
	UPROPERTY(EditAnywhere, Category = Duration, meta = (DisplayAfter = "EmitterDuration"))
	float EmitterDurationLow;

	/** 
	 * When EmitterNormalsMode is ENM_Cylindrical, 
	 * particle normals are created to face away from the cylinder going through NormalsSphereCenter in the direction NormalsCylinderDirection. 
	 * NormalsCylinderDirection is in local space.
	 */
	UPROPERTY(EditAnywhere, Category=Normals)
	FVector NormalsCylinderDirection;

	/** 
	*	Named material overrides for this emitter. 
	*	Overrides this emitter's material(s) with those in the correspondingly named slot(s) of the owning system.
	*/
	UPROPERTY(EditAnywhere, Category = Materials)
	TArray<FName> NamedMaterialOverrides;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void FinishDestroy() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual	bool AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries) override
	{
		// Overide the base implementation to prevent old SpawnRate from being added...
		return true;
	}
	virtual EModuleType	GetModuleType() const override {	return EPMT_Required;	}
	virtual bool GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel) override;
	//~ End UParticleModule Interface

	bool ShouldUseVelocityForMotionBlur() const;

	inline int32 GetNumFrames() const
	{
		return SubImages_Vertical * SubImages_Horizontal;
	}

	inline bool IsBoundingGeometryValid() const
	{
		return CutoutTexture != nullptr;
	}

	inline FRHIShaderResourceView* GetBoundingGeometrySRV() const
	{
		return BoundingGeometryBuffer->ShaderResourceView;
	}

	inline int32 GetNumBoundingVertices() const
	{
		return BoundingMode == BVC_FourVertices ? 4 : 8;
	}

	inline int32 GetNumBoundingTriangles() const
	{
		return BoundingMode == BVC_FourVertices ? 2 : 6;
	}

	inline const FVector2f* GetFrameData(int32 FrameIndex) const
	{
		return &DerivedData.BoundingGeometry[FrameIndex * GetNumBoundingVertices()];
	}

	FParticleRequiredModule *CreateRendererResource()
	{
		FParticleRequiredModule *FReqMod = new FParticleRequiredModule();
		FReqMod->bCutoutTexureIsValid = IsBoundingGeometryValid();
		FReqMod->bUseVelocityForMotionBlur = ShouldUseVelocityForMotionBlur();
		FReqMod->NumFrames = GetNumFrames();
		FReqMod->FrameData = DerivedData.BoundingGeometry;
		FReqMod->NumBoundingVertices = GetNumBoundingVertices();
		FReqMod->NumBoundingTriangles = GetNumBoundingTriangles();
		check(FReqMod->NumBoundingTriangles == 2 || FReqMod->NumBoundingTriangles == 6);
		FReqMod->AlphaThreshold = AlphaThreshold;
		FReqMod->BoundingGeometryBufferSRV = GetBoundingGeometrySRV();
		return FReqMod;
	}

protected:
	friend class FParticleModuleRequiredDetails;
	friend struct FParticleEmitterInstance;

private:
	/** Derived data for this asset, generated off of SubUVTexture. */
	FSubUVDerivedData DerivedData;

	/** Tracks progress of BoundingGeometryBuffer release during destruction. */
	FRenderCommandFence ReleaseFence;

	/** Used on platforms that support instancing, the bounding geometry is fetched from a vertex shader instead of on the CPU. */
	FSubUVBoundingGeometryBuffer* BoundingGeometryBuffer;

	void CacheDerivedData();
	void InitBoundingGeometryBuffer();
	void GetDefaultCutout();
};



