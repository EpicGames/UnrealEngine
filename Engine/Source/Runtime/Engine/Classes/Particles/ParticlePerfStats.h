// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Atomic.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "Containers/Array.h"
#include "ProfilingDebugging/CsvProfiler.h"


#define WITH_GLOBAL_RUNTIME_FX_BUDGET (!UE_SERVER)
#ifndef WITH_PARTICLE_PERF_STATS
	#if (WITH_UNREAL_DEVELOPER_TOOLS || WITH_UNREAL_TARGET_DEVELOPER_TOOLS || (!UE_BUILD_SHIPPING) || WITH_GLOBAL_RUNTIME_FX_BUDGET)
	#define WITH_PARTICLE_PERF_STATS 1
	#else
	#define	WITH_PARTICLE_PERF_STATS 0
	#endif
#else
	#if !WITH_PARTICLE_PERF_STATS
		//If perf stats are explicitly disabled then we must also disable the runtime budget tracking.
		#undef WITH_GLOBAL_RUNTIME_FX_BUDGET
		#define WITH_GLOBAL_RUNTIME_FX_BUDGET 0
	#endif
#endif

#define WITH_PER_SYSTEM_PARTICLE_PERF_STATS (WITH_PARTICLE_PERF_STATS && !UE_BUILD_SHIPPING)
#define WITH_PER_COMPONENT_PARTICLE_PERF_STATS (WITH_PARTICLE_PERF_STATS && !UE_BUILD_SHIPPING)

#define WITH_PARTICLE_PERF_CSV_STATS WITH_PER_SYSTEM_PARTICLE_PERF_STATS && CSV_PROFILER_STATS && !UE_BUILD_SHIPPING
CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Particles);

struct FParticlePerfStats;
class UWorld;
class UFXSystemAsset;
class UFXSystemComponent;

#if WITH_PARTICLE_PERF_STATS

/** Stats gathered on the game thread or game thread spawned tasks. */
struct FParticlePerfStats_GT
{
	uint64 NumInstances;
	uint64 TickGameThreadCycles;
	TAtomic<uint64> TickConcurrentCycles;
	uint64 FinalizeCycles;
	TAtomic<uint64> EndOfFrameCycles;
	TAtomic<uint64> ActivationCycles;
	uint64 WaitCycles;
	uint64 MemoryBytes;

	FParticlePerfStats_GT() { Reset(); }
	
	FParticlePerfStats_GT(const FParticlePerfStats_GT& Other)
	{
		NumInstances = Other.NumInstances;
		TickGameThreadCycles = Other.TickGameThreadCycles;
		TickConcurrentCycles = Other.TickConcurrentCycles.Load();
		FinalizeCycles = Other.FinalizeCycles;
		EndOfFrameCycles = Other.EndOfFrameCycles.Load();
		ActivationCycles = Other.ActivationCycles.Load();
		WaitCycles = Other.WaitCycles;
		MemoryBytes = Other.MemoryBytes;
	}

	FParticlePerfStats_GT& operator=(const FParticlePerfStats_GT& Other)
	{
		NumInstances = Other.NumInstances;
		TickGameThreadCycles = Other.TickGameThreadCycles;
		TickConcurrentCycles = Other.TickConcurrentCycles.Load();
		FinalizeCycles = Other.FinalizeCycles;
		EndOfFrameCycles = Other.EndOfFrameCycles.Load();
		ActivationCycles = Other.ActivationCycles.Load();
		WaitCycles = Other.WaitCycles;
		MemoryBytes = Other.MemoryBytes;
		return *this;
	}

	FParticlePerfStats_GT(FParticlePerfStats_GT&& Other)
	{
		*this = Other;
		Other.Reset();
	}

	FParticlePerfStats_GT& operator=(FParticlePerfStats_GT&& Other)
	{
		*this = Other;
		Other.Reset();		
		return *this;
	}

	FParticlePerfStats_GT& operator+=(FParticlePerfStats_GT& Other)
	{
		NumInstances += Other.NumInstances;
		TickGameThreadCycles += Other.TickGameThreadCycles;
		TickConcurrentCycles += Other.TickConcurrentCycles.Load();
		FinalizeCycles += Other.FinalizeCycles;
		EndOfFrameCycles += Other.EndOfFrameCycles.Load();
		ActivationCycles += Other.ActivationCycles.Load();
		WaitCycles += Other.WaitCycles;
		MemoryBytes += Other.MemoryBytes;
		return *this;
	}

	inline void Reset()
	{
		NumInstances = 0;
		TickGameThreadCycles = 0;
		TickConcurrentCycles = 0;
		FinalizeCycles = 0;
		EndOfFrameCycles = 0;
		ActivationCycles = 0;
		WaitCycles = 0;
		MemoryBytes = 0;
	}

	inline uint64 GetTotalCycles_GTOnly() const { return TickGameThreadCycles + FinalizeCycles + ActivationCycles + WaitCycles; }
	inline uint64 GetPerInstanceAvgCycles_GTOnly() const { return NumInstances > 0 ? GetTotalCycles_GTOnly() / NumInstances : 0; }

	inline uint64 GetTotalCycles() const { return GetTotalCycles_GTOnly() + TickConcurrentCycles + EndOfFrameCycles; }
	inline uint64 GetPerInstanceAvgCycles() const { return NumInstances > 0 ? GetTotalCycles() / NumInstances : 0; }
};

/** Stats gathered on the render thread. */
struct FParticlePerfStats_RT
{
	uint64 NumInstances = 0;
	uint64 RenderUpdateCycles = 0;
	uint64 GetDynamicMeshElementsCycles = 0;
	
	FParticlePerfStats_RT()	{ Reset();	}
	inline void Reset()
	{
		NumInstances = 0;
		RenderUpdateCycles = 0;
		GetDynamicMeshElementsCycles = 0;
	}
	inline uint64 GetTotalCycles() const { return RenderUpdateCycles + GetDynamicMeshElementsCycles; }
	inline uint64 GetPerInstanceAvgCycles() const { return NumInstances > 0 ? (RenderUpdateCycles + GetDynamicMeshElementsCycles) / NumInstances : 0; }

	FParticlePerfStats_RT& operator+=(FParticlePerfStats_RT& Other)
	{
		NumInstances += Other.NumInstances;
		RenderUpdateCycles += Other.RenderUpdateCycles;
		GetDynamicMeshElementsCycles += Other.GetDynamicMeshElementsCycles;
		return *this;
	}
};

/** Stats gathered from the GPU */
struct FParticlePerfStats_GPU
{
	uint64 NumInstances = 0;
	uint64 TotalMicroseconds = 0;

	inline uint64 GetTotalMicroseconds() const { return TotalMicroseconds; }
	inline uint64 GetPerInstanceAvgMicroseconds() const { return NumInstances > 0 ? GetTotalMicroseconds() / NumInstances : 0; }

	FParticlePerfStats_GPU() { Reset(); }
	inline void Reset()
	{
		NumInstances = 0;
		TotalMicroseconds = 0;
	}

	FParticlePerfStats_GPU& operator+=(FParticlePerfStats_GPU& Other)
	{
		NumInstances += Other.NumInstances;
		TotalMicroseconds += Other.TotalMicroseconds;
		return *this;
	}
};

struct FParticlePerfStats
{
	ENGINE_API FParticlePerfStats();

	ENGINE_API void Reset(bool bSyncWithRT);
	ENGINE_API void ResetGT();
	ENGINE_API void ResetRT();
	ENGINE_API void Tick();
	ENGINE_API void TickRT();

	inline static bool GetCSVStatsEnabled() { return bCSVStatsEnabled.Load(EMemoryOrder::Relaxed); }
	inline static bool GetStatsEnabled() { return bStatsEnabled.Load(EMemoryOrder::Relaxed); }
	inline static bool GetGatherWorldStats() { return WorldStatsReaders.Load(EMemoryOrder::Relaxed) > 0; }
	inline static bool GetGatherSystemStats() { return SystemStatsReaders.Load(EMemoryOrder::Relaxed) > 0; }
	inline static bool GetGatherComponentStats() { return ComponentStatsReaders.Load(EMemoryOrder::Relaxed) > 0; }
	inline static bool ShouldGatherStats() 
	{
		return GetStatsEnabled() && 
			(GetGatherWorldStats() 
			#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
			|| GetGatherSystemStats() 
			#endif
			#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
			|| GetGatherComponentStats()
			#endif
			);
	}


	inline static void SetCSVStatsEnabled(bool bEnabled) { bCSVStatsEnabled.Store(bEnabled); }
	inline static void SetStatsEnabled(bool bEnabled) { bStatsEnabled.Store(bEnabled); }
	inline static void AddWorldStatReader() { ++WorldStatsReaders; }
	inline static void RemoveWorldStatReader() { --WorldStatsReaders; }
	inline static void AddSystemStatReader() { ++SystemStatsReaders; }
	inline static void RemoveSystemStatReader() { --SystemStatsReaders; }
	inline static void AddComponentStatReader() { ++ComponentStatsReaders; }
	inline static void RemoveComponentStatReader() { --ComponentStatsReaders; }

	static inline FParticlePerfStats* GetStats(const UWorld* World)
	{
		if (World && GetGatherWorldStats() && GetStatsEnabled())
		{
			return GetWorldPerfStats(World);
		}
		return nullptr;
	}
	
	static inline FParticlePerfStats* GetStats(const UFXSystemAsset* System)
	{
	#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		if (System && GetGatherSystemStats() && GetStatsEnabled())
		{
			return GetSystemPerfStats(System);
		}
	#endif
		return nullptr;
	}

	static inline FParticlePerfStats* GetStats(const UFXSystemComponent* Component)
	{
	#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
		if (Component && GetGatherComponentStats() && GetStatsEnabled())
		{
			return GetComponentPerfStats(Component);
		}
	#endif
		return nullptr;
	}

	static ENGINE_API TAtomic<bool>	bStatsEnabled;
	static ENGINE_API TAtomic<int32>	WorldStatsReaders;
	static ENGINE_API TAtomic<int32>	SystemStatsReaders;
	static ENGINE_API TAtomic<int32>	ComponentStatsReaders;

	static ENGINE_API TAtomic<bool>	bCSVStatsEnabled;

	/** Stats on GT and GT spawned concurrent work. */
	FParticlePerfStats_GT GameThreadStats;

	/** Stats on RT work. */
	FParticlePerfStats_RT RenderThreadStats;

	/** Stats from GPU work. */
	FParticlePerfStats_GPU GPUStats;

	/** Returns the current frame Game Thread stats. */
	inline FParticlePerfStats_GT& GetGameThreadStats()
	{
		return GameThreadStats; 
	}

	/** Returns the current frame Render Thread stats. */
	inline FParticlePerfStats_RT& GetRenderThreadStats()
	{
		return RenderThreadStats;
	}

	/** Returns the current frame GPU stats. */
	inline FParticlePerfStats_GPU& GetGPUStats()
	{
		return GPUStats;
	}

	//Cached CSV Stat names for this system.
#if WITH_PARTICLE_PERF_CSV_STATS
	FName CSVStat_Count = NAME_None;
	FName CSVStat_Total = NAME_None;
	FName CSVStat_GTOnly = NAME_None;
	FName CSVStat_InstAvgGT = NAME_None;
	FName CSVStat_RT = NAME_None;
	FName CSVStat_InstAvgRT = NAME_None;
	FName CSVStat_GPU = NAME_None;
	FName CSVStat_InstAvgGPU = NAME_None;
	FName CSVStat_Activation = NAME_None;
	FName CSVStat_Waits = NAME_None;
	FName CSVStat_Culled = NAME_None;
	FName CSVStat_MemoryKB = NAME_None;

	void PopulateStatNames(const FName InName);
	void ResetStatNames();
#endif

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	TOptional<uint64> MemoryKB_Asset;
#endif

private:
	static ENGINE_API FParticlePerfStats* GetWorldPerfStats(const UWorld* World);
	static ENGINE_API FParticlePerfStats* GetSystemPerfStats(const UFXSystemAsset* FXAsset);
	static ENGINE_API FParticlePerfStats* GetComponentPerfStats(const UFXSystemComponent* FXComponent);

};

struct FParticlePerfStatsContext
{
	FParticlePerfStatsContext()
	: WorldStats(nullptr)
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	, SystemStats(nullptr)
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	, ComponentStats(nullptr)
#endif
	{}

	inline FParticlePerfStatsContext(FParticlePerfStats* InWorldStats, FParticlePerfStats* InSystemStats, FParticlePerfStats* InComponentStats)
	{
		SetWorldStats(InWorldStats);
		SetSystemStats(InSystemStats);
		SetComponentStats(InComponentStats);
	}

	inline FParticlePerfStatsContext(FParticlePerfStats* InWorldStats, FParticlePerfStats* InSystemStats)
	{
		SetWorldStats(InWorldStats);
		SetSystemStats(InSystemStats);
	}

	inline FParticlePerfStatsContext(FParticlePerfStats* InComponentStats)
	{
		SetComponentStats(InComponentStats);
	}

	inline FParticlePerfStatsContext(const UWorld* InWorld, const UFXSystemAsset* InSystem, const UFXSystemComponent* InComponent)
	{
		SetWorldStats(FParticlePerfStats::GetStats(InWorld));
		SetSystemStats(FParticlePerfStats::GetStats(InSystem));
		SetComponentStats(FParticlePerfStats::GetStats(InComponent));
	}

	inline FParticlePerfStatsContext(const UWorld* InWorld, const UFXSystemAsset* InSystem)
	{
		SetWorldStats(FParticlePerfStats::GetStats(InWorld));
		SetSystemStats(FParticlePerfStats::GetStats(InSystem));
	}

	inline FParticlePerfStatsContext(const UFXSystemComponent* InComponent)
	{
		SetComponentStats(FParticlePerfStats::GetStats(InComponent));
	}

	inline bool IsValid()
	{
		return GetWorldStats() != nullptr || GetSystemStats() != nullptr || GetComponentStats() != nullptr;
	}

	FParticlePerfStats* WorldStats = nullptr;
	inline FParticlePerfStats* GetWorldStats() { return WorldStats; }
	inline void SetWorldStats(FParticlePerfStats* Stats) { WorldStats = Stats; }

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	FParticlePerfStats* SystemStats = nullptr;
	inline FParticlePerfStats* GetSystemStats() { return SystemStats; }
	inline void SetSystemStats(FParticlePerfStats* Stats) { SystemStats = Stats; }
#else
	inline FParticlePerfStats* GetSystemStats() { return nullptr; }
	inline void SetSystemStats(FParticlePerfStats* Stats) { }
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	FParticlePerfStats* ComponentStats = nullptr;
	inline FParticlePerfStats* GetComponentStats() { return ComponentStats; }
	inline void SetComponentStats(FParticlePerfStats* Stats) { ComponentStats = Stats; }
#else
	inline FParticlePerfStats* GetComponentStats() { return nullptr; }
	inline void SetComponentStats(FParticlePerfStats* Stats) { }
#endif
};

typedef TFunction<void(FParticlePerfStats* Stats, uint64 Cycles)> FParticlePerfStatsWriterFunc;

template<typename TWriterFunc>
struct FParticlePerfStatScope
{
	inline FParticlePerfStatScope(FParticlePerfStatsContext InContext, int32 InCount=0)
	: Context(InContext)
	, StartCycles(INDEX_NONE)
	, Count(InCount)
	{
		if (Context.IsValid())
		{
			StartCycles = FPlatformTime::Cycles64();
		}
	}

	inline ~FParticlePerfStatScope()
	{
		if (StartCycles != INDEX_NONE)
		{
			uint64 Cycles = FPlatformTime::Cycles64() - StartCycles;
			TWriterFunc::Write(Context.GetWorldStats(), Cycles, Count);
			TWriterFunc::Write(Context.GetSystemStats(), Cycles, Count);
			TWriterFunc::Write(Context.GetComponentStats(), Cycles, Count);
		}
	}
	
	FParticlePerfStatsContext Context;
	uint64 StartCycles = 0;
	int32 Count = 0;
};

#define PARTICLE_PERF_STAT_CYCLES_COMMON(CONTEXT, THREAD, NAME)\
struct FParticlePerfStatsWriterCycles_##THREAD##_##NAME\
{\
	inline static void Write(FParticlePerfStats* Stats, uint64 Cycles, int32 Count)\
	{\
		if (Stats){ Stats->Get##THREAD##Stats().NAME##Cycles += Cycles;	}\
	}\
};\
FParticlePerfStatScope<FParticlePerfStatsWriterCycles_##THREAD##_##NAME> ANONYMOUS_VARIABLE(ParticlePerfStatScope##THREAD##NAME)(CONTEXT);

#define PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_COMMON(CONTEXT, THREAD, NAME, COUNT)\
struct FParticlePerfStatsWriterCyclesAndCount_##THREAD##_##NAME\
{\
	inline static void Write(FParticlePerfStats* Stats, uint64 Cycles, int32 Count)\
	{\
		if(Stats)\
		{\
			Stats->Get##THREAD##Stats().NAME##Cycles += Cycles; \
			Stats->Get##THREAD##Stats().NumInstances += Count; \
		}\
	}\
};\
FParticlePerfStatScope<FParticlePerfStatsWriterCyclesAndCount_##THREAD##_##NAME> ANONYMOUS_VARIABLE(ParticlePerfStatScope##THREAD##NAME)(CONTEXT, COUNT);

#define PARTICLE_PERF_STAT_CYCLES_GT(CONTEXT, NAME) PARTICLE_PERF_STAT_CYCLES_COMMON(CONTEXT, GameThread, NAME)
#define PARTICLE_PERF_STAT_CYCLES_RT(CONTEXT, NAME) PARTICLE_PERF_STAT_CYCLES_COMMON(CONTEXT, RenderThread, NAME)

#define PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(CONTEXT, NAME, COUNT) PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_COMMON(CONTEXT, GameThread, NAME, COUNT)
#define PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_RT(CONTEXT, NAME, COUNT) PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_COMMON(CONTEXT, RenderThread, NAME, COUNT)

#else //WITH_PARTICLE_PERF_STATS

#define PARTICLE_PERF_STAT_CYCLES_GT(CONTEXT, NAME)
#define PARTICLE_PERF_STAT_CYCLES_RT(CONTEXT, NAME)

#define PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(CONTEXT, NAME, COUNT)
#define PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_RT(CONTEXT, NAME, COUNT)

struct FParticlePerfStatsContext
{
	inline FParticlePerfStatsContext(FParticlePerfStats* InWorldStats, FParticlePerfStats* InSystemStats, FParticlePerfStats* InComponentStats){}
	inline FParticlePerfStatsContext(FParticlePerfStats* InWorldStats, FParticlePerfStats* InSystemStats) {}
	inline FParticlePerfStatsContext(FParticlePerfStats* InComponentStats) {}
	inline FParticlePerfStatsContext(UWorld* InWorld, UFXSystemAsset* InSystem, const UFXSystemComponent* InComponent) {}
	inline FParticlePerfStatsContext(UWorld* InWorld, UFXSystemAsset* InSystem) {}
	inline FParticlePerfStatsContext(const UFXSystemComponent* InComponent) {}
};

#endif //WITH_PARTICLE_PERF_STATS
