// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Particles/ParticlePerfStats.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/WeakObjectPtr.h"
#include "RenderingThread.h"

#if WITH_PARTICLE_PERF_STATS

#define ENABLE_PARTICLE_PERF_STATS_RENDER !UE_BUILD_SHIPPING

class FParticlePerfStatsListener_DebugRender;

#define ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES 10

struct FAccumulatedParticlePerfStats_GT
{
	uint32 NumFrames;
	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>> MaxPerFrameTotalCycles;
	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>> MaxPerInstanceCycles;

	FParticlePerfStats_GT AccumulatedStats;

	ENGINE_API FAccumulatedParticlePerfStats_GT();
	ENGINE_API void Reset();
	ENGINE_API void Tick(FParticlePerfStats& Stats);

	/** Returns the total cycles used by all GameThread stats. */
	inline uint64 GetTotalCycles() const { return AccumulatedStats.GetTotalCycles(); }
	inline uint64 GetTotalCycles_GTOnly() const { return AccumulatedStats.GetTotalCycles_GTOnly(); }

	/** Returns the average cycles per frame by all GameThread stats. */
	inline uint64 GetPerFrameAvgCycles() const { return NumFrames > 0 ? AccumulatedStats.GetTotalCycles() / NumFrames : 0; }
	inline uint64 GetPerFrameAvgCycles_GTOnly() const { return NumFrames > 0 ? AccumulatedStats.GetTotalCycles_GTOnly() / NumFrames : 0; }
	/** Returns the max cycles per frame by all GameThread stats. */
	inline uint64 GetPerFrameMaxCycles(int32 Index = 0) const { return MaxPerFrameTotalCycles[Index]; }

	/** Returns the average time in µs per frame by all GameThread stats. */
	inline float GetPerFrameAvg() const { return float(FPlatformTime::ToMilliseconds64(GetPerFrameAvgCycles()) * 1000.0); }
	inline float GetPerFrameAvg_GTOnly() const { return float(FPlatformTime::ToMilliseconds64(GetPerFrameAvgCycles_GTOnly()) * 1000.0); }
	/** Returns the max time in µs per frame by all GameThread stats. */
	inline float GetPerFrameMax(int32 Index = 0) const { return float(FPlatformTime::ToMilliseconds64(GetPerFrameMaxCycles(Index)) * 1000.0); }

	/** Returns the average cycles per instance by all GameThread stats. */
	inline uint64 GetPerInstanceAvgCycles() const { return AccumulatedStats.GetPerInstanceAvgCycles(); }
	/** Returns the max cycles per instance by all GameThread stats. */
	inline uint64 GetPerInstanceMaxCycles(int32 Index = 0) const { return MaxPerInstanceCycles[Index]; }

	/** Returns the average time in µs per instance by all GameThread stats. */
	inline float GetPerInstanceAvg() const { return float(FPlatformTime::ToMilliseconds64(GetPerInstanceAvgCycles()) * 1000.0); }
	inline float GetPerInstanceAvg_GTOnly() const { return float(FPlatformTime::ToMilliseconds64(AccumulatedStats.GetPerInstanceAvgCycles_GTOnly()) * 1000.0); }
	/** Returns the max time in µs per instance by all GameThread stats. */
	inline float GetPerInstanceMax(int32 Index = 0) const { return float(FPlatformTime::ToMilliseconds64(GetPerInstanceMaxCycles(Index)) * 1000.0); }
};

struct FAccumulatedParticlePerfStats_RT
{
	uint32 NumFrames;
	FParticlePerfStats_RT AccumulatedStats;

	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>	MaxPerFrameTotalCycles;
	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>	MaxPerInstanceCycles;

	ENGINE_API FAccumulatedParticlePerfStats_RT();
	ENGINE_API void Reset();
	ENGINE_API void Tick(FParticlePerfStats& Stats);

	/** Returns the total cycles used by all RenderThread stats. */
	inline uint64 GetTotalCycles() const { return AccumulatedStats.GetTotalCycles(); }

	/** Returns the average cycles per frame by all RenderThread stats. */
	inline uint64 GetPerFrameAvgCycles() const { return NumFrames > 0 ? AccumulatedStats.GetTotalCycles() / NumFrames : 0; }
	/** Returns the max cycles per frame by all RenderThread stats. */
	inline uint64 GetPerFrameMaxCycles(int32 Index = 0) const { return MaxPerFrameTotalCycles[Index]; }

	/** Returns the average time in µs per frame by all RenderThread stats. */
	inline float GetPerFrameAvg() const { return float(FPlatformTime::ToMilliseconds64(GetPerFrameAvgCycles()) * 1000.0); }
	/** Returns the max time in µs per frame by all RenderThread stats. */
	inline float GetPerFrameMax(int32 Index = 0) const { return float(FPlatformTime::ToMilliseconds64(GetPerFrameMaxCycles(Index)) * 1000.0); }

	/** Returns the average cycles per instance by all RenderThread stats. */
	inline uint64 GetPerInstanceAvgCycles() const { return AccumulatedStats.GetPerInstanceAvgCycles(); }
	/** Returns the max cycles per instance by all RenderThread stats. */
	inline uint64 GetPerInstanceMaxCycles(int32 Index = 0) const { return MaxPerInstanceCycles[Index];  }

	/** Returns the average time in µs per instance by all RenderThread stats. */
	inline float GetPerInstanceAvg() const { return float(FPlatformTime::ToMilliseconds64(GetPerInstanceAvgCycles()) * 1000.0); }
	/** Returns the max time in µs per instance by all RenderThread stats. */
	inline float GetPerInstanceMax(int32 Index = 0) const { return float(FPlatformTime::ToMilliseconds64(GetPerInstanceMaxCycles(Index)) * 1000.0); }
};

struct FAccumulatedParticlePerfStats_GPU
{
	uint32 NumFrames;
	FParticlePerfStats_GPU AccumulatedStats;

	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>	MaxPerFrameTotalMicroseconds;
	TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>	MaxPerInstanceMicroseconds;

	ENGINE_API FAccumulatedParticlePerfStats_GPU();
	ENGINE_API void Reset();
	ENGINE_API void Tick(FParticlePerfStats& Stats);

	/** Returns the total microseconds used by GPU. */
	inline uint64 GetTotalMicroseconds() const { return AccumulatedStats.GetTotalMicroseconds(); }

	inline uint64 GetPerFrameAvgMicroseconds() const { return NumFrames > 0 ? AccumulatedStats.GetTotalMicroseconds() / NumFrames : 0; }
	inline uint64 GetPerFrameMaxMicroseconds(int32 Index = 0) const { return MaxPerFrameTotalMicroseconds[Index]; }

	inline uint64 GetPerInstanceAvgMicroseconds() const { return AccumulatedStats.GetPerInstanceAvgMicroseconds(); }
	inline uint64 GetPerInstanceMaxMicroseconds(int32 Index = 0) const { return MaxPerInstanceMicroseconds[Index]; }
};

/** Utility class for accumulating many frames worth of stats data. */
struct FAccumulatedParticlePerfStats
{
	ENGINE_API FAccumulatedParticlePerfStats();

	ENGINE_API void Reset(bool bSyncWithRT);
	ENGINE_API void ResetGT();
	ENGINE_API void ResetRT();
	ENGINE_API void Tick(FParticlePerfStats& Stats);
	ENGINE_API void TickRT(FParticlePerfStats& Stats);

	FAccumulatedParticlePerfStats_GT GameThreadStats;
	FAccumulatedParticlePerfStats_RT RenderThreadStats;
	FAccumulatedParticlePerfStats_GPU GPUStats;

	static ENGINE_API void AddMax(TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>& MaxArray, int64 NewValue);
	static ENGINE_API void ResetMaxArray(TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>& MaxArray);

	/** Returns the current Game Thread stats. */
	inline FAccumulatedParticlePerfStats_GT& GetGameThreadStats()
	{
		return GameThreadStats;
	}

	/** Returns the RenderThread stats with an optional flush when on the GameThread. */
	inline FAccumulatedParticlePerfStats_RT& GetRenderThreadStats(bool bFlushForGameThread = false)
	{
		if ( IsInGameThread() )
		{
			if ( bFlushForGameThread )
			{
				FlushRenderingCommands();
			}
		}
		else
		{
			ensure(IsInRenderingThread());
		}
		return RenderThreadStats;
	}

	/** Returns the GPU stats with an optional flush when on the GameThread. */
	inline FAccumulatedParticlePerfStats_GPU& GetGPUStats(bool bFlushForGameThread = false)
	{
		if (IsInGameThread())
		{
			if (bFlushForGameThread)
			{
				FlushRenderingCommands();
			}
		}
		else
		{
			ensure(IsInRenderingThread());
		}
		return GPUStats;
	}
};

class FParticlePerfStatsListener : public TSharedFromThis<FParticlePerfStatsListener, ESPMode::ThreadSafe>
{
public:
	virtual ~FParticlePerfStatsListener() = default;

	/** Called when the listener begins receiving data. */
	virtual void Begin(){}
	/** Called when the listener stops receiving data. */
	virtual void End(){}
	/** Called every frame with the current frame data. Returns true if we should continue listening. If false is returned the listener will be removed. */
	virtual bool Tick() { return true; }
	/** Called every frame from the render thread gather any RT stats. */
	virtual void TickRT() {}

	/** Called when a new world is seen for the first time. */
	virtual void OnAddWorld(const TWeakObjectPtr<const UWorld>& NewWorld) {}
	/** Called when a world has been freed and is no longer tracked by the stats. */
	virtual void OnRemoveWorld(const TWeakObjectPtr<const UWorld>& World) {}
	/** Called when a new system is seen for the first time. */
	virtual void OnAddSystem(const TWeakObjectPtr<const UFXSystemAsset>& NewSystem){}
	/** Called when a system has been freed and is no longer tracked by the stats. */
	virtual void OnRemoveSystem(const TWeakObjectPtr<const UFXSystemAsset>& System) {}
	/** Called when a new component is seen for the first time. */
	virtual void OnAddComponent(const TWeakObjectPtr<const UFXSystemComponent>& NewComponent) {}
	/** Called when a component has been freed and is no longer tracked by the stats. */
	virtual void OnRemoveComponent(const TWeakObjectPtr<const UFXSystemComponent>& Component) {}

	virtual bool NeedsMemoryStats() const { return false; }
	virtual bool NeedsWorldStats() const = 0;
	virtual bool NeedsSystemStats() const = 0;
	virtual bool NeedsComponentStats() const = 0;

	/** 
	Controls whether this listener should be stooped and cleaned up when it's orphaned, i.e. the manager is the only one with a reference.
	In some cases we want to signal that a listener should stop by clearing an external reference.
	In other cases we want to have fire and forget listeners that can signal their own termination via their tick function.
	*/
	virtual bool AllowOrphaned() const { return false; }
};

typedef TSharedPtr<FParticlePerfStatsListener, ESPMode::ThreadSafe> FParticlePerfStatsListenerPtr;
class FParticlePerfStatsManager
{
public:
	static ENGINE_API FDelegateHandle BeginFrameHandle;
#if CSV_PROFILER_STATS
	static ENGINE_API FDelegateHandle CSVStartHandle;
	static ENGINE_API FDelegateHandle CSVEndHandle;
#endif

	static ENGINE_API int32 StatsEnabled;
	static ENGINE_API FCriticalSection WorldToPerfStatsGuard;
	static ENGINE_API TMap<TWeakObjectPtr<const UWorld>, TUniquePtr<FParticlePerfStats>> WorldToPerfStats;
	static ENGINE_API TArray<TUniquePtr<FParticlePerfStats>> FreeWorldStatsPool;
	static const TMap<TWeakObjectPtr<const UWorld>, TUniquePtr<FParticlePerfStats>>& GetCurrentWorldStats() { return WorldToPerfStats; }

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	static ENGINE_API FCriticalSection SystemToPerfStatsGuard;
	static ENGINE_API TMap<TWeakObjectPtr<const UFXSystemAsset>, TUniquePtr<FParticlePerfStats>> SystemToPerfStats;
	static ENGINE_API TArray<TUniquePtr<FParticlePerfStats>> FreeSystemStatsPool;
	static const TMap<TWeakObjectPtr<const UFXSystemAsset>, TUniquePtr<FParticlePerfStats>>& GetCurrentSystemStats() { return SystemToPerfStats; }
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	static ENGINE_API FCriticalSection ComponentToPerfStatsGuard;
	static ENGINE_API TMap<TWeakObjectPtr<const UFXSystemComponent>, TUniquePtr<FParticlePerfStats>> ComponentToPerfStats;
	static ENGINE_API TArray<TUniquePtr<FParticlePerfStats>> FreeComponentStatsPool;
	static const TMap<TWeakObjectPtr<const UFXSystemComponent>, TUniquePtr<FParticlePerfStats>>& GetCurrentComponentStats() { return ComponentToPerfStats; }
#endif

	static ENGINE_API TArray<FParticlePerfStatsListenerPtr, TInlineAllocator<8>> Listeners;
	static ENGINE_API FParticlePerfStats* GetWorldPerfStats(const UWorld* World);
	static ENGINE_API FParticlePerfStats* GetSystemPerfStats(const UFXSystemAsset* FXAsset);
	static ENGINE_API FParticlePerfStats* GetComponentPerfStats(const UFXSystemComponent* FXComponent);

	static ENGINE_API void OnStartup();
	static ENGINE_API void OnShutdown();

	static ENGINE_API void TogglePerfStatsRender(UWorld* World);
	static ENGINE_API int32 RenderStats(UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

	static ENGINE_API void Reset();
	static ENGINE_API void Tick();

	static ENGINE_API void AddListener(FParticlePerfStatsListenerPtr Listener, bool bReset = true);
	static ENGINE_API void RemoveListener(FParticlePerfStatsListener* Listener);
	static ENGINE_API void RemoveListener(FParticlePerfStatsListenerPtr Listener);

#if ENABLE_PARTICLE_PERF_STATS_RENDER
	/** Track active worlds needing to render stats. Though we only create one listener. TODO: Maybe better to track by viewport than world? */
	static ENGINE_API TMap<TWeakObjectPtr<UWorld>, TSharedPtr<FParticlePerfStatsListener_DebugRender ,ESPMode::ThreadSafe>> DebugRenderListenerUsers;
#endif

	/** Calls the supplied function for all tracked UWorld stats. */
	template<typename TAction>
	static void ForAllWorldStats(TAction Func)
	{
		FScopeLock Lock(&WorldToPerfStatsGuard);
		for (TPair<TWeakObjectPtr<const UWorld>, TUniquePtr<FParticlePerfStats>>& Pair : WorldToPerfStats)
		{
			Func(Pair.Key, Pair.Value);
		}
	}	
	
	/** Calls the supplied function for all tracked UFXSysteAsset stats. */
	template<typename TAction>
	static void ForAllSystemStats(TAction Func)
	{
	#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FScopeLock Lock(&SystemToPerfStatsGuard);
		for (TPair<TWeakObjectPtr<const UFXSystemAsset>, TUniquePtr<FParticlePerfStats>>& Pair : SystemToPerfStats)
		{
			Func(Pair.Key, Pair.Value);
		}
	#endif
	}

	/** Calls the supplied function for all tracked UFXSystemComponent stats. */
	template<typename TAction>
	static void ForAllComponentStats(TAction Func)
	{
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FScopeLock Lock(&ComponentToPerfStatsGuard);
		for (TPair<TWeakObjectPtr<const UFXSystemComponent>, TUniquePtr<FParticlePerfStats>>& Pair : ComponentToPerfStats)
		{
			Func(Pair.Key, Pair.Value);
		}
#endif
	}
};

/** Base class for listeners that gather stats on all systems in the scene. */
class FParticlePerfStatsListener_GatherAll: public FParticlePerfStatsListener
{
public:
	FParticlePerfStatsListener_GatherAll(bool bNeedsWorldStats, bool bNeedsSystemStats, bool bNeedsComponentStats)
	: bGatherWorldStats(bNeedsWorldStats)
	, bGatherSystemStats(bNeedsSystemStats)
	, bGatherComponentStats(bNeedsComponentStats)
	{}

	virtual ~FParticlePerfStatsListener_GatherAll() = default;

	ENGINE_API virtual void Begin() override;
	ENGINE_API virtual void End() override;
	ENGINE_API virtual bool Tick() override;
	ENGINE_API virtual void TickRT() override;

	ENGINE_API virtual void OnAddWorld(const TWeakObjectPtr<const UWorld>& NewWorld)override;
	ENGINE_API virtual void OnRemoveWorld(const TWeakObjectPtr<const UWorld>& World)override;

	#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	ENGINE_API virtual void OnAddSystem(const TWeakObjectPtr<const UFXSystemAsset>& NewSystem)override;
	ENGINE_API virtual void OnRemoveSystem(const TWeakObjectPtr<const UFXSystemAsset>& System)override;
	#endif

	#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	ENGINE_API virtual void OnAddComponent(const TWeakObjectPtr<const UFXSystemComponent>& NewComponent)override;
	ENGINE_API virtual void OnRemoveComponent(const TWeakObjectPtr<const UFXSystemComponent>& Component)override;
	#endif

	virtual bool NeedsMemoryStats() const override { return true; }
	virtual bool NeedsWorldStats() const override { return bGatherWorldStats; }
	virtual bool NeedsSystemStats() const override { return bGatherSystemStats; }
	virtual bool NeedsComponentStats() const override { return bGatherComponentStats; }

	ENGINE_API void DumpStatsToDevice(FOutputDevice& Ar);
	ENGINE_API void DumpStatsToFile();

#if WITH_PARTICLE_PERF_STATS
	ENGINE_API FAccumulatedParticlePerfStats* GetStats(const UWorld* World);
#else
	FAccumulatedParticlePerfStats* GetStats(const UWorld* World) { return nullptr; }
#endif
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	ENGINE_API FAccumulatedParticlePerfStats* GetStats(const UFXSystemAsset* System);
#else
	FAccumulatedParticlePerfStats* GetStats(const UFXSystemAsset* System){ return nullptr; }
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	ENGINE_API FAccumulatedParticlePerfStats* GetStats(const UFXSystemComponent* Component);
#else
	FAccumulatedParticlePerfStats* GetStats(const UFXSystemComponent* Component){ return nullptr; }
#endif

protected:
	FCriticalSection AccumulatedStatsGuard;

	const uint8 bGatherWorldStats : 1;
	const uint8 bGatherSystemStats : 1;
	const uint8 bGatherComponentStats : 1;

	TMap<TWeakObjectPtr<const UWorld>, TUniquePtr<FAccumulatedParticlePerfStats>> AccumulatedWorldStats;

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	TMap<TWeakObjectPtr<const UFXSystemAsset>, TUniquePtr<FAccumulatedParticlePerfStats>> AccumulatedSystemStats;
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	TMap<TWeakObjectPtr<const UFXSystemComponent>, TUniquePtr<FAccumulatedParticlePerfStats>> AccumulatedComponentStats;
#endif

	template<typename T, typename TFunc>
	void TickStats_Internal(TMap<TWeakObjectPtr<T>, TUniquePtr<FAccumulatedParticlePerfStats>>& StatsMap, TFunc Func);
};

/** Simple stats listener that will gather stats on all systems for N frames and dump the results to a CSV and the Log. */
class FParticlePerfStatsListener_TimedTest : public FParticlePerfStatsListener_GatherAll
{
public:
	ENGINE_API FParticlePerfStatsListener_TimedTest(int32 NumFrames, bool bInGatherWorldStats, bool bInGatherSystemStats, bool bInGatherComponentStats);

	ENGINE_API virtual void End()override;
	ENGINE_API virtual bool Tick()override;

	virtual bool AllowOrphaned() const { return true; }

private:
	int32 FramesRemaining;
};

/** Listener that hooks into the engine wide CSV Profiling systems. */
class FParticlePerfStatsListener_CSVProfiler : public FParticlePerfStatsListener_GatherAll
{
public:
	FParticlePerfStatsListener_CSVProfiler() : FParticlePerfStatsListener_GatherAll(false, true, false) {}

#if WITH_PARTICLE_PERF_CSV_STATS
	ENGINE_API virtual bool Tick()override;
	ENGINE_API virtual void TickRT()override;
	ENGINE_API virtual void End()override;

	static ENGINE_API void OnCSVStart();
	static ENGINE_API void OnCSVEnd();
#endif

private:
	static ENGINE_API FParticlePerfStatsListenerPtr CSVListener;
};

/**
This listener displays stats onto a debug canvas in a viewport.
It does not sync with the Render Thread and so RT stats are one or more frames delayed.
*/
class FParticlePerfStatsListener_DebugRender : public FParticlePerfStatsListener_GatherAll
{
public:
	FParticlePerfStatsListener_DebugRender() : FParticlePerfStatsListener_GatherAll(false, true, false){}
	ENGINE_API int32 RenderStats(UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);
};

#else

struct FAccumulatedParticlePerfStats{};

#endif
