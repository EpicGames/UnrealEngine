// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleEmitter.h"
#include "ParticleSpriteEmitter.generated.h"

class UParticleSystemComponent;

UENUM()
enum EParticleScreenAlignment : int
{
	PSA_FacingCameraPosition,
	PSA_Square,
	PSA_Rectangle,
	PSA_Velocity,
	PSA_AwayFromCenter,
	PSA_TypeSpecific,
	PSA_FacingCameraDistanceBlend,
	PSA_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, MinimalAPI)
class UParticleSpriteEmitter : public UParticleEmitter
{
	GENERATED_UCLASS_BODY()


	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UParticleEmitter Interface
	virtual FParticleEmitterInstance* CreateInstance(IParticleEmitterInstanceOwner& InComponent) override;
	virtual void SetToSensibleDefaults() override;
	//~ End UParticleEmitter Interface
};

