// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/ScriptMacros.h"
#include "Particles/ParticlePerfStats.h"
#include "Async/TaskGraphInterfaces.h"
#include "RHIDefinitions.h"
#include "PSOPrecache.h"

#include "ParticleSystem.generated.h"

enum class EParticleSignificanceLevel : uint8;
enum class EParticleSystemInsignificanceReaction: uint8;
class UInterpCurveEdSetup;
class UMaterialInterface;
class UParticleSystemComponent;
class UParticleEmitter;

/**
 *	ParticleSystemUpdateMode
 *	Enumeration indicating the method by which the system should be updated
 */
UENUM()
enum EParticleSystemUpdateMode : int
{
	/** RealTime	- update via the delta time passed in				*/
	EPSUM_RealTime UMETA(DisplayName="Real-Time"),
	/** FixedTime	- update via a fixed time step						*/
	EPSUM_FixedTime UMETA(DisplayName="Fixed-Time")
};

/**
 *	ParticleSystemLODMethod
 */
UENUM()
enum ParticleSystemLODMethod : int
{
	// Automatically set the LOD level, checking every LODDistanceCheckTime seconds.
	PARTICLESYSTEMLODMETHOD_Automatic UMETA(DisplayName="Automatic"),

	// LOD level is directly set by the game code.
	PARTICLESYSTEMLODMETHOD_DirectSet UMETA(DisplayName = "Direct Set"),

	// LOD level is determined at Activation time, then left alone unless directly set by game code.
	PARTICLESYSTEMLODMETHOD_ActivateAutomatic UMETA(DisplayName = "Activate Automatic")
};

/** Occlusion method enumeration */
UENUM()
enum EParticleSystemOcclusionBoundsMethod : int
{
	/** Don't determine occlusion on this particle system */
	EPSOBM_None UMETA(DisplayName="None"),
	/** Use the bounds of the particle system component when determining occlusion */
	EPSOBM_ParticleBounds UMETA(DisplayName="Particle Bounds"),
	/** Use the custom occlusion bounds when determining occlusion */
	EPSOBM_CustomBounds UMETA(DisplayName="Custom Bounds")
};

/** Structure containing per-LOD settings that pertain to the entire UParticleSystem. */
USTRUCT()
struct FParticleSystemLOD
{
	GENERATED_USTRUCT_BODY()

	FParticleSystemLOD()
	{
	}

	static FParticleSystemLOD CreateParticleSystemLOD()
	{
		FParticleSystemLOD NewLOD;
		return NewLOD;
	}
};

/**
 *	Temporary array for tracking 'solo' emitter mode.
 *	Entry will be true if emitter was enabled 
 */
USTRUCT()
struct FLODSoloTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(transient)
	TArray<uint8> SoloEnableSetting;

};

USTRUCT()
struct FNamedEmitterMaterial
{
	GENERATED_USTRUCT_BODY()

	FNamedEmitterMaterial()
	: Name(NAME_None)
	, Material(nullptr)
	{
	}

	UPROPERTY(EditAnywhere, Category = NamedMaterial)
	FName Name;

	UPROPERTY(EditAnywhere, Category = NamedMaterial)
	TObjectPtr<UMaterialInterface> Material;
};

using FMaterialPSOPrecacheRequestID = uint32;

UCLASS(Abstract, MinimalAPI, BlueprintType)
class UFXSystemAsset : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	UFXSystemAsset() {}

	/** Max number of components of this system to keep resident in the world component pool. */
	UPROPERTY(EditAnywhere, Category = Performance, AdvancedDisplay)
	uint32 MaxPoolSize;
	//TODO: Allow pool size overriding per world and possibly implement some preallocation too.

	/**
	* How many instances we should use to initially prime the pool.
	* This can amortize runtime activation cost by moving it to load time.
	* Use with care as this could cause large hitches for systems loaded/unloaded during play rather than at level load.
	*/
	UPROPERTY(EditAnywhere, Category = Performance, AdvancedDisplay)
	uint32 PoolPrimeSize = 0;

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	mutable FParticlePerfStats* ParticlePerfStats = nullptr;
#endif

	virtual void PrecacheAssetPSOs() {};
	const bool HasLaunchedPSOPrecaching() const { return PSOPrecachingLaunched; }
	const FGraphEventRef& GetPrecachePSOsEvent() const { return PrecachePSOsEvent; }
	const TArray<FMaterialPSOPrecacheRequestID>& GetMaterialPSOPrecacheRequestIDs() const { return MaterialPSOPrecacheRequestIDs; }
	ENGINE_API bool IsLoadingPSOCache(UMaterialInterface* Material) const;
	
protected:
	
	ENGINE_API void LaunchPSOPrecachingInternal(const FMaterialInterfacePSOPrecacheParamsList& PSOPrecacheParamsList);

	bool PSOPrecachingLaunched = false;
	FGraphEventRef PrecachePSOsEvent;
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	TMap<FObjectKey, FGraphEventArray> MaterialPSORequests;
};

/**
 * A ParticleSystem is a complete particle effect that contains any number of ParticleEmitters. By allowing multiple emitters
 * in a system, the designer can create elaborate particle effects that are held in a single system. Once created using
 * Cascade, a ParticleSystem can then be inserted into a level or created in script.
 */
UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class UParticleSystem : public UFXSystemAsset
{
	GENERATED_UCLASS_BODY()

	UParticleSystem()
	:OcclusionBoundsMethod(EPSOBM_ParticleBounds)
	{

	}

	/** UpdateTime_FPS	- the frame per second to update at in FixedTime mode		*/
	UPROPERTY(EditAnywhere, Category=ParticleSystem)
	float UpdateTime_FPS;

	/** UpdateTime_Delta	- internal												*/
	UPROPERTY()
	float UpdateTime_Delta;

	/** 
	 * WarmupTime - the time to warm-up the particle system when first rendered	
	 * Warning: WarmupTime is implemented by simulating the particle system for the time requested upon activation.  
	 * This is extremely prone to cause hitches, especially with large particle counts - use with caution.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleSystem, meta=(DisplayName="Warmup Time - beware hitches!"))
	float WarmupTime;

	/**	WarmupTickRate - the time step for each tick during warm up.
		Increasing this improves performance. Decreasing, improves accuracy.
		Set to 0 to use the default tick time.										*/
	UPROPERTY(EditAnywhere, Category=ParticleSystem)
	float WarmupTickRate;

	/** Emitters	- internal - the array of emitters in the system				*/
	UPROPERTY(instanced)
	TArray<TObjectPtr<UParticleEmitter>> Emitters;

	/** The component used to preview the particle system in Cascade				*/
	UPROPERTY(transient)
	TObjectPtr<UParticleSystemComponent> PreviewComponent;

#if WITH_EDITORONLY_DATA
	/** The angle to use when rendering the thumbnail image							*/
	UPROPERTY()
	FRotator ThumbnailAngle;

	/** The distance to place the system when rendering the thumbnail image			*/
	UPROPERTY()
	float ThumbnailDistance;

	/** The time to warm-up the system for the thumbnail image						*/
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	float ThumbnailWarmup;

#endif // WITH_EDITORONLY_DATA
	/** Used for curve editor to remember curve-editing setup.						*/
	UPROPERTY(export)
	TObjectPtr<UInterpCurveEdSetup> CurveEdSetup;

	//
	//	LOD
	//
	/**
	 *	How often (in seconds) the system should perform the LOD distance check.
	 */
	UPROPERTY(EditAnywhere, Category=LOD, AssetRegistrySearchable)
	float LODDistanceCheckTime;

	/** World space radius that UVs generated with the ParticleMacroUV material node will tile based on. */
	UPROPERTY(EditAnywhere, Category = MacroUV)
	float MacroUVRadius;

	/**
	 *	The array of distances for each LOD level in the system.
	 *	Used when LODMethod is set to PARTICLESYSTEMLODMETHOD_Automatic.
	 *
	 *	Example: System with 3 LOD levels
	 *		LODDistances(0) = 0.0
	 *		LODDistances(1) = 2500.0
	 *		LODDistances(2) = 5000.0
	 *
	 *		In this case, when the system is [   0.0 ..   2499.9] from the camera, LOD level 0 will be used.
	 *										 [2500.0 ..   4999.9] from the camera, LOD level 1 will be used.
	 *										 [5000.0 .. INFINITY] from the camera, LOD level 2 will be used.
	 *
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category=LOD)
	TArray<float> LODDistances;

#if WITH_EDITORONLY_DATA
	/** LOD setting for intepolation (set by Cascade) Range [0..100]				*/
	UPROPERTY()
	int32 EditorLODSetting;

#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category=LOD)
	TArray<FParticleSystemLOD> LODSettings;

	/**	Fixed relative bounding box for particle system.							*/
	UPROPERTY(EditAnywhere, Category=Bounds)
	FBox FixedRelativeBoundingBox;

	/**
	 * Number of seconds of emitter not being rendered that need to pass before it
	 * no longer gets ticked/ becomes inactive.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleSystem)
	float SecondsBeforeInactive;

#if WITH_EDITORONLY_DATA
	//
	//	Cascade 'floor' mesh information.
	//
	UPROPERTY()
	FString FloorMesh;

	UPROPERTY()
	FVector FloorPosition;

	UPROPERTY()
	FRotator FloorRotation;

	UPROPERTY()
	float FloorScale;

	UPROPERTY()
	FVector FloorScale3D;

	/** The background color to display in Cascade */
	UPROPERTY()
	FColor BackgroundColor;

#endif // WITH_EDITORONLY_DATA
	
	/** How long this Particle system should delay when ActivateSystem is called on it. */
	UPROPERTY(EditAnywhere, Category=Delay, AssetRegistrySearchable)
	float Delay;

	/** The low end of the emitter delay if using a range. */
	UPROPERTY(EditAnywhere, Category=Delay)
	float DelayLow;

	/** If true, the system's Z axis will be oriented toward the camera				*/
	UPROPERTY(EditAnywhere, Category = ParticleSystem)
	uint8 bOrientZAxisTowardCamera : 1;

	/** Whether to use the fixed relative bounding box or calculate it every frame. */
	UPROPERTY(EditAnywhere, Category = Bounds)
	uint8 bUseFixedRelativeBoundingBox : 1;

	/** EDITOR ONLY: Indicates that Cascade would like to have the PeakActiveParticles count reset */
	UPROPERTY()
	uint8 bShouldResetPeakCounts : 1;

	/** Set during load time to indicate that physics is used... */
	UPROPERTY(transient)
	uint8 bHasPhysics : 1;

	/** Inidicates the old 'real-time' thumbnail rendering should be used	*/
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	uint8 bUseRealtimeThumbnail : 1;

	/** Internal: Indicates the PSys thumbnail image is out of date			*/
	UPROPERTY()
	uint8 ThumbnailImageOutOfDate : 1;

private:
	/** if true, this psys can tick in any thread **/
	uint8 bIsElligibleForAsyncTick : 1;
	/** if true, bIsElligibleForAsyncTick is set up **/
	uint8 bIsElligibleForAsyncTickComputed : 1;
public:

#if WITH_EDITORONLY_DATA
	/** Internal: The PSys thumbnail image									*/
	UPROPERTY()
	TObjectPtr<class UTexture2D> ThumbnailImage;

#endif // WITH_EDITORONLY_DATA

	/**
	 *	If true, select the emitter delay from the range 
	 *		[DelayLow..Delay]
	 */
	UPROPERTY(EditAnywhere, Category=Delay)
	uint8 bUseDelayRange : 1;

	UPROPERTY(EditAnywhere, Category = Performance, meta = (ToolTip = "Whether or not to allow instances of this system to have their ticks managed."), AdvancedDisplay)
	uint8 bAllowManagedTicking : 1;

	UPROPERTY(EditAnywhere, Category = Performance, meta = (ToolTip = "Auto-deactivate system if all emitters are determined to not spawn particles again, regardless of lifetime."))
	uint8 bAutoDeactivate : 1;

	/**
	 *	Internal value that tracks the regenerate LOD levels preference.
	 *	If true, when autoregenerating LOD levels in code, the low level will
	 *	be a duplicate of the high.
	 */	
	UPROPERTY()
	uint8 bRegenerateLODDuplicate : 1;

	UPROPERTY(EditAnywhere, Category = ParticleSystem, AssetRegistrySearchable)
	TEnumAsByte<enum EParticleSystemUpdateMode> SystemUpdateMode;

	/**
	 *	The method of LOD level determination to utilize for this particle system
	 *	  PARTICLESYSTEMLODMETHOD_Automatic - Automatically set the LOD level, checking every LODDistanceCheckTime seconds.
	 *    PARTICLESYSTEMLODMETHOD_DirectSet - LOD level is directly set by the game code.
	 *    PARTICLESYSTEMLODMETHOD_ActivateAutomatic - LOD level is determined at Activation time, then left alone unless directly set by game code.
	 */
	UPROPERTY(EditAnywhere, Category=LOD)
	TEnumAsByte<enum ParticleSystemLODMethod> LODMethod;

	/** The reaction this system takes when all emitters are insignificant. */
	UPROPERTY(EditAnywhere, Category = Performance)
	EParticleSystemInsignificanceReaction InsignificantReaction;

	/**
	 *	Which occlusion bounds method to use for this particle system.
	 *	EPSOBM_None - Don't determine occlusion for this system.
	 *	EPSOBM_ParticleBounds - Use the bounds of the component when determining occlusion.
	 */
	UPROPERTY(EditAnywhere, Category = Occlusion)
	TEnumAsByte<enum EParticleSystemOcclusionBoundsMethod> OcclusionBoundsMethod;

private:
	/** Does any emitter loop forever? */
	uint8 bAnyEmitterLoopsForever : 1;

public:

	/** The maximum level of significance for emitters in this system. Any emitters with a higher significance will be capped at this significance level. */
	UPROPERTY(EditAnywhere, Category = Performance)
	EParticleSignificanceLevel MaxSignificanceLevel;

	UPROPERTY(EditAnywhere, Category = Performance, meta = (ToolTip = "Minimum duration between ticks; 33=tick at max. 30FPS, 16=60FPS, 8=120FPS"))
	uint32 MinTimeBetweenTicks;

	/** Time delay between all emitters becoming insignificant and the systems insignificant reaction. */
	UPROPERTY(EditAnywhere, Category = Performance)
	float InsignificanceDelay;

	/** Local space position that UVs generated with the ParticleMacroUV material node will be centered on. */
	UPROPERTY(EditAnywhere, Category=MacroUV)
	FVector MacroUVPosition;

	/** The occlusion bounds to use if OcclusionBoundsMethod is set to EPSOBM_CustomBounds */
	UPROPERTY(EditAnywhere, Category=Occlusion)
	FBox CustomOcclusionBounds;

	UPROPERTY(transient)
	TArray<struct FLODSoloTrack> SoloTracking;

	/** 
	*	Array of named material slots for use by emitters of this system. 
	*	Emitters can use these instead of their own materials by providing the name to the NamedMaterialOverrides property of their required module.
	*	These materials can be overridden using CreateNamedDynamicMaterialInstance() on a ParticleSystemComponent.
	*/
	UPROPERTY(EditAnywhere, Category = Materials)
	TArray<FNamedEmitterMaterial> NamedMaterialSlots;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void PostLoad() override;
	virtual bool IsPostLoadThreadSafe() const override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	bool UsesCPUCollision() const;
	virtual bool CanBeClusterRoot() const override;
	virtual void Serialize(FArchive& Ar) override;

	//~ End UObject Interface.

	virtual void PrecacheAssetPSOs() override;

	bool CanBePooled()const;

	// @todo document
	void UpdateColorModuleClampAlpha(class UParticleModuleColorBase* ColorModule);

	/**
	 *	CalculateMaxActiveParticleCounts
	 *	Determine the maximum active particles that could occur with each emitter.
	 *	This is to avoid reallocation during the life of the emitter.
	 *
	 *	@return	true	if the numbers were determined for each emitter
	 *			false	if not be determined
	 */
	virtual bool		CalculateMaxActiveParticleCounts();
	
	/**
	 *	Retrieve the parameters associated with this particle system.
	 *
	 *	@param	ParticleSysParamList	The list of FParticleSysParams used in the system
	 *	@param	ParticleParameterList	The list of ParticleParameter distributions used in the system
	 */
	ENGINE_API void GetParametersUtilized(TArray<TArray<FString> >& ParticleSysParamList,
							   TArray<TArray<FString> >& ParticleParameterList);

	/**
	 *	Setup the soloing information... Obliterates all current soloing.
	 */
	ENGINE_API void SetupSoloing();

	/**
	 *	Toggle the bIsSoloing flag on the given emitter.
	 *
	 *	@param	InEmitter		The emitter to toggle.
	 *
	 *	@return	bool			true if ANY emitters are set to soloing, false if none are.
	 */
	ENGINE_API bool ToggleSoloing(class UParticleEmitter* InEmitter);

	/**
	 *	Turn soloing off completely - on every emitter
	 *
	 *	@return	bool			true if successful, false if not.
	 */
	ENGINE_API bool TurnOffSoloing();

	/**
	 *	Editor helper function for setting the LOD validity flags used in Cascade.
	 */
	ENGINE_API void SetupLODValidity();

	/**
	 * Set the time to delay spawning the particle system
	 */
	ENGINE_API void SetDelay(float InDelay);

#if WITH_EDITOR
	/**
	 *	Remove all duplicate modules.
	 *
	 *	@param	bInMarkForCooker	If true, mark removed objects to not cook out.
	 *	@param	OutRemovedModules	Optional map to fill in w/ removed modules...
	 *
	 *	@return	bool				true if successful, false if not
	 */
	ENGINE_API bool RemoveAllDuplicateModules(bool bInMarkForCooker, TMap<UObject*,bool>* OutRemovedModules);

	/**
	 *	Update all emitter module lists
	 */
	ENGINE_API void UpdateAllModuleLists();
#endif
	/** Return the currently set LOD method											*/
	virtual enum ParticleSystemLODMethod GetCurrentLODMethod();

	/**
	 *	Return the number of LOD levels for this particle system
	 *
	 *	@return	The number of LOD levels in the particle system
	 */
	virtual int32 GetLODLevelCount();
	
	/**
	 *	Return the distance for the given LOD level
	 *
	 *	@param	LODLevelIndex	The LOD level that the distance is being retrieved for
	 *
	 *	@return	-1.0f			If the index is invalid
	 *			Distance		The distance set for the LOD level
	 */
	virtual float GetLODDistance(int32 LODLevelIndex);
	
	/**
	 *	Set the LOD method
	 *
	 *	@param	InMethod		The desired method
	 */
	virtual void SetCurrentLODMethod(ParticleSystemLODMethod InMethod);

	/**
	 *	Set the distance for the given LOD index
	 *
	 *	@param	LODLevelIndex	The LOD level to set the distance of
	 *	@param	InDistance		The distance to set
	 *
	 *	@return	true			If successful
	 *			false			Invalid LODLevelIndex
	 */
	virtual bool SetLODDistance(int32 LODLevelIndex, float InDistance);

	/**
	*	Checks if any of the emitters have motion blur at a specific lod level.
	*
	*	@param	LODLevelIndex	The LOD level to check motion blur availability
	*
	*	@return	true			If any emitter has motion blur
	*			false			None of the emitters have motion blur
	*/
	bool DoesAnyEmitterHaveMotionBlur(int32 LODLevelIndex) const;

	/**
	 * Builds all emitters in the particle system.
	 */
	ENGINE_API void BuildEmitters();

	/** return true if this psys can tick in any thread */
	inline bool CanTickInAnyThread()
	{
		if (!bIsElligibleForAsyncTickComputed)
		{
			ComputeCanTickInAnyThread();
		}
		return bIsElligibleForAsyncTick;
	}
	/** Decide if this psys can tick in any thread, and set bIsElligibleForAsyncTick */
	ENGINE_API void ComputeCanTickInAnyThread();

	/** Returns true if this system contains any GPU emitters. */
	bool HasGPUEmitter() const;

	/** 
	Returns true if this system contains an emitter of the pasesd type. 
	@ param TypeData - The emitter type to check for. Must be a child class of UParticleModuleTypeDataBase
	*/
	UFUNCTION(BlueprintCallable, Category = "Particle System")
	bool ContainsEmitterType(UClass* TypeData);

	/** Returns true if the particle system is looping (contains one or more looping emitters) */
	bool IsLooping() const { return bAnyEmitterLoopsForever; }
	bool IsImmortal() const { return bIsImmortal; }
	bool WillBecomeZombie() const { return bWillBecomeZombie; }

	EParticleSignificanceLevel GetHighestSignificance()const { return HighestSignificance; }
	EParticleSignificanceLevel GetLowestSignificance()const { return LowestSignificance; }
	bool ShouldManageSignificance()const { return bShouldManageSignificance; }

	inline bool AllowManagedTicking()const { return bAllowManagedTicking; }
private:

	/** The highest significance of any emitter. Clamped by MaxSignificanceLevel.*/
	EParticleSignificanceLevel HighestSignificance;
	/** The lowest significance of any emitter. Clamped by MaxSignificanceLevel.*/
	EParticleSignificanceLevel LowestSignificance;
	
	uint8 bShouldManageSignificance : 1;

	/** Does any emitter never die due to infinite looping AND indefinite duration? */
	uint8 bIsImmortal : 1;
	/** Does any emitter ever become a zombie (is immortal AND stops spawning at some point, i.e. is burst only)? */
	uint8 bWillBecomeZombie : 1;
};