// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Components/SceneComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInterface.h"
#include "Particles/ParticleSystem.h"
#include "Particles/Emitter.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Particles/WorldPSCPool.h"
#include "Particles/ParticlePerfStats.h"
#include "StateStream/ParticleSystemStateStreamHandle.h"
#include "ParticleSystemComponent.generated.h"

class FParticleDynamicData;
class FPrimitiveSceneProxy;
class UParticleSystemReplay;
class UPhysicalMaterial;
struct EventData;
struct FDynamicEmitterDataBase;
struct FDynamicEmitterReplayDataBase;
struct FParticleEmitterInstance;
enum class EParticleSignificanceLevel : uint8;
struct FPSCTickData;
class FParticleSystemWorldManager;

//
// Forward declarations.
//
class UAnimNotifyState;
class FParticleDynamicData;
class FParticleSystemSceneProxy;
class UAnimNotifyState;
class UFXSystemAsset;
struct FDynamicEmitterDataBase;
struct FDynamicEmitterReplayDataBase;
struct FParticleAnimTrailEmitterInstance;

/** Enum for specifying type of a name instance parameter. */
UENUM(BlueprintType)
enum EParticleSysParamType : int
{
	PSPT_None UMETA(DisplayName="None"),
	PSPT_Scalar UMETA(DisplayName="Scalar"),
	PSPT_ScalarRand UMETA(DisplayName="Scalar Random"),
	PSPT_Vector UMETA(DisplayName="Vector"),
	PSPT_VectorRand UMETA(DisplayName="Vector Random"),
	PSPT_Color UMETA(DisplayName="Color"),
	PSPT_Actor UMETA(DisplayName="Actor"),
	PSPT_Material UMETA(DisplayName="Material"),
	PSPT_VectorUnitRand UMETA(DisplayName = "Vector Unit Random"),
	PSPT_MAX,
};

/** Particle system replay state */
UENUM()
enum ParticleReplayState : int
{
	/** Replay system is disabled.  Particles are simulated and rendered normally. */
	PRS_Disabled UMETA(DisplayName="Disabled"),
	/** Capture all particle data to the clip specified by ReplayClipIDNumber.  The frame to capture
		must be specified using the ReplayFrameIndex */
	PRS_Capturing UMETA(DisplayName="Capturing"),
	/** Replay captured particle state from the clip specified by ReplayClipIDNumber.  The frame to play
		must be specified using the ReplayFrameIndex */
	PRS_Replaying UMETA(DisplayName="Replaying"),
	PRS_MAX,
};

/**
 *	Event type
 */
UENUM()
enum EParticleEventType : int
{
	/** Any - allow any event */
	EPET_Any UMETA(DisplayName="Any"),
	/** Spawn - a particle spawn event */
	EPET_Spawn UMETA(DisplayName="Spawn"),
	/** Death - a particle death event */
	EPET_Death UMETA(DisplayName="Death"),
	/** Collision - a particle collision event */
	EPET_Collision UMETA(DisplayName="Collision"),
	/** Burst - a particle burst event */
	EPET_Burst UMETA(DisplayName="Burst"),
	/** Blueprint - an event generated by level script */
	EPET_Blueprint UMETA(DisplayName="Blueprint"),
	EPET_MAX,
};

// Called when the particle system is done
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FOnSystemFinished, class UParticleSystemComponent*, PSystem );

//Called just before the activation of a component changes.
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSystemPreActivationChange, class UParticleSystemComponent*, bool);

/** Struct used for a particular named instance parameter for this ParticleSystemComponent. */
USTRUCT(BlueprintType)
struct FParticleSysParam
{
	GENERATED_USTRUCT_BODY()

	/** The name of the parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	FName Name;

	/**
	 *	The type of parameters
	 *	PSPT_None       - There is no data type
	 *	PSPT_Scalar     - Use the scalar value
	 *	PSPT_ScalarRand - Select a scalar value in the range [Scalar_Low..Scalar)
	 *	PSPT_Vector     - Use the vector value
	 *	PSPT_VectorRand - Select a vector value in the range [Vector_Low..Vector)
	 *	PSPT_Color      - Use the color value
	 *	PSPT_Actor      - Use the actor value
	 *	PSPT_Material   - Use the material value
	 *	PSPT_VectorUnitRand - Select a random unit vector and scale along the range [Vector_Low..Vector)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	TEnumAsByte<enum EParticleSysParamType> ParamType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	float Scalar;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	float Scalar_Low;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	FVector Vector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	FVector Vector_Low;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	FColor Color;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	TObjectPtr<class AActor> Actor;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParticleSysParam)
	TObjectPtr<class UMaterialInterface> Material;

	FParticleSysParam()
		: ParamType(0)
		, Scalar(0)
		, Scalar_Low(0)
		, Vector(ForceInit)
		, Vector_Low(ForceInit)
		, Color(ForceInit)
		, Actor(NULL)
		, Material(NULL)
		, AsyncActorToWorld(FTransform::Identity)
		, AsyncActorVelocity(FVector::ZeroVector)
		, bAsyncDataCopyIsValid(false)
	{
	}

	void UpdateAsyncActorCache()
	{
		check(IsInGameThread());
		if(Actor)
		{
			AsyncActorToWorld = Actor->ActorToWorld();
			AsyncActorVelocity = Actor->GetVelocity();
		}
		
		bAsyncDataCopyIsValid = true;
	}

	void ResetAsyncActorCache()
	{
		check(IsInGameThread());
		bAsyncDataCopyIsValid = false;
	}

	FTransform GetAsyncActorToWorld() const
	{
		if(bAsyncDataCopyIsValid)
		{
			return AsyncActorToWorld;
		}
		else if(Actor)
		{
			check(IsInGameThread());
			return Actor->ActorToWorld();
		}

		return FTransform::Identity;
	}

	FVector GetAsyncActorVelocity() const
	{
		if (bAsyncDataCopyIsValid)
		{
			return AsyncActorVelocity;
		}
		else if (Actor)
		{
			check(IsInGameThread());
			return Actor->GetVelocity();
		}

		return FVector::ZeroVector;
	}

private:
	FTransform AsyncActorToWorld;
	FVector AsyncActorVelocity;
	bool bAsyncDataCopyIsValid;

};

template <> struct TIsPODType<FParticleSysParam> { enum { Value = true }; };

/**
 *	The base class for all particle event data.
 */
struct FParticleEventData
{
	/** The type of event that was generated. */
	int32 Type;

	/** The name of the event. */
	FName EventName;

	/** The emitter time at the event. */
	float EmitterTime;

	/** The location of the event. */
	FVector Location;

	/** The velocity at the time of the event. */
	FVector Velocity;

	/** Game specific event metadata */
	TArray<class UParticleModuleEventSendToGame*> EventData;

	FParticleEventData()
		: Type(0)
		, EmitterTime(0)
	{
	}
};

/**
 *	Particle event data for particles that already existed at the time of the event
 */
struct FParticleExistingData : FParticleEventData
{
	/** How long the particle had been alive at the time of the event. */
	float ParticleTime;

	/** The direction of the particle at the time of the event. */
	FVector Direction;

	FParticleExistingData()
		: ParticleTime(0)
		, Direction(ForceInit)
	{
	}
};

/**
 *	Spawn particle event data.
 */
struct FParticleEventSpawnData : public FParticleEventData
{
};

/**
 *	Killed particle event data.
 */
struct FParticleEventDeathData : public FParticleExistingData
{

};

/**
 *	Collision particle event data.
 */
struct FParticleEventCollideData : public FParticleExistingData
{
	/** Normal vector in coordinate system of the returner. Zero=none. */
	FVector Normal;

	/** Time until hit, if line check. */
	float Time;

	/** Primitive data item which was hit, INDEX_NONE=none. */
	int32 Item;

	/** Name of bone we hit (for skeletal meshes). */
	FName BoneName;

	/** The physical material for this collision. */
	UPhysicalMaterial* PhysMat;

	FParticleEventCollideData()
		: Normal(ForceInit)
		, Time(0)
		, Item(0)
	{
	}

};

/**
 *	Particle burst event data.
 */
struct FParticleEventBurstData : public FParticleEventData
{
	int32 ParticleCount;

	FParticleEventBurstData()
		: ParticleCount(0)
	{
	}
};

/**
 *	Kismet particle event data.
 */
struct FParticleEventKismetData : public FParticleEventData
{
};

/** Parameters controlling the spawning behavior of FX systems via the SpawnSystemAtLocation and SpawnSystemAttached. */
USTRUCT(BlueprintType)
struct FFXSystemSpawnParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	TObjectPtr<const UObject> WorldContextObject;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	TObjectPtr<UFXSystemAsset> SystemTemplate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	FVector Location = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	FRotator Rotation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	FVector Scale = FVector(1.f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	TObjectPtr<USceneComponent> AttachToComponent;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	FName AttachPointName = NAME_None;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	TEnumAsByte<EAttachLocation::Type> LocationType = EAttachLocation::KeepWorldPosition;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	bool bAutoDestroy = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects|Components|ParticleSystem")
	bool bAutoActivate = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	EPSCPoolMethod PoolingMethod = EPSCPoolMethod::None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	bool bPreCullCheck = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Effects|Components|ParticleSystem")
	bool bIsPlayerEffect = false;
};

UCLASS(Abstract, MinimalAPI)
class UFXSystemComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()
public:

	/**Change a named boolean parameter, ParticleSystemComponent converts to float.*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void SetBoolParameter(FName ParameterName, bool Param) {}

	/** Change a named int parameter */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	virtual void SetIntParameter(FName ParameterName, int Param) {}

	/** Change a named float parameter */
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void SetFloatParameter(FName ParameterName, float Param) {}

	/** 
	 *	Set a named vector instance parameter on this ParticleSystemComponent.
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	virtual void SetVectorParameter(FName ParameterName, FVector Param) {}

	/** 
	 *	Set a named color instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	virtual void SetColorParameter(FName ParameterName, FLinearColor Param) {}

	/** 
	 *	Set a named actor instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	virtual void SetActorParameter(FName ParameterName, class AActor* Param) {}

	/** 
	 * Get the referenced FXSystem asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual UFXSystemAsset* GetFXSystemAsset() const { return nullptr; };

	/**
	 *	Enables / disables an emitter by halting spawning of new particles.
	 *  You will still pay the cost of the emitter update.
	 *
	 *	@param	EmitterName			The name of the emitter
	 *	@param	bNewEnableState		The value to set it to
	 */
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void SetEmitterEnable(FName EmitterName, bool bNewEnableState) {};

	/**
	 * Set AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule to the specified parameters. Does not change bAutoManageAttachment; that must be set separately.
	 * @param  Parent			Component to attach to.
	 * @param  SocketName		Socket on Parent to attach to.
	 * @param  LocationRule		Option for how we handle our location when we attach to Parent.
	 * @param  RotationRule		Option for how we handle our rotation when we attach to Parent.
	 * @param  ScaleRule		Option for how we handle our scale when we attach to Parent.
	 * @see bAutoManageAttachment, AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule
	 */
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void SetAutoAttachmentParameters(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule) {}

	/**
	 * Sets whether we should automatically attach to AutoAttachParent when activated, and detach from our parent when completed.
	 * This overrides any current attachment that may be present at the time of activation (deferring initial attachment until activation, if AutoAttachParent is null).
	 * When enabled, detachment occurs regardless of whether AutoAttachParent is assigned, and the relative transform from the time of activation is restored.
	 * This also disables attachment on dedicated servers, where we don't actually activate even if bAutoActivate is true.
	 * @see SetAutoAttachmentParameters()
	 */
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void SetUseAutoManageAttachment(bool bAutoManage) {}

	/**
	 * Deactivates this system and releases it to the pool on completion.
	 * Usage of this PSC reference after this call is unsafe.
	 * You should clear out your references to it.
	 */
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	virtual void ReleaseToPool() {}

	/** Returns an approximate memory usage value for this component. */
	virtual uint32 GetApproxMemoryUsage() const { return 0; }

	virtual void ActivateSystem(bool bFlagAsJustAttached = false) { };

	/** Forces component to deactivate immediately. */
	virtual void DeactivateImmediate() {}

	/** Returns true if we have crossed LWC tiles to the point that we may introduce artifacts. */
	static ENGINE_API bool RequiresLWCTileRecache(const FVector3f CurrentTile, const FVector CurrentLocation);

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	mutable FParticlePerfStats* ParticlePerfStats = nullptr;
#endif

protected:
	ENGINE_API void PrecacheAssetPSOs(UFXSystemAsset* FXSystemAsset);

private:
	/** Helper flag to check if PSOs have been precached already */
	uint8 bPSOPrecacheCalled : 1 = 0;
};


/** 
 * A particle emitter.
 */
UCLASS(ClassGroup=(Rendering), hidecategories=Object, hidecategories=Physics, hidecategories=Collision, showcategories=Trigger, editinlinenew, meta=(BlueprintSpawnableComponent, DisplayName = "Cascade Particle System Component"), MinimalAPI)
class UParticleSystemComponent : public UFXSystemComponent
{
	friend class FParticleSystemWorldManager;

	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Particles)
	TObjectPtr<class UParticleSystem> Template;

	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class UMaterialInterface>> EmitterMaterials;

	/**
	 *	The skeletal mesh components used with the socket location module.
	 *	This is to prevent them from being garbage collected.
	 */
	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<class USkeletalMeshComponent>> SkelMeshComponents;

	uint8 bWasCompleted:1;

	uint8 bSuppressSpawning:1;

private:
	/** if true, this psys can tick in any thread **/
	uint8 bIsElligibleForAsyncTick:1;
	/** if true, bIsElligibleForAsyncTick is set up **/
	uint8 bIsElligibleForAsyncTickComputed:1;
public:

	uint8 bWasDeactivated:1;

	/** True if this was active before being unregistered or otherwise reset, if so reactivate it */
	uint8 bWasActive:1;

	/** If true, someone has requested this component reset. */
	uint8 bResetTriggered : 1;

	/** If true, someone has requested this component deactivate. */
	uint8 bDeactivateTriggered : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Particles)
	uint8 bResetOnDetach:1;

	/** whether to update the particle system on dedicated servers */
	UPROPERTY()
	uint8 bUpdateOnDedicatedServer:1;

	/** Indicates that the component has not been ticked since being registered. */
	uint8 bJustRegistered:1;

	/** This flag will be set the first time the PSysComp is activated... used to prevent auto activated PSysComps from calling InitParticles twice on level load */
	uint8 bHasBeenActivated:1;

	/**
	 * If true, this Particle System will be available for recycling after it has completed. Auto-destroyed systems cannot be recycled.
	 * Some systems (currently particle trail effects) can recycle components to avoid respawning them to play new effects.
	 * This is only an optimization and does not change particle system behavior, aside from not triggering normal component initialization events more than once.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Particles)
	uint8 bAllowRecycling:1;

	/**
	 * True if we should automatically attach to AutoAttachParent when activated, and detach from our parent when completed.
	 * This overrides any current attachment that may be present at the time of activation (deferring initial attachment until activation, if AutoAttachParent is null).
	 * When enabled, detachment occurs regardless of whether AutoAttachParent is assigned, and the relative transform from the time of activation is restored.
	 * This also disables attachment on dedicated servers, where we don't actually activate even if bAutoActivate is true.
	 * @see AutoAttachParent, AutoAttachSocketName, AutoAttachLocationType
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Attachment)
	uint8 bAutoManageAttachment:1;

	/**
	 * Option for how we handle bWeldSimulatedBodies when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	uint8 bAutoAttachWeldSimulatedBodies:1;
	
	/** If this component is having it's significance managed by gameplay code. */
	uint8 bIsManagingSignificance : 1;
	/** If this component was previously having it's significance managed by gameplay code. Allows us to refresh render data when this changes. */
	uint8 bWasManagingSignificance : 1;

	uint8 bIsDuringRegister : 1;

	UPROPERTY()
	uint8 bWarmingUp:1;

	/** indicates that the component's LODMethod overrides the Template's */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LOD)
	uint8 bOverrideLODMethod:1;

	/**
	 *	Flag indicating that dynamic updating of render data should NOT occur during Tick.
	 *	This is used primarily to allow for warming up and simulated effects to a certain state.
	 */
	UPROPERTY()
	uint8 bSkipUpdateDynamicDataDuringTick:1;

	/** This is set when any of our "don't tick me" timeout values have fired */
	uint8 bForcedInActive:1;

	/** If true, force an LOD update from the renderer. */
	uint8 bForceLODUpdateFromRenderer:1;

	/** If true, the ViewRelevanceFlags are dirty and should be recached */
	uint8 bIsViewRelevanceDirty : 1;

	uint8 bAutoDestroy : 1;

private:
	
	/** Indicates if the component's transform has been changed and requires updating */
	uint8 bIsTransformDirty:1;

	/** Did we auto attach during activation? Used to determine if we should restore the relative transform during detachment. */
	uint8 bDidAutoAttach : 1;

	/** Is AsyncComponentToWorld etc valid? */
	uint8 bAsyncDataCopyIsValid:1;
	uint8 bParallelRenderThreadUpdate:1;

	/** If true, it means the ASync work is done and the finalize is not */
	uint8 bNeedsFinalize:1;
	
	/** If true, it means the Async work is in process and not yet completed. */
	volatile bool bAsyncWorkOutstanding;
	
	/** Restore relative transform from auto attachment and optionally detach from parent (regardless of whether it was an auto attachment). */
	ENGINE_API void CancelAutoAttachment(bool bDetachFromParent, const UWorld* MyWorld);

	/** Handle into the FParticleSystemWorldManager. INDEX_NONE if this component does not have managed ticks. */
	int32 ManagerHandle : 30;
	/** Flag denoting we are currently in the pending queue to be added to the manager. */
	int32 bPendingManagerAdd : 1;
	/** Flag denoting we have been unregistered and are awaiting removal from the managers arrays. */
	int32 bPendingManagerRemove : 1;

public:

#if WITH_EDITOR
	virtual bool Editor_CanBeTickManaged()const { return true; }
#endif
	ENGINE_API bool ShouldBeTickManaged()const;

	inline bool IsTickManaged()const { return ManagerHandle != INDEX_NONE && !IsPendingManagerRemove(); }
	inline int32 GetManagerHandle()const { return ManagerHandle; }
	inline void SetManagerHandle(int32 InHandle) { ManagerHandle = InHandle; }

	inline int32 IsPendingManagerAdd()const { return bPendingManagerAdd; }
	inline void SetPendingManagerAdd(bool bValue) { bPendingManagerAdd = bValue; }
	inline int32 IsPendingManagerRemove()const { return bPendingManagerRemove; }
	inline void SetPendingManagerRemove(bool bValue) { bPendingManagerRemove = bValue; }

	ENGINE_API FPSCTickData& GetManagerTickData();
	ENGINE_API FParticleSystemWorldManager* GetWorldManager()const;


	/** If this PSC is pooling. */
	EPSCPoolMethod PoolingMethod;

	/** The method of LOD level determination to utilize for this particle system */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LOD)
	TEnumAsByte<enum ParticleSystemLODMethod> LODMethod;

	/** The significance this component requires of it's emitters for them to be enabled. */
	UPROPERTY()
	EParticleSignificanceLevel RequiredSignificance;

	/** Current particle 'replay state'.  This setting controls whether we're currently simulating/rendering particles normally, or whether we should capture or playback particle replay data instead. */
	TEnumAsByte<enum ParticleReplayState> ReplayState;

	void ResetNextTick()
	{
		bResetTriggered = true;
	}

	void DeactivaateNextTick()
	{
		bDeactivateTriggered = true;
	}

	/**
	 *	Array holding name instance parameters for this ParticleSystemComponent.
	 *	Parameters can be used in Cascade using DistributionFloat/VectorParticleParameters.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Particles)
	TArray<struct FParticleSysParam> InstanceParameters;

	UPROPERTY(BlueprintAssignable)
	FParticleSpawnSignature OnParticleSpawn;

	UPROPERTY(BlueprintAssignable)
	FParticleBurstSignature OnParticleBurst;

	UPROPERTY(BlueprintAssignable)
	FParticleDeathSignature OnParticleDeath;

	UPROPERTY(BlueprintAssignable)
	FParticleCollisionSignature OnParticleCollide;

	UPROPERTY(Transient, DuplicateTransient)
	bool bOldPositionValid;

	UPROPERTY(Transient, DuplicateTransient)
	FVector OldPosition;

	UPROPERTY(Transient, DuplicateTransient)
	FVector PartSysVelocity;

	UPROPERTY(Transient, DuplicateTransient)
	float WarmupTime;

	UPROPERTY(Transient, DuplicateTransient)
	float WarmupTickRate;

private:
	int32 LODLevel;

public:

	/**
	 * Number of seconds of emitter not being rendered that need to pass before it
	 * no longer gets ticked/ becomes inactive.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Particles)
	float SecondsBeforeInactive;

private:
	/** Tracks the time since the last forced UpdateTransform. */
	float TimeSinceLastForceUpdateTransform;

public:
	/**
	 * Time between forced UpdateTransforms for systems that use dynamically calculated bounds,
	 * Which is effectively how often the bounds are shrunk.
	 */
	UPROPERTY()
	float MaxTimeBeforeForceUpdateTransform;

#if WITH_EDITORONLY_DATA
	/**
	 *	INTERNAL. Used by the editor to set the LODLevel
	 */
	UPROPERTY()
	int32 EditorLODLevel;

	/**
	 * Used for applying Cascade's detail mode setting to in-level particle systems
	 */
	UPROPERTY()
	int32 EditorDetailMode;

#endif // WITH_EDITORONLY_DATA
	/** Used to accumulate total tick time to determine whether system can be skipped ticking if not visible. */
	float AccumTickTime;

	/** Time in seconds since we were last considered significant. */
	float LastSignificantTime;

	/** LOD updating... */
	float AccumLODDistanceCheckTime;

	/** The view relevance flags for each LODLevel. */
	TArray<FMaterialRelevance> CachedViewRelevanceFlags;

	/** Array of replay clips for this particle system component.  These are serialized to disk.  You really should never add anything to this in the editor.  It's exposed so that you can delete clips if you need to, but be careful when doing so! */
	UPROPERTY()
	TArray<TObjectPtr<class UParticleSystemReplay>> ReplayClips;

	/** Clip ID number we're either playing back or capturing to, depending on the value of ReplayState. */
	int32 ReplayClipIDNumber;

	/** The current replay frame for playback */
	int32 ReplayFrameIndex;

	/** Scales DeltaTime in UParticleSystemComponent::Tick(...) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Particles)
	float CustomTimeDilation;

	/** This is created at start up and then added to each emitter */
	float EmitterDelay;

	/** The Spawn events that occurred in this PSysComp. */
	TArray<struct FParticleEventSpawnData> SpawnEvents;

	/** The Death events that occurred in this PSysComp. */
	TArray<struct FParticleEventDeathData> DeathEvents;

	/** The Collision events that occurred in this PSysComp. */
	TArray<struct FParticleEventCollideData> CollisionEvents;

	/** The Burst events that occurred in this PSysComp. */
	TArray<struct FParticleEventBurstData> BurstEvents;

	/** The Kismet events that occurred for this PSysComp. */
	TArray<struct FParticleEventKismetData> KismetEvents;

public:
	/**
	 * Component we automatically attach to when activated, if bAutoManageAttachment is true.
	 * If null during registration, we assign the existing AttachParent and defer attachment until we activate.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	TWeakObjectPtr<USceneComponent> AutoAttachParent;

	/**
	 * Socket we automatically attach to on the AutoAttachParent, if bAutoManageAttachment is true.
	 * If no auto attach socket name is set during registration, the current attach socket will be
	 * assigned to AutoAttachSocketName and used when activated.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	FName AutoAttachSocketName;

#if WITH_EDITORONLY_DATA
	/**
	 * DEPRECATED: Options for how we handle our location when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachLocation::Type
	 */
	UPROPERTY()
	TEnumAsByte<EAttachLocation::Type> AutoAttachLocationType_DEPRECATED;
#endif

	/**
	 * Options for how we handle our location when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachLocationRule;

	/**
	 * Options for how we handle our rotation when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachRotationRule;

	/**
	 * Options for how we handle our scale when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachScaleRule;

	/**
	 * DEPRECATED: Set AutoAttachParent, AutoAttachSocketName, AutoAttachLocationType to the specified parameters. Does not change bAutoManageAttachment; that must be set separately.
	 * @param  Parent			Component to attach to. 
	 * @param  SocketName		Socket on Parent to attach to.
	 * @param  LocationType		Option for how we handle our location when we attach to Parent.
	 * @see bAutoManageAttachment, AutoAttachParent, AutoAttachSocketName, AutoAttachLocationType
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem", meta=(DeprecatedFunction, DeprecationMessage="Please use Set Auto Attachment Parameters"))
	void SetAutoAttachParams(USceneComponent* Parent, FName SocketName = NAME_None, EAttachLocation::Type LocationType = EAttachLocation::KeepRelativeOffset);

	/**
	 * Set AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule to the specified parameters. Does not change bAutoManageAttachment; that must be set separately.
	 * @param  Parent			Component to attach to. 
	 * @param  SocketName		Socket on Parent to attach to.
	 * @param  LocationRule		Option for how we handle our location when we attach to Parent.
	 * @param  RotationRule		Option for how we handle our rotation when we attach to Parent.
	 * @param  ScaleRule		Option for how we handle our scale when we attach to Parent.
	 * @see bAutoManageAttachment, AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule
	 */
	ENGINE_API void SetAutoAttachmentParameters(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule) override;

	virtual void SetUseAutoManageAttachment(bool bAutoManage) override { bAutoManageAttachment = bAutoManage; }

private:

	/** Saved relative transform before auto attachement. Used during detachment to restore the transform if we had automatically attached. */
	FVector SavedAutoAttachRelativeLocation;
	FRotator SavedAutoAttachRelativeRotation;
	FVector SavedAutoAttachRelativeScale3D;

public:
	class FFXSystem* FXSystem;

	// Called when the particle system is done
	UPROPERTY(BlueprintAssignable)
	FOnSystemFinished OnSystemFinished;

private:
	/** Cached copy of the instance params */
	TArray<struct FParticleSysParam> AsyncInstanceParameters;
	/** Player locations computed before kicking off async task. Safe to access from async task or game thread*/
	TArray<FVector> PlayerLocations;
	/** PlayerLODDistanceFactor computed before kicking off async task. Safe to access from async task or game thread*/
	TArray<float> PlayerLODDistanceFactor;
	/** Cached copy of bounds */
	FBoxSphereBounds AsyncBounds;
	/** Cached copy of PartSysVelocity */
	FVector AsyncPartSysVelocity;
	/** LWC tile offset for gpu simulations */
	FVector3f LWCTile = FVector3f::ZeroVector;
public:

	/** Called from game code when the significance required for a component changes. */
	ENGINE_API void SetRequiredSignificance(EParticleSignificanceLevel NewRequiredSignificance);
	/** Whether this component should have it's significance managed by game code. */
	ENGINE_API bool ShouldManageSignificance()const;
	/** When the overall significance for the component is changed. */
	ENGINE_API void OnSignificanceChanged(bool bSignificant, bool bApplyToEmitters, bool bAsync=false);

	/** Called from game code when the component begins having it's significance managed. */
	inline void SetManagingSignificance(bool bManageSignificance)
	{
		bWasManagingSignificance = bIsManagingSignificance;
		bIsManagingSignificance = bManageSignificance;
	}

	inline const FVector3f& GetLWCTile() const { return LWCTile;  }
	
	/** Returns the approximate distance squared from this component to the passed location. */
	ENGINE_API float GetApproxDistanceSquared(FVector Point)const;
	
	/** True if this component can be considered invisible and potentially culled. */
	ENGINE_API bool CanConsiderInvisible()const;

	ENGINE_API bool CanSkipTickDueToVisibility();

	/** true if this component can be occluded. */
	ENGINE_API bool CanBeOccluded()const;

	ENGINE_API void Complete();
	
	inline const FTransform& GetAsyncComponentToWorld()
	{
		if (!bParallelRenderThreadUpdate && !IsInGameThread())
		{
			check(bAsyncDataCopyIsValid); 
			return AsyncComponentToWorld;
		}
		return GetComponentTransform();
	}

	inline const TArray<struct FParticleSysParam>& GetAsyncInstanceParameters()
	{
		if (!bParallelRenderThreadUpdate && !IsInGameThread())
		{
			check(bAsyncDataCopyIsValid); 
			return AsyncInstanceParameters;
		}
		return InstanceParameters;
	}

	inline const FBoxSphereBounds& GetAsyncBounds()
	{
		if (!bParallelRenderThreadUpdate && !IsInGameThread())
		{
			check(bAsyncDataCopyIsValid);
			return AsyncBounds;
		}
		return Bounds;
	}

	inline const FVector& GetAsyncPartSysVelocity()
	{
		if (!bParallelRenderThreadUpdate && !IsInGameThread())
		{
			check(bAsyncDataCopyIsValid);
			return AsyncPartSysVelocity;
		}
		return PartSysVelocity;
	}

	inline const TArray<FVector>& GetPlayerLocations()
	{
		return PlayerLocations;
	}

	inline const TArray<float>& GetPlayerLODDistanceFactor()
	{
		return PlayerLODDistanceFactor;
	}

	//
	//	Beam-related script functions.
	//
	
	/**
	 *	Set the beam end point
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewEndPoint			The value to set it to
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamEndPoint(int32 EmitterIndex, FVector NewEndPoint);

	/**
	 *	Set the beam source point
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewSourcePoint		The value to set it to
	 *	@param	SourceIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamSourcePoint(int32 EmitterIndex, FVector NewSourcePoint, int32 SourceIndex);

	/**
	 *	Set the beam source tangent
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewTangentPoint		The value to set it to
	 *	@param	SourceIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamSourceTangent(int32 EmitterIndex, FVector NewTangentPoint, int32 SourceIndex);

	/**
	 *	Set the beam source strength
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewSourceStrength	The value to set it to
	 *	@param	SourceIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamSourceStrength(int32 EmitterIndex, float NewSourceStrength, int32 SourceIndex);

	/**
	 *	Set the beam target point
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewTargetPoint		The value to set it to
	 *	@param	TargetIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamTargetPoint(int32 EmitterIndex, FVector NewTargetPoint, int32 TargetIndex);

	/**
	 *	Set the beam target tangent
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewTangentPoint		The value to set it to
	 *	@param	TargetIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamTargetTangent(int32 EmitterIndex, FVector NewTangentPoint, int32 TargetIndex);

	/**
	 *	Set the beam target strength
	 *
	 *	@param	EmitterIndex		The index of the emitter to set it on
	 *	@param	NewTargetStrength	The value to set it to
	 *	@param	TargetIndex			Which beam within the emitter to set it on
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API virtual void SetBeamTargetStrength(int32 EmitterIndex, float NewTargetStrength, int32 TargetIndex);

	/**
	*	Get the beam end point
	*
	*	@param	EmitterIndex		The index of the emitter to get the value of
	*
	*	@return	true		EmitterIndex is valid and End point is set - OutEndPoint is valid
	*			false		EmitterIndex invalid or End point is not set - OutEndPoint is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamEndPoint(int32 EmitterIndex, FVector& OutEndPoint) const;
	
	/**
	*	Get the beam source point
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	SourceIndex			Which beam within the emitter to get
	*	@param	OutSourcePoint		Value of source point
	*
	*	@return	true		EmitterIndex and SourceIndex are valid - OutSourcePoint is valid
	*			false		EmitterIndex or SourceIndex is invalid - OutSourcePoint is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamSourcePoint(int32 EmitterIndex, int32 SourceIndex, FVector& OutSourcePoint) const;

	/**
	*	Get the beam source tangent
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	SourceIndex			Which beam within the emitter to get
	*	@param	OutTangentPoint		Value of source tangent
	*
	*	@return	true		EmitterIndex and SourceIndex are valid - OutTangentPoint is valid
	*			false		EmitterIndex or SourceIndex is invalid - OutTangentPoint is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamSourceTangent(int32 EmitterIndex, int32 SourceIndex, FVector& OutTangentPoint) const;

	/**
	*	Get the beam source strength
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	SourceIndex			Which beam within the emitter to get
	*	@param	OutSourceStrength		Value of source tangent
	*
	*	@return	true		EmitterIndex and SourceIndex are valid - OutSourceStrength is valid
	*			false		EmitterIndex or SourceIndex is invalid - OutSourceStrength is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamSourceStrength(int32 EmitterIndex, int32 SourceIndex, float& OutSourceStrength) const;

	/**
	*	Get the beam target point
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	TargetIndex			Which beam within the emitter to get
	*	@param	OutTargetPoint		Value of target point
	*
	*	@return	true		EmitterIndex and TargetIndex are valid - OutTargetPoint is valid
	*			false		EmitterIndex or TargetIndex is invalid - OutTargetPoint is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamTargetPoint(int32 EmitterIndex, int32 TargetIndex, FVector& OutTargetPoint) const;

	/**
	*	Get the beam target tangent
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	TargetIndex			Which beam within the emitter to get
	*	@param	OutTangentPoint		Value of target tangent
	*
	*	@return	true		EmitterIndex and TargetIndex are valid - OutTangentPoint is valid
	*			false		EmitterIndex or TargetIndex is invalid - OutTangentPoint is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamTargetTangent(int32 EmitterIndex, int32 TargetIndex, FVector& OutTangentPoint) const;
	
	/**
	*	Get the beam target strength
	*
	*	@param	EmitterIndex		The index of the emitter to get
	*	@param	TargetIndex			Which beam within the emitter to get
	*	@param	OutTargetStrength	Value of target tangent
	*
	*	@return	true		EmitterIndex and TargetIndex are valid - OutTargetStrength is valid
	*			false		EmitterIndex or TargetIndex is invalid - OutTargetStrength is invalid
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Components|ParticleSystem")
	ENGINE_API virtual bool GetBeamTargetStrength(int32 EmitterIndex, int32 TargetIndex, float& OutTargetStrength) const;
	
	/**
	 *	Enables/Disables a sub-emitter
	 *
	 *	@param	EmitterName			The name of the sub-emitter to set it on
	 *	@param	bNewEnableState		The value to set it to
	 */
	ENGINE_API void SetEmitterEnable(FName EmitterName, bool bNewEnableState) override;


	/**
	 *	Set a named float instance parameter on this ParticleSystemComponent.
	 *	This is for function parity with the VFX Marshaller. The bool is converted
	 *  and then set to a float.
	 */
	ENGINE_API void SetBoolParameter(FName ParameterName, bool Param) override;

	/**
	 *	Set a named float instance parameter on this ParticleSystemComponent.
	 *	This is for function parity with the VFX Marshaller. The int is converted
	 *  and then set to a float.
	 */
	ENGINE_API void SetIntParameter(FName ParameterName, int Param) override;

	/** Change a named float parameter */
	ENGINE_API void SetFloatParameter(FName ParameterName, float Param) override;

	/** 
	 *	Set a named vector instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	ENGINE_API void SetVectorParameter(FName ParameterName, FVector Param) override;

	/** 
	 *	Set a named color instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	ENGINE_API void SetColorParameter(FName ParameterName, FLinearColor Param) override;

	/** 
	 *	Set a named actor instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	ENGINE_API void SetActorParameter(FName ParameterName, class AActor* Param) override;

	/**
	 * Get the referenced FXSystem asset.
	*/
	virtual UFXSystemAsset* GetFXSystemAsset() const { return Template; };

	/** 
	 *	Set a named material instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API void SetMaterialParameter(FName ParameterName, class UMaterialInterface* Param);

	/**
	 *	Retrieve the Float parameter value for the given name.
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutFloat	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutFloat is valid
	 *			false		Parameter was not found - OutFloat is invalid
	 */
	ENGINE_API virtual bool GetFloatParameter(const FName InName, float& OutFloat);

	/**
	 *	Retrieve the Vector parameter value for the given name.
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutVector	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutVector is valid
	 *			false		Parameter was not found - OutVector is invalid
	 */
	ENGINE_API virtual bool GetVectorParameter(const FName InName, FVector& OutVector);

	/**
	 *	Retrieve the Vector parameter value for the given name...also looks for colors and floats and returns those
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutVector	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutVector is valid
	 *			false		Parameter was not found - OutVector is invalid
	 */
	ENGINE_API virtual bool GetAnyVectorParameter(const FName InName, FVector& OutVector);

	/**
	 *	Retrieve the Color parameter value for the given name.
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutColor	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutColor is valid
	 *			false		Parameter was not found - OutColor is invalid
	 */
	ENGINE_API virtual bool GetColorParameter(const FName InName, FLinearColor& OutColor);

	/**
	 *	Retrieve the Actor parameter value for the given name.
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutActor	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutActor is valid
	 *			false		Parameter was not found - OutActor is invalid
	 */
	ENGINE_API virtual bool GetActorParameter(const FName InName, class AActor*& OutActor);

	/**
	 *	Retrieve the Material parameter value for the given name.
	 *
	 *	@param	InName		Name of the parameter
	 *	@param	OutMaterial	The value of the parameter found
	 *
	 *	@return	true		Parameter was found - OutMaterial is valid
	 *			false		Parameter was not found - OutMaterial is invalid
	 */
	ENGINE_API virtual bool GetMaterialParameter(const FName InName, class UMaterialInterface*& OutMaterial);

	/** clears the specified parameter, returning it to the default value set in the template
	 * @param ParameterName name of parameter to remove
	 * @param ParameterType type of parameter to remove; if omitted or PSPT_None is specified, all parameters with the given name are removed
	 */
	ENGINE_API void ClearParameter(FName ParameterName, enum EParticleSysParamType ParameterType = EParticleSysParamType(0));

	/** Change the ParticleSystem used by this ParticleSystemComponent */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API void SetTemplate(class UParticleSystem* NewTemplate);

	/** Get the current number of active particles in this system */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API int32 GetNumActiveParticles() const;


	/** Array of trail emitters. */
	typedef TArray< struct FParticleAnimTrailEmitterInstance*, TInlineAllocator<8> > TrailEmitterArray;

	/**
	* Fills the passed array with all trail emitters associated with a particular object.
	* @param OutTrailEmitters	The array to fill with pointers to the trail emitters.
	* @param InOwner			The object that triggered this trail. Can be NULL if no assosiation was set by the owner. Not to be confused with the result of GetOwner().
	* @param bSetOwner			If true, all trail emitters will be set as owned by InOwner.
	*/
	ENGINE_API virtual void GetOwnedTrailEmitters(TrailEmitterArray& OutTrailEmitters, const void* InOwner, bool bSetOwner = false);
	
	/**
	* Begins all trail emitters in this component.
	*
	* @param	InFirstSocketName	The name of the first socket for the trail.
	* @param	InSecondSocketName	The name of the second socket for the trail.
	* @param	InWidthMode			How the width value is applied to the trail.
	* @param	InWidth				The width of the trail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Particles|Trails")
	ENGINE_API void BeginTrails(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth);

	/**
	* Ends all trail emitters in this component.
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Particles|Trails")
	ENGINE_API void EndTrails();

	ENGINE_API void ReleaseToPool() override;

	/**
	* Sets the defining data for all trails in this component.
	*
	* @param	InFirstSocketName	The name of the first socket for the trail.
	* @param	InSecondSocketName	The name of the second socket for the trail.
	* @param	InWidthMode			How the width value is applied to the trail.
	* @param	InWidth				The width of the trail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Effects|Particles|Trails")
	ENGINE_API void SetTrailSourceData(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth);

public:
	/** Command fence used to shut down properly */
	class FRenderCommandFence* ReleaseResourcesFence;

	TArray<struct FParticleEmitterInstance*> EmitterInstances;

	/** Static delegate called for all systems on an activation change. */
	static ENGINE_API FOnSystemPreActivationChange OnSystemPreActivationChange;

	/** Stream of random values to use with this component */
	FRandomStream RandomStream;

	/** Owner for particle emitter instances. Reroutes calls to this component. (This is used to abstract away dependency on UParticleSystemComponent) */
	class FInstanceOwner;
	TUniquePtr<FInstanceOwner> InstanceOwner;

private:
	/** In some cases the async work for this PSC can be created externally by the manager. */
	inline void SetAsyncWork(FGraphEventRef& InAsyncWork) { AsyncWork = InAsyncWork; }

	/** Cached copy of the transform for async work */
	FTransform AsyncComponentToWorld;

	/** Task ref for parallel portion */
	FGraphEventRef AsyncWork;

	/** Copy of delta time coming into TickComponent */
	float DeltaTimeTick;
	/** Info from async tick */
	int32 TotalActiveParticles;
	/** Number of significant emitters. When this is 0, the system can either be deactivated or stopped ticking. */
	uint32 NumSignificantEmitters;
	/** Time in ms since a tick was last performed; used with MinTimeBetweenTicks (on UParticleSystem) to control tick rate */
	uint32 TimeSinceLastTick;

#if WITH_STATE_STREAM_ACTOR
	FParticleSystemHandle ParticleSystemHandle;
#endif

public:

	//~ Begin UActorComponent Interface.
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
#endif
	ENGINE_API virtual bool IsReadyForOwnerToAutoDestroy() const override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	virtual UObject const* AdditionalStatObject() const override
	{
		return Template;
	}
	ENGINE_API virtual void SetComponentTickEnabled(bool bEnabled)override;
	ENGINE_API virtual bool IsComponentTickEnabled() const override;

	ENGINE_API virtual void OnAttachmentChanged()override;
	ENGINE_API virtual void OnChildAttached(USceneComponent* ChildComponent)override;
	ENGINE_API virtual void OnChildDetached(USceneComponent* ChildComponent)override;

	ENGINE_API virtual void OnEndOfFrameUpdateDuringTick() override;
protected:
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void SendRenderTransform_Concurrent() override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister()  override;
	ENGINE_API virtual void SendRenderDynamicData_Concurrent() override;
public:
#if WITH_EDITORONLY_DATA
	virtual bool RequiresGameThreadEndOfFrameUpdates() const override { return true; }
	virtual bool RequiresGameThreadEndOfFrameRecreate() const override { return true; }
#endif
	//~ End UActorComponent Interface.

	enum EForceAsyncWorkCompletion
	{
		STALL,
		ENSURE_AND_STALL,
		SILENT, // this would only be appropriate for editor only or other unusual things that we never see in game
	};
	/** If there is async work outstanding, force it to be completed now **/
	inline void ForceAsyncWorkCompletion(EForceAsyncWorkCompletion Behavior, bool bDefinitelyGameThread, bool InSkipUpdateDynamicDataDuringTick)
	{
		if (AsyncWork.GetReference())
		{
			const bool bSavedSkipUpdate = bSkipUpdateDynamicDataDuringTick;
			bSkipUpdateDynamicDataDuringTick |= InSkipUpdateDynamicDataDuringTick;
			WaitForAsyncAndFinalize(Behavior, bDefinitelyGameThread);
			bSkipUpdateDynamicDataDuringTick = bSavedSkipUpdate;
		}
	}

	/** If there is async work outstanding, force it to be completed now **/
	inline void ForceAsyncWorkCompletion(EForceAsyncWorkCompletion Behavior, bool bDefinitelyGameThread = true) const
	{
		if (AsyncWork.GetReference())
		{
			WaitForAsyncAndFinalize(Behavior, bDefinitelyGameThread);
		}
	}

	/** 
	  * Decide which detail mode should be applied to this particle system. If we have an editor
	  * override specified, use that. Otherwise use the global cached value 
	  */
	ENGINE_API int32 GetCurrentDetailMode() const;

	/** Possibly parallel phase of TickComponent **/
	ENGINE_API void ComputeTickComponent_Concurrent();

	/** After the possibly parallel phase of TickComponent, we fire events, etc **/
	ENGINE_API void FinalizeTickComponent();

	ENGINE_API void ForceReset();

	ENGINE_API void MarshalParamsForAsyncTick();
private:
	/** Wait on the async task and call finalize on the tick **/
	ENGINE_API void WaitForAsyncAndFinalize(EForceAsyncWorkCompletion Behavior, bool bDefinitelyGameThread = true) const;

	/** Cache view relevance flags. */
	ENGINE_API void CacheViewRelevanceFlags(class UParticleSystem* TemplateToCache);

public:

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual FString GetDetailedInfoInternal() const override;
	//~ End UObject Interface.

	//Begin UPrimitiveComponent Interface
	ENGINE_API virtual int32 GetNumMaterials() const override;
	ENGINE_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	ENGINE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	ENGINE_API virtual void GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false ) const override;
	ENGINE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	ENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	//End UPrimitiveComponent Interface

	//~ Begin USceneComonent Interface
protected:
	ENGINE_API virtual bool ShouldActivate() const override;

public:
#if WITH_EDITOR
	ENGINE_API virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Activate(bool bReset=false) override;
	ENGINE_API virtual void Deactivate() override;
	ENGINE_API virtual void DeactivateImmediate() override;
	ENGINE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End USceneComponent Interface

	/** Activate the system */
	ENGINE_API virtual void ActivateSystem(bool bFlagAsJustAttached = false) override;
	/** Deactivate the system */
	ENGINE_API void DeactivateSystem();
	// Collision Handling...
	ENGINE_API virtual bool ParticleLineCheck(FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& HalfExtent, const FCollisionObjectQueryParams& ObjectParams);

	/** return true if this psys can tick in any thread */
	inline bool CanTickInAnyThread()
	{
		if (!bIsElligibleForAsyncTickComputed)
		{
			ComputeCanTickInAnyThread();
		}
		return bIsElligibleForAsyncTick;
	}
	/** Decide if this psys can tick in any thread, and set bIsElligibleForAsyncTick */
	ENGINE_API void ComputeCanTickInAnyThread();

	/**
	* Creates a Dynamic Material Instance for the specified named material override, optionally from the supplied material.
	* @param Name - The slot name of the material to replace.  If invalid, the material is unchanged and NULL is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual class UMaterialInstanceDynamic* CreateNamedDynamicMaterialInstance(FName InName, class UMaterialInterface* SourceMaterial = NULL);

	ENGINE_API virtual UMaterialInterface* GetMaterialByName(FName MaterialSlotName) const override;
	ENGINE_API virtual void SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* SourceMaterial) override;

	/** Returns a named material. If this named material is not found, returns NULL. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual class UMaterialInterface* GetNamedMaterial(FName InName) const;

	/** Returns the index into the EmitterMaterials array for this named. If there are no named material slots or this material is not found, INDEX_NONE is returned. */
	ENGINE_API virtual int32 GetNamedMaterialIndex(FName InName) const;

	/** returns the name associated with the passed in material, returns NAME_None if it is not found */
	ENGINE_API virtual FName GetNameForMaterial(UMaterialInterface* InMaterial) const;

	/** Returns an approximate memory usage value for this component. */
	ENGINE_API uint32 GetApproxMemoryUsage() const override;

protected:

	// @todo document
	ENGINE_API virtual void UpdateLODInformation();

	/**
	 * Static: Supplied with a chunk of replay data, this method will create dynamic emitter data that can
	 * be used to render the particle system
	 *
	 * @param	EmitterInstance		Emitter instance this replay is playing on
	 * @param	EmitterReplayData	Incoming replay data of any time, cannot be NULL
	 * @param	bSelected			true if the particle system is currently selected
	 * @param	InFeatureLevel		The relevant shader feature level
	 *
	 * @return	The newly created dynamic data, or NULL on failure
	 */
	static ENGINE_API FDynamicEmitterDataBase* CreateDynamicDataFromReplay( FParticleEmitterInstance* EmitterInstance, const FDynamicEmitterReplayDataBase* EmitterReplayData, bool bSelected, ERHIFeatureLevel::Type InFeatureLevel );

	/**
	 * Creates dynamic particle data for rendering the particle system this frame.  This function
	 * handle creation of dynamic data for regularly simulated particles, but also handles capture
	 * and playback of particle replay data.
     * @param InFeatureLevel - The relevant shader feature level.
	 * @return	Returns the dynamic data to render this frame
	 */
	ENGINE_API FParticleDynamicData* CreateDynamicData(ERHIFeatureLevel::Type InFeatureLevel);

	/** Orients the Z axis of the ParticleSystemComponent toward the camera while preserving the X axis direction */
	ENGINE_API void OrientZAxisTowardCamera();

	/** Clears dynamic data on the rendering thread. */
	ENGINE_API void ClearDynamicData();

	// @todo document
	ENGINE_API virtual void UpdateDynamicData();

public:
	inline int32 GetCurrentLODIndex() const
	{
		return LODLevel;
	}

	// If particles have not already been initialised (ie. EmitterInstances created) do it now.
	ENGINE_API virtual void InitParticles();


	// @todo document
	ENGINE_API void ResetParticles(bool bEmptyInstances = false);


	// @todo document
	ENGINE_API void ResetBurstLists();


	// @todo document
	ENGINE_API void UpdateInstances(bool bEmptyInstances = false);


	// @todo document
	ENGINE_API bool HasCompleted();


	// @todo document
	ENGINE_API void InitializeSystem();

	/**
	 *	Cache the view-relevance for each emitter at each LOD level if needed.
	 *
	 *	@param	NewTemplate		The UParticleSystem* to use as the template.
	 *							If NULL, use the currently set template.
	 */
	ENGINE_API void ConditionalCacheViewRelevanceFlags(class UParticleSystem* NewTemplate = NULL);

	/** Auto-populate the instance parameters based on contained modules. */
	ENGINE_API void	AutoPopulateInstanceProperties();

	/** Event reporting... */
	/**
	 *	Record a spawning event.
	 *
	 *	@param	InEventName			The name of the event that fired.
	 *	@param	InEmitterTime		The emitter time when the event fired.
	 *	@param	InLocation			The location of the particle when the event fired.
	 *	@param	InVelocity			The velocity of the particle when the event fired.
	 *  @param  InEventData         Gamespecific event data payload
	 */
	ENGINE_API void ReportEventSpawn(const FName InEventName, const float InEmitterTime,
		const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData);

	/**
	 *	Record a death event.
	 *
	 *	@param	InEventName			The name of the event that fired.
	 *	@param	InEmitterTime		The emitter time when the event fired.
	 *	@param	InLocation			The location of the particle when the event fired.
	 *	@param	InVelocity			The velocity of the particle when the event fired.
	 *  @param  InEventData         Gamespecific event data payload
	 *	@param	InParticleTime		The relative life of the particle when the event fired.
	 */
	ENGINE_API void ReportEventDeath(const FName InEventName, const float InEmitterTime,
		const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime);

	/**
	 *	Record a collision event.
	 *
	 *	@param	InEventName		The name of the event that fired.
	 *	@param	InEmitterTime	The emitter time when the event fired.
	 *	@param	InLocation		The location of the particle when the event fired.
	 *	@param	InDirection		The direction of the particle when the event fired.
	 *	@param	InVelocity		The velocity of the particle when the event fired.
	 *  @param  InEventData         Gamespecific event data payload
	 *	@param	InParticleTime	The relative life of the particle when the event fired.
	 *	@param	InNormal		Normal vector of the collision in coordinate system of the returner. Zero=none.
	 *	@param	InTime			Time until hit, if line check.
	 *	@param	InItem			Primitive data item which was hit, INDEX_NONE=none.
	 *	@param	InBoneName		Name of bone we hit (for skeletal meshes).
	 */
	ENGINE_API void ReportEventCollision(const FName InEventName, const float InEmitterTime, const FVector InLocation,
		const FVector InDirection, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, 
		const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat);

	/**
	 *	Record a bursting event.
	 *
	 *	@param	InEventName			The name of the event that fired.
	 *	@param	InEmitterTime		The emitter time when the event fired.
	 *  @param  InParticleCount     The number of particles spawned in the burst
	 *	@param	InLocation			The location of the particle emitter when the event fired.
	 */
	ENGINE_API void ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 ParticleCount,
		const FVector InLocation, const TArray<class UParticleModuleEventSendToGame*>& InEventData);

	/**
	 *	Record a kismet event.
	 *
	 *	@param	InEventName				The name of the event that fired.
	 *	@param	InEmitterTime			The emitter time when the event fired.
	 *	@param	InLocation				The location of the particle when the event fired.
	 *	@param	InVelocity				The velocity of the particle when the event fired.
	 *	@param	InNormal				Normal vector of the collision in coordinate system of the returner. Zero=none.
	 */
	UFUNCTION(BlueprintCallable, Category="Effects|Components|ParticleSystem")
	ENGINE_API void GenerateParticleEvent(const FName InEventName, const float InEmitterTime,
		const FVector InLocation, const FVector InDirection, const FVector InVelocity);


	/**
	 * Finds the replay clip of the specified ID number
	 *
	 * @return Returns the replay clip or NULL if none
	 */
	ENGINE_API UParticleSystemReplay* FindReplayClipForIDNumber( const int32 InClipIDNumber );

	// @todo document
	ENGINE_API void KillParticlesForced();
	
	/** 
	 *	Set a named random vector instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	ENGINE_API void SetVectorRandParameter(FName ParameterName, const FVector& Param, const FVector& ParamLow);



	/**
	 *	Set a named random unit vector instance parameter on this ParticleSystemComponent.
	 *	Updates the parameter if it already exists, or creates a new entry if not.
	 */
	ENGINE_API void SetVectorUnitRandParameter(FName ParameterName, const FVector& Param, const FVector& ParamLow);

	/** 
	 *	Set a named random float instance parameter on this ParticleSystemComponent. 
	 *	Updates the parameter if it already exists, or creates a new entry if not. 
	 */
	ENGINE_API void SetFloatRandParameter(FName ParameterName, float Param, float ParamLow);

	/**
	 * Force the component to update its bounding box.
	 */
	ENGINE_API void ForceUpdateBounds();	
	

	// @todo document
	ENGINE_API virtual void RewindEmitterInstances();
	
	/**
	 * This will determine which LOD to use based off the specific ParticleSystem passed in
	 * and the distance to where that PS is being displayed.
	 *
	 * @note:  This is distance based LOD not perf based.  Perf and distance are orthogonal concepts.
	 */
	ENGINE_API virtual int32 DetermineLODLevelForLocation(const FVector& EffectLocation);
	
	/** Set the LOD level of the particle system */
	ENGINE_API virtual void SetLODLevel(int32 InLODLevel);
	
	/** Get the LOD level of the particle system */
	ENGINE_API virtual int32 GetLODLevel();
	
	/** stops the emitter, unregisters the component, and resets the component's properties to the values of its template */
	ENGINE_API void ResetToDefaults();

private:
	/** 
	 * Returns true if this component can and should compute it's LOD without
	 * visibility information from the renderer.
	 */
	ENGINE_API bool ShouldComputeLODFromGameThread();

public:
	inline FParticlePerfStatsContext GetPerfStatsContext(){ return FParticlePerfStatsContext(GetWorld(), Template, this); }
};


//////////////////////////////////////////////////////////////////////////
// ParticleSystemComponent inlines

inline void UParticleSystemComponent::SetAutoAttachParams(USceneComponent* Parent, FName SocketName, EAttachLocation::Type LocationType)
{
	AutoAttachParent = Parent;
	AutoAttachSocketName = SocketName;

	USceneComponent::ConvertAttachLocation(LocationType, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule);
}


