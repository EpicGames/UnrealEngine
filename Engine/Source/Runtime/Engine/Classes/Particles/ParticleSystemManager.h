// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"

#include "ParticleSystemManager.generated.h"

class UActorComponent;
class UParticleSystemComponent;
class FParticleSystemWorldManager;

//Whether to use dynamic or static lists.
//Static lists are possibly cheaper but dynamic list have benefit of reduced complexity, likely easier to implement budgeted system in future and will access tick data more contiguously which may offset the cost of building them.
#define PSC_MAN_USE_STATIC_TICK_LISTS 1

//If we our final TG should wait for our async tasks to complete before completing itself.
//Likely required to ensure nothing overruns EOF updates.
#define PSC_MAN_TG_WAIT_FOR_ASYNC 1

/**
* All data relating to a particle system's ticking. Kept in a cache friendly package.
*/
struct FPSCTickData
{
	FPSCTickData();
		
	/** In most cases, PSCs can just consider a single prerequisite if any. i.e. their attach parent. */
	TObjectPtr<UActorComponent> PrereqComponent;

#if PSC_MAN_USE_STATIC_TICK_LISTS
	/** Handle into a static tick list, if we're using static lists. */
	int32 TickListHandle;
#endif

	//Tick group 
	TEnumAsByte<ETickingGroup> TickGroup;

	/** True if this PSC can have it's concurrent tick run on task threads. If not, everything is done on the GT. */
 	uint8 bCanTickConcurrent : 1;
	/** True if we've unregistered during this frame. Skips the tick this frame and will remove from the lists next frame. */
	uint8 bPendingUnregister : 1;
	/** True if we've been registered on the same frame we've just unregistered. These need to be immediately re-registered once they've been removed. */
	uint8 bPendingReregister : 1;
};

USTRUCT()
struct FParticleSystemWorldManagerTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	//~ FTickFunction Interface
	ENGINE_API virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	ENGINE_API virtual FString DiagnosticMessage() override;
	ENGINE_API virtual FName DiagnosticContext(bool bDetailed)  override;
	//~ FTickFunction Interface

	FParticleSystemWorldManager* Owner;
};

template<>
struct TStructOpsTypeTraits<FParticleSystemWorldManagerTickFunction> : public TStructOpsTypeTraitsBase2<FParticleSystemWorldManagerTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

//Define this so we can create inline alloc arrays of the right size but still allow runtime modification if desired.
#define INITIAL_PSC_MANAGER_ASYNC_BATCH_SIZE 8
typedef TArray<int32, TInlineAllocator<INITIAL_PSC_MANAGER_ASYNC_BATCH_SIZE>> FPSCManagerAsyncTickBatch;

/**
* Main manager class for particle systems in the world.
*/
class FParticleSystemWorldManager : public FGCObject
{
private:
	static const int32 MaxTickGroup = (int32)TG_NewlySpawned;
	static TMap<UWorld*, FParticleSystemWorldManager*> WorldManagers;

	static void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	static void OnPreWorldFinishDestroy(UWorld* World);
	static void OnWorldBeginTearDown(UWorld* World);

	typedef TFunction<void(UParticleSystemComponent*, FPSCTickData&, int32)> FPSCTickListFunction;

	struct FTickList
	{
		FTickList(FParticleSystemWorldManager* InOwner)
			:Owner(InOwner)
		{
		}

		FParticleSystemWorldManager* Owner;

		/** List of handles in the world manager list. */
		TArray<int32> TickList;

		void Add(int32 Handle);
		void Remove(int32 Handle);
		void Reset();
		inline int32 Num()const { return TickList.Num(); }
		inline int32& operator[](const int32 Index) { return TickList[Index]; }
		inline TArray<int32>& Get() { return TickList; }
	};

public:
	static void OnStartup();
	static void OnShutdown();

	inline static FParticleSystemWorldManager* Get(UWorld* World)
	{
		FParticleSystemWorldManager** Ret =  WorldManagers.Find(World);
		if (Ret)
		{
			checkSlow(*Ret);
			return *Ret;
		}
		return nullptr;
	}

	FParticleSystemWorldManager(UWorld* InWorld);
	virtual ~FParticleSystemWorldManager();
	void Cleanup();

	bool RegisterComponent(UParticleSystemComponent* PSC);
	void UnregisterComponent(UParticleSystemComponent* PSC);

	inline FPSCTickData& GetTickData(int32 Handle) { return PSCTickData[Handle]; }
	inline UParticleSystemComponent* GetManagedComponent(int32 Handle) { return ManagedPSCs[Handle]; }

	void Tick(ETickingGroup TickGroup, float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	void Dump();

	void SetTickGroupPriority(const int32 ParticleSystemTickGroupIndex, const bool bHighPriority);
	static void SetFirstHighPriTickGroup(int FirstHighPriTickGroupIndex);

protected:
	/** Clean up dead components post GC */
	void HandlePostGarbageCollect();


private:

	void BuildTickLists(int32 StartIndex, ETickingGroup CurrTickGroup);

	template<bool bAsync>
	void ProcessTickList(float DeltaTime, ELevelTick TickType, ETickingGroup TickGroup, TArray<FTickList>& TickLists, const FGraphEventRef& TickGroupCompletionGraphEvent);
	inline void FlushAsyncTicks(const FGraphEventRef& TickGroupCompletionGraphEvent);
	inline void QueueAsyncTick(int32 Handle, const FGraphEventRef& TickGroupCompletionGraphEvent);

	void AddPSC(UParticleSystemComponent* PSC);
	void RemovePSC(int32 PSCIndex);

	void HandleManagerEnabled();

	void ClearPendingUnregister();

	UWorld* World;

	TArray<FParticleSystemWorldManagerTickFunction> TickFunctions;
	   
	TArray<TObjectPtr<UParticleSystemComponent>> ManagedPSCs;
	TArray<FPSCTickData> PSCTickData;
	
	/** Array of PSCs to tick per tick group. These can have their concurrent ticks done in parallel */
	TArray<FTickList> TickLists_Concurrent;

	/** Array of PSCs to tick per tick group. These must be done entirely on the GT. */
	TArray<FTickList> TickLists_GT;

	/**
	PSCs that have been registered with the manager but have not yet been processed into the appropriate buffers.
	This is done at the beginning of each tick group.
	*/
	TArray<TObjectPtr<UParticleSystemComponent>> PendingRegisterPSCs;

	/** Cached value of r.EnablePSCWorldManager. Allows us to reset all systems when this changes. */
	int32 bCachedParticleWorldManagerEnabled;

	/** Callback to remove and PSCs that the GC frees. */
	FDelegateHandle PostGarbageCollectHandle;

	FPSCManagerAsyncTickBatch AsyncTickBatch;

#if !UE_BUILD_SHIPPING
	static const UEnum* TickGroupEnum;
#endif
};

extern ENGINE_API int32 GbEnablePSCWorldManager;
