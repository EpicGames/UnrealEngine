// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ParticleHelper.h"
#include "ParticleSystemReplay.generated.h"

USTRUCT()
struct FParticleEmitterReplayFrame
{
	GENERATED_USTRUCT_BODY()

	/** Type of emitter (EDynamicEmitterType) */
	int32 EmitterType;

	/** Original index into the particle systems list of particle emitter indices.  This is currently
		only needed for mesh emitters. */
	int32 OriginalEmitterIndex;



		/** State for the emitter this frame.  The actual object type  */
		struct FDynamicEmitterReplayDataBase* FrameState;

		FParticleEmitterReplayFrame()
		: EmitterType( DET_Unknown )
		,	OriginalEmitterIndex( 0 )
		,	FrameState( NULL )
		{
		}
		~FParticleEmitterReplayFrame()
		{
			// Clean up frame state
			if( FrameState != NULL )
			{
				delete FrameState;
				FrameState = NULL;
			}
		}

		/** Serialization operator */
		friend FArchive& operator<<( FArchive& Ar, FParticleEmitterReplayFrame& Obj );
	
};

/** A single frame within this replay */
USTRUCT()
struct FParticleSystemReplayFrame
{
	GENERATED_USTRUCT_BODY()

	/** Emitter frame state data */
	TArray<struct FParticleEmitterReplayFrame> Emitters;



	/** Serialization operator */
	friend FArchive& operator<<( FArchive& Ar, FParticleSystemReplayFrame& Obj );
	
};

UCLASS(hidecategories=Object, AutoExpandCategories=ParticleSystemReplay)
class UParticleSystemReplay : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Unique ID number for this replay clip */
	UPROPERTY(EditAnywhere, transient, duplicatetransient, Category=ParticleSystemReplay)
	int32 ClipIDNumber;

	/** Ordered list of frames */
	TArray<struct FParticleSystemReplayFrame> Frames;


	//~ Begin UObject Interface
	virtual void Serialize( FArchive& Ar ) override;
	//~ End UObject Interface
};

