// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Rotation/ParticleModuleRotationBase.h"
#include "ParticleModuleRotation.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Initial Rotation"))
class UParticleModuleRotation : public UParticleModuleRotationBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Initial rotation of the particle (1 = 360 degrees).
	 *	The value is retrieved using the EmitterTime.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionFloat StartRotation;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	//End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



