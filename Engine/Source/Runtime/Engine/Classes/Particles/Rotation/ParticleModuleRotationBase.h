// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleRotationBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Rotation"))
class UParticleModuleRotationBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

};

