// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Rotation/ParticleModuleRotationBase.h"
#include "ParticleModuleRotationOverLifetime.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Rotation/Life"))
class UParticleModuleRotationOverLifetime : public UParticleModuleRotationBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The rotation of the particle (1.0 = 360 degrees).
	 *	The value is retrieved using the RelativeTime of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionFloat RotationOverLife;

	/**
	 *	If true,  the particle rotation is multiplied by the value retrieved from RotationOverLife.
	 *	If false, the particle rotation is incremented by the value retrieved from RotationOverLife.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	uint32 Scale:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//End UObject Interface

	//Begin UParticleModule Interface
	virtual void	Update(const FUpdateContext& Context) override;
	//End UParticleModule Interface
};



