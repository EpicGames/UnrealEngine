// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/RotationRate/ParticleModuleRotationRateBase.h"
#include "ParticleModuleMeshRotationRate.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Init Mesh Rotation Rate"))
class UParticleModuleMeshRotationRate : public UParticleModuleRotationRateBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Initial rotation rate, in rotations per second.
	 *	The value is retrieved using the EmitterTime.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionVector StartRotationRate;

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual bool	TouchesMeshRotation() const override { return true; }
	//~ End UParticleModule Interface

	/** Initializes the default values for this property */
	void InitializeDefaults();

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);

};



