// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/RotationRate/ParticleModuleRotationRateBase.h"
#include "ParticleModuleMeshRotationRateOverLife.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Mesh Rotation Rate over Life"))
class UParticleModuleMeshRotationRateOverLife : public UParticleModuleRotationRateBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The rotation rate desired.
	 *	The value is retrieved using the RelativeTime of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionVector RotRate;

	/**
	 *	If true, scale the current rotation rate by the value retrieved.
	 *	Otherwise, set the rotation rate to the value retrieved.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	uint32 bScaleRotRate:1;

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual bool TouchesMeshRotation() const override { return true; }
	//~ End UParticleModule Interface

	/** Initializes the default values for this property */
	void InitializeDefaults();
};



