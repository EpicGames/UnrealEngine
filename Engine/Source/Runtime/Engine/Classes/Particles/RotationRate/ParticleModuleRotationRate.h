// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/RotationRate/ParticleModuleRotationRateBase.h"
#include "ParticleModuleRotationRate.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Initial Rotation Rate"))
class UParticleModuleRotationRate : public UParticleModuleRotationRateBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	Initial rotation rate, in rotations per second.
	 *	The value is retrieved using the EmitterTime.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionFloat StartRotationRate;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//~ Begin UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



