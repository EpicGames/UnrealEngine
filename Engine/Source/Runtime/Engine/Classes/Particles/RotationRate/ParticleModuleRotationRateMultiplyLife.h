// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/RotationRate/ParticleModuleRotationRateBase.h"
#include "ParticleModuleRotationRateMultiplyLife.generated.h"

class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Rotation Rate * Life"))
class UParticleModuleRotationRateMultiplyLife : public UParticleModuleRotationRateBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The scale factor that should be applied to the rotation rate.
	 *	The value is retrieved using the RelativeTime of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Rotation)
	struct FRawDistributionFloat LifeMultiplier;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	// End UParticleModule Interface
};



