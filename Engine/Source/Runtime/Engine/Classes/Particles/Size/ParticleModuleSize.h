// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Size/ParticleModuleSizeBase.h"
#include "ParticleModuleSize.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Initial Size"))
class UParticleModuleSize : public UParticleModuleSizeBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The initial size that should be used for a particle.
	 *	The value is retrieved using the EmitterTime during the spawn of a particle.
	 *	It is added to the Size and BaseSize fields of the spawning particle.
	 */
	UPROPERTY(EditAnywhere, Category=Size)
	struct FRawDistributionVector StartSize;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	//~ End UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



