// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Size/ParticleModuleSizeBase.h"
#include "ParticleModuleSizeMultiplyLife.generated.h"

class UParticleEmitter;
class UParticleLODLevel;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Size By Life"))
class UParticleModuleSizeMultiplyLife : public UParticleModuleSizeBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The scale factor for the size that should be used for a particle.
	 *	The value is retrieved using the RelativeTime of the particle during its update.
	 */
	UPROPERTY(EditAnywhere, Category=Size)
	struct FRawDistributionVector LifeMultiplier;

	/** 
	 *	If true, the X-component of the scale factor will be applied to the particle size X-component.
	 *	If false, the X-component is left unaltered.
	 */
	UPROPERTY(EditAnywhere, Category=Size)
	uint32 MultiplyX:1;

	/** 
	 *	If true, the Y-component of the scale factor will be applied to the particle size Y-component.
	 *	If false, the Y-component is left unaltered.
	 */
	UPROPERTY(EditAnywhere, Category=Size)
	uint32 MultiplyY:1;

	/** 
	 *	If true, the Z-component of the scale factor will be applied to the particle size Z-component.
	 *	If false, the Z-component is left unaltered.
	 */
	UPROPERTY(EditAnywhere, Category=Size)
	uint32 MultiplyZ:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual bool   IsSizeMultiplyLife() override { return true; };

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

	//~ End UParticleModule Interface

protected:
	friend class FParticleModuleSizeMultiplyLifeDetails;
};



