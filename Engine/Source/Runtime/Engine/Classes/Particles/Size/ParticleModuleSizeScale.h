// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Size/ParticleModuleSizeBase.h"
#include "ParticleModuleSizeScale.generated.h"

class UParticleEmitter;
class UParticleLODLevel;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Size Scale"))
class UParticleModuleSizeScale : public UParticleModuleSizeBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The amount the BaseSize should be scaled before being used as the size of the particle. 
	 *	The value is retrieved using the RelativeTime of the particle during its update.
	 *	NOTE: this module overrides any size adjustments made prior to this module in that frame.
	 */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScale)
	struct FRawDistributionVector SizeScale;

	/** Ignored */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScale)
	uint32 EnableX:1;

	/** Ignored */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScale)
	uint32 EnableY:1;

	/** Ignored */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScale)
	uint32 EnableZ:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

	//~ End UParticleModule Interface

protected:
	friend class FParticleModuleSizeScaleDetails;
};



