// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleSizeScaleBySpeed: Scale the size of a particle by its velocity.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/Size/ParticleModuleSizeBase.h"
#include "ParticleModuleSizeScaleBySpeed.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Size By Speed"), MinimalAPI)
class UParticleModuleSizeScaleBySpeed : public UParticleModuleSizeBase
{
	GENERATED_UCLASS_BODY()

	/** By how much speed affects the size of the particle in each dimension. */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScaleBySpeed)
	FVector2D SpeedScale;

	/** The maximum amount by which to scale a particle in each dimension. */
	UPROPERTY(EditAnywhere, Category=ParticleModuleSizeScaleBySpeed)
	FVector2D MaxScale;


	//~ Begin UParticleModule Interface
	virtual void Update(const FUpdateContext& Context) override;
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	// End UParticleModule Interface
};



