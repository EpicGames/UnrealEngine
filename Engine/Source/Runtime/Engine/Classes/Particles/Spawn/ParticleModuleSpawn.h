// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/Spawn/ParticleModuleSpawnBase.h"
#include "ParticleModuleSpawn.generated.h"

class UParticleLODLevel;

UCLASS(editinlinenew, hidecategories=Object, hidecategories=ParticleModuleSpawnBase, MinimalAPI, meta=(DisplayName = "Spawn"))
class UParticleModuleSpawn : public UParticleModuleSpawnBase
{
	GENERATED_UCLASS_BODY()

	/** The rate at which to spawn particles. */
	UPROPERTY(EditAnywhere, Category=Spawn)
	struct FRawDistributionFloat Rate;

	/** The scalar to apply to the rate. */
	UPROPERTY(EditAnywhere, Category=Spawn)
	struct FRawDistributionFloat RateScale;

	/** The array of burst entries. */
	UPROPERTY(EditAnywhere, export, noclear, Category=Burst)
	TArray<FParticleBurst> BurstList;

	/** Scale all burst entries by this amount. */
	UPROPERTY(EditAnywhere, Category=Burst)
	struct FRawDistributionFloat BurstScale;

	/** The method to utilize when burst-emitting particles. */
	UPROPERTY(EditAnywhere, Category = Burst)
	TEnumAsByte<EParticleBurstMethod> ParticleBurstMethod;

	/**	If true, the SpawnRate will be scaled by the global CVar r.EmitterSpawnRateScale */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint32 bApplyGlobalSpawnRateScale : 1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	virtual void	PostLoad() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual bool	GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel) override;
	//~ End UParticleModule Interface

	//~ Begin UParticleModuleSpawnBase Interface
	virtual bool GetSpawnAmount(const FContext& Context, int32 Offset, float OldLeftover, 
		float DeltaTime, int32& Number, float& Rate) override;
	virtual float GetMaximumSpawnRate() override;
	virtual float GetEstimatedSpawnRate() override;
	virtual int32 GetMaximumBurstCount() override;
	//~ End UParticleModuleSpawnBase Interface

	float GetGlobalRateScale()const;
};



