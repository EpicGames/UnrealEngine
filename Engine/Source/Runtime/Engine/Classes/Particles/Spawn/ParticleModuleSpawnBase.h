// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleSpawnBase.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Spawn"))
class UParticleModuleSpawnBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	If true, the SpawnRate of the SpawnModule of the emitter will be processed.
	 *	If mutliple Spawn modules are 'stacked' in an emitter, if ANY of them 
	 *	have this set to false, it will not process the SpawnModule SpawnRate.
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint32 bProcessSpawnRate:1;

	/** 
	 *	If true, the BurstList of the SpawnModule of the emitter will be processed.
	 *	If mutliple Spawn modules are 'stacked' in an emitter, if ANY of them 
	 *	have this set to false, it will not process the SpawnModule BurstList.
	 */
	UPROPERTY(EditAnywhere, Category=Burst)
	uint32 bProcessBurstList:1;


	//~ Begin UParticleModule Interface
	virtual EModuleType	GetModuleType() const override {	return EPMT_Spawn;	}
	//~ End UParticleModule Interface

	/**
	 *	Retrieve the spawn amount this module is contributing.
	 *	Note that if multiple Spawn-specific modules are present, if any one
	 *	of them ignores the SpawnRate processing it will be ignored.
	 *
	 *	@param	Owner		The particle emitter instance that is spawning.
	 *	@param	Offset		The offset into the particle payload for the module.
	 *	@param	OldLeftover	The bit of timeslice left over from the previous frame.
	 *	@param	DeltaTime	The time that has expired since the last frame.
	 *	@param	Number		The number of particles to spawn. (OUTPUT)
	 *	@param	Rate		The spawn rate of the module. (OUTPUT)
	 *
	 *	@return	bool		false if the SpawnRate should be ignored.
	 *						true if the SpawnRate should still be processed.
	 */
	virtual bool GetSpawnAmount(const FContext& Context, int32 Offset, float OldLeftover, 
		float DeltaTime, int32& Number, float& Rate)
	{
		return bProcessSpawnRate;
	}
	
	/**
	 *	Retrieve the burst count this module is contributing.
	 *	Note that if multiple Spawn-specific modules are present, if any one
	 *	of them ignores the default BurstList, it will be ignored.
	 *
	 *	@param	Owner		The particle emitter instance that is spawning.
	 *	@param	Offset		The offset into the particle payload for the module.
	 *	@param	OldLeftover	The bit of timeslice left over from the previous frame.
	 *	@param	DeltaTime	The time that has expired since the last frame.
	 *	@param	Number		The number of particles to burst. (OUTPUT)
	 *
	 *	@return	bool		false if the default BurstList should be ignored.
	 *						true if the default BurstList should still be processed.
	 */
	virtual bool GetBurstCount(FParticleEmitterInstance* Owner, int32 Offset, float OldLeftover, 
		float DeltaTime, int32& Number)
	{
		Number = 0;
		return bProcessBurstList;
	}

	/**
	 *	Retrieve the maximum spawn rate for this module...
	 *	Used in estimating the number of particles that could be used.
	 *
	 *	@return	float		The maximum spawn rate
	 */
	virtual float GetMaximumSpawnRate() { return 0.0f; }

	/**
	 *	Retrieve the estimated spawn rate for this module...
	 *	Used in estimating the number of particles that could be used.
	 *
	 *	@return	float			The maximum spawn rate
	 */
	virtual float GetEstimatedSpawnRate() { return 0.0f; }

	/**
	 *	Retrieve the maximum number of particles this module could burst.
	 *	Used in estimating the number of particles that could be used.
	 *
	 *	@return	int32			The maximum burst count
	 */
	virtual int32 GetMaximumBurstCount() { return 0; }

};



