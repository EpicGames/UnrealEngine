// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Spawn/ParticleModuleSpawnBase.h"
#include "ParticleModuleSpawnPerUnit.generated.h"

struct FParticleEmitterInstance;
struct FRawDistributionFloat;

UCLASS(editinlinenew, MinimalAPI, hidecategories=Object, meta=(DisplayName = "Spawn PerUnit"))
class UParticleModuleSpawnPerUnit : public UParticleModuleSpawnBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The scalar to apply to the distance traveled.
	 *	The value from SpawnPerUnit is divided by this value to give the actual
	 *	number of particles per unit. 
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	float UnitScalar;

	/**
	 *	The tolerance for moving vs. not moving w.r.t. the bIgnoreSpawnRateWhenMoving flag.
	 *	Ie, if (DistanceMoved < (UnitScalar x MovementTolerance)) then consider it not moving.
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	float MovementTolerance;

	/** 
	 *	The amount to spawn per meter distribution.
	 *	The value is retrieved using the EmitterTime.
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	FRawDistributionFloat SpawnPerUnit;

	/**
	 *	The maximum valid movement for a single frame.
	 *	If 0.0, then the check is not performed.
	 *	Currently, if the distance moved between frames is greater than this
	 *	then NO particles will be spawned.
	 *	This is primiarily intended to cover cases where the PSystem is
	 *	attached to teleporting objects.
	 */
	UPROPERTY(EditAnywhere, Category = Spawn)
	float MaxFrameDistance;

	/**
	 *	If true, process the default spawn rate when not moving...
	 *	When not moving, skip the default spawn rate.
	 *	If false, return the bProcessSpawnRate setting.
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint8 bIgnoreSpawnRateWhenMoving : 1;

	/** If true, ignore the X-component of the movement */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint8 bIgnoreMovementAlongX : 1;

	/** If true, ignore the Y-component of the movement */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint8 bIgnoreMovementAlongY : 1;

	/** If true, ignore the Z-component of the movement */
	UPROPERTY(EditAnywhere, Category=Spawn)
	uint8 bIgnoreMovementAlongZ : 1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual uint32	RequiredBytesPerInstance() override;
	//~ End UParticleModule Interface

	//~ Begin UParticleModuleSpawnBase Interface
	virtual bool GetSpawnAmount(const FContext& Context, int32 Offset, float OldLeftover, 
		float DeltaTime, int32& Number, float& Rate) override;
	//~ End UParticleModuleSpawnBase Interface
};



