// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/SubUV/ParticleModuleSubUVBase.h"
#include "Particles/ParticleEmitter.h"
#include "ParticleModuleSubUV.generated.h"

class UParticleLODLevel;
class USubUVAnimation;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "SubImage Index"))
class UParticleModuleSubUV : public UParticleModuleSubUVBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 * SubUV animation asset to use.
	 * When specified, optimal bounding geometry for each SubUV frame will be used when rendering the sprites for this emitter instead of full quads.
	 * This reduction in overdraw can reduce the GPU cost of rendering the emitter by 2x or 3x, depending on how much unused space was in the texture.
	 * The bounding geometry is generated off of the texture alpha setup in the SubUV Animation asset, so that has to match what the material is using for opacity, or clipping will occur.
	 * When specified, SubImages_Horizontal and SubImages_Vertical will come from the asset instead of the Required Module.
	 */
	UPROPERTY()
	TObjectPtr<USubUVAnimation> Animation;

	/**
	 *	The index of the sub-image that should be used for the particle.
	 *	The value is retrieved using the RelativeTime of the particles.
	 */
	UPROPERTY(EditAnywhere, Category=SubUV)
	struct FRawDistributionFloat SubImageIndex;

	/** 
	 *	If true, use *real* time when updating the image index.
	 *	The movie will update regardless of the slomo settings of the game.
	 */
	UPROPERTY(EditAnywhere, Category=Realtime)
	uint8 bUseRealTime:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();
	
	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual bool IsPostLoadThreadSafe() const override;
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo ) override;
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void Update(const FUpdateContext& Context) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//~ End UParticleModule Interface

	/**
	 *	Determine the current image index to use...
	 *
	 *	@param	Owner					The emitter instance being updated.
	 *	@param	Offset					The offset to the particle payload for this module.
	 *	@param	Particle				The particle that the image index is being determined for.
	 *	@param	InterpMethod			The EParticleSubUVInterpMethod method used to update the subUV.
	 *	@param	SubUVPayload			The FFullSubUVPayload for this particle.
	 *
	 *	@return	float					The image index with interpolation amount as the fractional portion.
	 */
	virtual float DetermineImageIndex(const FContext& Context, int32 Offset, FBaseParticle* Particle, 
		EParticleSubUVInterpMethod InterpMethod, FFullSubUVPayload& SubUVPayload, float DeltaTime);

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif

protected:
	friend class FParticleModuleSubUVDetails;
};



