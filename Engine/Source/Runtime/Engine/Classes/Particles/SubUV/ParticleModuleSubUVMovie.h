// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "Particles/SubUV/ParticleModuleSubUV.h"
#include "ParticleModuleSubUVMovie.generated.h"

class UParticleModuleTypeDataBase;

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "SubUV Movie"), MinimalAPI)
class UParticleModuleSubUVMovie : public UParticleModuleSubUV
{
	GENERATED_UCLASS_BODY()

	/**
	 *	If true, use the emitter time to look up the frame rate.
	 *	If false (default), use the particle relative time.
	 */
	UPROPERTY(EditAnywhere, Category=Flipbook)
	uint32 bUseEmitterTime:1;

	/**
	 *	The frame rate the SubUV images should be 'flipped' thru at.
	 
	 */
	UPROPERTY(EditAnywhere, Category=Flipbook)
	struct FRawDistributionFloat FrameRate;

	/**
	 *	The starting image index for the SubUV (1 = the first frame).
	 *	Assumes order of Left->Right, Top->Bottom
	 *	If greater than the last frame, it will clamp to the last one.
	 *	If 0, then randomly selects a starting frame.
	 */
	UPROPERTY(EditAnywhere, Category=Flipbook)
	int32 StartingFrame;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual void SetToSensibleDefaults(UParticleEmitter* Owner) override;
	virtual void GetCurveObjects(TArray<FParticleCurvePair>& OutCurves) override;
	//~ End UParticleModule Interface
	
	//~ Begin UParticleModuleSubUV Interface
	virtual float DetermineImageIndex(const FContext& Context, int32 Offset, FBaseParticle* Particle, 
		EParticleSubUVInterpMethod InterpMethod, FFullSubUVPayload& SubUVPayload, float DeltaTime) override;
	//~ End UParticleModuleSubUV Interface
	
};



