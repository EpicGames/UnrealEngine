// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * SubUV animation asset
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "Containers/ResourceArray.h"
#include "ProfilingDebugging/CookStats.h"
#include "SubUVAnimation.generated.h"

class UTexture2D;
struct FPropertyChangedEvent;

#if ENABLE_COOK_STATS
class SubUVAnimationCookStats
{
public:
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats;
};
#endif

// Can change this guid to force SubUV derived data to be regenerated on next load
#define SUBUV_DERIVEDDATA_VER TEXT("9E83F251AF83E44D8192C237A7C2433E")

/** 
 * More bounding vertices results in reduced overdraw, but adds more triangle overhead.
 * The eight vertex mode is best used when the SubUV texture has a lot of space to cut out that is not captured by the four vertex version,
 * and when the particles using the texture will be few and large.
 */
UENUM()
enum ESubUVBoundingVertexCount : int
{
	BVC_FourVertices,
	BVC_EightVertices
};

UENUM()
enum EOpacitySourceMode : int
{
	OSM_Alpha,
	OSM_ColorBrightness,
	OSM_RedChannel,
	OSM_GreenChannel,
	OSM_BlueChannel
};

class FSubUVDerivedData
{
public:
	TArray<FVector2f> BoundingGeometry;

	static ENGINE_API FString GetDDCKeyString(const FGuid& StateId, int32 SizeX, int32 SizeY, int32 Mode, float AlphaThreshold, int32 OpacitySourceMode);
	static ENGINE_API void GetFeedback(UTexture2D* SubUVTexture, int32 SubImages_Horizontal, int32 SubImages_Vertical, ESubUVBoundingVertexCount BoundingMode, float AlphaThreshold, EOpacitySourceMode OpacitySourceMode,
		TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo);
	ENGINE_API void Serialize(FStructuredArchive::FSlot Slot);
	ENGINE_API void Build(UTexture2D* SubUVTexture, int32 SubImages_Horizontal, int32 SubImages_Vertical, ESubUVBoundingVertexCount BoundingMode, float AlphaThreshold, EOpacitySourceMode OpacitySourceMode);
};

class FSubUVBoundingGeometryBuffer : public FVertexBuffer
{
public:
	TArray<FVector2f>* Vertices;
	FShaderResourceViewRHIRef ShaderResourceView;

	FSubUVBoundingGeometryBuffer();
	FSubUVBoundingGeometryBuffer(TArray<FVector2f>* InVertices);
	~FSubUVBoundingGeometryBuffer();

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
};

using FSubUVVertexResourceArray UE_DEPRECATED(5.5, "FResourceArrayUploadArrayView should be used directly.") = FResourceArrayUploadArrayView;

/**
 * SubUV animation asset, which caches bounding geometry for regions in the SubUVTexture with non-zero opacity.
 * Particle emitters with a SubUV module which use this asset leverage the optimal bounding geometry to reduce overdraw.
 */
UCLASS(hidecategories=object, MinimalAPI)
class USubUVAnimation : public UObject
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Texture to generate bounding geometry from.
	 */
	UPROPERTY(EditAnywhere, Category=SubUV)
	TObjectPtr<UTexture2D> SubUVTexture;

	/** The number of sub-images horizontally in the texture							*/
	UPROPERTY(EditAnywhere, Category=SubUV)
	int32 SubImages_Horizontal;

	/** The number of sub-images vertically in the texture								*/
	UPROPERTY(EditAnywhere, Category=SubUV)
	int32 SubImages_Vertical;

	/** 
	 * More bounding vertices results in reduced overdraw, but adds more triangle overhead.
	 * The eight vertex mode is best used when the SubUV texture has a lot of space to cut out that is not captured by the four vertex version,
	 * and when the particles using the texture will be few and large.
	 */
	UPROPERTY(EditAnywhere, Category=SubUV)
	TEnumAsByte<enum ESubUVBoundingVertexCount> BoundingMode;

	UPROPERTY(EditAnywhere, Category=SubUV)
	TEnumAsByte<enum EOpacitySourceMode> OpacitySourceMode;

	/** 
	 * Alpha channel values larger than the threshold are considered occupied and will be contained in the bounding geometry.
	 * Raising this threshold slightly can reduce overdraw in particles using this animation asset.
	 */
	UPROPERTY(EditAnywhere, Category=SubUV, meta=(UIMin = "0", UIMax = "1"))
	float AlphaThreshold;

private:

	/** Derived data for this asset, generated off of SubUVTexture. */
	FSubUVDerivedData DerivedData;

	/** Tracks progress of BoundingGeometryBuffer release during destruction. */
	FRenderCommandFence ReleaseFence;

	/** Used on platforms that support instancing, the bounding geometry is fetched from a vertex shader instead of on the CPU. */
	FSubUVBoundingGeometryBuffer* BoundingGeometryBuffer;

public:

	inline int32 GetNumBoundingVertices() const 
	{ 
		if (BoundingMode == BVC_FourVertices)
		{
			return 4;
		}

		return 8;
	}

	inline int32 GetNumBoundingTriangles() const 
	{ 
		if (BoundingMode == BVC_FourVertices)
		{
			return 2;
		}

		return 6;
	}

	inline int32 GetNumFrames() const
	{
		return SubImages_Vertical * SubImages_Horizontal;
	}

	inline bool IsBoundingGeometryValid() const
	{
		return DerivedData.BoundingGeometry.Num() != 0;
	}

	inline const FVector2f* GetFrameData(int32 FrameIndex) const
	{
		return &DerivedData.BoundingGeometry[FrameIndex * GetNumBoundingVertices()];
	}

	inline FRHIShaderResourceView* GetBoundingGeometrySRV() const
	{
		return BoundingGeometryBuffer->ShaderResourceView;
	}

	//~ Begin UObject Interface.
    virtual void PostInitProperties() override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual void FinishDestroy() override;
	//~ End UObject Interface.

private:
	void CacheDerivedData();
};



