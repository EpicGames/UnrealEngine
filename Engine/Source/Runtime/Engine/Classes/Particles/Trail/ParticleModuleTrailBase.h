// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleTrailBase
 *	Provides the base class for Trail emitter modules
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleTrailBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Trail"))
class UParticleModuleTrailBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()


	//~ Begin UParticleModule Interface
	virtual EModuleType	GetModuleType() const override {	return EPMT_Trail;	}
	virtual bool CanTickInAnyThread() override
	{
		return true;	
	}
	//~ End UParticleModule Interface
};



