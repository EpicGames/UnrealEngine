// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleTrailSource
 *
 *	This module implements a single source for a Trail emitter.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Trail/ParticleModuleTrailBase.h"
#include "ParticleModuleTrailSource.generated.h"

class UParticleSystemComponent;
struct FParticleEmitterInstance;

UENUM()
enum ETrail2SourceMethod : int
{
	/** Default	- use the emitter position. 
	 *	This is the fallback for when other modes can't be resolved.
	 */
	PET2SRCM_Default,
	/** Particle	- use the particles from a given emitter in the system.		
	 *	The name of the emitter should be set in SourceName.
	 */
	PET2SRCM_Particle,
	/** Actor		- use the actor as the source.
	 *	The name of the actor should be set in SourceName.
	 */
	PET2SRCM_Actor,
	PET2SRCM_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Source"))
class UParticleModuleTrailSource : public UParticleModuleTrailBase
{
	GENERATED_UCLASS_BODY()

	/** The source method for the trail. */
	UPROPERTY(EditAnywhere, Category=Source)
	TEnumAsByte<enum ETrail2SourceMethod> SourceMethod;

	/** The name of the source - either the emitter or Actor. */
	UPROPERTY(EditAnywhere, Category=Source)
	FName SourceName;

	/** The strength of the tangent from the source point for each Trail. */
	UPROPERTY(EditAnywhere, Category=Source)
	struct FRawDistributionFloat SourceStrength;

	/** Whether to lock the source to the life of the particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bLockSourceStength:1;

	/**
	 *	SourceOffsetCount
	 *	The number of source offsets that can be expected to be found on the instance.
	 *	These must be named
	 *		TrailSourceOffset#
	 */
	UPROPERTY(EditAnywhere, Category=Source)
	int32 SourceOffsetCount;

	/** 
	 *	Default offsets from the source(s). 
	 *	If there are < SourceOffsetCount slots, the grabbing of values will simply wrap.
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category=Source)
	TArray<FVector> SourceOffsetDefaults;

	/**	Particle selection method, when using the SourceMethod of Particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	TEnumAsByte<enum EParticleSourceSelectionMethod> SelectionMethod;

	/**	Interhit particle rotation - only valid for SourceMethod of PET2SRCM_Particle. */
	UPROPERTY(EditAnywhere, Category=Source)
	uint32 bInheritRotation:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void	AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp) override;
	virtual void GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList) override;
	//~ End UParticleModule Interface

	/**
	 *	Retrieve the SourceOffset for the given trail index.
	 *	Currently, this is only intended for use by Ribbon emitters
	 *
	 *	@param	InTrailIdx			The index of the trail whose offset is being retrieved
	 *	@param	InEmitterInst		The EmitterInstance requesting the SourceOffset
	 *	@param	OutSourceOffset		The source offset for the trail of interest
	 *
	 *	@return	bool				true if successful, false if not (including no offset)
	 */
	bool	ResolveSourceOffset(int32 InTrailIdx, FParticleEmitterInstance* InEmitterInst, FVector& OutSourceOffset);


};



