// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleTypeDataAnimTrail
 *	Provides the base data for animation-based trail emitters.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "ParticleModuleTypeDataAnimTrail.generated.h"

class UParticleEmitter;
class UParticleSystemComponent;
struct FParticleEmitterInstance;

UCLASS(MinimalAPI, editinlinenew, hidecategories=Object, meta=(DisplayName = "AnimTrail Data"))
class UParticleModuleTypeDataAnimTrail : public UParticleModuleTypeDataBase
{
	GENERATED_UCLASS_BODY()

	//*****************************************************************************
	// General Trail Variables
	//*****************************************************************************

	/** 
	 *	If true, when the system is deactivated, mark trails as dead.
	 *	This means they will still render, but will not have more particles
	 *	added to them, even if the system re-activates...
	 */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bDeadTrailsOnDeactivate:1;

	/** If true, recalculate the previous tangent when a new particle is spawned */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bEnablePreviousTangentRecalculation:1;

	/** If true, recalculate tangents every frame to allow velocity/acceleration to be applied */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bTangentRecalculationEveryFrame:1;

	/** 
	 *	The (estimated) covered distance to tile the 2nd UV set at.
	 *	If 0.0, a second UV set will not be passed in.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float TilingDistance;

	/** 
	 *	The distance step size for tessellation.
	 *	# Tessellation Points = TruncToInt((Distance Between Spawned Particles) / DistanceTessellationStepSize)). If 0 then there is no distance tessellation.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float DistanceTessellationStepSize;

	/** 
	 *	The tangent scalar for tessellation.
	 *	This is the degree change in the tangent direction [0...180] required to warrant an additional tessellation point. If 0 then there is no tangent tessellation.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float TangentTessellationStepSize;

	/** 
	 *	The width step size for tessellation.
	 *	This is the number of world units change in the width required to warrant an additional tessellation point. If 0 then there is no width tessellation.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float WidthTessellationStepSize;


	//~ Begin UParticleModule Interface
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual bool CanTickInAnyThread() override
	{
		return true;
	}
	//~ End UParticleModule Interface

	//~ Begin UParticleModuleTypeDataBase Interface
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent) override;
	virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual EPrimitiveType GetPrimitiveType() const override { return PT_TriangleStrip; }
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) override;
	//~ End UParticleModuleTypeDataBase Interface
};



