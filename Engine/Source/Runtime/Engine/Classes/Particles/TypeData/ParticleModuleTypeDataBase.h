// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "PSOPrecache.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleTypeDataBase.generated.h"

class IParticleEmitterInstanceOwner;
class UParticleEmitter;
struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, abstract, MinimalAPI)
class UParticleModuleTypeDataBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()


	//~ Begin UParticleModule Interface
	virtual EModuleType	GetModuleType() const override {	return EPMT_TypeData;	}
	//~ End UParticleModule Interface

	/**
	 * Build any resources required for simulating the emitter.
	 * @param EmitterBuildInfo - Information compiled for the emitter.
	 */
	virtual void Build( struct FParticleEmitterBuildInfo& EmitterBuildInfo ) {}

	/**
	 * Return whether the type data module requires a build step.
	 */
	virtual bool RequiresBuild() const { return false; }

	// @todo document
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent);

	virtual const FVertexFactoryType* GetVertexFactoryType() const { return nullptr; }
	virtual EPrimitiveType GetPrimitiveType() const { return PT_TriangleList; }

	/** Cache any desired module pointers inside this type data */
	virtual void CacheModuleInfo(UParticleEmitter* Emitter) {}

	// @todo document
	virtual bool		SupportsSpecificScreenAlignmentFlags() const	{	return false;			}
	// @todo document
	virtual bool		SupportsSubUV() const	{ return false; }
	// @todo document
	virtual bool		IsAMeshEmitter() const	{ return false; }

	/** Determine if motion blur is enabled for the owning emitter. */
	virtual bool		IsMotionBlurEnabled() const  { return false; }


	/**
	* Collect all the data required for PSO precaching 
	*/
	struct FPSOPrecacheParams
	{
		EPrimitiveType PrimitiveType;
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
	};
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) {};
};



