// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "ParticleModuleTypeDataBeam2.generated.h"

class UInterpCurveEdSetup;
class UParticleEmitter;
class UParticleModuleBeamModifier;
class UParticleModuleBeamNoise;
class UParticleModuleBeamSource;
class UParticleModuleBeamTarget;
class UParticleSystemComponent;
struct FCurveEdEntry;
struct FParticleEmitterInstance;

UENUM()
enum EBeam2Method : int
{
	PEB2M_Distance UMETA(DisplayName="Distance"),
	PEB2M_Target UMETA(DisplayName="Target"),
	PEB2M_Branch UMETA(DisplayName="Branch"),
	PEB2M_MAX,
};

//
// Beam Tapering Variables.
//
UENUM()
enum EBeamTaperMethod : int
{
	PEBTM_None UMETA(DisplayName="None"),
	PEBTM_Full UMETA(DisplayName="Full"),
	PEBTM_Partial UMETA(DisplayName="Partial"),
	PEBTM_MAX,
};

//
// Beam Multi-target Variables.
//
USTRUCT()
struct FBeamTargetData
{
	GENERATED_USTRUCT_BODY()

	/** Name of the target.																	*/
	UPROPERTY(EditAnywhere, Category=BeamTargetData)
	FName TargetName;

	/** Percentage chance the target will be selected (100 = always).						*/
	UPROPERTY(EditAnywhere, Category=BeamTargetData)
	float TargetPercentage;


	FBeamTargetData()
		: TargetPercentage(0)
	{
	}

};

UCLASS(editinlinenew, dontcollapsecategories, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Beam Data"))
class UParticleModuleTypeDataBeam2 : public UParticleModuleTypeDataBase
{
	GENERATED_UCLASS_BODY()

	//
	// General Beam Variables.
	//
	
	/** 
	 *	The method with which to form the beam(s). Must be one of the following:
	 *		PEB2M_Distance	- Use the distance property to emit a beam along the X-axis of the emitter.
	 *		PEB2M_Target	- Emit a beam from the source to the supplied target.
	 *		PEB2M_Branch	- Currently unimplemented.
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	TEnumAsByte<enum EBeam2Method> BeamMethod;

	/** The number of times to tile the texture along each beam. 
	 *  Overridden by TextureTilingDistance if it is > 0.0.
	 *	1st UV set only. 2nd UV set does not Tile.		
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	int32 TextureTile;

	/** The distance per texture tile. 
	 *	1st UV set only. 2nd UV set does not Tile.				
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	float TextureTileDistance;

	/** The number of sheets to render															*/
	UPROPERTY(EditAnywhere, Category=Beam)
	int32 Sheets;

	/** The number of live beams																*/
	UPROPERTY(EditAnywhere, Category=Beam)
	int32 MaxBeamCount;

	/** The speed at which the beam should move from source to target when firing up.
	 *	'0' indicates instantaneous
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	float Speed;

	/** 
	 * Indicates whether the beam should be interpolated.
	 *     <= 0 --> no
	 *     >  0 --> yes (and is equal to the number of interpolation steps that should be taken.
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	int32 InterpolationPoints;

	/** If true, there will ALWAYS be a beam...													*/
	UPROPERTY(EditAnywhere, Category=Beam)
	uint32 bAlwaysOn:1;

	/** 
	 *	The approach to use for determining the Up vector(s) for the beam.
	 *
	 *	0 indicates that the Up FVector should be calculated at EVERY point in the beam.
	 *	1 indicates a single Up FVector should be determined at the start of the beam and used at every point.
	 *	N indicates an Up FVector should be calculated every N points of the beam and interpolated between them.
	 *	    [NOTE: This mode is currently unsupported.]
	 */
	UPROPERTY(EditAnywhere, Category=Beam)
	int32 UpVectorStepSize;

	//
	// Beam Branching Variables.
	//
	
	/** The name of the emitter to branch from (if mode is PEB2M_Branch)
	 * MUST BE IN THE SAME PARTICLE SYSTEM!
	 */
	UPROPERTY(EditAnywhere, Category=Branching)
	FName BranchParentName;

	//
	// Beam Distance Variables.
	//
	
	/** 
	 *	The distance along the X-axis to stretch the beam
	 *	Distance is only used if BeamMethod is PEB2M_Distance
	 */
	UPROPERTY(EditAnywhere, Category=Distance)
	struct FRawDistributionFloat Distance;

	/**
	 *	Tapering mode - one of the following:
	 *	PEBTM_None		- No tapering is applied
	 *	PEBTM_Full		- Taper the beam relative to source-->target, regardless of current beam length
	 *	PEBTM_Partial	- Taper the beam relative to source-->location, 0=source,1=endpoint
	 */
	UPROPERTY(EditAnywhere, Category=Taper)
	TEnumAsByte<enum EBeamTaperMethod> TaperMethod;

	/** Tapering factor, 0 = source of beam, 1 = target											*/
	UPROPERTY(EditAnywhere, Category=Taper)
	struct FRawDistributionFloat TaperFactor;

	/**
	 *  Tapering scaling
	 *	This is intended to be either a constant, uniform or a ParticleParam.
	 *	If a curve is used, 0/1 mapping of source/target... which could be integrated into
	 *	the taper factor itself, and therefore makes no sense.
	 */
	UPROPERTY(EditAnywhere, Category=Taper)
	struct FRawDistributionFloat TaperScale;

	//
	// Beam Rendering Variables.
	//
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 RenderGeometry:1;

	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 RenderDirectLine:1;

	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 RenderLines:1;

	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 RenderTessellation:1;

	//////////////////////////////////////////////////////////////////////////
	TArray<UParticleModuleBeamSource*>		LOD_BeamModule_Source;
	TArray<UParticleModuleBeamTarget*>		LOD_BeamModule_Target;
	TArray<UParticleModuleBeamNoise*>		LOD_BeamModule_Noise;
	TArray<UParticleModuleBeamModifier*>	LOD_BeamModule_SourceModifier;
	TArray<UParticleModuleBeamModifier*>	LOD_BeamModule_TargetModifier;
	//////////////////////////////////////////////////////////////////////////

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void	PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void	PostInitProperties() override;
	//~ End UObject Interface


	//~ Begin UParticleModule Interface
	virtual void	Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	virtual uint32	RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	virtual	bool	AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries) override;
	virtual bool CanTickInAnyThread() override
	{
		return true;
	}
	//~ End UParticleModule Interface


	//~ Begin UParticleModuleTypeDataBase Interface
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent) override;
	virtual void CacheModuleInfo(UParticleEmitter* Emitter) override;
	virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual EPrimitiveType GetPrimitiveType() const { return PT_TriangleStrip; }
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) override;
	//~ End UParticleModuleTypeDataBase Interface


	/**
	 *	GetDataPointers
	 *	Retrieves the data pointers stored in the particle payload.
	 *	
	 *	@param	Owner				The owning emitter instance of the particle.
	 *	@param	ParticleBase		Pointer to the particle of interest
	 *	@param	CurrentOffset		The offset to the particle payload
	 *	@param	BeamData			The FBeam2TypeDataPayload pointer - output
	 *	@param	InterpolatedPoints	The FVector interpolated points pointer - output
	 *	@param	NoiseRate			The float NoiseRate pointer - output
	 *	@param	NoiseDeltaTime		The float NoiseDeltaTime pointer - output
	 *	@param	TargetNoisePoints	The FVector TargetNoisePoints pointer - output
	 *	@param	NextNoisePoints		The FVector NextNoisePoints pointer - output
	 *	@param	TaperValues			The float TaperValues pointer - output
	 *	@param	NoiseDistanceScale	The float NoiseDistanceScale pointer - output
	 *	@param	SourceModifier		The FBeamParticleModifierPayloadData for the source - output
	 *	@param	TargetModifier		The FBeamParticleModifierPayloadData for the target - output
	 */
	virtual void	GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
			int32& CurrentOffset, FBeam2TypeDataPayload*& BeamData, FVector*& InterpolatedPoints, 
			float*& NoiseRate, float*& NoiseDeltaTime, FVector*& TargetNoisePoints, 
			FVector*& NextNoisePoints, float*& TaperValues, float*& NoiseDistanceScale,
			FBeamParticleModifierPayloadData*& SourceModifier,
			FBeamParticleModifierPayloadData*& TargetModifier);

	/**
	 *	GetDataPointerOffsets
	 *	Retrieves the offsets to the data stored in the particle payload.
	 *	
	 *	@param	Owner						The owning emitter instance of the particle.
	 *	@param	ParticleBase				Pointer to the particle of interest
	 *	@param	CurrentOffset				The offset to the particle payload
	 *	@param	BeamDataOffset				The FBeam2TypeDataPayload pointer - output
	 *	@param	InterpolatedPointsOffset	The FVector interpolated points pointer - output
	 *	@param	NoiseRateOffset				The float NoiseRate pointer - output
	 *	@param	NoiseDeltaTimeOffset		The float NoiseDeltaTime pointer - output
	 *	@param	TargetNoisePointsOffset		The FVector TargetNoisePoints pointer - output
	 *	@param	NextNoisePointsOffset		The FVector NextNoisePoints pointer - output
	 *	@param	TaperCount					The int32 TaperCount - output
	 *	@param	TaperValuesOffset			The float TaperValues pointer - output
	 *	@param	NoiseDistanceScaleOffset	The float NoiseDistanceScale pointer - output
	 */
	virtual void	GetDataPointerOffsets(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
			int32& CurrentOffset, int32& BeamDataOffset, int32& InterpolatedPointsOffset, int32& NoiseRateOffset, 
			int32& NoiseDeltaTimeOffset, int32& TargetNoisePointsOffset, int32& NextNoisePointsOffset, 
			int32& TaperCount, int32& TaperValuesOffset, int32& NoiseDistanceScaleOffset);

	/**
	 *	GetNoiseRange
	 *	Retrieves the range of noise
	 *	
	 *	@param	NoiseMin		The minimum noise - output
	 *	@param	NoiseMax		The maximum noise - output
	 */
	void	GetNoiseRange(FVector& NoiseMin, FVector& NoiseMax);
};



