// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModuleTypeDataGpu: Type data definition for GPU particle simulation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "ParticleModuleTypeDataGpu.generated.h"

class UParticleSystemComponent;

/**
 * Data needed for local vector fields.
 */
USTRUCT()
struct FGPUSpriteLocalVectorFieldInfo
{
	GENERATED_USTRUCT_BODY()

	/** Local vector field to apply to this emitter. */
	UPROPERTY()
	TObjectPtr<class UVectorField> Field;

	/** Local vector field transform. */
	UPROPERTY()
	FTransform Transform;

	/** Minimum initial rotation. */
	UPROPERTY()
	FRotator MinInitialRotation;

	/** Maximum initial rotation. */
	UPROPERTY()
	FRotator MaxInitialRotation;

	/** Local vector field rotation rate. */
	UPROPERTY()
	FRotator RotationRate;

	/** Local vector field intensity. */
	UPROPERTY()
	float Intensity;

	/** Local vector field tightness. */
	UPROPERTY()
	float Tightness;

	/* Ignore Components Transform	*/
	UPROPERTY() 
	uint32 bIgnoreComponentTransform:1;
	/** Tile vector field in x axis? */
	UPROPERTY() 
	uint32 bTileX:1;
	/** Tile vector field in y axis? */
	UPROPERTY() 
	uint32 bTileY:1;
	/** Tile vector field in z axis? */
	UPROPERTY() 
	uint32 bTileZ:1;
	/** Use fix delta time in the simulation? */
	UPROPERTY()
	uint32 bUseFixDT:1;


	FGPUSpriteLocalVectorFieldInfo()
		: Field(NULL)
		, MinInitialRotation(ForceInit)
		, MaxInitialRotation(ForceInit)
		, RotationRate(ForceInit)
		, Intensity(0)
		, Tightness(0)
		, bIgnoreComponentTransform(false)
		, bTileX(false)
		, bTileY(false)
		, bTileZ(false)
		, bUseFixDT(false)
	{
	}

};

/**
 * The data needed by the runtime to simulate sprites.
 */
USTRUCT()
struct FGPUSpriteEmitterInfo
{
	GENERATED_USTRUCT_BODY()

	/** The required module. Needed for now, but should be divorced from the runtime. */
	UPROPERTY()
	TObjectPtr<class UParticleModuleRequired> RequiredModule;

	/** The spawn module. Needed for now, but should be divorced from the runtime. */
	UPROPERTY()
	TObjectPtr<class UParticleModuleSpawn> SpawnModule;

	/** The spawn-per-unit module. */
	UPROPERTY()
	TObjectPtr<class UParticleModuleSpawnPerUnit> SpawnPerUnitModule;

	/** List of spawn modules that must be evaluated at runtime. */
	UPROPERTY()
	TArray<TObjectPtr<class UParticleModule>> SpawnModules;

	/** Local vector field info. */
	UPROPERTY()
	struct FGPUSpriteLocalVectorFieldInfo LocalVectorField;

	/** Per-particle vector field scale. */
	UPROPERTY()
	FFloatDistribution VectorFieldScale;

	/** Per-particle drag coefficient. */
	UPROPERTY()
	FFloatDistribution DragCoefficient;

	/** Point attractor strength over time. */
	UPROPERTY()
	FFloatDistribution PointAttractorStrength;

	/** Damping factor applied to particle collisions. */
	UPROPERTY()
	FFloatDistribution Resilience;

	/** Constant acceleration to apply to particles. */
	UPROPERTY()
	FVector ConstantAcceleration;

	/** Point attractor position. */
	UPROPERTY()
	FVector PointAttractorPosition;

	/** Point attractor radius, squared. */
	UPROPERTY()
	float PointAttractorRadiusSq;

	/** Amount by which to offset particles when they are spawned. */
	UPROPERTY()
	FVector OrbitOffsetBase;

	UPROPERTY()
	FVector OrbitOffsetRange;

	/** One over the maximum size of a sprite particle. */
	UPROPERTY()
	FVector2D InvMaxSize;

	/** The inverse scale to apply to rotation rate. */
	UPROPERTY()
	float InvRotationRateScale;

	/** The maximum lifetime of particles in this emitter. */
	UPROPERTY()
	float MaxLifetime;

	/** The maximum number of particles expected for this emitter. */
	UPROPERTY()
	int32 MaxParticleCount;

	/** The method for aligning the particle based on the camera. */
	UPROPERTY()
	TEnumAsByte<EParticleScreenAlignment> ScreenAlignment;

	/** The method for locking the particles to a particular axis. */
	UPROPERTY()
	TEnumAsByte<EParticleAxisLock> LockAxisFlag;

	/** If true, collisions are enabled for this emitter. */
	UPROPERTY()
	uint32 bEnableCollision : 1;

	UPROPERTY()
	TEnumAsByte<EParticleCollisionMode::Type> CollisionMode;
	
	/** If true, use velocity to approximate motion blur */
	UPROPERTY()
	uint32 bUseVelocityForMotionBlur :1;

	/** If true, removes the HMD view roll (e.g. in VR) */
	UPROPERTY()
	uint32 bRemoveHMDRoll : 1;
	
	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_Square */
	UPROPERTY()
	float MinFacingCameraBlendDistance;

	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_FacingCameraPosition */
	UPROPERTY()
	float MaxFacingCameraBlendDistance;

	/** Dynamic color scale from the ColorOverLife module. */
	UPROPERTY()
	FRawDistributionVector DynamicColor;

	/** Dynamic alpha scale from the ColorOverLife module. */
	UPROPERTY()
	FRawDistributionFloat DynamicAlpha;

	/** Dynamic color scale from the ColorScaleOverLife module. */
	UPROPERTY()
	FRawDistributionVector DynamicColorScale;

	/** Dynamic alpha scale from the ColorScaleOverLife module. */
	UPROPERTY()
	FRawDistributionFloat DynamicAlphaScale;

	FGPUSpriteEmitterInfo()
		: RequiredModule(NULL)
		, SpawnModule(NULL)
		, SpawnPerUnitModule(NULL)
		, ConstantAcceleration(ForceInit)
		, PointAttractorPosition(ForceInit)
		, PointAttractorRadiusSq(0)
		, OrbitOffsetBase(ForceInit)
		, OrbitOffsetRange(ForceInit)
		, InvMaxSize(ForceInit)
		, InvRotationRateScale(0)
		, MaxLifetime(0)
		, MaxParticleCount(0)
		, ScreenAlignment(0)
		, LockAxisFlag(0)
		, bEnableCollision(false)
		, CollisionMode(EParticleCollisionMode::SceneDepth)
		, bUseVelocityForMotionBlur(0)
		, bRemoveHMDRoll(0)
		, MinFacingCameraBlendDistance(0.f)
		, MaxFacingCameraBlendDistance(0.f)
		, Resources(nullptr)
	{
	}


	/** Pointer to runtime resources. */
	class FGPUSpriteResources* Resources;
};

/**
 * The source data for runtime resources.
 */
USTRUCT()
struct FGPUSpriteResourceData
{
	GENERATED_USTRUCT_BODY()

	/** Quantized color samples. */
	UPROPERTY()
	TArray<FColor> QuantizedColorSamples;

	/** Quantized samples for misc curve attributes to be evaluated at runtime. */
	UPROPERTY()
	TArray<FColor> QuantizedMiscSamples;

	/** Quantized samples for simulation attributes. */
	UPROPERTY()
	TArray<FColor> QuantizedSimulationAttrSamples;

	/** Scale and bias to be applied to the color of sprites. */
	UPROPERTY()
	FVector4 ColorScale;

	UPROPERTY()
	FVector4 ColorBias;

	/** Scale and bias to be applied to the misc curve. */
	UPROPERTY()
	FVector4 MiscScale;

	UPROPERTY()
	FVector4 MiscBias;

	/** Scale and bias to be applied to the simulation attribute curves. */
	UPROPERTY()
	FVector4 SimulationAttrCurveScale;

	UPROPERTY()
	FVector4 SimulationAttrCurveBias;

	/** Size of subimages. X:SubImageCountH Y:SubImageCountV Z:1/SubImageCountH W:1/SubImageCountV */
	UPROPERTY()
	FVector4 SubImageSize;

	/** SizeBySpeed parameters. XY=SpeedScale ZW=MaxSpeedScale. */
	UPROPERTY()
	FVector4 SizeBySpeed;

	/** Constant acceleration to apply to particles. */
	UPROPERTY()
	FVector ConstantAcceleration;

	/** Offset at which to orbit. */
	UPROPERTY()
	FVector OrbitOffsetBase;

	UPROPERTY()
	FVector OrbitOffsetRange;

	/** Frequency at which the particle orbits around each axis. */
	UPROPERTY()
	FVector OrbitFrequencyBase;

	UPROPERTY()
	FVector OrbitFrequencyRange;

	/** Phase offset of orbit around each axis. */
	UPROPERTY()
	FVector OrbitPhaseBase;

	UPROPERTY()
	FVector OrbitPhaseRange;

	/** Scale to apply to global vector fields. */
	UPROPERTY()
	float GlobalVectorFieldScale;

	/** Tightness override value for the global vector fields. */
	UPROPERTY()
	float GlobalVectorFieldTightness;

	/** Scale to apply to per-particle vector field scale. */
	UPROPERTY()
	float PerParticleVectorFieldScale;

	/** Bias to apply to per-particle vector field scale. */
	UPROPERTY()
	float PerParticleVectorFieldBias;

	/** Scale to apply to per-particle drag coefficient. */
	UPROPERTY()
	float DragCoefficientScale;

	/** Bias to apply to per-particle drag coefficient. */
	UPROPERTY()
	float DragCoefficientBias;

	/** Scale to apply to per-particle damping factor. */
	UPROPERTY()
	float ResilienceScale;

	/** Bias to apply to per-particle damping factor. */
	UPROPERTY()
	float ResilienceBias;

	/** Scale to apply to per-particle size for collision. */
	UPROPERTY()
	float CollisionRadiusScale;

	/** Bias to apply to per-particle size for collision. */
	UPROPERTY()
	float CollisionRadiusBias;

	/** Bias applied to relative time upon collision. */
	UPROPERTY()
	float CollisionTimeBias;

	/** Control on reflection's random distribution spread. */
	UPROPERTY()
	float CollisionRandomSpread;

	/** Control on reflection's random distribution when colliding. (1=uniform distribution) */
	UPROPERTY()
	float CollisionRandomDistribution;

	/** One minus the coefficient of friction applied to particles upon collision. */
	UPROPERTY()
	float OneMinusFriction;

	/** Scale to apply to per-particle rotation rate. */
	UPROPERTY()
	float RotationRateScale;

	/** How much to stretch sprites based on camera motion blur. */
	UPROPERTY()
	float CameraMotionBlurAmount;

	/** Screen alignment for particles. */
	UPROPERTY()
	TEnumAsByte<enum EParticleScreenAlignment> ScreenAlignment;

	/** The method for locking the particles to a particular axis. */
	UPROPERTY()
	TEnumAsByte<enum EParticleAxisLock> LockAxisFlag;

	/** Pivot offset in UV space for placing the verts of each particle. */
	UPROPERTY()
	FVector2D PivotOffset;
	
	/** If true, use velocity to approximate motion blur */
	UPROPERTY()
	uint32 bUseVelocityForMotionBlur :1;

	/** If true, removes the HMD view roll (e.g. in VR) */
	UPROPERTY()
	uint32 bRemoveHMDRoll:1;

	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_Square */
	UPROPERTY()
	float MinFacingCameraBlendDistance;

	/** The distance at which PSA_FacingCameraDistanceBlend	is fully PSA_FacingCameraPosition */
	UPROPERTY()
	float MaxFacingCameraBlendDistance;

	FGPUSpriteResourceData()
		: ColorScale(ForceInit)
		, ColorBias(ForceInit)
		, MiscScale(ForceInit)
		, MiscBias(ForceInit)
		, SimulationAttrCurveScale(ForceInit)
		, SimulationAttrCurveBias(ForceInit)
		, SubImageSize(ForceInit)
		, SizeBySpeed(ForceInit)
		, ConstantAcceleration(ForceInit)
		, OrbitOffsetBase(ForceInit)
		, OrbitOffsetRange(ForceInit)
		, OrbitFrequencyBase(ForceInit)
		, OrbitFrequencyRange(ForceInit)
		, OrbitPhaseBase(ForceInit)
		, OrbitPhaseRange(ForceInit)
		, GlobalVectorFieldScale(0)
		, GlobalVectorFieldTightness(-1)
		, PerParticleVectorFieldScale(0)
		, PerParticleVectorFieldBias(0)
		, DragCoefficientScale(0)
		, DragCoefficientBias(0)
		, ResilienceScale(0)
		, ResilienceBias(0)
		, CollisionRadiusScale(0)
		, CollisionRadiusBias(0)
		, CollisionTimeBias(0)
		, CollisionRandomSpread(0)
		, CollisionRandomDistribution(2)
		, OneMinusFriction(0)
		, RotationRateScale(0)
		, CameraMotionBlurAmount(0)
		, ScreenAlignment(0)
		, LockAxisFlag(0)
		, PivotOffset(-0.5f,-0.5f)
		, bUseVelocityForMotionBlur(0)
		, bRemoveHMDRoll(0)
		, MinFacingCameraBlendDistance(0.f)
		, MaxFacingCameraBlendDistance(0.f)		
	{
	}

};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "GPU Sprites"))
class UParticleModuleTypeDataGpu : public UParticleModuleTypeDataBase
{
	GENERATED_UCLASS_BODY()

	/** Information for runtime simulation. */
	UPROPERTY()
	struct FGPUSpriteEmitterInfo EmitterInfo;

	/** Data used to initialize runtime resources. */
	UPROPERTY()
	struct FGPUSpriteResourceData ResourceData;

	/** How much to stretch sprites based on camera motion blur. */
	UPROPERTY(EditAnywhere, Category=ParticleModuleTypeDataGpu)
	float CameraMotionBlurAmount;

	/** When true, all existing partilces are cleared when the emitter is initialized. */
	UPROPERTY(EditAnywhere, Category = ParticleModuleTypeDataGpu)
	uint32 bClearExistingParticlesOnInit:1;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin UParticleModuleTypeDataBase Interface
	virtual void Build( struct FParticleEmitterBuildInfo& EmitterBuildInfo ) override;
	virtual bool RequiresBuild() const override { return true; }
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent) override;
	virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) override;
	//~ End UParticleModuleTypeDataBase Interface
};

