// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Math/RandomStream.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Orientation/ParticleModuleOrientationAxisLock.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "ParticleModuleTypeDataMesh.generated.h"

class UParticleEmitter;
class UParticleSystemComponent;
struct FParticleEmitterInstance;
class UStaticMesh;

UENUM()
enum EMeshScreenAlignment : int
{
	PSMA_MeshFaceCameraWithRoll UMETA(DisplayName="Face Camera With Roll"),
	PSMA_MeshFaceCameraWithSpin UMETA(DisplayName="Face Camera With Spin"),
	PSMA_MeshFaceCameraWithLockedAxis UMETA(DisplayName="Face Camera With Locked-Axis"),
	PSMA_MAX,
};

UENUM()
enum EMeshCameraFacingUpAxis : int
{
	CameraFacing_NoneUP UMETA(DisplayName="None"),
	CameraFacing_ZUp UMETA(DisplayName="Z Up"),
	CameraFacing_NegativeZUp UMETA(DisplayName="-Z Up"),
	CameraFacing_YUp UMETA(DisplayName="Y Up"),
	CameraFacing_NegativeYUp UMETA(DisplayName="-Y Up"),
	CameraFacing_MAX,
};

UENUM()
enum EMeshCameraFacingOptions : int
{
	XAxisFacing_NoUp UMETA(DisplayName="X Axis Facing : No Up"),
	XAxisFacing_ZUp UMETA(DisplayName="X Axis Facing : Z Up"),
	XAxisFacing_NegativeZUp UMETA(DisplayName="X Axis Facing : -Z Up"),
	XAxisFacing_YUp UMETA(DisplayName="X Axis Facing : Y Up"),
	XAxisFacing_NegativeYUp UMETA(DisplayName="X Axis Facing : -Y Up"),

	LockedAxis_ZAxisFacing UMETA(DisplayName="Locked Axis : Z Axis Facing"),
	LockedAxis_NegativeZAxisFacing UMETA(DisplayName="Locked Axis : -Z Axis Facing"),
	LockedAxis_YAxisFacing UMETA(DisplayName="Locked Axis : Y Axis Facing"),
	LockedAxis_NegativeYAxisFacing UMETA(DisplayName="Locked Axis : -Y Axis Facing"),

	VelocityAligned_ZAxisFacing UMETA(DisplayName="Velocity Aligned : Z Axis Facing"),
	VelocityAligned_NegativeZAxisFacing UMETA(DisplayName="Velocity Aligned : -Z Axis Facing"),
	VelocityAligned_YAxisFacing UMETA(DisplayName="Velocity Aligned : Y Axis Facing"),
	VelocityAligned_NegativeYAxisFacing UMETA(DisplayName="Velocity Aligned : -Y Axis Facing"),

	EMeshCameraFacingOptions_MAX,
};

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Mesh Data"))
class UParticleModuleTypeDataMesh : public UParticleModuleTypeDataBase
{
	GENERATED_UCLASS_BODY()

	/** The static mesh to render at the particle positions */
	UPROPERTY(EditAnywhere, Category=Mesh)
	TObjectPtr<UStaticMesh> Mesh;

	/** Random stream for the initial rotation distribution */
	FRandomStream RandomStream;

	/** use the static mesh's LOD setup and switch LODs based on largest particle's screen size*/
	UPROPERTY(EditAnywhere, Category = Mesh)
	float LODSizeScale;

	/** use the static mesh's LOD setup and switch LODs based on largest particle's screen size*/
	UPROPERTY(EditAnywhere, Category = Mesh)
	uint8 bUseStaticMeshLODs : 1;

	/** If true, has the meshes cast shadows */
	UPROPERTY()
	uint8 CastShadows:1;

	/** UNUSED (the collision module dictates doing collisions) */
	UPROPERTY()
	uint8 DoCollisions:1;

	/** 
	 *	The alignment to use on the meshes emitted.
	 *	The RequiredModule->ScreenAlignment MUST be set to PSA_TypeSpecific to use.
	 *	One of the following:
	 *	PSMA_MeshFaceCameraWithRoll
	 *		Face the camera allowing for rotation around the mesh-to-camera FVector 
	 *		(amount provided by the standard particle sprite rotation).  
	 *	PSMA_MeshFaceCameraWithSpin
	 *		Face the camera allowing for the mesh to rotate about the tangential axis.  
	 *	PSMA_MeshFaceCameraWithLockedAxis
	 *		Face the camera while maintaining the up FVector as the locked direction.  
	 */
	UPROPERTY(EditAnywhere, Category=Mesh)
	TEnumAsByte<enum EMeshScreenAlignment> MeshAlignment;

	/**
	 *	If true, use the emitter material when rendering rather than the one applied 
	 *	to the static mesh model.
	 */
	UPROPERTY(EditAnywhere, Category=Mesh)
	uint8 bOverrideMaterial:1;

	UPROPERTY(EditAnywhere, Category = Mesh)
	uint8 bOverrideDefaultMotionBlurSettings : 1;

	UPROPERTY(EditAnywhere, Category = Mesh, meta=(EditCondition="bOverrideDefaultMotionBlurSettings"))
	uint8 bEnableMotionBlur : 1;

	/** The 'pre' rotation pitch (in degrees) to apply to the static mesh used. */
	UPROPERTY(EditAnywhere, Category=Orientation)
	FRawDistributionVector RollPitchYawRange;

	/**
	 *	The axis to lock the mesh on. This overrides TypeSpecific mesh alignment as well as the LockAxis module.
	 *		EPAL_NONE		 -	No locking to an axis.
	 *		EPAL_X			 -	Lock the mesh X-axis facing towards +X.
	 *		EPAL_Y			 -	Lock the mesh X-axis facing towards +Y.
	 *		EPAL_Z			 -	Lock the mesh X-axis facing towards +Z.
	 *		EPAL_NEGATIVE_X	 -	Lock the mesh X-axis facing towards -X.
	 *		EPAL_NEGATIVE_Y	 -	Lock the mesh X-axis facing towards -Y.
	 *		EPAL_NEGATIVE_Z	 -	Lock the mesh X-axis facing towards -Z.
	 *		EPAL_ROTATE_X	 -	Ignored for mesh emitters. Treated as EPAL_NONE.
	 *		EPAL_ROTATE_Y	 -	Ignored for mesh emitters. Treated as EPAL_NONE.
	 *		EPAL_ROTATE_Z	 -	Ignored for mesh emitters. Treated as EPAL_NONE.
	 */
	UPROPERTY(EditAnywhere, Category=Orientation)
	TEnumAsByte<EParticleAxisLock> AxisLockOption;

	/**
	 *	If true, then point the X-axis of the mesh towards the camera.
	 *	When set, AxisLockOption as well as all other locked axis/screen alignment settings are ignored.
	 */
	UPROPERTY(EditAnywhere, Category=CameraFacing)
	uint8 bCameraFacing:1;

	/**
	 *	The axis of the mesh to point up when camera facing the X-axis.
	 *		CameraFacing_NoneUP			No attempt to face an axis up or down.
	 *		CameraFacing_ZUp			Z-axis of the mesh should attempt to point up.
	 *		CameraFacing_NegativeZUp	Z-axis of the mesh should attempt to point down.
	 *		CameraFacing_YUp			Y-axis of the mesh should attempt to point up.
	 *		CameraFacing_NegativeYUp	Y-axis of the mesh should attempt to point down.
	 */
	UPROPERTY()
	TEnumAsByte<enum EMeshCameraFacingUpAxis> CameraFacingUpAxisOption_DEPRECATED;

	/**
	 *	The camera facing option to use:
	 *	All camera facing options without locked axis assume X-axis will be facing the camera.
	 *		XAxisFacing_NoUp				- X-axis camera facing, no attempt to face an axis up or down.
	 *		XAxisFacing_ZUp					- X-axis camera facing, Z-axis of the mesh should attempt to point up.
	 *		XAxisFacing_NegativeZUp			- X-axis camera facing, Z-axis of the mesh should attempt to point down.
	 *		XAxisFacing_YUp					- X-axis camera facing, Y-axis of the mesh should attempt to point up.
	 *		XAxisFacing_NegativeYUp			- X-axis camera facing, Y-axis of the mesh should attempt to point down.
	 *	All axis-locked camera facing options assume the AxisLockOption is set. EPAL_NONE will be treated as EPAL_X.
	 *		LockedAxis_ZAxisFacing			- X-axis locked on AxisLockOption axis, rotate Z-axis of the mesh to face towards camera.
	 *		LockedAxis_NegativeZAxisFacing	- X-axis locked on AxisLockOption axis, rotate Z-axis of the mesh to face away from camera.
	 *		LockedAxis_YAxisFacing			- X-axis locked on AxisLockOption axis, rotate Y-axis of the mesh to face towards camera.
	 *		LockedAxis_NegativeYAxisFacing	- X-axis locked on AxisLockOption axis, rotate Y-axis of the mesh to face away from camera.
	 *	All velocity-aligned options do NOT require the ScreenAlignment be set to PSA_Velocity.
	 *	Doing so will result in additional work being performed... (it will orient the mesh twice).
	 *		VelocityAligned_ZAxisFacing         - X-axis aligned to the velocity, rotate the Z-axis of the mesh to face towards camera.
	 *		VelocityAligned_NegativeZAxisFacing - X-axis aligned to the velocity, rotate the Z-axis of the mesh to face away from camera.
	 *		VelocityAligned_YAxisFacing         - X-axis aligned to the velocity, rotate the Y-axis of the mesh to face towards camera.
	 *		VelocityAligned_NegativeYAxisFacing - X-axis aligned to the velocity, rotate the Y-axis of the mesh to face away from camera.
	 */
	UPROPERTY(EditAnywhere, Category=CameraFacing)
	TEnumAsByte<enum EMeshCameraFacingOptions> CameraFacingOption;

	/** 
	 *	If true, apply 'sprite' particle rotation about the orientation axis (direction mesh is pointing).
	 *	If false, apply 'sprite' particle rotation about the camera facing axis.
	 */
	UPROPERTY(EditAnywhere, Category=CameraFacing)
	uint8 bApplyParticleRotationAsSpin : 1;

	/** 
	*	If true, all camera facing options will point the mesh against the camera's view direction rather than pointing at the cameras location. 
	*	If false, the camera facing will point to the cameras position as normal.
	*/
	UPROPERTY(EditAnywhere, Category=CameraFacing)
	uint8 bFaceCameraDirectionRatherThanPosition : 1;

	/**
	*	If true, all collisions for mesh particle on this emitter will take the particle size into account.
	*	If false, particle size will be ignored in collision checks.
	*/
	UPROPERTY(EditAnywhere, Category = Collision)
	uint8 bCollisionsConsiderPartilceSize : 1;

	static int32 GetCurrentDetailMode();
	static int32 GetMeshParticleMotionBlurMinDetailMode();

#if WITH_EDITOR
	void OnMeshChanged();
#endif

	virtual void PostLoad();

	//~ Begin UObject Interface
	virtual bool IsPostLoadThreadSafe() const override;
#if WITH_EDITOR
	virtual void BeginDestroy() override;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	virtual void	Serialize(FArchive& Ar) override;

	//~ End UObject Interface

	void CreateDistribution();

	//~ Begin UParticleModule Interface
	virtual void	SetToSensibleDefaults(UParticleEmitter* Owner) override;
	//~ End UParticleModule Interface

	//~ Begin UParticleModuleTypeDataBase Interface
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent) override;
	virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) override;
	virtual bool	SupportsSpecificScreenAlignmentFlags() const override {	return true;	}	
	virtual bool	SupportsSubUV() const override { return true; }
	virtual bool	IsAMeshEmitter() const override { return true; }
	virtual bool    IsMotionBlurEnabled() const override 
	{ 
		if (bOverrideDefaultMotionBlurSettings)
		{
			return bEnableMotionBlur;
		}
		else
		{
			return GetMeshParticleMotionBlurMinDetailMode() >= 0 && GetCurrentDetailMode() >= GetMeshParticleMotionBlurMinDetailMode();
		}
	}

	//~ End UParticleModuleTypeDataBase Interface
};



