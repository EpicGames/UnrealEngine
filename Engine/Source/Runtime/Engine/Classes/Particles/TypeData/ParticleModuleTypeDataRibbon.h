// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	ParticleModuleTypeDataRibbon
 *	Provides the base data for ribbon (drop trail) emitters.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "ParticleModuleTypeDataRibbon.generated.h"

UENUM()
enum ETrailsRenderAxisOption : int
{
	Trails_CameraUp UMETA(DisplayName="Camera Up"),
	Trails_SourceUp UMETA(DisplayName="Source Up"),
	Trails_WorldUp UMETA(DisplayName="World Up"),
	Trails_MAX,
};

UCLASS(MinimalAPI, editinlinenew, hidecategories=Object, meta=(DisplayName = "Ribbon Data"))
class UParticleModuleTypeDataRibbon : public UParticleModuleTypeDataBase
{
	GENERATED_UCLASS_BODY()

	//
	// General Trail Variables.
	//
	
	/**
	 *	The maximum amount to tessellate between two particles of the trail. 
	 *	Depending on the distance between the particles and the tangent change, the 
	 *	system will select a number of tessellation points 
	 *		[0..MaxTessellationBetweenParticles]
	 */
	UPROPERTY()
	int32 MaxTessellationBetweenParticles;

	/**
	 *	The number of sheets to render for the trail.
	 */
	UPROPERTY(EditAnywhere, Category=Trail)
	int32 SheetsPerTrail;

	/** The number of live trails									*/
	UPROPERTY(EditAnywhere, Category=Trail)
	int32 MaxTrailCount;

	/** Max particles per trail										*/
	UPROPERTY(EditAnywhere, Category=Trail)
	int32 MaxParticleInTrailCount;

	/** 
	 *	If true, when the system is deactivated, mark trails as dead.
	 *	This means they will still render, but will not have more particles
	 *	added to them, even if the system re-activates...
	 */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bDeadTrailsOnDeactivate:1;

	/**
	 *	If true, when the source of a trail is 'lost' (ie, the source particle
	 *	dies), mark the current trail as dead.
	 */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bDeadTrailsOnSourceLoss:1;

	/** If true, do not join the trail to the source position 		*/
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bClipSourceSegement:1;

	/** If true, recalculate the previous tangent when a new particle is spawned */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bEnablePreviousTangentRecalculation:1;

	/** If true, recalculate tangents every frame to allow velocity/acceleration to be applied */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bTangentRecalculationEveryFrame:1;

	/** If true, ribbon will spawn a particle when it first starts moving */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint32 bSpawnInitialParticle:1;

	/** 
	 *	The 'render' axis for the trail (what axis the trail is stretched out on)
	 *		Trails_CameraUp - Traditional camera-facing trail.
	 *		Trails_SourceUp - Use the up axis of the source for each spawned particle.
	 *		Trails_WorldUp  - Use the world up axis.
	 */
	UPROPERTY(EditAnywhere, Category=Trail)
	TEnumAsByte<enum ETrailsRenderAxisOption> RenderAxis;

	//
	// Trail Spawning Variables.
	//
	
	/**
	 *	The tangent scalar for spawning.
	 *	Angles between tangent A and B are mapped to [0.0f .. 1.0f]
	 *	This is then multiplied by TangentTessellationScalar to give the number of particles to spawn
	 */
	UPROPERTY(EditAnywhere, Category=Spawn)
	float TangentSpawningScalar;

	//
	// Trail Rendering Variables.
	//
	
	/** If true, render the trail geometry (this should typically be on) */
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 bRenderGeometry:1;

	/** If true, render stars at each spawned particle point along the trail */
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 bRenderSpawnPoints:1;

	/** If true, render a line showing the tangent at each spawned particle point along the trail */
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 bRenderTangents:1;

	/** If true, render the tessellated path between spawned particles */
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 bRenderTessellation:1;

	/** 
	 *	The (estimated) covered distance to tile the 2nd UV set at.
	 *	If 0.0, a second UV set will not be passed in.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float TilingDistance;

	/** 
	 *	The distance step size for tessellation.
	 *	# Tessellation Points = TruncToInt((Distance Between Spawned Particles) / DistanceTessellationStepSize))
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float DistanceTessellationStepSize;

	/** 
	 *	If this flag is enabled, the system will scale the number of interpolated vertices
	 *	based on the difference in the tangents of neighboring particles.
	 *	Each pair of neighboring particles will compute the following CheckTangent value:
	 *		CheckTangent = ((ParticleA Tangent DOT ParticleB Tangent) - 1.0f) * 0.5f
	 *	If CheckTangent is LESS THAN 0.5, then the DistanceTessellationStepSize will be 
	 *	scaled based on the result. This will map so that from parallel to orthogonal 
	 *	(0..90 degrees) will scale from [0..1]. Anything greater than 90 degrees will clamp 
	 *	at a scale of 1.
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	uint32 bEnableTangentDiffInterpScale:1;

	/** 
	 *	The tangent scalar for tessellation.
	 *	Angles between tangent A and B are mapped to [0.0f .. 1.0f]
	 *	This is then multiplied by TangentTessellationScalar to give the number of points to tessellate
	 */
	UPROPERTY(EditAnywhere, Category=Rendering)
	float TangentTessellationScalar;


	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual uint32 RequiredBytes(UParticleModuleTypeDataBase* TypeData) override;
	//~ End UParticleModule Interface

	//~ Begin UParticleModuleTypeDataBase Interface
	virtual FParticleEmitterInstance* CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent) override;
	virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual EPrimitiveType GetPrimitiveType() const override { return PT_TriangleStrip; }
	virtual void CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams) override;
	//~ End UParticleModuleTypeDataBase Interface
};



