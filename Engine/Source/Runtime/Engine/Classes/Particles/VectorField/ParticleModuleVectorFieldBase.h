// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldBase: Base class for organizing vector field
		related modules.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleVectorFieldBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Vector Field"))
class UParticleModuleVectorFieldBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

};

