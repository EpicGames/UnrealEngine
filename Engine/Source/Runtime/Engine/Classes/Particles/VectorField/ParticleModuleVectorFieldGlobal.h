// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldGlobal: Global vector field scale.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "ParticleModuleVectorFieldGlobal.generated.h"

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "Global Vector Fields"))
class UParticleModuleVectorFieldGlobal : public UParticleModuleVectorFieldBase
{
	GENERATED_UCLASS_BODY()

	/** Property override value for global vector field tightness.  */
	UPROPERTY(EditAnywhere, Category = VectorField, meta=(InlineEditConditionToggle))
	uint32 bOverrideGlobalVectorFieldTightness:1;

	/** Global vector field scale. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	float GlobalVectorFieldScale;

	/** Global vector field tightness override. */
	UPROPERTY(EditAnywhere, Category=VectorField, meta=(ClampMin = "0.0", UIMin = "0.0", UIMax="1.0", editcondition = "bOverrideGlobalVectorFieldTightness"))
	float GlobalVectorFieldTightness;

	//~ Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	//~ Begin UParticleModule Interface
};

