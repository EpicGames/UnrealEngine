// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldLocal: Emitter-local vector field.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "ParticleModuleVectorFieldLocal.generated.h"

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Local Vector Field"))
class UParticleModuleVectorFieldLocal : public UParticleModuleVectorFieldBase
{
	GENERATED_UCLASS_BODY()

	/** Vector field asset to use. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	TObjectPtr<class UVectorField> VectorField;

	/** Translation of the vector field relative to the emitter. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FVector RelativeTranslation;

	/** Rotation of the vector field relative to the emitter. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FRotator RelativeRotation;

	/** Scale of the vector field relative to the emitter. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FVector RelativeScale3D;

	/** Intensity of the local vector field. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	float Intensity;

	/** Tightness tweak value: 0: Force 1: Velocity. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	float Tightness;

	/** Ignore component transform. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	uint32 bIgnoreComponentTransform:1;
	/** Tile vector field in x axis? */
	UPROPERTY(EditAnywhere, Category=VectorField)
	uint32 bTileX:1;
	/** Tile vector field in y axis? */
	UPROPERTY(EditAnywhere, Category=VectorField)
	uint32 bTileY:1;
	/** Tile vector field in z axis? */
	UPROPERTY(EditAnywhere, Category=VectorField)
	uint32 bTileZ:1;
	/** Use fix delta time in the simulation? */
	UPROPERTY(EditAnywhere, Category=VectorField)
	uint32 bUseFixDT:1;

	//~ Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	//~ Begin UParticleModule Interface
};



