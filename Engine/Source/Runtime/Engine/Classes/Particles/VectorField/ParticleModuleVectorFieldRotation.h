// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldRotation: Random initial rotation for local
		vector fields.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "ParticleModuleVectorFieldRotation.generated.h"

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "VF Init Rotation"))
class UParticleModuleVectorFieldRotation : public UParticleModuleVectorFieldBase
{
	GENERATED_UCLASS_BODY()

	/** Minimum initial rotation applied to the local vector field. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FVector MinInitialRotation;

	/** Maximum initial rotation applied to the local vector field. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FVector MaxInitialRotation;


	//~ Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	//~ Begin UParticleModule Interface
};

