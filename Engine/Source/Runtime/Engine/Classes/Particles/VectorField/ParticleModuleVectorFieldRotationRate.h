// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldRotationRate: Rotation rate for vector fields.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "ParticleModuleVectorFieldRotationRate.generated.h"

UCLASS(editinlinenew, hidecategories=Object, meta=(DisplayName = "VF Rotation Rate"), MinimalAPI)
class UParticleModuleVectorFieldRotationRate : public UParticleModuleVectorFieldBase
{
	GENERATED_UCLASS_BODY()

	/** Constant rotation rate applied to the local vector field. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FVector RotationRate;


	//~ Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;
	//~ Begin UParticleModule Interface
};

