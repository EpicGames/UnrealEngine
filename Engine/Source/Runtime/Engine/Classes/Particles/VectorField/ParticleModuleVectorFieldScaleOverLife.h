// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModuleVectorFieldScaleOverLife: Per-particle vector field scale over life.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "ParticleModuleVectorFieldScaleOverLife.generated.h"

class UParticleLODLevel;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "VF Scale/Life"))
class UParticleModuleVectorFieldScaleOverLife : public UParticleModuleVectorFieldBase
{
	GENERATED_UCLASS_BODY()

	/** Per-particle vector field scale. Evaluated using particle relative time. */
	UPROPERTY()
	TObjectPtr<class UDistributionFloat> VectorFieldScaleOverLife_DEPRECATED;

	/** Per-particle vector field scale. Evaluated using particle relative time. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	FRawDistributionFloat VectorFieldScaleOverLifeRaw;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo) override;

#if WITH_EDITOR
	virtual bool IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString) override;
#endif
	//~ End UParticleModule Interface

protected:
	friend class FParticleModuleVectorFieldScaleOverLifeDetails;
};



