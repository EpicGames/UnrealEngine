// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Velocity/ParticleModuleVelocityBase.h"
#include "ParticleModuleVelocity.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Initial Velocity"))
class UParticleModuleVelocity : public UParticleModuleVelocityBase
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The velocity to apply to a particle when it is spawned.
	 *	Value is retrieved using the EmitterTime of the emitter.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	struct FRawDistributionVector StartVelocity;

	/** 
	 *	The velocity to apply to a particle along its radial direction.
	 *	Direction is determined by subtracting the location of the emitter from the particle location at spawn.
	 *	Value is retrieved using the EmitterTime of the emitter.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	struct FRawDistributionFloat StartVelocityRadial;

	/** Initializes the default values for this property */
	void InitializeDefaults();
	
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	//~ Begin UParticleModule Interface

	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);
};



