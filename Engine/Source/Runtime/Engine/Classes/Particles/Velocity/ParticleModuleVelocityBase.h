// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleModule.h"
#include "ParticleModuleVelocityBase.generated.h"

UCLASS(editinlinenew, hidecategories=Object, abstract, meta=(DisplayName = "Velocity"))
class UParticleModuleVelocityBase : public UParticleModule
{
	GENERATED_UCLASS_BODY()

	/**
	 *	If true, then treat the velocity as world-space defined.
	 *	NOTE: LocalSpace emitters that are moving will see strange results...
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	uint32 bInWorldSpace:1;

	/** If true, then apply the particle system components scale to the velocity value. */
	UPROPERTY(EditAnywhere, Category=Velocity)
	uint32 bApplyOwnerScale:1;

};

