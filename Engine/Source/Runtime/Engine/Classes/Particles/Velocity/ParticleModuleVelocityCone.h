// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionFloat.h"
#include "Particles/Velocity/ParticleModuleVelocityBase.h"
#include "ParticleModuleVelocityCone.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=( DisplayName = "Velocity Cone" ))
class UParticleModuleVelocityCone : public UParticleModuleVelocityBase
{
	GENERATED_UCLASS_BODY()

	/** The Min value represents the inner cone angle value and the Max value represents the outer cone angle value. */
	UPROPERTY(EditAnywhere, Category=Cone)
	struct FRawDistributionFloat Angle;

	/** The initial velocity of the particles. */
	UPROPERTY(EditAnywhere, Category=Cone)
	struct FRawDistributionFloat Velocity;

	/** The direction FVector of the cone. */
	UPROPERTY(EditAnywhere, Category=Cone)
	FVector Direction;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void	Render3DPreview(const FPreviewContext& Context) override;
	//~ Begin UParticleModule Interface
	
	/**
	 *	Extended version of spawn, allows for using a random stream for distribution value retrieval
	 *
	 *	@param	Owner				The particle emitter instance that is spawning
	 *	@param	Offset				The offset to the modules payload data
	 *	@param	SpawnTime			The time of the spawn
	 *	@param	InRandomStream		The random stream to use for retrieving random values
	 */
	void			SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream);

};



