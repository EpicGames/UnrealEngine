// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Velocity/ParticleModuleVelocityBase.h"
#include "ParticleModuleVelocityInheritParent.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Inherit Parent Velocity"))
class UParticleModuleVelocityInheritParent : public UParticleModuleVelocityBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The scale to apply to the parent velocity prior to adding it to the particle velocity during spawn.
	 *	Value is retrieved using the EmitterTime of the emitter.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	struct FRawDistributionVector Scale;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	//~ Begin UParticleModule Interface
};



