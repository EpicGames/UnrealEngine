// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Velocity/ParticleModuleVelocityBase.h"
#include "ParticleModuleVelocityOverLifetime.generated.h"

struct FParticleEmitterInstance;

UCLASS(editinlinenew, hidecategories=Object, MinimalAPI, meta=(DisplayName = "Velocity/Life"))
class UParticleModuleVelocityOverLifetime : public UParticleModuleVelocityBase
{
	GENERATED_UCLASS_BODY()

	/**
	 *	The scaling  value applied to the velocity.
	 *	Value is retrieved using the RelativeTime of the particle.
	 */
	UPROPERTY(EditAnywhere, Category=Velocity)
	struct FRawDistributionVector VelOverLife;

	/**
	 *	If true, the velocity will be SET to the value from the above dist.
	 *	If false, the velocity will be scaled by the above dist.
	 */
	UPROPERTY(EditAnywhere, export, Category=Velocity)
	uint32 Absolute:1;

	/** Initializes the default values for this property */
	void InitializeDefaults();

	//Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UParticleModule Interface
	virtual void Spawn(const FSpawnContext& Context) override;
	virtual void	Update(const FUpdateContext& Context) override;
	//~ Begin UParticleModule Interface
};



