// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "WorldPSCPool.generated.h"

class UParticleSystemComponent;
class UParticleSystem;

#define ENABLE_PSC_POOL_DEBUGGING (!UE_BUILD_SHIPPING)

UENUM()
enum class EPSCPoolMethod : uint8
{
	/**
	* PSC is will be created fresh and not allocated from the pool.
	*/
	None,

	/**
	* PSC is allocated from the pool and will be automatically released back to it.
	* User need not handle this any more that other PSCs but interaction with the PSC after the tick it's spawned in are unsafe.
	* This method is useful for one-shot fx that you don't need to keep a reference to and can fire and forget.
	*/
	AutoRelease,

	/**
	* PSC is allocated from the pool but will NOT be automatically released back to it. User has ownership of the PSC and must call ReleaseToPool when finished with it otherwise the PSC will leak.
	* Interaction with the PSC after it has been released are unsafe.
	* This method is useful for persistent FX that you need to modify parameters upon etc over it's lifetime.
	*/
	ManualRelease,


	/** 
	Special entry allowing manual release PSCs to be manually released but wait until completion to be returned to the pool.
	*/
	ManualRelease_OnComplete UMETA(Hidden),

	/**
	Special entry that marks a PSC as having been returned to the pool. All PSCs currently in the pool are marked this way.
	*/
	FreeInPool UMETA(Hidden),
};

USTRUCT()
struct FPSCPoolElem
{
	GENERATED_BODY()

	UPROPERTY(transient)
	TObjectPtr<UParticleSystemComponent> PSC;

	float LastUsedTime;

	FPSCPoolElem()
		: PSC(nullptr), LastUsedTime(0.0f)
	{

	}
	FPSCPoolElem(UParticleSystemComponent* InPSC, float InLastUsedTime)
		: PSC(InPSC), LastUsedTime(InLastUsedTime)
	{

	}
};


USTRUCT()
struct FPSCPool
{
	GENERATED_BODY()

	//Collection of all currently allocated, free items ready to be grabbed for use.
	//TODO: Change this to a FIFO queue to get better usage. May need to make this whole class behave similar to TCircularQueue.
	UPROPERTY(transient)
	TArray<FPSCPoolElem> FreeElements;

#if ENABLE_PSC_POOL_DEBUGGING
	//Array of currently in flight components that will auto release.
	TArray<TWeakObjectPtr<UParticleSystemComponent>> InUseComponents_Auto;

	//Array of currently in flight components that need manual release.
	TArray<TWeakObjectPtr<UParticleSystemComponent>> InUseComponents_Manual;
	
	/** Keeping track of max in flight systems to help inform any future pre-population we do. */
	int32 MaxUsed = 0;
#endif

public:

	void Cleanup();

	/** Gets a PSC from the pool ready for use. */
	UParticleSystemComponent* Acquire(UWorld* World, UParticleSystem* Template, EPSCPoolMethod PoolingMethod);
	/** Returns a PSC to the pool. */
	void Reclaim(UParticleSystemComponent* PSC, const float CurrentTimeSeconds);

	/** Kills any components that have not been used since the passed KillTime. */
	void KillUnusedComponents(float KillTime, UParticleSystem* Template);

	int32 NumComponents() { return FreeElements.Num(); }
};

USTRUCT()
struct FWorldPSCPool
{
	GENERATED_BODY()

private:
	UPROPERTY()
	TMap<TObjectPtr<UParticleSystem>, FPSCPool> WorldParticleSystemPools;

	float LastParticleSytemPoolCleanTime;

	/** Cached world time last tick just to avoid us needing the world when reclaiming systems. */
	float CachedWorldTime;
public:

	ENGINE_API FWorldPSCPool();
	ENGINE_API ~FWorldPSCPool();

	ENGINE_API void Cleanup(UWorld* World);

	ENGINE_API UParticleSystemComponent* CreateWorldParticleSystem(UParticleSystem* Template, UWorld* World, EPSCPoolMethod PoolingMethod);

	/** Called when an in-use particle component is finished and wishes to be returned to the pool. */
	ENGINE_API void ReclaimWorldParticleSystem(UParticleSystemComponent* PSC);

	/** Dumps the current state of the pool to the log. */
	ENGINE_API void Dump();
};
