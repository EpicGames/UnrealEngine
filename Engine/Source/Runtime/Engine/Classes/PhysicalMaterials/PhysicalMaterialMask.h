// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "EngineDefines.h"
#include "PhysicsSettingsEnums.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "PhysicalMaterialMask.generated.h"

struct FPropertyChangedEvent;

/**
 * Physical material masks are used to map multiple physical materials to a single rendering material
 */
UCLASS(BlueprintType, Blueprintable, CollapseCategories, HideCategories = Object, MinimalAPI)
class UPhysicalMaterialMask : public UObject
{
	GENERATED_UCLASS_BODY()

	//
	// Object properties.
	//
#if WITH_EDITORONLY_DATA

	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** Mask input texture, square aspect ratio recommended. Recognized mask colors include: white, black, red, green, yellow, cyan, turquoise, and magenta. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = TextureSource)
	TObjectPtr<UTexture> MaskTexture;

#endif

	/** StaticMesh UV channel index to use when performing lookups with this mask. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TextureSource)
	int32 UVChannelIndex;

	/** The addressing mode to use for the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "X-axis Tiling Method"), Category = TextureSource)
	TEnumAsByte<enum TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Y-axis Tiling Method"), Category = TextureSource)
	TEnumAsByte<enum TextureAddress> AddressY;

	static ENGINE_API uint32 INVALID_MASK_INDEX;

public:

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void FinishDestroy() override;
	//~ End UObject Interface

	ENGINE_API virtual ~UPhysicalMaterialMask();
	ENGINE_API UPhysicalMaterialMask(FVTableHelper& Helper);

#if WITH_EDITOR
	// Helper method to set mask texture
	ENGINE_API void SetMaskTexture(UTexture* InMaskTexture, const FString& InTextureFilename);

	// Dump mask data to log display.
	ENGINE_API void DumpMaskData();
#endif // WITH_EDITOR

	TUniquePtr<FPhysicsMaterialMaskHandle> MaterialMaskHandle;

	/** Get the physics-interface derived version of this material */
	ENGINE_API FPhysicsMaterialMaskHandle& GetPhysicsMaterialMask();

	// Helper method to generate mask data used at runtime based on mask texture
	ENGINE_API void GenerateMaskData(TArray<uint32>& OutMaskData, int32& OutSizeX, int32& OutSizeY) const;

	// Helper method to query phys mat index at a given UV position in mask data
	static ENGINE_API uint32 GetPhysMatIndex(const TArray<uint32>& MaskData, int32 SizeX, int32 SizeY, int32 AddressX, int32 AddressY, float U, float V);
};

