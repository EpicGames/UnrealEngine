// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/LevelSetElem.h"
#include "PhysicsEngine/MLLevelSetElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SkinnedLevelSetElem.h"
#include "PhysicsEngine/SkinnedTriangleMeshElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "Async/Mutex.h"
#include "AggregateGeom.generated.h"

class FMaterialRenderProxy;

/** Container for an aggregate of collision shapes */
USTRUCT()
struct FKAggregateGeom
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Spheres", TitleProperty = "Name"))
	TArray<FKSphereElem> SphereElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Boxes", TitleProperty = "Name"))
	TArray<FKBoxElem> BoxElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Capsules", TitleProperty = "Name"))
	TArray<FKSphylElem> SphylElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Convex Elements", TitleProperty = "Name"))
	TArray<FKConvexElem> ConvexElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Tapered Capsules", TitleProperty = "Name"))
	TArray<FKTaperedCapsuleElem> TaperedCapsuleElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "Level Sets", TitleProperty = "Name"))
	TArray<FKLevelSetElem> LevelSetElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "(Experimental) Skinned Level Sets", TitleProperty = "Name"), Experimental)
	TArray<FKSkinnedLevelSetElem> SkinnedLevelSetElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "(Experimental) ML Level Sets", TitleProperty = "Name"), Experimental)
	TArray<FKMLLevelSetElem> MLLevelSetElems;

	UPROPERTY(EditAnywhere, editfixedsize, Category = "Aggregate Geometry", meta = (DisplayName = "(Experimental) Skinned Triangle Meshes", TitleProperty = "Name"), Experimental)
	TArray<FKSkinnedTriangleMeshElem> SkinnedTriangleMeshElems;

	FKAggregateGeom()
		: RenderInfoPtr(nullptr)
	{
	}

	FKAggregateGeom(const FKAggregateGeom& Other)
		: RenderInfoPtr(nullptr)
	{
		CloneAgg(Other);
	}

	const FKAggregateGeom& operator=(const FKAggregateGeom& Other)
	{
		FreeRenderInfo();
		CloneAgg(Other);
		return *this;
	}

	int32 GetElementCount() const
	{
		return SphereElems.Num() + SphylElems.Num() + BoxElems.Num() + ConvexElems.Num() + TaperedCapsuleElems.Num() + LevelSetElems.Num() + SkinnedLevelSetElems.Num() + MLLevelSetElems.Num() + SkinnedTriangleMeshElems.Num();
	}

	ENGINE_API int32 GetElementCount(EAggCollisionShape::Type Type) const;

	SIZE_T GetAllocatedSize() const { return SphereElems.GetAllocatedSize() + SphylElems.GetAllocatedSize() + BoxElems.GetAllocatedSize() + ConvexElems.GetAllocatedSize() + TaperedCapsuleElems.GetAllocatedSize() + LevelSetElems.GetAllocatedSize() + SkinnedLevelSetElems.GetAllocatedSize() + MLLevelSetElems.GetAllocatedSize() + SkinnedTriangleMeshElems.GetAllocatedSize(); }

	template <typename Callable>
	auto VisitShapeAndContainer(FKShapeElem& InElement, Callable&& InCallable)
	{
		switch(InElement.GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			return InCallable(static_cast<FKSphereElem&>(InElement), SphereElems);
		case EAggCollisionShape::Box:
			return InCallable(static_cast<FKBoxElem&>(InElement), BoxElems);
		case EAggCollisionShape::Sphyl:
			return InCallable(static_cast<FKSphylElem&>(InElement), SphylElems);
		case EAggCollisionShape::Convex:
			return InCallable(static_cast<FKConvexElem&>(InElement), ConvexElems);
		case EAggCollisionShape::TaperedCapsule:
			return InCallable(static_cast<FKTaperedCapsuleElem&>(InElement), TaperedCapsuleElems);
		case EAggCollisionShape::LevelSet:
			return InCallable(static_cast<FKLevelSetElem&>(InElement), LevelSetElems);
		case EAggCollisionShape::SkinnedLevelSet:
			return InCallable(static_cast<FKSkinnedLevelSetElem&>(InElement), SkinnedLevelSetElems);
		case EAggCollisionShape::MLLevelSet:
			return InCallable(static_cast<FKMLLevelSetElem&>(InElement), MLLevelSetElems);
		case EAggCollisionShape::SkinnedTriangleMesh:
			return InCallable(static_cast<FKSkinnedTriangleMeshElem&>(InElement), SkinnedTriangleMeshElems);
		default:
			check(false);
		}

		using RetType = TInvokeResult_T<Callable, FKSphereElem&, TArray<FKSphereElem>&>;
		if constexpr(!std::is_same_v<RetType, void>)
		{
			return RetType{};
		}
	}

	template <typename Callable>
	auto VisitShapeAndContainer(const FKShapeElem& InElement, Callable&& InCallable)
	{
		switch(InElement.GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			return InCallable(static_cast<const FKSphereElem&>(InElement), SphereElems);
		case EAggCollisionShape::Box:
			return InCallable(static_cast<const FKBoxElem&>(InElement), BoxElems);
		case EAggCollisionShape::Sphyl:
			return InCallable(static_cast<const FKSphylElem&>(InElement), SphylElems);
		case EAggCollisionShape::Convex:
			return InCallable(static_cast<const FKConvexElem&>(InElement), ConvexElems);
		case EAggCollisionShape::TaperedCapsule:
			return InCallable(static_cast<const FKTaperedCapsuleElem&>(InElement), TaperedCapsuleElems);
		case EAggCollisionShape::LevelSet:
			return InCallable(static_cast<const FKLevelSetElem&>(InElement), LevelSetElems);
		case EAggCollisionShape::SkinnedLevelSet:
			return InCallable(static_cast<const FKSkinnedLevelSetElem&>(InElement), SkinnedLevelSetElems);
		case EAggCollisionShape::MLLevelSet:
			return InCallable(static_cast<const FKMLLevelSetElem&>(InElement), MLLevelSetElems);
		case EAggCollisionShape::SkinnedTriangleMesh:
			return InCallable(static_cast<const FKSkinnedTriangleMeshElem&>(InElement), SkinnedTriangleMeshElems);
		default:
			check(false);
		}

		using RetType = TInvokeResult_T<Callable, const FKSphereElem&, TArray<FKSphereElem>&>;
		if constexpr(!std::is_same_v<RetType, void>)
		{
			return RetType{};
		}
	}

	// Add a typed element to the appropriate container
	template <typename T>
	void AddElement(const T& Elem) requires (TIsDerivedFrom<T, FKShapeElem>::Value || std::is_same_v<T, FKShapeElem>)
	{
		VisitShapeAndContainer(Elem, [] <typename ElemType> (const ElemType& TypedElem, TArray<ElemType>& Container)
		{
			Container.Add(TypedElem);
		});
	}

	template<typename CallableType>
	void VisitSimElements(CallableType&& InCallable) const
	{
		for(const FKSphereElem& Elem : SphereElems)
		{
			InCallable(Elem);
		}

		for(const FKBoxElem& Elem : BoxElems)
		{
			InCallable(Elem);
		}

		for(const FKSphylElem& Elem : SphylElems)
		{
			InCallable(Elem);
		}

		for(const FKConvexElem& Elem : ConvexElems)
		{
			InCallable(Elem);
		}
	}

	FKShapeElem* GetElement(const EAggCollisionShape::Type Type, const int32 Index)
	{
		switch (Type)
		{
		case EAggCollisionShape::Sphere:
			if (SphereElems.IsValidIndex(Index))
			{
				return &SphereElems[Index];
			}
			break;
		case EAggCollisionShape::Box:
			if (BoxElems.IsValidIndex(Index))
			{
				return &BoxElems[Index];
			}
			break;
		case EAggCollisionShape::Sphyl:
			if (SphylElems.IsValidIndex(Index))
			{
				return &SphylElems[Index];
			}
			break;
		case EAggCollisionShape::Convex:
			if (ConvexElems.IsValidIndex(Index))
			{
				return &ConvexElems[Index];
			}
			break;
		case EAggCollisionShape::TaperedCapsule:
			if (TaperedCapsuleElems.IsValidIndex(Index))
			{
				return &TaperedCapsuleElems[Index];
			}
			break;
		case EAggCollisionShape::LevelSet:
			if (LevelSetElems.IsValidIndex(Index))
			{
				return &LevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::SkinnedLevelSet:
			if (SkinnedLevelSetElems.IsValidIndex(Index))
			{
				return &SkinnedLevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::MLLevelSet:
			if (MLLevelSetElems.IsValidIndex(Index))
			{
				return &MLLevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::SkinnedTriangleMesh:
			if (SkinnedTriangleMeshElems.IsValidIndex(Index))
			{
				return &SkinnedTriangleMeshElems[Index];
			}
			break;
		default:
			// Failure is OK/returned/handled
			break;
		}
		return nullptr;
	}

	const FKShapeElem* GetElement(const EAggCollisionShape::Type Type, const int32 Index) const
	{
		switch (Type)
		{
		case EAggCollisionShape::Sphere:
			if (SphereElems.IsValidIndex(Index))
			{
				return &SphereElems[Index];
			}
			break;
		case EAggCollisionShape::Box:
			if (BoxElems.IsValidIndex(Index))
			{
				return &BoxElems[Index];
			}
			break;
		case EAggCollisionShape::Sphyl:
			if (SphylElems.IsValidIndex(Index))
			{
				return &SphylElems[Index];
			}
			break;
		case EAggCollisionShape::Convex:
			if (ConvexElems.IsValidIndex(Index))
			{
				return &ConvexElems[Index];
			}
			break;
		case EAggCollisionShape::TaperedCapsule:
			if (TaperedCapsuleElems.IsValidIndex(Index))
			{
				return &TaperedCapsuleElems[Index];
			}
			break;
		case EAggCollisionShape::LevelSet:
			if (LevelSetElems.IsValidIndex(Index))
			{
				return &LevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::SkinnedLevelSet:
			if (SkinnedLevelSetElems.IsValidIndex(Index))
			{
				return &SkinnedLevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::MLLevelSet:
			if (MLLevelSetElems.IsValidIndex(Index))
			{
				return &MLLevelSetElems[Index];
			}
			break;
		case EAggCollisionShape::SkinnedTriangleMesh:
			if (SkinnedTriangleMeshElems.IsValidIndex(Index))
			{
				return &SkinnedTriangleMeshElems[Index];
			}
			break;
		default:
			// Failure is OK/returned/handled
			break;
		}
		return nullptr;
	}

	FKShapeElem* GetElement(const int32 InIndex)
	{
		int Index = InIndex;
		if (SphereElems.IsValidIndex(Index))
		{
			return &SphereElems[Index];
		}
		Index -= SphereElems.Num();
		if (BoxElems.IsValidIndex(Index))
		{
			return &BoxElems[Index];
		}
		Index -= BoxElems.Num();
		if (SphylElems.IsValidIndex(Index))
		{
			return &SphylElems[Index];
		}
		Index -= SphylElems.Num();
		if (ConvexElems.IsValidIndex(Index))
		{
			return &ConvexElems[Index];
		}
		Index -= ConvexElems.Num();
		if (TaperedCapsuleElems.IsValidIndex(Index))
		{
			return &TaperedCapsuleElems[Index];
		}
		Index -= TaperedCapsuleElems.Num();
		if (LevelSetElems.IsValidIndex(Index))
		{
			return &LevelSetElems[Index];
		}
		Index -= LevelSetElems.Num();
		if (SkinnedLevelSetElems.IsValidIndex(Index))
		{
			return &SkinnedLevelSetElems[Index];
		}
		Index -= SkinnedLevelSetElems.Num();
		if (MLLevelSetElems.IsValidIndex(Index))
		{
			return &MLLevelSetElems[Index];
		}
		Index -= MLLevelSetElems.Num();
		if (SkinnedTriangleMeshElems.IsValidIndex(Index))
		{
			return &SkinnedTriangleMeshElems[Index];
		}
		ensure(false);
		return nullptr;
	}

	const FKShapeElem* GetElement(const int32 InIndex) const
	{
		int Index = InIndex;
		if (SphereElems.IsValidIndex(Index))
		{
			return &SphereElems[Index];
		}
		Index -= SphereElems.Num();
		if (BoxElems.IsValidIndex(Index))
		{
			return &BoxElems[Index];
		}
		Index -= BoxElems.Num();
		if (SphylElems.IsValidIndex(Index))
		{
			return &SphylElems[Index];
		}
		Index -= SphylElems.Num();
		if (ConvexElems.IsValidIndex(Index))
		{
			return &ConvexElems[Index];
		}
		Index -= ConvexElems.Num();
		if (TaperedCapsuleElems.IsValidIndex(Index))
		{
			return &TaperedCapsuleElems[Index];
		}
		Index -= TaperedCapsuleElems.Num();
		if (LevelSetElems.IsValidIndex(Index))
		{
			return &LevelSetElems[Index];
		}
		Index -= LevelSetElems.Num();
		if (SkinnedLevelSetElems.IsValidIndex(Index))
		{
			return &SkinnedLevelSetElems[Index];
		}
		Index -= SkinnedLevelSetElems.Num();
		if (MLLevelSetElems.IsValidIndex(Index))
		{
			return &MLLevelSetElems[Index];
		}
		Index -= MLLevelSetElems.Num();
		if (SkinnedTriangleMeshElems.IsValidIndex(Index))
		{
			return &SkinnedTriangleMeshElems[Index];
		}
		ensure(false);
		return nullptr;
	}

	const FKShapeElem* GetElementByName(const FName InName) const
	{
		if (const FKShapeElem* FoundSphereElem = GetElementByName<FKSphereElem>(MakeArrayView(SphereElems), InName))
		{
			return FoundSphereElem;
		}
		else if (const FKShapeElem* FoundBoxElem = GetElementByName<FKBoxElem>(MakeArrayView(BoxElems), InName))
		{
			return FoundBoxElem;
		}
		else if (const FKShapeElem* FoundSphylElem = GetElementByName<FKSphylElem>(MakeArrayView(SphylElems), InName))
		{
			return FoundSphylElem;
		}
		else if (const FKShapeElem* FoundConvexElem = GetElementByName<FKConvexElem>(MakeArrayView(ConvexElems), InName))
		{
			return FoundConvexElem;
		}
		else if (const FKShapeElem* FoundTaperedCapsuleElem = GetElementByName<FKTaperedCapsuleElem>(MakeArrayView(TaperedCapsuleElems), InName))
		{
			return FoundTaperedCapsuleElem;
		}
		else if (const FKShapeElem* FoundLevelSetElem = GetElementByName<FKLevelSetElem>(MakeArrayView(LevelSetElems), InName))
		{
			return FoundLevelSetElem;
		}
		else if (const FKShapeElem* FoundSkinnedLevelSetElem = GetElementByName<FKSkinnedLevelSetElem>(MakeArrayView(SkinnedLevelSetElems), InName))
		{
			return FoundSkinnedLevelSetElem;
		}
		else if (const FKShapeElem* FoundMLLevelSetElem = GetElementByName<FKMLLevelSetElem>(MakeArrayView(MLLevelSetElems), InName))
		{
			return FoundMLLevelSetElem;
		}
		else if (const FKShapeElem* FoundSkinnedTriangleMeshElem = GetElementByName<FKSkinnedTriangleMeshElem>(MakeArrayView(SkinnedTriangleMeshElems), InName))
		{
			return FoundSkinnedLevelSetElem;
		}

		return nullptr;
	}

	int32 GetElementIndexByName(const FName InName) const
	{
		int32 FoundIndex = GetElementIndexByName<FKSphereElem>(MakeArrayView(SphereElems), InName);
		int32 StartIndex = 0;
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += SphereElems.Num();

		FoundIndex = GetElementIndexByName<FKBoxElem>(MakeArrayView(BoxElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += BoxElems.Num();

		FoundIndex = GetElementIndexByName<FKSphylElem>(MakeArrayView(SphylElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += SphylElems.Num();

		FoundIndex = GetElementIndexByName<FKConvexElem>(MakeArrayView(ConvexElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += ConvexElems.Num();

		FoundIndex = GetElementIndexByName<FKTaperedCapsuleElem>(MakeArrayView(TaperedCapsuleElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += TaperedCapsuleElems.Num();

		FoundIndex = GetElementIndexByName<FKLevelSetElem>(MakeArrayView(LevelSetElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}
		StartIndex += LevelSetElems.Num();

		FoundIndex = GetElementIndexByName<FKSkinnedLevelSetElem>(MakeArrayView(SkinnedLevelSetElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}

		FoundIndex = GetElementIndexByName<FKMLLevelSetElem>(MakeArrayView(MLLevelSetElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}

		FoundIndex = GetElementIndexByName<FKSkinnedTriangleMeshElem>(MakeArrayView(SkinnedTriangleMeshElems), InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex + StartIndex;
		}

		return INDEX_NONE;
	}

#if WITH_EDITORONLY_DATA
	void EmptyImportedElements()
	{
		auto CleanUp = [](auto& Elems)
		{
			Elems.RemoveAllSwap([](const FKShapeElem& Elem)
			{
				return Elem.bIsGenerated == false;
			});
		};
		CleanUp(BoxElems);
		CleanUp(ConvexElems);
		CleanUp(SphylElems);
		CleanUp(SphereElems);
		CleanUp(TaperedCapsuleElems);
		CleanUp(LevelSetElems);
		CleanUp(SkinnedLevelSetElems);
		CleanUp(MLLevelSetElems);
		CleanUp(SkinnedTriangleMeshElems);

		FreeRenderInfo();
	}
#endif

	void EmptyElements()
	{
		BoxElems.Empty();
		ConvexElems.Empty();
		SphylElems.Empty();
		SphereElems.Empty();
		TaperedCapsuleElems.Empty();
		LevelSetElems.Empty();
		SkinnedLevelSetElems.Empty();
		MLLevelSetElems.Empty();
		SkinnedTriangleMeshElems.Empty();
		FreeRenderInfo();
	}

#if WITH_EDITORONLY_DATA
	ENGINE_API void FixupDeprecated(FArchive& Ar);
#endif

	ENGINE_API void GetAggGeom(const FTransform& Transform, const FColor Color, const FMaterialRenderProxy* MatInst, bool bPerHullColor, bool bDrawSolid, bool bOutputVelocity, int32 ViewIndex, class FMeshElementCollector& Collector) const;

	/** Release the RenderInfo (if its there) and safely clean up any resources. Call on the game thread. */
	ENGINE_API void FreeRenderInfo();

	ENGINE_API FBox CalcAABB(const FTransform& Transform) const;

	/**
	* Calculates a tight box-sphere bounds for the aggregate geometry; this is more expensive than CalcAABB
	* (tight meaning the sphere may be smaller than would be required to encompass the AABB, but all individual components lie within both the box and the sphere)
	*
	* @param Output The output box-sphere bounds calculated for this set of aggregate geometry
	*	@param LocalToWorld Transform
	*/
	ENGINE_API void CalcBoxSphereBounds(FBoxSphereBounds& Output, const FTransform& LocalToWorld) const;

	/** Returns the volume of this element */
	UE_DEPRECATED(5.1, "Changed to GetScaledVolume. Note that Volume calculation now includes non-uniform scale so values may have changed")
	ENGINE_API FVector::FReal GetVolume(const FVector& Scale3D) const;

	/** Returns the volume of this element */
	ENGINE_API FVector::FReal GetScaledVolume(const FVector& Scale3D) const;
	ENGINE_API FVector::FReal GetScaledVolumeForMassContribution(const FVector& Scale3D) const;

	ENGINE_API FGuid MakeDDCKey() const;

private:

	/** Helper function for safely copying instances */
	void CloneAgg(const FKAggregateGeom& Other)
	{
		SphereElems = Other.SphereElems;
		BoxElems = Other.BoxElems;
		SphylElems = Other.SphylElems;
		ConvexElems = Other.ConvexElems;
		TaperedCapsuleElems = Other.TaperedCapsuleElems;
		LevelSetElems = Other.LevelSetElems;
		SkinnedLevelSetElems = Other.SkinnedLevelSetElems;
		MLLevelSetElems = Other.MLLevelSetElems;
		SkinnedTriangleMeshElems = Other.SkinnedTriangleMeshElems;
	}

	template <class T>
	const FKShapeElem* GetElementByName(TArrayView<const T> Elements, const FName InName) const
	{
		const FKShapeElem* FoundElem = Elements.FindByPredicate(
			[InName](const T& Elem)
			{
				return InName == Elem.GetName();
			});
		return FoundElem;
	}

	template <class T>
	int32 GetElementIndexByName(TArrayView<const T> Elements, const FName InName) const
	{
		int32 FoundIndex = Elements.IndexOfByPredicate(
			[InName](const T& Elem)
			{
				return InName == Elem.GetName();
			});
		return FoundIndex;
	}

	// NOTE: RenderInfo is generated concurrently and lazily (hence being mutable)
	mutable std::atomic<class FKConvexGeomRenderInfo*> RenderInfoPtr;
	mutable UE::FMutex RenderInfoLock;
};
