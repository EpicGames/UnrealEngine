// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "EngineDefines.h"
#include "GameFramework/PlayerController.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PhysicsPublic.h"
#include "BodyInstanceCore.h"
#include "BodyInstance.generated.h"

#define UE_API ENGINE_API

class UBodySetup;
class UPhysicalMaterial;
class UPhysicalMaterialMask;
class UPrimitiveComponent;
class IPhysicsBodyInstanceOwner;
struct FBodyInstance;
struct FCollisionNotifyInfo;
struct FCollisionShape;
struct FConstraintInstance;
struct FPropertyChangedEvent;
struct FRigidBodyState;
struct FShapeData;

struct FShapeData;

ENGINE_API FArchive& operator<<(FArchive& Ar, FBodyInstance& BodyInst);
ENGINE_API int32 FillInlineShapeArray_AssumesLocked(PhysicsInterfaceTypes::FInlineShapeArray& Array, const FPhysicsActorHandle& Actor);

UENUM(BlueprintType)
namespace EDOFMode
{
	enum Type : int
	{
		/*Inherits the degrees of freedom from the project settings.*/
		Default,
		/*Specifies which axis to freeze rotation and movement along.*/
		SixDOF,
		/*Allows 2D movement along the Y-Z plane.*/
		YZPlane,
		/*Allows 2D movement along the X-Z plane.*/
		XZPlane,
		/*Allows 2D movement along the X-Y plane.*/
		XYPlane,
		/*Allows 2D movement along the plane of a given normal*/
		CustomPlane,
		/*No constraints.*/
		None
	};
}

struct FBodyInstance;
struct FBodyInstancePhysicsCreationInputs;

using FBodyInstancePhysicsCreationInputsPtr = TSharedPtr<FBodyInstancePhysicsCreationInputs, ESPMode::ThreadSafe>;

// Cache used when creating physics body instances
struct FBodyInstancePhysicsCreationInputs
{
	void GetReferencedObjects(TSet<UObject*>& OutReferencedObjects);

	// Cached body setup resolved on GT
	UBodySetup* ResolvedBodySetup = nullptr;

	// Cached simple material raw ptr
	UPhysicalMaterial* ResolvedSimpleMaterial = nullptr;

	// Cached complex materials and masks
	TArray<UPhysicalMaterial*> ResolvedComplexMaterials;
	TArray<FPhysicalMaterialMaskParams> ResolvedComplexMasks;

	// Pre-computed collision filter data (avoids OwnerComponent/SourceObject weak ptr access on worker thread)
	FBodyCollisionData ResolvedCollisionData;

	// Pre-computed initial velocity for InitDynamicProperties (avoids OwnerComponent dereference on worker thread)
	FVector ResolvedInitialLinearVelocity = FVector::ZeroVector;
	bool bResolvedComponentAwake = false;

	// Cached owner component pointer for use as a map key ONLY.
	// Do NOT dereference - use only for AddToComponentMaps / RegisterForCollisionEvents.
	UPrimitiveComponent* OwnerComponentMapKey = nullptr;
};

#ifndef CHAOS_DEBUG_NAME
#define CHAOS_DEBUG_NAME 0
#endif

#define USE_BODYINSTANCE_DEBUG_NAMES (!NO_LOGGING && CHAOS_DEBUG_NAME)

/** Helper struct to specify spawn behavior */
struct FInitBodySpawnParams
{
	ENGINE_API FInitBodySpawnParams(const UPrimitiveComponent* PrimComp);
	ENGINE_API FInitBodySpawnParams(bool bInStaticPhysics, bool bInPhysicsTypeDeterminesSimulation);

	/** Whether the created physics actor will be static */
	bool bStaticPhysics;

	/** Whether to use the BodySetup's PhysicsType to override if the instance simulates*/
	bool bPhysicsTypeDeterminesSimulation;

	/** An aggregate to place the body into */
	FPhysicsAggregateHandle Aggregate;
};

namespace Chaos
{
	struct FSerializedDataBuffer;
}

namespace Chaos::Private
{
	// Private base class necessary to keep friend access to FBodyInstance
	struct FInitBodiesHelperBaseInternal
	{
	protected:
		void SetBodyInstanceExternalCollisionProfileBodySetup(FBodyInstance* BodyInstance, UBodySetup* ExternalCollisionProfileBodySetup);
		void UpdateBodyInstanceSolverAsyncDeltaTime(FBodyInstance* BodyInstance);
	};
}

template <typename InAllocatorType = FDefaultAllocator>
struct TInitBodiesHelperBase : public Chaos::Private::FInitBodiesHelperBaseInternal
{
public:
	ENGINE_API TInitBodiesHelperBase(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms,
		class UBodySetup* InBodySetup, class UPrimitiveComponent* InPrimitiveComp, FPhysScene* InRBScene,
		const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr);
	
	ENGINE_API TInitBodiesHelperBase(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms,
		class UBodySetup* InBodySetup, class UPrimitiveComponent* InComponent, UObject* InSourceObject, FPhysScene* InRBScene,
		const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr);

	TInitBodiesHelperBase(const TInitBodiesHelperBase& InHelper) = delete;
	TInitBodiesHelperBase(TInitBodiesHelperBase&& InHelper) = delete;
	TInitBodiesHelperBase& operator=(const TInitBodiesHelperBase& InHelper) = delete;
	TInitBodiesHelperBase& operator=(TInitBodiesHelperBase&& InHelper) = delete;

	bool IsStatic() const { return bStatic; }

	//The arguments passed into InitBodies
	TArray<FBodyInstance*, InAllocatorType>& Bodies;
	TArray<FTransform, InAllocatorType>& Transforms;

	class UBodySetup* BodySetup;
	class UPrimitiveComponent* PrimitiveComp;
	IPhysicsBodyInstanceOwner* BodyInstanceOwner;
	UObject* SourceObject;
	FPhysScene* PhysScene;
	FPhysicsAggregateHandle Aggregate;

#if USE_BODYINSTANCE_DEBUG_NAMES
	TSharedPtr<FString, ESPMode::ThreadSafe> DebugName;
	TSharedPtr<TArray<ANSICHAR>> PhysXName; // Get rid of ANSICHAR in physics
#endif

	//The constants shared between PhysX and Box2D
	bool bStatic;
	bool bInstanceSimulatePhysics;
	float InstanceBlendWeight;

	const USkeletalMeshComponent* SkelMeshComp;

	const FInitBodySpawnParams& SpawnParams;

	bool DisableQueryOnlyActors;

	// Return to actor ref
	void CreateActor_AssumesLocked(FBodyInstance* Instance, const FTransform& Transform) const;
	bool CreateShapes_AssumesLocked(FBodyInstance* Instance) const;

	// Takes actor ref arrays.
	// #PHYS2 this used to return arrays of low-level physics bodies, which would be added to scene in InitBodies. Should it still do that, rather then later iterate over BodyInstances to get phys actor refs?
	bool CreateShapesAndActors();
	UE_API void InitBodies();

protected:
	void UpdateSimulatingAndBlendWeight();

};

using FInitBodiesHelperBase = TInitBodiesHelperBase<FDefaultAllocator>;

template <bool bCompileStatic, typename InAllocatorType = FDefaultAllocator>
struct FInitBodiesHelper : public TInitBodiesHelperBase<InAllocatorType>
{
	UE_API FInitBodiesHelper(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms, class UBodySetup* InBodySetup,
		class UPrimitiveComponent* InPrimitiveComp, FPhysScene* InRBScene,
		const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr);

	FInitBodiesHelper(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms, class UBodySetup* InBodySetup,
		class UPrimitiveComponent* InComponent, class UObject* InSourceObject, FPhysScene* InRBScene,
		const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr)
		: TInitBodiesHelperBase<InAllocatorType>(InBodies, InTransforms, InBodySetup, InComponent, InSourceObject, InRBScene, InSpawnParams, InAggregate, InBodyInstanceOwner)
	{
		//Compute all the needed constants
		this->bStatic = bCompileStatic || this->SpawnParams.bStaticPhysics;
		this->SkelMeshComp = bCompileStatic ? nullptr : Cast<USkeletalMeshComponent>(this->PrimitiveComp);
		if (this->SpawnParams.bPhysicsTypeDeterminesSimulation)
		{
			this->UpdateSimulatingAndBlendWeight();
		}
	}
};

template <bool bCompileStatic>
struct FInitBodiesHelperWithData : public FInitBodiesHelperBase
{
	FInitBodiesHelperWithData() { check(false); }
	FInitBodiesHelperWithData(TArray<FBodyInstance*>&& InBodies, TArray<FTransform>&& InTransforms, class UBodySetup* InBodySetup, class UPrimitiveComponent* InPrimitiveComp, FPhysScene* InRBScene, const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr)
	: FInitBodiesHelperBase(OwnedBodies, OwnedTransforms, InBodySetup, InPrimitiveComp, InRBScene, InSpawnParams, InAggregate, InBodyInstanceOwner), OwnedBodies(MoveTemp(InBodies)), OwnedTransforms(MoveTemp(InTransforms))
	{
		//Compute all the needed constants
		bStatic = bCompileStatic || SpawnParams.bStaticPhysics;
		SkelMeshComp = bCompileStatic ? nullptr : Cast<USkeletalMeshComponent>(PrimitiveComp);
		if(SpawnParams.bPhysicsTypeDeterminesSimulation)
		{
			this->UpdateSimulatingAndBlendWeight();
		}
	}

	FInitBodiesHelperWithData(const FInitBodiesHelperWithData& InHelper)
	: FInitBodiesHelperBase(OwnedBodies, OwnedTransforms, InHelper.BodySetup, InHelper.PrimitiveComp, InHelper.PhysScene, InHelper.SpawnParams, InHelper.Aggregate), OwnedBodies(InHelper.OwnedBodies), OwnedTransforms(InHelper.OwnedTransforms, InHelper.BodyInstanceOwner)
	{
		ensure(false);
	}

	FInitBodiesHelperWithData(FInitBodiesHelperWithData&& InHelper)
	: FInitBodiesHelperBase(OwnedBodies, OwnedTransforms, InHelper.BodySetup, InHelper.PrimitiveComp, InHelper.PhysScene, InHelper.SpawnParams, InHelper.Aggregate), OwnedBodies(MoveTemp(InHelper.OwnedBodies)), OwnedTransforms(MoveTemp(InHelper.OwnedTransforms))
	{
		//Compute all the needed constants
		bStatic = bCompileStatic || SpawnParams.bStaticPhysics;
		SkelMeshComp = bCompileStatic ? nullptr : Cast<USkeletalMeshComponent>(PrimitiveComp);
		if (SpawnParams.bPhysicsTypeDeterminesSimulation)
		{
			this->UpdateSimulatingAndBlendWeight();
		}
	}

	FInitBodiesHelperWithData& operator=(const FInitBodiesHelperWithData& InHelper) = delete;
	FInitBodiesHelperWithData& operator=(FInitBodiesHelperWithData&& InHelper) = delete;

	TArray<FBodyInstance*> OwnedBodies;
	TArray<FTransform> OwnedTransforms;
};

USTRUCT()
struct FCollisionResponse
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FCollisionResponse();
	ENGINE_API FCollisionResponse(ECollisionResponse DefaultResponse);

	/** Set the response of a particular channel in the structure. */
	ENGINE_API bool SetResponse(ECollisionChannel Channel, ECollisionResponse NewResponse);

	/** Set all channels to the specified response */
	ENGINE_API bool SetAllChannels(ECollisionResponse NewResponse);

	/** Replace the channels matching the old response with the new response */
	ENGINE_API bool ReplaceChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse);

	/** Returns the response set on the specified channel */
	ECollisionResponse GetResponse(ECollisionChannel Channel) const { return ResponseToChannels.GetResponse(Channel); }
	const FCollisionResponseContainer& GetResponseContainer() const { return ResponseToChannels; }

	/** Set all channels from ChannelResponse Array **/
	ENGINE_API bool SetCollisionResponseContainer(const FCollisionResponseContainer& InResponseToChannels);
	ENGINE_API void SetResponsesArray(const TArray<FResponseChannel>& InChannelResponses);
	ENGINE_API void UpdateResponseContainerFromArray();

	ENGINE_API bool operator==(const FCollisionResponse& Other) const;
	bool operator!=(const FCollisionResponse& Other) const
	{
		return !(*this == Other);
	}

private:

#if 1// @hack until PostLoad is disabled for CDO of BP - WITH_EDITOR
	/** Anything that updates array does not have to be done outside of editor
	 *	because we won't save outside of editor
	 *	During runtime, important data is ResponseToChannel
	 *	That is the data we care during runtime. But that data won't be saved.
	 */
	ENGINE_API bool RemoveReponseFromArray(ECollisionChannel Channel);
	ENGINE_API bool AddReponseToArray(ECollisionChannel Channel, ECollisionResponse Response);
	ENGINE_API void UpdateArrayFromResponseContainer();
#endif

	/** Types of objects that this physics objects will collide with. */
	// we have to still load them until resave
	UPROPERTY(transient)
	FCollisionResponseContainer ResponseToChannels;

	/** Custom Channels for Responses */
	UPROPERTY(EditAnywhere, Category = Custom)
	TArray<FResponseChannel> ResponseArray;

	friend struct FBodyInstance;
};

template<>
struct TStructOpsTypeTraits<FCollisionResponse> : public TStructOpsTypeTraitsBase2<FCollisionResponse>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

enum class BodyInstanceSceneState : uint8
{
	NotAdded,
	AwaitingAdd,
	Added,
	AwaitingRemove,
	Removed
};

namespace Chaos
{
	class FRigidBodyHandle_Internal;
}

USTRUCT(BlueprintType)
struct FBodyInstanceAsyncPhysicsTickHandle
{
	GENERATED_BODY()
	FPhysicsActorHandle Proxy = nullptr;

	ENGINE_API Chaos::FRigidBodyHandle_Internal* operator->();

	ENGINE_API bool IsValid() const;
	operator bool() const { return IsValid(); }
};

/** Container for a physics representation of an object */
USTRUCT(BlueprintType)
struct FBodyInstance : public FBodyInstanceCore
{
	GENERATED_USTRUCT_BODY()

	/** 
	 *	Index of this BodyInstance within the SkeletalMeshComponent/PhysicsAsset. 
	 *	Is INDEX_NONE if a single body component
	 */
	int32 InstanceBodyIndex;

	/** When we are a body within a SkeletalMeshComponent, we cache the index of the bone we represent, to speed up sync'ing physics to anim. */
	int16 InstanceBoneIndex;

	/** Override the project-settings position iteration count for this body (if bOverrideIterationCounts is set). Increasing this will be more CPU intensive, but better stabilized. 0 will lead to unresolved penetration and no static friction. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Physics, meta = (editcondition = "bOverrideIterationCounts", InlineEditConditionToggle, ClampMin = "0", UIMin = "0"))
	uint8 PositionSolverIterationCount;

	/** Override the project-settings velocity iteration count for this body (if bOverrideIterationCounts is set). Increasing this will be more CPU intensive, but better stabilized. 0 will lead to unresolved restitution and lower quality dynamic friction. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideIterationCounts", InlineEditConditionToggle, ClampMin = "0", UIMin = "0"))
	uint8 VelocitySolverIterationCount;

	/** Override the project-settings projection iteration count for this body (if bOverrideIterationCounts is set). Usually 0 or 1. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideIterationCounts", InlineEditConditionToggle, ClampMin = "0", UIMin = "0"))
	uint8 ProjectionSolverIterationCount;

private:
	/** Enum indicating what type of object this should be considered as when it moves */
	UPROPERTY(EditAnywhere, Category=Custom)
	TEnumAsByte<enum ECollisionChannel> ObjectType;


	/** Extra mask for filtering. Look at declaration for logic */
	FMaskFilter MaskFilter;

	/**
	* Type of collision enabled.
	* 
	*	No Collision      : Will not create any representation in the physics engine. Cannot be used for spatial queries (raycasts, sweeps, overlaps) or simulation (rigid body, constraints). Best performance possible (especially for moving objects)
	*	Query Only        : Only used for spatial queries (raycasts, sweeps, and overlaps). Cannot be used for simulation (rigid body, constraints). Useful for character movement and things that do not need physical simulation. Performance gains by keeping data out of simulation tree.
	*	Physics Only      : Only used only for physics simulation (rigid body, constraints). Cannot be used for spatial queries (raycasts, sweeps, overlaps). Useful for jiggly bits on characters that do not need per bone detection. Performance gains by keeping data out of query tree
	*	Collision Enabled : Can be used for both spatial queries (raycasts, sweeps, overlaps) and simulation (rigid body, constraints).
	*/
	UPROPERTY(EditAnywhere, Category=Custom)
	TEnumAsByte<ECollisionEnabled::Type> CollisionEnabled;

	/** When per-shape collision is changed at runtime, state is stored in an optional array of per-shape collision state.
	*	Before bShapeCollisionEnabledIsSet state is true, collision values from the BodySetup's AggGeom are used in GetShapeCollisionEnabled.
	*/
	TArray<TEnumAsByte<ECollisionEnabled::Type>> ShapeCollisionEnabled;

	/** When per-shape collision responses are changed at runtime, state is stored in an optional array of per-shape
	*	collision response settings. If bShapeCollisionResponsesIsSet is false, the base body instance's CollisionResponses member is used for all shapes.
	*/
	TArray<TPair<int32, FCollisionResponse>> ShapeCollisionResponses;

public:
	// Current state of the physics body for tracking deferred addition and removal.
	BodyInstanceSceneState CurrentSceneState;

	/** The set of values used in considering when put this body to sleep. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics)
	ESleepFamily SleepFamily;

	/** Locks physical movement along specified axis.*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Mode"))
	TEnumAsByte<EDOFMode::Type> DOFMode;

	/** If true Continuous Collision Detection (CCD) will be used for this component */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Collision)
	uint8 bUseCCD : 1;

private:

	uint8 bShapeCollisionEnabledIsSet : 1;
	uint8 bShapeCollisionResponsesIsSet : 1;

	/** [EXPERIMENTAL] If true Motion-Aware Collision Detection (MACD) will be used for this component */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Collision)
	uint8 bUseMACD : 1;

	/** [EXPERIMENTAL] If true Partial Island Sleeping is permitted for the entire island this body is in */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Physics)
	uint8 bAllowPartialIslandSleep : 1;

public:
	/** If true ignore analytic collisions and treat objects as a general implicit surface */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Collision)
	uint8 bIgnoreAnalyticCollisions : 1;

	/**	Should 'Hit' events fire when this object collides during physics simulation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision, meta = (DisplayName = "Simulation Generates Hit Events"))
	uint8 bNotifyRigidBodyCollision : 1;

	/**	Enable contact modification. Assumes custom contact modification has been provided (see FPhysXContactModifyCallback) */
	uint8 bContactModification : 1;

	/**
	 * Remove unnecessary edge collisions to allow smooth sliding over surfaces composed of multiple actors/components.
	 * This is fairly expensive and should only be enabled on hero objects. 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Collision)
	uint8 bSmoothEdgeCollisions : 1;

	/////////
	// SIM SETTINGS

	/** When a Locked Axis Mode is selected, will lock translation on the specified axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta=(DisplayName = "Lock Axis Translation"))
	uint8 bLockTranslation : 1;
	
	/** When a Locked Axis Mode is selected, will lock rotation to the specified axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta=(DisplayName = "Lock Axis Rotation"))
	uint8 bLockRotation : 1;

	/** Lock translation along the X-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "X"))
	uint8 bLockXTranslation : 1;

	/** Lock translation along the Y-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Y"))
	uint8 bLockYTranslation : 1;

	/** Lock translation along the Z-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Z"))
	uint8 bLockZTranslation : 1;

	/** Lock rotation about the X-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "X"))
	uint8 bLockXRotation : 1;

	/** Lock rotation about the Y-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Y"))
	uint8 bLockYRotation : 1;

	/** Lock rotation about the Z-axis*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Z"))
	uint8 bLockZRotation : 1;

	/** Override the default max angular velocity */
	UPROPERTY(EditAnywhere, Category = Physics, meta = (editcondition = "bSimulatePhysics", InlineEditConditionToggle))
	uint8 bOverrideMaxAngularVelocity : 1;


	/** 
	 * @HACK:
	 * These are ONLY used when the 'p.EnableDynamicPerBodyFilterHacks' CVar is set (disabled by default).
	 * Some games need to dynamically modify collision per skeletal body. These provide game code a way to 
	 * do that, until we're able to refactor how skeletal bodies work.
	 */
	uint8 bHACK_DisableCollisionResponse : 1;
	/* By default, an owning skel mesh component will override the body's collision filter. This will disable that behavior. */
	uint8 bHACK_DisableSkelComponentFilterOverriding : 1;

protected:

	/** Whether this body instance has its own custom MaxDepenetrationVelocity*/
	UPROPERTY(EditAnywhere, Category = Physics, meta=(InlineEditConditionToggle))
	uint8 bOverrideMaxDepenetrationVelocity : 1;

	/** Whether this instance of the object has its own custom walkable slope override setting. */
	UPROPERTY(EditAnywhere, Category = Physics, meta = (InlineEditConditionToggle))
	uint8 bOverrideWalkableSlopeOnInstance : 1;

	/** Enable per-body iterations counts */
	UPROPERTY(EditAnywhere, Category = Physics, meta = (InlineEditConditionToggle))
	uint8 bOverrideIterationCounts : 1;

	/** 
	 * Internal flag to allow us to quickly check whether we should interpolate when substepping 
	 * e.g. kinematic bodies that are QueryOnly do not need to interpolate as we will not be querying them
	 * at a sub-position.
	 * This is complicated by welding, where multiple the CollisionEnabled flag of the root must be considered.
	 */
	UPROPERTY()
	uint8 bInterpolateWhenSubStepping : 1;

	/** Whether we are pending a collision profile setup */
	uint8 bPendingCollisionProfileSetup : 1;

	/** 
	 * @brief Enable automatic inertia conditioning to stabilize constraints.
	 * 
	 * Inertia conditioning increases inertia when an object is long and thin and also when it has joints that are outside the
	 * collision shapes of the body. Increasing the inertia reduces the amount of rotation applied at joints which helps stabilize
	 * joint chains, especially when bodies are small. In principle you can get the same behaviour by setting the InertiaTensorScale
	 * appropriately, but this takes some of the guesswork out of it.
	 * 
	 * @note This only changes the inertia used in the low-level solver. That inertia is not visible to the BodyInstance
	 * which will still report the inertia calculated from the mass, shapes, and InertiaTensorScale.
	 * 
	 * @note When enabled, the effective inertia depends on the joints attached to the body so the inertia will change when
	 * joints are added or removed (automatically - no user action required).
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics)
	uint8 bInertiaConditioning : 1;

	/** If set to true, this body will treat bodies that do not have the flag set as having infinite mass */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics)
	uint8 bOneWayInteraction : 1;

public:
	/** Set the desired delta time for the body. **/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics)
	uint8 bOverrideSolverAsyncDeltaTime : 1;

	/** Override value for physics solver async delta time.  With multiple actors specifying this, the solver will use the smallest delta time **/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideSolverAsyncDeltaTime"))
	float SolverAsyncDeltaTime;

	float GetSolverAsyncDeltaTime() const { return SolverAsyncDeltaTime; }
	bool IsSolverAsyncDeltaTimeSet() const { return bOverrideSolverAsyncDeltaTime && SolverAsyncDeltaTime > 0.0; }

	void SetSolverAsyncDeltaTime(const float NewSolverAsyncDeltaTime);

private:
	void UpdateSolverAsyncDeltaTime();

	/** Collision Profile Name **/
	UPROPERTY(EditAnywhere, Category=Custom)
	FName CollisionProfileName;

public:
	/** Current scale of physics - used to know when and how physics must be rescaled to match current transform of OwnerComponent. */
	FVector Scale3D;

	/////////
	// COLLISION SETTINGS

#if WITH_EDITORONLY_DATA
	/** Types of objects that this physics objects will collide with. */
	UPROPERTY() 
	struct FCollisionResponseContainer ResponseToChannels_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

private:
	/** Custom Channels for Responses*/
	UPROPERTY(EditAnywhere, Category = Custom)
	struct FCollisionResponse CollisionResponses;

protected:
	/** 
	 * The maximum velocity used to depenetrate this object from others when spawned or teleported with initial overlaps (does not affect overlaps as a result of normal movement).
	 * A value of zero will allow objects that are spawned overlapping to go to sleep without moving rather than pop out of each other. E.g., use zero if you spawn dynamic rocks 
	 * partially embedded in the ground and want them to be interactive but not pop out of the ground when touched.
	 * A negative value is equivalent to bOverrideMaxDepenetrationVelocity = false, meaning use the project setting.
	 * This overrides the CollisionInitialOverlapDepenetrationVelocity project setting on a per-body basis (and not the MaxDepenetrationVelocity solver setting that will be deprecated).
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideMaxDepenetrationVelocity", ClampMin = "0.0", UIMin = "0.0"))
	float MaxDepenetrationVelocity;

	/**Mass of the body in KG. By default we compute this based on physical material and mass scale.
	*@see bOverrideMass to set this directly */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideMass", ClampMin = "0.001", UIMin = "0.001", DisplayName = "Mass (kg)"))
	float MassInKgOverride;

	/** The body setup holding the default body instance and its collision profile. */
	TWeakObjectPtr<UBodySetup> ExternalCollisionProfileBodySetup;

	/** Update the substepping interpolation flag */
	ENGINE_API void UpdateInterpolateWhenSubStepping();

public:

	/** Whether we should interpolate when substepping. @see bInterpolateWhenSubStepping */
	bool ShouldInterpolateWhenSubStepping() const { return bInterpolateWhenSubStepping; }

	/** Returns the mass override. See MassInKgOverride for documentation */
	float GetMassOverride() const { return MassInKgOverride; }

	bool IsUsingMACD() const { return bUseMACD; }

	bool IsPartialIslandSleepAllowed() const { return bAllowPartialIslandSleep; }

	/** Sets the mass override */
	ENGINE_API void SetMassOverride(float MassInKG, bool bNewOverrideMass = true);

	ENGINE_API bool GetRigidBodyState(FRigidBodyState& OutState);

	/** 'Drag' force added to reduce linear movement */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	float LinearDamping;

	/** 'Drag' force added to reduce angular movement */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	float AngularDamping;

	/** Locks physical movement along a custom plane for a given normal.*/
	UPROPERTY(EditAnywhere, Category = Physics, meta = (DisplayName = "Plane Normal"))
	FVector CustomDOFPlaneNormal;

	/** User specified offset for this object's Center of Mass. The offset is defined in bone space and will be added to the calculated location. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics, meta = (DisplayName = "Center Of Mass Offset"))
	FVector COMNudge;

	/** Per-instance scaling of mass */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics, meta = (ClampMin = "0.001", UIMin = "0.001"))
	float MassScale;

	/** What gravity group the BI should use, which determines rate of acceleration */
	UPROPERTY(EditAnywhere,BlueprintReadOnly,Category = Physics, meta = (ClampMin = "0", UIMin = "0", UIMax = "7"))
	uint8 GravityGroupIndex;

	/** Per-instance scaling of inertia (bigger number means  it'll be harder to rotate) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Physics)
	FVector InertiaTensorScale;

public:
	/** Use the collision profile found in the given BodySetup's default BodyInstance */
	ENGINE_API void UseExternalCollisionProfile(UBodySetup* InExternalCollisionProfileBodySetup);

	ENGINE_API void ClearExternalCollisionProfile();

	/** Locks physical movement along axis. */
	ENGINE_API void SetDOFLock(EDOFMode::Type NewDOFMode);

	ENGINE_API FVector GetLockedAxis() const;
	ENGINE_API void CreateDOFLock();

	static ENGINE_API EDOFMode::Type ResolveDOFMode(EDOFMode::Type DOFMode);

	/** Constraint used to allow for easy DOF setup per bodyinstance */
	FConstraintInstance* DOFConstraint;

	/** The parent body that we are welded to*/
	FBodyInstance* WeldParent;

protected:

	/**
	* Custom walkable slope override setting for this instance.
	* @see GetWalkableSlopeOverride(), SetWalkableSlopeOverride()
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideWalkableSlopeOnInstance"))
	struct FWalkableSlopeOverride WalkableSlopeOverride;

	/**	Allows you to override the PhysicalMaterial to use for simple collision on this body. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Collision)
	TObjectPtr<class UPhysicalMaterial> PhysMaterialOverride;

public:
	/** The maximum angular velocity for this instance [degrees/s]*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics, meta = (editcondition = "bOverrideMaxAngularVelocity"))
	float MaxAngularVelocity;


	/** If the SleepFamily is set to custom, multiply the natural sleep threshold by this amount. A higher number will cause the body to sleep sooner. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Physics)
	float CustomSleepThresholdMultiplier;

	/** Stabilization factor for this body if Physics stabilization is enabled. A higher number will cause more aggressive stabilization at the risk of loss of momentum at low speeds. A value of 0 will disable stabilization for this body.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Physics)
	float StabilizationThresholdMultiplier;

	/**	Influence of rigid body physics (blending) on the mesh's pose (0.0 == use only animation, 1.0 == use only physics) */
	/** Provide appropriate interface for doing this instead of allowing BlueprintReadWrite **/
	UPROPERTY()
	float PhysicsBlendWeight;

public:

	ENGINE_API UBodySetup* GetBodySetup() const;

	ENGINE_API FPhysicsActorHandle GetPhysicsActor() const;
	ENGINE_API FPhysicsActorHandle GetPhysicsActorWeldRoot() const;
	ENGINE_API void SetPhysicsActor(FPhysicsActorHandle InHandle);
	ENGINE_API void ReleasePhysicsActor();

	// TODO_CHAOSAPI: Deprecate ActorHandle Ref API
	ENGINE_API FPhysicsActorHandle& GetPhysicsActorHandle();
	ENGINE_API const FPhysicsActorHandle& GetPhysicsActorHandle() const;
	ENGINE_API const FPhysicsActorHandle& GetActorReferenceWithWelding() const;
	inline void SetPhysicsActorHandle(FPhysicsActorHandle InHandle) { SetPhysicsActor(InHandle); }

	// TODO_CHAOSAPI: Deprecate public ActorHandle
	// Internal physics representation of our body instance
	FPhysicsActorHandle ActorHandle;

public:
	FBodyInstanceAsyncPhysicsTickHandle GetBodyInstanceAsyncPhysicsTickHandle() const { return FBodyInstanceAsyncPhysicsTickHandle{ ActorHandle }; }

#if USE_BODYINSTANCE_DEBUG_NAMES
	TSharedPtr<TArray<ANSICHAR>> CharDebugName;
#endif

	/** PrimitiveComponent containing this body, if relevant.   */
	TWeakObjectPtr<class UPrimitiveComponent> OwnerComponent;

	/** Owner containing this body (when not a UPrimitiveComponent). */
	IPhysicsBodyInstanceOwner* BodyInstanceOwner;

	ENGINE_API UWorld* GetWorld() const;

	/** Source object for this body. If OwnerComponent is not null, this will typically be the owning actor. */
	TWeakObjectPtr<UObject> SourceObject;

	/** Constructor **/
	ENGINE_API FBodyInstance();
	ENGINE_API ~FBodyInstance();

	/**  
	 * Update profile data if required
	 * 
	 * @param : bVerifyProfile - if true, it makes sure it has correct set up with current profile, if false, it overwrites from profile data
	 *								(for backward compatibility)
	 * 
	 **/
	ENGINE_API void LoadProfileData(bool bVerifyProfile);

	void InitBody(UBodySetup* Setup, const FTransform& Transform, UPrimitiveComponent* PrimComp, FPhysScene* InRBScene)
	{
		InitBody(Setup, Transform, PrimComp, InRBScene, FInitBodySpawnParams(PrimComp));
	}


	/** Initialise a single rigid body (this FBodyInstance) for the given body setup
	*	@param Setup The setup to use to create the body
	*	@param Transform Transform of the body
	*	@param PrimComp The owning component
	*	@param InRBScene The physics scene to place the body into
	*	@param SpawnParams The parameters for determining certain spawn behavior
	*	@param BodyInstanceOwner The owning object of the body instance (optional and only used if PrimComp is not set)
	*/
	ENGINE_API void InitBody(UBodySetup* Setup, const FTransform& Transform, UPrimitiveComponent* PrimComp, FPhysScene* InRBScene, const FInitBodySpawnParams& SpawnParams, IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr);

	/**
	 * Initialise a single rigid body (this FBodyInstance) for the given body setup
	 *  @param Setup The setup to use to create the body
	 *  @param Transform Transform of the body
	 *  @param PrimComp The owning component
	 *  @param SourceObject The source object. If null and PrimComp is not null, it will be set
	 *   to the owning actor.
	 *  @param InRBScene The physics scene to place the body into
	 *  @param SpawnParams The parameters for determining certain spawn behavior
	 *	@param BodyInstanceOwner The owning object of the body instance (optional and only used if PrimComp is not set)
	 */
	ENGINE_API void InitBody(UBodySetup* Setup, const FTransform& Transform, 
		UPrimitiveComponent* PrimComp, UObject* SourceObject, 
		FPhysScene* InRBScene, const FInitBodySpawnParams& SpawnParams, 
		IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr);

	/** Validate a body transform, outputting debug info
	 *	@param Transform Transform to debug
	 *	@param DebugName Name of the instance for logging
	 *	@param Setup Body setup for this instance
	 */
	static ENGINE_API bool ValidateTransform(const FTransform &Transform, const FString& DebugName, const UBodySetup* Setup);

	/** Standalone path to batch initialize large amounts of static bodies, which will be deferred till the next scene update for fast scene addition.
	 *	@param Bodies
	 *	@param Transforms
	 *	@param BodySetup
	 *	@param PrimitiveComp
	 *	@param InRBScene
	 */
	static ENGINE_API void InitStaticBodies(const TArray<FBodyInstance*>& Bodies, const TArray<FTransform>& Transforms, UBodySetup* BodySetup, class UPrimitiveComponent* PrimitiveComp, FPhysScene* InRBScene);
	
	/**
	 * Standalone path to batch initialize large amounts of static bodies.
	 * Note that this function will steal Bodies and Transforms arrays.
	 *	@param Bodies The list of BodyInstances to initialize.
	 *	@param Transforms The list of BodyInstance transform
	 *	@param BodySetup The BodySetup to use
	 *	@param PrimitiveComp The associated primitive component
	 *	@param InRBScene The associated physics scene
	 *	@param BodyInstanceOwner The associated owning object of the body instance (optional and only used if PrimitiveComp is not set)
	 */
	static ENGINE_API void InitStaticBodies(TArray<FBodyInstance*>&& Bodies, TArray<FTransform>&& Transforms, UBodySetup* BodySetup, class UPrimitiveComponent* PrimitiveComp, FPhysScene* InRBScene, IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr);

	/** Get the scene that owns this body. */
	ENGINE_API FPhysScene* GetPhysicsScene();
	ENGINE_API const FPhysScene* GetPhysicsScene() const;

	/** Initialise dynamic properties for this instance when using physics - this must be done after scene addition.
	 *  Note: This function is not thread safe. Make sure to obtain the appropriate physics scene locks before calling this function
	 */
	ENGINE_API void InitDynamicProperties_AssumesLocked();

	/** Build the sim and query filter data (for simple and complex shapes) based on the settings of this BodyInstance (and its associated BodySetup)  */
	ENGINE_API void BuildBodyFilterData(FBodyCollisionFilterData& OutFilterData, const int32 ShapeIndex = INDEX_NONE) const;

	/** Build the flags to control which types of collision (sim and query) shapes owned by this BodyInstance should have. */
	static ENGINE_API void BuildBodyCollisionFlags(FBodyCollisionFlags& OutFlags, ECollisionEnabled::Type UseCollisionEnabled, bool bUseComplexAsSimple);

	/** 
	 *	Utility to get all the shapes from a FBodyInstance 
	 *	NOTE: This function is not thread safe. You must hold the physics scene lock while calling it and reading/writing from the shapes
	 */
	ENGINE_API int32 GetAllShapes_AssumesLocked(TArray<FPhysicsShapeHandle>& OutShapes) const;

	/**
	 * Terminates the body, releasing resources.
	 * Can be called from a worker thread IF:
	 * - UnRegisterForCollisionEvents was already called on the game thread (or bNotifyRigidBodyCollision is false)
	 * - The caller has exclusive ownership of this FBodyInstance (e.g., ISM bodies moved to async payload)
	 * @param bNeverDeferRelease In some cases orphaned actors can have their internal release deferred. If this isn't desired this flag will override that behavior
	 */
	ENGINE_API void TermBody(bool bNeverDeferRelease = false);

	// Contains the payload necessary for FBodyInstance::AsyncTermBody to release the body's resources asynchronously (outside of game thread).
	struct FAsyncTermBodyPayload
	{
		UE_API FAsyncTermBodyPayload();
		FPhysicsActorHandle GetPhysicsActor() const { return ActorHandle; }

	private:
		FPhysicsActorHandle ActorHandle;

		friend struct FBodyInstance;
	};

	/**
	 * Prepares the body to release its resources outside of the game thread. 
	 * @return the payload used by AsyncTermBody to release the resources.
	 */
	ENGINE_API FAsyncTermBodyPayload StartAsyncTermBody_GameThread();

	/** Releases body resources provided by Payload which was returned by StartAsyncTermBody_GameThread (can be called outside of game thread).
	 * @param Payload body resources to release.
	 */
	static ENGINE_API void AsyncTermBody(FAsyncTermBodyPayload& Payload);

	/** 
	 * Takes two body instances and welds them together to create a single simulated rigid body. Returns true if success.
	 */
	ENGINE_API bool Weld(FBodyInstance* Body, const FTransform& RelativeTM);

	/** 
	 * Takes a welded body and unwelds it. This function does not create the new body, it only removes the old one */
	ENGINE_API void UnWeld(FBodyInstance* Body);

	/** Finds all children that are technically welded to us (for example kinematics are welded but not as far as physx is concerned) and apply the actual physics engine weld on them*/
	ENGINE_API void ApplyWeldOnChildren();

	/**
	 * After adding/removing shapes call this function to update mass distribution etc... */
	ENGINE_API void PostShapeChange();

	/**
	 * Update Body Scale
	 * @param	InScale3D		New Scale3D. If that's different from previous Scale3D, it will update Body scale.
	 * @param	bForceUpdate	Will refresh shape dimensions from BodySetup, even if scale has not changed.
	 * @return true if succeed
	 */
	ENGINE_API bool UpdateBodyScale(const FVector& InScale3D, bool bForceUpdate = false);

	/** Dynamically update the vertices of per-poly collision for this body. */
	ENGINE_API void UpdateTriMeshVertices(const TArray<FVector> & NewPositions);

	/** Returns the center of mass of this body (in world space) */
	FVector GetCOMPosition() const
	{
		return GetMassSpaceToWorldSpace().GetLocation();
	}

	/** Returns the mass coordinate system to world space transform (position is world center of mass, rotation is world inertia orientation) */
	ENGINE_API FTransform GetMassSpaceToWorldSpace() const;

	/** Returns the mass coordinate system to local space transform (position is local center of mass, rotation should be identity) */
	ENGINE_API FTransform GetMassSpaceLocal() const;

	/** TODO: this only works at runtime when the physics state has been created. Any changes that result in recomputing mass properties will not properly remember this */
	ENGINE_API void SetMassSpaceLocal(const FTransform& NewMassSpaceLocalTM);

	/** Draws the center of mass as a wire star */
	ENGINE_API void DrawCOMPosition(class FPrimitiveDrawInterface* PDI, float COMRenderSize, const FColor& COMRenderColor);

	/** Utility for copying properties from one BodyInstance to another. */
	ENGINE_API void CopyBodyInstancePropertiesFrom(const FBodyInstance* FromInst);

	/** Utility for copying only the runtime instanced properties from one BodyInstance to another. */
	ENGINE_API void CopyRuntimeBodyInstancePropertiesFrom(const FBodyInstance* FromInst);

	/** Find the correct PhysicalMaterial for simple geometry on this body */
	ENGINE_API UPhysicalMaterial* GetSimplePhysicalMaterial() const;

	/** Find the correct PhysicalMaterial for simple geometry on a given body and owner. This is really for internal use during serialization */
	static ENGINE_API UPhysicalMaterial* GetSimplePhysicalMaterial(const FBodyInstance* BodyInstance, TWeakObjectPtr<UPrimitiveComponent> Owner, TWeakObjectPtr<UBodySetup> BodySetupPtr, IPhysicsBodyInstanceOwner* BodyInstanceOwner = nullptr);

	/** Get the complex PhysicalMaterials array for this body */
	ENGINE_API TArray<UPhysicalMaterial*> GetComplexPhysicalMaterials() const;

	/** Get the complex PhysicalMaterials and PhysicalMaterialMasks array for this body */
	ENGINE_API TArray<UPhysicalMaterial*> GetComplexPhysicalMaterials(TArray<FPhysicalMaterialMaskParams>& OutPhysMaterialMasks) const;

	/** Get the complex PhysicalMaterials for this body */
	ENGINE_API void GetComplexPhysicalMaterials(TArray<UPhysicalMaterial*> &OutPhysMaterials) const;

	/** Get the complex PhysicalMaterials and PhysicalMaterialMasks for this body */
	ENGINE_API void GetComplexPhysicalMaterials(TArray<UPhysicalMaterial*> &OutPhysMaterials, TArray<FPhysicalMaterialMaskParams>& OutPhysMaterialMasks) const;

	/** Find the correct PhysicalMaterial and PhysicalMaterialMasks for complex geometry on a given body and owner. This is really for internal use during serialization */
	static ENGINE_API void GetComplexPhysicalMaterials(const FBodyInstance* BodyInstance, TWeakObjectPtr<UPrimitiveComponent> Owner, TArray<UPhysicalMaterial*>& OutPhysMaterials, TArray<FPhysicalMaterialMaskParams>* OutPhysMaterialMasks = nullptr, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr);

	/** Returns the slope override struct for this instance. If we don't have our own custom setting, it will return the setting from the body setup. */
	ENGINE_API const struct FWalkableSlopeOverride& GetWalkableSlopeOverride() const;

	/** Sets a custom slope override struct for this instance. Implicitly sets bOverrideWalkableSlopeOnInstance to true. */
	ENGINE_API void SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride, bool bNewOverideSetting = true);

	/** Gets bOverrideWalkableSlopeOnInstance */
	ENGINE_API bool GetOverrideWalkableSlopeOnInstance() const;

	/** Returns true if the body is not static */
	ENGINE_API bool IsDynamic() const;

	/** Returns true if the body is non kinematic*/
	ENGINE_API bool IsNonKinematic() const;

	/** Returns the body's mass */
	ENGINE_API float GetBodyMass() const;
	/** Return bounds of physics representation in world space */
	ENGINE_API FBox GetBodyBounds() const;
	/** Return bounds of physics representation in world space */
	ENGINE_API FBox GetBodyBoundsLocal() const;
	/** Return the body's inertia tensor. This is returned in local mass space */
	ENGINE_API FVector GetBodyInertiaTensor() const;

	/** Whether inertia conditioning is enabled. @see bInertiaConditioning */
	bool IsInertiaConditioningEnabled() const { return bInertiaConditioning; }

	/** Enable or disable inertia conditionin.  @see bInertiaConditioning */
	ENGINE_API void SetInertiaConditioningEnabled(bool bEnabled);

	/** Apply async physics command onto the body instance*/
	ENGINE_API void ApplyAsyncPhysicsCommand(FAsyncPhysicsTimestamp TimeStamp, const bool bIsInternal, APlayerController* PlayerController, const TFunction<void()>& Command);

	/** 
	 * Set this body to either simulate or to be fixed/kinematic. 
	 * 
	 * @param bMaintainPhysicsBlending If true then the physics blend weight will not be adjusted. If false then 
	 *        it will get set to 0 or 1 depending on bSimulate.
	 * @param bPreserveExistingAttachments If true then any existing attachment between the owning component and 
	 *        its parent will be preserved, even when switching to simulate (most likely useful for skeletal meshes
	 *        that are parented to a moveable component). If false then the owning component will be detached 
	 *        from its parent if this is the root body and it is being set to simulate.
	 */
	ENGINE_API void SetInstanceSimulatePhysics(bool bSimulate, bool bMaintainPhysicsBlending=false, bool bPreserveExistingAttachment = false);
	/** Makes sure the current kinematic state matches the simulate flag */
	ENGINE_API void UpdateInstanceSimulatePhysics();
	/** Returns true if this body is simulating, false if it is fixed (kinematic) */
	bool IsInstanceSimulatingPhysics() const;
	/** Returns whether this body is awake */
	ENGINE_API bool IsInstanceAwake() const;
	/** Wake this body */
	ENGINE_API void WakeInstance();
	/** Force this body to sleep */
	ENGINE_API void PutInstanceToSleep();
	/** Gets the multiplier to the threshold where the body will go to sleep automatically. */
	ENGINE_API float GetSleepThresholdMultiplier() const;
	/** Add custom forces and torques on the body. The callback will be called more than once, if substepping enabled, for every substep.  */
	ENGINE_API void AddCustomPhysics(FCalculateCustomPhysics& CalculateCustomPhysics);
	/** Add a force to this body */
	ENGINE_API void AddForce(const FVector& Force, bool bAllowSubstepping = true, bool bAccelChange = false, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);
	/** Add a force at a particular position (world space when bIsLocalForce = false, body space otherwise) */
	ENGINE_API void AddForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping = true, bool bIsLocalForce = false, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);
	/** Clear accumulated forces on this body */
	ENGINE_API void ClearForces(bool bAllowSubstepping = true);

	/** If set to true, this body will treat bodies that do not have the flag set as having infinite mass */
	ENGINE_API void SetOneWayInteraction(bool InOneWayInteraction = true);

	/** Add a torque to this body */
	ENGINE_API void AddTorqueInRadians(const FVector& Torque, bool bAllowSubstepping = true, bool bAccelChange = false, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);
	/** Clear accumulated torques on this body */
	ENGINE_API void ClearTorques(bool bAllowSubstepping = true);

	/** Add a rotational impulse to this body */
	ENGINE_API void AddAngularImpulseInRadians(const FVector& Impulse, bool bVelChange, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);

	/** Add an impulse to this body */
	ENGINE_API void AddImpulse(const FVector& Impulse, bool bVelChange, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);
	
	/** Add an impulse to this body and a particular world position */
	ENGINE_API void AddImpulseAtPosition(const FVector& Impulse, const FVector& Position, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);

	/** Add a velocity change impulse to this body and a particular world position */
	ENGINE_API void AddVelocityChangeImpulseAtLocation(const FVector& Impulse, const FVector& Position, const FAsyncPhysicsTimestamp TimeStamp = FAsyncPhysicsTimestamp(), APlayerController* PlayerController = nullptr);

	/** Set the linear velocity of this body */
	ENGINE_API void SetLinearVelocity(const FVector& NewVel, bool bAddToCurrent, bool bAutoWake = true);

	/** Set the angular velocity of this body */
	ENGINE_API void SetAngularVelocityInRadians(const FVector& NewAngVel, bool bAddToCurrent, bool bAutoWake = true);

	/** Set the maximum angular velocity of this body */
	ENGINE_API void SetMaxAngularVelocityInRadians(float NewMaxAngVel, bool bAddToCurrent, bool bUpdateOverrideMaxAngularVelocity = true);

	/** Get the maximum angular velocity of this body */
	ENGINE_API float GetMaxAngularVelocityInRadians() const;

	/** Are we overriding the MaxDepenetrationVelocity. See SetMaxDepenetrationVelocity */
	bool GetOverrideMaxDepenetrationVelocity() const { return bOverrideMaxDepenetrationVelocity; }

	/** Enable/Disable override of MaxDepenetrationVelocity */
	ENGINE_API void SetOverrideMaxDepenetrationVelocity(bool bInEnabled);

	/**
	 * Set the maximum velocity used to depenetrate this object from others when spawned with initial overlaps or teleports (does not affect overlaps as a result of normal movement).
	 * A value of zero will allow objects that are spawned overlapping to go to sleep as they are rather than pop out of each other.
	 * Note: implicitly calls SetOverrideMaxDepenetrationVelocity(true)
	 * Note: MaxDepenetration overrides the CollisionInitialOverlapDepenetrationVelocity project setting (and not the MaxDepenetrationVelocity solver setting that will be deprecated)
	*/
	ENGINE_API void SetMaxDepenetrationVelocity(float MaxVelocity);

	/** The maximum velocity at which initally-overlapping bodies will separate. Does not affect normal contact resolution. */
	float GetMaxDepenetrationVelocity() const { return MaxDepenetrationVelocity; }

	/** Set whether we should get a notification about physics collisions */
	ENGINE_API void SetInstanceNotifyRBCollision(bool bNewNotifyCollision);
	/** Enables/disables whether this body is affected by gravity. */
	ENGINE_API void SetEnableGravity(bool bGravityEnabled);
	/** Sets the gravity group index, which determines acceleration when gravity is enabled. */
	ENGINE_API void SetGravityGroupIndex(int32 NewGravityGroupIndex);
	/** Enables/disables whether this body, when kinematic, is updated from the simulation rather than when setting the kinematic target. */
	ENGINE_API void SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation);
	/** Enabled/disables whether this body is affected by gyroscopic torque, mainly useful for long/thin objects that spin */
	ENGINE_API void SetGyroscopicTorqueEnabled(bool bInGyroscopicTorqueEnabled);
	/** Enables/disables contact modification */
	ENGINE_API void SetContactModification(bool bNewContactModification);
	/** Enables/disabled smoothed edge collisions */
	ENGINE_API void SetSmoothEdgeCollisionsEnabled(bool bNewSmoothEdgeCollisions);

	/** Enable/disable Continuous Collision Detection feature */
	ENGINE_API void SetUseCCD(bool bInUseCCD);

	/** 
	 * [EXPERIMENTAL] Enable/disable Motion-Aware Collision Detection feature. MACD attempts to take the movement of the
	 * body into account during collisions detection to reduce the chance of objects passing through each other at moderate
	 * speeds without the need for CCD. CCD is still required reliable collision between high-speed objects.
	 */
	ENGINE_API void SetUseMACD(bool bInUseMACD);

	/**
	 * [EXPERIMENTAL] Enable/disable Partial Island Sleeping for the entire island this body is in.
	 * Partial Island Sleeping permits the solver to freeze the body's dynamics ("sleep") if it has not moved significantly
	 * in the previous simulation frames even if connected bodies are still moving. The body will be unfrozen ("wake up")
	 * due to several possible triggers such as the detection of new collisions, creation of new constraints, and user interactions.
	 */
	ENGINE_API void SetAllowPartialIslandSleep(bool bInAllowPartialIslandSleep);

	// Set the per-body position iterations (only used if bOverrideIterationCounts is true)
	ENGINE_API void SetPositionSolverIterationCount(uint8 PositionSolverIterationCountIn);

	// Set the per-body velocity iterations (only used if bOverrideIterationCounts is true)
	ENGINE_API void SetVelocitySolverIterationCount(uint8 VelocitySolverIterationCountIn);

	// Set the per-body projection iterations (only used if bOverrideIterationCounts is true)
	ENGINE_API void SetProjectionSolverIterationCount(uint8 ProjectionSolverIterationCountIn);

	// Enable or disable the use of iteration count overrides
	ENGINE_API void SetOverrideIterationCounts(bool bOverride);

	// Get the per-body position iterations (or -1 if bOverrideIterationCounts is false)
	int32 GetPositionSolverIterationCount() const { return (!!bOverrideIterationCounts) ? PositionSolverIterationCount : -1; }

	// Get the per-body velocity iterations (or -1 if bOverrideIterationCounts is false)
	int32 GetVelocitySolverIterationCount() const { return (!!bOverrideIterationCounts) ? VelocitySolverIterationCount : -1; }

	// Get the per-body projection iterations (or -1 if bOverrideIterationCounts is false)
	int32 GetProjectionSolverIterationCount() const { return (!!bOverrideIterationCounts) ? ProjectionSolverIterationCount : -1; }

	/** [EXPERIMENTAL] Whether Motion-Aware Collision Detection is enabled */
	bool GetUseMACD() const { return bUseMACD != 0; }

	/** [EXPERIMENTAL] Whether Partial Island Sleep is permitted for the island this body is in */
	bool GetAllowPartialIslandSleep() const { return bAllowPartialIslandSleep != 0; }

	/** Disable/Re-Enable this body in the solver,  when disable, the body won't be part of the simulation ( regardless if it's dynamic or kinematic ) and no collision will occur 
	* this can be used for performance control situation for example
	*/
	ENGINE_API void SetPhysicsDisabled(bool bSetDisabled);

	ENGINE_API bool IsPhysicsDisabled() const;

	/** Get the EPhysicsReplicationMode from the owning actor. It's recommended to get the parameter directly from the Actor if possible. */
	ENGINE_API EPhysicsReplicationMode GetPhysicsReplicationMode() const;

private:

	/** Struct of body instance delegates that are rarely bound so that we can only allocate memory if one is actually being used. */
	struct FBodyInstanceDelegates
	{
		/** Custom projection for physics (callback to update component transform based on physics data) */
		FCalculateCustomProjection OnCalculateCustomProjection;

		/** Called whenever mass properties have been re-calculated. */
		FRecalculatedMassProperties OnRecalculatedMassProperties;
	};

	/** Specialization of TUniquePtr for storing body instance delegates as there is a need to copy the contents of the delegate structure. */
	struct FBodyInstanceDelegatesPtr : public TUniquePtr<FBodyInstanceDelegates>
	{
		FBodyInstanceDelegatesPtr() = default;
		FBodyInstanceDelegatesPtr(const FBodyInstanceDelegatesPtr& Other)
		{
			if (Other.IsValid())
			{
				Reset(new FBodyInstanceDelegates);
				*Get() = *Other;
			}
		}

		FBodyInstanceDelegatesPtr& operator=(const FBodyInstanceDelegatesPtr& Other)
		{
			if (Other.IsValid())
			{
				if (!IsValid())
				{
					Reset(new FBodyInstanceDelegates);
				}
				*Get() = *Other;
			}
			else
			{
				Reset();
			}

			return *this;
		}
	};

	/** Pointer to lazily created container for the body instance delegates. */
	FBodyInstanceDelegatesPtr BodyInstanceDelegates;

public:
	/** Executes the OnCalculateCustomProjection delegate if bound. */
	ENGINE_API void ExecuteOnCalculateCustomProjection(FTransform& WorldTM) const;

	/** Returns reference to the OnCalculateCustomProjection delegate. Will allocate delegate struct if not already created. */
	ENGINE_API FCalculateCustomProjection& OnCalculateCustomProjection();

	/** Returns reference to the OnRecalculatedMassProperties delegate. Will allocate delegate struct if not already created. */
	ENGINE_API FRecalculatedMassProperties& OnRecalculatedMassProperties();

	/** See if this body is valid. */
	ENGINE_API bool IsValidBodyInstance() const;

	/** Get current transform in world space from physics body. */
	ENGINE_API FTransform GetUnrealWorldTransform(bool bWithProjection = true, bool bForceGlobalPose = false) const;

	/** Get current transform in world space from physics body. */
	ENGINE_API FTransform GetUnrealWorldTransform_AssumesLocked(bool bWithProjection = true, bool bForceGlobalPose = false) const;

	/** Get the kinematic target transform in world space from physics body. Will only be relevant/useful if the body is kinematic */
	ENGINE_API FTransform GetKinematicTarget() const;

	/** Get the kinematic target transform in world space from physics body. Will only be relevant/useful if the body is kinematic */
	ENGINE_API FTransform GetKinematicTarget_AssumesLocked() const;

	/**
	 *	Move the physics body to a new pose.
	 *	@param	bTeleport	If true, no velocity is inferred on the kinematic body from this movement, but it moves right away.
	 */
	ENGINE_API void SetBodyTransform(const FTransform& NewTransform, ETeleportType Teleport, bool bAutoWake = true);

	/** Get current velocity in world space from physics body. */
	ENGINE_API FVector GetUnrealWorldVelocity() const;

	/** Get current velocity in world space from physics body. */
	ENGINE_API FVector GetUnrealWorldVelocity_AssumesLocked() const;

	/** Get current angular velocity in world space from physics body. */
	ENGINE_API FVector GetUnrealWorldAngularVelocityInRadians() const;

	/** Get current angular velocity in world space from physics body. */
	ENGINE_API FVector GetUnrealWorldAngularVelocityInRadians_AssumesLocked() const;

	/** Get current velocity of a point on this physics body, in world space. Point is specified in world space. */
	ENGINE_API FVector GetUnrealWorldVelocityAtPoint(const FVector& Point) const;

	/** Get current velocity of a point on this physics body, in world space. Point is specified in world space. */
	ENGINE_API FVector GetUnrealWorldVelocityAtPoint_AssumesLocked(const FVector& Point) const;

	/** Get physical material override for this body */
	UPhysicalMaterial* GetPhysMaterialOverride() const { return PhysMaterialOverride; }

	/** Set physical material override for this body */
	ENGINE_API void SetPhysMaterialOverride(class UPhysicalMaterial* NewPhysMaterial);

	/** Set a new contact report force threhold.  Threshold < 0 disables this feature. */
	ENGINE_API void SetContactReportForceThreshold(float Threshold);

	/** Set the collision response of this body to a particular channel */
	ENGINE_API bool SetResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse);

	/** Get the collision response of this body to a particular channel */
	inline ECollisionResponse GetResponseToChannel(ECollisionChannel Channel) const { return CollisionResponses.GetResponse(Channel); }

	/** Set the response of this body to all channels */
	ENGINE_API bool SetResponseToAllChannels(ECollisionResponse NewResponse);

	/** Replace the channels on this body matching the old response with the new response */
	ENGINE_API bool ReplaceResponseToChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse);

	/** Set the response of this body to the supplied settings */
	ENGINE_API bool SetResponseToChannels(const FCollisionResponseContainer& NewResponses);

	/** Set the response of a specific shape on this body to the supplied settings */
	ENGINE_API bool SetShapeResponseToChannels(const int32 ShapeIndex, const FCollisionResponseContainer& NewResponses);

	/** Get Collision ResponseToChannels container for this component **/
	inline const FCollisionResponseContainer& GetResponseToChannels() const { return CollisionResponses.GetResponseContainer(); }

	/** Get Collision ResponseToChannels container for a specific shape in this component **/
	ENGINE_API const FCollisionResponseContainer& GetShapeResponseToChannels(const int32 ShapeIndex) const;
	ENGINE_API const FCollisionResponseContainer& GetShapeResponseToChannels(const int32 ShapeIndex, const FCollisionResponseContainer& DefaultResponseContainer) const;

	/** Set the movement channel of this body to the one supplied */
	ENGINE_API void SetObjectType(ECollisionChannel Channel);

	/** Get the movement channel of this body **/
	inline ECollisionChannel GetObjectType() const { return ObjectType; }

	/** Controls what kind of collision is enabled for this body and allows optional disable physics rebuild */
	ENGINE_API void SetCollisionEnabled(ECollisionEnabled::Type NewType, bool bUpdatePhysicsFilterData = true);

	/** Controls what kind of collision is enabled for a particular shape */
	ENGINE_API void SetShapeCollisionEnabled(const int32 ShapeIndex, ECollisionEnabled::Type NewType, bool bUpdatePhysicsFilterData = true);

private:

	ENGINE_API ECollisionEnabled::Type GetCollisionEnabled_CheckOwner() const;

public:
	/** Get the current type of collision enabled */
	inline ECollisionEnabled::Type GetCollisionEnabled(bool bCheckOwner = true) const
	{
		return (bCheckOwner ? GetCollisionEnabled_CheckOwner() : CollisionEnabled.GetValue());
	}

	/** Get the current type of collision enabled for a particular shape */
	ENGINE_API ECollisionEnabled::Type GetShapeCollisionEnabled(const int32 ShapeIndex) const;

	/**  
	 * Set Collision Profile Name (deferred)
	 * This function is called by constructors when they set ProfileName
	 * This will change current CollisionProfileName, but collision data will not be set up until the physics state is created
	 * or the collision profile is accessed.
	 * @param InCollisionProfileName : New Profile Name
	 */
	ENGINE_API void SetCollisionProfileNameDeferred(FName InCollisionProfileName);

	/**  
	 * Set Collision Profile Name
	 * This function should be called outside of constructors to set profile name.
	 * This will change current CollisionProfileName to be this, and overwrite Collision Setting
	 * 
	 * @param InCollisionProfileName : New Profile Name
	 */
	ENGINE_API void SetCollisionProfileName(FName InCollisionProfileName);

	/** Updates the mask filter. */
	ENGINE_API void SetMaskFilter(FMaskFilter InMaskFilter);

	/** Return the ignore mask filter. */
	inline FMaskFilter GetMaskFilter() const { return MaskFilter; }
	/** Returns the collision profile name that will be used. */
	ENGINE_API FName GetCollisionProfileName() const;

	/** return true if it uses Collision Profile System. False otherwise*/
	ENGINE_API bool DoesUseCollisionProfile() const;

	/** Modify the mass scale of this body */
	ENGINE_API void SetMassScale(float InMassScale=1.f);

	/** Update instance's mass properties (mass, inertia and center-of-mass offset) based on MassScale, InstanceMassScale and COMNudge. */
	ENGINE_API void UpdateMassProperties();

	/** Update instance's linear and angular damping */
	ENGINE_API void UpdateDampingProperties();

	/** Update the instance's material properties (friction, restitution) */
	ENGINE_API void UpdatePhysicalMaterials();

	/** 
	 *  Apply a material directly to the passed in shape. Note this function is very advanced and requires knowledge of shape sharing as well as threading. Note: assumes the appropriate locks have been obtained
	 *  @param  PShape					The shape we are applying the material to
	 *  @param  SimplePhysMat			The material to use if a simple shape is provided (or complex materials are empty)
	 *  @param  ComplexPhysMats			The array of materials to apply if a complex shape is provided
	 */
	static ENGINE_API void ApplyMaterialToShape_AssumesLocked(const FPhysicsShapeHandle& InShape, UPhysicalMaterial* SimplePhysMat, const TArrayView<UPhysicalMaterial*>& ComplexPhysMats, const TArrayView<FPhysicalMaterialMaskParams>* ComplexPhysMatMasks = nullptr);

	/** Note: This function is not thread safe. Make sure you obtain the appropriate physics scene lock before calling it*/
	ENGINE_API void ApplyMaterialToInstanceShapes_AssumesLocked(UPhysicalMaterial* SimplePhysMat, TArray<UPhysicalMaterial*>& ComplexPhysMats, const TArrayView<FPhysicalMaterialMaskParams>& ComplexPhysMatMasks);

	/** Update the instances collision filtering data */ 
	ENGINE_API void UpdatePhysicsFilterData();

	friend ENGINE_API FArchive& operator<<(FArchive& Ar,FBodyInstance& BodyInst);

	/** Get the name for this body, for use in debugging */
	ENGINE_API FString GetBodyDebugName() const;

	/** 
	 *  Trace a ray against just this bodyinstance
	 *  @param  OutHit					Information about hit against this component, if true is returned
	 *  @param  Start					Start location of the ray
	 *  @param  End						End location of the ray
	 *	@param	bTraceComplex			Should we trace against complex or simple collision of this body
	 *  @param bReturnPhysicalMaterial	Fill in the PhysMaterial field of OutHit
	 *  @return true if a hit is found
	 */
	ENGINE_API bool LineTrace(struct FHitResult& OutHit, const FVector& Start, const FVector& End, bool bTraceComplex, bool bReturnPhysicalMaterial = false) const;

	/** 
	 *  Trace a shape against just this bodyinstance
	 *  @param  OutHit          	Information about hit against this component, if true is returned
	 *  @param  Start           	Start location of the box
	 *  @param  End             	End location of the box
	 *  @param  ShapeWorldRotation  The rotation applied to the collision shape in world space.
	 *  @param  CollisionShape		Collision Shape
	 *	@param	bTraceComplex		Should we trace against complex or simple collision of this body
	 *  @return true if a hit is found
	 */
	ENGINE_API bool Sweep(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& ShapeWorldRotation, const FCollisionShape& Shape, bool bTraceComplex) const;

	/**
	 *  Test if the bodyinstance overlaps with the specified shape at the specified position/rotation
	 *
	 *  @param  Position		Position to place the shape at before testing
	 *  @param  Rotation		Rotation to apply to the shape before testing
	 *	@param	CollisionShape	Shape to test against
	 *  @param  OutMTD			The minimum translation direction needed to push the shape out of this BodyInstance. (Optional)
	 *  @param  TraceComplex    Trace against complex or simple geometry (Defaults simple)
	 *  @return true if the geometry associated with this body instance overlaps the query shape at the specified location/rotation
	 */
	ENGINE_API bool OverlapTest(const FVector& Position, const FQuat& Rotation, const struct FCollisionShape& CollisionShape, FMTDResult* OutMTD = nullptr, bool bTraceComplex = false) const;

	/**
	 *  Test if the bodyinstance overlaps with the specified shape at the specified position/rotation
	 *  Note: This function is not thread safe. Make sure you obtain the physics scene read lock before calling it
	 *
	 *  @param  Position		Position to place the shape at before testing
	 *  @param  Rotation		Rotation to apply to the shape before testing
	 *	@param	CollisionShape	Shape to test against
	 *  @param  OutMTD			The minimum translation direction needed to push the shape out of this BodyInstance. (Optional)
	 * 	@param  TraceComplex    Trace against complex or simple geometry  (Defaults simple)
	 *  @return true if the geometry associated with this body instance overlaps the query shape at the specified location/rotation
	 */
	ENGINE_API bool OverlapTest_AssumesLocked(const FVector& Position, const FQuat& Rotation, const struct FCollisionShape& CollisionShape, FMTDResult* OutMTD = nullptr, bool bTraceComplex = false) const;

	/**
	 *  Test if the bodyinstance overlaps with the specified body instances
	 *
	 *  @param  Position		Position to place our shapes at before testing (shapes of this BodyInstance)
	 *  @param  Rotation		Rotation to apply to our shapes before testing (shapes of this BodyInstance)
	 *  @param  Bodies			The bodies we are testing for overlap with. These bodies will be in world space already
	 *  @param  TraceComplex    Trace against complex or simple geometry (Defaults simple)
	 *  @return true if any of the bodies passed in overlap with this
	 */
	bool OverlapTestForBodies(const FVector& Position, const FQuat& Rotation, const TArray<FBodyInstance*>& Bodies, bool bTraceComplex = false) const;
	bool OverlapTestForBody(const FVector& Position, const FQuat& Rotation, FBodyInstance* Body, bool bTraceComplex = false) const;

	/**
	 *  Determines the set of components that this body instance would overlap with at the supplied location/rotation
	 *  @note The overload taking rotation as an FQuat is slightly faster than the version using FRotator (which will be converted to an FQuat)..
	 *  @param  InOutOverlaps   Array of overlaps found between this component in specified pose and the world (new overlaps will be appended, the array is not cleared!)
	 *  @param  World			World to use for overlap test
	 *  @param  pWorldToComponent Used to convert the body instance world space position into local space before teleporting it to (Pos, Rot) [optional, use when the body instance isn't centered on the component instance]
	 *  @param  Pos             Location to place the component's geometry at to test against the world
	 *  @param  Rot             Rotation to place components' geometry at to test against the world
	 *  @param  TestChannel		The 'channel' that this ray is in, used to determine which components to hit
	 *  @param  Params
	 *  @param  ResponseParams
	 *	@param	ObjectQueryParams	List of object types it's looking for. When this enters, we do object query with component shape
	 *  @return TRUE if OutOverlaps contains any blocking results
	 */
	ENGINE_API bool OverlapMulti(TArray<struct FOverlapResult>& InOutOverlaps, const class UWorld* World, const FTransform* pWorldToComponent, const FVector& Pos, const FQuat& Rot,    ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const;
	bool OverlapMulti(TArray<struct FOverlapResult>& InOutOverlaps, const class UWorld* World, const FTransform* pWorldToComponent, const FVector& Pos, const FRotator& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectQueryParams = FCollisionObjectQueryParams::DefaultObjectQueryParam) const;

	/**
	 * Add an impulse to this bodyinstance, radiating out from the specified position.
	 *
	 * @param Origin		Point of origin for the radial impulse blast, in world space
	 * @param Radius		Size of radial impulse. Beyond this distance from Origin, there will be no affect.
	 * @param Strength		Maximum strength of impulse applied to body.
	 * @param Falloff		Allows you to control the strength of the impulse as a function of distance from Origin.
	 * @param bVelChange	If true, the Strength is taken as a change in velocity instead of an impulse (ie. mass will have no effect).
	 */
	ENGINE_API void AddRadialImpulseToBody(const FVector& Origin, float Radius, float Strength, uint8 Falloff, bool bVelChange = false);

	/**
	 *	Add a force to this bodyinstance, originating from the supplied world-space location.
	 *
	 *	@param Origin		Origin of force in world space.
	 *	@param Radius		Radius within which to apply the force.
	 *	@param Strength		Strength of force to apply.
	 *  @param Falloff		Allows you to control the strength of the force as a function of distance from Origin.
	 *  @param bAccelChange If true, Strength is taken as a change in acceleration instead of a physical force (i.e. mass will have no effect).
	 *  @param bAllowSubstepping Whether we should sub-step this radial force. You should only turn this off if you're calling it from a sub-step callback, otherwise there will be energy loss
	 */
	ENGINE_API void AddRadialForceToBody(const FVector& Origin, float Radius, float Strength, uint8 Falloff, bool bAccelChange = false, bool bAllowSubstepping = true);

	/**
	 * Get distance to the body surface if available
	 * It is only valid if BodyShape is convex
	 * If point is inside distance it will be 0
	 * Returns false if geometry is not supported
	 *
	 * @param Point				Point in world space
	 * @param OutDistanceSquared How far from the instance the point is. 0 if inside the shape
	 * @param OutPointOnBody	Point on the surface of body closest to Point
	 * @return true if a distance to the body was found and OutDistanceSquared has been populated
	 */
	ENGINE_API bool GetSquaredDistanceToBody(const FVector& Point, float& OutDistanceSquared, FVector& OutPointOnBody) const;

	/**
	* Get the square of the distance to the body surface if available
	* It is only valid if BodyShape is convex
	* If point is inside or shape is not convex, it will return 0.f
	*
	* @param Point				Point in world space
	* @param OutPointOnBody	Point on the surface of body closest to Point
	*/
	ENGINE_API float GetDistanceToBody(const FVector& Point, FVector& OutPointOnBody) const;

	/** 
	 * Returns memory used by resources allocated for this body instance ( ex. physics resources )
	 **/
	ENGINE_API void GetBodyInstanceResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	/**
	 * UObject notification by OwningComponent
	 */
	ENGINE_API void FixupData(class UObject* Loader);

	const FCollisionResponse& GetCollisionResponse() const { return CollisionResponses; }

	/** Applies a deferred collision profile */
	ENGINE_API void ApplyDeferredCollisionProfileName();

	/** Returns the original owning body instance. This is needed for welding */
	ENGINE_API const FBodyInstance* GetOriginalBodyInstance(const FPhysicsShapeHandle& InShape) const;

	/** Returns the relative transform between root body and welded instance owned by the shape.*/
	ENGINE_API const FTransform& GetRelativeBodyTransform(const FPhysicsShapeHandle& InShape) const;

	/** Check if the shape is owned by this body instance */
	ENGINE_API bool IsShapeBoundToBody(const FPhysicsShapeHandle& Shape) const;

public:
	// #PHYS2 Rename, not just for physx now.
	FPhysicsUserData PhysicsUserData;

	struct FWeldInfo
	{
		FWeldInfo(FBodyInstance* InChildBI, const FTransform& InRelativeTM)
			: ChildBI(InChildBI)
			, RelativeTM(InRelativeTM)
		{}

		FBodyInstance* ChildBI;
		FTransform RelativeTM;
	};

	ENGINE_API const TMap<FPhysicsShapeHandle, FWeldInfo>* GetCurrentWeldInfo() const;

private:

	ENGINE_API void UpdateOneWayInteraction();
	ENGINE_API void UpdateMaxDepenetrationVelocity();

	/**
	 * Invalidate Collision Profile Name
	 * This gets called when it invalidates the reason of Profile Name
	 * for example, they would like to re-define CollisionEnabled or ObjectType or ResponseChannels
	 */
	ENGINE_API void InvalidateCollisionProfileName();

	/** Moves welded bodies within a rigid body (updates their shapes) */
	ENGINE_API void SetWeldedBodyTransform(FBodyInstance* TheirBody, const FTransform& NewTransform);
		
	/**
	 * Return true if the collision profile name is valid
	 */
	static ENGINE_API bool IsValidCollisionProfileName(FName InCollisionProfileName);

	template<typename AllocatorType>
	ENGINE_API bool OverlapTestForBodiesImpl(const FVector& Position, const FQuat& Rotation, const TArray<FBodyInstance*, AllocatorType>& Bodies, bool bTraceComplex = false) const;

	friend class UPhysicsAsset;
	friend class UCollisionProfile;
	friend class FBodyInstanceCustomization;
	friend struct FUpdateCollisionResponseHelper;
	friend class FBodySetupDetails;
	
	friend struct Chaos::Private::FInitBodiesHelperBaseInternal;
	friend class FBodyInstanceCustomizationHelper;
	friend class FFoliageTypeCustomizationHelpers;

public:

	/** Serialized out the physics thread state of this body as an array of bytes */
	UE_INTERNAL ENGINE_API void SerializeState(Chaos::FSerializedDataBuffer& OutSerializedState, Chaos::ESerializedDataContext DataContext);

	/** Deserializes the provided array of bytes representing a saved physics thread state and applies it to this body */
	UE_INTERNAL ENGINE_API void ApplySerializedState(Chaos::FSerializedDataBufferPtr&& InSerializedState, Chaos::ESerializedDataContext DataContext);

private:

	void MarkNonFrequentDirty();

	ENGINE_API void UpdateDebugRendering();

	/** Used to map between shapes and welded bodies. We do not create entries if the owning body instance is root*/
	TSharedPtr<TMap<FPhysicsShapeHandle, FWeldInfo>> ShapeToBodiesMap;


public:
	// Cache resolved Inputs used by physics state creation (currently only used by async mode)
	ENGINE_API const FBodyInstancePhysicsCreationInputsPtr& CachePhysicsCreationInputs(UPrimitiveComponent* InOwnerComponent, IPhysicsBodyInstanceOwner* InBodyInstanceOwner = nullptr);
	
	// Clear cached physics creation inputs
	void ClearCachedPhysicsCreationInputs() { PhysicsCreationInputs.Reset(); }

	/**
	 * Returns the cached physics creation inputs when called from a worker thread (not game thread, not parallel game thread).
	 * Returns nullptr on the game thread or if no cache is available.
	 * Use this to safely branch between cached data (off-GT) and live data (on-GT).
	 */
	ENGINE_API const FBodyInstancePhysicsCreationInputs* GetAsyncPhysicsCreationInputs() const;

private:

	/** Static version of BuildBodyFilterData taking explicit parameters (avoids weak ptr dereference, used for pre-computation and off-GT calls) */
	static ENGINE_API void BuildBodyFilterData(const FBodyInstance* InInstance, UPrimitiveComponent* InOwnerComp, UObject* InSourceObj, UBodySetup* InSetup, FBodyCollisionFilterData& OutFilterData, const int32 InShapeIndex = INDEX_NONE);

	FBodyInstancePhysicsCreationInputsPtr PhysicsCreationInputs;
};

template<>
struct TStructOpsTypeTraits<FBodyInstance> : public TStructOpsTypeTraitsBase2<FBodyInstance>
{
	enum
	{
		WithCopy = false
	};
};


#if WITH_EDITOR

// Helper methods for classes with body instances
struct FBodyInstanceEditorHelpers
{
	ENGINE_API static void EnsureConsistentMobilitySimulationSettingsOnPostEditChange(UPrimitiveComponent* Component, FPropertyChangedEvent& PropertyChangedEvent);

private:
	FBodyInstanceEditorHelpers() {}
};
#endif

//////////////////////////////////////////////////////////////////////////
// BodyInstance inlines

/// @cond DOXYGEN_WARNINGS

inline bool FBodyInstance::OverlapMulti(TArray<struct FOverlapResult>& InOutOverlaps, const class UWorld* World, const FTransform* pWorldToComponent, const FVector& Pos, const FRotator& Rot, ECollisionChannel TestChannel, const FComponentQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectQueryParams) const
{
	// Pass on to FQuat version
	return OverlapMulti(InOutOverlaps, World, pWorldToComponent, Pos, Rot.Quaternion(), TestChannel, Params, ResponseParams, ObjectQueryParams);
}

/// @endcond

inline bool FBodyInstance::OverlapTestForBodies(const FVector& Position, const FQuat& Rotation, const TArray<FBodyInstance*>& Bodies, bool bTraceComplex) const
{
	return OverlapTestForBodiesImpl(Position, Rotation, Bodies, bTraceComplex);
}

inline bool FBodyInstance::OverlapTestForBody(const FVector& Position, const FQuat& Rotation, FBodyInstance* Body, bool bTraceComplex) const
{
	TArray<FBodyInstance*, TInlineAllocator<1>> InlineArray;
	InlineArray.Add(Body);
	return OverlapTestForBodiesImpl(Position, Rotation, InlineArray, bTraceComplex);
}

inline bool FBodyInstance::IsInstanceSimulatingPhysics() const
{
	return ShouldInstanceSimulatingPhysics() && IsValidBodyInstance();
}

#undef UE_API
