// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "EngineDefines.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Serialization/BulkData.h"
#include "BodySetupEnums.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "HAL/ThreadSafeBool.h"
#include "Async/TaskGraphInterfaces.h"
#include "BodySetupCore.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if WITH_EDITOR
#include "Factories.h"
#endif

#include "PhysicsEngine/BodySetupObjectTextFactory.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "BodySetup.generated.h"

class ITargetPlatform;
class USkeletalMesh;
class UPhysicalMaterial;
class UPhysicalMaterialMask;
class UPrimitiveComponent;
struct FShapeData;
enum class EPhysXMeshCookFlags : uint8;

DECLARE_DELEGATE_OneParam(FOnAsyncPhysicsCookFinished, bool);

namespace Chaos
{
	class FImplicitObject;
	class FTriangleMeshImplicitObject;
	struct FCookHelper;
}

template<typename T, int d>
class FChaosDerivedDataReader;

DECLARE_CYCLE_STAT_EXTERN(TEXT("BodySetup Cooking"), STAT_PhysXCooking, STATGROUP_Physics, );


/** UV information for BodySetup, only created if UPhysicsSettings::bSupportUVFromHitResults */
struct FBodySetupUVInfo
{
	/** Index buffer, required to go from face index to UVs */
	TArray<int32> IndexBuffer;
	/** Vertex positions, used to determine barycentric co-ords */
	TArray<FVector> VertPositions;
	/** UV channels for each vertex */
	TArray< TArray<FVector2D> > VertUVs;

	friend FArchive& operator<<(FArchive& Ar, FBodySetupUVInfo& UVInfo)
	{
		Ar << UVInfo.IndexBuffer;
		Ar << UVInfo.VertPositions;
		Ar << UVInfo.VertUVs;

		return Ar;
	}

	/** Get resource size of UV info */
	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	void FillFromTriMesh(const FTriMeshCollisionData& TriMeshCollisionData);
};

/** Helper struct to indicate which geometry needs to be cooked */
struct FCookBodySetupInfo
{
	ENGINE_API FCookBodySetupInfo();

	/** Trimesh data for cooking */
	FTriMeshCollisionData TriangleMeshDesc;

	/** Trimesh cook flags */
	EPhysXMeshCookFlags TriMeshCookFlags;

	/** Convex cook flags */
	EPhysXMeshCookFlags ConvexCookFlags;

	/** Vertices of NonMirroredConvex hulls */
	TArray<TArray<FVector>> NonMirroredConvexVertices;

	/** Vertices of NonMirroredConvex hulls */
	TArray<TArray<FVector>> MirroredConvexVertices;

	/** Debug name helpful for runtime cooking warnings */
	FString OuterDebugName;

	/** Whether to cook the regular convex hulls */
	bool bCookNonMirroredConvex;

	/** Whether to cook the mirror convex hulls */
	bool bCookMirroredConvex;

	/** Whether the convex being cooked comes from a deformable mesh */
	bool bConvexDeformableMesh;

	/** Whether to cook trimesh collision*/
	bool bCookTriMesh;

	/** Whether to support UV from hit results */
	bool bSupportUVFromHitResults;

	/** Whether to support face remap, needed for physical material masks. Note: Also set bSupportFaceRemapOnMeshBVH to false to fully remove the face remap data. */
	bool bSupportFaceRemap;

	/** Whether to support face remap on the mesh BVH specifically. To fully remove face remap data, both this and the above bSupportFaceRemap must be false. */
	bool bSupportFaceRemapOnMeshBVH;

	/** Whether to support vertex remap, needed for skeletal mesh per-poly collision and any other FBodyInstance::UpdateTriMeshVertices calls */
	bool bSupportVertexRemap;

	/** Error generating cook info for trimesh*/
	bool bTriMeshError;
};

struct FPhysXCookHelper;

/** Option for GetScaledVolume to either query everything, or only geometries that will be used to calculate mass */
enum class EVolumeCalculationMethod
{
	AllGeoms,
	OnlyMassRelevantGeoms
};

/**
 * BodySetup contains all collision information that is associated with a single asset.
 * A single BodySetup instance is shared among many BodyInstances so that geometry data is not duplicated.
 * Assets typically implement a GetBodySetup function that is used during physics state creation.
 * 
 * @see GetBodySetup
 * @see FBodyInstance
 */

UCLASS(collapseCategories, MinimalAPI)
class UBodySetup : public UBodySetupCore
{
	GENERATED_UCLASS_BODY()

	/** Needs implementation in BodySetup.cpp to compile UniquePtr for forward declared class */
	UBodySetup(FVTableHelper& Helper);

	virtual ~UBodySetup();

	/** Simplified collision representation of this  */
	UPROPERTY(EditAnywhere, Category = BodySetup, meta=(DisplayName = "Primitives", NoResetToDefault))
	struct FKAggregateGeom AggGeom;

	/** 
	 *	If true (and bEnableFullAnimWeightBodies in SkelMeshComp is true), the physics of this bone will always be blended into the skeletal mesh, regardless of what PhysicsWeight of the SkelMeshComp is. 
	 *	This is useful for bones that should always be physics, even when blending physics in and out for hit reactions (eg cloth or pony-tails).
	 */
	UPROPERTY()
	uint8 bAlwaysFullAnimWeight_DEPRECATED:1;

	/** 
	 *	Should this BodySetup be considered for the bounding box of the PhysicsAsset (and hence SkeletalMeshComponent).
	 *	There is a speed improvement from having less BodySetups processed each frame when updating the bounds.
	 */
	UPROPERTY(EditAnywhere, Category=BodySetup)
	uint8 bConsiderForBounds:1;

	/** 
	 *	If true, the physics of this mesh (only affects static meshes) will always contain ALL elements from the mesh - not just the ones enabled for collision. 
	 *	This is useful for forcing high detail collisions using the entire render mesh.
	 */
	UPROPERTY(Transient)
	uint8 bMeshCollideAll:1;

	/**
	*	If true, the physics triangle mesh will use double sided faces when doing scene queries.
	*	This is useful for planes and single sided meshes that need traces to work on both sides.
	*/
	UPROPERTY(EditAnywhere, Category=Physics)
	uint8 bDoubleSidedGeometry : 1;

	/**	Should we generate data necessary to support collision on normal (non-mirrored) versions of this body. */
	UPROPERTY()
	uint8 bGenerateNonMirroredCollision:1;

	/** Whether the cooked data is shared by multiple body setups. This is needed for per poly collision case where we don't want to duplicate cooked data, but still need multiple body setups for in place geometry changes */
	UPROPERTY()
	uint8 bSharedCookedData : 1;

	/** 
	 *	Should we generate data necessary to support collision on mirrored versions of this mesh. 
	 *	This halves the collision data size for this mesh, but disables collision on mirrored instances of the body.
	 */
	UPROPERTY()
	uint8 bGenerateMirroredCollision:1;

	/** 
	 * If true, the physics triangle mesh will store UVs and the face remap table. This is needed
	 * to support physical material masks in scene queries.
	 * Note: Also set bSupportFaceRemapOnMeshBVH to false to fully remove the remap table.
	 */
	UPROPERTY()
	uint8 bSupportUVsAndFaceRemap:1;

	/**
	 * If true, or the above bSupportUVsAndFaceRemap is true, the physics triangle mesh will store the face remap table.
	 * If both are false, the physics triangle mesh will use less memory, and queries for the original face index will return INDEX_NONE
	 */
	UPROPERTY()
	uint8 bSupportFaceRemapOnMeshBVH : 1;

	/**
	 * If true, the physics triangle mesh will store the vertex remap table. This is needed
	 * to support skeletal mesh per-poly collision, and any other FBodyInstance::UpdateTriMeshVertices calls.
	 * Both this flag and the p.Chaos.TriMeshPerPolySupport cvar must be true for the vertex remap table to be set.
	 */
	UPROPERTY()
	uint8 bSupportVertexRemap:1;

	/** Flag used to know if we have created the physics convex and tri meshes from the cooked data yet */
	uint8 bCreatedPhysicsMeshes:1;

	/** Flag used to know if we have failed to create physics meshes. Note that this is not the inverse of bCreatedPhysicsMeshes which is true even on failure */
	uint8 bFailedToCreatePhysicsMeshes:1;

	/** Indicates whether this setup has any cooked collision data. This is set when writing to or reading from cooked data, but not authoritative for assets in the editor. */
	uint8 bHasCookedCollisionData:1;
	/**
	 * Reports whether CookedFormatProvider was set and we should give an error when trying to read CookedData if
	 * the provider is no longer available. Transient flag set in CreateBodySetup.
	 */
	uint8 bHasCookedFormatProvider : 1;

	/** 
	 * Indicates that we will never use convex or trimesh shapes. This is an optimization to skip checking for binary data.
	 * TODO Chaos this is to opt out of CreatePhysicsMeshes for certain meshes
	 * Better long term mesh is to not call CreatePhysicsMeshes until it is known there is a mesh instance that needs it.
	 */
	UPROPERTY(EditAnywhere, Category = Collision)
	uint8 bNeverNeedsCookedCollisionData:1;
	
	/** Physical material to use for simple collision on this body. Encodes information about density, friction etc. */
	UPROPERTY(EditAnywhere, Category=Physics, meta=(DisplayName="Simple Collision Physical Material"))
	TObjectPtr<class UPhysicalMaterial> PhysMaterial;

	/** Custom walkable slope setting for this body. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Physics)
	struct FWalkableSlopeOverride WalkableSlopeOverride;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float BuildScale_DEPRECATED;
#endif

	/** Cooked physics data for each format */
	FFormatContainer CookedFormatData;

#if WITH_EDITORONLY_DATA
	/** 
	 * The format container may be queried from multiple threads and mutated if a thread fails to find a format.
	 * This mutex protects the cook path in TryGetCookedData so we don't have multiple threads attempting to
	 * write to the format data
	 */
	FRWLock CookedFormatDataMutex;
#endif

	/** GUID used to uniquely identify this setup so it can be found in the DDC */
	FGuid BodySetupGuid;
	
	/** list of chaos trimesh objects */
	TArray<Chaos::FTriangleMeshImplicitObjectPtr> TriMeshGeometries;

	/** Additional UV info, if available. Used for determining UV for a line trace impact. */
	FBodySetupUVInfo UVInfo;

	/** Additional face remap table, if available. Used for determining face index mapping from collision mesh to static mesh, for use with physical material masks */
	TArray<int32> FaceRemap;

	/** Default properties of the body instance, copied into objects on instantiation, was URB_BodyInstance */
	UPROPERTY(EditAnywhere, Category=Collision, meta=(FullyExpand = "true"))
	FBodyInstance DefaultInstance;

	/**
	 * Cooked physics data override. This is needed in cases where some other body setup has the
	 * cooked data and you don't want to own it or copy it. See per poly skeletal mesh.
	 * This is a transient pointer that can be set during CreateBodySetup.
	 */
	TWeakObjectPtr<const USkeletalMesh> CookedFormatProvider;

	/** Build scale for this body setup (static mesh settings define this value) */
	UPROPERTY()
	FVector BuildScale3D;

	/** References the current async cook helper. Used to be able to abort a cook task */
	using FAsyncCookHelper = Chaos::FCookHelper;
	FAsyncCookHelper* CurrentCookHelper;

	// Will contain deserialized data from the serialization function that can be used at PostLoad time.
	TUniquePtr<FChaosDerivedDataReader<float, 3>> ChaosDerivedDataReader;
	
	UE_DEPRECATED(5.4, "Please use TriMeshGeometries instead")
    TArray<TSharedPtr<Chaos::FTriangleMeshImplicitObject, ESPMode::ThreadSafe>> ChaosTriMeshes;

public:
	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void BeginDestroy() override;
	virtual void FinishDestroy() override;
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface.

	//
	//~ Begin UBodySetup Interface.
	//
	ENGINE_API void CopyBodyPropertiesFrom(const UBodySetup* FromSetup);

	/** Add collision shapes from another body setup to this one */
	ENGINE_API void AddCollisionFrom(class UBodySetup* FromSetup);
	ENGINE_API void AddCollisionFrom(const FKAggregateGeom& FromAggGeom);
	ENGINE_API bool AddCollisionElemFrom(const FKAggregateGeom& FromAggGeom, const EAggCollisionShape::Type ShapeType, const int32 ElemIndex);
	
	/** Create Physics meshes (ConvexMeshes, TriMesh & TriMeshNegX) from cooked data */
	/** Release Physics meshes (ConvexMeshes, TriMesh & TriMeshNegX). Must be called before the BodySetup is destroyed */
	ENGINE_API virtual void CreatePhysicsMeshes();

	/** Create Physics meshes (ConvexMeshes, TriMesh & TriMeshNegX) from cooked data async (useful for runtime cooking as it can go wide off the game thread) */
	/** Release Physics meshes (ConvexMeshes, TriMesh & TriMeshNegX). Must be called before the BodySetup is destroyed */
	/** NOTE: You cannot use the body setup until this operation is done. You must create the physics state (call CreatePhysicsState, or InitBody, etc..) , this does not automatically update the BodyInstance state for you */
	ENGINE_API void CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished OnAsyncPhysicsCookFinished);

	/** Aborts an async cook that hasn't begun. See CreatePhysicsMeshesAsync.  (Useful for cases where frequent updates at runtime would otherwise cause a backlog) */
	ENGINE_API void AbortPhysicsMeshAsyncCreation();

	enum class EBodySetupCookResult
	{
		NoError,
		FailedToLock,
		FailedToCook,
		DataFormatProviderUnavailable,
	};

	struct FBodySetupTryCookResult
	{
		FByteBulkData* Data;
		EBodySetupCookResult CookResult;
	};

private:
	FByteBulkData* GetCookedFormatData();

	/**
	* Given a format name returns its cooked data.
	*
	* @param Format Physics format name.
	* @return Cooked data or NULL of the data was not found.
	*/
	FByteBulkData* GetCookedData(FName Format);

	/**
	 * Attempt to get cooked physics data, with extended information about
	 * potential failure cases during the cooking process
	 * @param Format Physics format name
	 * @return Cooked data, nullptr if no data or failure occured, plus a failure code
	 */
	FBodySetupTryCookResult TryGetCookedData(FName Format);

	// #TODO MRMesh for some reason needs to be able to call this - that case needs fixed to correctly use the create meshes flow
	friend class UMRMeshComponent;
	bool ProcessFormatData_Chaos(FByteBulkData* FormatData);
	bool ProcessFormatData_Chaos(FChaosDerivedDataReader<float, 3>& Reader);
	bool RuntimeCookPhysics_Chaos();
	void FinishCreatingPhysicsMeshes_Chaos(FChaosDerivedDataReader<float, 3>& InReader);
	void FinishCreatingPhysicsMeshes_Chaos(Chaos::FCookHelper& InHelper);
	void FinishCreatingPhysicsMeshes_Chaos(TArray<Chaos::FConvexPtr>& InConvexImplicits, 
										   TArray<Chaos::FTriangleMeshImplicitObjectPtr>& InTrimeshImplicits,
										   FBodySetupUVInfo& InUvInfo,
										   TArray<int32>& InFaceRemap);

	/** 
	 * Finalize game thread data before calling back user's delegate 
	 * @param AsyncPhysicsCookHelper - The cook helper that has finished async cooking
	 * @param OnAsyncPhysicsCookFinished - User callback to call once we're finished
	 */
	void FinishCreatePhysicsMeshesAsync(FAsyncCookHelper* AsyncPhysicsCookHelper, FOnAsyncPhysicsCookFinished OnAsyncPhysicsCookFinished);

public:

	/**
	 * Generate a string to uniquely describe the state of the geometry in this setup to populate the DDC
	 *
	 * @param OutString The generated string will be place in this FString
	 */
	void GetGeometryDDCKey(FString& OutString) const;

	/** Returns the volume of this element */
	UE_DEPRECATED(5.1, "Use GetScaledVolume which uses the same scaling technique as the generated collision geometry")
	ENGINE_API virtual float GetVolume(const FVector& Scale) const;

	/** Returns the volume of this element givent the scale */
	ENGINE_API virtual FVector::FReal GetScaledVolume(const FVector& Scale, EVolumeCalculationMethod InMethod = EVolumeCalculationMethod::AllGeoms) const;

	/** Release Physics meshes (ConvexMeshes, TriMesh & TriMeshNegX) */
	ENGINE_API void ClearPhysicsMeshes();

	/** Calculates the mass. You can pass in the component where additional information is pulled from ( Scale, PhysMaterialOverride ) */
	ENGINE_API virtual float CalculateMass(const UPrimitiveComponent* Component = nullptr) const;

	/** Returns the physics material used for this body. If none, specified, returns the default engine material. */
	ENGINE_API class UPhysicalMaterial* GetPhysMaterial() const;

	/** Clear all simple collision */
	ENGINE_API void RemoveSimpleCollision();

	/** 
	 * Rescales simple collision geometry.  Note you must recreate physics meshes after this 
	 *
	 * @param BuildScale	The scale to apply to the geometry
	 */
	ENGINE_API void RescaleSimpleCollision( FVector BuildScale );

	/** Invalidate physics data */
	ENGINE_API virtual void	InvalidatePhysicsData();	

	/**
	 * Converts a UModel to a set of convex hulls for simplified collision.  Any convex elements already in
	 * this BodySetup will be destroyed.  WARNING: the input model can have no single polygon or
	 * set of coplanar polygons which merge to more than FPoly::MAX_VERTICES vertices.
	 *
	 * @param		InModel					The input BSP.
	 * @param		bRemoveExisting			If true, clears any pre-existing collision
	 * @return								true on success, false on failure because of vertex count overflow.
	 */
	ENGINE_API bool CreateFromModel(class UModel* InModel, bool bRemoveExisting);

	/**
	 * Updates the tri mesh collision with new positions, and refits the BVH to match. 
	 * This is not a full collision cook, and so you can only safely move positions and not change the structure
	 * @param	NewPositions		The new mesh positions to use
	 */
	ENGINE_API void UpdateTriMeshVertices(const TArray<FVector> & NewPositions);

	/**	
	 * Finds the shortest distance between the body setup and a world position. Input and output are given in world space
	 * @param	WorldPosition	The point we are trying to get close to
	 * @param	BodyToWorldTM	The transform to convert BodySetup into world space
	 * @param	bUseConvexShapes When true also check the convex shapes if any (false by default)
	 * @return					The distance between WorldPosition and the body setup. 0 indicates WorldPosition is inside one of the shapes.
	 *
	 * NOTE: This function ignores trimesh data
	 */
	ENGINE_API float GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BodyToWorldTM, bool bUseConvexShapes = false) const;

	/** 
	 * Finds the closest point in the body setup. Input and outputs are given in world space.
	 * @param	WorldPosition			The point we are trying to get close to
	 * @param	BodyToWorldTM			The transform to convert BodySetup into world space
	 * @param	ClosestWorldPosition	The closest point on the body setup to WorldPosition
	 * @param	FeatureNormal			The normal of the feature associated with ClosestWorldPosition
	 * @param	bUseConvexShapes When true also check the convex shapes if any (false by default)
	 * @return							The distance between WorldPosition and the body setup. 0 indicates WorldPosition is inside one of the shapes.
	 *
	 * NOTE: This function ignores trimesh data
	 */
	ENGINE_API float GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BodyToWorldTM, FVector& ClosestWorldPosition, FVector& FeatureNormal, bool bUseConvexShapes = false) const;

	/**
	* Generates the information needed for cooking geometry.
	* @param	OutCookInfo				Info needed during cooking
	* @param	InCookFlags				Any flags desired for TriMesh cooking
	*/
	ENGINE_API void GetCookInfo(FCookBodySetupInfo& OutCookInfo, EPhysXMeshCookFlags InCookFlags) const;

	/** 
	 *	Given a location in body space, and face index, find the UV of the desired UV channel.
	 *	Note this ONLY works if 'Support UV From Hit Results' is enabled in Physics Settings.
	 */
	bool CalcUVAtLocation(const FVector& BodySpaceLocation, int32 FaceIndex, int32 UVChannel, FVector2D& UV) const;


#if WITH_EDITOR
	ENGINE_API virtual void BeginCacheForCookedPlatformData(  const ITargetPlatform* TargetPlatform ) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(  const ITargetPlatform* TargetPlatform ) override;
	ENGINE_API virtual void ClearCachedCookedPlatformData(  const ITargetPlatform* TargetPlatform ) override;

	/*
	* Copy all UPROPERTY settings except the collision geometry.
	* This function is use when we restore the original data after a re-import of a static mesh.
	* All FProperty should be copy here except the collision geometry (i.e. AggGeom)
	*/
	ENGINE_API virtual void CopyBodySetupProperty(const UBodySetup* Other);
#endif // WITH_EDITOR

	/** 
	 *   Add the shapes defined by this body setup to the supplied PxRigidBody. 
	 */
	ENGINE_API void AddShapesToRigidActor_AssumesLocked(
		FBodyInstance* OwningInstance, 
		FVector& Scale3D, 
		UPhysicalMaterial* SimpleMaterial,
		TArray<UPhysicalMaterial*>& ComplexMaterials,
		TArray<FPhysicalMaterialMaskParams>& ComplexMaterialMasks,
		const FBodyCollisionData& BodyCollisionData,
		const FTransform& RelativeTM = FTransform::Identity, 
		TArray<FPhysicsShapeHandle>* NewShapes = NULL);

	friend struct FIterateBodySetupHelper;

};
