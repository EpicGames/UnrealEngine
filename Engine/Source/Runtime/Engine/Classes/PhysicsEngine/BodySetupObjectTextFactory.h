// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Factories.h"
#include "PhysicsEngine/BodySetup.h"

class FBodySetupObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FBodySetupObjectTextFactory() : FCustomizableTextObjectFactory(GWarn) { }
	ENGINE_API virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override;
	ENGINE_API virtual void ProcessConstructedObject(UObject* NewObject) override;

public:
	TArray<UBodySetup*> NewBodySetups;
};

#endif // WITH_EDITOR
