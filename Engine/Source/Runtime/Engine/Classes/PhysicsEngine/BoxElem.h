// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "BoxElem.generated.h"

class FMaterialRenderProxy;
class FMeshElementCollector;

/** Box shape used for collision */
USTRUCT()
struct FKBoxElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FMatrix TM_DEPRECATED;
	UPROPERTY()
	FQuat Orientation_DEPRECATED;
#endif

	/** Position of the box's origin */
	UPROPERTY(Category=Box, EditAnywhere)
	FVector Center;

	/** Rotation of the box */
	UPROPERTY(Category=Box, EditAnywhere, meta = (ClampMin = "-360", ClampMax = "360"))
	FRotator Rotation;

	/** Extent of the box along the x-axis */
	UPROPERTY(Category= Box, EditAnywhere, meta =(DisplayName = "X Extent"))
	float X;

	/** Extent of the box along the y-axis */
	UPROPERTY(Category= Box, EditAnywhere, meta = (DisplayName = "Y Extent"))
	float Y;

	/** Extent of the box along the z-axis */
	UPROPERTY(Category= Box, EditAnywhere, meta = (DisplayName = "Z Extent"))
	float Z;


	FKBoxElem()
	: FKShapeElem(EAggCollisionShape::Box)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
	, Orientation_DEPRECATED( FQuat::Identity )
#endif
	, Center( FVector::ZeroVector )
	, Rotation( FRotator::ZeroRotator )
	, X(1), Y(1), Z(1)
	{

	}

	FKBoxElem( float s )
	: FKShapeElem(EAggCollisionShape::Box)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
	, Orientation_DEPRECATED( FQuat::Identity )
#endif
	, Center( FVector::ZeroVector )
	, Rotation(FRotator::ZeroRotator)
	, X(s), Y(s), Z(s)
	{

	}

	FKBoxElem( float InX, float InY, float InZ ) 
	: FKShapeElem(EAggCollisionShape::Box)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
	, Orientation_DEPRECATED( FQuat::Identity )
#endif
	, Center( FVector::ZeroVector )
	, Rotation(FRotator::ZeroRotator)
	, X(InX), Y(InY), Z(InZ)
	{

	}

	ENGINE_API virtual ~FKBoxElem();

#if WITH_EDITORONLY_DATA
	void FixupDeprecated( FArchive& Ar );
#endif

	friend bool operator==( const FKBoxElem& LHS, const FKBoxElem& RHS )
	{
		return ( LHS.Center == RHS.Center &&
			LHS.Rotation == RHS.Rotation &&
			LHS.X == RHS.X &&
			LHS.Y == RHS.Y &&
			LHS.Z == RHS.Z );
	};

	// Utility function that builds an FTransform from the current data
	virtual FTransform GetTransform() const override final
	{
		return FTransform( Rotation, Center );
	};

	void SetTransform( const FTransform& InTransform )
	{
		ensure(InTransform.IsValid());
		Rotation = InTransform.Rotator();
		Center = InTransform.GetLocation();
	}

	UE_DEPRECATED(5.1, "Changed to GetScaledVolume. Note that Volume calculation now includes non-uniform scale so values may have changed")
	FVector::FReal GetVolume(const FVector& Scale3D) const { return GetScaledVolume(Scale3D); }

	FVector::FReal GetScaledVolume(const FVector& Scale3D) const { return FMath::Abs(Scale3D.X * Scale3D.Y * Scale3D.Z * X * Y * Z); }

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const override;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const;
	
	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const;

	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, float Scale) const;

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	ENGINE_API FKBoxElem GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const;

	ENGINE_API static EAggCollisionShape::Type StaticShapeType;

	/**	
	 * Finds the shortest distance between the element and a world position. Input and output are given in world space
	 * @param	WorldPosition	The point we are trying to get close to
	 * @param	BodyToWorldTM	The transform to convert BodySetup into world space
	 * @return					The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside one of the shapes.
	 */
	ENGINE_API float GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BodyToWorldTM) const;

	
	/**	
	 * Finds the closest point on the shape given a world position. Input and output are given in world space
	 * @param	WorldPosition			The point we are trying to get close to
	 * @param	BodyToWorldTM			The transform to convert BodySetup into world space
	 * @param	ClosestWorldPosition	The closest point on the shape in world space
	 * @param	Normal					The normal of the feature associated with ClosestWorldPosition.
	 * @return							The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside the shape.
	 */
	ENGINE_API float GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BodyToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const;
};
