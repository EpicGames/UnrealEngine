// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ChaosBlueprintLibrary.generated.h"

class UChaosEventRelay;

UCLASS()
class UChaosBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintPure, Category = "Physics Object")
	static const UChaosEventRelay* GetEventRelayFromContext(UObject* ContextObject);
};