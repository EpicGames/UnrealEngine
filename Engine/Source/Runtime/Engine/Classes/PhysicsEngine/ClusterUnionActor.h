// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ClusterUnionActor.generated.h"

class UClusterUnionComponent;

/**
 * A lightweight actor that can be used to own a cluster union component.
 */
UCLASS(MinimalAPI)
class AClusterUnionActor : public AActor
{
	GENERATED_BODY()
public:
	ENGINE_API AClusterUnionActor(const FObjectInitializer& ObjectInitializer);

	UFUNCTION()
	UClusterUnionComponent* GetClusterUnionComponent() const { return ClusterUnion; }

protected:
	/** The pivot used while building. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cluster Union")
	TObjectPtr<UClusterUnionComponent> ClusterUnion;
};
