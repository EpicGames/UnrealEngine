// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/PhysicsObject.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Engine/EngineTypes.h"
#include "Logging/LogMacros.h"
#include "PhysicsEngine/ExternalSpatialAccelerationPayload.h"
#include "PhysicsEngine/SafePhysicsObjectHandle.h"
#include "PhysicsInterfaceTypesCore.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "UObject/ObjectKey.h"

#include "ClusterUnionComponent.generated.h"

class AActor;
struct FExternalSpatialAccelerationPayload;
class FPhysScene_Chaos;
class UClusterUnionReplicatedProxyComponent;

DECLARE_LOG_CATEGORY_EXTERN(LogClusterUnion, Log, All);

namespace Chaos
{
	class FPBDRigidsSolver;

	template <typename TPayload, typename T, int d>
	class ISpatialAccelerationCollection;
}

USTRUCT()
struct FClusterUnionBoneData
{
	GENERATED_BODY()

	FClusterUnionBoneData() = default;
	FClusterUnionBoneData(int32 InBoneId) : ID(InBoneId)
	{	
	}
	
	FClusterUnionBoneData(int32 InBoneId, Chaos::FUniqueIdx InParticleID) : ID(InBoneId), ParticleID(InParticleID)
	{	
	}

	int32 ID = INDEX_NONE;
	Chaos::FUniqueIdx ParticleID;

	bool operator==(const FClusterUnionBoneData& Other) const
	{
		return ID == Other.ID;
	}

	bool operator!=(const FClusterUnionBoneData& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FClusterUnionBoneData& Data)
	{
		return GetTypeHash(Data.ID);
	}
};

USTRUCT()
struct FClusteredComponentData
{
	GENERATED_BODY()
	
	FClusteredComponentData() = default;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "This property is deprecated. Please use BonesData instead.")
	// Set of bone Ids that we actually added into the cluster union.
	TSet<int32> BoneIds;

	UE_DEPRECATED(5.4, "This property is deprecated and no longer used. It will be removed in future versions.")
	// Cached acceleration structure handles - needed to properly cleanup the component from the accel structure.
	TSet<FExternalSpatialAccelerationPayload> CachedAccelerationPayloads;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FClusteredComponentData(const FClusteredComponentData& Other) = default;
	FClusteredComponentData(FClusteredComponentData&& Other) = default;
	FClusteredComponentData& operator=(const FClusteredComponentData& Other) = default;
	FClusteredComponentData& operator=(FClusteredComponentData&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	// Array of bone Ids that we actually added into the cluster union.
	TArray<FClusterUnionBoneData> BonesData;

	// Using a TWeakObjectPtr here because the UClusterUnionReplicatedProxyComponent will have a pointer back
	// and we don't want to get into a situation where a circular reference occurs.
	UPROPERTY()
	TWeakObjectPtr<UClusterUnionReplicatedProxyComponent> ReplicatedProxyComponent;

	UPROPERTY()
	TWeakObjectPtr<AActor> Owner;

	UPROPERTY()
	bool bWasReplicating = true;

	UPROPERTY()
	bool bPendingDeletion = false;
};

USTRUCT()
struct FClusteredActorData
{
	GENERATED_BODY()

	TSet<TObjectKey<UPrimitiveComponent>> Components;

	UPROPERTY()
	bool bWasReplicatingMovement = true;
};

USTRUCT()
struct FClusterUnionReplicatedData
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 ObjectState = 0;

	UPROPERTY()
	bool bIsAnchored = true;

	FClusterUnionReplicatedData& operator=(const FClusterUnionReplicatedData&) = default;
	bool operator==(const FClusterUnionReplicatedData&) const = default;
	bool operator!=(const FClusterUnionReplicatedData&) const = default;
};

USTRUCT()
struct FClusterUnionPendingAddData
{
	GENERATED_BODY()

	FClusterUnionPendingAddData() = default;
	
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "This property is deprecated. Please use BonesData instead.")
	TArray<int32> BoneIds;

	UE_DEPRECATED(5.4, "This property is deprecated and no longer used. It will be removed in future versions.")
	UPROPERTY()
	TArray<FExternalSpatialAccelerationPayload> AccelerationPayloads;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FClusterUnionPendingAddData(const FClusterUnionPendingAddData& Other) = default;
	FClusterUnionPendingAddData(FClusterUnionPendingAddData&& Other) = default;
	FClusterUnionPendingAddData& operator=(const FClusterUnionPendingAddData& Other) = default;
	FClusterUnionPendingAddData& operator=(FClusterUnionPendingAddData&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	TSet<FClusterUnionBoneData> BonesData;
};

USTRUCT()
struct FClusterUnionInitializationData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UClusterUnionComponent> ClusterUnionComponent;

	UPROPERTY()
	TArray<TObjectPtr<UPrimitiveComponent>> ProcessedComponents;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnClusterUnionAddedComponent, UPrimitiveComponent*, Component, const TSet<int32>&, BoneIds, bool, bIsNew);
DECLARE_MULTICAST_DELEGATE_FourParams(FOnClusterUnionAddedComponentNative, UPrimitiveComponent*, const TArray<FClusterUnionBoneData>& /*BoneIds*/, const TArray<FClusterUnionBoneData>& /*RemovedBoneIds*/, bool /*bIsNew*/);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnClusterUnionRemovedComponent, UPrimitiveComponent*, Component);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClusterUnionRemovedComponentNative, UPrimitiveComponent*, const TArray<FClusterUnionBoneData>& /*RemovedBonesData*/);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnClusterUnionBoundsChanged, UClusterUnionComponent*, Component, const FBoxSphereBounds&, Bounds);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnClusterUnionPostSyncBodies, const FClusterUnionInitializationData&);

/**
 * This does the bulk of the work exposing a physics cluster union to the game thread.
 * This component needs to be a primitive component primarily because of how physics
 * proxies need to be registered with the solver with an association with a primitive component.
 * This component can be used as part of AClusterUnionActor or on its own as its list of
 * clustered components/actors can be specified dynamically at runtime and/or statically
 * on asset creation.
 * 
 * The cluster union component needs to not only maintain a game thread representation of what's happening on the
 * physics thread but it also needs to make sure this data gets replicated to every client. A general model of how
 * the data flow happens is as follows:
 * 
 *  [Server GT Command] -> [Server PT Command] -> [Server Modifies PT Data] -> [Server Sync PT Data back to GT Data].
 * 
 * This enables GT control over what happens to the cluster union BUT ALSO maintains a physics-first approach
 * to the cluster union where a physics event can possibly cause the cluster union to break.
 * 
 * The GT data is replicated from the server to the clients either via the FClusterUnionReplicatedData on the cluster union component
 * or per-child component data is replicated via the UClusterUnionReplicatedProxyComponent. Generally, the same flow is
 * replicated on the client. The only exception is for replicating the X/R/V/W properties on the cluster union particle which does
 * a GT -> PT data sync. There's no particula reason this happens...it just mirrors the single particle physics proxy here.
 *
 */
UCLASS(MinimalAPI)
class UClusterUnionComponent : public UPrimitiveComponent
{
	GENERATED_BODY()
public:

	using FSpatialAcceleration = Chaos::ISpatialAcceleration<FExternalSpatialAccelerationPayload, Chaos::FReal, 3>;

	ENGINE_API UClusterUnionComponent(const FObjectInitializer& ObjectInitializer);
	ENGINE_API virtual ~UClusterUnionComponent();

	UFUNCTION(BlueprintCallable, Category="Cluster Union")
	ENGINE_API void AddComponentToCluster(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds, bool bRebuildGeometry = true);

	UFUNCTION(BlueprintCallable, Category = "Cluster Union")
	ENGINE_API void RemoveComponentFromCluster(UPrimitiveComponent* InComponent);

	UFUNCTION(BlueprintCallable, Category = "Cluster Union")
	ENGINE_API void RemoveComponentBonesFromCluster(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds);

	UFUNCTION(BlueprintCallable, Category = "Cluster Union")
	ENGINE_API TArray<UPrimitiveComponent*> GetPrimitiveComponents();

	UFUNCTION(BlueprintCallable, Category = "Cluster Union")
	ENGINE_API TArray<AActor*> GetActors();

	UE_DEPRECATED(5.5, "SetIsAnchored is deprecated, Use SetSimulatePhysics instead")
	UFUNCTION(BlueprintCallable, Category = "Cluster Union", meta = (DeprecatedFunction, DeprecationMessage = "SetIsAnchored is deprecated, Use SetSimulatePhysics instead"))
	ENGINE_API virtual void SetIsAnchored(bool bIsAnchored);

	ENGINE_API bool IsAnchored() const;

	UFUNCTION(BlueprintCallable, Category = "Cluster Union")
	ENGINE_API void SetEnableDamageFromCollision(bool bValue);

	/** Whether or not collisions against this geometry collection will apply strain which could cause the geometry collection to fracture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetEnableDamageFromCollision, Category = "Cluster Union")
	bool bEnableDamageFromCollision;

	/** Structure that stores an object key and its raw ptr, so it can be accessed without going trough
	* the global uobjects array (so it could be garbage, therefore it has to be used withing the same frame)
	*/
	struct FMappedComponentKey
	{
		FMappedComponentKey() : ComponentPtr(nullptr)
		{
		}

		FMappedComponentKey(TObjectKey<UPrimitiveComponent> InComponentKey, UPrimitiveComponent* InComponent)
		{
			ComponentKey = InComponentKey;
			ComponentPtr = InComponent;
		}
		
		FMappedComponentKey(UPrimitiveComponent* InComponent)
		{
			ComponentKey = InComponent;
			ComponentPtr = InComponent;
		}
		
		bool operator<(const FMappedComponentKey& Other) const
		{
			return ComponentKey < Other.ComponentKey;
		}

		bool operator==(const FMappedComponentKey& Other) const
		{
			return ComponentKey == Other.ComponentKey;
		}

		friend uint32 GetTypeHash(const FMappedComponentKey& InMappedComponentKey)
		{
			return GetTypeHash(InMappedComponentKey.ComponentKey);
		}

		TObjectKey<UPrimitiveComponent> ComponentKey;
		UPrimitiveComponent* ComponentPtr;
	};

	struct FMappedBoneData
	{
		FMappedBoneData() = default;
		
		FMappedBoneData(Chaos::FPhysicsObjectHandle PhysicsObjectHande, Chaos::FPBDRigidParticle* RigidParticle, const Chaos::FUniqueIdx& ParticleID, const FTransform& ChildToParentTransform)
			: PhysicsObjectHandle(PhysicsObjectHande)
			, RigidParticle(RigidParticle)
			, ParticleID(ParticleID)
			, ChildToParentTransform(ChildToParentTransform)
		{
		}

		Chaos::FPhysicsObjectHandle PhysicsObjectHandle = nullptr;
		Chaos::FPBDRigidParticle* RigidParticle = nullptr;
		Chaos::FUniqueIdx ParticleID;
		FTransform ChildToParentTransform;
	};

	// There are several instances where we have 1 single elements. The inline allocator should help in these cases without a huge impact when we go over
	using FLocalBonesToTransformMap = TSortedMap<int32, FMappedBoneData, TInlineAllocator<1>>;

	// SyncClusterUnionFromProxy will examine the make up of the cluster union (particles, child to parent, etc.) and do whatever is needed on the GT in terms of bookkeeping.
	ENGINE_API void SyncClusterUnionFromProxy(const FTransform& NewTransform, TArray<TTuple<UPrimitiveComponent*, FTransform>>* OutNewComponents);

	UFUNCTION()
	bool IsComponentAdded(UPrimitiveComponent* Component) { return PerComponentData.Contains(Component) || PendingComponentSync.Contains(Component); }

	ENGINE_API virtual void WakeAllRigidBodies() override;
	ENGINE_API virtual bool IsAnyRigidBodyAwake() override;

	// Set the cluster union total mass. NOTE: if the cluster breaks the mass will be recalculated from the
	// remaining components. Use OnComponentAddedEvent and OnComponentRemovedEvent to trap this and re-apply custom masses.
	ENGINE_API virtual void SetMassOverrideInKg(FName BoneName, float MassInKg, bool bOverrideMass) override;

	// Multi-trace/sweep functions that only make sense in the context of a cluster union.
	ENGINE_API bool LineTraceComponent(TArray<FHitResult>& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams);
	ENGINE_API bool SweepComponent(TArray<FHitResult>& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams);

	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnClusterUnionAddedComponent OnComponentAddedEvent;

	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnClusterUnionRemovedComponent OnComponentRemovedEvent;

	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnClusterUnionBoundsChanged OnComponentBoundsChangedEvent;

	// native (fast, low overhead) versions 
	FOnClusterUnionAddedComponentNative OnComponentAddedNativeEvent;
	FOnClusterUnionRemovedComponentNative OnComponentRemovedNativeEvent;
	FOnClusterUnionPostSyncBodies OnClusterUnionPostSyncBodiesEvent;

	// Lambda returns whether or not iteration should continue;
	ENGINE_API void VisitAllCurrentChildComponents(const TFunction<bool(UPrimitiveComponent*)>& Lambda) const;
	ENGINE_API void VisitAllCurrentActors(const TFunction<bool(AActor*)>& Lambda) const;

	ENGINE_API int32 NumChildClusterComponents() const;

	// Force a rebuild of the GT geometry. This needs to happen immediately when we add/remove on the GT so that the SQ is up to date
	// and doesn't need to wait for the next OnSyncBodies.
	UE_DEPRECATED(5.4, "Cluster unions now incrementally update their geometry. A force rebuild should never be needed - this function will no longer book-keep properly either.")
	ENGINE_API void ForceRebuildGTParticleGeometry();

	ENGINE_API const FSpatialAcceleration* GetSpatialAcceleration() const;

	ENGINE_API TArray<int32> GetAddedBoneIdsForComponent(UPrimitiveComponent* Component) const;
	ENGINE_API void ChangeIfComponentBonesAreMainParticle(UPrimitiveComponent* Component, const TArray<int32>& BoneIds, bool bIsMain);

	// This function will return information from the latest physics sync. Thus adding a new bones
	// to a cluster union will not allow those indices to be valid for use in this function
	// until the next time we sync from the PT.
	ENGINE_API Chaos::FPhysicsObjectHandle FindChildPhysicsObjectByShapeIndex(int32 Index) const;

	friend class UClusterUnionReplicatedProxyComponent;
	friend class UModularVehicleBaseComponent;

	// This should only be called on the client when replication happens.
	UFUNCTION()
	ENGINE_API void ForceSetChildToParent(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds, const TArray<FTransform>& ChildToParent);

	virtual void OnChildToParentUpdated(UPrimitiveComponent* ChangedComponent, const FLocalBonesToTransformMap& PerBoneChildToParent, const FTransform& NewTransform, TArray<TTuple<UPrimitiveComponent*, FTransform>>* OutNewComponents) {}

	ENGINE_API virtual void SetDisabled(const bool Disabled);
	ENGINE_API virtual bool GetDisabled() const;
	
	ENGINE_API virtual void SetSleepType(const Chaos::ESleepType SleepType);
	ENGINE_API virtual Chaos::ESleepType GetSleepType() const;
	
	ENGINE_API virtual void SetRigidState(const Chaos::EObjectStateType ObjectState);
	ENGINE_API virtual Chaos::EObjectStateType GetRigidState() const;

	ENGINE_API virtual void SetGravityEnabled(const bool Enabled);
	ENGINE_API virtual bool GetGravityEnabled() const;
	
protected:

	ENGINE_API void BroadcastComponentAddedEvents(UPrimitiveComponent* ChangedComponent, const TArray<FClusterUnionBoneData>& BoneIds, bool bIsNew, const TArray<FClusterUnionBoneData>& RemovedBoneIDs);
	ENGINE_API void BroadcastComponentRemovedEvents(UPrimitiveComponent* ChangedComponent, const TArray<FClusterUnionBoneData>& InRemovedBonesData);

	Chaos::FClusterUnionPhysicsProxy* GetPhysicsProxy() const { return PhysicsProxy; }
	Chaos::FClusterUnionPhysicsProxy* GetPhysicsProxy() { return PhysicsProxy; }

	// We need to keep track of the mapping of primitive components to physics objects.
	// This way we know the right physics objects to pass when removing the component (because
	// it's possible to get a different list of physics objects when we get to removal). A
	// side benefit here is being able to track which components are clustered.
	TMap<TObjectKey<UPrimitiveComponent>, FClusteredComponentData> PerComponentData;


	// Whether or not this code is running on the server.
	UFUNCTION()
	ENGINE_API bool IsAuthority() const;

	const FClusterUnionReplicatedData& GetReplicatedRigidState() const { return ReplicatedRigidState; }

private:
	// These are the statically clustered components. These should
	// be specified in the editor and never change.
	UPROPERTY(EditAnywhere, Category = "Cluster Union")
	TArray<FComponentReference> ClusteredComponentsReferences;

	// If set to a value not equal to -1, will manually set the cluster union's gravity group
	// instead of automatically inheriting it from its children particles.
	UPROPERTY(EditAnywhere, Category = "Cluster Union")
	int32 GravityGroupIndexOverride;

	// Also keep track of which actors we are clustering and their components. We make modifications on
	// actors that get clustered so we need to make sure we undo those changes only once all its clustered
	// components are removed from the cluster.
	TMap<TObjectKey<AActor>, FClusteredActorData> ActorToComponents;

	// Sometimes we might be in the process of waiting for a component to create it physics state before adding to the cluster.
	// Make sure we don't try to add the component multiples times while the add is pending. Gets removed finally when PT syncs
	// back to the GT with this component.
	TMap<TObjectKey<UPrimitiveComponent>, FClusterUnionPendingAddData> PendingComponentsToAdd;

	// After we add to a cluster union, we need to wait for the sync from the PT back to the GT before removing the component from PendingComponentSync.
	// Before that happens, we need to perform operations on the GT assuming that the component was added already otherwise there'll be a few frames
	// where the component hasn't been added to the cluster union on the GT causing a mismatch in behavior.
	TMap<TObjectKey<UPrimitiveComponent>, FClusterUnionPendingAddData> PendingComponentSync;

	// At every physics sync, we keep track of which shape index matches which child primitive component and bone.
	TArray<FSafePhysicsObjectHandle> PerShapeComponentBone;

	// Data that can be changed at runtime to keep state about the cluster union consistent between the server and client.
	UPROPERTY(ReplicatedUsing=OnRep_RigidState)
	FClusterUnionReplicatedData ReplicatedRigidState;

	// Cached local bounds from the physics particle.
	mutable bool bHasCachedLocalBounds;
	mutable FBoxSphereBounds CachedLocalBounds;

	// If Physics Prediction is enabled
	bool bPhysicsPredictionEnabled;

protected:
	// Handles changes to ReplicatedRigidState. Note that this function does not handle replication of X/R since we make use
	// of the scene component's default replication for that.
	UFUNCTION()
	ENGINE_API virtual void OnRep_RigidState();

private:

	ENGINE_API FPhysScene_Chaos* GetChaosScene() const;

	Chaos::FClusterUnionPhysicsProxy* PhysicsProxy;

	// User data to be able to tie the cluster particle back to this component.
	FChaosUserData PhysicsUserData;

	// An acceleration structure of all children components managed by the cluster union itself.
	TUniquePtr<FSpatialAcceleration> AccelerationStructure;

	/** Takes a source bone data type and extracts Bone IDs from it*/
	template<typename TContainer, typename TSourceData>
	void GetBoneIDsFromComponentData(TContainer& OutBoneIDs, const TSourceData& InSourceData) const
	{
		static_assert(std::is_same_v<TSourceData, FClusterUnionPendingAddData> || std::is_same_v<TSourceData, FClusteredComponentData>, "Only FClusterUnionPendingAddData and FClusteredComponentData are supported");

		OutBoneIDs.Reset();
		OutBoneIDs.Reserve(InSourceData.BonesData.Num());

		Algo::Transform(InSourceData.BonesData, OutBoneIDs, &FClusterUnionBoneData::ID);
	}

	// Need to handle the fact that this component may or may not be initialized prior to the components referenced in
	// ClusteredComponentsReferences. This function lets us listen to OnComponentPhysicsStateChanged on the incoming
	// primitive component so that once the physics state is properly created we can begin the process of adding it.
	UFUNCTION()
	ENGINE_API void HandleComponentPhysicsStateChange(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);

	// Once in the cluster union, if the component's physics state is destroyed, we should remove it from the cluster union.
	UFUNCTION()
	ENGINE_API void HandleComponentPhysicsStateChangePostAddIntoClusterUnion(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);

	// These functions only get called when the physics thread syncs to the game thread thereby enforcing a physics thread authoritative view of
	// what particles are currently contained within the cluster union.
	ENGINE_API void HandleAddOrModifiedClusteredComponent(const FMappedComponentKey& ChangedComponentData, const FLocalBonesToTransformMap& PerBoneChildToParent, const FTransform& NewTransform, TArray<TTuple<UPrimitiveComponent*, FTransform>>* OutNewComponents);
	ENGINE_API void HandleRemovedClusteredComponent(TObjectKey<UPrimitiveComponent> RemovedComponent, const FClusteredComponentData& ComponentData);

	ENGINE_API TArray<UPrimitiveComponent*> GetAllCurrentChildComponents() const;
	ENGINE_API TArray<AActor*> GetAllCurrentActors() const;
	ENGINE_API void VisitAllCurrentChildComponentsForCollision(ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const TFunction<bool(UPrimitiveComponent*)>& Lambda) const;

	// Merge all the physics objects geometries into the cluster union
	ENGINE_API void AddGTParticleGeometry(const TArray<Chaos::FPhysicsObjectHandle>& PhysicsObjects);

	// Remove all the physics objects geometries from the cluster union
	ENGINE_API void RemoveGTParticleGeometry(const TSet<Chaos::FPhysicsObjectHandle>& PhysicsObjects);

	//~ Begin UActorComponent Interface
public:
	ENGINE_API virtual void OnCreatePhysicsState() override;
	ENGINE_API virtual void OnDestroyPhysicsState() override;
	ENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	ENGINE_API virtual bool HasValidPhysicsState() const override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UActorComponent Interface

	//~ Begin UPrimitiveComponent Interface
public:
	using UPrimitiveComponent::LineTraceComponent;
	using UPrimitiveComponent::SweepComponent;
	using UPrimitiveComponent::OverlapComponentWithResult;

	virtual FBodyInstance* GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const override { return nullptr; }
	ENGINE_API virtual void SetSimulatePhysics(bool bSimulate) override;
	virtual bool CanEditSimulatePhysics() override { return true; }
	ENGINE_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams) override;
	ENGINE_API virtual bool SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams) override;
	ENGINE_API virtual bool OverlapComponentWithResult(const FVector& Pos, const FQuat& Rot, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& OutOverlap) const override;
	ENGINE_API virtual bool ComponentOverlapComponentWithResultImpl(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FQuat& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const override;
	virtual bool ShouldDispatchWakeEvents(FName BoneName) const override { return true; }
	ENGINE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	//~ End UPrimitiveComponent Interface

	//~ Begin USceneComponent Interface
public:
	ENGINE_API virtual void OnReceiveReplicatedState(const FVector X, const FQuat R, const FVector V, const FVector W) override;
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
	ENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	ENGINE_API virtual FVector GetComponentVelocity() const override;
	//~ End USceneComponent Interface

	//~ Begin IPhysicsComponent Interface.
public:
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	ENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectByName(const FName& Name) const override;
	ENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	ENGINE_API virtual Chaos::FPhysicsObjectId GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const override;
	//~ End IPhysicsComponent Interface.

	//~ Begin UObject Interface.
public:
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.
};
