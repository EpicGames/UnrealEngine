// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "GenericPlatform/GenericPlatformMisc.h"

#include "ClusterUnionReplicatedProxyComponent.generated.h"

class UClusterUnionComponent;
class UPrimitiveComponent;

/**
 * This component lets us store replicated information about how any particular UPrimitiveComponent
 * should be attached to its parent cluster union. The benefits of using a separate components are:
 * 
 *	1) It lets us avoid adding any additional overhead into the UPrimitiveComponent.
 *  2) It lets the replicated information have the same net relevancy as the actor being added to the cluster union
 *     rather than having the same net relevancy as the cluster union (i.e. in the case of replicating this data in
 *     an array in the ClusterUnionComponent).
 *  3) It lets us pinpoint what exactly is being added/removed (vs if all this data was stored in an array) which lets
 *     us be a bit more efficient in terms of modifying the cluster union.
 */
UCLASS(MinimalAPI)
class UClusterUnionReplicatedProxyComponent : public UActorComponent
{
	GENERATED_BODY()
public:
	ENGINE_API UClusterUnionReplicatedProxyComponent(const FObjectInitializer& ObjectInitializer);

	UFUNCTION()
	UClusterUnionComponent* GetParentClusterUnionComponent() const { return ParentClusterUnion.Get(); }

	UFUNCTION()
	ENGINE_API void SetParentClusterUnion(UClusterUnionComponent* InComponent);

	UFUNCTION()
	UPrimitiveComponent* GetChildClusteredComponent() const { return ChildClusteredComponent.Get(); }

	UFUNCTION()
	ENGINE_API void SetChildClusteredComponent(UPrimitiveComponent* InComponent);

	UFUNCTION()
	ENGINE_API void SetParticleBoneIds(const TArray<int32>& InIds);

	UFUNCTION()
	const TArray<int32>& GetParticleBoneIds() const { return ParticleBoneIds; }

	UFUNCTION()
	ENGINE_API void SetParticleChildToParent(int32 BoneId, const FTransform& ChildToParent);

	UFUNCTION()
	void MarkPendingDeletion() { bIsPendingDeletion = true; }

	UFUNCTION()
	bool IsPendingDeletion() const { return bIsPendingDeletion; }

	/* reinitialize the internal transient state of the replicated proxy */
	ENGINE_API void ResetTransientState();

	const TArray<FTransform>& GetParticleChildToParents() { return ParticleChildToParents; }

protected:

	UFUNCTION()
	ENGINE_API void OnRep_ParentClusterUnion();

	UFUNCTION()
	ENGINE_API void OnRep_ChildClusteredComponent();

	UFUNCTION()
	ENGINE_API void OnRep_ParticleBoneIds();

	UFUNCTION()
	ENGINE_API void OnRep_ParticleChildToParents();

private:
	UPROPERTY(ReplicatedUsing=OnRep_ParentClusterUnion)
	TWeakObjectPtr<UClusterUnionComponent> ParentClusterUnion;

	UPROPERTY(ReplicatedUsing=OnRep_ChildClusteredComponent)
	TWeakObjectPtr<UPrimitiveComponent> ChildClusteredComponent;

	UPROPERTY(ReplicatedUsing=OnRep_ParticleBoneIds)
	TArray<int32> ParticleBoneIds;
	TSet<int32> LastSyncedBoneIds;

	UPROPERTY(ReplicatedUsing=OnRep_ParticleChildToParents)
	TArray<FTransform> ParticleChildToParents;

	UPROPERTY()
	bool bNetUpdateParentClusterUnion;

	UPROPERTY()
	bool bNetUpdateChildClusteredComponent;

	UPROPERTY()
	bool bNetUpdateParticleBoneIds;

	UPROPERTY()
	bool bNetUpdateParticleChildToParents;

	UPROPERTY()
	bool bIsPendingDeletion = false;

	FTimerHandle DeferSetChildToParentHandle;
	ENGINE_API void DeferSetChildToParentChildUntilClusteredComponentInParentUnion();

	ENGINE_API void AddComponentToCluster();

	//~ Begin UActorComponent Interface
public:
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UActorComponent Interface

	//~ Begin UObject Interface
public:
	ENGINE_API virtual void PostRepNotifies() override;
	//~ End UObject Interface

private:
	void FlushNetDormancyIfNeeded();
};
