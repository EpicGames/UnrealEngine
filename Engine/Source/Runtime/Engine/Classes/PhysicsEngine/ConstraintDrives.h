// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "ConstraintDrives.generated.h"

UENUM()
namespace EAngularDriveMode
{
	enum Type : int
	{
		/** Spherical lerp between the current orientation/velocity and the target orientation/velocity. NOTE: This will NOT work if any angular constraints are set to Locked. */
		SLERP,
		/** Path is decomposed into twist (roll constraint) and swing (cone constraint). Doesn't follow shortest arc and may experience gimbal lock. Does work with locked angular constraints. */
		TwistAndSwing
	};
}

// Specifies how the constraint responds when the angular drive target would violate the joint's angular limits.
// Without a response, the drive and the limit constraint compete, producing spurious torques.
UENUM(BlueprintType)
enum class EAngularDriveLimitViolationResponse : uint8
{
	// Drive target rotation is left unchanged. The drive and the limit constraint may compete.
	None       = 0,
	// Project the drive target onto the joint limits using a linear swing-angle approximation.
	// Accurate for small angles; exact for pure Swing1 or pure Swing2 rotations.
	ClampLinear = 1,
	// Project the drive target onto the joint limits using the exact Chaos 4*atan2 formula.
	// More accurate near the limit boundary but slightly more expensive than ClampLinear.
	ClampExact  = 2,
	// Widen the joint's angular limits to accommodate the drive target instead of clipping the
	// target. Limits are never narrowed below their authored defaults; they are restored each
	// frame when the target moves back inside. Has no effect on locked axes.
	WidenLimits = 3,
};


USTRUCT()
struct FConstraintDrive
{
	GENERATED_BODY()

	/** The spring strength of the drive. Force proportional to the position error. In force mode the unit is kg * cm / s^-2. In acceleration mode the unit is cm / s^-2. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (ClampMin = "0.0"))
	float Stiffness;

	/** The damping strength of the drive. Force proportional to the velocity error. In force mode the unit is kg * cm / s^-1. In acceleration mode the unit is cm / s^-1.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (ClampMin = "0.0"))
	float Damping;

	/** The force limit of the drive. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (ClampMin = "0.0"))
	float MaxForce;

	/** Enables/Disables position drive (orientation if using angular drive)*/
	UPROPERTY(EditAnywhere, Category = Constraint)
	uint8 bEnablePositionDrive : 1;

	/** Enables/Disables velocity drive (angular velocity if using angular drive) */
	UPROPERTY(EditAnywhere, Category = Constraint)
	uint8 bEnableVelocityDrive : 1;

	ENGINE_API FConstraintDrive();

	/** Updates physx drive with properties from unreal */
	//void UpdatePhysXDrive_AssumesLocked(physx::PxD6Joint* Joint, int DriveType, bool bDriveEnabled) const;

private:
	friend struct FConstraintInstance;
	friend struct FLinearDriveConstraint;
	friend struct FAngularDriveConstraint;
	//These functions may leave the struct in an invalid state unless calling UpdatePhysX* functions.
	//They are only meant as helpers for FConstraintInstance
	ENGINE_API void SetDriveParams(float InStiffness, float InDamping, float InForceLimit);


};

/** Linear Drive */
USTRUCT()
struct FLinearDriveConstraint
{
	GENERATED_BODY()

	/** Target position the linear drive.*/
	UPROPERTY(EditAnywhere, Category = LinearMotor)
	FVector PositionTarget;

	/** Target velocity the linear drive. */
	UPROPERTY(EditAnywhere, Category = LinearMotor)
	FVector VelocityTarget;

	UPROPERTY(EditAnywhere, Category = LinearMotor)
	FConstraintDrive XDrive;

	UPROPERTY(EditAnywhere, Category = LinearMotor)
	FConstraintDrive YDrive;

	UPROPERTY(EditAnywhere, Category = LinearMotor)
	FConstraintDrive ZDrive;
	
	/** Whether the specified strength is acceleration or force. (default: acceleration) */
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	bool bAccelerationMode = true;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Enable/disable of drives is done inside the individual constraint drives."))
	uint8 bEnablePositionDrive_DEPRECATED : 1;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	ENGINE_API FLinearDriveConstraint();

	bool IsPositionDriveEnabled() const
	{
		return XDrive.bEnablePositionDrive || YDrive.bEnablePositionDrive || ZDrive.bEnablePositionDrive;
	}

	bool IsVelocityDriveEnabled() const
	{
		return XDrive.bEnableVelocityDrive || YDrive.bEnableVelocityDrive || ZDrive.bEnableVelocityDrive;
	}

	//These functions may leave the struct in an invalid state unless calling UpdatePhysX* functions.
	//They are only meant as helpers for FConstraintInstance
	ENGINE_API void SetLinearPositionDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive);
	ENGINE_API void SetLinearVelocityDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive);
	ENGINE_API void SetDriveParams(float InStiffness, float InDamping, float InForceLimit);
	ENGINE_API void SetDriveParams(const FVector& InStiffness, const FVector& InDamping, const FVector& InForceLimit);
	ENGINE_API void GetDriveParams(float& OutStiffness, float& OutDamping, float& OutForceLimit) const;
	ENGINE_API void GetDriveParams(FVector& OutStiffness, FVector& OutDamping, FVector& OutForceLimit) const;
	ENGINE_API void SetAccelerationMode(bool bAccelerationMode);
	ENGINE_API bool GetAccelerationMode() const;
};


/** Angular Drive */
USTRUCT()
struct FAngularDriveConstraint
{
	GENERATED_BODY()

	/** Controls the twist (roll) constraint drive between current orientation/velocity and target orientation/velocity. This is available as long as the twist limit is set to free or limited.*/
	UPROPERTY(EditAnywhere, Category = LinearMotor, meta = (DisplayName="Twist"))
	FConstraintDrive TwistDrive;

	/** Controls the cone constraint drive between current orientation/velocity and target orientation/velocity. This is available as long as there is at least one swing limit set to free or limited. */
	UPROPERTY(EditAnywhere, Category = LinearMotor, meta = (DisplayName = "Swing"))
	FConstraintDrive SwingDrive;

	/** Controls the SLERP (spherical lerp) drive between current orientation/velocity and target orientation/velocity. NOTE: This is only available when all three angular limits are either free or limited. Locking any angular limit will turn off the drive implicitly.*/
	UPROPERTY(EditAnywhere, Category = LinearMotor, meta = (DisplayName = "SLERP"))
	FConstraintDrive SlerpDrive;
	
	/** Target orientation relative to the the body reference frame.*/
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	FRotator OrientationTarget;

	/** Target angular velocity relative to the body reference frame in revolutions per second. */
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	FVector AngularVelocityTarget;

	/** Whether motors use SLERP (spherical lerp) or decompose into a Swing motor (cone constraints) and Twist motor (roll constraints). NOTE: SLERP will NOT work if any of the angular constraints are locked. */
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	TEnumAsByte<enum EAngularDriveMode::Type> AngularDriveMode;

	/** Whether to use acceleration mode for angular drive. The other option is force mode. (default:true) */
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	bool bAccelerationMode = true;

	// Specifies how the constraint responds when the drive target rotation would violate the joint's
	// angular limits. Without a response, the drive and the limit constraint compete, producing
	// spurious torques.
	UPROPERTY(EditAnywhere, Category = AngularMotor)
	EAngularDriveLimitViolationResponse LimitViolationResponse = EAngularDriveLimitViolationResponse::None;

	ENGINE_API FAngularDriveConstraint();

	bool IsOrientationDriveEnabled() const
	{
		if(AngularDriveMode == EAngularDriveMode::TwistAndSwing)
		{
			return TwistDrive.bEnablePositionDrive || SwingDrive.bEnablePositionDrive;
		}
		else
		{
			return SlerpDrive.bEnablePositionDrive;
		}
	}

	bool IsVelocityDriveEnabled() const
	{
		if (AngularDriveMode == EAngularDriveMode::TwistAndSwing)
		{
			return TwistDrive.bEnableVelocityDrive || SwingDrive.bEnableVelocityDrive;
		}
		else
		{
			return SlerpDrive.bEnableVelocityDrive;
		}
		
	}

	/** Updates physx drive with properties from unreal */
	//void UpdatePhysXAngularDrive_AssumesLocked(physx::PxD6Joint* Joint) const;

	//These functions may leave the struct in an invalid state unless calling UpdatePhysX* functions.
	//They are only meant as helpers for FConstraintInstance
	ENGINE_API void SetOrientationDriveTwistAndSwing(bool InEnableTwistDrive, bool InEnableSwingDrive);
	ENGINE_API void SetOrientationDriveSLERP(bool InEnableSLERP);
	ENGINE_API void SetAngularVelocityDriveTwistAndSwing(bool InEnableTwistDrive, bool InEnableSwingDrive);
	ENGINE_API void SetAngularVelocityDriveSLERP(bool InEnableSLERP);
	// Sets the three drive parameters (swing, twist and slerp) to the same value 
	ENGINE_API void SetDriveParams(float InStiffness, float InDamping, float InForceLimit);
	// Sets drive parameters in the order swing, twist, slerp
	ENGINE_API void SetDriveParams(const FVector& InStiffness, const FVector& InDamping, const FVector& InForceLimit);
	// Gets just the swing drive parameters - assuming the single-float set function has previously been used
	ENGINE_API void GetDriveParams(float& OutStiffness, float& OutDamping, float& OutForceLimit) const;
	// Gets drive parameters in the order swing, twist, slerp
	ENGINE_API void GetDriveParams(FVector& OutStiffness, FVector& OutDamping, FVector& OutForceLimit) const;
	ENGINE_API void SetAngularDriveMode(EAngularDriveMode::Type DriveMode);
	ENGINE_API void SetAccelerationMode(bool bAccelerationMode);
	ENGINE_API bool GetAccelerationMode() const;
};
