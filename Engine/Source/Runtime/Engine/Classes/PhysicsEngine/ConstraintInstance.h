// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Chaos/PhysicsObject.h"
#include "EngineDefines.h"
#include "PhysicsEngine/ConstraintTypes.h"
#include "PhysicsEngine/ConstraintDrives.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "ConstraintInstance.generated.h"

class FMaterialRenderProxy;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class UMaterialInterface;
struct FBodyInstance;

class FMaterialRenderProxy;
class FPrimitiveDrawInterface;
class FMaterialRenderProxy;
class UPhysicsAsset;
struct FReferenceSkeleton;

namespace Chaos
{
	struct FSerializedDataBuffer;
}

UENUM()
enum class EConstraintTransformComponentFlags : uint8
{
	None = 0,

	ChildPosition = 1 << 0,
	ChildRotation = 1 << 1,
	ParentPosition = 1 << 2,
	ParentRotation = 1 << 3,

	AllChild = ChildPosition | ChildRotation,
	AllParent = ParentPosition | ParentRotation,
	AllPosition = ChildPosition | ParentPosition,
	AllRotation = ChildRotation | ParentRotation,
	All = AllChild | AllParent
};

ENUM_CLASS_FLAGS(EConstraintTransformComponentFlags);

/** Returns the 'To' bone's transform relative to the 'From' bone. */
ENGINE_API FTransform CalculateRelativeBoneTransform(const FName ToBoneName, const FName FromBoneName, const FReferenceSkeleton& ReferenceSkeleton);

/** Container for properties of a physics constraint that can be easily swapped at runtime. This is useful for switching different setups when going from ragdoll to standup for example */
USTRUCT()
struct FConstraintProfileProperties
{
	GENERATED_USTRUCT_BODY()

	/** If the joint error is above this distance after the solve phase, the child body will be teleported to fix the error. Only used if bEnableProjection is true. */
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0.0"))
	float ProjectionLinearTolerance;

	/** If the joint error is above this distance after the solve phase, the child body will be teleported to fix the error. Only used if bEnableProjection is true. */
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0.0"))
	float ProjectionAngularTolerance;

	/**  How much semi-physical linear projection correction to apply [0-1]. Only used if bEnableProjection is true and if hard limits are used. */
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float ProjectionLinearAlpha;

	/** How much semi-physical angular projection correction to apply [0-1]. Only used if bEnableProjection is true and if hard limits are used. */
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float ProjectionAngularAlpha;

	/** 
	 * How much shock propagation to apply [0-1]. Shock propagation increases the mass of the parent body for the last iteration of the
	 * position and velocity solve phases. This can help stiffen up joint chains, but is also prone to introducing energy down the chain
	 * especially at high alpha. Only used in bEnableShockPropagation is true.
	 */
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float ShockPropagationAlpha;

	/** Force needed to break the distance constraint. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear, meta = (editcondition = "bLinearBreakable", ClampMin = "0.0"))
	float LinearBreakThreshold;

	/** [Chaos Only] Percent threshold from center of mass distance needed to reset the linear drive position target. This value can be greater than 1. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear, meta = (editcondition = "bLinearPlasticity", ClampMin = "0.0"))
	float LinearPlasticityThreshold;

	/** Torque needed to break the joint. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Angular, meta = (editcondition = "bAngularBreakable", ClampMin = "0.0"))
	float AngularBreakThreshold;

	/** [Chaos Only] Degree threshold from target angle needed to reset the target angle.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Angular, meta = (editcondition = "bAngularPlasticity", ClampMin = "0.0"))
	float AngularPlasticityThreshold;

	/** [Chaos Only] Colliison transfer on parent from the joints child. Range is 0.0-MAX*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear, meta = (ClampMin = "0.0"))
	float ContactTransferScale;

	UPROPERTY(EditAnywhere, Category = Linear)
	FLinearConstraint LinearLimit;

	UPROPERTY(EditAnywhere, Category = Angular)
	FConeConstraint ConeLimit;

	UPROPERTY(EditAnywhere, Category = Angular)
	FTwistConstraint TwistLimit;

	// Disable collision between bodies joined by this constraint.
	UPROPERTY(EditAnywhere, Category = Constraint)
	uint8 bDisableCollision : 1;

	// When set, the parent body in a constraint will not be affected by the motion of the child
	UPROPERTY(EditAnywhere, Category = Constraint)
	uint8 bParentDominates : 1;

	/**
	 * Shock propagation increases the mass of the parent body for the last iteration of the position and velocity solve phases. 
	 * This can help stiffen up joint chains, but is also prone to introducing energy down the chain especially at high alpha.
	 * It also does not work well if there are collisions on the bodies preventing the joints from correctly resolving - this 
	 * can lead to jitter, especially if collision shock propagation is also enabled.
	 */
	UPROPERTY(EditAnywhere, Category = Projection)
	uint8 bEnableShockPropagation : 1;

	/**
	* Projection is a post-solve position and angular fixup consisting of two correction procedures. First, if the constraint limits are exceeded
	* by more that the Linear or Angular Tolerance, the bodies are teleported to eliminate the error. Second, if the constraint limits are exceeded
	* by less than the tolerance, a semi-physical correction is applied,  with the parent body in the constraint is treated as having infinite mass.
	* The teleport tolerance are controlled by ProjectionLinearTolerance and ProjectionAngularTolerance. The semi-physical correction is controlled
	* by ProjectionLinearAlpha and ProjectionAnguilarAlpha. You may have one, none, or both systems enabled at the same time.
	*
	* Projection only works well if the chain is not interacting with other objects (e.g., through collisions) because the projection of the bodies in
	* the chain will cause other constraints to be violated. Likewise, if a body is influenced by multiple constraints, then enabling projection on more
	* than one constraint may lead to unexpected results - the  "last" constraint would win but the order in which constraints are solved cannot be directly controlled.
	*
	* Note that the semi-physical projection (ProjectionLinearAlpha and ProjectionAngularAlpha) is only applied to hard-limit constraints and not those with
	* soft limits because the soft limit is the point at which the soft-constraint (spring) kicks in, and not really a limit on how far the joint can be separated.
	*/
	UPROPERTY(EditAnywhere, Category = Projection)
	uint8 bEnableProjection : 1;

	/** 
	 * Whether mass conditioning is enabled for this joint. Mass conditioning applies a non-physical scale to the mass and inertia of the two
	 * bodies that only affects this joint, so that the mass and inertia ratios are smaller. This helps stabilize joints where the bodies
	 * are very different sizes, and especially when the parent body is heavier than the child. However, it can lead to unrealistic
	 * behaviour, especially when collisions are involved.
	*/
	UPROPERTY(EditAnywhere, Category = Projection)
	uint8 bEnableMassConditioning : 1;

	/** World Solver: Whether to use linear solver for this joint. Linear solver is faster and less accurate. The nonlinear solver is slower and more convergent.  */
	UPROPERTY(EditAnywhere, Category = Projection)
	uint8 bUseLinearJointSolver : 1;

	/** Whether it is possible to break the joint with angular force. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Angular)
	uint8 bAngularBreakable : 1;

	/** Whether it is possible to reset target rotations from the angular displacement. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Angular)
	uint8 bAngularPlasticity : 1;

	/** Whether it is possible to break the joint with linear force. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear)
	uint8 bLinearBreakable : 1;

	/** Whether it is possible to reset spring rest length from the linear deformation. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear)
	uint8 bLinearPlasticity : 1;

	UPROPERTY(EditAnywhere, Category = Linear)
	FLinearDriveConstraint LinearDrive;

	UPROPERTY(EditAnywhere, Category = Angular)
	FAngularDriveConstraint AngularDrive;

	/** Whether linear plasticity has a operation mode [free]*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Linear)
	TEnumAsByte<enum EConstraintPlasticityType> LinearPlasticityType;

	ENGINE_API FConstraintProfileProperties();

	/** Updates physx joint properties from unreal properties (limits, drives, flags, etc...) */
	ENGINE_API void Update_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass, float UseScale, bool InInitialize = false) const;

	/** Updates joint breakable properties (threshold, etc...)*/
	ENGINE_API void UpdateBreakable_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const;

	/** Updates joint breakable properties (threshold, etc...)*/
	ENGINE_API void UpdatePlasticity_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const;

	/** Updates joint linear mass scales.*/
	ENGINE_API void UpdateContactTransferScale_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const;

	/** Updates joint flag based on profile properties */
	ENGINE_API void UpdateConstraintFlags_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const;

#if WITH_EDITOR
	ENGINE_API void SyncChangedConstraintProperties(struct FPropertyChangedChainEvent& PropertyChangedEvent);
#endif
};

USTRUCT()
struct FConstraintInstanceBase
{
	GENERATED_USTRUCT_BODY()

	/** Constructor **/
	ENGINE_API FConstraintInstanceBase();
	ENGINE_API void Reset();


	/** Indicates position of this constraint within the array in SkeletalMeshComponent. */
	int32 ConstraintIndex;

	// Internal physics constraint representation
	FPhysicsConstraintHandle ConstraintHandle;

	// Scene thats using the constraint
	FPhysScene* PhysScene;

	FPhysScene* GetPhysicsScene() { return PhysScene; }
	const FPhysScene* GetPhysicsScene() const { return PhysScene; }

	/** Set the constraint broken delegate. */
	ENGINE_API void SetConstraintBrokenDelegate(FOnConstraintBroken InConstraintBrokenDelegate);

	/** Set the constraint violation delegate */
	ENGINE_API void SetConstraintViolatedDelegate(FOnConstraintViolated InConstraintViolatedDelegate);

	/** Set the plastic deformation delegate. */
	ENGINE_API void SetPlasticDeformationDelegate(FOnPlasticDeformation InPlasticDeformationDelegate);

	protected:

		FOnConstraintBroken OnConstraintBrokenDelegate;
		FOnConstraintViolated OnConstraintViolatedDelegate;
		FOnPlasticDeformation OnPlasticDeformationDelegate;

		friend struct FConstraintBrokenDelegateData;
		friend struct FConstraintBrokenDelegateWrapper;
		friend struct FConstraintViolatedDelegateWrapper;
		friend struct FPlasticDeformationDelegateWrapper;

};

/** Container for a physics representation of an object. */
USTRUCT()
struct FConstraintInstance : public FConstraintInstanceBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone that this joint is associated with. */
	UPROPERTY(VisibleAnywhere, Category=Constraint)
	FName JointName;

	///////////////////////////// CONSTRAINT GEOMETRY
	
	/** 
	 *	Name of first bone (body) that this constraint is connecting. 
	 *	This will be the 'child' bone in a PhysicsAsset.
	 */
	UPROPERTY(EditAnywhere, Category=Constraint)
	FName ConstraintBone1;

	/** 
	 *	Name of second bone (body) that this constraint is connecting. 
	 *	This will be the 'parent' bone in a PhysicsAset.
	 */
	UPROPERTY(EditAnywhere, Category=Constraint)
	FName ConstraintBone2;

private:
	/** The component scale passed in during initialization*/
	float LastKnownScale;

public:

	///////////////////////////// Body1 ref frame
	
	/** Location of constraint in Body1 reference frame (usually the "child" body for skeletal meshes). */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FVector Pos1;

	/** Primary (twist) axis in Body1 reference frame. */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FVector PriAxis1;

	/** Secondary axis in Body1 reference frame. Orthogonal to PriAxis1. */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FVector SecAxis1;

	///////////////////////////// Body2 ref frame
	
	/** Location of constraint in Body2 reference frame (usually the "parent" body for skeletal meshes). */
	UPROPERTY(EditAnywhere, Category= Constraint)
	FVector Pos2;

	/** Primary (twist) axis in Body2 reference frame. */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FVector PriAxis2;

	/** Secondary axis in Body2 reference frame. Orthogonal to PriAxis2. */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FVector SecAxis2;

	/** Specifies the angular offset between the two frames of reference. By default limit goes from (-Angle, +Angle)
	* This allows you to bias the limit for swing1 swing2 and twist. */
	UPROPERTY(EditAnywhere, Category = Angular)
	FRotator AngularRotationOffset;

	/** If true, linear limits scale using the absolute min of the 3d scale of the owning component */
	UPROPERTY(EditAnywhere, Category = Linear)
	uint32 bScaleLinearLimits : 1;

	float AverageMass;

	//Constraint Data (properties easily swapped at runtime based on different constraint profiles)
	UPROPERTY(EditAnywhere, Category = Constraint)
	FConstraintProfileProperties ProfileInstance;

public:
	/** Copies behavior properties from the given profile. Automatically updates the physx representation if it's been created */
	ENGINE_API void CopyProfilePropertiesFrom(const FConstraintProfileProperties& FromProperties);

	/** Get underlying physics engine constraint */
	ENGINE_API const FPhysicsConstraintHandle& GetPhysicsConstraintRef() const;

	/** Serialized out the physics thread state of this constraint as an array of bytes */
	UE_INTERNAL void SerializeState(Chaos::FSerializedDataBuffer& OutSerializedState, Chaos::ESerializedDataContext DataContext);

	/** Deserializes the provided array of bytes representing a saved physics thread state and applies it to this constraint */
	UE_INTERNAL void ApplySerializedState(Chaos::FSerializedDataBufferPtr&& InSerializedState, Chaos::ESerializedDataContext DataContext);

	FChaosUserData UserData;

	/** Constructor **/
	ENGINE_API FConstraintInstance();

	/** Get the child bone name */
	const FName& GetChildBoneName() const { return ConstraintBone1; }

	/** Get the parent bone name */
	const FName& GetParentBoneName() const { return ConstraintBone2; }

	/** Gets the linear limit size */
	float GetLinearLimit() const
	{
		return ProfileInstance.LinearLimit.Limit;
	}

	/** Sets the Linear XYZ Motion Type and the limit distance (Note distance is the same for all 3 axes) */
	void SetLinearLimits(ELinearConstraintMotion XConstraintType, ELinearConstraintMotion YConstraintType, ELinearConstraintMotion ZConstraintType, float InLinearLimitSize)
	{
		ProfileInstance.LinearLimit.XMotion = XConstraintType;
		ProfileInstance.LinearLimit.YMotion = YConstraintType;
		ProfileInstance.LinearLimit.ZMotion = ZConstraintType;
		ProfileInstance.LinearLimit.Limit = InLinearLimitSize;
		UpdateLinearLimit();
	}

	/** Gets the motion type for the linear X-axis limit. */
	ELinearConstraintMotion GetLinearXMotion() const
	{
		return ProfileInstance.LinearLimit.XMotion;
	}

	/** Sets the Linear XMotion type */
	void SetLinearXMotion(ELinearConstraintMotion XConstraintType)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(XConstraintType, prevLimits.YMotion, prevLimits.ZMotion, prevLimits.Limit);
	}

	/** Sets the LinearX Motion Type and the limit distance (Note distance is the same for all 3 axes) */
	void SetLinearXLimit(ELinearConstraintMotion XConstraintType, float InLinearLimitSize)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(XConstraintType, prevLimits.YMotion, prevLimits.ZMotion, InLinearLimitSize);
	}

	/** Gets the motion type for the linear Y-axis limit. */
	ELinearConstraintMotion GetLinearYMotion() const
	{
		return ProfileInstance.LinearLimit.YMotion;
	}

	/** Sets the Linear YMotion type */
	void SetLinearYMotion(ELinearConstraintMotion YConstraintType)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(prevLimits.XMotion, YConstraintType, prevLimits.ZMotion, prevLimits.Limit);
	}

	/** Sets the LinearY Motion Type and the limit distance (Note distance is the same for all 3 axes) */
	void SetLinearYLimit(ELinearConstraintMotion YConstraintType, float InLinearLimitSize)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(prevLimits.XMotion, YConstraintType, prevLimits.ZMotion, InLinearLimitSize);
	}

	/** Gets the motion type for the linear Z-axis limit. */
	ELinearConstraintMotion GetLinearZMotion() const
	{
		return ProfileInstance.LinearLimit.ZMotion;
	}

	/** Sets the Linear ZMotion type */
	void SetLinearZMotion(ELinearConstraintMotion ZConstraintType)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(prevLimits.XMotion, prevLimits.YMotion, ZConstraintType, prevLimits.Limit);
	}

	/** Sets the LinearZ Motion Type and the limit distance (Note distance is the same for all 3 axes) */
	void SetLinearZLimit(ELinearConstraintMotion ZConstraintType, float InLinearLimitSize)
	{
		const FLinearConstraint& prevLimits = ProfileInstance.LinearLimit;
		SetLinearLimits(prevLimits.XMotion, prevLimits.YMotion, ZConstraintType, InLinearLimitSize);
	}

	/** Gets the motion type for the swing1 of the cone constraint */
	EAngularConstraintMotion GetAngularSwing1Motion() const
	{
		return ProfileInstance.ConeLimit.Swing1Motion;
	}

	/** Sets the cone limit's swing1 motion type */
	void SetAngularSwing1Motion(EAngularConstraintMotion MotionType)
	{
		ProfileInstance.ConeLimit.Swing1Motion = MotionType;
		UpdateAngularLimit();
	}

	// The current swing1 of the constraint
	ENGINE_API float GetCurrentSwing1() const;

	/** Gets the cone limit swing1 angle in degrees */
	float GetAngularSwing1Limit() const
	{
		return ProfileInstance.ConeLimit.Swing1LimitDegrees;
	}

	/** Sets the Angular Swing1 Motion Type
	*	@param MotionType	New Motion Type
	*/
	void SetAngularSwing1Limit(EAngularConstraintMotion MotionType, float InSwing1LimitAngle)
	{
		ProfileInstance.ConeLimit.Swing1Motion = MotionType;
		ProfileInstance.ConeLimit.Swing1LimitDegrees = InSwing1LimitAngle;
		UpdateAngularLimit();
	}
	
	/** Gets the motion type for the swing2 of the cone constraint */
	EAngularConstraintMotion GetAngularSwing2Motion() const
	{
		return ProfileInstance.ConeLimit.Swing2Motion;
	}

	/** Sets the cone limit's swing2 motion type */
	void SetAngularSwing2Motion(EAngularConstraintMotion MotionType)
	{
		ProfileInstance.ConeLimit.Swing2Motion = MotionType;
		UpdateAngularLimit();
	}

	// The current swing2 of the constraint
	ENGINE_API float GetCurrentSwing2() const;

	/** Gets the cone limit swing2 angle in degrees */
	float GetAngularSwing2Limit() const
	{
		return ProfileInstance.ConeLimit.Swing2LimitDegrees;
	}

	/** Sets the Angular Swing2 Motion Type
	*	@param MotionType	New Motion Type
	*/
	void SetAngularSwing2Limit(EAngularConstraintMotion MotionType, float InSwing2LimitAngle)
	{
		ProfileInstance.ConeLimit.Swing2Motion = MotionType;
		ProfileInstance.ConeLimit.Swing2LimitDegrees = InSwing2LimitAngle;
		UpdateAngularLimit();
	}

	/** Gets the motion type for the twist of the cone constraint */
	EAngularConstraintMotion GetAngularTwistMotion() const
	{
		return ProfileInstance.TwistLimit.TwistMotion;
	}

	/** Sets the twist limit's motion type */
	void SetAngularTwistMotion(EAngularConstraintMotion MotionType)
	{
		ProfileInstance.TwistLimit.TwistMotion = MotionType;
		UpdateAngularLimit();
	}

	// The current twist of the constraint
	ENGINE_API float GetCurrentTwist() const;

	/** Gets the twist limit angle in degrees */
	float GetAngularTwistLimit() const
	{
		return ProfileInstance.TwistLimit.TwistLimitDegrees;
	}

	/** Sets the Angular Twist Motion Type
	*	@param MotionType	New Motion Type
	*/
	void SetAngularTwistLimit(EAngularConstraintMotion MotionType, float InTwistLimitAngle)
	{
		ProfileInstance.TwistLimit.TwistMotion = MotionType;
		ProfileInstance.TwistLimit.TwistLimitDegrees = InTwistLimitAngle;
		UpdateAngularLimit();
	}

	/** Whether the linear limits are soft (only if at least one axis if Limited) */
	bool GetIsSoftLinearLimit() const { return ProfileInstance.LinearLimit.bSoftConstraint;	}

	/** Linear stiffness if the constraint is set to use soft linear limits */
	float GetSoftLinearLimitStiffness() const {	return ProfileInstance.LinearLimit.Stiffness; }

	/** Linear damping if the constraint is set to use soft linear limits */
	float GetSoftLinearLimitDamping() const { return ProfileInstance.LinearLimit.Damping; }

	/** Linear restitution if the constraint is set to use soft linear limits */
	float GetSoftLinearLimitRestitution() const { return ProfileInstance.LinearLimit.Restitution; }

	/** Linear contact distance if the constraint is set to use soft linear limits */
	float GetSoftLinearLimitContactDistance() const { return ProfileInstance.LinearLimit.ContactDistance; }

	/** Set twist soft limit parameters */
	void SetSoftLinearLimitParams(bool bIsSoftLimit, float Stiffness, float Damping, float Restitution, float ContactDistance)
	{
		ProfileInstance.LinearLimit.bSoftConstraint = bIsSoftLimit;
		ProfileInstance.LinearLimit.Stiffness = Stiffness;
		ProfileInstance.LinearLimit.Damping = Damping;
		ProfileInstance.LinearLimit.Restitution = Restitution;
		ProfileInstance.LinearLimit.ContactDistance = ContactDistance;
		UpdateLinearLimit();
	}

	/** Whether the twist limits are soft (only available if twist is Limited) */
	bool GetIsSoftTwistLimit() const { return ProfileInstance.TwistLimit.bSoftConstraint; }

	/** Twist stiffness if the constraint is set to use soft limits */
	float GetSoftTwistLimitStiffness() const {	return ProfileInstance.TwistLimit.Stiffness; }

	/** Twist damping if the constraint is set to use soft limits */
	float GetSoftTwistLimitDamping() const { return ProfileInstance.TwistLimit.Damping;	}

	/** Twist restitution if the constraint is set to use soft limits */
	float GetSoftTwistLimitRestitution() const { return ProfileInstance.TwistLimit.Restitution; }

	/** Twist contact distance if the constraint is set to use soft limits */
	float GetSoftTwistLimitContactDistance() const { return ProfileInstance.TwistLimit.ContactDistance; }

	/** Set twist soft limit parameters */
	void SetSoftTwistLimitParams(bool bIsSoftLimit, float Stiffness, float Damping, float Restitution, float ContactDistance) 
	{ 
		ProfileInstance.TwistLimit.bSoftConstraint = bIsSoftLimit;
		ProfileInstance.TwistLimit.Stiffness = Stiffness;
		ProfileInstance.TwistLimit.Damping = Damping;
		ProfileInstance.TwistLimit.Restitution = Restitution;
		ProfileInstance.TwistLimit.ContactDistance = ContactDistance;
		UpdateAngularLimit();
	}

	/** Whether the swing limits are soft (only available if swing1 and/or swing2 is Limited) */
	bool GetIsSoftSwingLimit() const { return ProfileInstance.ConeLimit.bSoftConstraint; }

	/** Swing stiffness if the constraint is set to use soft limits */
	float GetSoftSwingLimitStiffness() const { return ProfileInstance.ConeLimit.Stiffness; }

	/** Swing damping if the constraint is set to use soft limits */
	float GetSoftSwingLimitDamping() const { return ProfileInstance.ConeLimit.Damping; }

	/** Swing restitution if the constraint is set to use soft limits */
	float GetSoftSwingLimitRestitution() const { return ProfileInstance.ConeLimit.Restitution; }

	/** Swing Contact distance if the constraint is set to use soft limits */
	float GetSoftSwingLimitContactDistance() const { return ProfileInstance.ConeLimit.ContactDistance; }

	/** Set twist soft limit parameters */
	void SetSoftSwingLimitParams(bool bIsSoftLimit, float Stiffness, float Damping, float Restitution, float ContactDistance)
	{
		ProfileInstance.ConeLimit.bSoftConstraint = bIsSoftLimit;
		ProfileInstance.ConeLimit.Stiffness = Stiffness;
		ProfileInstance.ConeLimit.Damping = Damping;
		ProfileInstance.ConeLimit.Restitution = Restitution;
		ProfileInstance.ConeLimit.ContactDistance = ContactDistance;
		UpdateAngularLimit();
	}

	/** Sets the Linear Breakable properties
	*	@param bInLinearBreakable		Whether it is possible to break the joint with linear force
	*	@param InLinearBreakThreshold	Force needed to break the joint
	*/
	void SetLinearBreakable(bool bInLinearBreakable, float InLinearBreakThreshold)
	{
		ProfileInstance.bLinearBreakable = bInLinearBreakable;
		ProfileInstance.LinearBreakThreshold = InLinearBreakThreshold;
		UpdateBreakable();
	}

	/** Gets Whether it is possible to break the joint with linear force */
	bool IsLinearBreakable() const
	{
		return ProfileInstance.bLinearBreakable;
	}

	/** Gets linear force needed to break the joint */
	float GetLinearBreakThreshold() const
	{
		return ProfileInstance.LinearBreakThreshold;
	}

	/** Sets the Linear Plasticity properties
	*	@param bInLinearPlasticity 	Whether it is possible to reset the target angles
	*	@param InLinearPlasticityThreshold	Delta from target needed to reset the target joint
	*/
	void SetLinearPlasticity(bool bInLinearPlasticity, float InLinearPlasticityThreshold, EConstraintPlasticityType InLinearPlasticityType)
	{
		ProfileInstance.bLinearPlasticity = bInLinearPlasticity;
		ProfileInstance.LinearPlasticityThreshold = InLinearPlasticityThreshold;
		ProfileInstance.LinearPlasticityType = InLinearPlasticityType;
		UpdatePlasticity();
	}

	/** Gets Whether it is possible to reset the target position */
	bool HasLinearPlasticity() const
	{
		return ProfileInstance.bLinearPlasticity;
	}

	/** Gets Delta from target needed to reset the target joint */
	float GetLinearPlasticityThreshold() const
	{
		return ProfileInstance.LinearPlasticityThreshold;
	}

	/** Gets Plasticity Type from joint */
	TEnumAsByte<enum EConstraintPlasticityType> GetLinearPlasticityType() const
	{
		return ProfileInstance.LinearPlasticityType;
	}

	/** Sets the Contact Transfer Scale properties
	*	@param InContactTransferScale 	Contact transfer scale to joints parent
	*/
	void SetContactTransferScale(float InContactTransferScale)
	{
		ProfileInstance.ContactTransferScale = InContactTransferScale;
		UpdateContactTransferScale();
	}

	/** Get the Contact Transfer Scale for the parent of the joint */
	float GetContactTransferScale() const
	{
		return ProfileInstance.ContactTransferScale;
	}

	/** Sets the Angular Breakable properties
	*	@param bInAngularBreakable		Whether it is possible to break the joint with angular force
	*	@param InAngularBreakThreshold	Torque needed to break the joint
	*/
	void SetAngularBreakable(bool bInAngularBreakable, float InAngularBreakThreshold)
	{
		ProfileInstance.bAngularBreakable = bInAngularBreakable;
		ProfileInstance.AngularBreakThreshold = InAngularBreakThreshold;
		UpdateBreakable();
	}

	/** Gets Whether it is possible to break the joint with angular force */
	bool IsAngularBreakable() const
	{
		return ProfileInstance.bAngularBreakable;
	}

	/** Gets Torque needed to break the joint */
	float GetAngularBreakThreshold() const
	{
		return ProfileInstance.AngularBreakThreshold;
	}

	/** Sets the Angular Plasticity properties
	*	@param bInAngularPlasticity 	Whether it is possible to reset the target angles
	*	@param InAngularPlasticityThreshold	Delta from target needed to reset the target joint
	*/
	void SetAngularPlasticity(bool bInAngularPlasticity, float InAngularPlasticityThreshold)
	{
		ProfileInstance.bAngularPlasticity = bInAngularPlasticity;
		ProfileInstance.AngularPlasticityThreshold = InAngularPlasticityThreshold;
		UpdatePlasticity();
	}

	/** Gets Whether it is possible to reset the target angles */
	bool HasAngularPlasticity() const
	{
		return ProfileInstance.bAngularPlasticity;
	}

	/** Gets Delta from target needed to reset the target joint */
	float GetAngularPlasticityThreshold() const
	{
		return ProfileInstance.AngularPlasticityThreshold;
	}

	/**
	 * Projects TargetRelRot (child frame expressed in parent-bone-local space) onto the joint's
	 * angular limits so the drive target is always feasible. Returns the clamped rotation in the
	 * same space. If bExactSwingClamp is true, uses the exact Chaos 4*atan2 swing convention to
	 * match what the physics solver enforces; otherwise a linear axis-projection approximation is
	 * used, which is cheaper but may leave a small residual at the boundary.
	 */
	ENGINE_API FQuat ClampDriveTargetToLimits(const FQuat& TargetRelRot, bool bExactSwingClamp) const;

	/**
	 * Widens this constraint's angular limits so that TargetRelRot sits inside them without
	 * clipping. DefaultConstraint provides the authored minimum values; limits are never narrowed
	 * below those defaults. WidenMarginDegrees is added when the target exceeds a default so the
	 * drive target sits clearly inside the constraint boundary.
	 * Returns true if any axis was left wider than its authored default (i.e. widening was needed),
	 * false if all axes are at their defaults or the axis is locked/free.
	 */
	ENGINE_API bool WidenLimitsForDriveTarget(
		const FQuat&               TargetRelRot,
		const FConstraintInstance& DefaultConstraint,
		float                      WidenMarginDegrees = 0.5f);

	/**
	 * Restores this constraint's angular limits to the authored values stored in DefaultConstraint.
	 * Only touches axes whose motion type is ACM_Limited in DefaultConstraint; does nothing when
	 * the live value already matches the default.
	 */
	ENGINE_API void RestoreAngularLimitsToDefault(const FConstraintInstance& DefaultConstraint);

	// @todo document
	ENGINE_API void CopyConstraintGeometryFrom(const FConstraintInstance* FromInstance);

	// @todo document
	ENGINE_API void CopyConstraintParamsFrom(const FConstraintInstance* FromInstance);

	/** Copies non-identifying properties from another constraint instance */
	ENGINE_API void CopyConstraintPhysicalPropertiesFrom(const FConstraintInstance* FromInstance, bool bKeepPosition, bool bKeepRotation);

	// Retrieve the constraint force most recently applied to maintain this constraint. Returns 0 forces if the constraint is not initialized or broken.
	ENGINE_API void GetConstraintForce(FVector& OutLinearForce, FVector& OutAngularForce);

	// Retrieve the status of constraint being broken.
	ENGINE_API bool IsBroken();

	/** Set which linear position drives are enabled */
	ENGINE_API void SetLinearPositionDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive);

	/** Get which linear position drives is enabled on XAxis */
	bool IsLinearPositionDriveXEnabled() const
	{
		return ProfileInstance.LinearDrive.XDrive.bEnablePositionDrive;
	}

	/** Get which linear position drives is enabled on YAxis */
	bool IsLinearPositionDriveYEnabled() const
	{
		return ProfileInstance.LinearDrive.YDrive.bEnablePositionDrive;
	}

	/** Get which linear position drives is enabled on ZAxis */
	bool IsLinearPositionDriveZEnabled() const
	{
		return ProfileInstance.LinearDrive.ZDrive.bEnablePositionDrive;
	}

	/** Whether the linear position drive is enabled */
	bool IsLinearPositionDriveEnabled() const
	{
		return ProfileInstance.LinearDrive.IsPositionDriveEnabled();
	}

	/** Set the linear drive's target position position */
	ENGINE_API void SetLinearPositionTarget(const FVector& InPosTarget);

	/** Get the linear drive's target position position */
	const FVector& GetLinearPositionTarget()
	{
		return ProfileInstance.LinearDrive.PositionTarget;
	}

	/** Set which linear velocity drives are enabled */
	ENGINE_API void SetLinearVelocityDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive);

	/** Get which linear position drives is enabled on XAxis */
	bool IsLinearVelocityDriveXEnabled() const
	{
		return ProfileInstance.LinearDrive.XDrive.bEnableVelocityDrive;
	}

	/** Get which linear position drives is enabled on YAxis */
	bool IsLinearVelocityDriveYEnabled() const
	{
		return ProfileInstance.LinearDrive.YDrive.bEnableVelocityDrive;
	}

	/** Get which linear position drives is enabled on ZAxis */
	bool IsLinearVelocityDriveZEnabled() const
	{
		return ProfileInstance.LinearDrive.ZDrive.bEnableVelocityDrive;
	}

	/** Whether the linear velocity drive is enabled */
	bool IsLinearVelocityDriveEnabled() const
	{
		return ProfileInstance.LinearDrive.IsVelocityDriveEnabled();
	}

	/** Set the linear drive's target velocity */
	ENGINE_API void SetLinearVelocityTarget(const FVector& InVelTarget);

	/** Get the linear drive's target velocity */
	const FVector& GetLinearVelocityTarget()
	{
		return ProfileInstance.LinearDrive.VelocityTarget;
	}

	/** Set the linear drive's strength parameters */
	ENGINE_API void SetLinearDriveParams(float InPositionStrength, float InVelocityStrength, float InForceLimit);

	/** Set the linear drive's strength parameters per-axis */
	ENGINE_API void SetLinearDriveParams(const FVector& InPositionStrength, const FVector& InVelocityStrength, const FVector& InForceLimit);

	/** Set the linear drive force-mode */
	ENGINE_API void SetLinearDriveAccelerationMode(bool bAccelerationMode);

	/** Get the linear drive's strength parameters. Assumes all axes are the same so only returns the X values */
	ENGINE_API void GetLinearDriveParams(float& OutPositionStrength, float& OutVelocityStrength, float& OutForceLimit);

	/** Get the linear drive's strength parameters. */
	ENGINE_API void GetLinearDriveParams(FVector& OutPositionStrength, FVector& OutVelocityStrength, FVector& OutForceLimit);

	/** Set which twist and swing orientation drives are enabled. Only applicable when Twist And Swing drive mode is used */
	ENGINE_API void SetOrientationDriveTwistAndSwing(bool bInEnableTwistDrive, bool bInEnableSwingDrive);

	/** Get which twist and swing orientation drives are enabled. Only applicable when Twist And Swing drive mode is used */
	ENGINE_API void GetOrientationDriveTwistAndSwing(bool& bOutEnableTwistDrive, bool& bOutEnableSwingDrive);

	/** Set whether the SLERP angular position drive is enabled. Only applicable when SLERP drive mode is used */
	ENGINE_API void SetOrientationDriveSLERP(bool bInEnableSLERP);

	/** Get whether the SLERP angular position drive is enabled. Only applicable when SLERP drive mode is used */
	bool GetOrientationDriveSLERP()
	{
		return ProfileInstance.AngularDrive.SlerpDrive.bEnablePositionDrive;
	}

	/** Whether the angular orientation drive is enabled */
	bool IsAngularOrientationDriveEnabled() const
	{
		return ProfileInstance.AngularDrive.IsOrientationDriveEnabled();
	}
	
	/** Set the angular drive's orientation target*/
	ENGINE_API void SetAngularOrientationTarget(const FQuat& InPosTarget);

	/** Get the angular drive's orientation target*/
	const FRotator& GetAngularOrientationTarget() const
	{
		return ProfileInstance.AngularDrive.OrientationTarget;
	}

	/** Set which twist and swing angular velocity drives are enabled. Only applicable when Twist And Swing drive mode is used */
	ENGINE_API void SetAngularVelocityDriveTwistAndSwing(bool bInEnableTwistDrive, bool bInEnableSwingDrive);

	/** Get which twist and swing angular velocity drives are enabled. Only applicable when Twist And Swing drive mode is used */
	ENGINE_API void GetAngularVelocityDriveTwistAndSwing(bool& bOutEnableTwistDrive, bool& bOutEnableSwingDrive);

	/** Set whether the SLERP angular velocity drive is enabled. Only applicable when SLERP drive mode is used */
	ENGINE_API void SetAngularVelocityDriveSLERP(bool bInEnableSLERP);

	/** Get whether the SLERP angular velocity drive is enabled. Only applicable when SLERP drive mode is used */
	bool GetAngularVelocityDriveSLERP()
	{
		return ProfileInstance.AngularDrive.SlerpDrive.bEnableVelocityDrive;
	}

	/** Whether the angular velocity drive is enabled */
	bool IsAngularVelocityDriveEnabled() const
	{
		return ProfileInstance.AngularDrive.IsVelocityDriveEnabled();
	}
	
	/** Set the angular drive's angular velocity target (in revolutions per second)*/
	ENGINE_API void SetAngularVelocityTarget(const FVector& InVelTarget);

	/** Get the angular drive's angular velocity target*/
	const FVector& GetAngularVelocityTarget() const
	{
		return ProfileInstance.AngularDrive.AngularVelocityTarget;
	}

	/** Set the angular drive's strength parameters*/
	ENGINE_API void SetAngularDriveParams(float InSpring, float InDamping, float InForceLimit);

	/** Get the angular drive's strength parameters*/
	ENGINE_API void GetAngularDriveParams(float& OutSpring, float& OutDamping, float& OutForceLimit) const;

	/** Set the angular drive mode */
	ENGINE_API void SetAngularDriveMode(EAngularDriveMode::Type DriveMode);

	/** Set the angular drive force-mode */
	ENGINE_API void SetAngularDriveAccelerationMode(bool bAccelerationMode);

	/** Set the angular drive mode */
	EAngularDriveMode::Type GetAngularDriveMode()
	{
		return ProfileInstance.AngularDrive.AngularDriveMode;
	}

	/** 
	 * This allows the most common drive parameters to be set in one call, avoiding lock overheads etc.
	 * Note that the individual drive elements will be enabled/disabled depending on the strength/damping
	 * values passed in.
	 * 
	 * Angular values are passed in as (swing, twist, slerp)
	 */
	ENGINE_API void SetDriveParams(
		const FVector& InPositionStrength, const FVector& InVelocityStrength, const FVector& InForceLimit,
		const FVector& InAngularSpring, const FVector& InAngularDamping, const FVector& InTorqueLimit,
		EAngularDriveMode::Type InAngularDriveMode);

	/** Refreshes the physics engine joint's linear limits. Only applicable if the joint has been created already.*/
	ENGINE_API void UpdateLinearLimit();

	/** Refreshes the physics engine joint's angular limits. Only applicable if the joint has been created already.*/
	ENGINE_API void UpdateAngularLimit();

	/** Scale Angular Limit Constraints (as defined in RB_ConstraintSetup). This only affects the physics engine and does not update the unreal side so you can do things like a LERP of the scale values. */
	ENGINE_API void SetAngularDOFLimitScale(float InSwing1LimitScale, float InSwing2LimitScale, float InTwistLimitScale);

	/** Allows you to dynamically change the size of the linear limit 'sphere'. */
	ENGINE_API void SetLinearLimitSize(float NewLimitSize);

	/** Create physics engine constraint. */
	ENGINE_API void InitConstraint(FBodyInstance* Body1, FBodyInstance* Body2, float Scale, UObject* DebugOwner, FOnConstraintBroken InConstraintBrokenDelegate = FOnConstraintBroken(), FOnPlasticDeformation InPlasticDeformationDelegate = FOnPlasticDeformation());
	ENGINE_API void InitConstraint(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, float Scale, UObject* DebugOwner, FOnConstraintBroken InConstraintBrokenDelegate = FOnConstraintBroken(), FOnPlasticDeformation InPlasticDeformationDelegate = FOnPlasticDeformation());

	/** Create physics engine constraint using physx actors. */
	ENGINE_API void InitConstraint_AssumesLocked(const FPhysicsActorHandle& ActorRef1, const FPhysicsActorHandle& ActorRef2, float InScale, FOnConstraintBroken InConstraintBrokenDelegate = FOnConstraintBroken(), FOnPlasticDeformation InPlasticDeformationDelegate = FOnPlasticDeformation());
	ENGINE_API void InitConstraint_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, float InScale, FOnConstraintBroken InConstraintBrokenDelegate = FOnConstraintBroken(), FOnPlasticDeformation InPlasticDeformationDelegate = FOnPlasticDeformation());

	/** Terminate physics engine constraint */
	ENGINE_API void TermConstraint();

	/** Whether the physics engine constraint has been terminated */
	ENGINE_API bool IsTerminated() const;

	/** See if this constraint is valid. */
	ENGINE_API bool IsValidConstraintInstance() const;

	// Get component ref frame
	ENGINE_API FTransform GetRefFrame(EConstraintFrame::Type Frame) const;

	// Pass in reference frame in. If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint. 
	ENGINE_API void SetRefFrame(EConstraintFrame::Type Frame, const FTransform& RefFrame);

	/** Get the position of this constraint in world space. */
	ENGINE_API FVector GetConstraintLocation();

	// Pass in reference position in (maintains reference orientation). If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint.
	ENGINE_API void SetRefPosition(EConstraintFrame::Type Frame, const FVector& RefPosition);

	// Pass in reference orientation in (maintains reference position). If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint.
	ENGINE_API void SetRefOrientation(EConstraintFrame::Type Frame, const FVector& PriAxis, const FVector& SecAxis);

	/** Whether collision is currently disabled */
	bool IsCollisionDisabled() const
	{
		return ProfileInstance.bDisableCollision;
	}

	/** Set whether jointed actors can collide with each other */
	ENGINE_API void SetDisableCollision(bool InDisableCollision);

	// @todo document
	void DrawConstraint(int32 ViewIndex, class FMeshElementCollector& Collector,
		float Scale, float LimitDrawScale, bool bDrawLimits, bool bDrawSelected,
		const FTransform& Con1Frame, const FTransform& Con2Frame, bool bDrawAsPoint, bool bDrawViolatedLimits) const
	{
		DrawConstraintImp(FPDIOrCollector(ViewIndex, Collector), Scale, LimitDrawScale, bDrawLimits, bDrawSelected, Con1Frame, Con2Frame, bDrawAsPoint, bDrawViolatedLimits);
	}

	void DrawConstraint(FPrimitiveDrawInterface* PDI,
		float Scale, float LimitDrawScale, bool bDrawLimits, bool bDrawSelected,
		const FTransform& Con1Frame, const FTransform& Con2Frame, bool bDrawAsPoint, bool bDrawViolatedLimits) const
	{
		DrawConstraintImp(FPDIOrCollector(PDI), Scale, LimitDrawScale, bDrawLimits, bDrawSelected, Con1Frame, Con2Frame, bDrawAsPoint, bDrawViolatedLimits);
	}

	ENGINE_API void GetUsedMaterials(TArray<UMaterialInterface*>& Materials);

	ENGINE_API bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif

	/** Whether projection is enabled for this constraint */
	bool IsProjectionEnabled() const
	{
		return ProfileInstance.bEnableProjection;
	}

	/** Turn on linear and angular projection */
	ENGINE_API void EnableProjection();

	/** Turn off linear and angular projection */
	ENGINE_API void DisableProjection();

	/** Set projection parameters */
	ENGINE_API void SetProjectionParams(bool bEnableProjection, float ProjectionLinearAlpha, float ProjectionAngularAlpha, float ProjectionLinearTolerance, float ProjectionAngularTolerance);

	/** Get projection parameters */
	ENGINE_API void GetProjectionParams(float& ProjectionLinearAlpha, float& ProjectionAngularAlpha, float& ProjectionLinearTolerance, float& ProjectionAngularTolerance) const;

	/** Set the shock propagation amount [0, 1] */
	ENGINE_API void SetShockPropagationParams(bool bEnableShockPropagation, float ShockPropagationAlpha);

	/** Get the shock propagation amount [0, 1] */
	ENGINE_API float GetShockPropagationAlpha() const;

	/** Whether parent domination is enabled (meaning the parent body cannot be be affected at all by a child) */
	bool IsParentDominatesEnabled() const
	{
		return ProfileInstance.bParentDominates;
	}

	/** Enable/Disable parent dominates (meaning the parent body cannot be be affected at all by a child) */
	ENGINE_API void EnableParentDominates();
	ENGINE_API void DisableParentDominates();
	ENGINE_API void SetParentDominates(bool bParentDominates);

	/** Whether mass conditioning is enabled. @see FConstraintProfileProperties::bEnableMassConditioning */
	bool IsMassConditioningEnabled() const
	{
		return ProfileInstance.bEnableMassConditioning;
	}

	/**
	 * Enable maxx conditioning. @see FConstraintProfileProperties::bEnableMassConditioning
	*/
	ENGINE_API void EnableMassConditioning();

	/**
	 * Disable maxx conditioning. @see FConstraintProfileProperties::bEnableMassConditioning
	*/
	ENGINE_API void DisableMassConditioning();

	float GetLastKnownScale() const { return LastKnownScale; }

	//Hacks to easily get zeroed memory for special case when we don't use GC
	static ENGINE_API void Free(FConstraintInstance * Ptr);
	static ENGINE_API FConstraintInstance * Alloc();

private:

	ENGINE_API bool CreateJoint_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2);
	ENGINE_API void UpdateAverageMass_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2);

	struct FPDIOrCollector
	{
		FPrimitiveDrawInterface* PDI;
		FMeshElementCollector* Collector;
		int32 ViewIndex;

		FPDIOrCollector(FPrimitiveDrawInterface* InPDI)
			: PDI(InPDI)
			, Collector(nullptr)
			, ViewIndex(INDEX_NONE)
		{
		}

		FPDIOrCollector(int32 InViewIndex, FMeshElementCollector& InCollector)
			: PDI(nullptr)
			, Collector(&InCollector)
			, ViewIndex(InViewIndex)
		{
		}

		bool HasCollector() const
		{
			return Collector != nullptr;
		}

		FPrimitiveDrawInterface* GetPDI() const;

		void DrawCylinder(const FVector& Start, const FVector& End, const float Thickness, const FMaterialRenderProxy* const MaterialProxy, const ESceneDepthPriorityGroup DepthPriority) const;
		void DrawCone(const FMatrix& ConeTransform, const float AngleWidth, const float AngleHeight, const uint32 NumSides, const FColor& PDIColor, const FMaterialRenderProxy* const MaterialRenderProxy, const ESceneDepthPriorityGroup DepthPriority) const;
		void DrawArrow(const FMatrix& ArrowTransform, const float Length, const float Thickness, const uint32 NumSides, const FColor& PDIColor, const FMaterialRenderProxy* const MaterialRenderProxy, const ESceneDepthPriorityGroup DepthPriority) const;

	};

	ENGINE_API void DrawConstraintImp(const FPDIOrCollector& PDIOrCollector,
		float Scale, float LimitDrawScale, bool bDrawLimits, bool bDrawSelected,
		const FTransform& Con1Frame, const FTransform& Con2Frame, bool bDrawAsPoint, bool bDrawViolatedLimits = false) const;

	ENGINE_API void UpdateBreakable();
	ENGINE_API void UpdatePlasticity();
	ENGINE_API void UpdateContactTransferScale();
	ENGINE_API void UpdateDriveTarget();

public:

#if WITH_EDITORONLY_DATA

	/** Returns this constraint's default transform relative to its parent bone. */
	ENGINE_API FTransform CalculateDefaultParentTransform(const UPhysicsAsset* const PhysicsAsset) const;

	/** Returns this constraint's default transform relative to its child bone. */
	ENGINE_API FTransform CalculateDefaultChildTransform() const;

	/** Set the constraint transform components specified by the SnapFlags to their default values (derived from the parent and child bone transforms). */
	ENGINE_API void SnapTransformsToDefault(const EConstraintTransformComponentFlags SnapFlags, const UPhysicsAsset* const PhysicsAsset);
#endif // WITH_EDITORONLY_DATA

	///////////////////////////// DEPRECATED
	// Most of these properties have moved inside the ProfileInstance member (FConstraintProfileProperties struct)
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint32 bDisableCollision_DEPRECATED : 1;
	UPROPERTY()
	uint32 bEnableProjection_DEPRECATED : 1;
	UPROPERTY()
	float ProjectionLinearTolerance_DEPRECATED;
	UPROPERTY()
	float ProjectionAngularTolerance_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearXMotion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearYMotion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearZMotion_DEPRECATED;
	UPROPERTY()
	float LinearLimitSize_DEPRECATED;
	UPROPERTY()
	uint32 bLinearLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	float LinearLimitStiffness_DEPRECATED;
	UPROPERTY()
	float LinearLimitDamping_DEPRECATED;
	UPROPERTY()
	uint32 bLinearBreakable_DEPRECATED : 1;
	UPROPERTY()
	float LinearBreakThreshold_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularSwing1Motion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularTwistMotion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularSwing2Motion_DEPRECATED;
	UPROPERTY()
	uint32 bSwingLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	uint32 bTwistLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	float Swing1LimitAngle_DEPRECATED;
	UPROPERTY()
	float TwistLimitAngle_DEPRECATED;
	UPROPERTY()
	float Swing2LimitAngle_DEPRECATED;
	UPROPERTY()
	float SwingLimitStiffness_DEPRECATED;
	UPROPERTY()
	float SwingLimitDamping_DEPRECATED;
	UPROPERTY()
	float TwistLimitStiffness_DEPRECATED;
	UPROPERTY()
	float TwistLimitDamping_DEPRECATED;
	UPROPERTY()
	uint32 bAngularBreakable_DEPRECATED : 1;
	UPROPERTY()
	float AngularBreakThreshold_DEPRECATED;
private:
	UPROPERTY()
	uint32 bLinearXPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearXVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearYPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearYVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearZPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearZVelocityDrive_DEPRECATED : 1;
public:
	UPROPERTY()
	uint32 bLinearPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bLinearVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	FVector LinearPositionTarget_DEPRECATED;
	UPROPERTY()
	FVector LinearVelocityTarget_DEPRECATED;
	UPROPERTY()
	float LinearDriveSpring_DEPRECATED;
	UPROPERTY()
	float LinearDriveDamping_DEPRECATED;
	UPROPERTY()
	float LinearDriveForceLimit_DEPRECATED;
	UPROPERTY()
	uint32 bSwingPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bSwingVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bTwistPositionDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bTwistVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bAngularSlerpDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bAngularOrientationDrive_DEPRECATED : 1;
private:
	UPROPERTY()
	uint32 bEnableSwingDrive_DEPRECATED : 1;
	UPROPERTY()
	uint32 bEnableTwistDrive_DEPRECATED : 1;
public:
	UPROPERTY()
	uint32 bAngularVelocityDrive_DEPRECATED : 1;
	UPROPERTY()
	FQuat AngularPositionTarget_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<EAngularDriveMode::Type> AngularDriveMode_DEPRECATED;
	UPROPERTY()
	FRotator AngularOrientationTarget_DEPRECATED;
	UPROPERTY()
	FVector AngularVelocityTarget_DEPRECATED;    // Revolutions per second
	UPROPERTY()
	float AngularDriveSpring_DEPRECATED;
	UPROPERTY()
	float AngularDriveDamping_DEPRECATED;
	UPROPERTY()
	float AngularDriveForceLimit_DEPRECATED;
#endif //EDITOR_ONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FConstraintInstance> : public TStructOpsTypeTraitsBase2<FConstraintInstance>
{
	enum 
	{
		WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true
#endif
	};
};


// Wrapping type around instance pointer to be returned per value in Blueprints
USTRUCT(BlueprintType)
struct FConstraintInstanceAccessor
{
	GENERATED_USTRUCT_BODY()

public:
	FConstraintInstanceAccessor()
		: Owner(nullptr)
		, Index(0)
	{}

	FConstraintInstanceAccessor(const TWeakObjectPtr<UObject>& Owner, uint32 Index = 0, TFunction<void(void)> InOnRelease = TFunction<void(void)>())
		: Owner(Owner)
		, Index(Index)
#if WITH_EDITOR
		, OnRelease(InOnRelease)
#endif
	{}

#if WITH_EDITOR
	~FConstraintInstanceAccessor()
	{
		if (OnRelease)
		{
			OnRelease();
		}
	}
#endif

	ENGINE_API FConstraintInstance* Get() const;

	/** Calls modify on the owner object to make sure the constraint is dirtied */
	ENGINE_API void Modify();

private:
	UPROPERTY()
	TWeakObjectPtr<UObject> Owner;

	UPROPERTY()
	uint32 Index;

#if WITH_EDITOR
	/** Warning: it would be unwieldy to make the accessor move only, so the OnRelease callback should be safe to be called multiple times */
	TFunction<void(void)> OnRelease;
#endif
};
