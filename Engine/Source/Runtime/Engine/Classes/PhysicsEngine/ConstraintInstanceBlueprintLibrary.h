// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineTypes.h"
#include "PhysicsEngine/ConstraintInstance.h"

#include "ConstraintInstanceBlueprintLibrary.generated.h"

UCLASS(MinimalAPI)
class UConstraintInstanceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	//---------------------------------------------------------------------------------------------------
	// 	   
	// CONSTRAINT BODIES
	// 
	//---------------------------------------------------------------------------------------------------

	/** Gets Attached body names 
	*	@param Accessor		Constraint accessor to query
	*	@param ParentBody	Parent body name of the constraint
	*	@param ChildBody	Child body name of the constraint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "rigid body name name parent child"))
		static ENGINE_API void GetAttachedBodyNames(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			FName& ParentBody,
			FName& ChildBody
		);

	//---------------------------------------------------------------------------------------------------
	//
	// CONSTRAINT BEHAVIOR 
	//
	//---------------------------------------------------------------------------------------------------

	/** Sets whether bodies attched to the constraint can collide or not
	*	@param Accessor				Constraint accessor to change
	*	@param bDisableCollision	true to disable collision between constrained bodies
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "disable enable collision"))
	static ENGINE_API void SetDisableCollision(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			bool bDisableCollision
		);

	/** Gets whether bodies attched to the constraint can collide or not
	*	@param Accessor		Constraint accessor to query
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "disable enable collision"))
	static ENGINE_API bool GetDisableCollsion(UPARAM(ref) FConstraintInstanceAccessor& Accessor);

	/** Sets projection parameters of the constraint
	*	@param Accessor					Constraint accessor to change
	*	@param bEnableProjection		true to enable projection
	*	@param ProjectionLinearAlpha	how much linear projection to apply in [0,1] range
	*	@param ProjectionAngularAlpha	how much angular projection to apply in [0,1] range
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "enable linear angular"))
	static ENGINE_API void SetProjectionParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bEnableProjection,
		float ProjectionLinearAlpha,
		float ProjectionAngularAlpha
	);

	/** Gets projection parameters of the constraint
	*	@param Accessor					Constraint accessor to query
	*	@param bEnableProjection		true to enable projection
	*	@param ProjectionLinearAlpha	how much linear projection to apply in [0,1] range
	*	@param ProjectionAngularAlpha	how much angular projection to apply in [0,1] range
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "enable linear angular"))
	static ENGINE_API void GetProjectionParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bEnableProjection,
		float& ProjectionLinearAlpha,
		float& ProjectionAngularAlpha
	);

	/** Sets whether the parent body is not affected by it's child motion 
	*	@param Accessor				Constraint accessor to change
	*	@param bParentDominates		true to avoid the parent being affected by its child motion
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetParentDominates(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bParentDominates
	);

	/** Gets whether the parent body is not affected by it's child motion
	*	@param Accessor Constraint accessor to query
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API bool GetParentDominates(UPARAM(ref) FConstraintInstanceAccessor& Accessor);

	/**
	 * @brief Enable or disable mass conditioning for the constraint.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetMassConditioningEnabled(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bEnableMassConditioning);

	/**
	 * @brief Gets whether mass conditioning is enabled for the constraint.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API bool GetMassConditioningEnabled(UPARAM(ref) FConstraintInstanceAccessor& Accessor);


	//---------------------------------------------------------------------------------------------------
	//
	// LINEAR LIMITS
	//
	//---------------------------------------------------------------------------------------------------

	/** Sets Constraint Linear Motion Ranges
	*	@param Accessor	Constraint accessor to change
	*	@param XMotion	Type of motion along the X axis
	*	@param YMotion	Type of motion along the Y axis
	*	@param ZMotion	Type of motion along the Z axis
	*	@param Limit	linear limit to apply to all axis
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motion free limited locked linear angular"))
	static ENGINE_API void SetLinearLimits(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		TEnumAsByte<ELinearConstraintMotion> XMotion,
		TEnumAsByte<ELinearConstraintMotion> YMotion,
		TEnumAsByte<ELinearConstraintMotion> ZMotion,
		float Limit
	);

	/** Gets Constraint Linear Motion Ranges
	*	@param Accessor	Constraint accessor to query
	*	@param XMotion	Type of motion along the X axis
	*	@param YMotion	Type of motion along the Y axis
	*	@param ZMotion	Type of motion along the Z axis
	*	@param Limit	linear limit applied to all axis
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motion free limited locked linear angular"))
	static ENGINE_API void GetLinearLimits(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		TEnumAsByte<ELinearConstraintMotion>& XMotion,
		TEnumAsByte<ELinearConstraintMotion>& YMotion,
		TEnumAsByte<ELinearConstraintMotion>& ZMotion,
		float& Limit
	);

	/** Sets Constraint Linear Soft Limit parameters
	*	@param Accessor						Constraint accessor to change
	* *	@param bSoftLinearLimit				True is the linear limit is soft
	*	@param LinearLimitStiffness			Stiffness of the soft linear limit. Only used when Soft limit is on ( positive value )
	*	@param LinearLimitDamping			Damping of the soft linear limit. Only used when Soft limit is on ( positive value )
	*   @param LinearLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param LinearLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
	static ENGINE_API void SetLinearSoftLimitParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bSoftLinearLimit,
		float LinearLimitStiffness,
		float LinearLimitDamping,
		float LinearLimitRestitution,
		float LinearLimitContactDistance
	);

	/** Gets Constraint Linear Soft Limit parameters
	*	@param Accessor						Constraint accessor to query
	* *	@param bSoftLinearLimit				True is the Linear limit is soft
	*	@param LinearLimitStiffness			Stiffness of the soft Linear limit. Only used when Soft limit is on ( positive value )
	*	@param LinearLimitDamping			Damping of the soft Linear limit. Only used when Soft limit is on ( positive value )
	*   @param LinearLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param LinearLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
	static ENGINE_API void GetLinearSoftLimitParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bSoftLinearLimit,
		float& LinearLimitStiffness,
		float& LinearLimitDamping,
		float& LinearLimitRestitution,
		float& LinearLimitContactDistance
	);

	/** Sets the Linear Breakable properties
	*	@param Accessor				Constraint accessor to change
	*	@param bLinearBreakable		Whether it is possible to break the joint with linear force
	*	@param LinearBreakThreshold	Force needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetLinearBreakable(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		bool bLinearBreakable, 
		float LinearBreakThreshold
	);

	/** Gets Constraint Linear Breakable properties
	*	@param Accessor				Constraint accessor to query
	*	@param bLinearBreakable		Whether it is possible to break the joint with linear force
	*	@param LinearBreakThreshold	Force needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void GetLinearBreakable(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bLinearBreakable,
		float& LinearBreakThreshold
	);


	/** Sets Constraint Linear Plasticity properties
	*	@param Accessor						Constraint accessor to change
	*	@param bLinearPlasticity			Whether it is possible to reset the target position from the linear displacement
	*	@param LinearPlasticityThreshold	Delta from target needed to reset the target joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
		static ENGINE_API void SetLinearPlasticity(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			bool bLinearPlasticity,
			float LinearPlasticityThreshold,
			TEnumAsByte<EConstraintPlasticityType> PlasticityType
		);

	/** Gets Constraint Linear Plasticity properties
	*	@param Accessor						Constraint accessor to query
	*	@param bAngularPlasticity			Whether it is possible to reset the target position from the linear displacement
	*	@param AngularPlasticityThreshold	Delta from target needed to reset the target joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
		static ENGINE_API void GetLinearPlasticity(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			bool& bLinearPlasticity,
			float& LinearPlasticityThreshold,
			TEnumAsByte<EConstraintPlasticityType>& PlasticityType
		);

	//---------------------------------------------------------------------------------------------------
	//
	// Contact Transfer
	//
	//---------------------------------------------------------------------------------------------------

	/** Gets Constraint Contact Transfer Scale properties
	*	@param Accessor						Constraint accessor to query
	*	@param ContactTransferScale			Scale for transfer of child energy to parent.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
		static ENGINE_API void GetContactTransferScale(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			float& ContactTransferScale
		);

	/** Set Contact Transfer Scale
	*	@param Accessor						Constraint accessor to change
	*	@param ContactTransferScale			Set Contact Transfer Scale onto joints parent
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetContactTransferScale(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		float ContactTransferScale
	);

	//---------------------------------------------------------------------------------------------------
	//
	// ANGULAR LIMITS 
	//
	//---------------------------------------------------------------------------------------------------

	/** Sets COnstraint Angular Motion Ranges
	*	@param Accessor				Constraint accessor to change
	*	@param Swing1MotionType		Type of swing motion ( first axis )
	*	@param Swing1LimitAngle		Size of limit in degrees in [0, 180] range
	*   @param Swing2MotionType		Type of swing motion ( second axis )
	*	@param Swing2LimitAngle		Size of limit in degrees in [0, 180] range
	*   @param TwistMotionType		Type of twist motion
	*	@param TwistLimitAngle		Size of limit in degrees in [0, 180] range	
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "swing twist motion free limited locked"))
	static ENGINE_API void SetAngularLimits(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		TEnumAsByte<EAngularConstraintMotion> Swing1MotionType,
		float Swing1LimitAngle, 
		TEnumAsByte<EAngularConstraintMotion> Swing2MotionType,
		float Swing2LimitAngle,
		TEnumAsByte<EAngularConstraintMotion> TwistMotionType,
		float TwistLimitAngle
	);

	/** Gets Constraint Angular Motion Ranges
	*	@param Accessor				Constraint accessor to query
	*	@param Swing1MotionType		Type of swing motion ( first axis )
	*	@param Swing1LimitAngle		Size of limit in degrees in [0, 180] range
	*   @param Swing2MotionType		Type of swing motion ( second axis )
	*	@param Swing2LimitAngle		Size of limit in degrees in [0, 180] range
	*   @param TwistMotionType		Type of twist motion
	*	@param TwistLimitAngle		Size of limit in degrees in [0, 180] range
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "swing twist motion free limited locked"))
	static ENGINE_API void GetAngularLimits(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		TEnumAsByte<EAngularConstraintMotion>& Swing1MotionType,
		float& Swing1LimitAngle,
		TEnumAsByte<EAngularConstraintMotion>& Swing2MotionType,
		float& Swing2LimitAngle,
		TEnumAsByte<EAngularConstraintMotion>& TwistMotionType,
		float& TwistLimitAngle
	);

	/** Sets Constraint Angular Soft Swing Limit parameters
	*	@param Accessor						Constraint accessor to change
	* *	@param bSoftSwingLimit				True is the swing limit is soft
	*	@param SwingLimitStiffness			Stiffness of the soft swing limit. Only used when Soft limit is on ( positive value )
	*	@param SwingLimitDamping			Damping of the soft swing limit. Only used when Soft limit is on ( positive value )
	*   @param SwingLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param SwingLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
	static ENGINE_API void SetAngularSoftSwingLimitParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bSoftSwingLimit,
		float SwingLimitStiffness,
		float SwingLimitDamping,
		float SwingLimitRestitution,
		float SwingLimitContactDistance
	);

	/** Gets Constraint Angular Soft Swing Limit parameters
	*	@param Accessor						Constraint accessor to query
	* *	@param bSoftSwingLimit				True is the swing limit is soft
	*	@param SwingLimitStiffness			Stiffness of the soft swing limit. Only used when Soft limit is on ( positive value )
	*	@param SwingLimitDamping			Damping of the soft swing limit. Only used when Soft limit is on ( positive value )
	*   @param SwingLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param SwingLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
	static ENGINE_API void GetAngularSoftSwingLimitParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bSoftSwingLimit,
		float& SwingLimitStiffness,
		float& SwingLimitDamping,
		float& SwingLimitRestitution,
		float& SwingLimitContactDistance
	);

	/** Sets Constraint Angular Soft Twist Limit parameters
	*	@param Accessor						Constraint accessor to change
	* *	@param bSoftTwistLimit				True is the twist limit is soft
	*	@param TwistLimitStiffness			Stiffness of the soft Twist limit. Only used when Soft limit is on ( positive value )
	*	@param TwistLimitDamping			Damping of the soft Twist limit. Only used when Soft limit is on ( positive value )
	*   @param TwistLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param TwistLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
	static ENGINE_API void SetAngularSoftTwistLimitParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bSoftTwistLimit,
		float TwistLimitStiffness,
		float TwistLimitDamping,
		float TwistLimitRestitution,
		float TwistLimitContactDistance
	);

	/** Gets Constraint Angular Soft Twist Limit parameters
	*	@param Accessor						Constraint accessor to query
	* *	@param bSoftTwistLimit				True is the Twist limit is soft
	*	@param TwistLimitStiffness			Stiffness of the soft Twist limit. Only used when Soft limit is on ( positive value )
	*	@param TwistLimitDamping			Damping of the soft Twist limit. Only used when Soft limit is on ( positive value )
	*   @param TwistLimitRestitution		Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead.
	*	@param TwistLimitContactDistance	Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "soft stiffness damping restitution contact distance"))
		static ENGINE_API void GetAngularSoftTwistLimitParams(
			UPARAM(ref) FConstraintInstanceAccessor& Accessor,
			bool& bSoftTwistLimit,
			float& TwistLimitStiffness,
			float& TwistLimitDamping,
			float& TwistLimitRestitution,
			float& TwistLimitContactDistance
		);

	/** Sets Constraint Angular Breakable properties
	*	@param Accessor					Constraint accessor to change
	*	@param bAngularBreakable		Whether it is possible to break the joint with angular force
	*	@param AngularBreakThreshold	Torque needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetAngularBreakable(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bAngularBreakable, 
		float AngularBreakThreshold
	);

	/** Gets Constraint Angular Breakable properties
	*	@param Accessor					Constraint accessor to query
	*	@param bAngularBreakable		Whether it is possible to break the joint with angular force
	*	@param AngularBreakThreshold	Torque needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void GetAngularBreakable(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bAngularBreakable,
		float& AngularBreakThreshold
	);

	/** Sets Constraint Angular Plasticity properties
	*	@param Accessor						Constraint accessor to change
	*	@param bAngularPlasticity			Whether it is possible to reset the target angle from the angular displacement
	*	@param AngularPlasticityThreshold	Degrees needed to reset the rest state of the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void SetAngularPlasticity(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bAngularPlasticity, 
		float AngularPlasticityThreshold
	);

	/** Sets Constraint Angular Plasticity properties
	*	@param Accessor						Constraint accessor to query
	*	@param bAngularPlasticity			Whether it is possible to reset the target angle from the angular displacement
	*	@param AngularPlasticityThreshold	Degrees needed to reset the rest state of the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void GetAngularPlasticity(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bAngularPlasticity,
		float& AngularPlasticityThreshold
	);

	//---------------------------------------------------------------------------------------------------
	//
	// LINEAR MOTOR
	//
	//---------------------------------------------------------------------------------------------------

	/** Enables/Disables linear position drive
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableDriveX		Indicates whether the drive for the X-Axis should be enabled
	*	@param bEnableDriveY		Indicates whether the drive for the Y-Axis should be enabled
	*	@param bEnableDriveZ		Indicates whether the drive for the Z-Axis should be enabled
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor position target"))
	static ENGINE_API void SetLinearPositionDrive(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		bool bEnableDriveX, 
		bool bEnableDriveY, 
		bool bEnableDriveZ
	);

	/** Gets whether linear position drive is enabled or not
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableDriveX		Indicates whether the drive for the X-Axis is enabled
	*	@param bOutEnableDriveY		Indicates whether the drive for the Y-Axis is enabled
	*	@param bOutEnableDriveZ		Indicates whether the drive for the Z-Axis is enabled
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor position target"))
	static ENGINE_API void GetLinearPositionDrive(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableDriveX,
		bool& bOutEnableDriveY,
		bool& bOutEnableDriveZ
	);

	/** Enables/Disables linear velocity drive
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableDriveX		Indicates whether the drive for the X-Axis should be enabled
	*	@param bEnableDriveY		Indicates whether the drive for the Y-Axis should be enabled
	*	@param bEnableDriveZ		Indicates whether the drive for the Z-Axis should be enabled
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor velocity target"))
	static ENGINE_API void SetLinearVelocityDrive(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bEnableDriveX, 
		bool bEnableDriveY, 
		bool bEnableDriveZ
	);

	/** Gets whether linear velocity drive is enabled or not
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableDriveX		Indicates whether the drive for the X-Axis is enabled
	*	@param bOutEnableDriveY		Indicates whether the drive for the Y-Axis is enabled
	*	@param bOutEnableDriveZ		Indicates whether the drive for the Z-Axis is enabled
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor velocity target"))
	static ENGINE_API void GetLinearVelocityDrive(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableDriveX,
		bool& bOutEnableDriveY,
		bool& bOutEnableDriveZ
	);

	/** Sets the target position for the linear drive.
	*	@param Accessor				Constraint accessor to change
	*	@param InPosTarget			Target position
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void SetLinearPositionTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		const FVector& InPosTarget
	);

	/** Gets the target position for the linear drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutPosTarget			Target position
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void GetLinearPositionTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		FVector& OutPosTarget
	);

	/** Sets the target velocity for the linear drive.
	*	@param Accessor				Constraint accessor to change
	*	@param InVelTarget			Target velocity
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void SetLinearVelocityTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		const FVector& InVelTarget
	);

	/** Gets the target velocity for the linear drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutVelTarget			Target velocity
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void GetLinearVelocityTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		FVector& OutVelTarget
	);

	/** Sets the drive params for the linear drive.
	*	@param Accessor				Constraint accessor to change
	*	@param PositionStrength		Positional strength for the drive (stiffness)
	*	@param VelocityStrength		Velocity strength of the drive (damping)
	*	@param InForceLimit			Max force applied by the drive
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor position velocity target strength max force"))
	static ENGINE_API void SetLinearDriveParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		float PositionStrength, 
		float VelocityStrength, 
		float InForceLimit
	);

	/** Gets the drive params for the linear drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutPositionStrength	Positional strength for the drive (stiffness)
	*	@param OutVelocityStrength	Velocity strength of the drive (damping)
	*	@param OutForceLimit		Max force applied by the drive
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor position velocity target strength max force"))
	static ENGINE_API void GetLinearDriveParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		float& OutPositionStrength,
		float& OutVelocityStrength,
		float& OutForceLimit
	);

	//---------------------------------------------------------------------------------------------------
	//
	// ANGULAR MOTOR
	//
	//---------------------------------------------------------------------------------------------------

	/** Enables/Disables angular orientation drive. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param bEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void SetOrientationDriveTwistAndSwing(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		bool bEnableTwistDrive, 
		bool bEnableSwingDrive
	);

	/** Gets whether angular orientation drive are enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableTwistDrive	Indicates whether the drive for the twist axis is enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param bOutEnableSwingDrive	Indicates whether the drive for the swing axis is enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void GetOrientationDriveTwistAndSwing(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableTwistDrive,
		bool& bOutEnableSwingDrive
	);

	/** Enables/Disables the angular orientation slerp drive. Only relevant if the AngularDriveMode is set to SLERP
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableSLERP			Indicates whether the SLERP drive should be enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void SetOrientationDriveSLERP(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		bool bEnableSLERP
	);

	/** Gets whether the angular orientation slerp drive is enabled or not. Only relevant if the AngularDriveMode is set to SLERP
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableSLERP		Indicates whether the SLERP drive should be enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void GetOrientationDriveSLERP(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableSLERP
	);

	/** Enables/Disables angular velocity twist and swing drive. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param bEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void SetAngularVelocityDriveTwistAndSwing(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor, 
		bool bEnableTwistDrive, 
		bool bEnableSwingDrive
	);

	/** Gets whether angular velocity twist and swing drive is enabled or not. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*   @param bOutEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void GetAngularVelocityDriveTwistAndSwing(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableTwistDrive,
		bool& bOutEnableSwingDrive
	);

	/** Enables/Disables the angular velocity slerp drive. Only relevant if the AngularDriveMode is set to SLERP
	*	@param Accessor				Constraint accessor to change
	*	@param bEnableSLERP			Indicates whether the SLERP drive should be enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void SetAngularVelocityDriveSLERP(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool bEnableSLERP
	);

	/** Gets whether the angular velocity slerp drive is enabled or not. Only relevant if the AngularDriveMode is set to SLERP
	*	@param Accessor				Constraint accessor to query
	*	@param bOutEnableSLERP		Indicates whether the SLERP drive is enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target"))
	static ENGINE_API void GetAngularVelocityDriveSLERP(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		bool& bOutEnableSLERP
	);

	/** Switches the angular drive mode between SLERP and Twist And Swing
	*	@param Accessor		Constraint accessor to change
	*	@param DriveMode	The angular drive mode to use. SLERP uses shortest spherical path, but will not work if any angular constraints are locked. Twist and Swing decomposes the path into the different angular degrees of freedom but may experience gimbal lock
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void SetAngularDriveMode(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		EAngularDriveMode::Type DriveMode
	);

	/** Gets the angular drive mode ( SLERP or Twist And Swing)
	*	@param Accessor		Constraint accessor to query
	*	@param OutDriveMode	The angular drive mode to use. SLERP uses shortest spherical path, but will not work if any angular constraints are locked. Twist and Swing decomposes the path into the different angular degrees of freedom but may experience gimbal lock
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void GetAngularDriveMode(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		TEnumAsByte<EAngularDriveMode::Type>& OutDriveMode
	);

	/** Sets the target orientation for the angular drive.
	*	@param Accessor				Constraint accessor to change
	*	@param InPosTarget			Target orientation
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void SetAngularOrientationTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		const FRotator& InPosTarget
	);

	/** Gets the target orientation for the angular drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutPosTarget			Target orientation
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void GetAngularOrientationTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		FRotator& OutPosTarget
	);

	/** Sets the target velocity for the angular drive.
	*	@param Accessor				Constraint accessor to change
	*	@param InVelTarget			Target velocity
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void SetAngularVelocityTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		const FVector& InVelTarget
	);

	/** Gets the target velocity for the angular drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutVelTarget			Target velocity
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor"))
	static ENGINE_API void GetAngularVelocityTarget(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		FVector& OutVelTarget
	);

	/** Sets the drive params for the angular drive.
	*	@param Accessor				Constraint accessor to change
	*	@param PositionStrength		Positional strength for the drive (stiffness)
	*	@param VelocityStrength 	Velocity strength of the drive (damping)
	*	@param InForceLimit			Max force applied by the drive
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target orientation velocity strength max force"))
	static ENGINE_API void SetAngularDriveParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		float PositionStrength, 
		float VelocityStrength, 
		float InForceLimit
	);

	/** Gets the drive params for the angular drive.
	*	@param Accessor				Constraint accessor to query
	*	@param OutPositionStrength	Positional strength for the drive (stiffness)
	*	@param OutVelocityStrength 	Velocity strength of the drive (damping)
	*	@param OutForceLimit		Max force applied by the drive
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints", meta = (Keywords = "motor target orientation velocity strength max force"))
	static ENGINE_API void GetAngularDriveParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		float& OutPositionStrength,
		float& OutVelocityStrength,
		float& OutForceLimit
	);

	/** Copies all properties from one constraint to another
	* @param Accessor	Constraint accessor to write to
	* @param SourceAccessor Constraint accessor to read from
	* @param bKeepPosition	Whether to keep original constraint positions
	* @param bKeepRotation	Whether to keep original constraint rotations
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	static ENGINE_API void CopyParams(
		UPARAM(ref) FConstraintInstanceAccessor& Accessor,
		UPARAM(ref) FConstraintInstanceAccessor& SourceAccessor,
		bool bKeepPosition = true,
		bool bKeepRotation = true
	);
};
