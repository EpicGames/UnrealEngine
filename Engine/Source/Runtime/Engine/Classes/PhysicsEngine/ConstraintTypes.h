// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "Chaos/ChaosEngineInterface.h"
#include "ConstraintTypes.generated.h"

USTRUCT()
struct FConstraintBaseParams
{
	GENERATED_BODY()
	
	/** Stiffness of the soft constraint. Only used when Soft Constraint is on. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (editcondition = "bSoftConstraint", ClampMin = "0.0"))
	float Stiffness;

	/** Damping of the soft constraint. Only used when Soft Constraint is on. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (editcondition = "bSoftConstraint", ClampMin = "0.0"))
	float Damping;

	/** Controls the amount of bounce when the constraint is violated. A restitution value of 1 will bounce back with the same velocity the limit was hit. A value of 0 will stop dead. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Restitution;

	/** Determines how close to the limit we have to get before turning the joint on. Larger value will be more expensive, but will do a better job not violating constraints. A smaller value will be more efficient, but easier to violate. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint, meta = (ClampMin = "0.0"))
	float ContactDistance;

	/** Whether we want to use a soft constraint (spring). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Constraint)
	uint8 bSoftConstraint : 1;

	ENGINE_API FConstraintBaseParams();
};

/** Distance constraint */
USTRUCT()
struct FLinearConstraint : public FConstraintBaseParams
{
	GENERATED_BODY()

	/** The distance allowed between the two joint reference frames. Distance applies on all axes enabled (one axis means line, two axes implies circle, three axes implies sphere) */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (ClampMin = "0.0"))
	float Limit;

	/** Indicates the linear constraint applied along the X-axis. Free implies no constraint at all. Locked implies no movement along X is allowed. Limited implies the distance in the joint along all active axes must be less than the Distance provided. */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (DisplayName= "X Motion"))
	TEnumAsByte<enum ELinearConstraintMotion> XMotion;

	/** Indicates the linear constraint applied along the Y-axis. Free implies no constraint at all. Locked implies no movement along Y is allowed. Limited implies the distance in the joint along all active axes must be less than the Distance provided. */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (DisplayName= "Y Motion"))
	TEnumAsByte<enum ELinearConstraintMotion> YMotion;

	/** Indicates the linear constraint applied along theZX-axis. Free implies no constraint at all. Locked implies no movement along Z is allowed. Limited implies the distance in the joint along all active axes must be less than the Distance provided. */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (DisplayName= "Z Motion"))
	TEnumAsByte<enum ELinearConstraintMotion> ZMotion;

	ENGINE_API FLinearConstraint();

	/** Updates physx linear constraint from unreal data. */
	ENGINE_API void UpdateLinearLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass, float Scale) const;
};


/** Cone constraint */
USTRUCT()
struct FConeConstraint : public FConstraintBaseParams
{
	GENERATED_BODY()

	/** Angle of movement along the XY plane. This defines the first symmetric angle of the cone. */
	UPROPERTY(EditAnywhere, Category = Angular, meta = (ClampMin = "0.0", ClampMax = "180.0", DisplayName = "Swing 1 Limit"))
	float Swing1LimitDegrees;

	/** Angle of movement along the XZ plane. This defines the second symmetric angle of the cone. */
	UPROPERTY(EditAnywhere, Category = Angular, meta = (ClampMin = "0.0", ClampMax = "180.0"), DisplayName = "Swing 2 Limit")
	float Swing2LimitDegrees;

	/** Indicates whether the Swing1 limit is used.*/
	UPROPERTY(EditAnywhere, Category = Angular, meta = (DisplayName= "Swing 1 Motion"))
	TEnumAsByte<enum EAngularConstraintMotion> Swing1Motion;

	/** Indicates whether the Swing2 limit is used.*/
	UPROPERTY(EditAnywhere, Category = Angular, meta = (DisplayName= "Swing 2 Motion"))
	TEnumAsByte<enum EAngularConstraintMotion> Swing2Motion;

	ENGINE_API FConeConstraint();

	/** Updates cone limit from unreal data. */
	ENGINE_API void UpdateConeLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass) const;
};

/** Angular roll constraint */
USTRUCT()
struct FTwistConstraint : public FConstraintBaseParams
{
	GENERATED_BODY()

	/** Symmetric angle of roll along the X-axis. */
	UPROPERTY(EditAnywhere, Category = Angular, meta = (ClampMin = "0.0", ClampMax = "180.0", DisplayName = "Twist Limit"))
	float TwistLimitDegrees;

	/** Indicates whether the Twist limit is used.*/
	UPROPERTY(EditAnywhere, Category = Angular)
	TEnumAsByte<enum EAngularConstraintMotion> TwistMotion;

	ENGINE_API FTwistConstraint();

	/** Updates twist limit from unreal data. */
	ENGINE_API void UpdateTwistLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass) const;
};

#define		RB_MinSizeToLockDOF				(0.1)
#define		RB_MinAngleToLockDOF			(5.0)
