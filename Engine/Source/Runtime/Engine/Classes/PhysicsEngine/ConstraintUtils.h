// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FConstraintInstance;

namespace ConstraintUtils
{
	ENGINE_API void ConfigureAsHinge(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits = true);
	ENGINE_API bool IsHinge(const FConstraintInstance& ConstraintInstance);

	ENGINE_API void ConfigureAsPrismatic(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits = true);
	ENGINE_API bool IsPrismatic(const FConstraintInstance& ConstraintInstance);

	ENGINE_API void ConfigureAsSkelJoint(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits = true);
	ENGINE_API bool IsSkelJoint(const FConstraintInstance& ConstraintInstance);

	ENGINE_API void ConfigureAsBallAndSocket(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits = true);
	ENGINE_API bool IsBallAndSocket(const FConstraintInstance& ConstraintInstance);
}
