// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Chaos/Serializable.h"

#include "ConvexElem.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;
struct FDynamicMeshVertex;
struct FKBoxElem;

namespace physx
{
	class PxConvexMesh;
}

namespace Chaos
{
	class FImplicitObject;

	class FConvex;
}

/** One convex hull, used for simplified collision. */
USTRUCT()
struct FKConvexElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

	/** Array of indices that make up the convex hull. */
	UPROPERTY()
	TArray<FVector> VertexData;

	UPROPERTY()
	TArray<int32> IndexData;

	/** Bounding box of this convex hull. */
	UPROPERTY()
	FBox ElemBox;

private:
	/** Transform of this element */
	UPROPERTY()
	FTransform Transform;

	Chaos::FConvexPtr ChaosConvex;

public:

	enum class EConvexDataUpdateMethod
	{
		/** Update convex index and vertex data from the chaos convex object only if they are missing */
		UpdateConvexDataOnlyIfMissing = 0,

		/** Always convex recompute index and vertex data from the set chaos convex object */
		AlwaysUpdateConvexData
	};

	ENGINE_API FKConvexElem();
	ENGINE_API FKConvexElem(const FKConvexElem& Other);
	ENGINE_API ~FKConvexElem();

	ENGINE_API const FKConvexElem& operator=(const FKConvexElem& Other);

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const override;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;

	ENGINE_API void AddCachedSolidConvexGeom(TArray<FDynamicMeshVertex>& VertexBuffer, TArray<uint32>& IndexBuffer, const FColor VertexColor) const;

	/** Reset the hull to empty all arrays */
	ENGINE_API void	Reset();

	/** Updates internal ElemBox based on current value of VertexData */
	ENGINE_API void	UpdateElemBox();

	/** Calculate a bounding box for this convex element with the specified transform and scale */
	ENGINE_API FBox	CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const;

	/** Get set of planes that define this convex hull */
	ENGINE_API void GetPlanes(TArray<FPlane>& Planes) const;

	/** Utility for creating a convex hull from a set of planes. Will reset current state of this elem. */
	ENGINE_API bool HullFromPlanes(const TArray<FPlane>& InPlanes, const TArray<FVector>& SnapVerts, float InSnapDistance = UE_SMALL_NUMBER);

	/** Utility for setting this convex element to match a supplied box element. Also copies transform. */
	ENGINE_API void ConvexFromBoxElem(const FKBoxElem& InBox);

	/** Apply current element transform to verts, and reset transform to identity */
	ENGINE_API void BakeTransformToVerts();

	/** Returns the volume of this element */
	UE_DEPRECATED(5.1, "Changed to GetScaledVolume. Note that Volume calculation now includes non-uniform scale so values may have changed")
	FVector::FReal GetVolume(const FVector& Scale) const;

	/** Returns the volume of this element */
	FVector::FReal GetScaledVolume(const FVector& Scale3D) const;

	const auto& GetChaosConvexMesh() const
	{
		return ChaosConvex;
	}

	/** 
	* Set the chaos convex mesh 
	* Note : This will by default invalidate the convex data and recompute them by calling ComputeChaosConvexIndices
	* Only set ConvexDataUpdateMethod to UpdateConvexDataOnlyIfMissing if you know the data is up to date with the chaos convex object or if you plan to call ComputeChaosConvexIndices later
	* @param InChaosConvex	Chaos convex mesh to set 
	* @param ConvexDataUpdateMethod	method to use to update internal convex data
	*/
	UE_DEPRECATED(5.4, "Use SetConvexMeshObject instead")
	void SetChaosConvexMesh(TSharedPtr<Chaos::FConvex, ESPMode::ThreadSafe>&& InChaosConvex, EConvexDataUpdateMethod ConvexDataUpdateMethod = EConvexDataUpdateMethod::AlwaysUpdateConvexData){check(false);}

	ENGINE_API void SetConvexMeshObject(Chaos::FConvexPtr&& InChaosConvex, EConvexDataUpdateMethod ConvexDataUpdateMethod = EConvexDataUpdateMethod::AlwaysUpdateConvexData);

	ENGINE_API void ResetChaosConvexMesh();

	ENGINE_API void ComputeChaosConvexIndices(bool bForceCompute = false);

	ENGINE_API TArray<int32> GetChaosConvexIndices() const;

	/** Get current transform applied to convex mesh vertices */
	virtual FTransform GetTransform() const override final
	{
		return Transform;
	};

	/** 
	 * Modify the transform to apply to convex mesh vertices 
	 * NOTE: When doing this, BodySetup convex meshes need to be recooked - usually by calling InvalidatePhysicsData() and CreatePhysicsMeshes()
	 */
	void SetTransform(const FTransform& InTransform)
	{
		ensure(InTransform.IsValid());
		Transform = InTransform;
	}

	friend FArchive& operator<<(FArchive& Ar, FKConvexElem& Elem);

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	/**
	 * Finds the closest point on the shape given a world position. Input and output are given in world space
	 * @param	WorldPosition			The point we are trying to get close to
	 * @param	BodyToWorldTM			The transform to convert BodySetup into world space
	 * @param	ClosestWorldPosition	The closest point on the shape in world space
	 * @param	Normal					The normal of the feature associated with ClosestWorldPosition.
	 * @return							The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside the shape.
	 */
	ENGINE_API float GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BodyToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const;

	/**	
	 * Finds the shortest distance between the element and a world position. Input and output are given in world space
	 * @param	WorldPosition	The point we are trying to get close to
	 * @param	BodyToWorldTM	The transform to convert BodySetup into world space
	 * @return					The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside one of the shapes.
	 */
	ENGINE_API float GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BodyToWorldTM) const;
	
	ENGINE_API static EAggCollisionShape::Type StaticShapeType;

private:
	/** Helper function to safely copy instances of this shape*/
	void CloneElem(const FKConvexElem& Other);
};
