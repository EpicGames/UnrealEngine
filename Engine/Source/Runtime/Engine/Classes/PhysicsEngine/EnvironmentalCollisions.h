// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "EngineDefines.h"

struct FClothCollisionData;
class USceneComponent;

struct FEnvironmentalCollisions
{
	static ENGINE_API void AppendCollisionDataFromEnvironment(const USceneComponent* SceneComponent, FClothCollisionData& CollisionData);
};
