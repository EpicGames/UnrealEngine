// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosArchive.h"
#include "Chaos/Declares.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/SpatialAccelerationFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"

#include "ExternalSpatialAccelerationPayload.generated.h"

class UPrimitiveComponent;

/**
 * This is a Chaos spatial acceleration payload that can be used for spatial acceleration structures that are meant to only be used
 * in external contexts. This provides safety as we aren't holding particle handles directly but rather going through the primitive component's
 * physics object interface to grab them when needed.
 */
USTRUCT()
struct FExternalSpatialAccelerationPayload
{
	GENERATED_BODY()
public:
	static constexpr bool bHasPayloadOnInternalThread = false;

	ENGINE_API FExternalSpatialAccelerationPayload();

	ENGINE_API void Initialize(TObjectKey<UPrimitiveComponent> InComponent, int32 InBoneId);
	ENGINE_API void Initialize(TObjectKey<UPrimitiveComponent> InComponent, int32 InBoneId, const Chaos::FUniqueIdx& UniqueIdx);

	ENGINE_API Chaos::FGeometryParticle* GetExternalGeometryParticle_ExternalThread() const;
	bool operator==(const FExternalSpatialAccelerationPayload& Other) const;
	bool operator!=(const FExternalSpatialAccelerationPayload& Other) const;
	ENGINE_API Chaos::FUniqueIdx UniqueIdx() const;
	ENGINE_API void Serialize(Chaos::FChaosArchive& Ar);
	UE_DEPRECATED(5.8, "This is deprecated in favor of PrePreFilter")
	bool PrePreQueryFilter(const void* QueryData) const { return false; }
	UE_DEPRECATED(5.8, "This is deprecated in favor of PrePreFilter")
	bool PrePreSimFilter(const void* SimData) const { return false; }
	template <typename FilterDataType>
	bool PrePreFilter(const FilterDataType&) const { return false; }
	bool IsValid() const 
	{
		return CachedUniqueIdx.IsValid(); // Don't use this payload if the UniqueIdx is invalid
	};

	friend ENGINE_API uint32 GetTypeHash(const FExternalSpatialAccelerationPayload& Payload);
private:
	TObjectKey<UPrimitiveComponent> Component;
	int32 BoneId = INDEX_NONE;
	Chaos::FUniqueIdx CachedUniqueIdx;

#if CHAOS_DEBUG_DRAW
public:
	ENGINE_API void DebugDraw(const bool bExternal, const bool bHit) const;
#endif
};

using IExternalSpatialAcceleration = Chaos::ISpatialAcceleration<FExternalSpatialAccelerationPayload, Chaos::FReal, 3>;

inline bool FExternalSpatialAccelerationPayload::operator==(const FExternalSpatialAccelerationPayload& Other) const
{
	return (Component == Other.Component) && (BoneId == Other.BoneId);
}

inline bool FExternalSpatialAccelerationPayload::operator!=(const FExternalSpatialAccelerationPayload& Other) const
{
	return !(*this == Other);
}

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FExternalSpatialAccelerationPayload& AccelerationHandle)
{
	AccelerationHandle.Serialize(Ar);
	return Ar;
}
