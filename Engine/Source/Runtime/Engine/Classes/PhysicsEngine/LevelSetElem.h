// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "LevelSetElem.generated.h"

namespace Chaos
{
	class FLevelSet;
}

USTRUCT()
struct FKLevelSetElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

	FKLevelSetElem() :
		FKShapeElem(EAggCollisionShape::LevelSet)
	{}

	FKLevelSetElem(const FKLevelSetElem& Other)
	{
		CloneElem(Other);
	}

	const FKLevelSetElem& operator=(const FKLevelSetElem& Other)
	{
		CloneElem(Other);
		return *this;
	}

	ENGINE_API void BuildLevelSet(const FTransform& GridTransform, const TArray<double>& GridValues, const FIntVector& GridDims, float GridCellSize);

	ENGINE_API void GetLevelSetData(FTransform& OutGridTransform, TArray<double>& OutGridValues, FIntVector& OutGridDims, float& OutGridCellSize) const;

	virtual FTransform GetTransform() const override final
	{
		return Transform;
	};

	void SetTransform(const FTransform& InTransform)
	{
		ensure(InTransform.IsValid());
		Transform = InTransform;
	}

	// Get the tranform of the center of the level set. In contrast, GetTransform() gets a transform relative to the corner.
	FTransform GetCenteredTransform() const
	{
		return FTransform(
			Transform.GetRotation(),
			Transform.GetLocation() + Transform.TransformVector(UntransformedAABB().GetCenter()),
			Transform.GetScale3D());
	}

	// Set the tranform of the center of the level set. In contrast, SetTransform() sets a transform relative to the corner.
	void SetCenteredTransform(const FTransform& CenteredTransform)
	{
		ensure(CenteredTransform.IsValid());
		Transform = FTransform(
			CenteredTransform.GetRotation(),
			CenteredTransform.GetLocation() - CenteredTransform.TransformVector(UntransformedAABB().GetCenter()),
			CenteredTransform.GetScale3D());
	}

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	// Draw helpers

	/** Get geometry of all cells where the level set function is less than or equal to InteriorThreshold */
	ENGINE_API void GetInteriorGridCells(TArray<FBox>& CellBoxes, double InteriorThreshold = 0.0) const;

	/** Get geometry of all cell faces where level set function changes sign */
	ENGINE_API void GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const;

	// Draw functions
	ENGINE_API virtual void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const override;
	ENGINE_API virtual void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;
	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const;

	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const;
	ENGINE_API FBox UntransformedAABB() const;
	ENGINE_API FIntVector3 GridResolution() const;

	bool Serialize(FArchive& Ar);

	const TSharedPtr<Chaos::FLevelSet, ESPMode::ThreadSafe> GetLevelSet() const
	{
		return LevelSet;
	}

private:

	/** Transform of this element */
	UPROPERTY()
	FTransform Transform;

	TSharedPtr<Chaos::FLevelSet, ESPMode::ThreadSafe> LevelSet;

	/** Helper function to safely copy instances of this shape*/
	ENGINE_API void CloneElem(const FKLevelSetElem& Other);
};

/* Enable our own serialization function to handle FLevelSet */
template<>
struct TStructOpsTypeTraits<FKLevelSetElem> : public TStructOpsTypeTraitsBase2<FKLevelSetElem>
{
	enum
	{
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
