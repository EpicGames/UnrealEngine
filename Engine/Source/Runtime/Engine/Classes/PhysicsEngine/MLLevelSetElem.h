// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsEngine/MLLevelSetModelAndBonesBinningInfo.h"
#include "NNEModelData.h"  
#include "NNERuntimeCPU.h"
#include "Chaos/MLLevelset.h" 
#include "MLLevelSetElem.generated.h"

extern ENGINE_API bool bEnableMLLevelSet;

USTRUCT()
struct FKMLLevelSetElem : public FKShapeElem
{
	GENERATED_BODY()

	FKMLLevelSetElem() :
		FKShapeElem(EAggCollisionShape::MLLevelSet)
	{}

	FKMLLevelSetElem(const FKMLLevelSetElem& Other)
	{
		CloneElem(Other);
	}

	const FKMLLevelSetElem& operator=(const FKMLLevelSetElem& Other)
	{
		CloneElem(Other);
		return *this;
	}

	ENGINE_API void BuildMLLevelSet(Chaos::FMLLevelSetImportData&& MLLevelSetImportData);

	virtual FTransform GetTransform() const override final
	{
		return FTransform();
	}

	// Draw helpers
	/** Get geometry of all cells where the level set function is less than or equal to InteriorThreshold */
	ENGINE_API void GetInteriorGridCells(TArray<FBox>& CellBoxes, double InteriorThreshold = 0.0) const;

	/** Get geometry of all cell faces where level set function changes sign */
	ENGINE_API void GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const;

	// Draw functions
	ENGINE_API virtual void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const override;
	ENGINE_API virtual void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;
	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const;

	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const;
	ENGINE_API FBox UntransformedAABB() const;
	ENGINE_API FIntVector3 GridResolution() const;

	bool Serialize(FArchive& Ar);

	const TSharedPtr<Chaos::FMLLevelSet, ESPMode::ThreadSafe> GetMLLevelSet() const
	{
		return MLLevelSet;
	}

private:
	TSharedPtr<Chaos::FMLLevelSet, ESPMode::ThreadSafe> MLLevelSet;

	UPROPERTY()
	TObjectPtr<UNNEModelData> NNESignedDistanceModelData;

	UPROPERTY()
	TObjectPtr<UNNEModelData> NNEIncorrectZoneModelData;

	/** Helper function to safely copy instances of this shape*/
	ENGINE_API void CloneElem(const FKMLLevelSetElem& Other);
};

/* Enable our own serialization function to handle FMLLevelSet.*/
template<>
struct TStructOpsTypeTraits<FKMLLevelSetElem> : public TStructOpsTypeTraitsBase2<FKMLLevelSetElem>
{
	enum
	{
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
