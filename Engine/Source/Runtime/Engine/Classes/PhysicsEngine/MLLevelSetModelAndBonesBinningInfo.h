// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/DataTable.h"
#include "MLLevelSetModelAndBonesBinningInfo.generated.h"

USTRUCT(BlueprintType)
struct FMLLevelSetModelAndBonesBinningInfo : public FTableRowBase
{
	GENERATED_BODY();

	/* The bone that MLLevelSet is attached to. Note that the deformations near this joint is not trained*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString ParentBoneName;

	/* The bones that are trained for deformation. We suggest only train one bone per MLLevelSet for efficiency*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString ActiveBoneNames;

	/* The path to the DataTable that includes information about NNE Model */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString MLModelInferenceInfoDataTablePath;

	/* The index of the DataTable that includes information about NNE Model for signed distance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString MLModelInferenceInfoDataTableIndex;

	/* The index of the DataTable that includes information about NNE Model for incorrect (safe-danger) zone. If uninitialized, this means no safe zone is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString MLModelInferenceForIncorrectZoneInfoDataTableIndex;

	/* Model is trained for the subset of the rotations for each active bone. */
	/* E.g. Use {1,2} if ActiveBone1 has one and ActiveBone2 has two active rotations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<int32> NumberOfRotationComponentsPerBone = {};

	/* The indices of the rotation components. E.g. use {1,1,2} if ActiveBone1 uses Rot.Y and ActiveBone2 uses Rot.Y and Rot.Z. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<int32> RotationComponentIndexes = {};

	/* Usually MLModels for SDF are trained so that output lies in [-1,1]. To do so the SignedDistances (in the dataset) are divided by SignedDistanceScaling. 
	SignedDistanceScaling is generally defined to be the max length of the training bounding box. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	double SignedDistanceScaling = -1.0;

	/* Resolution of the Grid that is used for debug visualization */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<int32> DebugGridResolution = {50,50,50};

	/* Reference Rotations for the Active Bones. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<double> ReferenceBoneRotations = {};
	
	/* Reference Translations for the Active Bones. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<double> ReferenceBoneTranslations = {};

	/* Min Corner of bounding box that MLLevelSet is trained on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<float> TrainingGridOrigin = {};

	/* First Edge of the bounding box that MLLevelSet is trained on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<float> TrainingGridAxisX = {};

	/* Second Edge of the bounding box that MLLevelSet is trained on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<float> TrainingGridAxisY = {};

	/* Third Edge of the bounding box that MLLevelSet is trained on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<float> TrainingGridAxisZ = {};
};

USTRUCT(BlueprintType)
struct FMLLevelSetModelInferenceInfo : public FTableRowBase
{
	GENERATED_BODY();

	/* The path to the NNE model. At the moment MLLevelSet Asset accepts only mlir.tosa models. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString NNEModelPath;

	/* Model Architecture. Number of neurons in each layer, including input layer and the output layer. */
	/* In most cases input layer is 3 (location input, X,Y,Z coordinates of the query), and output layer is 1 (Scaled Signed Distance or Incorrect Zone Indicator)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	TArray<int32> ModelArchitectureActivationNodeSizes = {};

	/* Model weights {W1,..,Wn} is tokanized into FString as "W1_0,W1_1,...,W1_k1|W2_0,W2_1,...,W2_k2|...|Wn_0,Wn_1,...,Wn_kn" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MLModelBone)
	FString MLModelWeights;
};