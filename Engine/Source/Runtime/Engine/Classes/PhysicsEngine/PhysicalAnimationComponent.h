// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicalAnimationComponent.generated.h"

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

/** Stores info on the type of motor that will be used for a given bone */
USTRUCT(BlueprintType)
struct FPhysicalAnimationData
{
	GENERATED_BODY()

	FPhysicalAnimationData()
		: BodyName(NAME_None)
		, bIsLocalSimulation(true)
		, OrientationStrength(0.f)
		, AngularVelocityStrength(0.f)
		, PositionStrength(0.f)
		, VelocityStrength(0.f)
		, MaxLinearForce(0.f)
		, MaxAngularForce(0.f)
	{
	}

	/** The body we will be driving. We specifically hide this from users since they provide the body name and bodies below in the component API. */
	UPROPERTY()
	FName BodyName;

	/** Whether the drive targets are in world space or local */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	uint8 bIsLocalSimulation : 1;

	/** The strength used to correct orientation error */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float OrientationStrength;

	/** The strength used to correct angular velocity error */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float AngularVelocityStrength;

	/** The strength used to correct linear position error. Only used for non-local simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float PositionStrength;

	/** The strength used to correct linear velocity error. Only used for non-local simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float VelocityStrength;

	/** The max force used to correct linear errors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float MaxLinearForce;

	/** The max force used to correct angular errors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalAnimation)
	float MaxAngularForce;
};

UCLASS(meta = (BlueprintSpawnableComponent), ClassGroup = Physics, Experimental, MinimalAPI)
class UPhysicalAnimationComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	/** Sets the skeletal mesh we are driving through physical animation. Will erase any existing physical animation data. */
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation)
	ENGINE_API void SetSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent);

	/** Applies the physical animation settings to the body given.*/
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation, meta = (UnsafeDuringActorConstruction))
	ENGINE_API void ApplyPhysicalAnimationSettings(FName BodyName, const FPhysicalAnimationData& PhysicalAnimationData);

	/** Applies the physical animation settings to the body given and all bodies below.*/
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation, meta = (UnsafeDuringActorConstruction))
	ENGINE_API void ApplyPhysicalAnimationSettingsBelow(FName BodyName, const FPhysicalAnimationData& PhysicalAnimationData, bool bIncludeSelf = true);

	/** Updates strength multiplyer and any active motors */
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation)
	ENGINE_API void SetStrengthMultiplyer(float InStrengthMultiplyer);
	
	/** Multiplies the strength of any active motors. (can blend from 0-1 for example)*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalAnimation, meta = (ClampMin = "0"))
	float StrengthMultiplyer;

	/**
	*  Applies the physical animation profile to the body given and all bodies below.
	*  @param  BodyName			The body from which we'd like to start applying the physical animation profile. Finds all bodies below in the skeleton hierarchy. None implies all bodies
	*  @param  ProfileName		The physical animation profile we'd like to apply. For each body in the physics asset we search for physical animation settings with this name.
	*  @param  bIncludeSelf		Whether to include the provided body name in the list of bodies we act on (useful to ignore for cases where a root has multiple children)
	*  @param  bClearNotFound	If true, bodies without the given profile name will have any existing physical animation settings cleared. If false, bodies without the given profile name are left untouched.
	*/
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation, meta = (UnsafeDuringActorConstruction))
	ENGINE_API void ApplyPhysicalAnimationProfileBelow(FName BodyName, FName ProfileName, bool bIncludeSelf = true, bool bClearNotFound = false);

	/** 
	 *	Returns the target transform for the given body. If physical animation component is not controlling this body, returns its current transform.
	 */
	UFUNCTION(BlueprintCallable, Category = PhysicalAnimation, meta = (UnsafeDuringActorConstruction))
	ENGINE_API FTransform GetBodyTargetTransform(FName BodyName) const;

	ENGINE_API virtual void InitializeComponent() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

#if WITH_EDITOR
	ENGINE_API void DebugDraw(FPrimitiveDrawInterface* PDI) const;
#endif

	USkeletalMeshComponent* GetSkeletalMesh() const { return SkeletalMeshComponent; }

private:
	UPROPERTY()
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	struct FPhysicalAnimationInstanceData
	{
		struct FConstraintInstance* ConstraintInstance;
		FPhysicsActorHandle TargetActor;
	};

	/** constraints used to apply the drive data */
	TArray<FPhysicalAnimationInstanceData> RuntimeInstanceData;
	TArray<FPhysicalAnimationData> DriveData;

	FDelegateHandle OnTeleportDelegateHandle;

	ENGINE_API void UpdatePhysicsEngine();
	ENGINE_API void UpdatePhysicsEngineImp();
	ENGINE_API void ReleasePhysicsEngine();
	ENGINE_API void InitComponent();

	ENGINE_API void OnTeleport();
	ENGINE_API void UpdateTargetActors(ETeleportType TeleportType);

	static ENGINE_API const FConstraintProfileProperties PhysicalAnimationProfile;

	bool bPhysicsEngineNeedsUpdating;
};
