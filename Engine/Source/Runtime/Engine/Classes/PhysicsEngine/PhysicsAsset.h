// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "PhysicsEngine/RigidBodyIndexPair.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "PhysicsAsset.generated.h"

namespace ECollisionEnabled { enum Type : int; }
namespace EAggCollisionShape { enum Type : int; }

struct FConstraintInstance;
struct FConstraintInstanceAccessor;
struct FReferenceSkeleton;

class UMaterialInterface;
class FMeshElementCollector;
class USkeletalBodySetup;


/**
 * Solver iterations settings for use by RigidBody AnimNode (RBAN) in the Anim Graph. Each RBAN node runs its own solver with these settings.
 *
 * @note These settings have no effect when the Physics Asset is used in a world simulation (i.e., as a ragdoll on a SkeletalMeshComponent).
 */
USTRUCT(BlueprintType)
struct FPhysicsAssetSolverSettings
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FPhysicsAssetSolverSettings();

	/**
	 * RBAN: The number of position iterations to run. The position solve is responsible for depenetration.
	 * Increasing this will improve simulation stability, but increase the cost.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings)
	int32 PositionIterations;

	/**
	 * RBAN: The number of velocity iterations to run. The velocity solve is responsible for restitution (bounce) and friction. 
	 * This should usually be 1, but could be 0 if you don't care about friction and restitution.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings)
	int32 VelocityIterations;

	/**
	 * RBAN: The number of projection iterations to run. The projection phase is a final pass over the constraints, applying
	 * a semi-physical correction to any joint errors remaining after the position and velocity solves. It can be
	 * very helpful to stabilize joint chains, but can cause issues with collision response. The projection magnitude
	 * can be controlled per-constraint in the constraint settings (assuming ProjectionIteration is not zero).
	 * This should be left as 1 in almost all cases.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings)
	int32 ProjectionIterations;

	/**
	 * RBAN: The distance at which collisions are ignored. In general you need this to be a bit larger than the typical relative body
	 * movement in your simulation, but small enough so that we don't have to speculatively create too many unused collisions.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMin = 0))
	float CullDistance;

	/**
	 * RBAN: When bodies are penetrating, this is the maximum velocity delta that can be applied in one frame.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMin = 0))
	float MaxDepenetrationVelocity;

	/**
	 * RBAN: The recommended fixed timestep for the RBAN solver. Set to 0 to run with variable timestep (default).
	 * NOTE: If this value is non-zero and less than the current frame time, the simulation will step multiple times
	 * which increases the cost.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMin = 0))
	float FixedTimeStep;

	/**
	 * RBAN: Whether to use the linear or non-linear solver for RBAN Joints. The linear solver is significantly cheaper than
	 * the non-linear solver when you are running multiple iterations, but is more likely to suffer from jitter. 
	 * In general you should try to use the linear solver and increase the PositionIterations to improve stability if 
	 * possible, only using the non-linear solver as a last resort.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings)
	bool bUseLinearJointSolver;

	/**
	 * RBAN: It enables the use of multi-point contact manifolds, which are created only once at the start of each tick.
	 * When disabled, a single-point contact is generated in each solver iteration which is more expensive.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CollisionSettings)
	bool bUseManifolds;
};


/**
 * Solver settings for use by the Legacy RigidBody AnimNode (RBAN) solver.
 * These settings are no longer used by default and will eventually be deprecated and then removed.
 * 
 * @note These settings have no effect when the Physics Asset is used in a world simulation (ragdoll).
 */
USTRUCT(BlueprintType)
struct FSolverIterations
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FSolverIterations();

	/**
	 * The recommended number of solver iterations. Increase this if collision and joints are fighting, or joint chains are stretching.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 SolverIterations;

	/**
	 * The recommended number of joint sub-iterations. Increasing this can help with chains of long-thin bodies.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 JointIterations;

	/**
	 * The recommended number of collision sub-iterations. Increasing this can help with collision jitter.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 CollisionIterations;

	/**
	 * The recommended number of solver push-out iterations. Increasing this can help with collision penetration problems.
	 */
	 /** Increase this if bodies remain penetrating */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 SolverPushOutIterations;

	/**
	 * The recommended number of joint sub-push-out iterations.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 JointPushOutIterations;

	/**
	 * The recommended number of joint sub-push-out iterations. Increasing this can help with collision penetration problems.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SolverSettings, meta = (ClampMax = 50))
		int32 CollisionPushOutIterations;
};




UENUM()
enum class EPhysicsAssetSolverType: uint8
{
	RBAN,
	World,
};

/**
 * PhysicsAsset contains a set of rigid bodies and constraints that make up a single ragdoll.
 * The asset is not limited to human ragdolls, and can be used for any physical simulation using bodies and constraints.
 * A SkeletalMesh has a single PhysicsAsset, which allows for easily turning ragdoll physics on or off for many SkeletalMeshComponents
 * The asset can be configured inside the Physics Asset Editor.
 *
 * @see https://docs.unrealengine.com/InteractiveExperiences/Physics/PhysicsAssetEditor
 * @see USkeletalMesh
 */

UCLASS(hidecategories=Object, BlueprintType, MinimalAPI, Config=Game, PerObjectConfig, AutoCollapseCategories=(OldSolverSettings))
class UPhysicsAsset : public UObject, public IInterface_PreviewMeshProvider, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** 
	 *	Default skeletal mesh to use when previewing this PhysicsAsset etc. 
	 *	Is the one that was used as the basis for creating this Asset.
	 */
	UPROPERTY()
	TObjectPtr<class USkeletalMesh>  DefaultSkelMesh_DEPRECATED;

	UPROPERTY(AssetRegistrySearchable)
	TSoftObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	UPROPERTY(EditAnywhere, Category = Profiles, meta=(DisableCopyPaste))
	TArray<FName> PhysicalAnimationProfiles;

	UPROPERTY(EditAnywhere, Category = Profiles, meta=(DisableCopyPaste))
	TArray<FName> ConstraintProfiles;

	/** A set of flags for each physics body. Used to store persistent per body editor data. */
	UPROPERTY()
	TArray<int32> EditorBodyFlags;

	UPROPERTY(transient)
	FName CurrentPhysicalAnimationProfileName;

	UPROPERTY(transient)
	FName CurrentConstraintProfileName;

#endif // WITH_EDITORONLY_DATA

	/** Index of bodies that are marked bConsiderForBounds */
	UPROPERTY()
	TArray<int32> BoundsBodies;

	/**
	*	Array of SkeletalBodySetup objects. Stores information about collision shape etc. for each body.
	*	Does not include body position - those are taken from mesh.
	*/
	UPROPERTY(instanced)
	TArray<TObjectPtr<USkeletalBodySetup>> SkeletalBodySetups;

	/** 
	 *	Array of RB_ConstraintSetup objects. 
	 *	Stores information about a joint between two bodies, such as position relative to each body, joint limits etc.
	 */
	UPROPERTY(instanced)
	TArray<TObjectPtr<class UPhysicsConstraintTemplate>> ConstraintSetup;

public:

	/** 
	 * Solver settings when the asset is used with a RigidBody Anim Node (RBAN).
	 */
	UPROPERTY(EditAnywhere, Category = SolverSettings, Config)
	FPhysicsAssetSolverSettings SolverSettings;

	/** 
	 * Old solver settings shown for reference. These will be removed at some point.
	 * When you open an old asset you should see that the settings were transferred to "SolverSettings" above. 
	 * You should usually see:
	 * SolverSettings.PositionIterations = OldSettings.SolverIterations * OldSetting.JointIterations;
	 * SolverSettings.VelocityIterations = 1;
	 * SolverSettings.ProjectionIterations = 1;
	*/
	UPROPERTY(VisibleAnywhere, Category = OldSolverSettings, Config, Meta = (DisplayName="[Not Used] Old Solver Settings"))
	FSolverIterations SolverIterations;

	/** 
	 * Solver type used in physics asset editor. This can be used to make what you see in the asset editor more closely resembles what you
	 * see in game (though there will be differences owing to framerate variation etc). If your asset will primarily be used as a ragdoll 
	 * select "World", but if it will be used in the AnimGraph select "RBAN".
	*/
	UPROPERTY(EditAnywhere, Category = SolverSettings)
	EPhysicsAssetSolverType SolverType;


	/** If true, we skip instancing bodies for this PhysicsAsset on dedicated servers */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Physics)
	uint8 bNotForDedicatedServer:1;

	/** This caches the BodySetup Index by BodyName to speed up FindBodyIndex */
	TMap<FName, int32>					BodySetupIndexMap;

	/** 
	 *	Table indicating which pairs of bodies have collision disabled between them. Used internally. 
	 *	Note, this is accessed from within physics engine, so is not safe to change while physics is running
	 */
	TMap<FRigidBodyIndexPair,bool>		CollisionDisableTable;

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual FString GetDesc() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITOR
	virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	virtual void PostEditUndo() override;

	const TArray<FName>& GetPhysicalAnimationProfileNames() const
	{
		return PhysicalAnimationProfiles;
	}

	const TArray<FName>& GetConstraintProfileNames() const
	{
		return ConstraintProfiles;
	}

	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~ End UObject Interface

	// Find the index of the physics bone that is controlling this graphics bone.
	ENGINE_API int32		FindControllingBodyIndex(const class USkeletalMesh* skelMesh, int32 BoneIndex) const;
	ENGINE_API int32		FindParentBodyIndex(const class USkeletalMesh * skelMesh, int32 StartBoneIndex) const;
	ENGINE_API int32		FindParentBodyIndex(const FReferenceSkeleton& RefSkeleton, const int32 StartBoneIndex) const;
	ENGINE_API int32		FindConstraintIndex(FName ConstraintName) const;
	ENGINE_API int32		FindConstraintIndex(FName Bone1Name, FName Bone2Name) const;
	FName					FindConstraintBoneName(int32 ConstraintIndex) const;
	ENGINE_API int32		FindMirroredBone(const class USkeletalMesh* skelMesh, int32 BoneIndex) const;

	/** Utility for getting indices of all bodies below (and including) the one with the supplied name. */
	ENGINE_API void			GetBodyIndicesBelow(TArray<int32>& OutBodyIndices, FName InBoneName, const USkeletalMesh* InSkelMesh, bool bIncludeParent = true) const;
	ENGINE_API void			GetBodyIndicesBelow(TArray<int32>& OutBodyIndices, const FName InBoneName, const FReferenceSkeleton& RefSkeleton, const bool bIncludeParent = true) const;

	ENGINE_API void			GetNearestBodyIndicesBelow(TArray<int32> & OutBodyIndices, FName InBoneName, const USkeletalMesh* InSkelMesh) const;

	ENGINE_API FBox			CalcAABB(const class USkinnedMeshComponent* MeshComponent, const FTransform& LocalToWorld) const;

	/** Clears physics meshes from all bodies */
	ENGINE_API void ClearAllPhysicsMeshes();
	
#if WITH_EDITOR
	/**
	 * Check if the Bounds can be calculate for the specified MeshComponent.
	 * return true if the skeleton match with the physic asset and the bounds can be calculated, otherwise it will return false.
	 */
	ENGINE_API bool CanCalculateValidAABB(const class USkinnedMeshComponent* MeshComponent, const FTransform& LocalToWorld) const;

	/** Invalidates physics meshes from all bodies. Data will be rebuilt completely. */
	ENGINE_API void InvalidateAllPhysicsMeshes();
#endif

	// @todo document
	void GetCollisionMesh(int32 ViewIndex, FMeshElementCollector& Collector, const FReferenceSkeleton& RefSkeleton, const TArray<FTransform>& SpaceBases, const FTransform& LocalToWorld, const FVector& Scale3D) const;

	// @todo document
	void DrawConstraints(int32 ViewIndex, FMeshElementCollector& Collector, const FReferenceSkeleton& RefSkeleton, const TArray<FTransform>& SpaceBases, const FTransform& LocalToWorld, float Scale) const;

	ENGINE_API void GetUsedMaterials(TArray<UMaterialInterface*>& Materials);

	// Disable collsion between the bodies specified by index
	ENGINE_API void DisableCollision(int32 BodyIndexA, int32 BodyIndexB);

	// Enable collsion between the bodies specified by index
	ENGINE_API void EnableCollision(int32 BodyIndexA, int32 BodyIndexB);

	// Check whether the two bodies specified are enabled for collision
	ENGINE_API bool IsCollisionEnabled(int32 BodyIndexA, int32 BodyIndexB) const;

	// Get the per-primitive collision filtering mode for a body
	ENGINE_API void SetPrimitiveCollision(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex, ECollisionEnabled::Type CollisionEnabled);

	// Get the per-primitive collision filtering mode for a body
	ENGINE_API ECollisionEnabled::Type GetPrimitiveCollision(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex) const;

	// Set whether or not a primitive volume contributes to the mass of the object
	ENGINE_API void SetPrimitiveContributeToMass(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex, bool bContributesToMass);

	// Get whether or not a primitive volume contributes to the mass of the object
	ENGINE_API bool GetPrimitiveContributeToMass(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex) const;

	/** Update the BoundsBodies array and cache the indices of bodies marked with bConsiderForBounds to BoundsBodies array. */
	ENGINE_API void UpdateBoundsBodiesArray();

	/** Update the BodySetup Array Index Map.  */
	ENGINE_API void UpdateBodySetupIndexMap();


	// @todo document
	ENGINE_API int32 FindBodyIndex(FName BodyName) const;

	/** Find all the constraints that are connected to a particular body.
	 * 
	 * @param	BodyIndex		The index of the body to find the constraints for
	 * @param	Constraints		Returns the found constraints
	 **/
	ENGINE_API void BodyFindConstraints(int32 BodyIndex, TArray<int32>& Constraints);

#if WITH_EDITOR
	/** Update skeletal meshes when physics asset changes*/
	ENGINE_API void RefreshPhysicsAssetChange() const;
#endif

	/** Delegate fired when physics asset changes */
	DECLARE_MULTICAST_DELEGATE_OneParam(FRefreshPhysicsAssetChangeDelegate, const UPhysicsAsset*);
	ENGINE_API static FRefreshPhysicsAssetChangeDelegate OnRefreshPhysicsAssetChange;
	/** IPreviewMeshProviderInterface interface */
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true);
	virtual USkeletalMesh* GetPreviewMesh() const;

#if WITH_EDITOR
	/** Gets a constraint by its joint name
	* @param ConstraintName name of the constraint
	* @return ConstraintInstance accessor to the constraint data
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	FConstraintInstanceAccessor GetConstraintByName(FName ConstraintName);

	/** Gets a constraint by its joint name
	* @param Bone1Name name of the first bone in the joint
	* @param Bone2Name name of the second bone in the joint
	* @return ConstraintInstance accessor to the constraint data
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	FConstraintInstanceAccessor GetConstraintByBoneNames(FName Bone1Name, FName Bone2Name);

	/** Gets all constraints
	* @param IncludesTerminated whether or not to return terminated constraints
	* @param OutConstraints returned list of constraints matching the parameters
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Constraints")
	void GetConstraints(bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints);

	FConstraintInstanceAccessor GetConstraintInstanceAccessorByIndex(int32 Index);
	FConstraintInstance* GetConstraintInstanceByIndex(uint32 Index);
#endif

private:

#if WITH_EDITORONLY_DATA
	/** Editor only arrays that are used for rename operations in pre/post edit change*/
	TArray<FName> PrePhysicalAnimationProfiles;
	TArray<FName> PreConstraintProfiles;
#endif


	UPROPERTY(instanced)
	TArray<TObjectPtr<class UBodySetup>> BodySetup_DEPRECATED;

	// Asset User Data
public:
	ENGINE_API void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;

private:
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = UserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;
};
