// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Containers/Array.h"
#include "Chaos/PhysicsObject.h"
#include "Engine/EngineTypes.h"
#include "PhysicsBodyInstanceOwnerInterface.generated.h"

class UObject;
class UBodySetup;
class UMaterialInterface;
class UPhysicalMaterial;
class IPhysicsBodyInstanceOwner;
struct FBodyInstance;
struct FHitResult;
struct FOverlapResult;
struct FPhysicalMaterialMaskParams;
enum ECollisionResponse : int;
enum ECollisionChannel : int;

UINTERFACE(MinimalAPI)
class UPhysicsBodyInstanceOwnerResolver : public UInterface
{
	GENERATED_BODY()
};

class IPhysicsBodyInstanceOwnerResolver
{
	GENERATED_BODY()

public:
	virtual IPhysicsBodyInstanceOwner* ResolvePhysicsBodyInstanceOwner(Chaos::FConstPhysicsObjectHandle PhysicsObject) = 0;
};

/** Interface representing the owner of a FBodyInstance. */
class IPhysicsBodyInstanceOwner
{
public:
	virtual ~IPhysicsBodyInstanceOwner() = default;

	/** Returns the IPhysicsBodyInstanceOwner based on a given hit result. 
	 * @param HiResult Hit result
	 * @param bPreferComponent Whether to prioritize the hit result component (if valid) when resolving the IPhysicsBodyInstanceOwner.
	 */
	static ENGINE_API const IPhysicsBodyInstanceOwner* GetPhysicsBodyInstanceOwnerFromHitResult(const FHitResult& HiResult, bool bPreferComponent = true);

	/** Returns the IPhysicsBodyInstanceOwner based on a given overlap result.
	 * @param OverlapResult Overlap result
	 * @param bPreferComponent Whether to prioritize the overlap result component (if valid) when resolving the IPhysicsBodyInstanceOwner.
	 */
	static ENGINE_API const IPhysicsBodyInstanceOwner* GetPhysicsBodyInstanceOwnerFromOverlapResult(const FOverlapResult& OverlapResult, bool bPreferComponent = true);

	/** Whether the physics is static. */
	virtual bool IsStaticPhysics() const = 0;

	/**
	 * If true, this body instance owner will generate individual overlaps for each overlapping physics body if it is a multi-body object. When false, it will
	 * generate only one overlap, regardless of how many physics bodies it has and how many of them are overlapping another body. This flag has no
	 * influence on single body objects.
	 */
	virtual bool IsMultiBodyOverlap() const = 0;

	/** Returns the source UObject of this body instance owner. */
	virtual UObject* GetSourceObject() const = 0;

	/** Returns the owner of the source object for the body instance */
	virtual UObject* GetSourceObjectOwner() const = 0;
	
	/** Returns the transform of this body instance owner. */
	virtual FTransform GetPhysicsOwnerTransform() const = 0;

	/** 
	 * Get world-space socket transform of the socket matching InSocketName. 
	 */
	virtual FTransform GetPhysicsOwnerSocketTransform(FName InSocketName) const = 0;
	
	/** Returns the channel that the body instance owner belongs to. */
	virtual ECollisionChannel GetCollisionObjectType() const = 0;

	/** Returns the form of collision for this body instance owner. */
	virtual ECollisionEnabled::Type GetCollisionEnabled() const = 0;

	/** Return the BodySetup to use for this body instance owner (single body case) */
	virtual UBodySetup* GetPhysicsBodySetup() const = 0;

	/** Gets the response type given a specific channel. */
	virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const = 0;

	/** Returns the slope override struct for this physics body instance. */
	virtual const struct FWalkableSlopeOverride& GetWalkableSlopeOverride() const = 0;
	
	/** Returns a physics object corresponding to the specified Id. */
	virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const = 0;

	/** Returns true if the body instance owner is movable */
	virtual bool IsPhysicsOwnerMovable() const = 0;
	
	/** Returns true if the body instance owner simulating physics */
	virtual bool IsPhysicsOwnerSimulatingPhysics() const = 0;

	/** Returns velocity of body instance owner. It is often the velocity of the body instance unless overriden by another system like movement */
	virtual FVector GetPhysicsOwnerVelocity() const = 0;

	/** Get the attachment root of the body instance owner object */
	virtual UObject* GetPhysicsOwnerAttachmentRoot() const = 0;
	
	/** Returns true if this body instance should be considered world geometry */
	virtual bool IsPhysicsObjectWorldGeometry() const = 0;
	
	/** Checks if a socket of the name passed in exists on the object owning the body instance */
	virtual bool DoesSocketExistOnPhysicsOwner(FName InSocketName) const = 0;

	/** Returns all physics objects associated to the physics body instance owner. */
	virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const = 0;

	/**
	 * Returns the body instance of the body instance owner.
	 *
	 * @param BoneName		Used to get body associated with specific bone. NAME_None automatically gets the root most body
	 * @param bGetWelded	If the component has been welded to another component and bGetWelded is true we return the single welded BodyInstance that is used in the simulation
	 * @param Index			Index used in Components with multiple body instances
	 *
	 * @return Returns the BodyInstance based on various states (does component have multiple bodies? Is the body welded to another body?)
	 */
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const = 0;

	/** Returns the body instance's physics material override. */
	virtual UPhysicalMaterial* GetPhysicsMaterialOverride() const = 0;

	/** Returns the body instance owner mesh material. */
	virtual UMaterialInterface* GetPhysicsMaterialBase() const = 0;

	/** Returns the body instance owner number of materials. */
	virtual int32 GetNumMaterials() const = 0;

	/** Returns the body instance owner material for the provided material index. */
	virtual UMaterialInterface* GetMaterial(int32 Index) const = 0;

	//~ Begin Deprecation

	UE_DEPRECATED(5.8, "Use GetPhysicsBodyInstanceOwnerFromHitResult instead.")
	static IPhysicsBodyInstanceOwner* GetPhysicsBodyInstandeOwnerFromHitResult(const FHitResult& Result)
	{
		return const_cast<IPhysicsBodyInstanceOwner*>(GetPhysicsBodyInstanceOwnerFromHitResult(Result));
	}

	UE_DEPRECATED(5.8, "Use GetPhysicsBodyInstanceOwnerFromOverlapResult instead.")
	static IPhysicsBodyInstanceOwner* GetPhysicsBodyInstandeOwnerFromOverlapResult(const FOverlapResult& OverlapResult)
	{
		return const_cast<IPhysicsBodyInstanceOwner*>(GetPhysicsBodyInstanceOwnerFromOverlapResult(OverlapResult));
	}
	UE_DEPRECATED(5.8, "GetComplexPhysicalMaterials was removed.")
	virtual void GetComplexPhysicalMaterials(TArray<UPhysicalMaterial*>& OutPhysMaterials, TArray<FPhysicalMaterialMaskParams>* OutPhysMaterialMasks) const {}

	//~ End Deprecation

	friend struct FBodyInstance;
};