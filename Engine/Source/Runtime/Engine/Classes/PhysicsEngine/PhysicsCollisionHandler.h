// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/World.h"
#include "PhysicsCollisionHandler.generated.h"

struct FCollisionNotifyInfo;
struct FRigidBodyCollisionInfo;

UCLASS(hidecategories=(Object), Blueprintable, MinimalAPI)
class UPhysicsCollisionHandler : public UObject
{
	GENERATED_UCLASS_BODY()

	// This class is SUPER BASIC, but games can implement their own handlers to do more advanced physics collisions

	/** How hard an impact must be to trigger effect/sound */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Impact)
	float ImpactThreshold;

	/** Min time between effect/sound being triggered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Impact)
	float ImpactReFireDelay;

	/** Sound to play  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Impact)
	TObjectPtr<class USoundBase> DefaultImpactSound;

	/** Time since last impact sound */
	UPROPERTY()
	float LastImpactSoundTime;

	/** Get the world we are handling collisions for */
	virtual UWorld* GetWorld() const override
	{
		return !HasAnyFlags(RF_ClassDefaultObject) ? CastChecked<UWorld>(GetOuter()) : nullptr;
	}

	/** Gives game-specific ability to handle and filter all physics collisions in one place. This is a good place to play sounds and spawn effects, as it does not require special object-specific code. */
	ENGINE_API virtual void HandlePhysicsCollisions_AssumesLocked(TArray<FCollisionNotifyInfo>& PendingCollisionNotifies);

	/** Handle a single */
	ENGINE_API void DefaultHandleCollision_AssumesLocked(const FRigidBodyCollisionInfo& MyInfo, const FRigidBodyCollisionInfo& OtherInfo, const FCollisionImpactData& RigidCollisionData);

	/** Called after collision handler is allocated for a world. GetWorld should be valid inside this function. */
	virtual void InitCollisionHandler() {}
};
