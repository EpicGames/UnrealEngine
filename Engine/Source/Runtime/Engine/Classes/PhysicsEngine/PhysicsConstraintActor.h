// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//~=============================================================================
// The Basic constraint actor class.
//~=============================================================================

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/RigidBodyBase.h"
#include "PhysicsConstraintActor.generated.h"

UCLASS(ConversionRoot, MinimalAPI, ComponentWrapperClass)
class APhysicsConstraintActor : public ARigidBodyBase
{
	GENERATED_UCLASS_BODY()

	// Cached reference to constraint component
private:
	UPROPERTY(Category = ConstraintActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "JointDrive,Physics|Components|PhysicsConstraint", AllowPrivateAccess = "true"))
	TObjectPtr<class UPhysicsConstraintComponent> ConstraintComp;
public:
	
	UPROPERTY()
	TObjectPtr<class AActor> ConstraintActor1_DEPRECATED;
	UPROPERTY()
	TObjectPtr<class AActor> ConstraintActor2_DEPRECATED;
	UPROPERTY()
	uint32 bDisableCollision_DEPRECATED:1;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void LoadedFromAnotherClass(const FName& OldClassName) override;
#endif // WITH_EDITOR	
	//~ End UObject Interface

public:
	/** Returns ConstraintComp subobject **/
	class UPhysicsConstraintComponent* GetConstraintComp() const { return ConstraintComp; }
};



