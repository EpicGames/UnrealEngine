// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsConstraintComponent.generated.h"

class AActor;
class UPrimitiveComponent;
struct FBodyInstance;

#if UE_WITH_REMOTE_OBJECT_HANDLE
enum class EObjectMigrationSendType : uint8;
#endif

/**
 *	This is effectively a joint that allows you to connect 2 rigid bodies together. You can create different types of joints using the various parameters of this component.
 */
UCLASS(ClassGroup=Physics, meta=(BlueprintSpawnableComponent), HideCategories=(Activation,"Components|Activation", Physics, Mobility), ShowCategories=("Physics|Components|PhysicsConstraint"), MinimalAPI)
class UPhysicsConstraintComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Pointer to first Actor to constrain.  */
	UPROPERTY(EditInstanceOnly, Category=Constraint)
	TObjectPtr<AActor> ConstraintActor1;

	/** 
	 *	Name of first component property to constrain. If Actor1 is NULL, will look within Owner.
	 *	If this is NULL, will use RootComponent of Actor1
	 */
	UPROPERTY(EditAnywhere, Category=Constraint)
	FConstrainComponentPropName ComponentName1;


	/** Pointer to second Actor to constrain. */
	UPROPERTY(EditInstanceOnly, Category=Constraint)
	TObjectPtr<AActor> ConstraintActor2;

	/** 
	 *	Name of second component property to constrain. If Actor2 is NULL, will look within Owner. 
	 *	If this is NULL, will use RootComponent of Actor2
	 */
	UPROPERTY(EditAnywhere, Category=Constraint)
	FConstrainComponentPropName ComponentName2;


	/** Allows direct setting of first component to constraint. */
	TWeakObjectPtr<UPrimitiveComponent> OverrideComponent1;

	/** Allows direct setting of second component to constraint. */
	TWeakObjectPtr<UPrimitiveComponent> OverrideComponent2;


	UPROPERTY(instanced)
	TObjectPtr<class UPhysicsConstraintTemplate> ConstraintSetup_DEPRECATED;

	/** Notification when constraint is broken. */
	UPROPERTY(BlueprintAssignable)
	FConstraintBrokenSignature OnConstraintBroken;

	/** Notification when constraint plasticity drive target changes. */
	UPROPERTY(BlueprintAssignable)
	FPlasticDeformationEventSignature OnPlasticDeformation;

public:
	/** All constraint settings */
	UPROPERTY(EditAnywhere, Category=ConstraintComponent, meta=(ShowOnlyInnerProperties))
	FConstraintInstance			ConstraintInstance;

	//Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
#if WITH_EDITORONLY_DATA || UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#endif
	//End UObject interface

	//Begin ActorComponent interface
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual void OnRegister() override;
#endif // WITH_EDITOR
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void InitializeComponent() override;
	//End ActorComponent interface

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
	ENGINE_API void ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData = nullptr);
	ENGINE_API bool HasValidPhysicsState() const;
	ENGINE_API void GetConstrainedBodyIds(TPair<FRemoteServerId, FRemoteServerId>& OwnerServerIds, TPair<FRemoteObjectId, FRemoteObjectId>& ObjectIds) const;
	ENGINE_API bool AreConstrainedComponentsInSameActor() const;
	ENGINE_API void AddReInitDelegatesToComponents();
	ENGINE_API void InitComponentConstraintWithoutUpdatingFrames();
	ENGINE_API void CheckAndUpdateConstraintPT();
	bool GetIsBrokenLocal() const { return bIsBrokenLocal; };
#endif

	UFUNCTION()
	ENGINE_API void ReInitConstraintAndApplyPhysicsState(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);

	ENGINE_API void ReInitConstraintAndApplyPhysicsState(FPhysScene* PhysScene, float DeltaTime);

	//~ Begin SceneComponent Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditComponentMove(bool bFinished) override;
#endif // WITH_EDITOR
	//~ End SceneComponent Interface

	/** Get the body frame. Works without constraint being created */
	ENGINE_API FTransform GetBodyTransform(EConstraintFrame::Type Frame) const;
	
	/** Get body bounding box. Works without constraint being created */
	ENGINE_API FBox GetBodyBox(EConstraintFrame::Type Frame) const;

	/** Initialize the frames and creates constraint */
	ENGINE_API void InitComponentConstraint();

	/** Break the constraint */
	ENGINE_API void TermComponentConstraint();

	/** Directly specify component to connect. Will update frames based on current position. */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetConstrainedComponents(UPrimitiveComponent* Component1, FName BoneName1, UPrimitiveComponent* Component2, FName BoneName2);

	/** Get connected components and potential related attachement bones */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void GetConstrainedComponents(UPrimitiveComponent*& OutComponent1, FName& OutBoneName1, UPrimitiveComponent*& OutComponent2, FName& OutBoneName2);

	/** Break this constraint */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void BreakConstraint();

	/** Enables/Disables linear position drive 
	 *	
	 *	@param bEnableDriveX	Indicates whether the drive for the X-Axis should be enabled
	 *	@param bEnableDriveY	Indicates whether the drive for the Y-Axis should be enabled
	 *	@param bEnableDriveZ	Indicates whether the drive for the Z-Axis should be enabled
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearPositionDrive(bool bEnableDriveX, bool bEnableDriveY, bool bEnableDriveZ);

	/** Enables/Disables linear position drive 
	 *	
	 *	@param bEnableDriveX	Indicates whether the drive for the X-Axis should be enabled
	 *	@param bEnableDriveY	Indicates whether the drive for the Y-Axis should be enabled
	 *	@param bEnableDriveZ	Indicates whether the drive for the Z-Axis should be enabled
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearVelocityDrive(bool bEnableDriveX, bool bEnableDriveY, bool bEnableDriveZ);

	/** Enables/Disables angular orientation drive. Only relevant if the AngularDriveMode is set to Twist and Swing 
	 *	
	 *	@param bEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	 *	@param bEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint", meta=(DeprecatedFunction, DeprecationMessage = "Use SetOrientationDriveTwistAndSwing instead."))
	void SetAngularOrientationDrive(bool bEnableSwingDrive, bool bEnableTwistDrive)
	{
		SetOrientationDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
	}
	/** Enables/Disables angular orientation drive. Only relevant if the AngularDriveMode is set to Twist and Swing
	*
	*	@param bEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param bEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetOrientationDriveTwistAndSwing(bool bEnableTwistDrive, bool bEnableSwingDrive);

	/** Enables/Disables the angular orientation slerp drive. Only relevant if the AngularDriveMode is set to SLERP
	*
	*	@param bEnableSLERP     	Indicates whether the SLERP drive should be enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetOrientationDriveSLERP(bool bEnableSLERP);

	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint", meta = (DeprecatedFunction, DeprecationMessage = "Use SetAngularVelocityDriveTwistAndSwing instead."))
	void SetAngularVelocityDrive(bool bEnableSwingDrive, bool bEnableTwistDrive)
	{
		SetAngularVelocityDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
	}

	/** Enables/Disables angular velocity twist and swing drive. Only relevant if the AngularDriveMode is set to Twist and Swing
	*
	*	@param bEnableSwingDrive	Indicates whether the drive for the swing axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*	@param bEnableTwistDrive	Indicates whether the drive for the twist axis should be enabled. Only relevant if the AngularDriveMode is set to Twist and Swing
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularVelocityDriveTwistAndSwing(bool bEnableTwistDrive, bool bEnableSwingDrive);

	/** Enables/Disables the angular velocity slerp drive. Only relevant if the AngularDriveMode is set to SLERP
	*
	*	@param bEnableSLERP     	Indicates whether the SLERP drive should be enabled. Only relevant if the AngularDriveMode is set to SLERP
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularVelocityDriveSLERP(bool bEnableSLERP);

	/** Switches the angular drive mode between SLERP and Twist And Swing
	*
	*	@param DriveMode	The angular drive mode to use. SLERP uses shortest spherical path, but will not work if any angular constraints are locked. Twist and Swing decomposes the path into the different angular degrees of freedom but may experience gimbal lock
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularDriveMode(EAngularDriveMode::Type DriveMode);

	/** Toggles the angular drive acceleration mode. When enabled, the acceleration of the angular drive is not affected by the inertia of the constrained objects around the drive axis. */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularDriveAccelerationMode(bool bAccelerationMode);

	/** Sets the target position for the linear drive. 
	 *	@param InPosTarget		Target position
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearPositionTarget(const FVector& InPosTarget);
	
	/** Sets the target velocity for the linear drive. 
	 *	@param InVelTarget		Target velocity
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearVelocityTarget(const FVector& InVelTarget);

	/** Sets the drive params for the linear drive. 
	 *	@param PositionStrength		Positional strength for the drive (stiffness)
	 *	@param VelocityStrength 	Velocity strength of the drive (damping)
	 *	@param InForceLimit	Max force applied by the drive
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearDriveParams(float PositionStrength, float VelocityStrength, float InForceLimit);

	/** Toggles the linear drive acceleration mode. When enabled, the acceleration of the angular drive is not affected by the inertia of the constrained objects along the drive axis. */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearDriveAccelerationMode(bool bAccelerationMode);

	/** Sets the target orientation for the angular drive. 
	 *	@param InPosTarget		Target orientation
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularOrientationTarget(const FRotator& InPosTarget);

	
	/** Sets the target velocity for the angular drive. 
	 *	@param InVelTarget		Target velocity
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularVelocityTarget(const FVector& InVelTarget);

	/** Sets the drive params for the angular drive. 
	 *	@param PositionStrength		Positional strength for the drive (stiffness)
	 *	@param VelocityStrength 	Velocity strength of the drive (damping)
	 *	@param InForceLimit	Max force applied by the drive
	 */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularDriveParams(float PositionStrength, float VelocityStrength, float InForceLimit);


	/** Sets the LinearX Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param LimitSize		Size of limit
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearXLimit(ELinearConstraintMotion ConstraintType, float LimitSize);

	/** Sets the LinearY Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param LimitSize		Size of limit
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearYLimit(ELinearConstraintMotion ConstraintType, float LimitSize);

	/** Sets the LinearZ Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param LimitSize		Size of limit
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearZLimit(ELinearConstraintMotion ConstraintType, float LimitSize);

	/** Sets the Angular Swing1 Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param Swing1LimitAngle	Size of limit in degrees
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularSwing1Limit(EAngularConstraintMotion MotionType, float Swing1LimitAngle);

	/** Sets the Angular Swing2 Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param Swing2LimitAngle	Size of limit in degrees
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularSwing2Limit(EAngularConstraintMotion MotionType, float Swing2LimitAngle);

	/** Sets the Angular Twist Motion Type
	*	@param ConstraintType	New Constraint Type
	*	@param TwistLimitAngle	Size of limit in degrees
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularTwistLimit(EAngularConstraintMotion ConstraintType, float TwistLimitAngle);

	/** Sets the Linear Breakable properties
	*	@param bLinearBreakable		Whether it is possible to break the joint with linear force
	*	@param LinearBreakThreshold	Force needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearBreakable(bool bLinearBreakable, float LinearBreakThreshold);

	/** Sets the Linear Plasticity properties
	*	@param bLinearPlasticity		    Whether it is possible to reset the target angle from the Linear displacement
	*	@param LinearPlasticityThreshold	Percent deformation needed to reset the rest length of the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetLinearPlasticity(bool bLinearPlasticity, float LinearPlasticityThreshold, EConstraintPlasticityType PlasticityType);

	/** Sets the Angular Breakable properties
	*	@param bAngularBreakable		Whether it is possible to break the joint with angular force
	*	@param AngularBreakThreshold	Torque needed to break the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularBreakable(bool bAngularBreakable, float AngularBreakThreshold);

	/** Sets the Angular Plasticity properties
	*	@param bAngularPlasticity		Whether it is possible to reset the target angle from the angular displacement
	*	@param AngularPlasticityThreshold	Degrees needed to reset the rest state of the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetAngularPlasticity(bool bAngularPlasticity, float AngularPlasticityThreshold);

	/** Sets the contact transfer scale properties
	*	@param ContactTransferScale		   Set the contact transfer scale for the parent of the joint
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetContactTransferScale(float ContactTransferScale);

	/** Gets the current Angular Twist of the constraint */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API float GetCurrentTwist() const;

	/** Gets the current Swing1 of the constraint */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API float GetCurrentSwing1() const;

	/** Gets the current Swing2 of the constraint */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API float GetCurrentSwing2() const;

	/** 
	 *	Update the reference frames held inside the constraint that indicate the joint location in the reference frame 
	 *	of the two connected bodies. You should call this whenever the constraint or either Component moves, or if you change
	 *	the connected Components. This function does nothing though once the joint has been initialized.
	 */
	ENGINE_API void UpdateConstraintFrames();

	// Pass in reference frame in. If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint. 
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetConstraintReferenceFrame(EConstraintFrame::Type Frame, const FTransform& RefFrame);
	
	// Pass in reference position in (maintains reference orientation). If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint.
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetConstraintReferencePosition(EConstraintFrame::Type Frame, const FVector& RefPosition);
	
	// Pass in reference orientation in (maintains reference position). If the constraint is currently active, this will set its active local pose. Otherwise the change will take affect in InitConstraint.
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetConstraintReferenceOrientation(EConstraintFrame::Type Frame, const FVector& PriAxis, const FVector& SecAxis);

	// If true, the collision between the two rigid bodies of the constraint will be disabled.
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetDisableCollision(bool bDisableCollision);

	// If true, joint projection is enabled. Projection is a semi-physics post-solve correction for fixing small errors, and a teleport for fixing larger errors. See SetProjectionParams
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetProjectionEnabled(bool bProjectionEnabled);

	// Is projection enabled. See SetProjectionEnabled
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	bool IsProjectionEnabled() const;

	/**
	* Set the projection settings for use when projection is enabled. See SetProjectionEnabled.
	* For ragdolls you usually require that ProjectionLinearAlpha and ProjectionAngularAlpha be zero. They are most useful for cosmetic chains etc. ProjectionLinearTolerance and ProjectionAngularTolerance
	* are emergency error recovery settings and should only rarely affect the simulation. If the simulation is significantly affected by the Tolerance settings, it likely indicates a setup stability issue.
	* 
	* @param ProjectionLinearAlpha Controls the semi-physical correction of linear error remaining after the joint solve. Will add energy into the system. Best for joint chains connected to a kinematic.
	* @param ProjectionAngularAlpha Controls the semi-physical correction of angular error remaining after the joint solve. Will add energy into the system. Best for joint chains connected to a kinematic.
	* @param ProjectionLinearTolerance Linear errors above this will be corrected with a non-physical teleport.
	* @param ProjectionAngularTolerance Angular errors above this will be corrected with a non-physical teleport.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void SetProjectionParams(float ProjectionLinearAlpha, float ProjectionAngularAlpha, float ProjectionLinearTolerance, float ProjectionAngularTolerance);

	// Retrieve the constraint force most recently applied to maintain this constraint. Returns 0 forces if the constraint is not initialized or broken.
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API void GetConstraintForce(FVector& OutLinearForce, FVector& OutAngularForce);

	// Retrieve the status of constraint being broken.
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API bool IsBroken();

	/** Gets the constraint object */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsConstraint")
	ENGINE_API FConstraintInstanceAccessor GetConstraint();

#if WITH_EDITOR
	ENGINE_API void UpdateSpriteTexture();
#endif

protected:

	friend class FConstraintComponentVisualizer;

	/** Get the body instance that we want to constrain to */
	ENGINE_API FBodyInstance* GetBodyInstance(EConstraintFrame::Type Frame) const;

	/** Get the physics object that we want to constrain to. */
	ENGINE_API Chaos::FPhysicsObject* GetPhysicsObject(EConstraintFrame::Type Frame) const;

	/** Internal util to get body transform from actor/component name/bone name information */
	ENGINE_API FTransform GetBodyTransformInternal(EConstraintFrame::Type Frame, FName InBoneName) const;
	/** Internal util to get body box from actor/component name/bone name information */
	ENGINE_API FBox GetBodyBoxInternal(EConstraintFrame::Type Frame, FName InBoneName) const;
	/** Internal util to get component from actor/component name */
	ENGINE_API UPrimitiveComponent* GetComponentInternal(EConstraintFrame::Type Frame) const;

	/** Routes the FConstraint callback to the dynamic delegate */
	ENGINE_API void OnConstraintBrokenHandler(FConstraintInstance* BrokenConstraint);

	/** Routes the FConstraint callback to the dynamic delegate */
	ENGINE_API void OnPlasticDeformationHandler(FConstraintInstance* Constraint);

	/** Returns the scale of the constraint as it will be passed into the ConstraintInstance*/
	ENGINE_API float GetConstraintScale() const;

private:

	bool bIsBrokenLocal;

	/** Wrapper that calls our constraint broken delegate */
	ENGINE_API void OnConstraintBrokenWrapper(int32 ConstraintIndex);

	/** Wrapper that calls our constraint plasticity delegate */
	ENGINE_API void OnPlasticDeformationWrapper(int32 ConstraintIndex);
#if UE_WITH_REMOTE_OBJECT_HANDLE
	bool CanReInitConstraintAndApplyPhysicsState() const;

	void MigratePhysicsState(FArchive& Ar);
	
	ENGINE_API bool CanSendMigratedPhysicsState(const struct FUObjectMigrationContext* MigrationContext, EObjectMigrationSendType ObjMigrationSendType) const;

	void CheckAndUnbindPhysicsStateDelegates(bool bCheckIfBound = true);

	void UnBindPreTickDelegate();

	FDelegateHandle PreTickDelegateHandle;
#endif
};

