// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// Complete constraint definition used by rigid body physics.
// 
// Defaults here will give you a ball and socket joint.
// Positions are in Physics scale.
// When adding stuff here, make sure to update URB_ConstraintSetup::CopyConstraintParamsFrom
//~=============================================================================

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsConstraintTemplate.generated.h"

USTRUCT()
struct FPhysicsConstraintProfileHandle
{
	GENERATED_BODY()
	
	UPROPERTY()
	FConstraintProfileProperties ProfileProperties;

	UPROPERTY(EditAnywhere, Category=Constraint)
	FName ProfileName;
};

UCLASS(hidecategories=Object, MinimalAPI)
class UPhysicsConstraintTemplate : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Joint, meta=(ShowOnlyInnerProperties))
	FConstraintInstance DefaultInstance;

	/** Handles to the constraint profiles applicable to this constraint*/
	UPROPERTY()
	TArray<FPhysicsConstraintProfileHandle> ProfileHandles;
	
	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

	/** Whether the constraint template has this profile. Note we cannot return a pointer to the constraint profile as it's in a TArray by value and could be resized leading to an unsafe API */
	bool ContainsConstraintProfile(FName ProfileName) const
	{
		const FPhysicsConstraintProfileHandle* FoundHandle = ProfileHandles.FindByPredicate([ProfileName](const FPhysicsConstraintProfileHandle& ProfileHandle) { return ProfileHandle.ProfileName == ProfileName; });
		return FoundHandle != nullptr;
	}

#if WITH_EDITOR
	void AddConstraintProfile(FName ProfileName)
	{
		FPhysicsConstraintProfileHandle& NewHandle = ProfileHandles.AddDefaulted_GetRef();
		NewHandle.ProfileName = ProfileName;
		NewHandle.ProfileProperties = DefaultInstance.ProfileInstance;	//Copy current settings into new profile
	}

	void RemoveConstraintProfile(FName ProfileName)
	{
		for(int32 HandleIdx = ProfileHandles.Num() - 1; HandleIdx >= 0; --HandleIdx)
		{
			if(ProfileHandles[HandleIdx].ProfileName == ProfileName)
			{
				ProfileHandles.RemoveAtSwap(HandleIdx);
			}
		}
	}
#endif

	/** Find profile with given name and apply it to constraint instance. Note we cannot return a pointer to the constraint profile as it's in a TArray by value and could be resized leading to an unsafe API */
	void ApplyConstraintProfile(FName ProfileName, FConstraintInstance& CI, bool bDefaultIfNotFound) const
	{
		bool bFound = false;
		if (ProfileName == NAME_None)
		{
			CI.CopyProfilePropertiesFrom(DefaultProfile);
			bFound = true;
		}
		else
		{
			for(const FPhysicsConstraintProfileHandle& Handle : ProfileHandles)
			{
				if(Handle.ProfileName == ProfileName)
				{
					CI.CopyProfilePropertiesFrom(Handle.ProfileProperties);
					bFound = true;
					break;
				}
			}
		}

		if(!bFound && bDefaultIfNotFound)
		{
			CI.CopyProfilePropertiesFrom(DefaultProfile);
		}
	}

	/** 
	 * Returns the properties associated with the named constraint profile. If the profile isn't found 
	 * it returns the default profile properties.
	 */
	const FConstraintProfileProperties& GetConstraintProfilePropertiesOrDefault(FName ProfileName) const
	{
		if (ProfileName != NAME_None)
		{
			for (const FPhysicsConstraintProfileHandle& Handle : ProfileHandles)
			{
				if (Handle.ProfileName == ProfileName)
				{
					return Handle.ProfileProperties;
				}
			}
		}
		return DefaultProfile;
	}

#if WITH_EDITOR
	/** Copy constraint instance into default profile.*/
	void SetDefaultProfile(FConstraintInstance& CI)
	{
		DefaultProfile = CI.ProfileInstance;
	}


	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Update the profile instance if anything changes */
	ENGINE_API void UpdateProfileInstance();

	ENGINE_API FName GetCurrentConstraintProfileName() const;
	ENGINE_API void UpdateConstraintProfiles(const TArray<FName>& Profiles);
	ENGINE_API void DuplicateConstraintProfile(FName DuplicateFromName, FName DuplicateToName);
	ENGINE_API void RenameConstraintProfile(FName CurrentName, FName NewName);
#endif


private:
	// Only needed for old content! Pre VER_UE4_ALL_PROPS_TO_CONSTRAINTINSTANCE
	void CopySetupPropsToInstance(FConstraintInstance* Instance);

	/** When no profile is selected, use these settings. Only needed in editor as we serialize it into DefaultInstance on save*/
	UPROPERTY(transient)
	FConstraintProfileProperties DefaultProfile;


public:	//DEPRECATED
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName JointName_DEPRECATED;
	UPROPERTY()
	FName ConstraintBone1_DEPRECATED;
	UPROPERTY()
	FName ConstraintBone2_DEPRECATED;
	UPROPERTY()
	FVector Pos1_DEPRECATED;
	UPROPERTY()
	FVector PriAxis1_DEPRECATED;
	UPROPERTY()
	FVector SecAxis1_DEPRECATED;
	UPROPERTY()
	FVector Pos2_DEPRECATED;
	UPROPERTY()
	FVector PriAxis2_DEPRECATED;
	UPROPERTY()
	FVector SecAxis2_DEPRECATED;
	UPROPERTY()
	uint32 bEnableProjection_DEPRECATED : 1;
	UPROPERTY()
	float ProjectionLinearTolerance_DEPRECATED;
	UPROPERTY()
	float ProjectionAngularTolerance_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearXMotion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearYMotion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum ELinearConstraintMotion> LinearZMotion_DEPRECATED;
	UPROPERTY()
	float LinearLimitSize_DEPRECATED;
	UPROPERTY()
	uint32 bLinearLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	float LinearLimitStiffness_DEPRECATED;
	UPROPERTY()
	float LinearLimitDamping_DEPRECATED;
	UPROPERTY()
	uint32 bLinearBreakable_DEPRECATED : 1;
	UPROPERTY()
	float LinearBreakThreshold_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularSwing1Motion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularSwing2Motion_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<enum EAngularConstraintMotion> AngularTwistMotion_DEPRECATED;
	UPROPERTY()
	uint32 bSwingLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	uint32 bTwistLimitSoft_DEPRECATED : 1;
	UPROPERTY()
	float Swing1LimitAngle_DEPRECATED;
	UPROPERTY()
	float Swing2LimitAngle_DEPRECATED;
	UPROPERTY()
	float TwistLimitAngle_DEPRECATED;
	UPROPERTY()
	float SwingLimitStiffness_DEPRECATED;
	UPROPERTY()
	float SwingLimitDamping_DEPRECATED;
	UPROPERTY()
	float TwistLimitStiffness_DEPRECATED;
	UPROPERTY()
	float TwistLimitDamping_DEPRECATED;
	UPROPERTY()
	uint32 bAngularBreakable_DEPRECATED : 1;
	UPROPERTY()
	float AngularBreakThreshold_DEPRECATED;
#endif
};



