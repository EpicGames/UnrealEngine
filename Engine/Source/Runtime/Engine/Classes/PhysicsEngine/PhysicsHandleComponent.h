// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsHandleComponent.generated.h"

/**
 *	Utility object for moving physics objects around.
 */
UCLASS(collapsecategories, ClassGroup=Physics, hidecategories=Object, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPhysicsHandleComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	/** Component we are currently holding */
	UPROPERTY()
	TObjectPtr<class UPrimitiveComponent> GrabbedComponent;

	/** Name of bone, if we are grabbing a skeletal component */
	FName GrabbedBoneName;

	/** Are we currently constraining the rotation of the grabbed object. */
	uint32 bRotationConstrained:1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle)
	uint32 bSoftAngularConstraint : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicsHandle)
	uint32 bSoftLinearConstraint : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsHandle)
	uint32 bInterpolateTarget : 1;

	/** Linear damping of the handle spring. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle, meta = (EditCondition = "bSoftLinearConstraint"))
	float LinearDamping;

	/** Linear stiffness of the handle spring */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle, meta = (EditCondition = "bSoftLinearConstraint"))
	float LinearStiffness;

	/** Angular damping of the handle spring */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle, meta = (EditCondition = "bSoftAngularConstraint"))
	float AngularDamping;

	/** Angular stiffness of the handle spring */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle, meta = (EditCondition = "bSoftAngularConstraint"))
	float AngularStiffness;

	/** Target transform */
	FTransform TargetTransform;
	/** Current transform */
	FTransform CurrentTransform;

	/** How quickly we interpolate the physics target transform */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicsHandle, meta = (EditCondition = "bInterpolateTarget"))
	float InterpolationSpeed;


protected:

	FTransform PreviousTransform;
	bool bPendingConstraint;

	FPhysicsUserData PhysicsUserData;
	FConstraintInstance ConstraintInstance;
	FPhysicsActorHandle GrabbedHandle;
	FPhysicsActorHandle KinematicHandle;
	FPhysicsConstraintHandle ConstraintHandle;
	FVector ConstraintLocalPosition; // Position of constraint in the grabbed body local space (updated when grabbing)
	FRotator ConstraintLocalRotation;

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface.

public:

	/** Grab the specified component */
	UE_DEPRECATED(4.14, "Please use GrabComponentAtLocation or GrabComponentAtLocationWithRotation")
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle", meta = (DeprecatedFunction, DeprecationMessage = "Please use GrabComponentAtLocation or GrabComponentAtLocationWithRotation"))
	ENGINE_API virtual void GrabComponent(class UPrimitiveComponent* Component, FName InBoneName, FVector GrabLocation, bool bConstrainRotation);

	/** Grab the specified component at a given location. Does NOT constraint rotation which means the handle will pivot about GrabLocation.*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void GrabComponentAtLocation(class UPrimitiveComponent* Component, FName InBoneName, FVector GrabLocation);

	/** Grab the specified component at a given location and rotation. Constrains rotation.*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void GrabComponentAtLocationWithRotation(class UPrimitiveComponent* Component, FName InBoneName, FVector Location, FRotator Rotation);

	/** Release the currently held component */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle")
	ENGINE_API virtual void ReleaseComponent();

	/** Returns the currently grabbed component, or null if nothing is grabbed. */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API class UPrimitiveComponent* GetGrabbedComponent() const;

	/** Set the target location */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle")
	ENGINE_API void SetTargetLocation(FVector NewLocation);

	/** Set the target rotation */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle")
	ENGINE_API void SetTargetRotation(FRotator NewRotation);

	/** Set target location and rotation */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle")
	ENGINE_API void SetTargetLocationAndRotation(FVector NewLocation, FRotator NewRotation);

	/** Get the current location and rotation */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|PhysicsHandle")
	ENGINE_API void GetTargetLocationAndRotation(FVector& TargetLocation, FRotator& TargetRotation) const;

	/** Set linear damping */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void SetLinearDamping(float NewLinearDamping);

	/** Set linear stiffness */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void SetLinearStiffness(float NewLinearStiffness);

	/** Set angular damping */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void SetAngularDamping(float NewAngularDamping);

	/** Set angular stiffness */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void SetAngularStiffness(float NewAngularStiffness);

	/** Set interpolation speed */
	UFUNCTION(BlueprintCallable, Category = "Physics|Components|PhysicsHandle")
	ENGINE_API void SetInterpolationSpeed(float NewInterpolationSpeed);

protected:
	/** Move the kinematic handle to the specified */
	ENGINE_API virtual void UpdateHandleTransform(const FTransform& NewTransform);

	/** Update the underlying constraint drive settings from the params in this component */
	ENGINE_API virtual void UpdateDriveSettings();

	ENGINE_API virtual void GrabComponentImp(class UPrimitiveComponent* Component, FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);

};



