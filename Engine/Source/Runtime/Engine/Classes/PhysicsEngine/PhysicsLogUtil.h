// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

class AActor;

class FString;
class UActorComponent;

namespace PhysicsLogUtil
{
	ENGINE_API FString MakeComponentNetIDString(const UActorComponent* Component);
	ENGINE_API FString MakeComponentNameString(const UActorComponent* Component);
	ENGINE_API FString MakeActorNameString(const UActorComponent* ActorComponent);
	ENGINE_API FString MakeActorNameString(const AActor* Actor);
}
