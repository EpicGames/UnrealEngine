// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Chaos/PhysicsObjectInterface.h"
#include "Math/MathFwd.h"

#include "PhysicsObjectBlueprintLibrary.generated.h"

UCLASS()
class UPhysicsObjectBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintPure, Category = "Physics Object")
	static FClosestPhysicsObjectResult GetClosestPhysicsObjectFromWorldLocation(UPrimitiveComponent* Component, const FVector& WorldLocation);

	UFUNCTION(BlueprintPure, Category = "Physics Object")
	static bool ExtractClosestPhysicsObjectResults(const FClosestPhysicsObjectResult& Result, FName& OutName);

	UFUNCTION(BlueprintPure, Category = "Physics Object")
	static FTransform GetPhysicsObjectWorldTransform(UPrimitiveComponent* Component, FName BoneName);

	/**
	* Apply a physics radial impulse with an optional strain on a specific component
	* Effect is applied within a sphere. When using linear falloff the effect will be minimum at the outer edge of the sphere and maximum at its center
	* @param Component		Primitive component to apply the impulse / strain on
	* @param Origin			Positition of the origin of the radial effect in world space
	* @param Radius			Radius of the radial effect ( beyond the radius, impulse will not be applied )
	* @param Strength		Strength of the impulse to apply ( Unit : (Kg * m / s) or ( m /s ) if bVelChange is true
	* @param FallOff		Type of falloff to use ( constant, linear )
	* @param bApplyStrain	Whether or not to apply strain on top of the impulse ( for destructible objects )
	* @param Strain			If bApplyStrain is true, Strain to apply to the physics particles ( for destructible objects )
	* @param bVelChange		If true, impulse Strength parameter is interpretation as a change of velocity
	* @param MinValue		When using linear falloff, this define the falloff value at the outer edge of the sphere
	* @param MaxValue		When using linear falloff, this define the falloff value at the center of the sphere
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics Object")
	static ENGINE_API void ApplyRadialImpulse(UPrimitiveComponent* Component, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, float Strain, bool bVelChange = false, float MinValue = 0.f, float MaxValue = 1.f);
};