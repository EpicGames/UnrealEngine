// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/PhysicsObject.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Physics/Experimental/ChaosScopedSceneLock.h"
#include "PhysicsObjectPhysicsCoreInterface.h"
#include "Containers/ArrayView.h"
#include <type_traits>

template<EPhysicsInterfaceScopedLockType LockType>
class FLockedPhysicsObjectExternalInterface
{
public:
	using InterfaceType = std::conditional_t<LockType == EPhysicsInterfaceScopedLockType::Read, Chaos::FReadPhysicsObjectInterface_External, Chaos::FWritePhysicsObjectInterface_External>;

	FLockedPhysicsObjectExternalInterface(FChaosScene* Scene, InterfaceType&& InInterface)
		: Lock(Scene, LockType)
		, Interface(InInterface)
	{
	}

	FLockedPhysicsObjectExternalInterface(FLockedPhysicsObjectExternalInterface& Other) = delete;
	FLockedPhysicsObjectExternalInterface& operator=(FLockedPhysicsObjectExternalInterface& Other) = delete;

	FLockedPhysicsObjectExternalInterface(FLockedPhysicsObjectExternalInterface&& Other) = default;
	FLockedPhysicsObjectExternalInterface& operator=(FLockedPhysicsObjectExternalInterface&& Other) = default;

	InterfaceType& GetInterface() { return Interface; }
	InterfaceType* operator->() { return &Interface; }

	// The FORCEINLINE gets around some linker issues when trying to call Destroy.
	FORCEINLINE void Release()
	{
		Lock.Release();
	}

private:
	FScopedSceneLock_Chaos Lock;
	InterfaceType Interface;
};

using FLockedReadPhysicsObjectExternalInterface = FLockedPhysicsObjectExternalInterface<EPhysicsInterfaceScopedLockType::Read>;
using FLockedWritePhysicsObjectExternalInterface = FLockedPhysicsObjectExternalInterface<EPhysicsInterfaceScopedLockType::Write>;

class FPhysicsObjectExternalInterface : public Chaos::FPhysicsObjectInterface, public PhysicsObjectPhysicsCoreInterface
{
public:
	static ENGINE_API FLockedReadPhysicsObjectExternalInterface LockRead(FChaosScene* Scene);
	static ENGINE_API FLockedReadPhysicsObjectExternalInterface LockRead(Chaos::FConstPhysicsObjectHandle InObject);
	static ENGINE_API FLockedReadPhysicsObjectExternalInterface LockRead(TArrayView<const Chaos::FConstPhysicsObjectHandle> InObjects);
	static ENGINE_API FLockedWritePhysicsObjectExternalInterface LockWrite(FChaosScene* Scene);
	static ENGINE_API FLockedWritePhysicsObjectExternalInterface LockWrite(TArrayView<const Chaos::FPhysicsObjectHandle> InObjects);

	static ENGINE_API Chaos::FReadPhysicsObjectInterface_External GetRead_AssumesLocked();
	static ENGINE_API Chaos::FWritePhysicsObjectInterface_External GetWrite_AssumesLocked();

	static ENGINE_API UPrimitiveComponent* GetComponentFromPhysicsObject(UWorld* World, Chaos::FPhysicsObjectHandle PhysicsObject);
};
