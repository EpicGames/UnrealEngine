// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsSettings.h: Declares the PhysicsSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Templates/Casts.h"
#include "Engine/DeveloperSettings.h"
#include "PhysicsSettingsEnums.h"
#include "BodySetupEnums.h"
#include "Chaos/ChaosEngineInterface.h"
#include "GameFramework/WorldSettings.h"
#include "PhysicsCoreTypes.h"
#include "PhysicsSettingsCore.h"

#include "PhysicsSettings.generated.h"

/**
 * Structure that represents the name of physical surfaces.
 */
USTRUCT()
struct FPhysicalSurfaceName
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TEnumAsByte<enum EPhysicalSurface> Type;

	UPROPERTY()
	FName Name;

	FPhysicalSurfaceName()
		: Type(SurfaceType_Max)
	{}
	FPhysicalSurfaceName(EPhysicalSurface InType, const FName& InName)
		: Type(InType)
		, Name(InName)
	{}
};

/** 
  * Settings container for Chaos physics engine settings, accessed in Chaos through a setting provider interface.
  * See: IChaosSettingsProvider
  */
USTRUCT()
struct FChaosPhysicsSettings
{
	GENERATED_BODY()

	ENGINE_API FChaosPhysicsSettings();

	/** Default threading model to use on module initialisation. Can be switched at runtime using p.Chaos.ThreadingModel */
	UPROPERTY(EditAnywhere, Category = ChaosPhysics)
	EChaosThreadingMode DefaultThreadingModel;

	/** The framerate/timestep ticking mode when running with a dedicated thread */
	UPROPERTY(EditAnywhere, Category = Framerate)
	EChaosSolverTickMode DedicatedThreadTickMode;

	/** The buffering mode to use when running with a dedicated thread */
	UPROPERTY(EditAnywhere, Category = Framerate)
	EChaosBufferMode DedicatedThreadBufferMode;

	ENGINE_API void OnSettingsUpdated();
};

UENUM()
namespace ESettingsDOF
{
	enum Type : int
	{
		/** Allows for full 3D movement and rotation. */
		Full3D,
		/** Allows 2D movement along the Y-Z plane. */
		YZPlane,
		/** Allows 2D movement along the X-Z plane. */
		XZPlane,
		/** Allows 2D movement along the X-Y plane. */
		XYPlane,
	};
}

/** Default settings for physics replication using EPhysicsReplicationMode::Resimulation */
USTRUCT()
struct FPhysicsReplicationResimulationSettings
{
	GENERATED_BODY();

	/** Enable positional error threshold to trigger resimulation */
	UPROPERTY(config, EditDefaultsOnly, Category = "Replication", Meta = (InlineEditConditionToggle))
	bool bEnableResimulationErrorPositionThreshold;

	/** Enable positional error threshold to trigger resimulation and set distance in centimeters before a state discrepancy triggers a resimulation */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (editcondition = "bEnableResimulationErrorPositionThreshold"))
	float ResimulationErrorPositionThreshold;

	/** Enable rotational error threshold to trigger resimulation */
	UPROPERTY(config, EditDefaultsOnly, Category = "Replication", Meta = (InlineEditConditionToggle))
	bool bEnableResimulationErrorRotationThreshold;

	/** Enable rotational error threshold to trigger resimulation and set rotation difference in degrees before a state discrepancy triggers a resimulation */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (editcondition = "bEnableResimulationErrorRotationThreshold"))
	float ResimulationErrorRotationThreshold;

	/** Enable linear velocity error threshold to trigger resimulation */
	UPROPERTY(config, EditDefaultsOnly, Category = "Replication", Meta = (InlineEditConditionToggle))
	bool bEnableResimulationErrorLinearVelocityThreshold;

	/** Enable linear velocity error threshold to trigger resimulation and set velocity difference in centimeters / second before a state discrepancy triggers a resimulation */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (editcondition = "bEnableResimulationErrorLinearVelocityThreshold"))
	float ResimulationErrorLinearVelocityThreshold;

	/** Enable angular velocity error threshold to trigger resimulation */
	UPROPERTY(config, EditDefaultsOnly, Category = "Replication", Meta = (InlineEditConditionToggle))
	bool bEnableResimulationErrorAngularVelocityThreshold;

	/** Enable angular velocity error threshold to trigger resimulation and set velocity degrees / second before a state discrepancy triggers a resimulation */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (editcondition = "bEnableResimulationErrorAngularVelocityThreshold"))
	float ResimulationErrorAngularVelocityThreshold;

	FPhysicsReplicationResimulationSettings()
		: bEnableResimulationErrorPositionThreshold(true)
		, ResimulationErrorPositionThreshold(10.0f)
		, bEnableResimulationErrorRotationThreshold(true)
		, ResimulationErrorRotationThreshold(4.0f)
		, bEnableResimulationErrorLinearVelocityThreshold(false)
		, ResimulationErrorLinearVelocityThreshold(5.0f)
		, bEnableResimulationErrorAngularVelocityThreshold(false)
		, ResimulationErrorAngularVelocityThreshold(2.0f)
	{ }
};

/** Default settings for physics replication LOD */
USTRUCT()
struct FPhysicsReplicationLODSettings
{
	GENERATED_BODY();

	/** Enable physics replication LOD */
	UPROPERTY(config, EditDefaultsOnly, Category = "Replication")
	bool bEnablePhysicsReplicationLOD;

	/** Minimum distance in the physics replication LODs Base Distance. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MinimumBaseDistance;

	/** Multiplier for adding focal particles bounding box radius to the physics replication LODs Base Distance. Value of 0.75 adds 75% of the radius on top of MinimumBaseDistance. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float BaseDistanceRadiusMultiplier;

	/** 'BaseDistance' is a result of MinimumBaseDistance + (the focal particles radius * BaseDistanceRadiusMultiplier)
	* Set how many BaseDistance's from the focal particle where within physics replicated objects will use EPhysicsReplicationMode::Resimulation.
	* Resimulation is a fully forward predicted physics replication, for a seamless LOD transition this should be equal to or lower than BaseDistancesForFullPrediction, read that property for information.
	* NOTE: The resulting distance will be added on top of the focal particles radius. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float BaseDistancesForResimulationMode;

	/** 'BaseDistance' is a result of MinimumBaseDistance + (the focal particles radius * BaseDistanceRadiusMultiplier)
	* Set how many BaseDistance's from the focal particle where within physics replicated objects will be in the clients forward predicted timeline.
	* Physics replication LOD gradually forward predicts the physics replication when objects get closer to focal points in LOD.
	* Read the TimeOverDistance property for details on how forward prediction is gradually applied up until the distance of DistanceForFullPrediction. 
	* NOTE: The local player is typically forward predicted (and should then be a focal particle in LOD), while other physics replicated objects are typically replicated without forward prediction.
	* NOTE: The resulting distance will be added on top of the focal particles radius. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float BaseDistancesForFullPrediction;


	/** Time/Distance alignment value, used as Distance * TimeOverDistance = Time
	* Time is how far behind the current (forward predicted) timeline replication should run
	* Starts outside of DistanceForFullPrediction (calculated from BaseDistancesForFullPrediction) and gets clamped by the received states timeline.
	* Lower value = Slower timeline transition which produces a larger LOD radius.
	* EXAMPLE: At 500cm away an object should be 37,5ms behind the clients timeline if DistanceForFullPrediction is 250cm and TimeOverDistance is 0.15ms/cm. (500cm - 250cm) * 0.15ms/cm = 37,5ms */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float TimeOverDistance;

	FPhysicsReplicationLODSettings()
		: bEnablePhysicsReplicationLOD(false)
		, MinimumBaseDistance(200.0f)
		, BaseDistanceRadiusMultiplier(0.75f)
		, BaseDistancesForResimulationMode(0.25f)
		, BaseDistancesForFullPrediction(0.8f)
		, TimeOverDistance(0.15f)
	{ }
};

/** Physics Prediction Settings */
USTRUCT()
struct FPhysicsPredictionSettings
{
	GENERATED_BODY();

	/** bEnablePhysicsResimulation has been renamed, please use bEnablePhysicsHistoryCapture*/
	UPROPERTY(config)
	bool bEnablePhysicsResimulation_DEPRECATED;

	/** ResimulationErrorThreshold has been renamed and moved, please use FPhysicsReplicationResimulationSettings.ResimulationErrorPositionThreshold*/
	UPROPERTY(config)
	float ResimulationErrorThreshold_DEPRECATED;

	/** Enable networked physics prediction (experimental)
	* This syncs the physics tick number between client and server and keeps it in sync via time dilation performed on the client, see APlayerController::GetPhysicsTimestamp().
	* If an AActor::PhysicsReplicationMode is set to use Resimulation this will also enable RewindData to cache physics history on the client which is required by resimulation replication.
	* IMPORTANT: Physics Prediction needs Physics -> Framerate -> Tick Physics Async enabled to function as intended. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	bool bEnablePhysicsPrediction;

	/** Enables FRewindData to cache physics history
	* Note: This is not recommended for networked physics unless developing a custom resimulation solution since this starts caching physics on both client and server, 
	 instead only enable bEnablePhysicsPrediction which will automatically enable FRewindData caching on the client if needed by the chosen replication mode. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	bool bEnablePhysicsHistoryCapture;

	/** Amount of RTT (Round Trip Time) latency for the prediction to support in milliseconds. */
	UPROPERTY(EditAnywhere, Category = "Replication")
	float MaxSupportedLatencyPrediction;

	/** Default settings for physics replication using EPhysicsReplicationMode::Resimulation. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (DisplayName = "Default Resimulation Replication Settings (EPhysicsReplicationMode::Resimulation)"))
	FPhysicsReplicationResimulationSettings ResimulationSettings;

	/** Default settings for physics replication LOD. */
	UPROPERTY(EditAnywhere, Category = "Replication", meta = (DisplayName = "Default settings for physics replication LOD"))
	FPhysicsReplicationLODSettings PhysicsReplicationLODSettings;

	FPhysicsPredictionSettings()
		: bEnablePhysicsResimulation_DEPRECATED(false)
		, ResimulationErrorThreshold_DEPRECATED(10.0f)
		, bEnablePhysicsPrediction(false)
		, bEnablePhysicsHistoryCapture(false)
		, MaxSupportedLatencyPrediction(1000)
	{ }
};

UENUM()
namespace ESettingsLockedAxis
{
	enum Type : int
	{
		/** No axis is locked. */
		None,
		/** Lock movement along the x-axis. */
		X,
		/** Lock movement along the y-axis. */
		Y,
		/** Lock movement along the z-axis. */
		Z,
		/** Used for backwards compatibility. Indicates that we've updated into the new struct. */
		Invalid
	};
}

/**
 * Default physics settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Physics"), MinimalAPI)
class UPhysicsSettings : public UPhysicsSettingsCore
{
	GENERATED_UCLASS_BODY()

	/** Settings for Networked Physics Prediction, experimental. */
	UPROPERTY(Config, EditAnywhere, Category = Replication, meta = (DisplayName = "Physics Prediction (Experimental)"))
	FPhysicsPredictionSettings PhysicsPrediction;

	/** Default settings for physics replication using EPhysicsReplicationMode::Default */
	UPROPERTY(config, EditAnywhere, Category = Replication, meta = (DisplayName = "Default Physics Replication Settings (EPhysicsReplicationMode::Default)"))
	FRigidBodyErrorCorrection PhysicErrorCorrection;

	UPROPERTY(config)
	TEnumAsByte<ESettingsLockedAxis::Type> LockedAxis_DEPRECATED;

	/** Useful for constraining all objects in the world, for example if you are making a 2D game using 3D environments.*/
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	TEnumAsByte<ESettingsDOF::Type> DefaultDegreesOfFreedom;
	
	/**
	*  If true, the internal physx face to UE face mapping will not be generated. This is a memory optimization available if you do not rely on face indices returned by scene queries. */
	UPROPERTY(config, EditAnywhere, Category = Optimization)
	bool bSuppressFaceRemapTable;

	/** If true, store extra information to allow FindCollisionUV to derive UV info from a line trace hit result, using the FindCollisionUV utility */
	UPROPERTY(config, EditAnywhere, Category = Optimization, meta = (DisplayName = "Support UV From Hit Results", ConfigRestartRequired = true))
	bool bSupportUVFromHitResults;

	/**
	* If true, physx will not update unreal with any bodies that have moved during the simulation. This should only be used if you have no physx simulation or you are manually updating the unreal data via polling physx.  */
	UPROPERTY(config, EditAnywhere, Category = Optimization)
	bool bDisableActiveActors;

	/** Whether to disable generating KS pairs, enabling this makes switching between dynamic and static slower for actors - but speeds up contact generation by early rejecting these pairs*/
	UPROPERTY(config, EditAnywhere, Category = Optimization)
	bool bDisableKinematicStaticPairs;

	/** Whether to disable generating KK pairs, enabling this speeds up contact generation, however it is required when using APEX destruction. */
	UPROPERTY(config, EditAnywhere, Category = Optimization)
	bool bDisableKinematicKinematicPairs;

	/**
	*  If true CCD will be ignored. This is an optimization when CCD is never used which removes the need for physx to check it internally. */
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	bool bDisableCCD;

	/** Min Delta Time below which anim dynamics and rigidbody nodes will not simulate. */
	UPROPERTY(config, EditAnywhere, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"), Category = Framerate)
	float AnimPhysicsMinDeltaTime;

	/** Whether to simulate anim physics nodes in the tick where they're reset. */
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	bool bSimulateAnimPhysicsAfterReset;

	/** Min Physics Delta Time; the simulation will not step if the delta time is below this value */
	UPROPERTY(config, EditAnywhere, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "0.0001", UIMax = "0.0001"), Category = Framerate)
	float MinPhysicsDeltaTime;

	/** Max Physics Delta Time to be clamped. */
	UPROPERTY(config, EditAnywhere, meta=(ClampMin="0.0013", UIMin = "0.0013", ClampMax="1.0", UIMax="1.0"), Category=Framerate)
	float MaxPhysicsDeltaTime;

	/** Whether to substep the physics simulation. This feature is still experimental. Certain functionality might not work correctly*/
	UPROPERTY(config, EditAnywhere, Category = Framerate)
	bool bSubstepping;

	/** Whether to substep the async physics simulation. This feature is still experimental. Certain functionality might not work correctly*/
	UPROPERTY(config, EditAnywhere, Category = Framerate)
	bool bSubsteppingAsync;

	/** Whether to tick physics simulation on an async thread. This feature is still experimental. Certain functionality might not work correctly*/
	UPROPERTY(config, EditAnywhere, Category = Framerate)
	bool bTickPhysicsAsync;

	/** If using async, the time step size to tick at. This feature is still experimental. Certain functionality might not work correctly*/
	UPROPERTY(config, EditAnywhere, Category = Framerate, meta=(editcondition = "bTickPhysicsAsync"))
	float AsyncFixedTimeStepSize;

	/** Max delta time (in seconds) for an individual simulation substep. */
	UPROPERTY(config, EditAnywhere, meta = (ClampMin = "0.0013", UIMin = "0.0013", ClampMax = "1.0", UIMax = "1.0", editcondition = "bSubStepping"), Category=Framerate)
	float MaxSubstepDeltaTime;

	/** Max number of substeps for physics simulation. */
	UPROPERTY(config, EditAnywhere, meta = (ClampMin = "1", UIMin = "1", ClampMax = "16", UIMax = "16", editcondition = "bSubstepping"), Category=Framerate)
	int32 MaxSubsteps;

	/** Physics delta time smoothing factor for sync scene. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"), Category = Framerate)
	float SyncSceneSmoothingFactor;

	/** Physics delta time initial average. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, meta = (ClampMin = "0.0013", UIMin = "0.0013", ClampMax = "1.0", UIMax = "1.0"), Category = Framerate)
	float InitialAverageFrameRate;

	/** The number of frames it takes to rebuild the PhysX scene query AABB tree. The bigger the number, the smaller fetchResults takes per frame, but the more the tree deteriorates until a new tree is built */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, meta = (ClampMin = "4", UIMin = "4"), Category = Framerate)
	int PhysXTreeRebuildRate;

	// PhysicalMaterial Surface Types
	UPROPERTY(config, EditAnywhere, Category=PhysicalSurfaces)
	TArray<FPhysicalSurfaceName> PhysicalSurfaces;

	/** If we want to Enable MPB or not globally. This is then overridden by project settings if not enabled. **/
	UPROPERTY(config, EditAnywhere, Category = Broadphase)
	FBroadphaseSettings DefaultBroadphaseSettings;

	/** Minimum velocity delta required on a collinding object for Chaos to send a hit event */
	UPROPERTY(config, EditAnywhere, Category = ChaosPhysics)
	float MinDeltaVelocityForHitEvents;

	/** Chaos physics engine settings */
	UPROPERTY(config, EditAnywhere, Category = ChaosPhysics)
	FChaosPhysicsSettings ChaosSettings;

	/** Allow/disallow partial island sleep for all islands.
	* We disable the dynamics of a rigid body at rest ("put it to sleep") even if the surrounding bodies in the island are still actively simulated ("the island is awake").
	* This is a CPU optimization for large simulations with many motionless bodies (e.g. structures stacks and unstructured piles at rest).
	* Note that partial island sleep is only executed internally if there are enough motionless bodies in the respective island (see Minimum Motionless). */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep", ConfigRestartRequired = false))
	bool bPartialIslandSleep;

	/** Enable/disable momentum propagation through the constraint graph during partial island waking. 
	* New collisions, constraints or user-induced actor manipulation generate partial wake events with sleeping bodies.
	* The linear and angular momentum of an active, dynamic body is defined as its mass and inertia times its linear and angular velocity, respectively.
	* This parameter defines the decimal percentage propagated through constraints between sleeping bodies. 
	* 0.3 means that 30% of the incoming momentum is carried over to the next body.
	* (0.0 to disable, 1.0 to propagate without dissipation). */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = bPartialIslandSleep, ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep.MomentumPropagation", ConfigRestartRequired = false))
	float MomentumPropagation;

	/** Enable/disable end of step partial waking caused by velocity solver impulses between any active and sleeping body pair.
	* The solver impulse is converted to a change in velocity through division by the rigid body mass.
	* The change in velocity induced by the velocity solver needs to be above this threshold to trigger the wake of the sleep.
	* Note that only one layer of sleeping bodies can be woken up per step, i.e. propagation through a chain of X bodies will take X steps.
	* (0.0 to disable). */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ClampMin = "0.0", EditCondition = bPartialIslandSleep, ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep.PostStepWakeThreshold", ConfigRestartRequired = false))
	float PostStepWakeThreshold;

	/** The minimum decimal percentage of rigid bodies in an island which need to be motionless to activate partial island sleep internally. 
	* 0.0: Partial island sleep and wake is always executed internally.
	* 1.0: All bodies need to be motionless for partial island sleep and wake to be executed internally. */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = bPartialIslandSleep, ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep.MinimumMotionlessRatio", ConfigRestartRequired = false))
	float MinimumMotionless;

	/** The linear wake threshold is computed as this multiplier times the body's linear sleep threshold. 
	* It is used to determine when an active, dynamic body involved in a new constraint or collision will trigger a partial wake.
	* It also defines the minimum motion momentum (multiplied by the body's mass) to limit momentum propagation throughout the constraint graph. */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ClampMin = "1.0", EditCondition = bPartialIslandSleep, ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep.LinearWakeThresholdMultiplier", ConfigRestartRequired = false))
	float LinearWakeThresholdMultiplier;

	/** The angular wake threshold is computed as this multiplier times the body's angular sleep threshold.
	* It is used to determine when an active, dynamic body involved in a new constraint or collision will trigger a partial wake.
	* It also defines the minimum motion momentum (multiplied by the body's inertia) to limit momentum propagation throughout the constraint graph. */
	UPROPERTY(config, EditAnywhere, Category = "ChaosPhysics|SleepSettings", meta = (ClampMin = "1.0", EditCondition = bPartialIslandSleep, ConsoleVariable = "p.Chaos.Solver.Sleep.PartialIslandSleep.AngularWakeThresholdMultiplier", ConfigRestartRequired = false))
	float AngularWakeThresholdMultiplier;


public:
	static UPhysicsSettings* Get()
	{
		return GetMutableDefault<UPhysicsSettings>();
	}

	UFUNCTION(BlueprintCallable, Category = "Physics")
	int32 GetPhysicsHistoryCount() const
	{
		return FMath::Max<int32>(1, FMath::CeilToInt(0.001f * PhysicsPrediction.MaxSupportedLatencyPrediction / AsyncFixedTimeStepSize));
	}

	ENGINE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange( const FProperty* Property ) const override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Load Material Type data from INI file **/
	/** this changes displayname meta data. That means we won't need it outside of editor*/
	ENGINE_API void LoadSurfaceType();

#endif // WITH_EDITOR
};
