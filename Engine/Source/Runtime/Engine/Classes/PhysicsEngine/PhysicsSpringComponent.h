// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "PhysicsSpringComponent.generated.h"

class UPrimitiveComponent;

/** 
 *  Note: this component is still work in progress. Uses raycast springs for simple vehicle forces
 *	Used with objects that have physics to create a spring down the X direction
 *	ie. point X in the direction you want generate spring.
 */
UCLASS(hidecategories=(Object, Mobility, LOD), ClassGroup=Physics, showcategories=Trigger, MinimalAPI)
class UPhysicsSpringComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Specifies how much strength the spring has. The higher the SpringStiffness the more force the spring can push on a body with. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Physics)
	float SpringStiffness;

	/** Specifies how quickly the spring can absorb energy of a body. The higher the damping the less oscillation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Physics)
	float SpringDamping;

	/** Determines how long the spring will be along the X-axis at rest. The spring will apply 0 force on a body when it's at rest. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Physics, meta = (UIMin = 0.01, ClampMin = 0.01))
	float SpringLengthAtRest;

	/** Determines the radius of the spring. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Physics, meta = (UIMin = 0.01, ClampMin = 0.01))
	float SpringRadius;

	/** Strength of thrust force applied to the base object. */
	UPROPERTY(BlueprintReadWrite, Category = Physics)
	TEnumAsByte<enum ECollisionChannel> SpringChannel;

	/** If true, the spring will ignore all components in its own actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Physics, meta = (UIMin = 0.01, ClampMin = 0.01))
	bool bIgnoreSelf;

	/** The current compression of the spring. A spring at rest will have SpringCompression 0. */
	UPROPERTY(BlueprintReadOnly, Category = Physics, transient)
	float SpringCompression;

	//~ Begin UActorComponent Interface
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface

	/** Returns the spring compression as a normalized scalar along spring direction.
	 *  0 implies spring is at rest
	 *  1 implies fully compressed */
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API float GetNormalizedCompressionScalar() const;

	/** Returns the spring resting point in world space.*/
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API FVector GetSpringRestingPoint() const;

	/** Returns the spring current end point in world space.*/
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API FVector GetSpringCurrentEndPoint() const;

	/** Returns the spring direction from start to resting point */
	UFUNCTION(BlueprintCallable, Category = Physics)
	ENGINE_API FVector GetSpringDirection() const;

private:

	/** Sweeps along spring direction to see if spring needs to compress. Returned Collision time is independent of spring radius */
	UPrimitiveComponent* GetSpringCollision(const FVector& Start, const FVector& End, float& CollisionTime) const;
	
	/** Computes new spring compression and force */
	FVector ComputeNewSpringCompressionAndForce(const FVector& End, const float DeltaTime, float& NewSpringCompression) const;

	/** We want to automatically set relative position of attached children */
	void UpdateAttachedPosition() const;

	/** Given a length, returns the point along the spring that is Length units away from the spring start */
	FVector SpringPositionFromLength(float Length) const;

	FVector CurrentEndPoint;
};



