// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/RigidBodyBase.h"
#include "PhysicsThruster.generated.h"

/** 
 *	Attach one of these on an object using physics simulation and it will apply a force down the negative-X direction
 *	ie. point X in the direction you want the thrust in.
 */
UCLASS(hideCategories=(Input,Collision,Replication), showCategories=("Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass)
class APhysicsThruster : public ARigidBodyBase
{
	GENERATED_UCLASS_BODY()

private:
	/** Thruster component */
	UPROPERTY(Category = Physics, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Activation,Components|Activation", AllowPrivateAccess = "true"))
	TObjectPtr<class UPhysicsThrusterComponent> ThrusterComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
#endif

public:
	/** Returns ThrusterComponent subobject **/
	class UPhysicsThrusterComponent* GetThrusterComponent() const { return ThrusterComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif
};



