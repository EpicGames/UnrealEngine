// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "PhysicsThrusterComponent.generated.h"

/** 
 *	Used with objects that have physics to apply a force down the negative-X direction
 *	ie. point X in the direction you want the thrust in.
 */
UCLASS(hidecategories=(Object, Mobility, LOD), ClassGroup=Physics, showcategories=Trigger, MinimalAPI, meta=(BlueprintSpawnableComponent))
class UPhysicsThrusterComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** Strength of thrust force applied to the base object. */
	UPROPERTY(BlueprintReadWrite, interp, Category=Physics)
	float ThrustStrength;

	//~ Begin UActorComponent Interface
#if WITH_EDITORONLY_DATA
	virtual void OnRegister() override;
#endif // WITH_EDITORONLY_DATA
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface
};



