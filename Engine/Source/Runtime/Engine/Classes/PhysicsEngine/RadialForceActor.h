// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/RigidBodyBase.h"
#include "RadialForceActor.generated.h"

class UBillboardComponent;

UCLASS(MinimalAPI, hideCategories=(Collision, Input), showCategories=("Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass)
class ARadialForceActor : public ARigidBodyBase
{
	GENERATED_UCLASS_BODY()

private:
	/** Force component */
	UPROPERTY(Category = RadialForceActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Activation,Components|Activation,Physics,Physics|Components|RadialForce", AllowPrivateAccess = "true"))
	TObjectPtr<class URadialForceComponent> ForceComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif
public:

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(DeprecatedFunction))
	virtual void FireImpulse();
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(DeprecatedFunction))
	virtual void EnableForce();
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(DeprecatedFunction))
	virtual void DisableForce();
	UFUNCTION(BlueprintCallable, Category="Physics", meta=(DeprecatedFunction))
	virtual void ToggleForce();
	// END DEPRECATED


#if WITH_EDITOR
	//~ Begin AActor Interface.
	virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	//~ End AActor Interface.
#endif

public:
	/** Returns ForceComponent subobject **/
	class URadialForceComponent* GetForceComponent() const { return ForceComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};



