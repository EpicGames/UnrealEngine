// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "CollisionQueryParams.h"
#include "RadialForceComponent.generated.h"

/**
 *	Used to emit a radial force or impulse that can affect physics objects and or destructible objects.
 */
UCLASS(hidecategories = (Object, Mobility, LOD, Physics), ClassGroup = Physics, showcategories = Trigger, meta = (BlueprintSpawnableComponent), MinimalAPI)
class URadialForceComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	/** The radius to apply the force or impulse in */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category=RadialForceComponent)
	float Radius;

	/** How the force or impulse should fall off as object are further away from the center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RadialForceComponent)
	TEnumAsByte<enum ERadialImpulseFalloff> Falloff;

	/** How strong the impulse should be */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Impulse)
	float ImpulseStrength;

	/** If true, the impulse will ignore mass of objects and will always result in a fixed velocity change */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Impulse)
	uint32 bImpulseVelChange:1;

	/** If true, do not apply force/impulse to any physics objects that are part of the Actor that owns this component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Impulse)
	uint32 bIgnoreOwningActor:1;

	/** How strong the force should be */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Force)
	float ForceStrength;

	/** If > 0.f, will cause damage to destructible meshes as well  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Destructible)
	float DestructibleDamage;

	/** Fire a single impulse */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|RadialForce")
	ENGINE_API virtual void FireImpulse();

	/** Add an object type for this radial force to affect */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|RadialForce")
	ENGINE_API virtual void AddObjectTypeToAffect(TEnumAsByte<enum EObjectTypeQuery> ObjectType);

	/** Remove an object type that is affected by this radial force */
	UFUNCTION(BlueprintCallable, Category="Physics|Components|RadialForce")
	ENGINE_API virtual void RemoveObjectTypeToAffect(TEnumAsByte<enum EObjectTypeQuery> ObjectType);

	/** Add a collision channel for this radial force to affect */
	ENGINE_API void AddCollisionChannelToAffect(enum ECollisionChannel CollisionChannel);

protected:
	/** The object types that are affected by this radial force */
	UPROPERTY(EditAnywhere, Category=RadialForceComponent)
	TArray<TEnumAsByte<enum EObjectTypeQuery> > ObjectTypesToAffect;

	/** Cached object query params derived from ObjectTypesToAffect */
	FCollisionObjectQueryParams CollisionObjectQueryParams;

protected:
	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	
	ENGINE_API virtual void BeginPlay() override;
	//~ End UActorComponent Interface.

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface.

	/** Update CollisionObjectQueryParams from ObjectTypesToAffect */
	ENGINE_API void UpdateCollisionObjectQueryParams();
};



