// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// RigidBodyBase: The base class of all rigid bodies.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "RigidBodyBase.generated.h"

UCLASS(ClassGroup=Physics, abstract,MinimalAPI)
class ARigidBodyBase : public AActor
{
	GENERATED_UCLASS_BODY()

};

