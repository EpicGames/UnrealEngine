// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Endian save storage for a pair of rigid body indices used as a key in the CollisionDisableTable TMap.
 */
struct FRigidBodyIndexPair
{
	/** Pair of indices */
	int32		Indices[2];
	
	/** Default constructor required for use with TMap */
	FRigidBodyIndexPair()
	{}

	/**
	 * Constructor, taking unordered pair of indices and generating a key.
	 *
	 * @param	Index1	1st unordered index
	 * @param	Index2	2nd unordered index
	 */
	FRigidBodyIndexPair( int32 Index1, int32 Index2 )
	{
		Indices[0] = FMath::Min( Index1, Index2 );
		Indices[1] = FMath::Max( Index1, Index2 );
	}

	/**
	 * == operator required for use with TMap
	 *
	 * @param	Other	FRigidBodyIndexPair to compare this one to.
	 * @return	true if the passed in FRigidBodyIndexPair is identical to this one, false otherwise
	 */
	bool operator==( const FRigidBodyIndexPair &Other ) const
	{
		return (Indices[0] == Other.Indices[0]) && (Indices[1] == Other.Indices[1]);
	}

	/**
	 * Serializes the rigid body index pair to the passed in archive.	
	 *
	 * @param	Ar		Archive to serialize data to.
	 * @param	Pair	FRigidBodyIndexPair to serialize
	 * @return	returns the passed in FArchive after using it for serialization
	 */
	friend FArchive& operator<< ( FArchive &Ar, FRigidBodyIndexPair &Pair )
	{
		Ar << Pair.Indices[0] << Pair.Indices[1];
		return Ar;
	}
};

/**
 * Generates a hash value in accordance with the uint64 implementation of GetTypeHash which is
 * required for backward compatibility as older versions of UPhysicsAssetInstance stored
 * a TMap<uint64,bool>.
 * 
 * @param	Pair	FRigidBodyIndexPair to calculate the hash value for
 * @return	the hash value for the passed in FRigidBodyIndexPair
 */
inline uint32 GetTypeHash( const FRigidBodyIndexPair Pair )
{
	return Pair.Indices[0] + (Pair.Indices[1] * 23);
}
