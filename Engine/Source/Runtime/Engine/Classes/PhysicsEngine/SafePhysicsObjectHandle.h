// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/PhysicsObject.h"
#include "Interfaces/IPhysicsComponent.h"
#include "UObject/WeakInterfacePtr.h"

struct FSafePhysicsObjectHandle
{
	TWeakInterfacePtr<IPhysicsComponent> Component;
	Chaos::FPhysicsObjectId BoneId;

	Chaos::FPhysicsObjectHandle Get() const
	{
		if (IPhysicsComponent* ComponentPtr = Component.Get())
		{
			return ComponentPtr->GetPhysicsObjectById(BoneId);
		}

		return nullptr;
	}
};
