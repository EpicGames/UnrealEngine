// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "PhysxUserData.h"
#include "PhysicsInterfaceTypesCore.h"
#include "Engine/EngineTypes.h"
#include "ShapeElem.generated.h"

UENUM()
namespace EAggCollisionShape
{
	enum Type : int
	{
		Sphere,
		Box,
		Sphyl,
		Convex,
		TaperedCapsule,
		LevelSet,
		SkinnedLevelSet,
		MLLevelSet,
		SkinnedTriangleMesh,

		Unknown
	};
}

/** Base class of shapes used for collision, such as Sphere, Box, Sphyl, Convex, TaperedCapsule or LevelSet */
USTRUCT()
struct FKShapeElem
{
	GENERATED_USTRUCT_BODY()

	FKShapeElem()
	: RestOffset(0.f)
#if WITH_EDITORONLY_DATA
	, bIsGenerated(false)
#endif
	, ShapeType(EAggCollisionShape::Unknown)
	, bContributeToMass(true)
	, CollisionEnabled(ECollisionEnabled::QueryAndPhysics)
	, UserData(this)
	{}

	FKShapeElem(EAggCollisionShape::Type InShapeType)
	: RestOffset(0.f)
#if WITH_EDITORONLY_DATA
	, bIsGenerated(false)
#endif
	, ShapeType(InShapeType)
	, bContributeToMass(true)
	, CollisionEnabled(ECollisionEnabled::QueryAndPhysics)
	, UserData(this)
	{}

	FKShapeElem(const FKShapeElem& Copy)
	: RestOffset(Copy.RestOffset)
#if WITH_EDITORONLY_DATA
	, bIsGenerated(Copy.bIsGenerated)
#endif
	, Name(Copy.Name)
	, ShapeType(Copy.ShapeType)
	, bContributeToMass(Copy.bContributeToMass)
	, CollisionEnabled(Copy.CollisionEnabled)
	, UserData(this)
	{
	}

	ENGINE_API virtual ~FKShapeElem();

	const FKShapeElem& operator=(const FKShapeElem& Other)
	{
		CloneElem(Other);
		return *this;
	}

	template <typename T>
	T* GetShapeCheck()
	{
		check(T::StaticShapeType == ShapeType);
		return (T*)this;
	}

	const FUserData* GetUserData() const { FUserData::Set<FKShapeElem>((void*)&UserData, const_cast<FKShapeElem*>(this));  return &UserData; }

	ENGINE_API static EAggCollisionShape::Type StaticShapeType;

	/** Get the user-defined name for this shape */
	const FName& GetName() const { return Name; }

	/** Set the user-defined name for this shape */
	void SetName(const FName& InName) { Name = InName; }

	/** Get the type of this shape */
	EAggCollisionShape::Type GetShapeType() const { return ShapeType; }

	/** Get whether this shape contributes to the mass of the body */
	bool GetContributeToMass() const { return bContributeToMass; }

	/** Set whether this shape will contribute to the mass of the body */
	void SetContributeToMass(bool bInContributeToMass) { bContributeToMass = bInContributeToMass; }

	/** Set whether this shape should be considered for query or sim collision */
	void SetCollisionEnabled(ECollisionEnabled::Type InCollisionEnabled) { CollisionEnabled = InCollisionEnabled; }

	/** Get whether this shape should be considered for query or sim collision */
	ECollisionEnabled::Type GetCollisionEnabled() const { return CollisionEnabled; }

	virtual FTransform GetTransform() const 
	{
		return FTransform();
	}

	virtual void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const {}
	virtual void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const class FMaterialRenderProxy* MaterialRenderProxy) const {}

	/** Offset used when generating contact points. This allows you to smooth out
		the Minkowski sum by radius R. Useful for making objects slide smoothly
		on top of irregularities  */
	UPROPERTY(Category = Shape, EditAnywhere)
	float RestOffset;

#if WITH_EDITORONLY_DATA
	/** True when the shape was created by the engine and was not imported. */
	UPROPERTY(Category = Shape, EditAnywhere, DisplayName="Collision Generated")
	uint8 bIsGenerated : 1;
#endif

protected:
	/** Helper function to safely clone instances of this shape element */
	void CloneElem(const FKShapeElem& Other)
	{
		RestOffset = Other.RestOffset;
		ShapeType = Other.ShapeType;
		Name = Other.Name;
		bContributeToMass = Other.bContributeToMass;
		CollisionEnabled = Other.CollisionEnabled;
#if WITH_EDITORONLY_DATA
		bIsGenerated = Other.bIsGenerated;
#endif
	}

private:
	/** User-defined name for this shape */
	UPROPERTY(Category=Shape, EditAnywhere)
	FName Name;

	EAggCollisionShape::Type ShapeType;

	/** True if this shape should contribute to the overall mass of the body it
		belongs to. This lets you create extra collision volumes which do not affect
		the mass properties of an object. */
	UPROPERTY(Category=Shape, EditAnywhere)
	uint8 bContributeToMass : 1;

	/** Course per-primitive collision filtering. This allows for individual primitives to
		be toggled in and out of sim and query collision without changing filtering details. */
	UPROPERTY(Category=Shape, EditAnywhere)
	TEnumAsByte<ECollisionEnabled::Type> CollisionEnabled;

	FUserData UserData;
};
