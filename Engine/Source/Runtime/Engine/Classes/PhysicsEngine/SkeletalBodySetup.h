// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicalAnimationComponent.h"

#include "SkeletalBodySetup.generated.h"

USTRUCT()
struct FPhysicalAnimationProfile
{
	GENERATED_BODY()
	
	/** Profile name used to identify set of physical animation parameters */
	UPROPERTY()
	FName ProfileName;

	/** Physical animation parameters used to drive animation */
	UPROPERTY(EditAnywhere, Category = PhysicalAnimation)
	FPhysicalAnimationData PhysicalAnimationData;
};

UCLASS(MinimalAPI)
class USkeletalBodySetup : public UBodySetup
{
	GENERATED_BODY()
public:
	const FPhysicalAnimationProfile* FindPhysicalAnimationProfile(const FName ProfileName) const
	{
		return PhysicalAnimationData.FindByPredicate([ProfileName](const FPhysicalAnimationProfile& Profile){ return ProfileName == Profile.ProfileName; });
	}

	FPhysicalAnimationProfile* FindPhysicalAnimationProfile(const FName ProfileName)
	{
		return PhysicalAnimationData.FindByPredicate([ProfileName](const FPhysicalAnimationProfile& Profile) { return ProfileName == Profile.ProfileName; });
	}

	const TArray<FPhysicalAnimationProfile>& GetPhysicalAnimationProfiles() const
	{
		return PhysicalAnimationData;
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	ENGINE_API FName GetCurrentPhysicalAnimationProfileName() const;
	
	/** Creates a new physical animation profile entry */
	ENGINE_API void AddPhysicalAnimationProfile(FName ProfileName);

	/** Removes physical animation profile */
	ENGINE_API void RemovePhysicalAnimationProfile(FName ProfileName);

	ENGINE_API void UpdatePhysicalAnimationProfiles(const TArray<FName>& Profiles);

	ENGINE_API void DuplicatePhysicalAnimationProfile(FName DuplicateFromName, FName DuplicateToName);

	ENGINE_API void RenamePhysicalAnimationProfile(FName CurrentName, FName NewName);
#endif

#if WITH_EDITORONLY_DATA
	//dummy place for customization inside phat. Profiles are ordered dynamically and we need a static place for detail customization
	UPROPERTY(EditAnywhere, Category = PhysicalAnimation)
	FPhysicalAnimationProfile CurrentPhysicalAnimationProfile;
#endif

	/** If true we ignore scale changes from animation. This is useful for subtle scale animations like breathing where the physics collision should remain unchanged*/
	UPROPERTY(EditAnywhere, Category = BodySetup)
	bool bSkipScaleFromAnimation;

private:
	UPROPERTY()
	TArray<FPhysicalAnimationProfile> PhysicalAnimationData;
};
