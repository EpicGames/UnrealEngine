// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/Levelset.h"
#include "SkinnedLevelSetElem.generated.h"

USTRUCT()
struct FKSkinnedLevelSetElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

	FKSkinnedLevelSetElem() :
		FKShapeElem(EAggCollisionShape::SkinnedLevelSet)
	{}

	FKSkinnedLevelSetElem(const FKSkinnedLevelSetElem& Other)
	{
		CloneElem(Other);
	}

	const FKSkinnedLevelSetElem& operator=(const FKSkinnedLevelSetElem& Other)
	{
		CloneElem(Other);
		return *this;
	}

	ENGINE_API void SetWeightedLevelSet(TRefCountPtr< Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>&& InWeightedLevelSet);
	
	UE_DEPRECATED(5.4, "Please use SetWeightedLevelSet with TRefCountPtr instead")
	void SetWeightedLevelSet(TUniquePtr< Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>&& InWeightedLevelSet) {check(false);}

	virtual FTransform GetTransform() const override final
	{
		return FTransform();
	}

	// Draw functions
	ENGINE_API virtual void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const override;
	ENGINE_API virtual void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;
	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const;

	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const;
	ENGINE_API FIntVector3 LevelSetGridResolution() const;
	ENGINE_API FIntVector3 LatticeGridResolution() const;

	ENGINE_API bool Serialize(FArchive& Ar);

	const TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>& WeightedLevelSet() const
	{
		return WeightedLatticeLevelSet;
	}

	UE_DEPRECATED(5.4, "Please use WeightedLevelSet instead")
	const TSharedPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>& GetWeightedLevelSet() const
    {
		static TSharedPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>> DummyPtr;
    	return DummyPtr;
    }

private:

	TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>> WeightedLatticeLevelSet;

	/** Helper function to safely copy instances of this shape*/
	 ENGINE_API void CloneElem(const FKSkinnedLevelSetElem& Other);
};

/* Enable our own serialization function to handle FKSkinnedLevelSetElem */
template<>
struct TStructOpsTypeTraits<FKSkinnedLevelSetElem> : public TStructOpsTypeTraitsBase2<FKSkinnedLevelSetElem>
{
	enum
	{
		WithSerializer = true
	};
};
