// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "SkinnedTriangleMeshElem.generated.h"

USTRUCT()
struct FKSkinnedTriangleMeshElem : public FKShapeElem
{
	GENERATED_BODY()

	FKSkinnedTriangleMeshElem() :
		FKShapeElem(EAggCollisionShape::SkinnedTriangleMesh)
	{}

	FKSkinnedTriangleMeshElem(const FKSkinnedTriangleMeshElem& Other)
	{
		CloneElem(Other);
	}

	const FKSkinnedTriangleMeshElem& operator=(const FKSkinnedTriangleMeshElem& Other)
	{
		CloneElem(Other);
		return *this;
	}

	ENGINE_API void SetSkinnedTriangleMesh(TRefCountPtr<Chaos::FSkinnedTriangleMesh>&& InSkinnedTriangleMesh);
	ENGINE_API const TRefCountPtr<Chaos::FSkinnedTriangleMesh>& GetSkinnedTriangleMesh() const;

	virtual FTransform GetTransform() const override final
	{
		return FTransform();
	}

	// Draw functions
	ENGINE_API virtual void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const override;
	ENGINE_API virtual void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;
	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const;

	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, const FVector& Scale3D) const;

	ENGINE_API bool Serialize(FArchive& Ar);

private:

	TRefCountPtr<Chaos::FSkinnedTriangleMesh> SkinnedTriangleMesh;

	/** Helper function to safely copy instances of this shape*/
	 ENGINE_API void CloneElem(const FKSkinnedTriangleMeshElem& Other);
};

/* Enable our own serialization function to handle FKSkinnedTriangleMeshElem */
template<>
struct TStructOpsTypeTraits<FKSkinnedTriangleMeshElem> : public TStructOpsTypeTraitsBase2<FKSkinnedTriangleMeshElem>
{
	enum
	{
		WithSerializer = true
	};
};
