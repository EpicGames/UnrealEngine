// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "SphereElem.generated.h"

class FMaterialRenderProxy;

/** Sphere shape used for collision */
USTRUCT()
struct FKSphereElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FMatrix TM_DEPRECATED;
#endif

	/** Position of the sphere's origin */
	UPROPERTY(Category=Sphere, EditAnywhere)
	FVector Center;

	/** Radius of the sphere */
	UPROPERTY(Category=Sphere, EditAnywhere)
	float Radius;

	FKSphereElem() 
	: FKShapeElem(EAggCollisionShape::Sphere)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
#endif
	, Center( FVector::ZeroVector )
	, Radius(1)
	{

	}

	FKSphereElem( float r ) 
	: FKShapeElem(EAggCollisionShape::Sphere)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
#endif
	, Center( FVector::ZeroVector )
	, Radius(r)
	{

	}

	ENGINE_API virtual ~FKSphereElem();

#if WITH_EDITORONLY_DATA
	void FixupDeprecated( FArchive& Ar );
#endif

	friend bool operator==( const FKSphereElem& LHS, const FKSphereElem& RHS )
	{
		return ( LHS.Center == RHS.Center &&
			LHS.Radius == RHS.Radius );
	}

	// Utility function that builds an FTransform from the current data
	virtual FTransform GetTransform() const override final
	{
		return FTransform( Center );
	};

	void SetTransform(const FTransform& InTransform)
	{
		ensure(InTransform.IsValid());
		Center = InTransform.GetLocation();
	}

	UE_DEPRECATED(5.1, "Changed to GetScaledVolume to support non-uniform scales on other element types")
	FVector::FReal GetVolume(const FVector& Scale) const { return GetScaledVolume(Scale); }

	FVector::FReal GetScaledVolume(const FVector& Scale) const { return 1.3333f * UE_PI * FMath::Pow(Radius * Scale.GetAbsMin(), 3); }

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const override;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const;

	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const;
	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, float Scale) const;

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	ENGINE_API static EAggCollisionShape::Type StaticShapeType;

	ENGINE_API FKSphereElem GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const;

	/**	
	 * Finds the shortest distance between the element and a world position. Input and output are given in world space
	 * @param	WorldPosition	The point we are trying to get close to
	 * @param	BodyToWorldTM	The transform to convert BodySetup into world space
	 * @return					The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside the shape.
	 */
	ENGINE_API float GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BodyToWorldTM) const;

	/**	
	 * Finds the closest point on the shape given a world position. Input and output are given in world space
	 * @param	WorldPosition			The point we are trying to get close to
	 * @param	BodyToWorldTM			The transform to convert BodySetup into world space
	 * @param	ClosestWorldPosition	The closest point on the shape in world space
	 * @param	Normal					The normal of the feature associated with ClosestWorldPosition.
	 * @return							The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside the shape.
	 */
	ENGINE_API float GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BodyToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const;
};
