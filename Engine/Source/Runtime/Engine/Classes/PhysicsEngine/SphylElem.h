// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "SphylElem.generated.h"

class FMaterialRenderProxy;
class FMeshElementCollector;

/** Capsule shape used for collision. Z axis is capsule axis. */
USTRUCT()
struct FKSphylElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FMatrix TM_DEPRECATED;
	UPROPERTY()
	FQuat Orientation_DEPRECATED;
#endif

	/** Position of the capsule's origin */
	UPROPERTY(Category=Capsule, EditAnywhere)
	FVector Center;

	/** Rotation of the capsule */
	UPROPERTY(Category = Capsule, EditAnywhere, meta = (ClampMin = "-360", ClampMax = "360"))
	FRotator Rotation;

	/** Radius of the capsule */
	UPROPERTY(Category= Capsule, EditAnywhere)
	float Radius;

	/** This is of line-segment ie. add Radius to both ends to find total length. */
	UPROPERTY(Category= Capsule, EditAnywhere)
	float Length;

	FKSphylElem()
	: FKShapeElem(EAggCollisionShape::Sphyl)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
	, Orientation_DEPRECATED( FQuat::Identity )
#endif
	, Center( FVector::ZeroVector )
	, Rotation(FRotator::ZeroRotator)
	, Radius(1), Length(1)
	{

	}

	FKSphylElem( float InRadius, float InLength )
	: FKShapeElem(EAggCollisionShape::Sphyl)
#if WITH_EDITORONLY_DATA
	, TM_DEPRECATED(ForceInitToZero)
	, Orientation_DEPRECATED( FQuat::Identity )
#endif
	, Center( FVector::ZeroVector )
	, Rotation(FRotator::ZeroRotator)
	, Radius(InRadius), Length(InLength)
	{

	}

	ENGINE_API virtual ~FKSphylElem();

#if WITH_EDITORONLY_DATA
	void FixupDeprecated( FArchive& Ar );
#endif

	friend bool operator==( const FKSphylElem& LHS, const FKSphylElem& RHS )
	{
		return ( LHS.Center == RHS.Center &&
			LHS.Rotation == RHS.Rotation &&
			LHS.Radius == RHS.Radius &&
			LHS.Length == RHS.Length );
	};

	// Utility function that builds an FTransform from the current data
	virtual FTransform GetTransform() const override final
	{
		return FTransform(Rotation, Center );
	};

	void SetTransform( const FTransform& InTransform )
	{
		ensure(InTransform.IsValid());
		Rotation = InTransform.Rotator();
		Center = InTransform.GetLocation();
	}

	UE_DEPRECATED(5.1, "Changed to GetScaledVolume. Note that Volume calculation now includes non-uniform scale so values may have changed")
	FVector::FReal GetVolume(const FVector& Scale) const { return GetScaledVolume(Scale); }
	
	FVector::FReal GetScaledVolume(const FVector& Scale3D) const 
	{  
		FVector::FReal ScaledRadius = GetScaledRadius(Scale3D);
		FVector::FReal ScaledLength = GetScaledCylinderLength(Scale3D);
		return UE_PI * FMath::Square(ScaledRadius) * (1.3333f * ScaledRadius + ScaledLength);
	}

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const override;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const;

	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const;
	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, float Scale) const;

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	ENGINE_API FKSphylElem GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const;
	
	/** Returns the scaled radius for this Sphyl, which is determined by the Max scale on X/Y and clamped by half the total length */
	ENGINE_API float GetScaledRadius(const FVector& Scale3D) const;
	/** Returns the scaled length of the cylinder part of the Sphyl **/
	ENGINE_API float GetScaledCylinderLength(const FVector& Scale3D) const;
	/** Returns half of the total scaled length of the Sphyl, which includes the scaled top and bottom caps */
	ENGINE_API float GetScaledHalfLength(const FVector& Scale3D) const;

	/**	
	 * Finds the shortest distance between the element and a world position. Input and output are given in world space
	 * @param	WorldPosition	The point we are trying to get close to
	 * @param	BodyToWorldTM	The transform to convert BodySetup into world space
	 * @return					The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside one of the shapes.
	 */

	ENGINE_API float GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BodyToWorldTM) const;

	/**	
	 * Finds the closest point on the shape given a world position. Input and output are given in world space
	 * @param	WorldPosition			The point we are trying to get close to
	 * @param	BodyToWorldTM			The transform to convert BodySetup into world space
	 * @param	ClosestWorldPosition	The closest point on the shape in world space
	 * @param	Normal					The normal of the feature associated with ClosestWorldPosition.
	 * @return							The distance between WorldPosition and the shape. 0 indicates WorldPosition is inside the shape.
	 */
	ENGINE_API float GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BodyToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const;

	ENGINE_API static EAggCollisionShape::Type StaticShapeType;
};