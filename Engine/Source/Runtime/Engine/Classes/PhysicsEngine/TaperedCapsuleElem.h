// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "TaperedCapsuleElem.generated.h"

class FMaterialRenderProxy;
class FMeshElementCollector;

/** Capsule shape used for collision. Z axis is capsule axis. Has a start and end radius that can differ. */
USTRUCT()
struct FKTaperedCapsuleElem : public FKShapeElem
{
	GENERATED_USTRUCT_BODY()

	/** Position of the capsule's origin */
	UPROPERTY(Category=Capsule, EditAnywhere)
	FVector Center;

	/** Rotation of the capsule */
	UPROPERTY(Category = Capsule, EditAnywhere, meta = (ClampMin = "-360", ClampMax = "360"))
	FRotator Rotation;

	/** Radius of the capsule start point */
	UPROPERTY(Category= Capsule, EditAnywhere)
	float Radius0;

	/** Radius of the capsule end point */
	UPROPERTY(Category= Capsule, EditAnywhere)
	float Radius1;

	/** Length of line-segment. Add Radius0 and Radius 1 to find total length. */
	UPROPERTY(Category= Capsule, EditAnywhere)
	float Length;

	/** (Cloth-only) Line segment width for extruded tapered capsules. */
	UPROPERTY(Category = Capsule, EditAnywhere, meta = (ClampMin = 0.f))
	float Width = 0.f;

	/** (Cloth-only) Treat as one-sided collider, where all collisions are pushed to the +x side. Not supported when Width != 0*/
	UPROPERTY(Category = Capsule, EditAnywhere, meta = (EditCondition = "Width <= 0.00000001"))
	bool bOneSidedCollision = false;

	ENGINE_API FKTaperedCapsuleElem();
	ENGINE_API FKTaperedCapsuleElem( float InRadius0, float InRadius1, float InLength );
	ENGINE_API FKTaperedCapsuleElem(const FKTaperedCapsuleElem&);
	ENGINE_API ~FKTaperedCapsuleElem();

	friend bool operator==( const FKTaperedCapsuleElem& LHS, const FKTaperedCapsuleElem& RHS )
	{
		return ( LHS.Center == RHS.Center &&
			LHS.Rotation == RHS.Rotation &&
			LHS.Radius0 == RHS.Radius0 &&
			LHS.Radius1 == RHS.Radius1 &&
			LHS.Length == RHS.Length &&
			LHS.Width == RHS.Width &&
			LHS.bOneSidedCollision == RHS.bOneSidedCollision);
	};

	// Utility function that builds an FTransform from the current data
	virtual FTransform GetTransform() const override final
	{
		return FTransform(Rotation, Center );
	}

	bool IsExtruded() const
	{
		return Width > UE_SMALL_NUMBER;
	}

	bool IsOneSidedCollision() const
	{
		return bOneSidedCollision && !IsExtruded();
	}

	void SetTransform( const FTransform& InTransform )
	{
		ensure(InTransform.IsValid());
		Rotation = InTransform.Rotator();
		Center = InTransform.GetLocation();
	}

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const override;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const override;

	ENGINE_API void DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const;
	ENGINE_API void DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const;

	ENGINE_API void GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const;
	ENGINE_API FBox CalcAABB(const FTransform& BoneTM, float Scale) const;

	ENGINE_API void ScaleElem(FVector DeltaSize, float MinSize);

	ENGINE_API FKTaperedCapsuleElem GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const;
	
	/** Returns the scaled radius for this Sphyl, which is determined by the Max scale on X/Y and clamped by half the total length */
	ENGINE_API void GetScaledRadii(const FVector& Scale3D, float& OutRadius0, float& OutRadius1) const;
	/** Returns the scaled length of the cylinder part of the Sphyl **/
	ENGINE_API float GetScaledCylinderLength(const FVector& Scale3D) const;
	/** Returns half of the total scaled length of the Sphyl, which includes the scaled top and bottom caps */
	ENGINE_API float GetScaledHalfLength(const FVector& Scale3D) const;
	/** Returns the scaled length of the two cylinders for extruded Sphyls **/
	ENGINE_API void GetScaledExtrudedCylinderLengths(const FVector& Scale3D, float& OutLength0, float& OutLength1) const;
	/** Returns half of the total scaled length of the extruded Sphyl, which includes the scaled top and bottom caps */
	ENGINE_API void GetScaledExtrudedHalfLengths(const FVector& Scale3D, float& OutLength0, float& OutLength1) const;

	/** 
	 * Draws just the sides of a tapered capsule specified by provided Spheres that can have different radii.  Does not draw the spheres, just the sleeve.
	 * Extent geometry endpoints not necessarily coplanar with sphere origins (uses hull horizon)
	 * Otherwise uses the great-circle cap assumption.
	 */

	ENGINE_API static void DrawTaperedCapsuleSides(FPrimitiveDrawInterface* PDI, const FQuat& CapsuleOrientation, const FVector& InCenter0, const FVector& InCenter1, float InRadius0, float InRadius1, const FColor& Color, bool bInSplitTaperedCylinder = false);
	static void DrawTaperedCapsuleSides(FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& InCenter0, const FVector& InCenter1, float InRadius0, float InRadius1, const FColor& Color, bool bInSplitTaperedCylinder = false)
	{
		DrawTaperedCapsuleSides(PDI, ElemTM.GetRotation(), InCenter0, InCenter1, InRadius0, InRadius1, Color, bInSplitTaperedCylinder);
	}

	ENGINE_API static EAggCollisionShape::Type StaticShapeType;
};