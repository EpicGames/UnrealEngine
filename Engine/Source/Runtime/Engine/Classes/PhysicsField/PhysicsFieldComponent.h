// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Field/FieldSystem.h"
#include "UObject/ObjectMacros.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderResource.h"
#include "PrimitiveSceneProxy.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "PhysicsFieldComponent.generated.h"

enum class EFieldCommandBuffer : uint8
{
	GPUFieldBuffer = 0,
	CPUReadBuffer = 1,
	CPUWriteBuffer = 2,
	GPUDebugBuffer = 3,
	NumFieldBuffers = 4
};

struct FPhysicsFieldInfos
{
	/** Type of targets offsets */
	using BufferOffsets = TStaticArray<UE::Core::TAlignedElement<int32, 16>, MAX_PHYSICS_FIELD_TARGETS>;

	/** Size of the datas stored on each voxels*/
	int32 TargetCount = 1;

	/** Target types to be processed */
	TArray<EFieldPhysicsType> TargetTypes;

	/** Vector Targets Offsets*/
	BufferOffsets VectorTargets;

	/** Scalar Targets Offsets*/
	BufferOffsets ScalarTargets;

	/** Integer targets offsets */
	BufferOffsets IntegerTargets;

	/** Physics targets offsets */
	BufferOffsets PhysicsTargets;

	/** Valid targets offsets */
	BufferOffsets ValidTargets;

	/** Physics Targets bounds */
	TStaticArray<UE::Core::TAlignedElement<FIntVector4, 16>, MAX_PHYSICS_FIELD_TARGETS> PhysicsBounds;

	/** Clipmap  Center */
	FVector ClipmapCenter = FVector::ZeroVector;

	/** Clipmap Distance */
	float ClipmapDistance = 10000;

	/** Clipmap Count */
	int32 ValidCount = 0;

	/** Clipmap Count */
	int32 ClipmapCount = 4;

	/** Clipmap Exponent */
	int32 ClipmapExponent = 2;

	/** Clipmap Resolution */
	int32 ClipmapResolution = 64;

	/** Clipmap Resolution */
	FVector ViewOrigin = FVector::ZeroVector;

	/** Bounds Cells offsets */
	TArray<int32> CellsOffsets;

	/** Min Bounds for each target/clipmap */
	TArray<FIntVector4> CellsMin;

	/** Max Bounds for each target/clipmap */
	TArray<FIntVector4> CellsMax;

	/** Min Bounds for each target/clipmap */
	TStaticArray<UE::Core::TAlignedElement<int32, 16>, MAX_PHYSICS_FIELD_TARGETS> BoundsOffsets;

	/** Time in seconds for field evaluation */
	float TimeSeconds;

	/** Boolean to check if we are building the clipmap or not */
	bool bBuildClipmap;

	/** Boolean to check if we are visualizing the field */
	bool bShowFields;
};

/**
 * Physics Field render resource.
 */
class FPhysicsFieldResource : public FRenderResource
{
public:

	/** Field cached clipmap buffer */
	FRWBuffer ClipmapBuffer;

	/** Field nodes params buffer */
	FRWBuffer NodesParams;

	/** Field nodes offsets buffer */
	FRWBuffer NodesOffsets;

	/** Field targets nodes buffer */
	FRWBuffer TargetsOffsets;

	/** Cells offsets buffer */
	FRWBuffer CellsOffsets;

	/** Cells Min buffer */
	FRWBuffer CellsMin;

	/** Cells max buffer */
	FRWBuffer CellsMax;

	/** Bounds Min buffer */
	FRWBuffer BoundsMin;

	/** Bounds max buffer */
	FRWBuffer BoundsMax;

	/** Field infos that will be used to allocate memory and to transfer information */
	FPhysicsFieldInfos FieldInfos;

	/** Default constructor. */
	FPhysicsFieldResource(const int32 TargetCount, const TArray<EFieldPhysicsType>& TargetTypes,
		const FPhysicsFieldInfos::BufferOffsets& VectorTargets, const FPhysicsFieldInfos::BufferOffsets& ScalarTargets,
		const FPhysicsFieldInfos::BufferOffsets& IntegerTargets, const FPhysicsFieldInfos::BufferOffsets& PhysicsTargets,
		const TStaticArray<UE::Core::TAlignedElement<FIntVector4, 16>, MAX_PHYSICS_FIELD_TARGETS>& PhysicsBounds, const bool bBuildClipmap);

	/** Release Field resources. */
	virtual void ReleaseRHI() override;

	/** Init Field resources. */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Update RHI resources. */
	void UpdateResource(FRHICommandList& RHICmdList,
		const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& TargetsOffsetsDatas, const TArray<int32>& NodesOffsetsDatas, const TArray<float>& NodesParamsDatas,
		const TArray<FVector>& TargetsMinDatas, const TArray<FVector>& TargetsMaxDatas, const float TimeSeconds, 
		const TArray<FVector4>& BoundsMinDatas, const TArray<FVector4>& BoundsMaxDatas, const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& BoundsOffsetsDatas);

	/** Update Bounds. */
	void UpdateBounds(const TArray<FVector>& TargetsMin, const TArray<FVector>& TargetsMax, const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& TargetOffsets,
					  const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& BoundsOffsets);
};


/**
 * An instance of a Physics Field.
 */
class FPhysicsFieldInstance
{
public:

	/** Default constructor. */
	FPhysicsFieldInstance()
		: FieldResource(nullptr)
	{}

	/** Destructor. */
	~FPhysicsFieldInstance() = default;

	/**
	 * Initializes the instance for the given resource.
	 * @param TextureSize - The resource texture size to be used.
	 */
	void InitInstance(const TArray<EFieldPhysicsType>& TargetTypes, const bool bBuildClipmap, const bool bNeedsResource);

	/**
	 * Release the resource of the instance.
	 */
	void ReleaseInstance();

	/**
	 * Update the datas based on the new bounds and commands
	 * @param FieldCommands - Field commands to be sampled
	 */
	void UpdateInstance(const float TimeSeconds, const bool bIsDebugBuffer);

	/** The field system resource. */
	FPhysicsFieldResource* FieldResource = nullptr;

	/** Targets offsets in the nodes array*/
	TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1> TargetsOffsets;
	
	/** Bounds offsets in the bounds array*/
	TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1> BoundsOffsets;

	/** Nodes offsets in the paramter array */
	TArray<int32> NodesOffsets;

	/** Nodes input parameters and connection */
	TArray<float> NodesParams;

	/** Commands bounds min sorted per target type */
	TArray<FVector4> BoundsMin;

	/** Commands bounds max sorted per target type */
	TArray<FVector4> BoundsMax;

	/** List of all the field commands in the world */
	TArray<FFieldSystemCommand> FieldCommands;

	/** Min Bounds for each target/clipmap */
	TArray<FVector> TargetsMin;

	/** Max Bounds for each target/clipmap */
	TArray<FVector> TargetsMax;
};

/**
*	PhysicsFieldComponent
*/

UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UPhysicsFieldComponent : public USceneComponent
{
	GENERATED_BODY()

public:

	ENGINE_API UPhysicsFieldComponent();

	//~ Begin UActorComponent Interface.
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void OnUnregister() override;
	ENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	ENGINE_API virtual void SendRenderDynamicData_Concurrent() override;
	ENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	ENGINE_API virtual void DestroyRenderState_Concurrent() override;
	//~ End UActorComponent Interface.

	/** Add the transient field command */
	ENGINE_API void AddTransientCommand(const FFieldSystemCommand& FieldCommand, const bool bIsGPUField);

	/** Add the persistent field command */
	ENGINE_API void AddPersistentCommand(const FFieldSystemCommand& FieldCommand, const bool bIsGPUField);

	/** Add the construction field command */
	ENGINE_API void AddConstructionCommand(const FFieldSystemCommand& FieldCommand);

	/** Remove the transient field command */
	ENGINE_API void RemoveTransientCommand(const FFieldSystemCommand& FieldCommand, const bool bIsGPUField);

	/** Remove the persistent field command */
	ENGINE_API void RemovePersistentCommand(const FFieldSystemCommand& FieldCommand, const bool bIsGPUField);

	/** Fill the transient commands intersecting the bounding box from the physics field */
	ENGINE_API void FillTransientCommands(const bool bIsWorldField, const FBox& BoundingBox, const float TimeSeconds, TArray<FFieldSystemCommand>& OutputCommands) const;

	/** Fill the persistent commands intersecting the bounding box from the physics field */
	ENGINE_API void FillPersistentCommands(const bool bIsWorldField, const FBox& BoundingBox, const float TimeSeconds, TArray<FFieldSystemCommand>& OutputCommands) const;

	/** Build the command bounds */
	static ENGINE_API void BuildCommandBounds(FFieldSystemCommand& FieldCommand);

	// These types are not static since we probably want in the future to be able to pick the vector/scalar/integer fields we are interested in

	/** List of all the field transient commands in the world */
	TArray<FFieldSystemCommand> TransientCommands[(uint8)(EFieldCommandBuffer::NumFieldBuffers)];

	/** List of all the field persistent commands in the world */
	TArray<FFieldSystemCommand> PersistentCommands[(uint8)(EFieldCommandBuffer::NumFieldBuffers)];

	/** List of all the field construction commands in the world */
	TArray<FFieldSystemCommand> ConstructionCommands[(uint8)(EFieldCommandBuffer::NumFieldBuffers)];

	/** The instance of the GPU field system. */
	FPhysicsFieldInstance* FieldInstance = nullptr;

	/** The instance of the CPU field system. */
	FPhysicsFieldInstance* DebugInstance = nullptr;

	/** Scene proxy to be sent to the render thread. */
	class FPhysicsFieldSceneProxy* FieldProxy = nullptr;
};

/** Compute the field indexand output given a field type */
void ENGINE_API GetFieldIndex(const uint32 FieldType, int32& FieldIndex, EFieldOutputType& FieldOutput);

//class FPhysicsFieldSceneProxy final : public FPrimitiveSceneProxy
class FPhysicsFieldSceneProxy 
{
public:
	//SIZE_T GetTypeHash() const override;

	/** Initialization constructor. */
	explicit FPhysicsFieldSceneProxy(class UPhysicsFieldComponent* PhysicsFieldComponent);

	/** Destructor. */
	~FPhysicsFieldSceneProxy();

	/** The GPU physics field resource which this proxy is visualizing. */
	FPhysicsFieldResource* FieldResource = nullptr;

	/** The CPU physics field resource which this proxy is visualizing. */
	FPhysicsFieldResource* DebugResource = nullptr;
};

/** Static function with world field evaluation */
UCLASS(MinimalAPI)
class UPhysicsFieldStatics : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
public:
	
	/** Evaluate the world physics vector field from BP */
	UFUNCTION(BlueprintCallable, Category="Field", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API FVector EvalPhysicsVectorField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldVectorType VectorType);

	/** Evaluate the world physics scalar field from BP */
	UFUNCTION(BlueprintCallable, Category="Field", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API float EvalPhysicsScalarField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldScalarType ScalarType);

	/** Evaluate the world physics integer field from BP */
	UFUNCTION(BlueprintCallable, Category="Field", meta=(WorldContext="WorldContextObject"))
	static ENGINE_API int32 EvalPhysicsIntegerField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldIntegerType IntegerType);
};

void ENGINE_API EvaluateFieldVectorNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<FVector>& ResultsArray, TArray<FVector>& MaxArray);

void ENGINE_API EvaluateFieldScalarNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<float>& ResultsArray, TArray<float>& MaxArray);

void ENGINE_API EvaluateFieldIntegerNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<int32>& ResultsArray, TArray<int32>& MaxArray);




