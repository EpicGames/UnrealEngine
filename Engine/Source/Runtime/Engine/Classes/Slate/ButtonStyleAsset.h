// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Styling/SlateTypes.h"
#include "ButtonStyleAsset.generated.h"

/**
 * An asset describing a button's appearance.
 * Just a wrapper for the struct with real data in it.style factory
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UButtonStyleAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** The actual data describing the button's appearance. */
	UPROPERTY(Category=Appearance, EditAnywhere, meta=(ShowOnlyInnerProperties))
	FButtonStyle ButtonStyle;
};
