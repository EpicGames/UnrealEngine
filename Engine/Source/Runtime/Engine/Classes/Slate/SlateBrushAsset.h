// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Styling/SlateBrush.h"
#include "SlateBrushAsset.generated.h"

/**
 * An asset describing how a texture can exist in slate's DPI-aware environment
 * and how this texture responds to resizing. e.g. Scale9-stretching? Tiling?
 */
UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class USlateBrushAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** The slate brush resource describing the texture's behavior. */
	UPROPERTY(Category=Brush, EditAnywhere, meta=(ShowOnlyInnerProperties))
	FSlateBrush Brush;

#if WITH_EDITORONLY_DATA
	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
#endif
};
