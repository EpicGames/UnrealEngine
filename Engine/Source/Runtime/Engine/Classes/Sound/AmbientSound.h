// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "AmbientSound.generated.h"

/** A sound actor that can be placed in a level */
UCLASS(AutoExpandCategories=Audio, ClassGroup=Sounds, hideCategories(Collision, Input, Game), showCategories=("Input|MouseInput", "Input|TouchInput", "Game|Damage"), ComponentWrapperClass, MinimalAPI)
class AAmbientSound : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** Audio component that handles sound playing */
	UPROPERTY(Category = Sound, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Sound,Audio,Audio|Components|Audio", AllowPrivateAccess = "true"))
	TObjectPtr<class UAudioComponent> AudioComponent;
public:
	
	ENGINE_API FString GetInternalSoundCueName();

	//~ Begin AActor Interface.
#if WITH_EDITOR
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const override;
#endif
	ENGINE_API virtual void PostRegisterAllComponents() override;
	//~ End AActor Interface.

	// BEGIN DEPRECATED (use component functions now in level script)
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(DeprecatedFunction))
	ENGINE_API void FadeIn(float FadeInDuration, float FadeVolumeLevel = 1.f);
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(DeprecatedFunction))
	ENGINE_API void FadeOut(float FadeOutDuration, float FadeVolumeLevel);
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(DeprecatedFunction))
	ENGINE_API void AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel);
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(DeprecatedFunction))
	ENGINE_API void Play(float StartTime = 0.f);
	UFUNCTION(BlueprintCallable, Category="Audio", meta=(DeprecatedFunction))
	ENGINE_API void Stop();
	// END DEPRECATED

public:
	/** Returns AudioComponent subobject **/
	class UAudioComponent* GetAudioComponent() const { return AudioComponent; }
};



