// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IAudioProxyInitializer.h"
#include "AudioBus.generated.h"

class FAudioDevice;

// The number of channels to mix audio into the source bus
UENUM(BlueprintType)
enum class EAudioBusChannels : uint8
{
	Mono = 0,
	Stereo = 1,
	Quad = 3,
	FivePointOne = 5 UMETA(DisplayName = "5.1"),
	SevenPointOne = 7 UMETA(DisplayName = "7.1"),
	MaxChannelCount = 8
};

namespace AudioBusUtils
{
	static EAudioBusChannels ConvertIntToEAudioBusChannels(const int32 InValue)
	{
		switch (InValue)
		{
		case 1: return EAudioBusChannels::Mono;
		case 2:	return EAudioBusChannels::Stereo;
		case 4:	return EAudioBusChannels::Quad;
		case 6:	return EAudioBusChannels::FivePointOne;
		case 8:	return EAudioBusChannels::SevenPointOne;
		default:
		{
			// Channel count doesn't map to a standard bus configuration (e.g. CAT formats like 4.1).
			// Clamp to the nearest supported configuration rather than crashing. (UE-370803)
			// Log once to avoid spam if unexpectedly called from a hot path.
			static bool bLoggedOnce = false;
			if (!bLoggedOnce)
			{
				bLoggedOnce = true;
				UE_LOGF(LogCore, Warning, "AudioBus channel count %d is not a supported configuration (1, 2, 4, 6, 8). Clamping to nearest supported format for visualization.", InValue);
			}
			if (InValue <= 1)      return EAudioBusChannels::Mono;
			else if (InValue <= 2) return EAudioBusChannels::Stereo;
			else if (InValue <= 4) return EAudioBusChannels::Quad;
			else if (InValue <= 6) return EAudioBusChannels::FivePointOne;
			else                   return EAudioBusChannels::SevenPointOne;
		}
		}
	};
}

class FAudioBusProxy;

using FAudioBusProxyPtr = TSharedPtr<FAudioBusProxy, ESPMode::ThreadSafe>;
class FAudioBusProxy final : public Audio::TProxyData<FAudioBusProxy>
{
public:
	IMPL_AUDIOPROXY_CLASS(FAudioBusProxy);

	ENGINE_API explicit FAudioBusProxy(UAudioBus* InAudioBus);

	FAudioBusProxy(const FAudioBusProxy& Other) = default;

	virtual ~FAudioBusProxy() override = default;

	uint32 AudioBusId = uint32(INDEX_NONE);
	int32 NumChannels = INDEX_NONE;

	friend inline uint32 GetTypeHash(const FAudioBusProxy& InProxy)
	{
		return InProxy.TypeHash;
	}

private: 
	uint32 TypeHash = uint32(INDEX_NONE);
};

// Function to retrieve an audio bus buffer given a handle
// static float* GetAudioBusBuffer(const FAudioBusHandle& AudioBusHandle);

// An audio bus is an object which represents an audio patch cord. Audio can be sent to it. It can be sonified using USoundSourceBuses.
// Instances of the audio bus are created in the audio engine. 
UCLASS(ClassGroup = Sound, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UAudioBus : public UObject, public IAudioProxyDataFactory
{
	GENERATED_UCLASS_BODY()

public:

	/** Number of channels to use for the Audio Bus. */
	UPROPERTY(EditAnywhere, Category = BusProperties)
	EAudioBusChannels AudioBusChannels = EAudioBusChannels::Mono;

	//~ Begin UObject
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject

	// Returns the number of channels of the audio bus in integer format
	int32 GetNumChannels() const { return (int32)AudioBusChannels + 1; }

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin IAudioProxy Interface
	ENGINE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	//~ End IAudioProxy Interface
};
