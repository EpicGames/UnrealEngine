// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Interfaces/ITargetPlatformSettings.h"
#include "UObject/NameTypes.h"

class FString;
class FConfigCacheIni;
class USoundWave;
class IAudioFormat;

namespace Audio
{
	class FAudioFormatSettings
	{
	public:
		ENGINE_API FAudioFormatSettings(FConfigCacheIni* InConfigSystem, const FString& InConfigFilename, const FString& IniPlatformName);

		~FAudioFormatSettings() = default;

		ENGINE_API FName GetWaveFormat(const USoundWave* Wave) const;
		ENGINE_API void GetAllWaveFormats(TArray<FName>& OutFormats) const;
		ENGINE_API void GetWaveFormatModuleHints(TArray<FName>& OutHints) const;

		FName GetFallbackFormat() const { return FallbackFormat; }

	private:
		ENGINE_API void ReadConfiguration(FConfigCacheIni*, const FString& InConfigFilename);

		struct FPlatformWaveState;
		friend struct FPlatformWaveState;
				
		bool IsFormatAllowed(const FPlatformWaveState& InWave) const;
		const IAudioFormat* FindFormat(const FName& InFormatName) const;

		FName IniPlatformName;
		TArray<FName> AllWaveFormats;
		TArray<FName> WaveFormatModuleHints;
		FName PlatformFormat;
		FName PlatformStreamingFormat;
		FName FallbackFormat;
		mutable FCriticalSection AudioFormatCacheCs;
		mutable TMap<FName, IAudioFormat*> AudioFormatCache;
	};

	
}
