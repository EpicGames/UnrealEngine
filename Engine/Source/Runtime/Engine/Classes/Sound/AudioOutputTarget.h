// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"

#include "AudioOutputTarget.generated.h"

UENUM()
namespace EAudioOutputTarget
{
	enum Type : int
	{
		Speaker UMETA(ToolTip = "Sound plays only from speakers."),
		Controller UMETA(ToolTip = "Sound plays only from controller if present."),
		ControllerFallbackToSpeaker UMETA(ToolTip = "Sound plays on the controller if present. If not present, it plays from speakers.")
	};
}
