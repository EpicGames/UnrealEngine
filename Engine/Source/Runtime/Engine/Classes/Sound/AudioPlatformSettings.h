// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/PlatformSettings.h"
#include "AudioPlatformSettings.generated.h"

USTRUCT()
struct FDecodeCacheSettings
{
	GENERATED_USTRUCT_BODY()

	// Whether or not the decode cache is enabled by default
	UPROPERTY(config, EditAnywhere, Category = "Decode Cache", meta = (ClampMin = 5))
	bool EnableDecodeCache = false;

	// The desired maximum decode cache size in megabytes
	UPROPERTY(config, EditAnywhere, Category = "Decode Cache", meta = (ClampMin = 5))
	float MaxDecodeCacheSizeMB = 10.0f;

	// The desired maximum decode cache frames per chunk
	UPROPERTY(config, EditAnywhere, Category = "Decode Cache", meta = (ClampMin = 1024))
	int32 FramesPerChunk = 4096;

	// Threshold to opt a sound to use the decode cache based on it's duration (in seconds). If less than threshold, it will use the decode cache.
	UPROPERTY(config, EditAnywhere, Category = "Decode Cache", meta = (ClampMin = 0))
	float AutoDurationThreshold = 2.0f;
};

/*
* Class to store platform-specific settings.
*/
UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UAudioPlatformSettings : public UPlatformSettings
{
	GENERATED_BODY()

public:
	// Whether or not the decode cache is enabled by default
	UPROPERTY(config, EditAnywhere, Category = "Decode Cache", meta = (ShowOnlyInnerProperties))
	FDecodeCacheSettings DecodeCacheSettings;
};