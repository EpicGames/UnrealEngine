// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/DeveloperSettings.h"
#include "AudioMixerTypes.h"
#include "AudioPlatformSettings.h"
#include "AudioSettings.generated.h"

struct FPropertyChangedChainEvent;
struct FDecodeCacheSettings;

class USoundClass;
class USoundConcurrency;

// Enumeration for what our options are for sample rates used for VOIP.
UENUM()
enum class EVoiceSampleRate : int32
{
	Low16000Hz = 16000,
	Normal24000Hz = 24000,
	/* High48000Hz = 48000 //TODO: 48k VOIP requires serious performance optimizations on encoding and decoding. */
};

// Enumeration defines what method of panning to use (for non-binaural audio) with the audio-mixer.
UENUM()
enum class EPanningMethod : int8
{
	// Linear panning maintains linear amplitude when panning between speakers.
	Linear,

	// Equal power panning maintains equal power when panning between speakers.
	EqualPower
};

// Enumeration defines how to treat mono 2D playback. Mono sounds need to upmixed to stereo when played back in 2D.
UENUM()
enum class EMonoChannelUpmixMethod : int8
{
	// The mono channel is split 0.5 left/right
	Linear,

	// The mono channel is split 0.707 left/right
	EqualPower,

	// The mono channel is split 1.0 left/right
	FullVolume
};


// The sound asset compression type to use for assets using the compression type "project defined".
UENUM()
enum class EDefaultAudioCompressionType : uint8
{
	// Perceptual-based codec which supports all features across all platforms.
	BinkAudio,

	// Will encode the asset using ADPCM, a time-domain codec that has fixed-sized quality and about ~4x compression ratio, but is relatively cheap to decode.
	ADPCM,

	// Uncompressed audio. Large memory usage (streamed chunks contain less audio per chunk) but extremely cheap to decode and supports all features. 
	PCM,

	// Opus is a highly versatile audio codec. It is primarily designed for interactive speech and music transmission over the Internet, but is also applicable to storage and streaming applications.
	Opus,

	// Encodes the asset to a platform specific format and will be different depending on the platform. It does not currently support seeking.
	PlatformSpecific,

	// As BinkAudio, except better quality. Comparable CPU usage on newer platforms, higher CPU on older platforms.
	RADAudio UMETA(DisplayName = "RAD Audio")
};


USTRUCT()
struct FAudioQualitySettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Quality")
	FText DisplayName;

	// The number of audio channels that can be used at once
	// NOTE: Some platforms may cap this value to a lower setting regardless of what the settings request
	UPROPERTY(EditAnywhere, Category="Quality", meta=(ClampMin="1"))
	int32 MaxChannels;

	FAudioQualitySettings()
		: MaxChannels(32)
	{
	}
};

USTRUCT()
struct FSoundDebugEntry
{
	GENERATED_USTRUCT_BODY()

	/** Short name to use when referencing sound (ex. in the command line) */
	UPROPERTY(config, EditAnywhere, Category="Debug", meta=(DisplayName="Name"))
	FName DebugName;

	/** Reference to a Debug Sound */
	UPROPERTY(config, EditAnywhere, Category="Debug", meta=(AllowedClasses="/Script/Engine.SoundBase"))
	FSoftObjectPath Sound;
};

USTRUCT()
struct FDefaultAudioBusSettings
{
	GENERATED_BODY()

	/** The audio bus to start up by default on init. */
	UPROPERTY(EditAnywhere, Category = "Mix", meta = (AllowedClasses = "/Script/Engine.AudioBus"))
	FSoftObjectPath AudioBus;
};

/**
 * Audio settings.
 */
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Audio"), MinimalAPI)
class UAudioSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Event to listen for when settings reflected properties are changed. */
	DECLARE_EVENT(UAudioSettings, FAudioSettingsChanged)
#endif // WITH_EDITOR

	/** The SoundClass assigned to newly created sounds */
	UPROPERTY(config, EditAnywhere, Category="Audio", meta=(AllowedClasses="/Script/Engine.SoundClass", DisplayName="Default Sound Class"))
	FSoftObjectPath DefaultSoundClassName;

	/** The SoundClass assigned to media player assets */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (AllowedClasses = "/Script/Engine.SoundClass", DisplayName = "Default Media Sound Class"))
	FSoftObjectPath DefaultMediaSoundClassName;

	/** The SoundConcurrency assigned to newly created sounds */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (AllowedClasses = "/Script/Engine.SoundConcurrency", DisplayName = "Default Sound Concurrency"))
	FSoftObjectPath DefaultSoundConcurrencyName;

	/** The SoundMix to use as base when no other system has specified a Base SoundMix */
	UPROPERTY(config, EditAnywhere, Category="Audio", meta=(AllowedClasses="/Script/Engine.SoundMix"))
	FSoftObjectPath DefaultBaseSoundMix;

	/** Sound class to be used for the VOIP audio component */
	UPROPERTY(config, EditAnywhere, Category="Audio", meta=(AllowedClasses="/Script/Engine.SoundClass", DisplayName = "VOIP Sound Class"))
	FSoftObjectPath VoiPSoundClass;

	/** The default submix through which all sounds are routed to. The root submix that outputs to audio hardware. */
	UPROPERTY(config, EditAnywhere, Category="Mix", meta=(AllowedClasses="/Script/Engine.SoundSubmix"))
	FSoftObjectPath MasterSubmix;

	/** The default submix to use for implicit submix sends (i.e. if the base submix send is null or if a submix parent is null) */
	UPROPERTY(config, EditAnywhere, Category = "Mix", meta = (AllowedClasses = "/Script/Engine.SoundSubmix"), AdvancedDisplay)
	FSoftObjectPath BaseDefaultSubmix;

	/** The submix through which all sounds set to use reverb are routed */
	UPROPERTY(config, EditAnywhere, Category="Mix", meta=(AllowedClasses="/Script/Engine.SoundSubmix"))
	FSoftObjectPath ReverbSubmix;

	/** The submix through which all sounds set to use legacy EQ system are routed */
	UPROPERTY(config, EditAnywhere, Category="Mix", meta=(AllowedClasses="/Script/Engine.SoundSubmix", DisplayName = "EQ Submix (Legacy)"), AdvancedDisplay)
	FSoftObjectPath EQSubmix;

	/** Sample rate used for voice over IP. VOIP audio is resampled to the application's sample rate on the receiver side. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (DisplayName = "VOIP Sample Rate"))
	EVoiceSampleRate VoiPSampleRate;

	/** Default audio compression type to use for audio assets. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	EDefaultAudioCompressionType DefaultAudioCompressionType;

	/**
	 * The default interval at which to re-test occlusion (in seconds).  Values < 0.f will only check occlusion once.
	 *   NOTE! Will be overridden by sounds coming from an audio component!
	 */
	UPROPERTY(config, EditAnywhere, Category = Occlusion)
	FPerPlatformFloat DefaultOcclusionCheckInterval = .1f;

	/** The default compression quality (e.g. for new SoundWaves) */
	UPROPERTY(config)
	int32 DefaultCompressionQuality_DEPRECATED = 80;

	/** The amount of audio to send to reverb submixes if no reverb send is setup for the source through attenuation settings. Only used in audio mixer. */
	UPROPERTY(config)
	float DefaultReverbSendLevel_DEPRECATED;

	/** How many streaming sounds can be played at the same time (if more are played they will be sorted by priority) */
	UPROPERTY(config, EditAnywhere, Category="Audio", meta=(ClampMin=0))
	int32 MaximumConcurrentStreams;

	/** The value to use to clamp the min pitch scale */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = 0.001, UIMin = 0.001, UIMax = 4.0))
	float GlobalMinPitchScale;

	/** The value to use to clamp the max pitch scale */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = 0.001, UIMin = 0.001, UIMax = 4.0))
	float GlobalMaxPitchScale;

	UPROPERTY(config, EditAnywhere, Category="Quality")
	TArray<FAudioQualitySettings> QualityLevels;

	/** Allows sounds to play at 0 volume. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", AdvancedDisplay)
	uint32 bAllowPlayWhenSilent:1;

	/** Disables master EQ effect in the audio DSP graph. */
	UPROPERTY(config, EditAnywhere, Category = "Mix", AdvancedDisplay)
	uint32 bDisableMasterEQ : 1;

	/** Enables the surround sound spatialization calculations to include the center channel. */
	UPROPERTY(config, EditAnywhere, Category = "Panning", AdvancedDisplay)
	uint32 bAllowCenterChannel3DPanning : 1;

	/**
	 * The max number of sources to reserve for "stopping" sounds. A "stopping" sound applies a fast fade in the DSP
	 * render to prevent discontinuities when stopping sources.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Audio", AdvancedDisplay)
	uint32 NumStoppingSources;

	/**
	* The method to use when doing non-binaural or object-based panning.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Panning", AdvancedDisplay)
	EPanningMethod PanningMethod;

	/**
	* The upmixing method for mono sound sources. Defines how mono channels are up-mixed to stereo channels.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Mix", AdvancedDisplay, meta = (ConfigRestartRequired = true))
	EMonoChannelUpmixMethod MonoChannelUpmixMethod;

	/**
	 * The format string to use when generating the filename for contexts within dialogue waves. This must generate unique names for your project.
	 * Available format markers:
	 *   * {DialogueGuid} - The GUID of the dialogue wave. Guaranteed to be unique and stable against asset renames.
	 *   * {DialogueHash} - The hash of the dialogue wave. Not guaranteed to be unique or stable against asset renames, however may be unique enough if combined with the dialogue name.
	 *   * {DialogueName} - The name of the dialogue wave. Not guaranteed to be unique or stable against asset renames, however may be unique enough if combined with the dialogue hash.
	 *   * {ContextId}    - The ID of the context. Guaranteed to be unique within its dialogue wave. Not guaranteed to be stable against changes to the context.
	 *   * {ContextIndex} - The index of the context within its parent dialogue wave. Guaranteed to be unique within its dialogue wave. Not guaranteed to be stable against contexts being removed.
	 */
	UPROPERTY(config, EditAnywhere, Category="Dialogue")
	FString DialogueFilenameFormat;

	/**
	* Sounds only packaged in non-shipped builds for debugging.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Debug")
	TArray<FSoundDebugEntry> DebugSounds;

	/** Array of AudioBuses that are automatically initialized when the AudioEngine is initialized */
	UPROPERTY(config, EditAnywhere, Category="Mix")
	TArray<FDefaultAudioBusSettings> DefaultAudioBuses;

#if WITH_EDITORONLY_DATA
	/** Whether or not project supports the creation of legacy audio asset types. Currently includes:
	  * - USoundSimple
	  */
	UPROPERTY(config, EditAnywhere, Category="Audio", AdvancedDisplay)
	bool bEnableLegacyAssetTypes = false;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	FAudioSettingsChanged AudioSettingsChanged;
#endif // WITH_EDITOR

private:
	UPROPERTY(Transient)
	TObjectPtr<USoundClass> DefaultSoundClass;

	UPROPERTY(Transient)
	TObjectPtr<USoundClass> DefaultMediaSoundClass;

	UPROPERTY(Transient)
	TObjectPtr<USoundConcurrency> DefaultSoundConcurrency;


	// Per-platform audio options container
	UPROPERTY(config, EditAnywhere, Category="Platform Settings", meta = (ShowOnlyInnerProperties))
	FPerPlatformSettings PlatformSettings;

public:
	// Loads default object instances from soft object path properties
	ENGINE_API void LoadDefaultObjects();

	ENGINE_API USoundClass* GetDefaultSoundClass() const;
	ENGINE_API USoundClass* GetDefaultMediaSoundClass() const;
	ENGINE_API USoundConcurrency* GetDefaultSoundConcurrency() const;

	// Registers Parameter Interfaces defined by the engine module.
	// Called on engine start-up. Can be called when engine is not
	// initialized as well by consuming plugins (ex. on cook by plugins
	// requiring interface system to be loaded).
	ENGINE_API void RegisterParameterInterfaces();

	// Get the quality level settings at the provided level index
	ENGINE_API const FAudioQualitySettings& GetQualityLevelSettings(int32 QualityLevel) const;

	UE_DEPRECATED(5.5, "Default Compression Quality has moved to SoundWave class defaults")
	ENGINE_API int32 GetDefaultCompressionQuality() const;
	
	// Get the quality name level for a given index
	ENGINE_API FString FindQualityNameByIndex(int32 Index) const;

	// Get the total number of quality level settings
	ENGINE_API int32 GetQualityLevelSettingsNum() const;

	/** Returns the highest value for MaxChannels among all quality levels */
	ENGINE_API int32 GetHighestMaxChannels() const;

	// Convenience accessors for per-platform settings
	ENGINE_API FDecodeCacheSettings GetDecodeCacheSettings() const;

#if WITH_EDITOR
	/** Returns event to be bound to if caller wants to know when audio settings are modified */
	FAudioSettingsChanged& OnAudioSettingsChanged() { return AudioSettingsChanged; }
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	TArray<FAudioQualitySettings> CachedQualityLevels;
	FSoftObjectPath CachedAmbisonicSubmix;
	FSoftObjectPath CachedMasterSubmix;
	FSoftObjectPath CachedSoundClass;
#endif // WITH_EDITOR

	ENGINE_API void AddDefaultSettings();

	bool bParameterInterfacesRegistered;
};
