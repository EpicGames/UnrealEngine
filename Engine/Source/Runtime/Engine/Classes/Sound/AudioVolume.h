// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Used to affect audio settings in the game and editor.
 */

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/Volume.h"
#include "ReverbSettings.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "Sound/SoundSubmixSend.h"

#include "AudioVolume.generated.h"

class AAudioVolume;
struct FBodyInstance;
class USoundSubmix;

// Enum describing the state of checking audio volume location
UENUM(BlueprintType)
enum class EAudioVolumeLocationState : uint8
{
	// Used for when the listener is located inside the Audio Volume
	InsideTheVolume,

	// Used for when the listener is located outside the Audio Volume
	OutsideTheVolume,
};

/** Struct to determine dynamic submix send data for use with audio volumes. */
USTRUCT(BlueprintType)
struct FAudioVolumeSubmixSendSettings
{
	GENERATED_USTRUCT_BODY()

	// The state the listener needs to be in, relative to the audio volume, for this submix send list to be used for a given sound
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioVolumeSubmixSends)
	EAudioVolumeLocationState ListenerLocationState = EAudioVolumeLocationState::InsideTheVolume;

	UPROPERTY()
	EAudioVolumeLocationState SourceLocationState_DEPRECATED = EAudioVolumeLocationState::InsideTheVolume;

	// Submix send array for sounds that are in the ListenerLocationState at the same time as the listener
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioVolumeSubmixSends)
	TArray<FSoundSubmixSendInfo> SubmixSends;
};

USTRUCT(BlueprintType)
struct FAudioVolumeSubmixOverrideSettings
{
	GENERATED_USTRUCT_BODY()

	// The submix to override the effect chain of
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioVolumeSubmixSends)
	TObjectPtr<USoundSubmix> Submix = nullptr;

	// The submix effect chain to override
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundSubmix)
	TArray<TObjectPtr<USoundEffectSubmixPreset>> SubmixEffectChain;

	// The amount of time to crossfade to the override for the submix chain
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundSubmix)
	float CrossfadeTime = 0.0f;
};


/** Struct encapsulating settings for interior areas. */
USTRUCT(BlueprintType)
struct FInteriorSettings
{
	GENERATED_USTRUCT_BODY()

	// Whether these interior settings are the default values for the world
	UPROPERTY()
	bool bIsWorldSettings;

	// The desired volume of sounds outside the volume when the player is inside the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float ExteriorVolume;

	// The time over which to interpolate from the current volume to the desired volume of sounds outside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float ExteriorTime;

	// The desired LPF frequency cutoff in hertz of sounds outside the volume when the player is inside the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InteriorSettings)
	float ExteriorLPF;

	// The time over which to interpolate from the current LPF to the desired LPF of sounds outside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float ExteriorLPFTime;

	// The desired volume of sounds inside the volume when the player is outside the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float InteriorVolume;

	// The time over which to interpolate from the current volume to the desired volume of sounds inside the volume when the player exits the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float InteriorTime;

	// The desired LPF frequency cutoff in hertz of sounds inside the volume when the player is outside the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InteriorSettings)
	float InteriorLPF;

	// The time over which to interpolate from the current LPF to the desired LPF of sounds inside the volume when the player exits the volume
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InteriorSettings)
	float InteriorLPFTime;

	ENGINE_API FInteriorSettings();

	ENGINE_API bool operator==(const FInteriorSettings& Other) const;
	ENGINE_API bool operator!=(const FInteriorSettings& Other) const;

#if WITH_EDITORONLY_DATA
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FInteriorSettings> : public TStructOpsTypeTraitsBase2<FInteriorSettings>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif

struct FAudioVolumeProxy
{
	FAudioVolumeProxy(const AAudioVolume* AudioVolume);

	uint32 AudioVolumeID = 0;
	uint32 WorldID = 0;
	float Priority = 0.0f;
	FReverbSettings ReverbSettings;
	FInteriorSettings InteriorSettings;
	TArray<FAudioVolumeSubmixSendSettings> SubmixSendSettings;
	TArray<FAudioVolumeSubmixOverrideSettings> SubmixOverrideSettings;
	FBodyInstance* BodyInstance = nullptr;
	bool bChanged = false;
};

UCLASS(hidecategories=(Advanced, Attachment, Collision, Volume), MinimalAPI)
class AAudioVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

private:
	/**
	 * Priority of this volume. In the case of overlapping volumes, the one with the highest priority
	 * is chosen. The order is undefined if two or more overlapping volumes have the same priority.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioVolume, meta=(AllowPrivateAccess="true"))
	float Priority;

	/** Whether this volume is currently enabled and able to affect sounds */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, ReplicatedUsing=OnRep_bEnabled, Category=AudioVolume, meta=(AllowPrivateAccess="true"))
	uint32 bEnabled:1;

	/** Reverb settings to use for this volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Reverb, meta=(AllowPrivateAccess="true"))
	FReverbSettings Settings;

	/** Impacts sounds that have "Apply Ambient Volumes" set to true in their Sound Class, based on whether the sound sources and the player are inside or outside the audio volume */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AmbientZone, meta=(AllowPrivateAccess="true"))
	FInteriorSettings AmbientZoneSettings;

	/** Submix send settings to use for this volume. Allows audio to dynamically send to submixes based on source and listener locations relative to this volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submixes, meta = (AllowPrivateAccess = "true"))
	TArray<FAudioVolumeSubmixSendSettings> SubmixSendSettings;

	/** Submix effect chain override settings. Will override the effect chains on the given submixes when the conditions are met. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submixes, meta = (AllowPrivateAccess = "true"))
	TArray<FAudioVolumeSubmixOverrideSettings> SubmixOverrideSettings;

public:

	float GetPriority() const { return Priority; }
	
	UFUNCTION(BlueprintCallable, Category=AudioVolume)
	ENGINE_API void SetPriority(float NewPriority);

	bool GetEnabled() const { return bEnabled; }
	
	UFUNCTION(BlueprintCallable, Category=AudioVolume)
	ENGINE_API void SetEnabled(bool bNewEnabled);

	const FReverbSettings& GetReverbSettings() const { return Settings; }
	
	UFUNCTION(BlueprintCallable, Category=AudioVolume)
	ENGINE_API void SetReverbSettings(const FReverbSettings& NewReverbSettings);

	const FInteriorSettings& GetInteriorSettings() const { return AmbientZoneSettings; }

	UFUNCTION(BlueprintCallable, Category=AudioVolume)
	ENGINE_API void SetInteriorSettings(const FInteriorSettings& NewInteriorSettings);

	const TArray<FAudioVolumeSubmixSendSettings>& GetSubmixSendSettings() const { return SubmixSendSettings; }

	UFUNCTION(BlueprintCallable, Category = AudioVolume)
	ENGINE_API void SetSubmixSendSettings(const TArray<FAudioVolumeSubmixSendSettings>& NewSubmixSendSettings);

	const TArray<FAudioVolumeSubmixOverrideSettings>& GetSubmixOverrideSettings() const { return SubmixOverrideSettings; }

	UFUNCTION(BlueprintCallable, Category = AudioVolume)
	ENGINE_API void SetSubmixOverrideSettings(const TArray<FAudioVolumeSubmixOverrideSettings>& NewSubmixOverrideSettings);

private:

	UFUNCTION()
	ENGINE_API virtual void OnRep_bEnabled();

	ENGINE_API void TransformUpdated(USceneComponent* RootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	ENGINE_API void AddProxy() const;
	ENGINE_API void RemoveProxy() const;
	ENGINE_API void UpdateProxy() const;

public:

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UObject Interface

	//~ Begin AActor Interface
	ENGINE_API virtual void PostUnregisterAllComponents() override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	//~ End AActor Interface
};
