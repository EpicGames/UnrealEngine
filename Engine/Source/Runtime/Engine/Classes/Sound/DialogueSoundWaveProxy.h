// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundBase.h"
#include "DialogueSoundWaveProxy.generated.h"

class USoundWave;
struct FActiveSound;
struct FSoundParseParameters;
struct FSubtitleCue;

UCLASS()
class UDialogueSoundWaveProxy : public USoundBase
{
	GENERATED_UCLASS_BODY()

	friend class UDialogueWave;

	USoundWave* SoundWave;
	TArray<struct FSubtitleCue> Subtitles;
	FText SubtitleText;

public:
	/** Returns whether the sound base is set up in a playable manner */
	virtual bool IsPlayable() const override;

	/** Returns a pointer to the attenuation settings that are to be applied for this node */
	virtual const FSoundAttenuationSettings* GetAttenuationSettingsToApply() const override;

	/** 
	 * Returns the farthest distance at which the sound could be heard
	 */
	virtual float GetMaxDistance() const override;

	/** 
	 * Returns the length of the sound
	 */
	virtual float GetDuration() const override;

	virtual float GetVolumeMultiplier() override;
	virtual float GetPitchMultiplier() override;

	/** 
	 * Parses the Sound to generate the WaveInstances to play
	 */
	virtual void Parse( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& AudioComponent, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;

	virtual USoundClass* GetSoundClass() const override;

	virtual class UCurveTable* GetCurveData() const override;

	inline const FText& GetSubtitleText() const
	{
		return SubtitleText;
	}
};
