// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *	This will hold all of our enums and types and such that we need to
 *	use in multiple files where the enum can't be mapped to a specific file.
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DialogueTypes.generated.h"

class UDialogueVoice;
class UDialogueWave;

UENUM()
namespace EGrammaticalGender
{
	enum Type : int
	{
		Neuter		UMETA( DisplayName = "Neuter" ),
		Masculine	UMETA( DisplayName = "Masculine" ),
		Feminine	UMETA( DisplayName = "Feminine" ),
		Mixed		UMETA( DisplayName = "Mixed" ),
	};
}

UENUM()
namespace EGrammaticalNumber
{
	enum Type : int
	{
		Singular	UMETA( DisplayName = "Singular" ),
		Plural		UMETA( DisplayName = "Plural" ),
	};
}

class UDialogueVoice;
class UDialogueWave;

USTRUCT(BlueprintType)
struct FDialogueContext
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FDialogueContext();

	/** The person speaking the dialogue. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DialogueContext )
	TObjectPtr<UDialogueVoice> Speaker;

	/** The people being spoken to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DialogueContext )
	TArray<TObjectPtr<UDialogueVoice>> Targets;

	/** Gets a generated hash created from the source and targets. */
	ENGINE_API FString GetContextHash() const;

	friend ENGINE_API bool operator==(const FDialogueContext& LHS, const FDialogueContext& RHS);
	friend ENGINE_API bool operator!=(const FDialogueContext& LHS, const FDialogueContext& RHS);
};

USTRUCT()
struct FDialogueWaveParameter
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FDialogueWaveParameter();

	/** The dialogue wave to play. */
	UPROPERTY(EditAnywhere, Category=DialogueWaveParameter )
	TObjectPtr<UDialogueWave> DialogueWave;

	/** The context to use for the dialogue wave. */
	UPROPERTY(EditAnywhere, Category=DialogueWaveParameter )
	FDialogueContext Context;
};
