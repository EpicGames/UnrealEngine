// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 * Playable sound object for spoken dialogue 
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Sound/DialogueTypes.h"

#include "DialogueVoice.generated.h"

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, BlueprintType)
class UDialogueVoice : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DialogueVoice, AssetRegistrySearchable)
	TEnumAsByte<EGrammaticalGender::Type> Gender;

	UPROPERTY(EditAnywhere, Category=DialogueVoice, AssetRegistrySearchable)
	TEnumAsByte<EGrammaticalNumber::Type> Plurality;

	UPROPERTY()
	FGuid LocalizationGUID;

public:
	//~ Begin UObject Interface. 
	virtual bool IsReadyForFinishDestroy() override;
	virtual FName GetExporterName() override;
	virtual FString GetDesc() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface. 
};
