// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 * Context to sound wave map for spoken dialogue 
 */

#include "HAL/IConsoleManager.h"
#include "Misc/Guid.h"
#include "Sound/DialogueTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DialogueWave.generated.h"

class UDialogueSoundWaveProxy;
class UDialogueVoice;
class USoundBase;
class USoundWave;
struct FPropertyChangedChainEvent;

extern ENGINE_API FAutoConsoleVariableRef CVarUseNewSubtitles;

struct FDialogueConstants
{
	static ENGINE_API const FString DialogueNamespace;
	static ENGINE_API const FString DialogueNotesNamespace;
	static ENGINE_API const FString SubtitleKeySuffix;
#if WITH_EDITORONLY_DATA
	static ENGINE_API const FString ActingDirectionKeySuffix;
	static ENGINE_API const FString PropertyName_AudioFile;
	static ENGINE_API const FString PropertyName_VoiceActorDirection;
	static ENGINE_API const FString PropertyName_Speaker;
	static ENGINE_API const FString PropertyName_Targets;
	static ENGINE_API const FString PropertyName_GrammaticalGender;
	static ENGINE_API const FString PropertyName_GrammaticalPlurality;
	static ENGINE_API const FString PropertyName_TargetGrammaticalGender;
	static ENGINE_API const FString PropertyName_TargetGrammaticalNumber;
	static ENGINE_API const FString PropertyName_DialogueContext;
	static ENGINE_API const FString PropertyName_IsMature;
#endif //WITH_EDITORONLY_DATA
};

USTRUCT()
struct FDialogueContextMapping
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FDialogueContextMapping();

	/** The context of the dialogue. */
	UPROPERTY(EditAnywhere, Category=DialogueContextMapping )
	FDialogueContext Context;

	/** The soundwave to play for this dialogue. */
	UPROPERTY(EditAnywhere, Category=DialogueContextMapping, meta = (DisallowedClasses= "/Script/MetasoundEngine.MetaSoundSource, /Script/Engine.SoundSourceBus"))
	TObjectPtr<USoundWave> SoundWave;

	/**
	 * The format string to use when generating the localization key for this context. This must be unique within the owner dialogue wave.
	 * Available format markers:
	 *   * {ContextHash} - A hash generated from the speaker and target voices.
	 */
	UPROPERTY(EditAnywhere, Category=DialogueContextMapping )
	FString LocalizationKeyFormat;

	/** Cached object for playing the soundwave with subtitle information included. */
	UPROPERTY(Transient)
	TObjectPtr<UDialogueSoundWaveProxy> Proxy;

	/** Gets the localization key to use for this context mapping */
	ENGINE_API FString GetLocalizationKey() const;
	ENGINE_API FString GetLocalizationKey(const FString& InOwnerDialogueWaveKey) const;
};

ENGINE_API bool operator==(const FDialogueContextMapping& LHS, const FDialogueContextMapping& RHS);
ENGINE_API bool operator!=(const FDialogueContextMapping& LHS, const FDialogueContextMapping& RHS);

class UDialogueWaveFactory;

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, BlueprintType, Meta = (LoadBehavior = "LazyOnDemand"))
class UDialogueWave : public UObject
{
	GENERATED_UCLASS_BODY()

	/** true if this dialogue is considered to contain mature/adult content. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Filter, AssetRegistrySearchable)
	uint32 bMature:1;

	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Script, meta=(InlineEditConditionToggle))
	uint32 bOverride_SubtitleOverride : 1;

	/** A localized version of the text that is actually spoken phonetically in the audio. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Script)
	FString SpokenText;

	/** A localized version of the subtitle text that should be displayed for this audio. By default this will be the same as the Spoken Text. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Script, meta=(EditCondition="bOverride_SubtitleOverride"))
	FString SubtitleOverride;

#if WITH_EDITORONLY_DATA
	/** Provides general notes to the voice actor intended to direct their performance, as well as contextual information to the translator. */
	UPROPERTY(EditAnywhere, Category=Script)
	FString VoiceActorDirection;
#endif // WITH_EDITORONLY_DATA

	/* Mappings between dialogue contexts and associated soundwaves. */
	UPROPERTY(EditAnywhere, Category=DialogueContexts)
	TArray<FDialogueContextMapping> ContextMappings;

	UPROPERTY()
	FGuid LocalizationGUID;

public:
	//~ Begin UObject Interface. 
	virtual void Serialize( FArchive& Ar ) override;
	virtual bool IsReadyForFinishDestroy() override;
	virtual FString GetDesc() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	//~ End UObject Interface. 

	//~ Begin UDialogueWave Interface.
	ENGINE_API bool SupportsContext(const FDialogueContext& Context) const;
	ENGINE_API USoundBase* GetWaveFromContext(const FDialogueContext& Context) const;
	ENGINE_API USoundBase* GetWaveFromContext(const FDialogueContextMapping& ContextMapping) const;
	ENGINE_API FText GetLocalizedSpokenText() const;
	ENGINE_API FText GetLocalizedSpokenText(const FDialogueContext& Context) const;
	ENGINE_API FText GetLocalizedSpokenText(const FDialogueContextMapping& ContextMapping) const;
	ENGINE_API FText GetLocalizedSubtitle() const;
	ENGINE_API FText GetLocalizedSubtitle(const FDialogueContext& Context) const;
	ENGINE_API FText GetLocalizedSubtitle(const FDialogueContextMapping& ContextMapping) const;
	ENGINE_API FString GetContextLocalizationKey(const FDialogueContext& Context) const;
	ENGINE_API FString GetContextLocalizationKey(const FDialogueContextMapping& ContextMapping) const;
	ENGINE_API FString GetContextRecordedAudioFilename(const FDialogueContext& Context) const;
	ENGINE_API FString GetContextRecordedAudioFilename(const FDialogueContextMapping& ContextMapping) const;
	ENGINE_API static FString BuildRecordedAudioFilename(const FString& FormatString, const FGuid& DialogueGuid, const FString& DialogueName, const FString& ContextId, const int32 ContextIndex);
	//~ End UDialogueWave Interface.

	ENGINE_API void UpdateContext(FDialogueContextMapping& ContextMapping, USoundWave* SoundWave, UDialogueVoice* Speaker, const TArray<UDialogueVoice*>& Targets);

protected:

	void UpdateMappingProxy(FDialogueContextMapping& ContextMapping);

private:
	void CopySoundBasePropertiesToProxy(const USoundBase* InSoundBase, USoundBase* Proxy);

};
