// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioUnknown.h"
#include "SoundWave.h"
#include "UObject/NameTypes.h"

namespace Audio
{
	struct UE_EXPERIMENTAL(5.8, "Wave Container Interface is experimental") ISoundWaveContainer
	{
		static FName GetInterfaceId()
		{
			static const FName Name(TEXT("ISoundWaveContainer"));
			return Name;
		}

		virtual ~ISoundWaveContainer() = default;

		virtual TArray<FSoundWaveProxyPtr> GetContainedWaveProxies() const = 0;

		// Validate static API implemented correctly.
		static_assert(Audio::CHasInterfaceIdFunction<ISoundWaveContainer>);
	};
}
