// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Templates/UniquePtr.h"
#include "QuartzCompileTimeVisitor.h"


// forwards
namespace Audio::Quartz::PrivateDefs
{
	template <typename... Ts>
	class  TQuartzCommandQueue : public TVisitorPatternBase<Ts...>
	{
	private:
		// type aliasing for private code...
		template <class ConsumerBaseType>
		using TCommand = typename TVisitorPatternBase<Ts...>::template TVisitWithLambda<ConsumerBaseType>;
		using ICommandBase = typename TVisitorPatternBase<Ts...>::IVisitorBase;
		
	public:
		// type aliasing for client code...
		// IConsumerBase supports polymorphic access / containers of consumers
		using IConsumerBase = typename TVisitorPatternBase<Ts...>::IListenerBase;

		// TConsumerBase is the template base class which concrete consumer types should inherit from
		template <typename... ConsumerInterfaces>
		using TConsumerBase = typename TVisitorPatternBase<Ts...>::template TElementBase<ConsumerInterfaces...>;
	
	public:
		// begin TQuartzCommandQueue interface
		TQuartzCommandQueue() = default;
		~TQuartzCommandQueue() = default;

		/**
			PushLambda():
			* push a function that takes a TargetInterface& as an argument.
			* Listeners will execute this code with (*static_cast<TargetInterface*>(this)) as the input()
		* */
		template <class TargetInterface>
		void PushLambda(TFunction<void(TargetInterface&)> InLambda)
		{
			Queue.ProduceItem(MakeUnique<TCommand<TargetInterface>>(MoveTempIfPossible(InLambda)));
		}

		// note: this is not currently needed for this implementation, but is an example of
		// polymorphic consumer usage and having multiple listeners listen to a single
		// set of pending commands.
		/**
		void PumpCommandQueue(TArray<IConsumerBase*>& InListeners)
		{
			Queue.ConsumeAllFifo([&InListeners](TUniquePtr<ICommandBase> InCommand)
			{
				for(TUniquePtr<ICommandBase> Listener : InListeners)
				{
					if (ensure(InCommand))
					{
						Listener->Accept(*InCommand);
					}
				}
			});
		}
	**/

		void PumpCommandQueue(IConsumerBase& InListener)
		{
			Queue.ConsumeAllFifo([&InListener](TUniquePtr<ICommandBase> InCommand)
			{
				InListener.Accept(*InCommand);
			});
		}

	private:
		UE::TConsumeAllMpmcQueue<TUniquePtr<ICommandBase>> Queue;
	}; // class TQuartzCommandQueue
} // namespace Audio::Quartz::PrivateDefs