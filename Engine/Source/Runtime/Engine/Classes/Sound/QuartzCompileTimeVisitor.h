// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include <type_traits>

// forwards
namespace Audio::Quartz::PrivateDefs
{
	// Notes on the Visitor Pattern (https://en.wikipedia.org/wiki/Visitor_pattern)
	// this is a pattern that helps solve two things:
	// - adds operations for a class without modifying the class itself
	// 		(and more importantly for Quartz's use-case:)
	// - implements Double-Dispatch (https://en.wikipedia.org/wiki/Double_dispatch)
	// 
	// C++ supports single-dispatch through polymorphism, where a concrete function
	// called depends on the dynamic type of a SINGLE object.
	// 
	// i.e.:  MyBasePtr->DoThing(MyConcreteType);
	// 
	// Double dispatch is being able resolve to a concrete function for how
	// TWO dynamic types should interact
	//
	// i.e.: MyBaseA_ptr->DoThing(MyBaseB_ptr)
	//
	// where we could resolve to different concrete functions for each combination of 
	// the RUNTIME TYPES of objects derived from MyBaseA and MyBaseB.
	//
	// Concretely, Quartz has metronome listeners, command listeners, etc
	// and has things that can be both a metronome and a command listener.
	//
	// The DOWNSIDE to the visitor pattern is the element (or listener) types must be known
	// at compile time.  This makes it hard to write reusable objects like command queues.
	// 
	// This implementation uses some template metaprogramming to abstract away the visitor pattern
	// and let client code build reusable things like command queues and FSMs without needing to know
	// the final concrete types.
	//
	// It also helps avoid some diamond inheritance problems that are easy to run into
	// when attempting to have a "consumer" base type, and then types that can be multiple
	// kinds of consumers.

	
	/** class TVisitorPatternBase:
	* Summary:
	* 	Used as a base class to implement the visitor pattern for the provided Ts types.
	* 
	* Examples:
	* 	For an example usage for client code derived from this see TQuartzCommandQueue
	* 	For an example of TQuartzCommandQueue client code that accepts multiple visitors see FQuartzTickableObject
	* 
	* 
	* 
	**/
	template <typename... Ts>
	class  TVisitorPatternBase
	{ 
	private:
		/**
		IVisitorInterface and IVisitorBase:
		* These classes use template metaprogramming to recursively build
		* a virtual listener interface (IVisitorBase) for our visitor pattern
		*  with a virtual Visit(T&) overload for each type in the parameter pack (Ts...)
		*
		*  for example, if we want an IVisitorBase class which looks like:
		*  class IVisitorBase
		*  {
		*		public:
		*			virtual void Visit(ConsumerTypeA& InConsumer) {}
		*			virtual void Visit(ConsumerTypeB& InConsumer) {}
		*			virtual void Visit(ConsumerTypeC& InConsumer) {}
		*  };
		*
		*  TVisitorInterface expands a template  parameter pack so that
		*  IVisitorBase ends up with a virtual Visit() overload for each type in
		*  the Interfaces parameter pack
		* */
		
		// Forward declaration for the primary template
		template <typename... Interfaces>
		class TVisitorInterface;
		
		// Specialization for the base case
		template <typename Interface>
		class TVisitorInterface<Interface>
		{
		public:
			virtual ~TVisitorInterface<Interface>() = default;
			virtual void Visit(Interface&) {};
		}; // class TVisitorInterface

		// Specialization for the recursive case
		template <typename First, typename... Rest>
		class TVisitorInterface<First, Rest...> : public TVisitorInterface<Rest...>
		{
		public:
			// Use Visit methods from the base class
			using TVisitorInterface<Rest...>::Visit;

			// Override Visit for the first type in the pack
			virtual void Visit(First&) {};

		}; // class TVisitorInterface

	protected:
		class IVisitorBase : public TVisitorInterface<Ts...>
		{
			// todo: comment about the inherited interface
		}; // class IVisitorBase

		/*
		IListenerBase: 
		* this should not be inherited from directly, but is public for
		* polymorphic access to client listeners via IListenerBase*
		*/

		class IListenerBase
		{
		public:
			virtual ~IListenerBase() = default;
			virtual void Accept(IVisitorBase&) = 0;

		}; // class IListenerBase

	public:
		/**
		TElementBase:
		* Client code defines listener interfaces, and then the client's concrete types
		* should inherit from those through this template base class.
		* (Concrete types can inherit from multiple listener interfaces)
		* the automatically-defined Accept method will loop through the interfaces
		* and cast this to each of the interface types to visit.
		* 
		* */
		template <typename... ListenerInterfaces>
		class TElementBase : public IListenerBase, public ListenerInterfaces...
		{
		public:
			virtual void Accept(IVisitorBase& InVisitor) override final
			{
				(..., (void)InVisitor.Visit(*static_cast<ListenerInterfaces*>(this)));
			}
		}; // class TElementBase

	protected:
		/**
		TVisitWithLambda
		* This templatized visitor is used in ::PushLambda()
		* external code does not need to worry about the visitor pattern
		* */
		template <class TargetInterface>
		class TVisitWithLambda : public IVisitorBase
		{
		public:
			// Bring all versions of Visit into the scope
			// otherwise the compiler complains we hide the ones we aren't overriding
			using IVisitorBase::Visit; 
			
			TVisitWithLambda() = delete;

			TVisitWithLambda<TargetInterface>(TFunction<void(TargetInterface&)> InLambda)
			: Lambda (MoveTempIfPossible(InLambda))
			{}

			virtual ~TVisitWithLambda() override = default;

			virtual void Visit(TargetInterface& InTarget) override final
			{
				if(Lambda)
				{
					Lambda(InTarget);
				}
			}

		private:
			TFunction<void(TargetInterface&)> Lambda;
		}; // class TVisitWithLambda
	}; // class TVisitorPatternBase
} // namespace Audio::Quartz::PrivateDefs