// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuartzQuantizationUtilities.h"
#include "QuartzCommandQueue.h"

namespace Audio
{
	// Struct used to communicate command state back to the game play thread
	struct FQuartzQuantizedCommandDelegateData : public FQuartzCrossThreadMessage
	{
		EQuartzCommandType CommandType;
		EQuartzCommandDelegateSubType DelegateSubType;

		// ID so the clock handle knows which delegate to fire
		int32 DelegateID{ -1 };

	}; // struct FQuartzQuantizedCommandDelegateData

	// Struct used to communicate metronome events back to the game play thread
	struct FQuartzMetronomeDelegateData : public FQuartzCrossThreadMessage
	{
		int32 Bar;
		int32 Beat;
		float BeatFraction;
		EQuartzCommandQuantization Quantization;
		FName ClockName;
		int32 FrameOffset = 0;
	}; // struct FQuartzMetronomeDelegateData

	// Struct used to queue events to be sent to the Audio Render thread closer to their start time
	struct FQuartzQueueCommandData : public FQuartzCrossThreadMessage
	{
		FAudioComponentCommandInfo AudioComponentCommandInfo;
		FName ClockName;

		FQuartzQueueCommandData(const FAudioComponentCommandInfo& InAudioComponentCommandInfo, FName InClockName)
			: AudioComponentCommandInfo(InAudioComponentCommandInfo)
			, ClockName(InClockName)
		{}
	}; // struct FQuartzQueueCommandData
}

namespace Audio::Quartz
{
	class IMetronomeEventListener
	{
	public:
		virtual ~IMetronomeEventListener() = default;
		virtual void OnMetronomeEvent(const FQuartzMetronomeDelegateData&) = 0;
	}; // class IMetronomeEventListener

	class ICommandListener
	{
	public:
		virtual ~ICommandListener() = default;
		virtual void OnCommandEvent(const FQuartzQuantizedCommandDelegateData&) = 0;
	}; // class ICommandListener

	class IQueueCommandListener
	{
	public:
		virtual ~IQueueCommandListener() = default;
		virtual void OnQueueCommandEvent(const FQuartzQueueCommandData&) = 0;
	};

	class IQuartzClock
	{
	public:
		virtual ~IQuartzClock() = default;

		// Transport control
		virtual void Resume() = 0;
		virtual void Pause() = 0;
		virtual void Restart(bool bPause = true) = 0;
		virtual void Stop(bool CancelPendingEvents) = 0; // Pause + Restart

		// Metronome Event Subscription:
		virtual void SubscribeToTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary) = 0;
		virtual void SubscribeToAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber) = 0;
		virtual void UnsubscribeFromTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary) = 0;
		virtual void UnsubscribeFromAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber) = 0;

		// Quantized Command Management:
		virtual void AddQuantizedCommand(FQuartzQuantizedRequestData& InQuantizedRequestData) = 0;
		virtual void AddQuantizedCommand(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo) = 0;
		virtual void AddQuantizedCommand(FQuartzQuantizationBoundary InQuantizationBoundary, TSharedPtr<IQuartzQuantizedCommand> InNewEvent) = 0;
	}; // IQuartzClock
} // namespace Audio::Quartz
