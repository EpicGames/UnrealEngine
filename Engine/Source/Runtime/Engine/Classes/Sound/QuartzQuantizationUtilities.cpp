// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/QuartzQuantizationUtilities.h"

#include "AudioMixerDevice.h"
#include "Quartz/AudioMixerClock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QuartzQuantizationUtilities)

#define INVALID_DURATION -1

DEFINE_LOG_CATEGORY(LogAudioQuartz);

EQuartzCommandQuantization TimeSignatureQuantizationToCommandQuantization(const EQuartzTimeSignatureQuantization& BeatType)
{
	switch (BeatType)
	{
		case EQuartzTimeSignatureQuantization::HalfNote :
			return EQuartzCommandQuantization::HalfNote;

		case EQuartzTimeSignatureQuantization::QuarterNote :
			return EQuartzCommandQuantization::QuarterNote;

		case EQuartzTimeSignatureQuantization::EighthNote :
			return EQuartzCommandQuantization::EighthNote;

		case EQuartzTimeSignatureQuantization::SixteenthNote :
			return EQuartzCommandQuantization::SixteenthNote;

		case EQuartzTimeSignatureQuantization::ThirtySecondNote :
			return EQuartzCommandQuantization::ThirtySecondNote;

		default:
			return EQuartzCommandQuantization::Count;
	}
}

FQuartzTimeSignature::FQuartzTimeSignature(const FQuartzTimeSignature& Other)
	: NumBeats(Other.NumBeats)
	, BeatType(Other.BeatType)
	, OptionalPulseOverride(Other.OptionalPulseOverride)
{
}

FQuartzTimeSignature& FQuartzTimeSignature::operator=(const FQuartzTimeSignature& Other)
{
	NumBeats = Other.NumBeats;
	BeatType = Other.BeatType;
	OptionalPulseOverride = Other.OptionalPulseOverride;

	return *this;
}

bool FQuartzTimeSignature::operator==(const FQuartzTimeSignature& Other) const
{
	bool Result = (NumBeats == Other.NumBeats);
	Result &= (BeatType == Other.BeatType);
	Result &= (OptionalPulseOverride.Num() == Other.OptionalPulseOverride.Num());

	const int32 NumPulseEntries = OptionalPulseOverride.Num();


	if (Result && NumPulseEntries)
	{
		for (int32 i = 0; i < NumPulseEntries; ++i)
		{
			const bool NumPulsesMatch = (OptionalPulseOverride[i].NumberOfPulses == Other.OptionalPulseOverride[i].NumberOfPulses);
			const bool DurationsMatch = (OptionalPulseOverride[i].PulseDuration == Other.OptionalPulseOverride[i].PulseDuration);

			if (!(NumPulsesMatch && DurationsMatch))
			{
				Result = false;
				break;
			}
		}
	}

	return Result;
}


FQuartLatencyTracker::FQuartLatencyTracker()
	: Min(TNumericLimits<float>::Max())
	, Max(TNumericLimits<float>::Min())
{
}

void FQuartLatencyTracker::PushLatencyTrackerResult(const double& InResult)
{
	ResultQueue.Enqueue((float)InResult);

	if (!IsInGameThread())
	{
		return;
	}

	DigestQueue();
}

float FQuartLatencyTracker::GetLifetimeAverageLatency()
{
	if (IsInGameThread())
	{
		DigestQueue();
	}

	return LifetimeAverage;
}

float FQuartLatencyTracker::GetMinLatency()
{
	if (IsInGameThread())
	{
		DigestQueue();
	}

	return Min;
}

float FQuartLatencyTracker::GetMaxLatency()
{
	if (IsInGameThread())
	{
		DigestQueue();
	}

	return Max;
}

void FQuartLatencyTracker::PushSingleResult(const double& InResult)
{
	if (++NumEntries == 0)
	{
		LifetimeAverage = (float)InResult;
	}
	else
	{
		LifetimeAverage = (LifetimeAverage * (NumEntries - 1) + (float)InResult) / NumEntries;
	}

	Min = FMath::Min(Min, (float)InResult);
	Max = FMath::Max(Max, (float)InResult);
}

void FQuartLatencyTracker::DigestQueue()
{
	check(IsInGameThread());

	float Result;
	while (ResultQueue.Dequeue(Result))
	{
		PushSingleResult(Result);
	}
}

static FString EnumToString(EQuartzCommandQuantization inEnum)
{
	switch (inEnum)
	{
	case EQuartzCommandQuantization::Bar:
		return(TEXT("Bar"));
	case EQuartzCommandQuantization::Beat:
		return(TEXT("Beat"));
		
	case EQuartzCommandQuantization::ThirtySecondNote:
		return(TEXT("ThirtySecondNote"));
	case EQuartzCommandQuantization::SixteenthNote:
		return(TEXT("SixteenthNote"));
	case EQuartzCommandQuantization::EighthNote:
		return(TEXT("EighthNote"));
	case EQuartzCommandQuantization::QuarterNote:
		return(TEXT("QuarterNote"));
	case EQuartzCommandQuantization::HalfNote:
		return(TEXT("HalfNote"));
	case EQuartzCommandQuantization::WholeNote:
		return(TEXT("WholeNote"));

	case EQuartzCommandQuantization::DottedSixteenthNote:
		return(TEXT("DottedSixteenthNote"));
	case EQuartzCommandQuantization::DottedEighthNote:
		return(TEXT("DottedEighthNote"));
	case EQuartzCommandQuantization::DottedQuarterNote:
		return(TEXT("DottedQuarterNote"));
	case EQuartzCommandQuantization::DottedHalfNote:
		return(TEXT("DottedHalfNote"));
	case EQuartzCommandQuantization::DottedWholeNote:
		return(TEXT("WholeNote"));

	case EQuartzCommandQuantization::SixteenthNoteTriplet:
		return(TEXT("SixteenthNoteTriplet"));
	case EQuartzCommandQuantization::EighthNoteTriplet:
		return(TEXT("EighthNoteTriplet"));
	case EQuartzCommandQuantization::QuarterNoteTriplet:
		return(TEXT("QuarterNoteTriplet"));
	case EQuartzCommandQuantization::HalfNoteTriplet:
		return(TEXT("HalfNoteTriplet"));

	case EQuartzCommandQuantization::None:
		return(TEXT("None"));
		
	default:
		return {};
	}
	
}

static FString EnumToString(EQuarztQuantizationReference inEnum)
{
	switch (inEnum)
	{
	case EQuarztQuantizationReference::BarRelative:
			return(TEXT("BarRelative"));
	case EQuarztQuantizationReference::TransportRelative:
		return(TEXT("TransportRelative"));
	case EQuarztQuantizationReference::CurrentTimeRelative:
		return(TEXT("CurrentTimeRelative"));
		
		default:
			return {};
	}
	
}

FString FQuartzQuantizationBoundary::ToString() const 
{
	FString String;

	String
	.Append(*FString::Printf(TEXT("Quant:(%s) - "), *EnumToString(Quantization)))
	.Append(*FString::Printf(TEXT("Mult:(%f) - "), Multiplier))
	.Append(*FString::Printf(TEXT("Ref:(%s)"), *EnumToString(CountingReferencePoint)));
	
	return String;
}


namespace Audio
{
	FQuartzClockTickRate::FQuartzClockTickRate()
	{
		SetBeatsPerMinute(60.0);
	}

	void FQuartzClockTickRate::SetFramesPerTick(int32 InNewFramesPerTick)
	{
		if (InNewFramesPerTick < 1)
		{
			UE_LOGF(LogAudioQuartz, Warning, "Quartz Metronme requires at least 1 frame per tick, clamping request");
			InNewFramesPerTick = 1;
		}

		FramesPerTick = (double)InNewFramesPerTick;
		RecalculateDurationsBasedOnFramesPerTick();
	}

	void FQuartzClockTickRate::SetMillisecondsPerTick(double InNewMillisecondsPerTick)
	{
		FramesPerTick = FMath::Max(1.0, (InNewMillisecondsPerTick * SampleRate) / 1000.0);
		RecalculateDurationsBasedOnFramesPerTick();
	}

	void FQuartzClockTickRate::SetThirtySecondNotesPerMinute(double InNewThirtySecondNotesPerMinute)
	{
		check(InNewThirtySecondNotesPerMinute > 0);

		FramesPerTick = FMath::Max(1.0, (60. * SampleRate ) / InNewThirtySecondNotesPerMinute);
		RecalculateDurationsBasedOnFramesPerTick();
	}

	void FQuartzClockTickRate::SetBeatsPerMinute(double InNewBeatsPerMinute)
	{
		// same as 1/32nd notes,
		// except there are 1/8th the number of quarter notes than thirty-second notes in a minute
		// (So FramesPerTick should be 8 times shorter than it was when setting 32nd notes)

		// FramesPerTick = 1/8 * (60.f / (InNewBeatsPerMinute)) * SampleRate;
		// (60.0 / 8.0) = 7.5f

		FramesPerTick = FMath::Max(1.0, (7.5 * SampleRate) / InNewBeatsPerMinute);
		RecalculateDurationsBasedOnFramesPerTick();
	}

	void FQuartzClockTickRate::SetSampleRate(double InNewSampleRate)
	{
		check(InNewSampleRate >= 0);

		FramesPerTick = FMath::Max(1.0, (InNewSampleRate / SampleRate) * FramesPerTick);
		SampleRate = InNewSampleRate;

		RecalculateDurationsBasedOnFramesPerTick();
	}

	double FQuartzClockTickRate::GetFramesPerDuration(EQuartzCommandQuantization InDuration) const
	{
		const double FramesPerDotted16th = FramesPerTick * 3.0;
		const double FramesPer16thTriplet = 4.0 * FramesPerTick / 3.0;

		switch (InDuration)
		{
		case EQuartzCommandQuantization::None:
			return 0;

			// NORMAL
		case EQuartzCommandQuantization::Tick:
		case EQuartzCommandQuantization::ThirtySecondNote:
			return FramesPerTick; // same as 1/32nd note

		case EQuartzCommandQuantization::SixteenthNote:
			return FramesPerTick * 2.0;

		case EQuartzCommandQuantization::EighthNote:
			return FramesPerTick * 4.0;

		case EQuartzCommandQuantization::Beat: // default to quarter note (should be overridden for non-basic meters)
		case EQuartzCommandQuantization::QuarterNote:
			return FramesPerTick * 8.0;

		case EQuartzCommandQuantization::HalfNote:
			return FramesPerTick * 16.0;

		case EQuartzCommandQuantization::Bar: // default to whole note (should be overridden for non-4/4 meters)
		case EQuartzCommandQuantization::WholeNote:
			return FramesPerTick * 32.0;

			// DOTTED
		case EQuartzCommandQuantization::DottedSixteenthNote:
			return FramesPerDotted16th;

		case EQuartzCommandQuantization::DottedEighthNote:
			return FramesPerDotted16th * 2.0;

		case EQuartzCommandQuantization::DottedQuarterNote:
			return FramesPerDotted16th * 4.0;

		case EQuartzCommandQuantization::DottedHalfNote:
			return FramesPerDotted16th * 8.0;

		case EQuartzCommandQuantization::DottedWholeNote:
			return FramesPerDotted16th * 16.0;


			// TRIPLETS
		case EQuartzCommandQuantization::SixteenthNoteTriplet:
			return FramesPer16thTriplet;

		case EQuartzCommandQuantization::EighthNoteTriplet:
			return FramesPer16thTriplet * 2.0;

		case EQuartzCommandQuantization::QuarterNoteTriplet:
			return FramesPer16thTriplet * 4.0;

		case EQuartzCommandQuantization::HalfNoteTriplet:
			return FramesPer16thTriplet * 8.0;


		default:
			checkf(false, TEXT("Unexpected EAudioMixerCommandQuantization: Need to update switch statement for new quantization enumeration?"));
			break;
		}

		return INVALID_DURATION;
	}

	double FQuartzClockTickRate::GetFramesPerDuration(EQuartzTimeSignatureQuantization InDuration) const
	{
		switch (InDuration)
		{
		case EQuartzTimeSignatureQuantization::HalfNote:
			return GetFramesPerDuration(EQuartzCommandQuantization::HalfNote);

		case EQuartzTimeSignatureQuantization::QuarterNote:
			return GetFramesPerDuration(EQuartzCommandQuantization::QuarterNote);

		case EQuartzTimeSignatureQuantization::EighthNote:
			return GetFramesPerDuration(EQuartzCommandQuantization::EighthNote);

		case EQuartzTimeSignatureQuantization::SixteenthNote:
			return GetFramesPerDuration(EQuartzCommandQuantization::SixteenthNote);

		case EQuartzTimeSignatureQuantization::ThirtySecondNote:
			return GetFramesPerDuration(EQuartzCommandQuantization::ThirtySecondNote);

		default:
			checkf(false, TEXT("Unexpected EQuartzTimeSignatureQuantization: Need to update switch statement for new quantization enumeration?"));
			break;
		}

		return INVALID_DURATION;
	}

	bool FQuartzClockTickRate::IsValid(int32 InEventResolutionThreshold) const
	{
		ensureMsgf(InEventResolutionThreshold > 0
			, TEXT("Querying a the validity of an FQuartzClockTickRate object w/ a zero or negative threshold of (%i)")
			, InEventResolutionThreshold);

		if (FramesPerTick < InEventResolutionThreshold)
		{
			return false;
		}

		return true;
	}

	bool FQuartzClockTickRate::IsSameTickRate(const FQuartzClockTickRate& Other, bool bAccountForDifferentSampleRates) const
	{
		if (!bAccountForDifferentSampleRates)
		{
			const bool Result = FramesPerTick == Other.FramesPerTick;

			// All other members SHOULD be equal if the FramesPerTick (ground truth) are equal
			checkSlow(!Result ||
				(FMath::IsNearlyEqual(MillisecondsPerTick, Other.MillisecondsPerTick)
					&& FMath::IsNearlyEqual(ThirtySecondNotesPerMinute, Other.ThirtySecondNotesPerMinute)
					&& FMath::IsNearlyEqual(BeatsPerMinute, Other.BeatsPerMinute)
					&& FMath::IsNearlyEqual(SampleRate, Other.SampleRate)));

			return Result;
		}
		else
		{
			// Perform SampleRate conversion on a temporary to see if
			FQuartzClockTickRate TempTickRate = Other;
			TempTickRate.SetSampleRate(SampleRate);

			const bool Result = FramesPerTick == TempTickRate.FramesPerTick;

			// All other members SHOULD be equal if the FramesPerTick (ground truth) are equal
			checkSlow(!Result ||
				(FMath::IsNearlyEqual(MillisecondsPerTick, TempTickRate.MillisecondsPerTick)
					&& FMath::IsNearlyEqual(ThirtySecondNotesPerMinute, TempTickRate.ThirtySecondNotesPerMinute)
					&& FMath::IsNearlyEqual(BeatsPerMinute, TempTickRate.BeatsPerMinute)
					&& FMath::IsNearlyEqual(SampleRate, TempTickRate.SampleRate)));

			return Result;
		}
	}

	void FQuartzClockTickRate::RecalculateDurationsBasedOnFramesPerTick()
	{
		check(FramesPerTick > 0.0);
		check(SampleRate > 0.0);

		SecondsPerTick = FramesPerTick / SampleRate;
		MillisecondsPerTick = SecondsPerTick * 1000.0;
		ThirtySecondNotesPerMinute = (60.0 * SampleRate) / FramesPerTick;
		BeatsPerMinute = ThirtySecondNotesPerMinute / 8.0;
	}


	void FQuartzClockTickRate::SetSecondsPerTick(double InNewSecondsPerTick)
	{
		SetMillisecondsPerTick(InNewSecondsPerTick * 1000.0);
	}


	FQuartzQuantizedCommandInitInfo::FQuartzQuantizedCommandInitInfo(
		const FQuartzQuantizedRequestData& RHS
		, float InSampleRate
		, int32 InSourceID
	)
		: ClockName(RHS.ClockName)
		, OtherClockName(RHS.OtherClockName)
		, QuantizedCommandPtr(RHS.QuantizedCommandPtr)
		, QuantizationBoundary(RHS.QuantizationBoundary)
		, GameThreadSubscribers(RHS.GameThreadSubscribers)
		, GameThreadDelegateID(RHS.GameThreadDelegateID)
		, OwningClockPointer(nullptr)
		, SampleRate(InSampleRate)
		, SourceID(InSourceID)
	{
	}

	IQuartzQuantizedCommand::IQuartzQuantizedCommand() = default;
	IQuartzQuantizedCommand::~IQuartzQuantizedCommand() = default;

	TSharedPtr<IQuartzQuantizedCommand> IQuartzQuantizedCommand::GetDeepCopyOfDerivedObject() const
	{
		// implement this method to allow copies to be made from pointers to base class
		checkSlow(false);
		return nullptr;
	}

	void IQuartzQuantizedCommand::AddSubscriber(FQuartzGameThreadSubscriber InSubscriber)
	{
		GameThreadSubscribers.AddUnique(InSubscriber);
	}

	void IQuartzQuantizedCommand::OnQueued(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::OnQueued);

		if (Audio::FMixerDevice* MixerDevice = InCommandInitInfo.OwningClockPointer->GetMixerDevice())
		{
			MixerDevice->QuantizedEventClockManager.PushLatencyTrackerResult(FQuartzCrossThreadMessage::RequestReceived());
		}

		GameThreadSubscribers.Append(InCommandInitInfo.GameThreadSubscribers);
		GameThreadDelegateID = InCommandInitInfo.GameThreadDelegateID;

		if (GameThreadSubscribers.Num())
		{
			FQuartzQuantizedCommandDelegateData Data;

			Data.CommandType = GetCommandType();
			Data.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnQueued;
			Data.DelegateID = GameThreadDelegateID;

			for (auto& Subscriber : GameThreadSubscribers)
			{
				Subscriber.PushEvent(Data);
			}
		}

		UE_LOGF(LogAudioQuartz, Verbose, "OnQueued() called for quantized event type: [%ls]", *GetCommandName().ToString());
		OnQueuedCustom(InCommandInitInfo);
	}

	void IQuartzQuantizedCommand::OnScheduled(const FQuartzClockTickRate& InTickRate)
	{
		for(auto& Subscriber : GameThreadSubscribers)
		{
			Subscriber.FinalizeOffset(InTickRate);
		}
	}

	void IQuartzQuantizedCommand::Update(int32 NumFramesUntilDeadline)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::Countdown);

		FQuartzQuantizedCommandDelegateData Data;
		Data.CommandType = GetCommandType();
		Data.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnAboutToStart;
		Data.DelegateID = GameThreadDelegateID;

		for(auto& Subscriber : GameThreadSubscribers)
		{
			// we only want to send this notification to the subscriber once
			const int32 NumFramesOfAnticipation = Subscriber.GetOffsetAsAudioFrames();
			if(!Subscriber.HasBeenNotifiedOfAboutToStart()
				&&  (NumFramesOfAnticipation >= NumFramesUntilDeadline))
			{
				Subscriber.PushEvent(Data);
			}
		}
	}

	void IQuartzQuantizedCommand::FailedToQueue(FQuartzQuantizedRequestData& InGameThreadData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::FailedToQueue);

		GameThreadSubscribers.Append(InGameThreadData.GameThreadSubscribers);
		GameThreadDelegateID = InGameThreadData.GameThreadDelegateID;

		if (GameThreadSubscribers.Num())
		{
			FQuartzQuantizedCommandDelegateData Data;
			Data.CommandType = GetCommandType();
			Data.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnFailedToQueue;
			Data.DelegateID = GameThreadDelegateID;

			for (auto& Subscriber : GameThreadSubscribers)
			{
				Subscriber.PushEvent(Data);
			}
		}

		UE_LOGF(LogAudioQuartz, Verbose, "FailedToQueue() called for quantized event type: [%ls]", *GetCommandName().ToString());
		FailedToQueueCustom();
	}

	void IQuartzQuantizedCommand::AboutToStart()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::AboutToStart);

		FQuartzQuantizedCommandDelegateData Data;
		Data.CommandType = GetCommandType();
		Data.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnAboutToStart;
		Data.DelegateID = GameThreadDelegateID;

		for(auto& Subscriber : GameThreadSubscribers)
		{
			// we only want to send this notification to the subscriber once
			if(!Subscriber.HasBeenNotifiedOfAboutToStart())
			{
				Subscriber.PushEvent(Data);
			}
		}

		UE_LOGF(LogAudioQuartz, Verbose, "AboutToStart() called for quantized event type: [%ls]", *GetCommandName().ToString());
		AboutToStartCustom();
	}

	void IQuartzQuantizedCommand::OnFinalCallback(int32 InNumFramesLeft)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::OnFinalCallback);
		if (GameThreadSubscribers.Num())
		{
			FQuartzQuantizedCommandDelegateData OnStartedData;

			OnStartedData.CommandType = GetCommandType();
			OnStartedData.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnStarted;
			OnStartedData.DelegateID = GameThreadDelegateID;

			for (auto& Subscriber : GameThreadSubscribers)
			{
				Subscriber.PushEvent(OnStartedData);
			}
		}

		UE_LOGF(LogAudioQuartz, Verbose, "OnFinalCallback() called for quantized event type: [%ls]", *GetCommandName().ToString());
		OnFinalCallbackCustom(InNumFramesLeft);
	}

	void IQuartzQuantizedCommand::OnClockPaused()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::OnClockPaused);
		UE_LOGF(LogAudioQuartz, Verbose, "OnClockPaused() called for quantized event type: [%ls]", *GetCommandName().ToString());
		OnClockPausedCustom();
	}

	void IQuartzQuantizedCommand::OnClockStarted()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::OnClockStarted);
		UE_LOGF(LogAudioQuartz, Verbose, "OnClockStarted() called for quantized event type: [%ls]", *GetCommandName().ToString());
		OnClockStartedCustom();
	}

	void IQuartzQuantizedCommand::Cancel()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzQuantizedCommand::Cancel);
		FQuartzQuantizedCommandDelegateData Data;

		Data.CommandType = GetCommandType();
		Data.DelegateSubType = EQuartzCommandDelegateSubType::CommandOnCanceled;
		Data.DelegateID = GameThreadDelegateID;

		for (auto& Subscriber : GameThreadSubscribers)
		{
			Subscriber.PushEvent(Data);
		}

		UE_LOGF(LogAudioQuartz, Verbose, "Cancel() called for quantized event type: [%ls]", *GetCommandName().ToString());
		CancelCustom();
	}


	bool FQuartzQuantizedCommandHandle::Cancel()
	{
		checkSlow(MixerDevice);
		checkSlow(MixerDevice->IsAudioRenderingThread());

		if (CommandPtr && !OwningClockName.IsNone())
		{
			UE_LOGF(LogAudioQuartz, Verbose, "OnQueued() called for quantized event type: [%ls]", *CommandPtr->GetCommandName().ToString());
			return MixerDevice->QuantizedEventClockManager.CancelCommandOnClock(OwningClockName, CommandPtr);
		}

		return false;
	}

	void FQuartzQuantizedCommandHandle::Reset()
	{
		MixerDevice = nullptr;
		CommandPtr.Reset();
		OwningClockName = FName();
	}

	FQuartzLatencyTimer::FQuartzLatencyTimer()
		: JourneyStartCycles(-1)
		, JourneyEndCycles(-1)
	{
	}

	void FQuartzLatencyTimer::StartTimer()
	{
		JourneyStartCycles = FPlatformTime::Cycles64();
	}

	void FQuartzLatencyTimer::ResetTimer()
	{
		JourneyStartCycles = -1;
		JourneyEndCycles = -1;
	}

	void FQuartzLatencyTimer::StopTimer()
	{
		JourneyEndCycles = FPlatformTime::Cycles64();

	}

	double FQuartzLatencyTimer::GetCurrentTimePassedMs()
	{
		if (!IsTimerRunning())
		{
			return 0.0;
		}

		return FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - JourneyStartCycles);
	}

	double FQuartzLatencyTimer::GetResultsMilliseconds()
	{
		if (HasTimerRun())
		{
			return FPlatformTime::ToMilliseconds64(JourneyEndCycles - JourneyStartCycles);
		}

		return 0.0;
	}

	bool FQuartzLatencyTimer::HasTimerStarted()
	{
		return JourneyStartCycles > 0;
	}

	bool FQuartzLatencyTimer::HasTimerStopped()
	{
		return JourneyEndCycles > 0;
	}

	bool FQuartzLatencyTimer::IsTimerRunning()
	{
		return HasTimerStarted() && !HasTimerStopped();
	}

	bool FQuartzLatencyTimer::HasTimerRun()
	{
		return HasTimerStarted() && HasTimerStopped();
	}

	FQuartzCrossThreadMessage::FQuartzCrossThreadMessage(bool bAutoStartTimer)
	{
		if (bAutoStartTimer)
		{
			Timer.StartTimer();
		}
	}

	void FQuartzCrossThreadMessage::RequestSent()
	{
		Timer.StartTimer();
	}

	double FQuartzCrossThreadMessage::RequestReceived() const
	{
		Timer.StopTimer();
		return GetResultsMilliseconds();
	}

	double FQuartzCrossThreadMessage::GetResultsMilliseconds() const
	{
		return Timer.GetResultsMilliseconds();
	}

	double FQuartzCrossThreadMessage::GetCurrentTimeMilliseconds() const
	{
		return Timer.GetCurrentTimePassedMs();
	}

	bool FQuartzOffset::operator==(const FQuartzOffset& Other) const
	{
		return OffsetInMilliseconds == Other.OffsetInMilliseconds
			&& OffsetAsDuration == Other.OffsetAsDuration;
	}

	bool FQuartzGameThreadSubscriber::operator==(const FQuartzGameThreadSubscriber& Other) const
	{
		return Offset == Other.Offset
			&& Queue == Other.Queue;
	}

	void FQuartzGameThreadSubscriber::PushEvent(const FQuartzQuantizedCommandDelegateData& Data)
	{
		if(ensure(Queue.IsValid()))
		{
			Queue->PushLambda<ICommandListener>([=](ICommandListener& InListener)
			{
				InListener.OnCommandEvent(Data);
			});

			// raise the flag if this was a CommandOnAboutToStart notification
			if(!bHasBeenNotifiedOfAboutToStart)
			{
				bHasBeenNotifiedOfAboutToStart = (Data.DelegateSubType == EQuartzCommandDelegateSubType::CommandOnAboutToStart);
			}
		}
	}

	void FQuartzGameThreadSubscriber::PushEvent(const FQuartzMetronomeDelegateData& Data)
	{
		if(ensure(Queue.IsValid()))
		{
			Queue->PushLambda<IMetronomeEventListener>([=](IMetronomeEventListener& InListener)
			{
				InListener.OnMetronomeEvent(Data);
			});
		}
	}

	void FQuartzGameThreadSubscriber::PushEvent(const FQuartzQueueCommandData& Data)
	{
		if(ensure(Queue.IsValid()))
		{
			Queue->PushLambda<IQueueCommandListener>([=](IQueueCommandListener& InListener)
			{
				InListener.OnQueueCommandEvent(Data);
			});
		}
	}

	int32 FQuartzGameThreadSubscriber::FinalizeOffset(const FQuartzClockTickRate& TickRate)
	{
		bOffsetConvertedToFrames = true;
		return OffsetInAudioFrames = Offset.GetOffsetInAudioFrames(TickRate);
	}

	int32 FQuartzGameThreadSubscriber::GetOffsetAsAudioFrames() const
	{
		ensureAlwaysMsgf(bOffsetConvertedToFrames, TEXT("FinalizeOffset must be called before calling GetOffsetAsAudioFrames()"));
		return OffsetInAudioFrames;
	}
} // namespace Audio

bool FQuartzTransportTimeStamp::IsZero() const
{
	return (!Bars) && (!Beat) && FMath::IsNearlyZero(BeatFraction);
}

void FQuartzTransportTimeStamp::Reset()
{
	Bars = 0;
	Beat = 0;
	BeatFraction = 0.f;
}

