// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "QuartzCommandQueue.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "QuartzQuantizationUtilities.generated.h"

class FQuartzTickableObject;
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioQuartz, Log, All);

// forwards
struct FQuartzClockTickRate;
struct FQuartzQuantizationBoundary;
struct FQuartzTimeSignature;



namespace Audio
{
	// forwards (Audio::)
	class IAudioMixerQuantizedEventListener;
	class IQuartzQuantizedCommand;
	class FQuartzClock;
	class FShareableQuartzCommandQueue;

	class FMixerDevice;

	struct FQuartzQuantizedCommandDelegateData;
	struct FQuartzMetronomeDelegateData;
	struct FQuartzQueueCommandData;
	struct FQuartzQuantizedCommandInitInfo;
} // namespace Audio


// An enumeration for specifying quantization for Quartz commands
UENUM(BlueprintType)
enum class EQuartzCommandQuantization : uint8
{
	Bar						UMETA(DisplayName = "Bar", ToolTip = "(dependent on time signature)"),
	Beat					UMETA(DisplayName = "Beat", ToolTip = "(dependent on time signature and Pulse Override)"),

	ThirtySecondNote		UMETA(DisplayName = "1/32"),
	SixteenthNote			UMETA(DisplayName = "1/16"),
	EighthNote				UMETA(DisplayName = "1/8"),
	QuarterNote				UMETA(DisplayName = "1/4"),
	HalfNote				UMETA(DisplayName = "Half"),
	WholeNote				UMETA(DisplayName = "Whole"),

	DottedSixteenthNote		UMETA(DisplayName = "(dotted) 1/16"),
	DottedEighthNote		UMETA(DisplayName = "(dotted) 1/8"),
	DottedQuarterNote		UMETA(DisplayName = "(dotted) 1/4"),
	DottedHalfNote			UMETA(DisplayName = "(dotted) Half"),
	DottedWholeNote			UMETA(DisplayName = "(dotted) Whole"),

	SixteenthNoteTriplet	UMETA(DisplayName = "1/16 (triplet)"),
	EighthNoteTriplet		UMETA(DisplayName = "1/8 (triplet)"),
	QuarterNoteTriplet		UMETA(DisplayName = "1/4 (triplet)"),
	HalfNoteTriplet			UMETA(DisplayName = "1/2 (triplet)"),

	Tick					UMETA(DisplayName = "On Tick (Smallest Value, same as 1/32)", ToolTip = "(same as 1/32)"),

	Count					UMETA(Hidden),

	None					UMETA(DisplayName = "None", ToolTip = "(Execute as soon as possible)"),
	// (when using "Count" in various logic, we don't want to account for "None")
};

// An enumeration for specifying the denominator of time signatures
UENUM(BlueprintType)
enum class EQuartzTimeSignatureQuantization : uint8
{
	HalfNote				UMETA(DisplayName = "/2"),
	QuarterNote				UMETA(DisplayName = "/4"),
	EighthNote				UMETA(DisplayName = "/8"),
	SixteenthNote			UMETA(DisplayName = "/16"),
	ThirtySecondNote		UMETA(DisplayName = "/32"),

	Count				UMETA(Hidden),
};

EQuartzCommandQuantization ENGINE_API TimeSignatureQuantizationToCommandQuantization(const EQuartzTimeSignatureQuantization& BeatType);

// Allows the user to specify non-uniform beat durations in odd meters
USTRUCT(BlueprintType)
struct FQuartzPulseOverrideStep
{
	GENERATED_BODY()

	// The number of pulses for this beat duration
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Time Signature")
	int32 NumberOfPulses{ 1 };

	// This Beat duration
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Time Signature")
	EQuartzCommandQuantization PulseDuration{ EQuartzCommandQuantization::Beat };
};


// Quartz Time Signature
USTRUCT(BlueprintType)
struct FQuartzTimeSignature
{
	GENERATED_BODY()

	// default ctor
	FQuartzTimeSignature() {};

	// numerator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Time Signature", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumBeats { 4 };

	// denominator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Time Signature")
	EQuartzTimeSignatureQuantization BeatType { EQuartzTimeSignatureQuantization::QuarterNote };

	// beat override
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Time Signature")
	TArray<FQuartzPulseOverrideStep> OptionalPulseOverride { };


	// copy ctor
	ENGINE_API FQuartzTimeSignature(const FQuartzTimeSignature& Other);

	// assignment
	ENGINE_API FQuartzTimeSignature& operator=(const FQuartzTimeSignature& Other);

	// comparison
	ENGINE_API bool operator==(const FQuartzTimeSignature& Other) const;
};

// Transport Time stamp, used for tracking the musical time stamp on a clock
USTRUCT(BlueprintType)
struct FQuartzTransportTimeStamp
{
	GENERATED_BODY()

	// The current bar this clock is on
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Quantized Audio TimeStamp")
	int32 Bars { 0 };

	// The current beat this clock is on
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Quantized Audio TimeStamp")
	int32 Beat{ 0 };

	// A fractional representation of the time that's played since the last bear
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Quantized Audio TimeStamp")
	float BeatFraction{ 0.f };

	// The time in seconds that this TimeStamp occured at
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Quantized Audio TimeStamp")
	float Seconds{ 0.f };

	ENGINE_API bool IsZero() const;

	ENGINE_API void Reset();
};


// An enumeration for specifying different TYPES of delegates
UENUM(BlueprintType)
enum class EQuartzDelegateType : uint8
{
	MetronomeTick				UMETA(DisplayName = "Metronome Tick"), // uses EAudioMixerCommandQuantization to select subdivision
	CommandEvent				UMETA(DisplayName = "Command Event"),

	Count					UMETA(Hidden)
};


// An enumeration for specifying quantization boundary reference frame
UENUM(BlueprintType, meta = (DisplayName="EQuartz Quantization Reference")) // display name to hide c++ typo for now
enum class EQuarztQuantizationReference : uint8
{
	BarRelative				UMETA(DisplayName = "Bar Relative", ToolTip = "Will occur on the next occurence of this duration from the start of a bar (i.e. On beat 3)"),
	TransportRelative		UMETA(DisplayName = "Transport Relative", ToolTip = "Will occur on the next multiple of this duration since the clock started ticking (i.e. on the next 4 bar boundary)"),
	CurrentTimeRelative		UMETA(DisplayName = "Current Time Relative", ToolTip = "Will occur on the next multiple of this duration from the current time (i.e. In three beats)"),

	Count					UMETA(Hidden)
};

// An enumeration for specifying different TYPES of delegates
UENUM(BlueprintType)
enum class EQuartzCommandDelegateSubType : uint8
{
	CommandOnFailedToQueue		UMETA(DisplayName = "Failed To Queue", ToolTip = "The command will not execute (i.e. Clock doesn't exist or PlayQuantized failed concurrency)"),
	CommandOnQueued				UMETA(DisplayName = "Queued", ToolTip = "The command has been passed to the Audio Render Thread"),
	CommandOnCanceled			UMETA(DisplayName = "Canceled", ToolTip = "The command was stopped before it could execute"),
	CommandOnAboutToStart		UMETA(DisplayName = "About To Start", ToolTip = "execute off this to be in sync w/ sound starting"),
	CommandOnStarted			UMETA(DisplayName = "Started", ToolTip = "the command was just executed on the Audio Render Thrtead"),

	Count					UMETA(Hidden)
};

// An enumeration for specifying Quartz command types
UENUM(BlueprintType)
enum class EQuartzCommandType : uint8
{
	PlaySound UMETA(DisplayName = "Play Sound", ToolTip = "Play a sound on a spample-accurate boundary (taking a voice slot immediately)"),
	QueueSoundToPlay UMETA(DisplayName = "Queue Sound To Play", ToolTip = "Queue a sound to play when it gets closer to its quantization boundary (avoids stealing a voice slot right away)"),
	RetriggerSound UMETA(DisplayName = "Re-trigger Sound", ToolTip = "Quantized looping of the target sound (event tells the AudioComponent to play the sound again)"),
	TickRateChange UMETA(DisplayName = "Tick Rate Change", ToolTip = "Quantized change of the tick-rate (i.e. BPM change)"),
	TransportReset UMETA(DisplayName = "Transport Reset", ToolTip = "Quantized reset of the clocks transport (back to time = 0 on the boundary)"),
	StartOtherClock UMETA(DisplayName = "Start Other Clock", ToolTip = "Quantized start of another clock. Useful for sample accurate synchronization of clocks (i.e. to handle time signature changes)"),
	Notify UMETA(DisplayName = "Quantized Notify", ToolTip = "Command used only to get delegates for timing information (basically an empty command)"),
	Custom UMETA(DisplayName = "Custom", ToolTip = "Quantized custom command")
}; // EQuartzCommandType


// Delegate Declarations
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnQuartzMetronomeEvent, FName, ClockName, EQuartzCommandQuantization, QuantizationType, int32, NumBars, int32, Beat, float, BeatFraction);
DECLARE_DYNAMIC_DELEGATE_FiveParams(FOnQuartzMetronomeEventBP, FName, ClockName, EQuartzCommandQuantization, QuantizationType, int32, NumBars, int32, Beat, float, BeatFraction);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnQuartzCommandEvent, EQuartzCommandDelegateSubType, EventType, FName, Name);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnQuartzCommandEventBP, EQuartzCommandDelegateSubType, EventType, FName, Name);

// UStruct version of settings struct used to initialized a clock
USTRUCT(BlueprintType)
struct FQuartzClockSettings
{
	GENERATED_BODY()

	// Time Signature (defaults to 4/4)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Settings")
	FQuartzTimeSignature TimeSignature;

	// should the clock start Ticking
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Settings")
	bool bIgnoreLevelChange{ false };

}; // struct FQuartzClockSettings

// ---------

// Class to track latency trends
// will lazily calculate running average on the correct thread
class FQuartLatencyTracker
{
public:
	ENGINE_API FQuartLatencyTracker();

	ENGINE_API void PushLatencyTrackerResult(const double& InResult);

	ENGINE_API float GetLifetimeAverageLatency();

	ENGINE_API float GetMinLatency();

	ENGINE_API float GetMaxLatency();

private:
	void PushSingleResult(const double& InResult);

	void DigestQueue();

	TQueue<float, EQueueMode::Mpsc> ResultQueue;

	int64 NumEntries{ 0 };

	float LifetimeAverage{ 0.f };

	float Min{ 0.0f };

	float Max{ 0.0f };
};

// NON-UOBJECT LAYER:
namespace Audio
{
	class FAudioMixer;

	// Utility class to set/get/convert tick rate
	// In this context "Tick Rate" refers to the duration of smallest temporal resolution we may care about
	// in musical time, this is locked to a 1/32nd note

	struct FQuartzClockTickRate
	{

	public:
		// ctor
		ENGINE_API FQuartzClockTickRate();

		// Setters
		ENGINE_API void SetFramesPerTick(int32 InNewFramesPerTick);

		ENGINE_API void SetMillisecondsPerTick(double InNewMillisecondsPerTick);

		ENGINE_API void SetSecondsPerTick(double InNewSecondsPerTick);

		ENGINE_API void SetThirtySecondNotesPerMinute(double InNewThirtySecondNotesPerMinute);

		ENGINE_API void SetBeatsPerMinute(double InNewBeatsPerMinute);

		ENGINE_API void SetSampleRate(double InNewSampleRate);

		// Getters
		double GetFramesPerTick() const { return FramesPerTick; }

		double GetMillisecondsPerTick() const { return MillisecondsPerTick; }

		double GetSecondsPerTick() const { return SecondsPerTick; }

		double GetThirtySecondNotesPerMinute() const { return ThirtySecondNotesPerMinute; }

		double GetBeatsPerMinute() const { return BeatsPerMinute; }

		double GetSampleRate() const { return SampleRate; }

		ENGINE_API double GetFramesPerDuration(EQuartzCommandQuantization InDuration) const;

		ENGINE_API double GetFramesPerDuration(EQuartzTimeSignatureQuantization InDuration) const;

		ENGINE_API bool IsValid(int32 InEventResolutionThreshold = 1) const;

		ENGINE_API bool IsSameTickRate(const FQuartzClockTickRate& Other, bool bAccountForDifferentSampleRates = true) const;



	private:
		// FramesPerTick is our ground truth 
		// update FramesPerTick and call RecalculateDurationsBasedOnFramesPerTick() to update other members
		double FramesPerTick{ 1.0 };
		double MillisecondsPerTick{ 1.0 };
		double SecondsPerTick{ 1.0 };
		double ThirtySecondNotesPerMinute{ 1.0 };
		double BeatsPerMinute{ 0.0 };
		double SampleRate{ 44100.0 };

		ENGINE_API void RecalculateDurationsBasedOnFramesPerTick();

	}; // class FAudioMixerClockTickRate

	// Simple class to track latency as a request/action propagates from GT to ART (or vice versa)
	class FQuartzLatencyTimer
	{
	public:
		// ctor
		ENGINE_API FQuartzLatencyTimer();

		// record the start time
		ENGINE_API void StartTimer();

		// reset the start time
		ENGINE_API void ResetTimer();

		// stop the timer
		ENGINE_API void StopTimer();

		// get the current value of a running timer
		ENGINE_API double GetCurrentTimePassedMs();

		// get the final time of a stopped timer
		ENGINE_API double GetResultsMilliseconds();

		// returns true if the Timer was started (could be running or stopped)
		ENGINE_API bool HasTimerStarted();

		// returns true if the timer has been run and stopped
		ENGINE_API bool HasTimerStopped();

		// returns true if the timer is running
		ENGINE_API bool IsTimerRunning();

		// returns true if the timer has completed (we can get the results)
		ENGINE_API bool HasTimerRun();

	private:
		int64 JourneyStartCycles;

		int64 JourneyEndCycles;
	};

	// class to track time a QuartzMessage takes to get from one thread to another
	class FQuartzCrossThreadMessage : public FQuartzLatencyTimer
	{
	public:
		ENGINE_API FQuartzCrossThreadMessage(bool bAutoStartTimer = true);

		ENGINE_API void RequestSent();

		ENGINE_API double RequestReceived() const;

		ENGINE_API double GetResultsMilliseconds() const;

		ENGINE_API double GetCurrentTimeMilliseconds() const;

	private:
		mutable FQuartzLatencyTimer Timer;
	};

	struct FQuartzOffset
	{
	public:
		// ctor
		ENGINE_API FQuartzOffset(double InOffsetInMilliseconds = 0.0);
		ENGINE_API FQuartzOffset(EQuartzCommandQuantization InDuration, double InMultiplier);

		// offset get/set
		ENGINE_API void SetOffsetInMilliseconds(double InMilliseconds);

		ENGINE_API void SetOffsetMusical(EQuartzCommandQuantization Duration, double Multiplier);

		bool IsSet() const { return IsSetAsMilliseconds() || IsSetAsMusicalDuration(); }

		ENGINE_API bool IsSetAsMilliseconds() const;

		ENGINE_API bool IsSetAsMusicalDuration() const;

		ENGINE_API int32 GetOffsetInAudioFrames(const FQuartzClockTickRate& InTickRate);

		ENGINE_API bool operator==(const FQuartzOffset& Other) const;

	private:
		// only one of these optionals will be valid at a time
		// (depending on which setter is called)
		TOptional<double> OffsetInMilliseconds;
		TOptional<TPair<EQuartzCommandQuantization, double>> OffsetAsDuration;

	};
	
	namespace Quartz
	{
		class IQueueCommandListener;
		class ICommandListener;
		class IMetronomeEventListener;

		using FQuartzSubscriberCommandQueue = Audio::Quartz::PrivateDefs::TQuartzCommandQueue<IMetronomeEventListener, ICommandListener, IQueueCommandListener>;
		using FQuartzGameThreadCommandQueuePtr = TSharedPtr<FQuartzSubscriberCommandQueue, ESPMode::ThreadSafe>;
	} // namespace Quartz

	struct FQuartzGameThreadSubscriber
	{
		FQuartzGameThreadSubscriber() = default;

		// this is only for back-compat until metronomes support FQuartzGameThreadSubscribers instead of raw queue ptrs
		// (for Metronome event offsets)
		FQuartzGameThreadSubscriber(const Quartz::FQuartzGameThreadCommandQueuePtr& InQueuePtr)
		: Queue(InQueuePtr)
		{ }


		// copy ctor
		FQuartzGameThreadSubscriber(const Quartz::FQuartzGameThreadCommandQueuePtr& InQueuePtr, FQuartzOffset InOffset)
		: Offset(InOffset)
		, Queue(InQueuePtr)
		{ }

		// change offset
		void SetOffset(FQuartzOffset InOffset) { Offset = MoveTemp(InOffset); }

		bool HasBeenNotifiedOfAboutToStart() const { return bHasBeenNotifiedOfAboutToStart; }

		// comparison
		ENGINE_API bool operator==(const FQuartzGameThreadSubscriber& Other) const;

		// notify
		ENGINE_API void PushEvent(const FQuartzQuantizedCommandDelegateData& Data);
		ENGINE_API void PushEvent(const FQuartzMetronomeDelegateData& Data);
		ENGINE_API void PushEvent(const FQuartzQueueCommandData& Data);

		// allow implicit casting to the underlying queue
		operator Quartz::FQuartzGameThreadCommandQueuePtr() const { return Queue; }

		// positive: anticipatory amount, negative:
		ENGINE_API int32 FinalizeOffset(const FQuartzClockTickRate& TickRate);
		ENGINE_API int32 GetOffsetAsAudioFrames() const;



	private:
		FQuartzOffset Offset;
		Quartz::FQuartzGameThreadCommandQueuePtr Queue;
		bool bOffsetConvertedToFrames = false;
		bool bHasBeenNotifiedOfAboutToStart = false;
		int32 OffsetInAudioFrames = 0;
	};
} // namespace Audio


// struct used to specify the quantization boundary of an event
USTRUCT(BlueprintType)
struct FQuartzQuantizationBoundary
{
	GENERATED_BODY()

	// resolution we are interested in
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Settings")
	EQuartzCommandQuantization Quantization{ EQuartzCommandQuantization::None };

	// how many "Resolutions" to wait before the onset we care about
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Settings", meta = (ClampMin = "1.0"))
	float Multiplier{ 1.0f };

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quantized Audio Clock Settings")
	EQuarztQuantizationReference CountingReferencePoint{ EQuarztQuantizationReference::BarRelative };

	// If this is true and the Clock hasn't started yet, the event will fire immediately when the Clock starts
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Quantized Audio Clock Settings")
	bool bFireOnClockStart{ true };

	// If this is true, this command will be canceled if the Clock is stopped or otherwise not running
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Quantized Audio Clock Settings")
	bool bCancelCommandIfClockIsNotRunning{ false };

	// If this is true, queueing the sound will also call a Reset Clock command
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Quantized Audio Clock Settings")
	bool bResetClockOnQueued{ false };

	// If this is true, queueing the sound will also call a Resume Clock command
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Quantized Audio Clock Settings")
	bool bResumeClockOnQueued{ false };

	// Game thread subscribers that will be passed to command init data (for C++ implementations)
	TArray<Audio::FQuartzGameThreadSubscriber> GameThreadSubscribers;

	// ctor
	FQuartzQuantizationBoundary(
		EQuartzCommandQuantization InQuantization = EQuartzCommandQuantization::None,
		float InMultiplier = 1.0f,
		EQuarztQuantizationReference InReferencePoint = EQuarztQuantizationReference::BarRelative,
		bool bInFireOnClockStart = true)
		: Quantization(InQuantization)
		, Multiplier(InMultiplier)
		, CountingReferencePoint(InReferencePoint)
		, bFireOnClockStart(bInFireOnClockStart)
	{}

	ENGINE_API FString ToString() const;
}; // struct FQuartzQuantizationBoundary


namespace Audio
{
	// data that is gathered by the AudioThread to get passed from FActiveSound->FMixerSourceVoice
	// eventually converted to IQuartzQuantizedCommand for the Quantized Command itself
	struct FQuartzQuantizedRequestData
	{
		// shared with FQuartzQuantizedCommandInitInfo:
		FName ClockName;
		FName OtherClockName;
		TSharedPtr<IQuartzQuantizedCommand> QuantizedCommandPtr;
		FQuartzQuantizationBoundary QuantizationBoundary{ EQuartzCommandQuantization::Tick, 1.f, EQuarztQuantizationReference::BarRelative, true };
		TArray<FQuartzGameThreadSubscriber> GameThreadSubscribers;
		int32 GameThreadDelegateID{ -1 };
	};


	// data that is passed into IQuartzQuantizedCommand::OnQueued
	// info that derived classes need can be added here
	struct FQuartzQuantizedCommandInitInfo
	{
		FQuartzQuantizedCommandInitInfo() = default;

		// conversion ctor from FQuartzQuantizedRequestData
		ENGINE_API FQuartzQuantizedCommandInitInfo(const FQuartzQuantizedRequestData& RHS, float InSampleRate, int32 InSourceID = INDEX_NONE);

		void SetOwningClockPtr(TSharedPtr<Audio::FQuartzClock> InClockPointer)
		{
			OwningClockPointer = InClockPointer;
			ensure(OwningClockPointer);
		}

		// shared with FQuartzQuantizedRequestData
		FName ClockName;
		FName OtherClockName;
		TSharedPtr<IQuartzQuantizedCommand> QuantizedCommandPtr{ nullptr };
		FQuartzQuantizationBoundary QuantizationBoundary;
		TArray<FQuartzGameThreadSubscriber> GameThreadSubscribers;
		int32 GameThreadDelegateID{ -1 };

		// Audio Render thread-specific data:
		TSharedPtr<Audio::FQuartzClock> OwningClockPointer{ nullptr };
		float SampleRate{0};
		int32 SourceID{ -1 };

		// Number of frames used for any FramesTilExec overrides
		int32 FrameOverrideAmount{ 0 };
	};

	// base class for quantized commands. Virtual methods called by owning clock.
	class IQuartzQuantizedCommand : public FQuartzCrossThreadMessage
	{
	public:

		// ctor
		ENGINE_API IQuartzQuantizedCommand();

		// dtor
		ENGINE_API virtual ~IQuartzQuantizedCommand();

		// allocate a copy of the derived class
		ENGINE_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const;

		ENGINE_API void AddSubscriber(FQuartzGameThreadSubscriber InSubscriber);

		// Command has reached the AudioRenderThread
		ENGINE_API void OnQueued(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo);

		// scheduled (finalize subscriber offsets) - called by FQuartzClock
		ENGINE_API void OnScheduled(const FQuartzClockTickRate& InTickRate);

		// called during FQuartzClock::Tick() to let us call AboutToStart
		// at different times for different subscribers
		ENGINE_API void Update(int32 NumFramesUntilDeadline);

		// Perhaps the associated sound failed concurrency and will not be playing
		ENGINE_API void FailedToQueue(FQuartzQuantizedRequestData& InGameThreadData);

		// Called 2x Assumed thread latency before OnFinalCallback()
		ENGINE_API void AboutToStart();

		// Called on the final callback of this event boundary.
		// InNumFramesLeft is the number of frames into the callback the exact quantized event should take place
		ENGINE_API void OnFinalCallback(int32 InNumFramesLeft);

		// Called if the owning clock gets stopped
		ENGINE_API void OnClockPaused();

		// Called if the owning clock gets started
		ENGINE_API void OnClockStarted();

		// Called if the event is cancelled before OnFinalCallback() is called
		ENGINE_API void Cancel();


		//Called if the event type uses an altered amount of frames
		virtual int32 OverrideFramesUntilExec(int32 NumFramesUntilExec) { return NumFramesUntilExec; }


		virtual bool IsClockAltering() { return false; }
		virtual bool ShouldDeadlineIgnoresBpmChanges() { return false; }
		virtual bool RequiresAudioDevice() const { return false; }

		virtual FName GetCommandName() const = 0;
		virtual EQuartzCommandType GetCommandType() const = 0;


	private:
		// derived classes can override these to add extra functionality
		virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) {}
		virtual void FailedToQueueCustom() {}
		virtual void AboutToStartCustom() {}
		virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) {}
		virtual void OnClockPausedCustom() {}
		virtual void OnClockStartedCustom() {}
		virtual void CancelCustom() {}

		TArray<FQuartzGameThreadSubscriber> GameThreadSubscribers;

		int32 GameThreadDelegateID{ -1 };
	}; // class IAudioMixerQuantizedCommandBase

	// Audio Render Thread Handle to a queued command
	// Used by AudioMixerSourceVoices to access a pending associated command
	struct FQuartzQuantizedCommandHandle
	{
		FName OwningClockName;
		TSharedPtr<IQuartzQuantizedCommand> CommandPtr{ nullptr };
		FMixerDevice* MixerDevice{ nullptr };

		// Attempts to cancel the command. Returns true if the cancellation was successful.
		ENGINE_API bool Cancel();

		// Resets the handle to initial state.
		ENGINE_API void Reset();
	};
} // namespace Audio

struct FAudioComponentCommandInfo
{
	FAudioComponentCommandInfo() {}

 	FAudioComponentCommandInfo(Audio::FQuartzGameThreadSubscriber InSubscriber, FQuartzQuantizationBoundary InAnticaptoryBoundary)
		: Subscriber(InSubscriber)
		, AnticipatoryBoundary(InAnticaptoryBoundary)
	{
		static uint32 CommandIDs = 0;
		CommandID = CommandIDs++;
	}

	Audio::FQuartzGameThreadSubscriber Subscriber;
	FQuartzQuantizationBoundary AnticipatoryBoundary;
	uint32 CommandID{ (uint32)INDEX_NONE };
};
