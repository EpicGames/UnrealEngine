// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/QuartzSubscription.h"

#include "Quartz/QuartzSubsystem.h"
#include "HAL/IConsoleManager.h"


static int32 DecrementSlotIndexOnStartedCvar = 1;
FAutoConsoleVariableRef CVarDecrementSlotIndexOnStarted(
	TEXT("au.Quartz.DecrementSlotIndexOnStarted"),
	DecrementSlotIndexOnStartedCvar,
	TEXT("Defaults to 1 to enable the delegate leak fix.  Set to 0 to revert to pre-fix behavior.\n")
	TEXT("1: New Behavior, 0: Old Behavior"),
	ECVF_Default);

FQuartzTickableObject::FQuartzTickableObject()
{}

FQuartzTickableObject::~FQuartzTickableObject()
{
	QuartzUnsubscribe();
}

FQuartzTickableObject* FQuartzTickableObject::Init(UWorld* InWorldPtr)
{
	if (!ensure(InWorldPtr))
	{
		// can't initialize if we don't have a valid world
		return this;
	}

	if(!CommandQueuePtr.IsValid())
	{
		CommandQueuePtr = MakeShared<FQuartzSubscriberCommandQueue>();
	}

	UQuartzSubsystem* QuartzSubsystemPtr = UQuartzSubsystem::Get(InWorldPtr);
	QuartzSubscriptionToken.Subscribe(this, QuartzSubsystemPtr);

	return this;
}

void FQuartzTickableObject::QuartzUnsubscribe()
{
	QuartzSubscriptionToken.Unsubscribe();
}

int32 FQuartzTickableObject::AddCommandDelegate(const FOnQuartzCommandEventBP& InDelegate)
{
	const int32 Num = QuantizedCommandDelegates.Num();
	int32 SlotId = 0;

	for (; SlotId < Num; ++SlotId)
	{
		if (!QuantizedCommandDelegates[SlotId].MulticastDelegate.IsBound())
		{
			QuantizedCommandDelegates[SlotId].MulticastDelegate.AddUnique(InDelegate);
			return SlotId;
		}
	}

	// need a new slot
	QuantizedCommandDelegates.AddDefaulted_GetRef().MulticastDelegate.AddUnique(InDelegate);
	return SlotId;
}

void FQuartzTickableObject::OnCommandEvent(const Audio::FQuartzQuantizedCommandDelegateData& Data)
{
	checkSlow(Data.DelegateSubType < EQuartzCommandDelegateSubType::Count);

	if(const TSharedPtr<FQuartzTickableObjectsManager> ObjManagerPtr = QuartzSubscriptionToken.GetTickableObjectManager())
	{
		ObjManagerPtr->PushLatencyTrackerResult(Data.RequestReceived());
	}

	// Broadcast to the BP delegate if we have one bound
	if (Data.DelegateID >= 0 && Data.DelegateID < QuantizedCommandDelegates.Num()
		&& QuantizedCommandDelegates[Data.DelegateID].MulticastDelegate.IsBound())
	{
		FCommandDelegateGameThreadData& GameThreadEntry = QuantizedCommandDelegates[Data.DelegateID];

		GameThreadEntry.MulticastDelegate.Broadcast(Data.DelegateSubType, "Quartz Event");

		// track the number of active QuantizedCommands that may be sending info back to us.
		// this is a bit of a hack because sound cues can play multiple wave instances
		// and each of those wave instances is sending a delegate back to us.
		// todo: clean this up at the wave-instance level to avoid ref counting here.
		// (new command)
		if (Data.DelegateSubType == EQuartzCommandDelegateSubType::CommandOnQueued)
		{
			GameThreadEntry.RefCount.Increment();
		}

		// (end of a command)
		bool bShouldDecrement = Data.DelegateSubType == EQuartzCommandDelegateSubType::CommandOnCanceled;
		bShouldDecrement |= (DecrementSlotIndexOnStartedCvar && Data.DelegateSubType == EQuartzCommandDelegateSubType::CommandOnStarted);

		// are all the commands for this delegate done?
		if (bShouldDecrement && (GameThreadEntry.RefCount.Decrement() <= 0))
		{
			// free up the slot for new subscriptions on this clock handle
			GameThreadEntry.MulticastDelegate.Clear();
			GameThreadEntry.RefCount.Reset();
		}
	}

	// call base-class method
	ProcessCommand(Data);
}

void FQuartzTickableObject::OnMetronomeEvent(const Audio::FQuartzMetronomeDelegateData& Data)
{
	if (const TSharedPtr<FQuartzTickableObjectsManager> ObjManagerPtr = QuartzSubscriptionToken.GetTickableObjectManager())
	{
		ObjManagerPtr->PushLatencyTrackerResult(Data.RequestReceived());
	}

	MetronomeDelegates[static_cast<int32>(Data.Quantization)].MulticastDelegate
		.Broadcast(Data.ClockName, Data.Quantization, Data.Bar, Data.Beat, Data.BeatFraction);

	// call base-class method
	ProcessCommand(Data);
}

void FQuartzTickableObject::OnQueueCommandEvent(const Audio::FQuartzQueueCommandData& Data)
{
	// call base-class method
	ProcessCommand(Data);
}

void FQuartzTickableObject::SetNotificationAnticipationAmountMilliseconds(const double Milliseconds)
{
	// todo: update metronome subscriptions w/ new value (once metronome observes offsets)
	NotificationOffset.SetOffsetInMilliseconds(Milliseconds);
}

void FQuartzTickableObject::SetNotificationAnticipationAmountMusicalDuration(const EQuartzCommandQuantization Duration,
	const double Multiplier)
{
	NotificationOffset.SetOffsetMusical(Duration, Multiplier);
}

Audio::FQuartzGameThreadSubscriber FQuartzTickableObject::GetQuartzSubscriber()
 {
 	if (!CommandQueuePtr.IsValid())
 	{
 		CommandQueuePtr = MakeShared<FQuartzSubscriberCommandQueue, ESPMode::ThreadSafe>();
 	}

 	return { CommandQueuePtr, NotificationOffset };
}

void FQuartzTickableObject::QuartzTick(float DeltaTime)
{
	CommandQueuePtr->PumpCommandQueue(*this);
	
	if (ShouldUnsubscribe())
	{
		QuartzUnsubscribe();	
	}
}

bool FQuartzTickableObject::QuartzIsTickable() const
{
	return CommandQueuePtr.IsValid();
}

void FQuartzTickableObject::AddMetronomeBpDelegate(EQuartzCommandQuantization InQuantizationBoundary, const FOnQuartzMetronomeEventBP& OnQuantizationEvent)
{
	MetronomeDelegates[static_cast<int32>(InQuantizationBoundary)].MulticastDelegate.AddUnique(OnQuantizationEvent);
}

namespace Audio
{

	FQuartzOffset::FQuartzOffset(double InOffsetInMilliseconds)
	: OffsetInMilliseconds(MoveTemp(InOffsetInMilliseconds))
	{
	}

	FQuartzOffset::FQuartzOffset(EQuartzCommandQuantization InDuration, double InMultiplier)
	: OffsetAsDuration(TPair<EQuartzCommandQuantization, double>(MoveTemp(InDuration), MoveTemp(InMultiplier)))
	{
	}

	void FQuartzOffset::SetOffsetInMilliseconds(double InMilliseconds)
	{
		OffsetAsDuration.Reset();
		OffsetInMilliseconds.Emplace(InMilliseconds);
	}

	void FQuartzOffset::SetOffsetMusical(EQuartzCommandQuantization Duration, double Multiplier)
	{
		OffsetInMilliseconds.Reset();
		OffsetAsDuration.Emplace(TPair<EQuartzCommandQuantization, float>(Duration, Multiplier));
	}

	bool FQuartzOffset::IsSetAsMilliseconds() const
	{
		return OffsetInMilliseconds.IsSet();
	}

	bool FQuartzOffset::IsSetAsMusicalDuration() const
	{
		return OffsetAsDuration.IsSet();
	}

	int32 FQuartzOffset::GetOffsetInAudioFrames(const FQuartzClockTickRate& InTickRate)
	{
		// only one should be set at a time: updated by last SetOffset[In/As][Milliseconds/Duration]()
		check(!(OffsetInMilliseconds.IsSet() && OffsetAsDuration.IsSet()));

		if(OffsetInMilliseconds.IsSet())
		{
			const double OffsetInSeconds = OffsetInMilliseconds.GetValue() * 1000.0;
			const int32 OffsetInFrames = OffsetInSeconds * InTickRate.GetSampleRate();

			return OffsetInFrames;
		}
		else if(OffsetAsDuration.IsSet())
		{
			auto [QuantizationType, Multiplier] = OffsetAsDuration.GetValue();

			if (QuantizationType == EQuartzCommandQuantization::None)
			{
				return 0;
			}

			return static_cast<int32>(Multiplier * InTickRate.GetFramesPerDuration(QuantizationType));
		}

		ensure(false); // one of our durations should have been set (even if by a constructor)
		return 0;
	}

} // namespace Audio
