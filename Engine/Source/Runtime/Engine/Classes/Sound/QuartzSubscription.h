// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuartzSubscriptionToken.h"
#include "Sound/QuartzInterfaces.h"
#include "Sound/QuartzCommandQueue.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Containers/ConsumeAllMpmcQueue.h"

#define UE_API ENGINE_API

// forwards
class UQuartzSubsystem;
class UQuartzClockHandle;
class FQuartzTickableObject;
struct FQuartzTickableObjectsManager;

namespace Audio
{
	struct FQuartzGameThreadSubscriber;
	class FQuartzClockProxy;

	// old version of TQuartzCommandQueue
	template<class ListenerType>
	class UE_DEPRECATED(5.5, "Message") TQuartzShareableCommandQueue
	{
	};
} // namespace Audio

/**
 *	FQuartzTickableObject
 *
 *		This is the base class for non-Audio Render Thread objects that want to receive
 *		callbacks for Quartz events.
 *
 *		It is a wrapper around TQuartzShareableCommandQueue.
 *		(see UQuartzClockHandle or UAudioComponent as implementation examples)
 */
using namespace Audio::Quartz;

// TODO: comment up why we listen to these interfaces
class FQuartzTickableObject
	: public FQuartzSubscriberCommandQueue::TConsumerBase<
		  IMetronomeEventListener
		, ICommandListener
		, IQueueCommandListener
>
{
public:
	// ctor
	UE_API FQuartzTickableObject();

    FQuartzTickableObject(const FQuartzTickableObject& Other) = default;
    FQuartzTickableObject& operator=(const FQuartzTickableObject&) = default;

	// dtor
	UE_API virtual ~FQuartzTickableObject() override;

	UE_API FQuartzTickableObject* Init(UWorld* InWorldPtr);

	// called by the associated QuartzSubsystem
	UE_API void QuartzTick(float DeltaTime);

	UE_API bool QuartzIsTickable() const;

	UE_API void AddMetronomeBpDelegate(EQuartzCommandQuantization InQuantizationBoundary, const FOnQuartzMetronomeEventBP& OnQuantizationEvent);

	bool IsInitialized() const { return QuartzSubscriptionToken.IsSubscribed(); }

	UE_API Audio::FQuartzGameThreadSubscriber GetQuartzSubscriber();

	UE_API int32 AddCommandDelegate(const FOnQuartzCommandEventBP& InDelegate);

	UE_DEPRECATED(5.5, "This should not be called directly, use the ICommandListener interface instead.")
	void ExecCommand(const Audio::FQuartzQuantizedCommandDelegateData& Data) { OnCommandEvent(Data); }
	
	UE_DEPRECATED(5.5, "This should not be called directly, use the IMetronomeEventListener interface instead.")
	void ExecCommand(const Audio::FQuartzMetronomeDelegateData& Data) { OnMetronomeEvent(Data); }
	
	UE_DEPRECATED(5.5, "This should not be called directly, use the IQueueCommandListener interface instead.")
	void ExecCommand(const Audio::FQuartzQueueCommandData& Data) { OnQueueCommandEvent(Data);}

	// required by TQuartzShareableCommandQueue template
	UE_API virtual void OnCommandEvent(const Audio::FQuartzQuantizedCommandDelegateData& Data) override;
	UE_API virtual void OnMetronomeEvent(const Audio::FQuartzMetronomeDelegateData& Data) override;
	UE_API virtual void OnQueueCommandEvent(const Audio::FQuartzQueueCommandData& Data) override;

	// virtual interface (ExecCommand will forward the data to derived classes' ProcessCommand() call)
	virtual void ProcessCommand(const Audio::FQuartzQuantizedCommandDelegateData& Data) {}
	virtual void ProcessCommand(const Audio::FQuartzMetronomeDelegateData& Data) {}
	virtual void ProcessCommand(const Audio::FQuartzQueueCommandData& Data) {}
	

	const Audio::FQuartzOffset& GetQuartzOffset() const { return NotificationOffset; }

protected:
	UE_API void SetNotificationAnticipationAmountMilliseconds(const double Milliseconds);
	UE_API void SetNotificationAnticipationAmountMusicalDuration(const EQuartzCommandQuantization Duration,  const double Multiplier);

	UE_API void QuartzUnsubscribe();
	virtual bool ShouldUnsubscribe() { return false; } 

private:
	struct FMetronomeDelegateGameThreadData
	{
		FOnQuartzMetronomeEvent MulticastDelegate;
	};

	struct FCommandDelegateGameThreadData
	{
		FOnQuartzCommandEvent MulticastDelegate;
		FThreadSafeCounter RefCount;
	};

	// delegate containers
	FMetronomeDelegateGameThreadData MetronomeDelegates[static_cast<int32>(EQuartzCommandQuantization::Count)];
	TArray<FCommandDelegateGameThreadData> QuantizedCommandDelegates;

	TArray<TFunction<void(FQuartzTickableObject*)>> TempCommandQueue;

private:
	Audio::FQuartzOffset NotificationOffset;
	FQuartzGameThreadCommandQueuePtr CommandQueuePtr;
	FQuartzSubscriptionToken QuartzSubscriptionToken;
}; // class FQuartzTickableObject

#undef UE_API
