// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuartzSubscriptionToken.h"

#include "Quartz/QuartzSubsystem.h"
#include "Sound/QuartzQuantizationUtilities.h"

FQuartzSubscriptionToken::FQuartzSubscriptionToken()
	: SubscribingObject(nullptr)
	{}

FQuartzSubscriptionToken::~FQuartzSubscriptionToken()
{
	Unsubscribe();
}

void FQuartzSubscriptionToken::Subscribe(FQuartzTickableObject* Subscriber, UQuartzSubsystem* QuartzSubsystem)
{
	if(IsSubscribed())
	{
		return;
	}

	if(ensure(Subscriber))
	{
		SubscribingObject = Subscriber;
	}
	else
	{
		UE_LOGF(LogAudioQuartz, Error, "Invalid subscribing object.");
		return;
	}
	
	if(ensure(QuartzSubsystem))
	{
		TickableObjectManagerPtr = QuartzSubsystem->GetTickableObjectManager();
	}
	else
	{
		UE_LOGF(LogAudioQuartz, Error, "Unable to obtain Tickable Object Manager.");
		return;
	}
	
	TSharedPtr<FQuartzTickableObjectsManager> ObjManagerPtr = GetTickableObjectManager();
	if(ObjManagerPtr.IsValid())
	{
		ObjManagerPtr->SubscribeToQuartzTick(SubscribingObject);
	}
	else
	{
		UE_LOGF(LogAudioQuartz, Warning, "Unable to subscribe to Quartz tick. Tickable Object Manager pointer is not valid.");
	}
}

void FQuartzSubscriptionToken::Unsubscribe()
{
	const TSharedPtr<FQuartzTickableObjectsManager> ObjManagerPtr = GetTickableObjectManager();
	if(ObjManagerPtr.IsValid())
	{
		ObjManagerPtr->UnsubscribeFromQuartzTick(SubscribingObject);
	}

	SubscribingObject = nullptr;
	TickableObjectManagerPtr = nullptr;
}

TSharedPtr<FQuartzTickableObjectsManager> FQuartzSubscriptionToken::GetTickableObjectManager()
{
	TSharedPtr<FQuartzTickableObjectsManager> ObjManagerPtr = TickableObjectManagerPtr.Pin();
	return ObjManagerPtr;
}

bool FQuartzSubscriptionToken::IsSubscribed() const
{
	return TickableObjectManagerPtr.IsValid();
}