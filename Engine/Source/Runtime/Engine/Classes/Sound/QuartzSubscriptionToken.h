// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"

class UQuartzSubsystem;
class FQuartzTickableObject;
struct FQuartzTickableObjectsManager;

class FQuartzSubscriptionToken
{
public:
	ENGINE_API FQuartzSubscriptionToken();
	ENGINE_API ~FQuartzSubscriptionToken();
	
	ENGINE_API void Subscribe(FQuartzTickableObject* Subscriber, UQuartzSubsystem* QuartzSubsystem);
	ENGINE_API void Unsubscribe();

	ENGINE_API TSharedPtr<FQuartzTickableObjectsManager> GetTickableObjectManager();
	ENGINE_API bool IsSubscribed() const;
	
private:
	FQuartzTickableObject* SubscribingObject;
	TWeakPtr<FQuartzTickableObjectsManager> TickableObjectManagerPtr;
};
