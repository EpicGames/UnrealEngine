// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ReverbEffect.generated.h"

UCLASS(BlueprintType, hidecategories=object, MinimalAPI)
class UReverbEffect : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Bypasses early reflections */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EarlyReflections)
	bool bBypassEarlyReflections;

	/** Reflections Delay - 0.0 < 0.007 < 0.3 Seconds - the time between the listener receiving the direct path sound and the first reflection */
	UPROPERTY(Category=EarlyReflections, meta=(ClampMin = "0.0", ClampMax = "0.3", EditCondition = "!bBypassEarlyReflections"), EditAnywhere)
	float		ReflectionsDelay;

	/** Reverb Gain High Frequency - 0.0 < 0.89 < 1.0 - attenuates the high frequency reflected sound */
	UPROPERTY(Category=EarlyReflections, meta=(ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypassEarlyReflections"), EditAnywhere)
	float		GainHF;

	/** Reflections Gain - 0.0 < 0.05 < 3.16 - controls the amount of initial reflections */
	UPROPERTY(Category=EarlyReflections, meta=(ClampMin = "0.0", ClampMax = "3.16", EditCondition = "!bBypassEarlyReflections"), EditAnywhere)
	float		ReflectionsGain;


	/** Bypasses late reflections. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections)
	bool bBypassLateReflections;

	/** Late Reverb Delay - 0.0 < 0.011 < 0.1 Seconds - time difference between late reverb and first reflections */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "0.1", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		LateDelay;

	/** Decay Time - 0.1 < 1.49 < 20.0 Seconds - larger is more reverb */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.1", ClampMax = "20.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		DecayTime;

	/** Density - 0.0 < 1.0 < 1.0 - Coloration of the late reverb - lower value is more grainy */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		Density;

	/** Diffusion - 0.0 < 1.0 < 1.0 - Echo density in the reverberation decay - lower is more grainy */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		Diffusion;

	/** Air Absorption - 0.0 < 0.994 < 1.0 - lower value means more absorption */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		AirAbsorptionGainHF;

	/** Decay High Frequency Ratio - 0.1 < 0.83 < 2.0 - how much quicker or slower the high frequencies decay relative to the lower frequencies. */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.1", ClampMax = "2.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		DecayHFRatio;

	/** Late Reverb Gain - 0.0 < 1.26 < 10.0 - gain of the late reverb */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "10.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		LateGain;

	/** Reverb Gain - 0.0 < 0.32 < 1.0 - overall reverb gain - master volume control */
	UPROPERTY(Category=LateReflections, meta=(ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypassLateReflections"), EditAnywhere)
	float		Gain;

	/** Room Rolloff - 0.0 < 0.0 < 10.0 - multiplies the attenuation due to distance */
	UE_DEPRECATED("4.26", "RoomRolloffFactor is no longer used")
	UPROPERTY(meta=(ClampMin = "0.0", ClampMax = "10.0"))
	float		RoomRolloffFactor;

#if WITH_EDITORONLY_DATA
	/** Transient property used to trigger real-time updates of the reverb for real-time editor previewing */
	UPROPERTY(transient)
	uint32 bChanged : 1;
#endif

protected:

#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
