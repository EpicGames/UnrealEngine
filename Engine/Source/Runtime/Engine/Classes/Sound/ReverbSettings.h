// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "ReverbSettings.generated.h"

class UReverbEffect;
class USoundEffectSubmixPreset;


/**
 * DEPRECATED: Exists for backwards compatibility
 * Indicates a reverb preset to use.
 *
 */
UENUM()
enum ReverbPreset : int
{
	REVERB_Default,
	REVERB_Bathroom,
	REVERB_StoneRoom,
	REVERB_Auditorium,
	REVERB_ConcertHall,
	REVERB_Cave,
	REVERB_Hallway,
	REVERB_StoneCorridor,
	REVERB_Alley,
	REVERB_Forest,
	REVERB_City,
	REVERB_Mountains,
	REVERB_Quarry,
	REVERB_Plain,
	REVERB_ParkingLot,
	REVERB_SewerPipe,
	REVERB_Underwater,
	REVERB_SmallRoom,
	REVERB_MediumRoom,
	REVERB_LargeRoom,
	REVERB_MediumHall,
	REVERB_LargeHall,
	REVERB_Plate,
	REVERB_MAX,
};

/** Struct encapsulating settings for reverb effects. */
USTRUCT(BlueprintType)
struct FReverbSettings
{
	GENERATED_USTRUCT_BODY()

	/* Whether to apply the reverb settings below. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReverbSettings )
	bool bApplyReverb;

#if WITH_EDITORONLY_DATA
	/** The reverb preset to employ. */
	UPROPERTY()
	TEnumAsByte<ReverbPreset> ReverbType_DEPRECATED;
#endif

	/** The reverb asset to employ. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReverbSettings)
	TObjectPtr<UReverbEffect> ReverbEffect;

	/** This is used to apply plugin-specific settings when a Reverb Plugin is being used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbSettings)
	TObjectPtr<USoundEffectSubmixPreset> ReverbPluginEffect;

	/** Volume level of the reverb effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbSettings)
	float Volume;

	/** Time to fade from the current reverb settings into this setting, in seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ReverbSettings)
	float FadeTime;

	FReverbSettings()
		: bApplyReverb(true)
#if WITH_EDITORONLY_DATA
		, ReverbType_DEPRECATED(REVERB_Default)
#endif
		, ReverbEffect(nullptr)
		, ReverbPluginEffect(nullptr)
		, Volume(0.5f)
		, FadeTime(2.0f)
	{
	}

	ENGINE_API bool operator==(const FReverbSettings& Other) const;
	bool operator!=(const FReverbSettings& Other) const { return !(*this == Other); }

#if WITH_EDITORONLY_DATA
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif // WITH_EDITORONLY_DATA
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FReverbSettings> : public TStructOpsTypeTraitsBase2<FReverbSettings>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
#endif // WITH_EDITORONLY_DATA

