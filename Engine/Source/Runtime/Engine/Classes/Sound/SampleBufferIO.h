// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SampleBuffer.h"
#include "Misc/Paths.h"
#include "Sound/SoundEffectBase.h"
#include "Async/AsyncWork.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/GCObject.h"
#include "Tickable.h"

class USoundWave;
class FAudioDevice;

namespace Audio
{
	/************************************************************************/
	/* FSoundWavePCMLoader                                                  */
	/* This class loads and decodes a USoundWave asset into a TSampleBuffer.*/
	/* To use, call LoadSoundWave with the sound wave you'd like to load    */
	/* and call Update on every tick until it returns true, at which point  */
	/* you may call GetSampleBuffer to get the decoded audio.               */
	/************************************************************************/
	class FSoundWavePCMLoader : public FGCObject
	{
	public:
		ENGINE_API FSoundWavePCMLoader();

		// Loads a USoundWave, call on game thread. Unless called with bSynchnous set to true, this class will require Update() to be called on the game thread.
		ENGINE_API void LoadSoundWave(USoundWave* InSoundWave, TFunction<void(const USoundWave* SoundWave, const Audio::FSampleBuffer& OutSampleBuffer)> OnLoaded, bool bSynchrounous = false);


		// Update the loading state, should be called on the game thread. 
		ENGINE_API void Update();

		//~ GCObject Interface
		ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		ENGINE_API virtual FString GetReferencerName() const override;
		//~ GCObject Interface

	private:

		struct FLoadingSoundWaveInfo
		{
			// The sound wave which is loading PCM data
			TObjectPtr<USoundWave> SoundWave;

			// The lambda function to call when the sound wave finishes loading
			TFunction<void(const USoundWave* SoundWave, const Audio::FSampleBuffer& LoadedSampleBuffer)> OnLoaded;


		
			enum class LoadStatus : uint8
			{
				// No request to load has been issued (default)
				None = 0,

				// The sound wave load/decode is in-flight
				Loading,

				// The sound wave has already been loaded
				Loaded,
			};

			LoadStatus Status;

			FLoadingSoundWaveInfo()
				: SoundWave(nullptr)
				, Status(LoadStatus::None)
			{
			}
		};

		// Reference to current loading sound wave
		TArray<FLoadingSoundWaveInfo> LoadingSoundWaves;
		
		// MERGE-REVIEW - should be in object or moved into loading info?
		// Whether or not this object is tickable. I.e. a sound wave has been asked to load.
		bool bCanBeTicked;
	};

	// Enum used to express the current state of a FSoundWavePCMWriter's current operation.
	enum class ESoundWavePCMWriterState : uint8
	{
		Idle,
		Generating,
		WritingToDisk,
		Suceeded,
		Failed,
		Cancelled
	};

	// Enum used internally by the FSoundWavePCMWriter.
	enum class ESoundWavePCMWriteTaskType : uint8
	{
		GenerateSoundWave,
		GenerateAndWriteSoundWave,
		WriteSoundWave,
		WriteWavFile
	};

	/************************************************************************/
	/* FAsyncSoundWavePCMWriteWorker                                        */
	/* This class is used by FSoundWavePCMWriter to handle async writing.   */
	/************************************************************************/
	class FAsyncSoundWavePCMWriteWorker : public FNonAbandonableTask
	{
	protected:
		class FSoundWavePCMWriter* Writer;
		ESoundWavePCMWriteTaskType TaskType;

		FCriticalSection NonAbandonableSection;

		TFunction<void(const USoundWave*)> CallbackOnSuccess;

	public:
		
		ENGINE_API FAsyncSoundWavePCMWriteWorker(FSoundWavePCMWriter* InWriter, ESoundWavePCMWriteTaskType InTaskType, TFunction<void(const USoundWave*)> OnSuccess);
		ENGINE_API ~FAsyncSoundWavePCMWriteWorker();

		/**
		* Performs write operations async.
		*/
		ENGINE_API void DoWork();

		bool CanAbandon() 
		{ 
			return true;
		}

		ENGINE_API void Abandon();

		inline TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncSoundWavePCMWriteWorker, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	typedef FAsyncTask<FAsyncSoundWavePCMWriteWorker> FAsyncSoundWavePCMWriterTask;

	// This is the default chunk size, in bytes that FSoundWavePCMWriter writes to the disk at once.
	static const int32 WriterDefaultChunkSize = 8192;


	/************************************************************************/
	/* FSoundWavePCMWriter                                                  */
	/* This class can be used to save a TSampleBuffer to either a wav  file */
	/* or a USoundWave using BeginGeneratingSoundWaveFromBuffer,            */
	/* BeginWriteToSoundWave, or BeginWriteToWavFile on the game thread.    */
	/* This class uses an async task to generate and write the file to disk.*/
	/************************************************************************/
	class FSoundWavePCMWriter
	{
	public:
		friend class FAsyncSoundWavePCMWriteWorker;

		ENGINE_API FSoundWavePCMWriter(int32 InChunkSize = WriterDefaultChunkSize);
		ENGINE_API ~FSoundWavePCMWriter();

		// This kicks off an operation to write InSampleBuffer to SoundWaveToSaveTo.
		// If InSoundWave is not nullptr, the audio will be written directly into
		// Returns true on a successful start, false otherwise.
		ENGINE_API bool BeginGeneratingSoundWaveFromBuffer(const TSampleBuffer<>& InSampleBuffer, USoundWave* InSoundWave = nullptr, TFunction<void(const USoundWave*)> OnSuccess = [](const USoundWave* ResultingWave){});

		// This kicks off an operation to write InSampleBuffer to a USoundWave asset
		// at the specified file path relative to the project directory.
		// This function should only be used in the editor.
		// If a USoundWave asset already exists 
		// InSoundWaveTransformationTarget is used to overwrite SoundWave data with its own transformations. (ex. marker data)
		ENGINE_API bool BeginWriteToSoundWave(const FString& FileName, const TSampleBuffer<>& InSampleBuffer, FString InPath, TFunction<void(const USoundWave*)> OnSuccess = [](const USoundWave* ResultingWave) {}, TObjectPtr<USoundWave> InSoundWaveTransformationTarget = nullptr);
	
		// This writes out the InSampleBuffer as a wav file at the path specified by FilePath and FileName.
		// If FilePath is a relative path, it will be relative to the /Saved/BouncedWavFiles folder, otherwise specified absolute path will be used.
		// FileName should not contain the extension. This can be used in non-editor builds.
		ENGINE_API bool BeginWriteToWavFile(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath, TFunction<void()> OnSuccess = []() {});

		// This is a blocking call that will return the SoundWave generated from InSampleBuffer.
		// Optionally, if you're using the editor, you can also write the resulting soundwave out to the content browser using the FileName and FilePath parameters.
		// InSoundWaveTransformationTarget is used to overwrite SoundWave data with its own transformations. (ex. marker data)
		ENGINE_API USoundWave* SynchronouslyWriteSoundWave(const TSampleBuffer<>& InSampleBuffer, const FString* FileName = nullptr, const FString* FilePath = nullptr, TObjectPtr<USoundWave> InSoundWaveTransformationTarget = nullptr);

		// This blocking call writes out the InSampleBuffer as a wav file at the path specified by FilePath and FileName.
		// If FilePath is a relative path, it will be relative to the /Saved/BouncedWavFiles folder, otherwise specified absolute path will be used.
		// FileName should not contain the extension. This can be used in non-editor builds.
		ENGINE_API bool SynchronouslyWriteToWavFile(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath, FString* OutFilePathName = nullptr);

		// Call this on the game thread to continue the write operation. Optionally provide a pointer
		// to an ESoundWavePCMWriterState which will be written to with the current state of the write operation.
		// Returns a float value from 0 to 1 indicating how complete the write operation is.
		ENGINE_API float CheckStatus(ESoundWavePCMWriterState* OutCurrentState = nullptr);

		// Aborts the current write operation.
		ENGINE_API void CancelWrite();

		// Whether we have finished the write operation, by either succeeding, failing, or being cancelled.
		ENGINE_API bool IsDone();

		// Clean up all resources used.
		ENGINE_API void Reset();

		// Used to grab the a handle to the soundwave. 
		ENGINE_API USoundWave* GetFinishedSoundWave();

		// This function can be used after generating a USoundWave by calling BeginGeneratingSoundWaveFromBuffer
		// to save the generated soundwave to an asset.
		// This is handy if you'd like to preview or edit the USoundWave before saving it to disk.
		ENGINE_API void SaveFinishedSoundWaveToPath(const FString& FileName, FString InPath = FPaths::EngineContentDir());

	private:
		// Current pending buffer.
		TSampleBuffer<> CurrentBuffer;

		// Sound wave currently being written to.
		USoundWave* CurrentSoundWave;

		// Current state of the buffer.
		ESoundWavePCMWriterState CurrentState;

		// Current Absolute File Path we are writing to.
		FString AbsoluteFilePath;

		bool bWasPreviouslyAddedToRoot;

		TUniquePtr<FAsyncSoundWavePCMWriterTask> CurrentOperation;

		// Internal buffer for holding the serialized wav file in memory.
		TArray<uint8> SerializedWavData;

		// Internal progress
		FThreadSafeCounter Progress;

		int32 ChunkSize;

		UPackage* CurrentPackage;

	private:

		//  This is used to emplace CurrentBuffer in CurrentSoundWave.
		ENGINE_API void ApplyBufferToSoundWave();

		// This is used to save CurrentSoundWave within CurrentPackage.
		ENGINE_API void SerializeSoundWaveToAsset();

		// This is used to write a WavFile in disk.
		ENGINE_API void SerializeBufferToWavFile();

		// This checks to see if a directory exists and, if it does not, recursively adds the directory.
		ENGINE_API bool CreateDirectoryIfNeeded(FString& DirectoryPath);

		// Do some shared prep steps for WAV file export
		bool PrepWavFileOutput(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath);
	};

	/************************************************************************/
	/* FAudioRecordingData                                                  */
	/* This is used by USoundSubmix and the AudioMixerBlueprintLibrary      */
	/* to contain FSoundWavePCMWriter operations.                           */
	/************************************************************************/
	struct FAudioRecordingData
	{
		TSampleBuffer<int16> InputBuffer;
		FSoundWavePCMWriter Writer;

		~FAudioRecordingData() = default;
	};

}
