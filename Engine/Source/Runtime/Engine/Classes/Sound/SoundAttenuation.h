// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Attenuation.h"
#include "IAudioParameterInterfaceRegistry.h"
#include "AudioLinkSettingsAbstract.h"
#include "SoundAttenuationEditorSettings.h"
#include "Sound/SoundSubmixSend.h"

#include "SoundAttenuation.generated.h"

class UOcclusionPluginSourceSettingsBase;
class UReverbPluginSourceSettingsBase;
class USourceDataOverridePluginSourceSettingsBase;
class USoundSubmixBase;
class USpatializationPluginSourceSettingsBase;

// This enumeration is deprecated
UENUM()
enum ESoundDistanceCalc : int
{
	SOUNDDISTANCE_Normal,
	SOUNDDISTANCE_InfiniteXYPlane,
	SOUNDDISTANCE_InfiniteXZPlane,
	SOUNDDISTANCE_InfiniteYZPlane,
	SOUNDDISTANCE_MAX,
};

UENUM()
enum ESoundSpatializationAlgorithm : int
{
	// Standard panning method for spatialization (linear or equal power method defined in project settings)
	SPATIALIZATION_Default UMETA(DisplayName = "Panning"),

	// Spatialization method provided by spatialization plugin currently selected in the project settings
	SPATIALIZATION_HRTF UMETA(DisplayName = "Plugin-Spatialized"),
};

UENUM(BlueprintType)
enum class EAirAbsorptionMethod : uint8
{
	// Air absorption based on linear interpolation between a distance range and a remapping range
	Linear UMETA(DisplayName = "Distance Linear"),

	// Air absorption based on a supplied curve mapping (distance = X, air absorption = Y)
	CustomCurve UMETA(DisplayName = "Distance Custom"),
};


UENUM(BlueprintType)
enum class EReverbSendMethod : uint8
{
	// Reverb send based on linear interpolation between a distance range and a remapping range
	Linear UMETA(DisplayName = "Distance Linear"),

	// Reverb send based on a supplied curve mapping (distance = X, send level = Y)
	CustomCurve UMETA(DisplayName = "Distance Custom"),

	// Constant reverb send level (Uses the specified constant send level value. Useful for 2D sounds.)
	Manual UMETA(DisplayName = "Constant"),
};

UENUM(BlueprintType)
enum class EPriorityAttenuationMethod : uint8
{
	// Priority attenuation based on linear interpolation between a distance range and a remapping range
	Linear UMETA(DisplayName = "Distance Linear"),

	// Priority attenuation based on a supplied curve mapping (distance = X, priority = Y)
	CustomCurve UMETA(DisplayName = "Distance Custom"),

	// Constant priority attenuation (Uses the specified constant value. Useful for 2D sounds.)
	Manual UMETA(DisplayName = "Constant"),
};


USTRUCT(BlueprintType)
struct FSoundAttenuationPluginSettings
{
	GENERATED_USTRUCT_BODY()

	/** Settings to use with spatialization audio plugin. These are defined by the plugin creator. Not all audio plugins utilize this feature. This is an array so multiple plugins can have settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Spatialization Plugin Settings"))
	TArray<TObjectPtr<USpatializationPluginSourceSettingsBase>> SpatializationPluginSettingsArray;

	/** Settings to use with occlusion audio plugin. These are defined by the plugin creator. Not all audio plugins utilize this feature. This is an array so multiple plugins can have settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion, meta = (DisplayName = "Occlusion Plugin Settings"))
	TArray<TObjectPtr<UOcclusionPluginSourceSettingsBase>> OcclusionPluginSettingsArray;

	/** Settings to use with reverb audio plugin. These are defined by the plugin creator. Not all audio plugins utilize this feature. This is an array so multiple plugins can have settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Plugin Settings"))
	TArray<TObjectPtr<UReverbPluginSourceSettingsBase>> ReverbPluginSettingsArray;

	/** Settings to use with source data override audio plugin. These are defined by the plugin creator. Not all audio plugins utilize this feature. This is an array so multiple plugins can have settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSourceDataOverride, meta = (DisplayName = "Source Data Override Plugin Settings"))
	TArray<TObjectPtr<USourceDataOverridePluginSourceSettingsBase>> SourceDataOverridePluginSettingsArray;

	void AddStructReferencedObjects(FReferenceCollector& Collector);	
};

template<>
struct TStructOpsTypeTraits<FSoundAttenuationPluginSettings> : public TStructOpsTypeTraitsBase2<FSoundAttenuationPluginSettings>
{
	enum
	{
		WithAddStructReferencedObjects = true
	};
};


// Defines how to speaker map the sound when using the Non Spatialized Distance feature
UENUM(BlueprintType)
enum class ENonSpatializedRadiusSpeakerMapMode : uint8
{
	// Will blend the 3D sound to an omni-directional sound (equal output mapping in all directions)
	OmniDirectional,

	// Will blend the 3D source to the same representation speaker map used when playing the asset 2D
	Direct2D,

	// Will blend the 3D source to a multichannel 2D version (i.e. upmix stereo to quad) if rendering in surround
	Surround2D,
};

USTRUCT(BlueprintType)
struct FAttenuationSubmixSendSettings : public FSoundSubmixSendInfoBase
{
	GENERATED_BODY();
	
	FAttenuationSubmixSendSettings();
};

// Trait specializations don't inherit; re-specialize so PostSerialize fires for this derived type.
template<>
struct TStructOpsTypeTraits<FAttenuationSubmixSendSettings> : public TStructOpsTypeTraitsBase2<FAttenuationSubmixSendSettings>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/*
The settings for attenuating.
*/
USTRUCT(BlueprintType)
struct FSoundAttenuationSettings : public FBaseAttenuationSettings
{
	GENERATED_USTRUCT_BODY()

	/* Allows distance-based volume attenuation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationDistance, meta = (DisplayName = "Enable Volume Attenuation"))
	uint8 bAttenuate : 1;

	/* Allows the source to be 3D spatialized. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Enable Spatialization"))
	uint8 bSpatialize : 1;

	/** Allows simulation of air absorption by applying a filter with a cutoff frequency as a function of distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (DisplayName = "Enable Air Absorption"))
	uint8 bAttenuateWithLPF : 1;

	/** Enable listener focus-based adjustments. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus)
	uint8 bEnableListenerFocus : 1;

	/** Enables focus interpolation to smooth transition in and and of focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus)
	uint8 bEnableFocusInterpolation : 1;

	/** Enables realtime occlusion tracing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion)
	uint8 bEnableOcclusion : 1;

	/** Enables tracing against complex collision when doing occlusion traces. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion)
	uint8 bUseComplexCollisionForOcclusion : 1;

	/** Enables adjusting reverb sends based on distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Enable Reverb Send"))
	uint8 bEnableReverbSend : 1;

	/** Enables attenuation of sound priority based off distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (DisplayName = "Enable Priority Attenuation"))
	uint8 bEnablePriorityAttenuation : 1;

	/** Enables applying a -6 dB attenuation to stereo assets which are 3d spatialized. Avoids clipping when assets have spread of 0.0 due to channel summing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Normalize 3D Stereo Sounds"))
	uint8 bApplyNormalizationToStereoSounds : 1;

	/** Enables applying a log scale to frequency values (so frequency sweeping is perceptually linear). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (DisplayName = "Enable Log Frequency Scaling"))
	uint8 bEnableLogFrequencyScaling : 1;

	/** Enables submix sends based on distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSubmixSend, meta = (DisplayName = "Enable Submix Send"))
	uint8 bEnableSubmixSends : 1;

	/** Enables overriding WaveInstance data using source data override plugin */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSourceDataOverride, meta = (DisplayName = "Enable Source Data Override"))
	uint8 bEnableSourceDataOverride : 1;

	/** Enables/Disables AudioLink on all sources using this attenuation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAudioLink, meta = (DisplayName = "Enable Send to AudioLink"))
	uint8 bEnableSendToAudioLink : 1;

	/** What method we use to spatialize the sound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (ClampMin = "0", EditCondition = "bSpatialize", DisplayName = "Spatialization Method"))
	TEnumAsByte<enum ESoundSpatializationAlgorithm> SpatializationAlgorithm;

	/** AudioLink Setting Overrides */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAudioLink, meta = (DisplayName = "AudioLink Settings Override", EditCondition = "bEnableSendToAudioLink"))
	TObjectPtr<UAudioLinkSettingsAbstract> AudioLinkSettingsOverride;

	/** What min radius to use to swap to non-binaural audio when a sound starts playing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (ClampMin = "0", EditCondition = "bSpatialize"))
	float BinauralRadius;

	/* The normalized custom curve to use for the air absorption lowpass frequency values. Does a mapping from defined distance values (x-axis) and defined frequency values (y-axis) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption)
	FRuntimeFloatCurve CustomLowpassAirAbsorptionCurve;

	/* The normalized custom curve to use for the air absorption highpass frequency values. Does a mapping from defined distance values (x-axis) and defined frequency values (y-axis) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption)
	FRuntimeFloatCurve CustomHighpassAirAbsorptionCurve;

	/** What method to use to map distance values to frequency absorption values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption)
	EAirAbsorptionMethod AbsorptionMethod;

	/* Which trace channel to use for audio occlusion checks. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion)
	TEnumAsByte<enum ECollisionChannel> OcclusionTraceChannel;

	/** What method to use to control master reverb sends */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend)
	EReverbSendMethod ReverbSendMethod;

	/** What method to use to control priority attenuation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority)
	EPriorityAttenuationMethod PriorityAttenuationMethod;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TEnumAsByte<enum ESoundDistanceCalc> DistanceType_DEPRECATED;

 	UPROPERTY()
 	float OmniRadius_DEPRECATED;
#endif 

	/** The distance below which a sound begins to linearly interpolate towards being non-spatialized (2D). See "Non Spatialized Distance Min" to define the end of the interpolation and the "Non Spatialized Distance Mode" for the mode of the interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Non Spatialized Distance Max", ClampMin = "0", EditCondition = "bSpatialize"))
	float NonSpatializedRadiusStart;

	/** The distance below which a sound is fully non-spatialized (2D). See "Non Spatialized Distance Max" to define the start of the interpolation and the "Non Spatialized Distance Mode" for the mode of the interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Non Spatialized Distance Min", ClampMin = "0", EditCondition = "bSpatialize"))
	float NonSpatializedRadiusEnd;

	/** Defines how to interpolate a 3D sound towards a 2D sound when using the Non Spatialized Distance Min and Max properties. Note: this does not apply when using a 3rd party binaural plugin (audio will remain spatialized). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (DisplayName = "Non Spatialized Distance Mode", ClampMin = "0", EditCondition = "bSpatialize"))
	ENonSpatializedRadiusSpeakerMapMode NonSpatializedRadiusMode;

	/** The world-space distance between left and right stereo channels when stereo assets are 3D spatialized. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSpatialization, meta = (ClampMin = "0", EditCondition = "bSpatialize", DisplayName = "3D Stereo Spread"))
	float StereoSpread;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<USpatializationPluginSourceSettingsBase> SpatializationPluginSettings_DEPRECATED;

	UPROPERTY()
	float RadiusMin_DEPRECATED;

	UPROPERTY()
	float RadiusMax_DEPRECATED;
#endif

	/* The distance min range at which to apply our absorption filter(s). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (DisplayName = "Distance Min"))
	float LPFRadiusMin;

	/* The max distance range at which to apply our absorption filter(s). Absorption freq cutoff interpolates between filter frequency ranges within these distance values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (DisplayName = "Distance Max"))
	float LPFRadiusMax;

	/* The cutoff frequency (in Hz) of the lowpass absorption filter when at Distance Min. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (Units = "Hz", DisplayName = "Low Pass Frequency Min"))
	float LPFFrequencyAtMin;

	/* The cutoff frequency (in Hz) of the lowpass absorption filter when at Distance Max. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (Units = "Hz", DisplayName = "Low Pass Frequency Max"))
	float LPFFrequencyAtMax;
	
	/* The cutoff frequency (in Hz) of the highpass absorption filter when at Distance Min. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (Units = "Hz", DisplayName = "High Pass Frequency Min"))
	float HPFFrequencyAtMin;

	/* The cutoff frequency (in Hz) of the highpass absorption filter when at Distance Max. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationAirAbsorption, meta = (Units = "Hz", DisplayName = "High Pass Frequency Max"))
	float HPFFrequencyAtMax;

	/** Azimuth angle (in degrees) relative to the listener forward vector which defines the focus region of sounds. Sounds playing at an angle less than this will be in focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus)
	float FocusAzimuth;

	/** Azimuth angle (in degrees) relative to the listener forward vector which defines the non-focus region of sounds. Sounds playing at an angle greater than this will be out of focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus)
	float NonFocusAzimuth;

	/** Amount to scale the distance calculation of sounds that are in-focus. Can be used to make in-focus sounds appear to be closer or further away than they actually are. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableListenerFocus"))
	float FocusDistanceScale;

	/** Amount to scale the distance calculation of sounds that are not in-focus. Can be used to make in-focus sounds appear to be closer or further away than they actually are.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableListenerFocus"))
	float NonFocusDistanceScale;

	/** Amount to scale the priority of sounds that are in focus. Can be used to boost the priority of sounds that are in focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableListenerFocus"))
	float FocusPriorityScale;

	/** Amount to scale the priority of sounds that are not in-focus. Can be used to reduce the priority of sounds that are not in focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableListenerFocus"))
	float NonFocusPriorityScale;

	/** Amount to attenuate sounds that are in focus. Can be overridden at the sound-level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableListenerFocus"))
	float FocusVolumeAttenuation;

	/** Amount to attenuate sounds that are not in focus. Can be overridden at the sound-level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableListenerFocus"))
	float NonFocusVolumeAttenuation;

	/** Scalar used to increase interpolation speed upwards to the target Focus value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableFocusInterpolation"))
	float FocusAttackInterpSpeed;

	/** Scalar used to increase interpolation speed downwards to the target Focus value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationListenerFocus, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableFocusInterpolation"))
	float FocusReleaseInterpSpeed;

	/** The low pass filter frequency (in Hz) to apply if the sound playing in this audio component is occluded. This will override the frequency set in LowPassFilterFrequency. A frequency of 0.0 is the device sample rate and will bypass the filter. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableOcclusion"))
	float OcclusionLowPassFilterFrequency;

	/** The amount of volume attenuation to apply to sounds which are occluded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bEnableOcclusion"))
	float OcclusionVolumeAttenuation;

	/** The amount of time in seconds to interpolate to the target OcclusionLowPassFilterFrequency when a sound is occluded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationOcclusion, meta = (ClampMin = "0", UIMin = "0.0", EditCondition = "bEnableOcclusion"))
	float OcclusionInterpolationTime;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UOcclusionPluginSourceSettingsBase> OcclusionPluginSettings_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UReverbPluginSourceSettingsBase> ReverbPluginSettings_DEPRECATED;
#endif

	/** The amount to send to master reverb when sound is located at a distance equal to value specified in the reverb min send distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Send Level Min"))
	float ReverbWetLevelMin;

	/** The amount to send to master reverb when sound is located at a distance equal to value specified in the reverb max send distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Send Level Max"))
	float ReverbWetLevelMax;

	/** The distance from listener at which we should be sending the level defined via Reverb Send Level Min */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Send Distance Min"))
	float ReverbDistanceMin;

	/** The distance from listener at which we should be sending the level defined via Reverb Send Level Max */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Send Distance Max"))
	float ReverbDistanceMax;

	/* Constant reverb send level to use. Doesn't change as a function of distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend, meta = (DisplayName = "Reverb Send Level"))
	float ManualReverbSendLevel;

	/** Interpolated value to scale priority against when the sound is at the minimum priority attenuation distance from the closest listener. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", DisplayName = "Priority Attenuation At Min Distance"))
	float PriorityAttenuationMin;

	/** Interpolated value to scale priority against when the sound is at the maximum priority attenuation distance from the closest listener. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", DisplayName = "Priority Attenuation At Max Distance"))
	float PriorityAttenuationMax;

	/** The min distance to attenuate priority. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (ClampMin = "0.0", DisplayName = "Priority Attenuation Min Distance"))
	float PriorityAttenuationDistanceMin;

	/** The max distance to attenuate priority. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (ClampMin = "0.0", DisplayName = "Priority Attenuation Max Distance"))
	float PriorityAttenuationDistanceMax;

	/* Constant priority scalar to use (doesn't change as a function of distance). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0", DisplayName = "Attenuation Priority"))
	float ManualPriorityAttenuation;

	/* The custom reverb send curve to use for distance-based send level. Values clamped between 0.0 and 1.0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationReverbSend)
	FRuntimeFloatCurve CustomReverbSendCurve;

	/** Set of submix send settings to use to send audio to submixes as a function of distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationSubmixSend)
	TArray<FAttenuationSubmixSendSettings> SubmixSendSettings;

	/* The custom curve to use for distance-based priority attenuation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPriority)
	FRuntimeFloatCurve CustomPriorityAttenuationCurve;

	/** Sound attenuation plugin settings to use with sounds that play with this attenuation setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AttenuationPluginSettings, meta = (ShowOnlyInnerProperties))
	FSoundAttenuationPluginSettings PluginSettings;

	ENGINE_API FSoundAttenuationSettings();
	ENGINE_API ~FSoundAttenuationSettings();
	ENGINE_API FSoundAttenuationSettings(const FSoundAttenuationSettings&);

	ENGINE_API bool operator==(const FSoundAttenuationSettings& Other) const;
#if WITH_EDITORONLY_DATA
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif

	ENGINE_API virtual void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const override;
	ENGINE_API float GetFocusPriorityScale(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const;
	ENGINE_API float GetFocusAttenuation(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const;
	ENGINE_API float GetFocusDistanceScale(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const;

	ENGINE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
};

template<>
struct TStructOpsTypeTraits<FSoundAttenuationSettings> : public TStructOpsTypeTraitsBase2<FSoundAttenuationSettings>
{
	enum 
	{
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
		WithAddStructReferencedObjects = true
	};
};


/** 
 * Defines how a sound changes volume with distance to the listener
 */
UCLASS(BlueprintType, hidecategories=Object, editinlinenew, MinimalAPI)
class USoundAttenuation : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta = (CustomizeProperty))
	FSoundAttenuationSettings Attenuation;
};

namespace Audio
{
	namespace AttenuationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName Distance;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace AttenuationInterface

	namespace SpatializationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName Azimuth;
			ENGINE_API const extern FName Elevation;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace SpatializationInterface

	namespace SourceOrientationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName Azimuth;
			ENGINE_API const extern FName Elevation;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace EmitterInterface

	namespace ListenerOrientationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName Azimuth;
			ENGINE_API const extern FName Elevation;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace EmitterInterface

	namespace SourceLocationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName X;
			ENGINE_API const extern FName Y;
			ENGINE_API const extern FName Z;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace SourceLocationInterface

	namespace ListenerLocationInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName X;
			ENGINE_API const extern FName Y;
			ENGINE_API const extern FName Z;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace ListenerLocationInterface

	namespace SourceOcclusionInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName IsOccluded;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace SourceOcclusionInterface

	namespace WorldTimeInterface
	{
		ENGINE_API const extern FName Name;

		namespace Inputs
		{
			ENGINE_API const extern FName WorldTimeWhenPlayed;
			ENGINE_API const extern FName CurrentWorldTime;
			ENGINE_API const extern FName ElapsedWorldTime;
		} // namespace Inputs

		ENGINE_API Audio::FParameterInterfacePtr GetInterface();
	} // namespace WorldTimeInterface
} // namespace Audio
