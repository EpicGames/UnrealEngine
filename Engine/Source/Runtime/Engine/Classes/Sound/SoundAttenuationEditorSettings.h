// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoundAttenuationEditorSettings.generated.h"

/**
 * Implements Editor settings for sound attenuation assets and is used when switching editor modes.
 */
UCLASS(MinimalAPI)
class USoundAttenuationEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Enables adjusting reverb sends based on distance. */
	UPROPERTY()
	bool bEnableReverbSend = true;

	/** Enables/Disables AudioLink on all sources using this attenuation */
	UPROPERTY()
	bool bEnableSendToAudioLink = true;
};
