// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "AudioDefines.h"
#include "AudioPropertiesSheetAssetBase.h"
#include "CoreMinimal.h"
#include "Sound/SoundTimecodeOffset.h"
#include "SoundConcurrency.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Interfaces/Interface_AssetUserData.h"

#include "SoundBase.generated.h"


// Forward Declarations
namespace Audio
{
	class IParameterTransmitter;
	struct FParameterInterface;
	struct FParameterTransmitterInitParams;
	using FDeviceId = uint32;;
	using FParameterInterfacePtr = TSharedPtr<FParameterInterface, ESPMode::ThreadSafe>;
} // namespace Audio

class ISoundGenerator;
class UAssetUserData;
class UAudioPropertiesSheetAssetBase;
class UAudioPropertiesBindings;
class USoundAttenuation;
class USoundClass;
class USoundEffectSourcePreset;
class USoundEffectSourcePresetChain;
class USoundSourceBus;
class USoundSubmix;
class USoundSubmixBase;
class USoundWave;
enum class EBusSendType : uint8;
namespace EMaxConcurrentResolutionRule { enum Type : int; }
struct FActiveSound;
struct FAudioParameter;
struct FSoundAttenuationSettings;
struct FSoundGeneratorInitParams;
struct FSoundParseParameters;
struct FSoundSourceBusSendInfo;
struct FSoundSubmixSendInfo;
struct FWaveInstance;
typedef TSharedPtr<ISoundGenerator, ESPMode::ThreadSafe> ISoundGeneratorPtr;

/**
 * Method of virtualization when a sound is stopped due to playback constraints
 * (i.e. by concurrency, priority, and/or MaxChannelCount)
 * for a given sound.
 */
UENUM(BlueprintType)
enum class EVirtualizationMode : uint8
{
	/** Virtualization is disabled */
	Disabled,

	/** Sound continues to play when silent and not virtualize, continuing to use a voice. If
	 * sound is looping and stopped due to concurrency or channel limit/priority, sound will
	 * restart on realization. If any SoundWave referenced in a SoundCue's waveplayer is set
	 * to 'PlayWhenSilent', entire SoundCue will be overridden to 'PlayWhenSilent' (to maintain
	 * timing over all wave players).
	 */
	PlayWhenSilent,

	/** If sound is looping, sound restarts from beginning upon realization from being virtual */
	Restart,

	/** EXPERIMENTAL - Sound fully virtualizes, but attempts to seek to the proper part of its looping duration when realized.
	 * The playback behavior is similar to PlayWhenSilent, but does not take up a voice slot.
	 */
	SeekRestart UE_EXPERIMENTAL(5.6, "This virtualization mode is experimental and may not function as expected in all use cases.")
};

/**
 * There is a editor setting to use submixes when previewing sounds. This gives the user
 * the ability to override that locally for this specific asset.
 */
UENUM(BlueprintType)
enum class EUseSubmixOnPreviewMode : uint8
{
	UseEditorPreference,
	UseSubmixesOnPreview,
	PlayPreviewsWithoutSubmixes
};

// This struct allows us to modify the transient editor properties and minimize impact on the properties
USTRUCT()
struct FSoundBaseEditorData
{
	GENERATED_BODY()

public:
	/** Override the Editor Preference to use submixes on preview playback */
	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category = "Developer", meta = (DisplayName = "Use Submixes on preview"))
	EUseSubmixOnPreviewMode OverrideUseSubmixesOnPreview = EUseSubmixOnPreviewMode::UseEditorPreference;
};


/**
 * The base class for a playable sound object
 */
UCLASS(config=Engine, hidecategories=Object, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundBase : public UObject, public IAudioPropertiesSheetAssetUserInterface, public IInterface_AssetUserData
{

private:
	GENERATED_UCLASS_BODY()

public:
	/** Sound class this sound belongs to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sound", meta = (DisplayName = "Sound Class"), AssetRegistrySearchable)
	TObjectPtr<USoundClass> SoundClassObject;

#if WITH_EDITORONLY_DATA
	/** Supplement the USoundBase with a list of transient editor-only properties */
	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category = Developer, meta = (DisplayName = "Editor Data", ShowOnlyInnerProperties))
	FSoundBaseEditorData EditorData;
#endif //WITH_EDITORONLY_DATA

	/** When "au.3dVisualize.Attenuation" has been specified, draw this sound's attenuation shape when the sound is audible. For debugging purposes only. */
	UPROPERTY(EditAnywhere, Category = "Developer", meta = (DisplayName = "Enable Attenuation Debug"))
	uint8 bDebug : 1;

	/** Whether or not to override the sound concurrency object with local concurrency settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management", meta=(InlineEditConditionToggle))
	uint8 bOverrideConcurrency : 1;

#if WITH_EDITORONLY_DATA
	/** Whether or not to only send this audio's output to a bus. If true, will not be this sound won't be audible except through bus sends. */
	UPROPERTY()
	uint8 bOutputToBusOnly_DEPRECATED : 1;
#endif //WITH_EDITORONLY_DATA

	/** Whether or not to enable sending this audio's output to buses.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (InlineEditConditionToggle))
	uint8 bEnableBusSends : 1;

	/** If enabled, sound will route to the Master Submix by default or to the Base Submix if defined. If disabled, sound will route ONLY to the Submix Sends and/or Bus Sends */
	UPROPERTY(EditAnywhere, Category = "Routing", meta = (InlineEditConditionToggle))
	uint8 bEnableBaseSubmix : 1;

	/** Whether or not to enable Submix Sends other than the Base Submix. */
	UPROPERTY(EditAnywhere, Category = "Routing", meta = (InlineEditConditionToggle))
	uint8 bEnableSubmixSends : 1;

	/** Whether or not this sound has a delay node */
	UPROPERTY()
	uint8 bHasDelayNode : 1;

	/** Whether or not this sound has a concatenator node. If it does, we have to allow the sound to persist even though it may not have generate audible audio in a given audio thread frame. */
	UPROPERTY()
	uint8 bHasConcatenatorNode : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 bHasVirtualizeWhenSilent_DEPRECATED : 1;
#endif // WITH_EDITORONLY_DATA

	/** Bypass volume weighting priority upon evaluating whether sound should remain active when max channel count is met (See platform Audio Settings). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management")
	uint8 bBypassVolumeScaleForPriority : 1;

	/** Virtualization behavior, determining if a sound may revive and how it continues playing when culled or evicted (limited to looping sounds). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management")
	EVirtualizationMode VirtualizationMode;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TEnumAsByte<EMaxConcurrentResolutionRule::Type> MaxConcurrentResolutionRule_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Map of device handle to number of times this sound is currently being played using that device(counted if sound is virtualized). */
	TMap<Audio::FDeviceId, int32> CurrentPlayCount;

#if WITH_EDITORONLY_DATA
	/** If Override Concurrency is false, the sound concurrency settings to use for this sound. */
	UPROPERTY()
	TObjectPtr<USoundConcurrency> SoundConcurrencySettings_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Set of concurrency settings to observe (if override is set to false).  Sound must pass all concurrency settings to play. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management", meta = (EditCondition = "!bOverrideConcurrency"))
	TSet<TObjectPtr<USoundConcurrency>> ConcurrencySet;

	/** If Override Concurrency is true, concurrency settings to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management", meta = (EditCondition = "bOverrideConcurrency"))
	FSoundConcurrencySettings ConcurrencyOverrides;

#if WITH_EDITORONLY_DATA
	/** Maximum number of times this sound can be played concurrently. */
	UPROPERTY()
	int32 MaxConcurrentPlayCount_DEPRECATED;
#endif

	/** Duration of sound in seconds. */
	UPROPERTY(Category = "Info|Format", AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly)
	float Duration;

	/** The MaxDistance property is calculated statically on load or at asset edit time, but is not reliable at runtime.
	  * the GetMaxDistance function should be used to determine the applied max distance based on runtime behavior. */
	UPROPERTY(Category = "Developer", AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly)
	float MaxDistance;

	/** Total number of samples (in the thousands). Useful as a metric to analyze the relative size of a given sound asset in content browser. */
	UPROPERTY(Category = "Info|Format", AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly)
	float TotalSamples;

	/** Used to determine whether sound can play or remain active if channel limit is met, where higher value is higher priority
	  * (see platform's Audio Settings 'Max Channels' property). Unless bypassed, value is weighted with the final volume of the
	  * sound to produce final runtime priority value.
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Management", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "100.0", UIMax = "100.0", DisplayPriority=-1))
	float Priority;

	/** Attenuation settings package for the sound */
	UPROPERTY(EditAnywhere, Category = "Sound", meta = (EditCondition = "IsAttenuationSettingsEditable"))
	TObjectPtr<USoundAttenuation> AttenuationSettings;

	/** Submix to route sound output to. If unset, falls back to referenced SoundClass submix.
	  * If SoundClass submix is unset, sends to the 'Master Submix' as set in the 'Audio' category of Project Settings'. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (DisplayName = "Base Submix", EditCondition = "bEnableBaseSubmix"))
	TObjectPtr<USoundSubmixBase> SoundSubmixObject;

	/** Array of submix sends to which a prescribed amount (see 'Send Level') of this sound is sent. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (DisplayName = "Submix Sends", EditCondition = "bEnableSubmixSends"))
	TArray<FSoundSubmixSendInfo> SoundSubmixSends;

	/** The source effect chain to use for this sound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sound")
	TObjectPtr<USoundEffectSourcePresetChain> SourceEffectChain;

	/** This sound will send its audio output to this list of buses if there are bus instances playing after source effects are processed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (DisplayName = "Post-Effect Bus Sends", EditCondition = "bEnableBusSends"))
	TArray<FSoundSourceBusSendInfo> BusSends;

	/** This sound will send its audio output to this list of buses if there are bus instances playing before source effects are processed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (DisplayName = "Pre-Effect Bus Sends", EditCondition = "bEnableBusSends"))
	TArray<FSoundSourceBusSendInfo> PreEffectBusSends;

	/** This sound will send its audio output to this list of buses if there are bus instances playing after attenuation is applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Routing", meta = (DisplayName = "Post-Attenuation Bus Sends", EditCondition = "bEnableBusSends"))
	TArray<FSoundSourceBusSendInfo> PostAttenuationBusSends;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, Instanced, Category = "Advanced")
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA	
	UPROPERTY(EditAnywhere, Category = "Sound")
	TObjectPtr<UAudioPropertiesSheetAssetBase> AudioPropertiesSheet;

	// //~ Begin IAudioPropertiesSheetAssetUserInterface
    ENGINE_API virtual void AllowPropertyParsing(const FProperty& TargetProperty) override;
    ENGINE_API virtual void IgnorePropertyParsing(const FProperty& PropertyToIgnore) override;
    ENGINE_API virtual bool ShouldParseProperty(const FProperty& TargetProperty) const override;
	//~ End IAudioPropertiesSheetAssetUserInterface
#endif

private:
	UPROPERTY()
	FSoundTimecodeOffset TimecodeOffset;

	//Property Sheets support
	void InjectPropertySheet();
	void BindToPropertySheetChanges();
	void UnbindFromPropertySheetChanges();

#if WITH_EDITORONLY_DATA
    UPROPERTY()
    TArray<FName> LocalAudioProperties;
#endif

public:
	//~ Begin UObject Interface.
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void PostLoad() override;
#endif
	ENGINE_API virtual bool CanBeClusterRoot() const override;
	ENGINE_API virtual bool CanBeInCluster() const override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
#endif

	//~ End UObject interface.

	/** Returns whether the sound base is set up in a playable manner */
	ENGINE_API virtual bool IsPlayable() const;

	/** Returns whether sound supports subtitles. */
	ENGINE_API virtual bool SupportsSubtitles() const;

	/** Returns whether or not this sound base has an attenuation node. */
	ENGINE_API virtual bool HasAttenuationNode() const;

	/** Returns a pointer to the attenuation settings that are to be applied for this node */
	ENGINE_API virtual const FSoundAttenuationSettings* GetAttenuationSettingsToApply() const;

	/**
	 * Returns the farthest distance at which the sound could be heard
	 */
	ENGINE_API virtual float GetMaxDistance() const;

	/**
	 * Returns the length of the sound
	 */
	ENGINE_API virtual float GetDuration() const;

	/** Returns whether or not this sound has a delay node, which means it's possible for the sound to not generate audio for a while. */
	ENGINE_API bool HasDelayNode() const;

	/** Returns whether or not this sound has a sequencer node, which means it's possible for the owning active sound to persist even though it's not generating audio. */
	ENGINE_API bool HasConcatenatorNode() const;

	/** Returns true if any of the sounds in the sound have "play when silent" enabled. */
	ENGINE_API virtual bool IsPlayWhenSilent() const;

	ENGINE_API virtual float GetVolumeMultiplier();
	ENGINE_API virtual float GetPitchMultiplier();

	/** Returns the subtitle priority */
	virtual float GetSubtitlePriority() const { return DEFAULT_SUBTITLE_PRIORITY; };

	/** Returns whether or not any part of this sound wants interior volumes applied to it */
	ENGINE_API virtual bool ShouldApplyInteriorVolumes();

	/** Returns curves associated with this sound if it has any. By default returns nullptr, but types
	*	supporting curves can return a corresponding curve table.
	*/
	virtual class UCurveTable* GetCurveData() const { return nullptr; }

	/** Returns whether or not this sound is looping. TODO: Deprecate this to only use IsOneshot() in a MetaSound world. */
	ENGINE_API virtual bool IsLooping() const;

	/** Query if it's one shot. One shot is defined as a sound which is intended to have a fixed duration. */
	ENGINE_API virtual bool IsOneShot() const;

	/** Query if it's a procedurally generated sound. */
	virtual bool IsProcedurallyGenerated() const { return false; }

	/** Parses the Sound to generate the WaveInstances to play. */
	virtual void Parse( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) { }

	/** Returns the SoundClass used for this sound. */
	ENGINE_API virtual USoundClass* GetSoundClass() const;

	/** Returns the SoundSubmix used for this sound. */
	ENGINE_API virtual USoundSubmixBase* GetSoundSubmix() const;

	/** Returns the sound submix sends for this sound. */
	ENGINE_API void GetSoundSubmixSends(TArray<FSoundSubmixSendInfo>& OutSends) const;

	/** Returns the sound source sends for this sound. */
	ENGINE_API void GetSoundSourceBusSends(EBusSendType BusSendType, TArray<FSoundSourceBusSendInfo>& OutSends) const;

	/** Returns an array of FSoundConcurrencySettings handles. */
	ENGINE_API void GetConcurrencyHandles(TArray<FConcurrencyHandle>& OutConcurrencyHandles) const;

	/** Returns the priority to use when evaluating concurrency. */
	ENGINE_API float GetPriority() const;
	/** Returns whether the sound has cooked analysis data (e.g. FFT or envelope following data) and returns sound waves which have cooked data. */
	ENGINE_API virtual bool GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves);

	/** Queries if the sound has cooked FFT or envelope data. */
	virtual bool HasCookedFFTData() const { return false; }
	virtual bool HasCookedAmplitudeEnvelopeData() const { return false; }

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	/** Called from the Game Thread prior to attempting to pass parameters to the ParameterTransmitter. */
	ENGINE_API virtual void InitParameters(TArray<FAudioParameter>& ParametersToInit, FName InFeatureName = NAME_None);

	/** Called from the Game Thread prior to attempting to initialize a sound instance. */
	virtual void InitResources() { }

	/** Whether or not the given sound is a generator and implements an interface with the given name. */
	virtual bool ImplementsParameterInterface(Audio::FParameterInterfacePtr InParameterInterface) const { return false; }

	/** Creates a sound generator instance from this sound base. Return true if this is being implemented by a subclass. Sound generators procedurally generate audio in the audio render thread. */
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) { return nullptr; }
	
	/** Creates a sound generator instance from this sound base. Return true if this is being implemented by a subclass. Sound generators procedurally generate audio in the audio render thread. */
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams, TArray<FAudioParameter>&& InDefaultParameters) { return CreateSoundGenerator(InParams); }

	/** Creates a parameter transmitter for communicating with active sound instances. */
	ENGINE_API virtual TSharedPtr<Audio::IParameterTransmitter> CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const;

	/** Returns whether parameter is valid input for the given sound */
	ENGINE_API virtual bool IsParameterValid(const FAudioParameter& InParameter) const;

	/** Gets all the default parameters for this Asset.  */
	virtual bool GetAllDefaultParameters(TArray<FAudioParameter>& OutParameters) const { return false; }

	/** Whether or not this sound allows submix sends on preview. */
	UE_DEPRECATED(5.7, "This is no longer used.") virtual bool EnableSubmixSendsOnPreview() const { return false; }

	/** Only used as an edit condition for AttenuationSettings member, as base classes may choose to provide an attenuation override implementation */
	UFUNCTION()
	virtual bool IsAttenuationSettingsEditable() const { return true; }

#if WITH_EDITORONLY_DATA
	ENGINE_API void SetTimecodeOffset(const FSoundTimecodeOffset& InTimecodeOffset);
	ENGINE_API TOptional<FSoundTimecodeOffset> GetTimecodeOffset() const;

#endif //WITH_EDITORONLY_DATA

protected:
	ENGINE_API virtual float ComputeMaxDistance() const;
};
