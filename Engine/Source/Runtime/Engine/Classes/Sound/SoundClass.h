// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioDefines.h"
#include "AudioDynamicParameter.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Sound/AudioOutputTarget.h"
#include "Sound/SoundModulationDestination.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#endif // WITH_EDITOR
#include "SoundWaveLoadingBehavior.h"
#include "UObject/PerPlatformProperties.h"

#include "SoundClass.generated.h"

class UAssetUserData;

USTRUCT()
struct FSoundClassEditorData
{
	GENERATED_USTRUCT_BODY()

	int32 NodePosX;

	int32 NodePosY;


	FSoundClassEditorData()
		: NodePosX(0)
		, NodePosY(0)
	{
	}


	friend FArchive& operator<<(FArchive& Ar,FSoundClassEditorData& MySoundClassEditorData)
	{
		return Ar << MySoundClassEditorData.NodePosX << MySoundClassEditorData.NodePosY;
	}
};

/**
 * Structure containing configurable properties of a sound class.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType)
struct FSoundClassProperties
{
	GENERATED_USTRUCT_BODY()

	/** Volume multiplier. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "0.0"), Category = General)
	float Volume;

	/** Pitch multiplier. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	float Pitch;

	/** Lowpass filter cutoff frequency */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	float LowPassFilterFrequency;

	/** Scales the distance measurement used by the audio engine when determining distance-based attenuation. 
	  * E.g., a sound 1000 units away with an AttenuationDistanceScale of .5 will be attenuated
	  * as if it is 500 units away from the listener.
	  * Allows adjusting attenuation settings dynamically. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	float AttenuationDistanceScale;

	/** The amount of a sound to bleed to the LFE channel */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Routing, meta = (DisplayName = "LFE Bleed"))
	float LFEBleed;

	/** The amount to send to center channel (does not propagate to child classes) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Routing)
	float VoiceCenterChannelVolume;

	/** Volume of the radio filter effect. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Legacy)
	float RadioFilterVolume;

	/** Volume at which the radio filter kicks in */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Legacy)
	float RadioFilterVolumeThreshold;

	/** Whether to use 'Master EQ Submix' as set in the 'Audio' category of Project Settings as the default submix for referencing sounds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Legacy, meta = (DisplayName = "Output to Master EQ Submix"))
	uint8 bApplyEffects:1;

	/** Whether to inflate referencing sound's priority to always play. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	uint8 bAlwaysPlay:1;

	/** Whether or not this sound plays when the game is paused in the UI */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Legacy)
	uint8 bIsUISound:1;

	/** Whether or not this is music (propagates to child classes only if parent is true) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = Legacy)
	uint8 bIsMusic:1;

	/** Whether or not this sound class forces sounds to the center channel */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Routing)
	uint8 bCenterChannelOnly:1;

	/** Whether the Interior/Exterior volume and LPF modifiers should be applied */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Routing)
	uint8 bApplyAmbientVolumes:1;

	/** Whether or not sounds referencing this class send to the reverb submix */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submix, meta = (DisplayName = "Send to Master Reverb Submix"))
	uint8 bReverb:1;

	/** Send amount to master reverb effect for referencing unattenuated (2D) sounds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submix)
	float Default2DReverbSendAmount;

	/** Default modulation settings for sounds directly referencing this class */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Modulation)
	FSoundModulationDefaultSettings ModulationSettings;

	/** Which output target the sound should be played through */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Legacy)
	TEnumAsByte<EAudioOutputTarget::Type> OutputTarget;

	/** Specifies how and when compressed audio data is loaded for asset if stream caching is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Loading, meta = (DisplayName = "Loading Behavior Override"), AssetRegistrySearchable)
	ESoundWaveLoadingBehavior LoadingBehavior;

#if WITH_EDITORONLY_DATA
   	/** How much audio to add to First Audio Chunk (in seconds) */
	UPROPERTY(EditAnywhere, Category = Loading, meta = (UIMin = 0, UIMax = 10, EditCondition = "LoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad || LoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad"), DisplayName="Size of First Audio Chunk (seconds)")
   	FPerPlatformFloat SizeOfFirstAudioChunkInSeconds = 0.0f;
#endif //WITH_EDITORONLY_DATA

	/** Default output submix of referencing sounds. If unset, falls back to the 'Master Submix' as set in the 'Audio' category of Project Settings. 
	  * (Unavailable if legacy 'Output to Master EQ Submix' is set) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submix, meta = (EditCondition = "!bApplyEffects"))
	TObjectPtr<USoundSubmix> DefaultSubmix;

	FSoundClassProperties();
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Class for sound class properties which are intended to be dynamically changing without a sound mix. */
struct FSoundClassDynamicProperties
{
	FDynamicParameter AttenuationScaleParam;

	FSoundClassDynamicProperties()
		: AttenuationScaleParam(1.0f)
	{
	}
};

/**
 * Structure containing information on a SoundMix to activate passively.
 */
USTRUCT(BlueprintType)
struct FPassiveSoundMixModifier
{
	GENERATED_USTRUCT_BODY()

	/** The SoundMix to activate */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PassiveSoundMixModifier)
	TObjectPtr<class USoundMix> SoundMix;

	/** Minimum volume level required to activate SoundMix. Below this value the SoundMix will not be active. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PassiveSoundMixModifier)
	float MinVolumeThreshold;

	/** Maximum volume level required to activate SoundMix. Above this value the SoundMix will not be active. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PassiveSoundMixModifier)
	float MaxVolumeThreshold;

	FPassiveSoundMixModifier()
		: SoundMix(NULL)
		, MinVolumeThreshold(0.f)
		, MaxVolumeThreshold(10.f)
	{
	}
	
};

#if WITH_EDITOR
class USoundClass;

/** Interface for sound class graph interaction with the AudioEditor module. */
class ISoundClassAudioEditor
{
public:
	virtual ~ISoundClassAudioEditor() = default;

	/** Refreshes the sound class graph links. */
	virtual void RefreshGraphLinks(UEdGraph* SoundClassGraph) = 0;
};
#endif


UCLASS(config = Engine, hidecategories = Object, editinlinenew, BlueprintType, MinimalAPI)
class USoundClass : public UObject, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

public:
	/** Configurable properties like volume and priority. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General, meta = (ShowOnlyInnerProperties))
	FSoundClassProperties Properties;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	TArray<TObjectPtr<USoundClass>> ChildClasses;

	/** SoundMix Modifiers to activate automatically when a sound of this class is playing. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = General)
	TArray<FPassiveSoundMixModifier> PassiveSoundMixModifiers;

	UPROPERTY(BlueprintReadOnly, Category = General)
	TObjectPtr<USoundClass> ParentClass;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Advanced)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;
	
#if WITH_EDITORONLY_DATA
	/** EdGraph based representation of the SoundClass */
	TObjectPtr<class UEdGraph> SoundClassGraph;
#endif // WITH_EDITORONLY_DATA

protected:

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	ENGINE_API virtual FString GetDesc( void ) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface.

public:
	/** 
	 * Get the parameters for the sound mix.
	 */
	ENGINE_API void Interpolate( float InterpValue, FSoundClassProperties& Current, const FSoundClassProperties& Start, const FSoundClassProperties& End );

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface
	
	// Sound Class Editor functionality
#if WITH_EDITOR
	/** 
	 * @return true if the child sound class exists in the tree 
	 */
	ENGINE_API bool RecurseCheckChild( USoundClass* ChildSoundClass );

	/**
	 * Set the parent class of this SoundClass, removing it as a child from its previous owner
	 *
	 * @param	InParentClass	The New Parent Class of this
	 */
	ENGINE_API void SetParentClass( USoundClass* InParentClass );

	/**
	 * Add Referenced objects
	 *
	 * @param	InThis SoundClass we are adding references from.
	 * @param	Collector Reference Collector
	 */
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Refresh all EdGraph representations of SoundClasses
	 *
	 * @param	bIgnoreThis	Whether to ignore this SoundClass if it's already up to date
	 */
	ENGINE_API void RefreshAllGraphs(bool bIgnoreThis);

	/** Sets the sound cue graph editor implementation. */
	static ENGINE_API void SetSoundClassAudioEditor(TSharedPtr<ISoundClassAudioEditor> InSoundClassAudioEditor);

	/** Gets the sound cue graph editor implementation. */
	static ENGINE_API TSharedPtr<ISoundClassAudioEditor> GetSoundClassAudioEditor();
	
private:

	/** Ptr to interface to sound class editor operations. */
	static TSharedPtr<ISoundClassAudioEditor> SoundClassAudioEditor;

#endif

};

