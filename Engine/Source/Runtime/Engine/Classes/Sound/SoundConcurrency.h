// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/PerPlatformProperties.h"

#include "SoundConcurrency.generated.h"

class FAudioDevice;
class FSoundConcurrencyManager;

struct FActiveSound;

/** Sound concurrency group ID. */
using FConcurrencyGroupID = uint32;

/** Sound concurrency unique object IDs. */
using FConcurrencyObjectID = uint32;

/** Sound owner object IDs */
using FSoundOwnerObjectID = uint32;

/** Sound instance (USoundBase) object ID. */
using FSoundObjectID = uint32;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioConcurrency, Log, All);

UENUM()
namespace EMaxConcurrentResolutionRule
{
	enum Type : int
	{
		/** When Max Concurrent sounds are active do not start a new sound. */
		PreventNew,

		/** When Max Concurrent sounds are active stop the oldest and start a new one. */
		StopOldest,

		/** When Max Concurrent sounds are active stop the furthest sound.  If all sounds are the same distance then do not start a new sound. */
		StopFarthestThenPreventNew,

		/** When Max Concurrent sounds are active stop the furthest sound.  If all sounds are the same distance then stop the oldest. */
		StopFarthestThenOldest,

		/** Stop the lowest priority sound in the group. If all sounds are the same priority, then it will stop the oldest sound in the group. */
		StopLowestPriority,

		/** Stop the sound that is quietest in the group. */
		StopQuietest,

		/** Stop the lowest priority sound in the group. If all sounds are the same priority, then it won't play a new sound. */
		StopLowestPriorityThenPreventNew,

		Count UMETA(Hidden)
	};
}

UENUM()
enum class EConcurrencyVolumeScaleMode
{
	/* Scales volume of older sounds more than newer sounds (default) */
	Default = 0,

	/* Scales distant sounds by volume scalar more than closer sounds */
	Distance,

	/* Scales lower priority sounds by volume scalar more than closer sounds */
	Priority
};

USTRUCT(BlueprintType)
struct FSoundConcurrencySettings
{
	GENERATED_BODY()

	/**
	 * The max number of allowable concurrent active voices for voices playing in this concurrency group.
	 * Can be mutated at runtime via Blueprint or code (see "Enable MaxCount Platform Scaling" for disablement
	 * of runtime manipulation, which in turn allows for platform scaling of the given value).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency, meta = (
		DisplayName = "Max Count (Dynamic)",
		DisplayAfter = "bEnableMaxCountPlatformScaling",
		EditCondition = "!bEnableMaxCountPlatformScaling",
		EditConditionHides,
		UIMin = "1", ClampMin = "1"))
	int32 MaxCount;

	/* Whether or not to limit the concurrency to per sound owner (i.e. the actor that plays the sound). If the sound doesn't have an owner, it falls back to global concurrency. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency)
	uint8 bLimitToOwner : 1 = 0;

	/**
	 * Whether or not volume scaling can recover volume ducking behavior when concurrency group sounds stop (default scale mode only).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volume Scaling", meta = (DisplayName = "Can Recover", DisplayAfter = "VolumeScaleMode", EditCondition = "VolumeScaleMode == EConcurrencyVolumeScaleMode::Default", EditConditionHides))
	uint8 bVolumeScaleCanRelease : 1 = 0;

private:
	/* If true, MaxCount supports platform scaling, but cannot be dynamically changed at runtime (ex. from Blueprint or Gameplay Code).  If false, MaxCount is dynamically assignable at runtime, but is not platform scalable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Concurrency, meta = (AllowPrivateAccess))
	uint8 bEnableMaxCountPlatformScaling : 1 = 0;

public:
	/** Which concurrency resolution policy to use if max voice count is reached. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency)
	TEnumAsByte<EMaxConcurrentResolutionRule::Type> ResolutionRule = EMaxConcurrentResolutionRule::StopFarthestThenOldest;

	/** Amount of time to wait (in seconds) between different sounds which play with this concurrency. Sounds rejected from this will ignore virtualization settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency, meta = (UIMin = "0.0", ClampMin = "0.0"))
	float RetriggerTime = 0.0f;

private:
	/**
	 * The max number of allowable concurrent active voices for voices playing in this concurrency group.
	 * Scalable per platform or platform group. Cannot be mutated at runtime (Disable "Enable MaxCount Platform
	 * Scaling" for enablement of MaxCount runtime manipulation).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Concurrency, meta = (
		DisplayName = "Max Count (Platform Scaled)",
		DisplayAfter = "bEnableMaxCountPlatformScaling",
		EditCondition = "bEnableMaxCountPlatformScaling",
		EditConditionHides,
		AllowPrivateAccess,
		UIMin = "1", ClampMin = "1"))
	FPerPlatformInt PlatformMaxCount;

	/**
	 * Ducking factor to apply per older voice instance (generation), which compounds based on scaling mode
	 * and (optionally) revives them as they stop according to the provided attack/release times.
	 * 
	 * Note: This is not applied until after StopQuietest rules are evaluated, in order to avoid thrashing sounds.
	 *
	 * AppliedVolumeScale = Math.Pow(DuckingScale, VoiceGeneration)
	 */
	UPROPERTY(EditAnywhere, Category = "Volume Scaling", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float VolumeScale = 1.0f;

public:
	/** Volume Scale mode designating how to scale voice volume based on number of member sounds active in group. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volume Scaling")
	EConcurrencyVolumeScaleMode VolumeScaleMode = EConcurrencyVolumeScaleMode::Default;

	/**
	 * Time taken to apply duck using volume scalar.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volume Scaling", meta = (DisplayName = "Duck Time", UIMin = "0.0", ClampMin = "0.0", UIMax = "10.0", ClampMax = "1000000.0"))
	float VolumeScaleAttackTime = 0.01f;

	/**
	 * Time taken to recover volume scalar duck (default scale mode only).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volume Scaling", meta = (
		DisplayName = "Recover Time",
		DisplayAfter = "bVolumeScaleCanRelease",
		EditCondition = "bVolumeScaleCanRelease && VolumeScaleMode == EConcurrencyVolumeScaleMode::Default",
		EditConditionHides,
		UIMin = "0.0",
		ClampMin = "0.0",
		UIMax = "10.0",
		ClampMax = "1000000.0"))
	float VolumeScaleReleaseTime = 0.5f;

	/**
	 * Time taken to fade out if voice is evicted or culled due to another voice in the group starting.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice Stealing", meta = (UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax="1000000.0"))
	float VoiceStealReleaseTime = 0.0f;

	FSoundConcurrencySettings();

	/**
	  * Returns the MaxCount as a per-platform integer value.
	  */
	ENGINE_API int32 GetMaxCount() const;

	/**
	  * Applies the given integer value as the group's MaxCount if passed valid MaxCount (greater than 0) and MaxCount platform scaling is enabled.
	  * Returns true if successfully set, false if not.
	  */
	ENGINE_API bool SetMaxCount(int32 InMaxCount);

#if WITH_EDITOR
	/**
	 * Enables or disables MaxCount platform scaling.
	 */
	ENGINE_API void SetEnableMaxCountPlatformScaling(bool bEnabled);

	/**
	 * Returns whether or not platform scaling is enabled for MaxCount.
	 */
	ENGINE_API bool IsMaxCountPlatformScalingEnabled() const;

	/**
	  * If MaxCount platform scaling is enabled, applying the given per-platform integer value as the MaxCount.
	  */
	ENGINE_API bool SetPlatformMaxCount(FPerPlatformInt InMaxCount);
#endif // WITH_EDITOR

	/**
	 * Retrieves the volume scale
	 */
	ENGINE_API float GetVolumeScale() const;

	/** Whether or not ResolutionRule supports eviction, wherein eviction is the ability to keep a sound
	  * from playing prior to start and culling is the requirement of a sound to initialize and actively parse
	  * prior to being removed from a concurrency group.
	  */
	ENGINE_API bool IsEvictionSupported() const;
};


UCLASS(BlueprintType, hidecategories=Object, editinlinenew, MinimalAPI)
class USoundConcurrency : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Settings, meta = (ShowOnlyInnerProperties))
	FSoundConcurrencySettings Concurrency;

	/**
	  * Applies the given integer value as the group's MaxCount if passed valid MaxCount (greater than 0) and MaxCount platform scaling is enabled.
	  * Returns true if successfully set, false if not.
	  */
	UFUNCTION(BlueprintCallable, Category = Concurrency)
	UPARAM(DisplayName = "Success") bool SetMaxCount(UPARAM(DisplayName = "MaxCount") int32 InMaxCount);
};

/** How the concurrency request is handled by the concurrency manager */
enum class EConcurrencyMode : uint8
{
	Group,
	Owner,
	OwnerPerSound,
	Sound,
};

/** Handle to all required data to create and catalog a concurrency group */
struct FConcurrencyHandle
{
	const FSoundConcurrencySettings& Settings;
	const FConcurrencyObjectID ObjectID;
	const bool bIsOverride;

	/** Constructs a handle from concurrency override settings */
	FConcurrencyHandle(const FSoundConcurrencySettings& InSettings);

	/** Constructs a handle to a concurrency asset */
	FConcurrencyHandle(const USoundConcurrency& Concurrency);

	EConcurrencyMode GetMode(const FActiveSound& ActiveSound) const;
};


/** Sound instance data pertaining to concurrency tracking */
struct FConcurrencySoundData
{
	int32 Generation;
	float LerpTime;

private:
	float Elapsed;
	float DbTargetVolume;
	float DbStartVolume;

public:
	FConcurrencySoundData()
		: Generation(0)
		, LerpTime(0.0f)
		, Elapsed(0.0f)
		, DbTargetVolume(0.0f)
		, DbStartVolume(0.0f)
	{
	}

	void Update(float InElapsed);

	float GetLerpTime() const;
	float GetVolume(bool bInDecibels = false) const;
	float GetTargetVolume(bool bInDecibels = false) const;

	void SetTarget(float InTargetVolume, float InLerpTime);
};


/** Class which tracks array of active sound pointers for concurrency management */
class FConcurrencyGroup
{
	/** Array of active sounds for this concurrency group. */
	TArray<FActiveSound*> ActiveSounds;

	FConcurrencyGroupID GroupID;
	FConcurrencyObjectID ObjectID;
	FSoundConcurrencySettings Settings;

	/** When a sound last played on this concurrency group. */
	float LastTimePlayed = 0.0f;

public:
	/** Constructor for the max concurrency active sound entry. */
	FConcurrencyGroup(FConcurrencyGroupID GroupID, const FConcurrencyHandle& ConcurrencyHandle);

	static FConcurrencyGroupID GenerateNewID();

	/** Returns the active sounds array. */
	const TArray<FActiveSound*>& GetActiveSounds() const;

	/** Returns the id of the concurrency group */
	FConcurrencyGroupID GetGroupID() const;

	/** Returns the current generation (effectively, the number of concurrency sound instances active) */
	const int32 GetNextGeneration() const;

	/** Returns the settings associated with the group */
	const FSoundConcurrencySettings& GetSettings() const;

	/** Returns the parent object ID */
	FConcurrencyObjectID GetObjectID() const;

	/** Determines if the group is full. */
	bool IsEmpty() const;

	/** Determines if the group is full. */
	bool IsFull() const;

	/** Adds an active sound to the active sound array. */
	void AddActiveSound(FActiveSound& ActiveSound);

	/** Removes an active sound from the active sound array. */
	void RemoveActiveSound(FActiveSound& ActiveSound);

	/** Updates volume based on distance generation if set as VolumeScaleMode */
	void UpdateGeneration(FActiveSound* NewActiveSound = nullptr);

	/** Sorts the active sound if concurrency settings require culling post playback */
	void CullSoundsDueToMaxConcurrency();

	/** Sets when the last time a sound was played on this concurrency group. */
	void SetLastTimePlayed(float InLastTimePlayed) { LastTimePlayed = InLastTimePlayed; }

	/** Whether or not a sound would be rate limited if it tried to play right now. */
	bool CanPlaySoundNow(float InCurrentTime) const;

};

typedef TMap<FConcurrencyGroupID, FConcurrencyGroup*> FConcurrencyGroups;

struct FSoundInstanceEntry
{
	TMap<FSoundObjectID, FConcurrencyGroupID> SoundInstanceToConcurrencyGroup;

	FSoundInstanceEntry(FSoundObjectID SoundObjectID, FConcurrencyGroupID GroupID)
	{
		SoundInstanceToConcurrencyGroup.Add(SoundObjectID, GroupID);
	}
};

/** Type for mapping an object id to a concurrency entry. */
typedef TMap<FConcurrencyObjectID, FConcurrencyGroupID> FConcurrencyMap;

struct FOwnerConcurrencyMapEntry
{
	FConcurrencyMap ConcurrencyObjectToConcurrencyGroup;

	FOwnerConcurrencyMapEntry(FConcurrencyObjectID ConcurrencyObjectID, FConcurrencyGroupID GroupID)
	{
		ConcurrencyObjectToConcurrencyGroup.Add(ConcurrencyObjectID, GroupID);
	}
};

/** Maps owners to concurrency maps */
typedef TMap<FSoundOwnerObjectID, FOwnerConcurrencyMapEntry> FOwnerConcurrencyMap;

/** Maps owners to sound instances */
typedef TMap<FSoundOwnerObjectID, FSoundInstanceEntry> FOwnerPerSoundConcurrencyMap;

/** Maps sound object ids to active sound array for global concurrency limiting */
typedef TMap<FSoundObjectID, FConcurrencyGroupID> FPerSoundToActiveSoundsMap;


class FSoundConcurrencyManager
{
public:
	FSoundConcurrencyManager(FAudioDevice* InAudioDevice);
	ENGINE_API ~FSoundConcurrencyManager();

	/** Returns a newly allocated active sound given the input active sound struct. Will return nullptr if the active sound concurrency evaluation doesn't allow for it. */
	FActiveSound* CreateNewActiveSound(const FActiveSound& NewActiveSound, bool bIsRetriggering);

	/** Removes the active sound from concurrency tracking when active sound is stopped. */
	void RemoveActiveSound(FActiveSound& ActiveSound);

	/** Stops sound, applying concurrency rules for how to stop. */
	void StopDueToVoiceStealing(FActiveSound& ActiveSound);

	/** Updates generations for concurrency groups set to scale active sound volumes by distance or priority */
	void UpdateVolumeScaleGenerations();

	/** Culls any active sounds due to max concurrency sound resolution rule constraints being met */
	void UpdateSoundsToCull();

private: // Methods

	/** Evaluates whether or not the sound can play given the concurrency group's rules. Appends permissible
	sounds to evict in order for sound to play (if required) and returns the desired concurrency group. */
	FConcurrencyGroup* CanPlaySound(const FActiveSound& NewActiveSound, const FConcurrencyGroupID GroupID, TArray<FActiveSound*>& OutSoundsToEvict, bool bIsRetriggering);

	/** Creates a new concurrency group and returns pointer to said group */
	FConcurrencyGroup& CreateNewConcurrencyGroup(const FConcurrencyHandle& ConcurrencyHandle);

	/** Creates new concurrency groups from handle array */
	void CreateNewGroupsFromHandles(
		const FActiveSound& NewActiveSound,
		const TArray<FConcurrencyHandle>& ConcurrencyHandles,
		TArray<FConcurrencyGroup*>& OutGroupsToApply
	);

	/**  Creates an active sound to play, assigning it to the provided concurrency groups, and evicting required sounds */
	FActiveSound* CreateAndEvictActiveSounds(const FActiveSound& NewActiveSound, const TArray<FConcurrencyGroup*>& GroupsToApply, const TArray<FActiveSound*>& SoundsToEvict);

	/** Finds an active sound able to be evicted based on the provided concurrency settings. */
	FActiveSound* GetEvictableSound(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering);

	/** Helper functions for finding evictable sounds for the given concurrency rule */
	FActiveSound* GetEvictableSoundStopFarthest(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering) const;
	FActiveSound* GetEvictableSoundStopOldest(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering) const;

	/** Handles concurrency evaluation that happens per USoundConcurrencyObject */
	FActiveSound* EvaluateConcurrency(const FActiveSound& NewActiveSound, TArray<FConcurrencyHandle>& ConcurrencyHandles, bool bIsRetriggering);

private: // Data
	/** Owning audio device ptr for the concurrency manager. */
	FAudioDevice* AudioDevice;

	/** Global concurrency map that maps individual sounds instances to shared USoundConcurrency UObjects. */
	FConcurrencyMap ConcurrencyMap;

	FOwnerConcurrencyMap OwnerConcurrencyMap;

	/** A map of owners to concurrency maps for sounds which are concurrency-limited per sound owner. */
	FOwnerPerSoundConcurrencyMap OwnerPerSoundConcurrencyMap;

	/** Map of sound objects concurrency-limited globally */
	FPerSoundToActiveSoundsMap SoundObjectToConcurrencyGroup;

	/** A map of concurrency active sound ID to concurrency active sounds */
	FConcurrencyGroups ConcurrencyGroups;
};
