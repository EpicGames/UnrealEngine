// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IAudioParameterTransmitter.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundWave.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#endif

#include "SoundCue.generated.h"


// Forward Declarations
#if WITH_EDITORONLY_DATA
class UEdGraph;
#endif // WITH_EDITORONLY_DATA

class USoundCue;
class USoundNode;
class USoundNodeAttenuation;
struct FActiveSound;
struct FSoundParseParameters;

USTRUCT()
struct FSoundNodeEditorData
{
	GENERATED_USTRUCT_BODY()

	int32 NodePosX;

	int32 NodePosY;


	FSoundNodeEditorData()
		: NodePosX(0)
		, NodePosY(0)
	{
	}


	friend FArchive& operator<<(FArchive& Ar,FSoundNodeEditorData& MySoundNodeEditorData)
	{
		return Ar << MySoundNodeEditorData.NodePosX << MySoundNodeEditorData.NodePosY;
	}
};

#if WITH_EDITOR
class USoundCue;

/** Interface for sound cue graph interaction with the AudioEditor module. */
class ISoundCueAudioEditor
{
public:
	virtual ~ISoundCueAudioEditor() = default;

	/** Called when creating a new sound cue graph. */
	virtual UEdGraph* CreateNewSoundCueGraph(USoundCue* InSoundCue) = 0;

	/** Sets up a sound node. */
	virtual void SetupSoundNode(UEdGraph* SoundCueGraph, USoundNode* SoundNode, bool bSelectNewNode) = 0;

	/** Links graph nodes from sound nodes. */
	virtual void LinkGraphNodesFromSoundNodes(USoundCue* SoundCue) = 0;

	/** Compiles sound nodes from graph nodes. */
	virtual void CompileSoundNodesFromGraphNodes(USoundCue* SoundCue) = 0;

	/** Removes nodes which are null from the sound cue graph. */
	virtual void RemoveNullNodes(USoundCue* SoundCue) = 0;

	/** Creates an input pin on the given sound cue graph node. */
	virtual void CreateInputPin(UEdGraphNode* SoundCueNode) = 0;

	/** Renames all pins in a sound cue node */
	virtual void RenameNodePins(USoundNode* SoundNode) = 0;
};
#endif

/**
 * The behavior of audio playback is defined within Sound Cues.
 */
UCLASS(hidecategories=object, BlueprintType, meta= (LoadBehavior = "LazyOnDemand"), MinimalAPI)
class USoundCue : public USoundBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Sound)
	TObjectPtr<USoundNode> FirstNode;

	/* Base volume multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sound, AssetRegistrySearchable)
	float VolumeMultiplier;

	/* Base pitch multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sound, AssetRegistrySearchable)
	float PitchMultiplier;

	/* Attenuation settings to use if Override Attenuation is set to true */
	UPROPERTY(EditAnywhere, Category = "Sound")
	FSoundAttenuationSettings AttenuationOverrides;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<USoundNode>> AllNodes;

	UPROPERTY()
	TObjectPtr<UEdGraph> SoundCueGraph;
#endif

protected:
	// NOTE: Use GetSubtitlePriority() to fetch this value for external use.
	UPROPERTY(EditAnywhere, Category = "Voice Management", Meta = (Tooltip = "The priority of the subtitle.  Defaults to 10000.  Higher values will play instead of lower values."))
	float SubtitlePriority;

private:
	float MaxAudibleDistance;

public:

	/* Makes this sound cue automatically load any sound waves it can play into the cache when it is loaded. */
	UPROPERTY(EditAnywhere, Category = "Memory")
	uint8 bPrimeOnLoad : 1;

	/* Indicates whether attenuation should use the Attenuation Overrides or the Attenuation Settings asset */
	UPROPERTY(EditAnywhere, Category = "Sound", meta=(DisplayAfter="AttenuationSettings"))
	uint8 bOverrideAttenuation : 1;

	/* Ignore per-platform random node culling for memory purposes */
	UPROPERTY(EditAnywhere, Category = "Memory", Meta = (DisplayName = "Disable Random Branch Culling"))
	uint8 bExcludeFromRandomNodeBranchCulling : 1;

private:

	/** Whether a sound has play when silent enabled (i.e. for a sound cue, if any sound wave player has it enabled). */
	UPROPERTY()
	uint8 bHasPlayWhenSilent : 1;

	uint8 bHasAttenuationNode : 1;
	uint8 bHasAttenuationNodeInitialized : 1;
	uint8 bShouldApplyInteriorVolumes : 1;
	uint8 bShouldApplyInteriorVolumesCached : 1;
	uint8 bIsRetainingAudio : 1;

	UPROPERTY()
	int32 CookedQualityIndex = INDEX_NONE;

public:

	//~ Begin UObject Interface.
	ENGINE_API virtual FString GetDesc() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	ENGINE_API virtual bool CanBeClusterRoot() const override;
	ENGINE_API virtual bool CanBeInCluster() const override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

	//~ Begin USoundBase Interface.
	ENGINE_API virtual bool IsPlayable() const override;
	ENGINE_API virtual bool IsPlayWhenSilent() const override;
	ENGINE_API virtual bool ShouldApplyInteriorVolumes() override;
	ENGINE_API virtual void Parse( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	ENGINE_API virtual float GetVolumeMultiplier() override;
	ENGINE_API virtual float GetPitchMultiplier() override;
	ENGINE_API virtual float GetMaxDistance() const override;
	ENGINE_API virtual float GetDuration() const override;
	ENGINE_API virtual const FSoundAttenuationSettings* GetAttenuationSettingsToApply() const override;
	ENGINE_API virtual float GetSubtitlePriority() const override;
	ENGINE_API virtual bool GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves) override;
	ENGINE_API virtual bool HasCookedFFTData() const override;
	ENGINE_API virtual bool HasCookedAmplitudeEnvelopeData() const override;
	ENGINE_API virtual TSharedPtr<Audio::IParameterTransmitter> CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const override;
	virtual bool IsAttenuationSettingsEditable() const override { return !bOverrideAttenuation; }
	//~ End USoundBase Interface.

	/** Construct and initialize a node within this Cue */
	template<class T>
	T* ConstructSoundNode(TSubclassOf<USoundNode> SoundNodeClass = T::StaticClass(), bool bSelectNewNode = true)
	{
		// Set flag to be transactional so it registers with undo system
		T* SoundNode = NewObject<T>(this, SoundNodeClass, NAME_None, RF_Transactional);
#if WITH_EDITOR
		AllNodes.Add(SoundNode);
		SetupSoundNode(SoundNode, bSelectNewNode);
#endif // WITH_EDITORONLY_DATA
		return SoundNode;
	}

	/**
	*	@param		Format		Format to check
	 *
	 *	@return		Sum of the size of waves referenced by this cue for the given platform.
	 */
	ENGINE_API virtual int32 GetResourceSizeForFormat(FName Format);

	/**
	 * Recursively finds all Nodes in the Tree
	 */
	ENGINE_API void RecursiveFindAllNodes( USoundNode* Node, TArray<USoundNode*>& OutNodes );

	/**
	 * Recursively finds sound nodes of type T
	 */
	template<typename T>
	void RecursiveFindNode(USoundNode* Node, TArray<T*>& OutNodes)
	{
		if (Node)
		{
			// Record the node if it is the desired type
			if (T* FoundNode = Cast<T>(Node))
			{
				OutNodes.AddUnique(FoundNode);
			}

			// Recurse.
			const int32 MaxChildNodes = Node->GetMaxChildNodes();
			for (int32 ChildIndex = 0; ChildIndex < Node->ChildNodes.Num() && ChildIndex < MaxChildNodes; ++ChildIndex)
			{
				RecursiveFindNode<T>(Node->ChildNodes[ChildIndex], OutNodes);
			}
		}
	}

	template<typename T>
	void RecursiveFindNode(const USoundNode* Node, TArray<const T*>& OutNodes) const
	{
		if (Node)
		{
			// Record the node if it is the desired type
			if (const T* FoundNode = Cast<T>(Node))
			{
				OutNodes.AddUnique(FoundNode);
			}

			// Recurse.
			const int32 MaxChildNodes = Node->GetMaxChildNodes();
			for (int32 ChildIndex = 0; ChildIndex < Node->ChildNodes.Num() && ChildIndex < MaxChildNodes; ++ChildIndex)
			{
				RecursiveFindNode<T>(Node->ChildNodes[ChildIndex], OutNodes);
			}
		}
	}

	/** Find the path through the sound cue to a node identified by its hash */
	ENGINE_API bool FindPathToNode(const UPTRINT NodeHashToFind, TArray<USoundNode*>& OutPath) const;

	/** Call when the audio quality has been changed */
	static ENGINE_API void StaticAudioQualityChanged(int32 NewQualityLevel);

	inline static int32 GetCachedQualityLevel() { return CachedQualityLevel; }
	
	/** Set the Quality level that the Cue was cooked at, called by the SoundQualityNodes */
	int32 GetCookedQualityIndex() const { return CookedQualityIndex; }

	/** Call to cache any values which need to be computed from the sound cue graph. e.g. MaxDistance, Duration, etc. */
	ENGINE_API void CacheAggregateValues();

	/** Call this when stream caching is enabled to prime all SoundWave assets referenced by this Sound Cue. */
	ENGINE_API void PrimeSoundCue();

	/** Call this when stream caching is enabled to retain all soundwave assets referenced by this sound cue. */
	ENGINE_API void RetainSoundCue();
	ENGINE_API void ReleaseRetainedAudio();

	/** Call this when stream caching is enabled to update sound waves of loading behavior they are inheriting via SoundCue */
	ENGINE_API void CacheLoadingBehavior(ESoundWaveLoadingBehavior InBehavior);

protected:
	ENGINE_API bool RecursiveFindPathToNode(USoundNode* CurrentNode, const UPTRINT CurrentHash, const UPTRINT NodeHashToFind, TArray<USoundNode*>& OutPath) const;
	ENGINE_API float ComputeMaxDistance() const override;

private:
	ENGINE_API void AudioQualityChanged();
	ENGINE_API void OnPostEngineInit();
	ENGINE_API void EvaluateNodes(bool bAddToRoot);

	FDelegateHandle OnPostEngineInitHandle;
	static ENGINE_API int32 CachedQualityLevel;

public:

	// This is used to cache the quality level if it has not been cached yet.
	static ENGINE_API void CacheQualityLevel();

	ENGINE_API void RecursiveFindAttenuation(USoundNode* Node, TArray<USoundNodeAttenuation*> &OutNodes);
	ENGINE_API void RecursiveFindAttenuation(const USoundNode* Node, TArray<const USoundNodeAttenuation*>& OutNodes) const;

	/**
	 * For SoundCues, use this instead of GetAttenuationSettingsToApply() -> Evaluate
	 * 
	 * In cases where a SoundCue has multiple Attenuation nodes within it, 
	 * this function evaluates them all and returns the maximum value
	 * 
	 * @return float representing the max evaluated attenuation curve value based on distance from Origin to Location (will be 1.0 if no attenuation is set)
	 */
	ENGINE_API float EvaluateMaxAttenuation(const FTransform& Origin, FVector Location, float DistanceScale = 1.f) const;

#if WITH_EDITOR
	/** Create the basic sound graph */
	ENGINE_API void CreateGraph();

	/** Clears all nodes from the graph (for old editor's buffer soundcue) */
	ENGINE_API void ClearGraph();

	/** Set up EdGraph parts of a SoundNode */
	ENGINE_API void SetupSoundNode(USoundNode* InSoundNode, bool bSelectNewNode = true);

	/** Use the SoundCue's children to link EdGraph Nodes together */
	ENGINE_API void LinkGraphNodesFromSoundNodes();

	/** Use the EdGraph representation to compile the SoundCue */
	ENGINE_API void CompileSoundNodesFromGraphNodes();

	/** Get the EdGraph of SoundNodes */
	ENGINE_API UEdGraph* GetGraph();

	/** Resets all graph data and nodes */
	ENGINE_API void ResetGraph();

	/** Sets the sound cue graph editor implementation.* */
	static ENGINE_API void SetSoundCueAudioEditor(TSharedPtr<ISoundCueAudioEditor> InSoundCueGraphEditor);

	/** Gets the sound cue graph editor implementation. */
	static ENGINE_API TSharedPtr<ISoundCueAudioEditor> GetSoundCueAudioEditor();

private:

	/** Recursively sets the branch culling exclusion on random nodes in this sound cue. */
	ENGINE_API void RecursivelySetExcludeBranchCulling(USoundNode* CurrentNode);

	/** Ptr to interface to sound cue editor operations. */
	static ENGINE_API TSharedPtr<ISoundCueAudioEditor> SoundCueAudioEditor;

	FCriticalSection EditorOnlyCs;
#endif // WITH_EDITOR
};

class FSoundCueParameterTransmitter : public Audio::FParameterTransmitterBase
{
	using Super = Audio::FParameterTransmitterBase;
public:
	ENGINE_API FSoundCueParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams);

	ENGINE_API virtual ~FSoundCueParameterTransmitter();

	UE_DEPRECATED(5.6, "Use AddReferencedObjects(...) instead")
	ENGINE_API TArray<const TObjectPtr<UObject>*> GetReferencedObjects() const override;

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;

	ENGINE_API virtual bool SetParameters(TArray<FAudioParameter>&& InParameters) override;

	ENGINE_API void CreateChildTransmitter(UPTRINT InWaveInstanceHash, USoundBase& InSound, Audio::FParameterTransmitterInitParams InTransmitterInitParams);

	ENGINE_API void RemoveChildTransmitter(UPTRINT InWaveInstanceHash);

	/** The SoundCueParameter keeps a running track of parameter changes from SetParameters(...). 
	 * When this method is called, the accumulated parameters are returned and the internal
	 * tracking of accumulated parameters is reset. */
	ENGINE_API TArray<FAudioParameter> ReleaseAccumulatedParameterUpdates();

	UE_DEPRECATED(5.6, "Do not access ParamsToSet. This will be moved to be a private member.")
	TArray<FAudioParameter> ParamsToSet;

	UE_DEPRECATED(5.6, "Do not access Transmitters. This will be moved to be a private member.")
	TMap<UPTRINT, TSharedPtr<Audio::IParameterTransmitter>> Transmitters;
};
