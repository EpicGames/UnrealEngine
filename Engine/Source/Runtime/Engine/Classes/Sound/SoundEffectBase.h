// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AudioResampler.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"


// Forward Declarations
class USoundEffectPreset;


// The following macro code creates boiler-plate code for a sound effect preset and hides unnecessary details from user-created effects.

// Macro chain to expand "MyEffectName" to "FMyEffectNameSettings"
#define EFFECT_SETTINGS_NAME2(CLASS_NAME, SUFFIX) F ## CLASS_NAME ## SUFFIX
#define EFFECT_SETTINGS_NAME1(CLASS_NAME, SUFFIX) EFFECT_SETTINGS_NAME2(CLASS_NAME, SUFFIX)
#define EFFECT_SETTINGS_NAME(CLASS_NAME)		  EFFECT_SETTINGS_NAME1(CLASS_NAME, Settings)

#define EFFECT_PRESET_NAME2(CLASS_NAME, SUFFIX)  U ## CLASS_NAME ## SUFFIX
#define EFFECT_PRESET_NAME1(CLASS_NAME, SUFFIX)  EFFECT_PRESET_NAME2(CLASS_NAME, SUFFIX)
#define EFFECT_PRESET_NAME(CLASS_NAME)			 EFFECT_PRESET_NAME1(CLASS_NAME, Preset)

#define GET_EFFECT_SETTINGS(EFFECT_NAME) \
		U##EFFECT_NAME##Preset* _Preset = Cast<U##EFFECT_NAME##Preset>(Preset); \
		F##EFFECT_NAME##Settings Settings = _Preset != nullptr ? _Preset->GetSettings() : F##EFFECT_NAME##Settings(); \

#define EFFECT_PRESET_METHODS(EFFECT_NAME) \
		virtual bool CanFilter() const override { return false; } \
		virtual bool HasAssetActions() const { return true; } \
		virtual FText GetAssetActionName() const override { return FText::FromString(#EFFECT_NAME); } \
		virtual UClass* GetSupportedClass() const override { return EFFECT_PRESET_NAME(EFFECT_NAME)::StaticClass(); } \
		virtual FSoundEffectBase* CreateNewEffect() const override { return new F##EFFECT_NAME; } \
		virtual USoundEffectPreset* CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const override \
		{ \
			USoundEffectPreset* NewPreset = NewObject<EFFECT_PRESET_NAME(EFFECT_NAME)>(InParent, GetSupportedClass(), Name, Flags); \
			NewPreset->Init(); \
			return NewPreset; \
		} \
		virtual void Init() override \
		{ \
			FScopeLock ScopeLock(&SettingsCritSect); \
			SettingsCopy = Settings; \
		} \
		void UpdateSettings(const F##EFFECT_NAME##Settings& InSettings) \
		{ \
			FScopeLock ScopeLock(&SettingsCritSect); \
			SettingsCopy = InSettings; \
			Update(); \
		} \
		void UpdateSettings(TUniqueFunction<void(F##EFFECT_NAME##Settings&)> InCommand) \
		{ \
			FScopeLock ScopeLock(&SettingsCritSect); \
			InCommand(SettingsCopy); \
			Update(); \
		} \
		F##EFFECT_NAME##Settings GetSettings() \
		{ \
			FScopeLock ScopeLock(&SettingsCritSect); \
			return SettingsCopy; \
		} \
		FCriticalSection SettingsCritSect; \
		F##EFFECT_NAME##Settings SettingsCopy; \

class FSoundEffectBase
{
public:
	virtual ~FSoundEffectBase() = default;

	/** Called when the sound effect's preset changed. */
	virtual void OnPresetChanged() { }

	/** Returns if the submix is active or bypassing audio. */
	ENGINE_API bool IsActive() const;

	/** Enables the submix effect. */
	ENGINE_API void SetEnabled(const bool bInIsEnabled);

	/** Updates preset on audio render thread. Returns true if update processed a preset update, false if not. */
	ENGINE_API bool Update();

	ENGINE_API USoundEffectPreset* GetPreset();
	ENGINE_API TWeakObjectPtr<USoundEffectPreset>& GetPresetPtr();

	/** Queries if the given preset object is the uobject preset for this preset instance, i.e. the preset which spawned this effect instance. */
	ENGINE_API bool IsPreset(USoundEffectPreset* InPreset) const;

	/** Enqueues a lambda command on a thread safe queue which is pumped from the audio render thread. */
	ENGINE_API void EffectCommand(TUniqueFunction<void()> Command);

	/** Returns the unique ID of the parent preset. */
	uint32 GetParentPresetId() const { return ParentPresetUniqueId; }

protected:
	ENGINE_API FSoundEffectBase();

	/** Pumps messages awaiting execution on the audio render thread */
	ENGINE_API void PumpPendingMessages();

	FCriticalSection SettingsCritSect;
	TArray<uint8> CurrentAudioThreadSettingsData;

	FThreadSafeBool bChanged;
	TWeakObjectPtr<USoundEffectPreset> Preset;
	uint32 ParentPresetUniqueId = uint32(INDEX_NONE);

	FThreadSafeBool bIsRunning;
	FThreadSafeBool bIsActive;

	// Effect command queue
	TQueue<TUniqueFunction<void()>> CommandQueue;

private:
	/** Removes the instance from the preset. */
	void ClearPreset();

	// Allow preset to re-register when editor update is requested
	// and create effects using the templated Create call, as well
	// as clear preset.
	friend class USoundEffectPreset;
};

