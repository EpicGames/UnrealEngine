// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IAudioExtensionPlugin.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "SoundEffectBase.h"

#include "SoundEffectPreset.generated.h"

// Forward Declarations
struct FAssetData;

class FMenuBuilder;
class FSoundEffectBase;
class IToolkitHost;


UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundEffectPreset : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API USoundEffectPreset(const FObjectInitializer& ObjectInitializer);
	virtual ~USoundEffectPreset() = default;


	virtual bool CanFilter() const { return true; }
	virtual FText GetAssetActionName() const PURE_VIRTUAL(USoundEffectPreset::GetAssetActionName, return FText(););
	virtual UClass* GetSupportedClass() const PURE_VIRTUAL(USoundEffectPreset::GetSupportedClass, return nullptr;);
	virtual USoundEffectPreset* CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const PURE_VIRTUAL(USoundEffectPreset::CreateNewPreset, return nullptr;);
	virtual FSoundEffectBase* CreateNewEffect() const PURE_VIRTUAL(USoundEffectPreset::CreateNewEffect, return nullptr;);
	virtual bool HasAssetActions() const { return false; }
	virtual void Init() PURE_VIRTUAL(USoundEffectPreset::Init, );
	virtual void OnInit() {};
	virtual FColor GetPresetColor() const { return FColor(200, 100, 100); }

	ENGINE_API void Update();
	ENGINE_API void AddEffectInstance(TSoundEffectPtr& InEffectPtr);
	ENGINE_API void RemoveEffectInstance(TSoundEffectPtr& InEffectPtr);

	ENGINE_API void AddReferencedEffects(FReferenceCollector& InCollector);

	ENGINE_API virtual void BeginDestroy() override;

protected:
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	using TSoundEffectWeakPtr = TWeakPtr<FSoundEffectBase, ESPMode::ThreadSafe>;

	// Array of instances which are using this preset
	TArray<TSoundEffectWeakPtr> Instances;
	FCriticalSection InstancesMutationCriticalSection;
	bool bInitialized;

	/* Immediately executes command for each active effect instance on the active thread */
	template <typename T>
	void IterateEffects(TFunction<void(T&)> InForEachEffect)
	{
		FScopeLock ScopeLock(&InstancesMutationCriticalSection);
		for (TSoundEffectWeakPtr& Instance : Instances)
		{
			TSoundEffectPtr EffectStrongPtr = Instance.Pin();
			if (EffectStrongPtr.IsValid())
			{
				InForEachEffect(*static_cast<T*>(EffectStrongPtr.Get()));
			}
		}
	}

	/* Defers execution of command on each active effect instance on the audio render thread */
	template <typename T>
	void EffectCommand(TFunction<void(T&)> InForEachEffect)
	{
		IterateEffects<T>([InForEachEffect](T& OutInstance)
		{
			T* InstancePtr = &OutInstance;
			OutInstance.EffectCommand([InstancePtr, InForEachEffect]()
			{
				InForEachEffect(*InstancePtr);
			});
		});
	}

public:
	// Creates a sound effect instance but does not initialize it.
	template <typename TSoundEffectType>
	static TSharedPtr<TSoundEffectType, ESPMode::ThreadSafe> CreateInstance(USoundEffectPreset& InOutPreset)
	{
		TSoundEffectType* NewEffect = static_cast<TSoundEffectType*>(InOutPreset.CreateNewEffect());
		NewEffect->Preset = &InOutPreset;
		NewEffect->ParentPresetUniqueId = InOutPreset.GetUniqueID();

		TSharedPtr<TSoundEffectType, ESPMode::ThreadSafe> NewEffectPtr(NewEffect);

		TSoundEffectPtr SoundEffectPtr = StaticCastSharedPtr<FSoundEffectBase, TSoundEffectType, ESPMode::ThreadSafe>(NewEffectPtr);
		InOutPreset.AddEffectInstance(SoundEffectPtr);

		return NewEffectPtr;
	}

	// Creates a sound effect instance and initializes it
	template <typename TInitData, typename TSoundEffectType>
	static TSharedPtr<TSoundEffectType, ESPMode::ThreadSafe> CreateInstance(const TInitData& InInitData, USoundEffectPreset& InOutPreset)
	{
		TSoundEffectType* NewEffect = static_cast<TSoundEffectType*>(InOutPreset.CreateNewEffect());
		NewEffect->Preset = &InOutPreset;
		NewEffect->ParentPresetUniqueId = InInitData.ParentPresetUniqueId;

		NewEffect->Setup(InInitData);

		TSharedPtr<TSoundEffectType, ESPMode::ThreadSafe> NewEffectPtr(NewEffect);

		TSoundEffectPtr SoundEffectPtr = StaticCastSharedPtr<FSoundEffectBase, TSoundEffectType, ESPMode::ThreadSafe>(NewEffectPtr);
		InOutPreset.AddEffectInstance(SoundEffectPtr);

		return NewEffectPtr;
	}

	static ENGINE_API void UnregisterInstance(TSoundEffectPtr InEffectPtr);

	static ENGINE_API void RegisterInstance(USoundEffectPreset& InPreset, TSoundEffectPtr InEffectPtr);
};
