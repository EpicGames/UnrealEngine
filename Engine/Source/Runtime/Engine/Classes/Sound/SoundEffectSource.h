// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundEffectBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "SoundEffectSource.generated.h"

class FSoundEffectSource;
class FSoundEffectBase;

/** Preset of a source effect that can be shared between chains. */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundEffectSourcePreset : public USoundEffectPreset
{
	GENERATED_BODY()

public:
	static constexpr int32 DefaultSupportedChannels = 2;

	virtual int32 GetMaxSupportedChannels() const { return DefaultSupportedChannels; }
};

USTRUCT(BlueprintType)
struct FSourceEffectChainEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect")
	TObjectPtr<USoundEffectSourcePreset> Preset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect")
	uint32 bBypass:1;

	FSourceEffectChainEntry()
	: Preset(nullptr)
	, bBypass(false)
	{}
};

/** Chain of source effect presets that can be shared between referencing sounds. */
UCLASS(BlueprintType, MinimalAPI)
class USoundEffectSourcePresetChain : public UObject
{
	GENERATED_BODY()

public:

	/** Chain of source effects to use for this sound source. */
	UPROPERTY(EditAnywhere, Category = "SourceEffect")
	TArray<FSourceEffectChainEntry> Chain;

	/** Whether to keep the source alive for the duration of the effect chain tails. */
	UPROPERTY(EditAnywhere, Category = Effects)
	uint32 bPlayEffectChainTails : 1;

	ENGINE_API void AddReferencedEffects(FReferenceCollector& Collector);

	/**  Get the number of channels this chain supports, which is the lowest channel count of all effects in the chain */
	int32 ENGINE_API GetSupportedChannelCount() const;

	bool ENGINE_API SupportsChannelCount(const int32 InNumChannels) const;

protected:

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};

/** Data required to initialize the source effect. */
struct FSoundEffectSourceInitData
{
	// Sample rate of the audio renderer this effect is processing on
	float SampleRate = 0.0f;

	// The number of channels of the source audio that is fed into the source effect
	int32 NumSourceChannels = 0;

	// The audio clock of the audio renderer this effect is processing on
	double AudioClock = 0.0;

	// The object id of the parent preset
	uint32 ParentPresetUniqueId = uint32(INDEX_NONE);

	// The audio device ID of the audio device instance this source instance was created from
	uint32 AudioDeviceId = uint32(INDEX_NONE);
};

/** Data required to update the source effect. */
struct FSoundEffectSourceInputData
{
	float CurrentVolume;
	float CurrentPitch;
	double AudioClock;
	float CurrentPlayFraction;
	FSpatializationParams SpatParams;
	float* InputSourceEffectBufferPtr;
	int32 NumSamples;

	FSoundEffectSourceInputData()
		: CurrentVolume(0.0f)
		, CurrentPitch(0.0f)
		, AudioClock(0.0)
		, SpatParams(FSpatializationParams())
		, InputSourceEffectBufferPtr(nullptr)
		, NumSamples(0)
	{
	}
};

class FSoundEffectSource : public FSoundEffectBase
{
public:
	virtual ~FSoundEffectSource() = default;

	// Called by the audio engine or systems internally. This function calls the virtual Init function implemented by derived classes.
	void Setup(const FSoundEffectSourceInitData& InInitData)
	{
		// Store the init data internally
		InitData_Internal = InInitData;

		// This may have been set before this call, so set it on the init data struct
		if (ParentPresetUniqueId != INDEX_NONE)
		{
			InitData_Internal.ParentPresetUniqueId = ParentPresetUniqueId;
		}
		// Call the virtual function
		Init(InitData_Internal);
	}

	// Returns the data that was given to the source effect when initialized.
	const FSoundEffectSourceInitData& GetInitializedData() const
	{
		// Return our copy of the initialization data
		return InitData_Internal;
	}

	// Process the input block of audio. Called on audio thread.
	virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) = 0;

	friend class USoundEffectPreset;

private:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	virtual void Init(const FSoundEffectSourceInitData& InInitData) = 0;

	// Copy of data used to initialize the source effect.
	// Can result in init being called again if init conditions have changed (Sample rate changed, channel count changed, etc)
	FSoundEffectSourceInitData InitData_Internal;
};
