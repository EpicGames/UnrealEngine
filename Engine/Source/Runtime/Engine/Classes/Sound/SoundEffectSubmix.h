// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioDefines.h"
#include "DSP/BufferVectorOperations.h"
#include "IAudioExtensionPlugin.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundEffectBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "SoundEffectSubmix.generated.h"

// Forward Declarations
class FAudioDevice;
class FSoundEffectSubmix;

struct FAudioEffectParameters;


/** Preset of a submix effect that can be shared between sounds. */
UCLASS(config = Engine, hidecategories = Object, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundEffectSubmixPreset : public USoundEffectPreset
{
	GENERATED_UCLASS_BODY()

	virtual FColor GetPresetColor() const override { return FColor(162, 84, 101); }

};

/** Struct which has data needed to initialize the submix effect. */
struct FSoundEffectSubmixInitData
{
	uint32 DeviceID = INDEX_NONE;
	void* PresetSettings = nullptr;
	float SampleRate = 0.0f;
	uint32 ParentPresetUniqueId = INDEX_NONE;
};

/** Struct which supplies audio data to submix effects on game thread. */
struct FSoundEffectSubmixInputData
{
	/** Ptr to preset data if new data is available. This will be nullptr if no new preset data has been set. */
	void* PresetData;
	
	/** The number of audio frames for this input data. 1 frame is an interleaved sample. */
	int32 NumFrames;

	/** The number of channels of the submix. */
	int32 NumChannels;

	/** The number of device channels. */
	int32 NumDeviceChannels;

	/** The listener transforms (one for each viewport index). */
	const TArray<FTransform>* ListenerTransforms;

	/** The raw input audio buffer. Size is NumFrames * NumChannels */
	Audio::FAlignedFloatBuffer* AudioBuffer;

	/** Sample accurate audio clock. */
	double AudioClock;

	FSoundEffectSubmixInputData()
		: PresetData(nullptr)
		, NumFrames(0)
		, NumChannels(0)
		, NumDeviceChannels(0)
		, ListenerTransforms(nullptr)
		, AudioBuffer(nullptr)
	{}
};

struct FSoundEffectSubmixOutputData
{
	/** The output audio buffer. */
	Audio::FAlignedFloatBuffer* AudioBuffer;

	/** The number of channels of the submix. */
	int32 NumChannels;
};

class FSoundEffectSubmix : public FSoundEffectBase
{
public:
	virtual ~FSoundEffectSubmix() = default;

	//  Provided for interpolating parameters from audio volume system, enabling transition between various settings
	virtual bool SetParameters(const FAudioEffectParameters& InParameters)
	{
		return false;
	}

	// Whether or not effect supports default reverb system
	virtual bool SupportsDefaultReverb() const
	{
		return false;
	}

	// Whether or not effect supports default EQ system
	virtual bool SupportsDefaultEQ() const
	{
		return false;
	}

	// Called on game thread to allow submix effect to query game data if needed.
	virtual void Tick()
	{
	}

	// Override to down mix input audio to a desired channel count.
	virtual uint32 GetDesiredInputChannelCountOverride() const
	{
		return INDEX_NONE;
	}

	// Process the input block of audio. Called on audio thread.
	virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
	{
	}

	// Allow effects to supply a drylevel.
	virtual float GetDryLevel() const { return 0.0f; }

	// Processes audio in the submix effect.
	//
	// If the audio cannot be processed, this function will return false and OutData will not be altered. 
	ENGINE_API bool ProcessAudio(FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData);

	friend class USoundEffectPreset;

	// Called by the audio engine or systems internally. This function calls the virtual Init function implemented by derived classes.
	void Setup(const FSoundEffectSubmixInitData& InInitData)
	{
		InitData_Internal = InInitData;
		Init(InInitData);
	}

	// Returns the data that was given to the source effect when initialized.
	const FSoundEffectSubmixInitData& GetInitData() const
	{
		return InitData_Internal;
	}

protected:
	FSoundEffectSubmix()
	{
	}

private:
	/** Called on an audio effect at initialization on main thread before audio processing begins. */
	virtual void Init(const FSoundEffectSubmixInitData& InInitData)
	{
	}

	FSoundEffectSubmixInitData InitData_Internal;
};
