// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceManager.h"
#include "CoreMinimal.h"
#include "Containers/MpscQueue.h"
#include "Templates/Function.h"

namespace Audio
{
	// UE_DEPRECATED(5.8, "Synchronized audio task queues are deprecated.")
	// Commented out because UE_DEPRECATED cannot be applied to a using directive, but should be considered deprecated.
	using AudioTaskQueueId = uint32;
}

// Parameters used for constructing a new ISoundGenerator.
struct FSoundGeneratorInitParams
{
	Audio::FDeviceId AudioDeviceID;
	uint64 AudioComponentId = 0;
	float SampleRate = 0.0f;
	int32 AudioMixerNumOutputFrames = 0;
	int32 NumChannels = 0;
	int32 NumFramesPerCallback = 0;
	uint64 InstanceID = 0;
	bool bIsPreviewSound = false;
	FString GraphName;
	float StartTime = 0.0f;
};

class ISoundGenerator
{
public:
	ENGINE_API ISoundGenerator();
	ENGINE_API virtual ~ISoundGenerator();

	// Called when a new buffer is required. 
	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) = 0;

	// Returns the number of samples to render per callback
	virtual int32 GetDesiredNumSamplesToRenderPerCallback() const { return 1024; }

	// Optional. Called on audio generator thread right when the generator begins generating.
	virtual void OnBeginGenerate() {}

	// Optional. Called on audio generator thread right when the generator ends generating.
	virtual void OnEndGenerate() {}

	// Optional. Can be overridden to end the sound when generating is finished.
	virtual bool IsFinished() const { return false; }

	// Return the cost to render this sound generator. Derived classes can optionally
	// override this method.
	//
	// This is called repeatedly to track the cost and managing the maximum number
	// of concurrently rendering voices. 
	virtual float GetRelativeRenderCost() const { return 1.f; }

	// Retrieves the next buffer of audio from the generator, called from the audio mixer.
	// Returns the number of samples actually written. The remainder are untouched.
	ENGINE_API int32 GetNextBuffer(float* OutAudio, int32 NumSamples, bool bRequireNumberSamples = false);

	UE_DEPRECATED(5.8, "Synchronized audio task queues are deprecated.")
	virtual Audio::AudioTaskQueueId GetSynchronizedRenderQueueId() const { return 0; }

	// Currently only used by generator source rendering, must be implemented to be used with a generator source.
	virtual int32 GetNumChannels() const { return 0; }

protected:

	// Protected method to execute lambda in audio render thread
	// Used for conveying parameter changes or events to the generator thread.
	ENGINE_API void SynthCommand(TFunction<void()> Command);

private:

	ENGINE_API void PumpPendingMessages();

	// The command queue used to convey commands from game thread to generator thread 
	TMpscQueue<TUniqueFunction<void()>> CommandQueue;

	friend class USynthComponent;
};

// Null implementation of ISoundGenerator which no-ops audio generation
class FSoundGeneratorNull : public ISoundGenerator
{
public:
	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override
	{
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
		return NumSamples;
	}
};

typedef TSharedPtr<ISoundGenerator, ESPMode::ThreadSafe> ISoundGeneratorPtr;
typedef TSharedRef<ISoundGenerator, ESPMode::ThreadSafe> ISoundGeneratorRef;
