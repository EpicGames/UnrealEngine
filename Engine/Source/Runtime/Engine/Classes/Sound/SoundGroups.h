// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SoundGroups.generated.h"

UENUM()
enum ESoundGroup : int
{
	SOUNDGROUP_Default UMETA(DisplayName="Default"),
	SOUNDGROUP_Effects UMETA(DisplayName="Effects"),
	SOUNDGROUP_UI UMETA(DisplayName="UI"),
	SOUNDGROUP_Music UMETA(DisplayName="Music"),
	SOUNDGROUP_Voice UMETA(DisplayName="Voice"),

	SOUNDGROUP_GameSoundGroup1 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup2 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup3 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup4 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup5 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup6 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup7 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup8 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup9 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup10 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup11 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup12 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup13 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup14 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup15 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup16 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup17 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup18 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup19 UMETA(Hidden),
	SOUNDGROUP_GameSoundGroup20 UMETA(Hidden),
};

USTRUCT()
struct FSoundGroup
{
	GENERATED_USTRUCT_BODY()

	// The sound group enumeration we are setting values for
	UPROPERTY(config)
	TEnumAsByte<ESoundGroup> SoundGroup;

	// An override display name for custom game sound groups
	UPROPERTY(config)
	FString DisplayName;

	// Whether sounds in this group should always decompress on load
	UPROPERTY(config)
	uint32 bAlwaysDecompressOnLoad:1;

	/**
	 * Sound duration in seconds below which sounds are entirely expanded to PCM at load time
	 * Disregarded if bAlwaysDecompressOnLoad is true
	*/
	UPROPERTY(config)
	float DecompressedDuration;

	FSoundGroup()
		: SoundGroup(SOUNDGROUP_Default)
		, bAlwaysDecompressOnLoad(false)
		, DecompressedDuration(0.f)
	{
	}

};

// This class is a singleton initialized from the ini
UCLASS(config=Engine, abstract, MinimalAPI)
class USoundGroups : public UObject
{
	GENERATED_UCLASS_BODY()

	ENGINE_API void Initialize() const;

	ENGINE_API const FSoundGroup& GetSoundGroup(const ESoundGroup SoundGroup) const;

private:

	// The ini editable array of profiles
	UPROPERTY(config)
	TArray<FSoundGroup> SoundGroupProfiles;

	// Easy lookup mechanism for sound group settings
	mutable TMap<ESoundGroup, FSoundGroup> SoundGroupMap;

};
