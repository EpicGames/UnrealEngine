// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AudioDefines.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SoundMix.generated.h"

class USoundClass;
struct FPropertyChangedEvent;

USTRUCT()
struct FAudioEffectParameters
{
	GENERATED_USTRUCT_BODY()

public:
	FAudioEffectParameters()
	{
	}

	virtual ~FAudioEffectParameters() = default;

	// Interpolates between one set of parameters and another and stores result in local copy
	virtual bool Interpolate(const FAudioEffectParameters& InStart, const FAudioEffectParameters& InEnd)
	{
		return false;
	}

	// Prints effect parameters
	virtual void PrintSettings() const
	{
	}
};

USTRUCT()
struct FAudioEQEffect : public FAudioEffectParameters
{
	GENERATED_USTRUCT_BODY()

	/* Start time of effect */
	double RootTime;

	/** Center frequency in Hz for band 0 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FrequencyCenter0;
	
	/** Boost/cut of band 0 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float Gain0;

	/** Bandwidth of band 0. Region is center frequency +/- Bandwidth /2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0"))
	float Bandwidth0;

	/** Center frequency in Hz for band 1 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FrequencyCenter1;

	/** Boost/cut of band 1 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float Gain1;

	/** Bandwidth of band 1. Region is center frequency +/- Bandwidth /2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0"))
	float Bandwidth1;

	/** Center frequency in Hz for band 2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FrequencyCenter2;

	/** Boost/cut of band 2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float Gain2;

	/** Bandwidth of band 2. Region is center frequency +/- Bandwidth /2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0"))
	float Bandwidth2;

	/** Center frequency in Hz for band 3 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FrequencyCenter3;

	/** Boost/cut of band 3 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float Gain3;

	/** Bandwidth of band 3. Region is center frequency +/- Bandwidth /2 */
	UPROPERTY(EditAnywhere, Category = Band0, meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0"))
	float Bandwidth3;

	FAudioEQEffect()
		: RootTime(0.0f)
		, FrequencyCenter0(600.0f)
		, Gain0(1.0f)
		, Bandwidth0(1.0f)
		, FrequencyCenter1(1000.0f)
		, Gain1(1.0f)
		, Bandwidth1(1.0f)
		, FrequencyCenter2(2000.0f)
		, Gain2(1.0f)
		, Bandwidth2(1.0f)
		, FrequencyCenter3(10000.0f)
		, Gain3(1.0f)
		, Bandwidth3(1.0f)
	{}

	/** 
	 * Interpolates between Start and End EQ effect settings, storing results locally and returning if interpolation is complete
	 */
	bool Interpolate(const FAudioEffectParameters& InStart, const FAudioEffectParameters& InEnd) override;
		
	/** 
	* Clamp all settings in range
	*/
	void ClampValues();

	virtual void PrintSettings() const override;
};

/**
 * Elements of data for sound group volume control
 */
USTRUCT(BlueprintType)
struct FSoundClassAdjuster
{
	GENERATED_USTRUCT_BODY()

	/* The sound class this adjuster affects. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClassAdjuster, DisplayName = "Sound Class" )
	TObjectPtr<USoundClass> SoundClassObject;

	/* A multiplier applied to the volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClassAdjuster, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "4.0"))
	float VolumeAdjuster;

	/* A multiplier applied to the pitch. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClassAdjuster, meta = (ClampMin = "0.0", ClampMax = "8.0", UIMin = "0.0", UIMax = "8.0"))
	float PitchAdjuster;

	/* Lowpass filter cutoff frequency to apply to sound sources. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundClassAdjuster, meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float LowPassFilterFrequency;

	/* Set to true to apply this adjuster to all children of the sound class. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClassAdjuster )
	uint32 bApplyToChildren:1;

	/* A multiplier applied to VoiceCenterChannelVolume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClassAdjuster, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "4.0"))
	float VoiceCenterChannelVolumeAdjuster;

	FSoundClassAdjuster()
		: SoundClassObject(NULL)
		, VolumeAdjuster(1)
		, PitchAdjuster(1)
		, LowPassFilterFrequency(MAX_FILTER_FREQUENCY)
		, bApplyToChildren(false)
		, VoiceCenterChannelVolumeAdjuster(1)
		{
		}
	
};

UCLASS(BlueprintType, hidecategories=object, MinimalAPI)
class USoundMix : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Whether to apply the EQ effect */
	UPROPERTY(EditAnywhere, Category=EQ, AssetRegistrySearchable )
	uint32 bApplyEQ:1;

	UPROPERTY(EditAnywhere, Category=EQ)
	float EQPriority;

	UPROPERTY(EditAnywhere, Category=EQ)
	struct FAudioEQEffect EQSettings;

	/* Array of changes to be applied to groups. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SoundClasses)
	TArray<struct FSoundClassAdjuster> SoundClassEffects;

	/* Initial delay in seconds before the mix is applied. */
	UPROPERTY(EditAnywhere, Category=SoundMix )
	float InitialDelay;

	/* Time taken in seconds for the mix to fade in. */
	UPROPERTY(EditAnywhere, Category=SoundMix )
	float FadeInTime;

	/* Duration of mix, negative means it will be applied until another mix is set. */
	UPROPERTY(EditAnywhere, Category=SoundMix )
	float Duration;

	/* Time taken in seconds for the mix to fade out. */
	UPROPERTY(EditAnywhere, Category=SoundMix )
	float FadeOutTime;

#if WITH_EDITORONLY_DATA
	/** Transient property used to trigger real-time updates of the active EQ filter for editor previewing */
	UPROPERTY(transient)
	uint32 bChanged:1;
#endif

#if WITH_EDITOR
	bool CausesPassiveDependencyLoop(TArray<USoundClass*>& ProblemClasses) const;
#endif // WITH_EDITOR

protected:
	//~ Begin UObject Interface.
	virtual FString GetDesc( void ) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void BeginDestroy() override;
	//~ End UObject Interface.

#if WITH_EDITOR
	bool CheckForDependencyLoop(USoundClass* SoundClass, TArray<USoundClass*>& ProblemClasses, bool CheckChildren) const;
#endif // WITH_EDITOR
};



