// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "DSP/BufferVectorOperations.h"
#include "HAL/CriticalSection.h"
#include "IAudioModulation.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "SoundModulationDestination.generated.h"


// Forward Declarations
class USoundModulatorBase;

namespace Audio
{
	struct FModulationParameter;
}


UENUM(BlueprintType)
enum class EModulationRouting : uint8
{
	/* Disables modulation routing */
	Disable,

	/* Inherits modulation routing (AudioComponent inherits from Sound, Sound inherits from SoundClass) */
	Inherit,

	/* Ignores inherited settings and uses modulation settings on this object */
	Override,

	/* Performs set union on local modulation sources with those inherited (AudioComponent inherits from Sound, Sound inherits from SoundClass) */
	Union
};

UENUM(BlueprintType)
enum class EModulationDestination : uint8
{
	/* Volume modulation */
	Volume,

	/* Pitch modulation */
	Pitch,

	/* Cutoff Frequency of a lowpass filter */
	Lowpass,

	/* Cutoff Frequency of a highpass filter */
	Highpass,

	Count UMETA(Hidden)
};

/** Parameter destination settings allowing modulation control override for parameter destinations opting in to the Modulation System. */
USTRUCT(BlueprintType)
struct FSoundModulationDestinationSettings
{
	GENERATED_USTRUCT_BODY()

	/** Base value of parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modulation)
	float Value = 1.0f;

#if WITH_EDITORONLY_DATA
	/** Whether or not to enable modulation */
	UPROPERTY(EditAnywhere, Category = Modulation, meta = (DisplayName = "Modulate"))
	bool bEnableModulation = false;

	UE_DEPRECATED(5.3, "Use the Modulators set property instead.")
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<USoundModulatorBase> Modulator = nullptr;
#endif // WITH_EDITORONLY_DATA

	/** Set of modulation sources, which provides values to mix with base value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modulation)
	TSet<TObjectPtr<USoundModulatorBase>> Modulators;

#if WITH_EDITORONLY_DATA
	/** Versioning utility function to upgrade single modulator field to set of modulators */
	ENGINE_API void VersionModulators();
#endif // WITH_EDITORONLY_DATA

	// We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSoundModulationDestinationSettings() = default;
	~FSoundModulationDestinationSettings() = default;
	FSoundModulationDestinationSettings(const FSoundModulationDestinationSettings&) = default;
	FSoundModulationDestinationSettings(FSoundModulationDestinationSettings&&) = default;
	FSoundModulationDestinationSettings& operator=(const FSoundModulationDestinationSettings&) = default;
	FSoundModulationDestinationSettings& operator=(FSoundModulationDestinationSettings&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

};

/** Default parameter destination settings for source audio object. */
USTRUCT(BlueprintType)
struct FSoundModulationDefaultSettings
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FSoundModulationDefaultSettings();

	/** Volume modulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta = (DisplayName = "Volume", AudioParam = "Volume", AudioParamClass = "SoundModulationParameterVolume"))
	FSoundModulationDestinationSettings VolumeModulationDestination;

	/** Pitch modulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta = (DisplayName = "Pitch", AudioParam = "Pitch", AudioParamClass = "SoundModulationParameterBipolar"))
	FSoundModulationDestinationSettings PitchModulationDestination;

	/** Highpass modulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta = (DisplayName = "Highpass", AudioParam = "HPFCutoffFrequency", AudioParamClass = "SoundModulationParameterHPFFrequency"))
	FSoundModulationDestinationSettings HighpassModulationDestination;

	/** Lowpass modulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta = (DisplayName = "Lowpass", AudioParam = "LPFCutoffFrequency", AudioParamClass = "SoundModulationParameterLPFFrequency"))
	FSoundModulationDestinationSettings LowpassModulationDestination;

	FSoundModulationDestinationSettings GetSettingsForDestination(EModulationDestination Destination) const;
	
#if WITH_EDITORONLY_DATA
	ENGINE_API void VersionModulators();
#endif // WITH_EDITORONLY_DATA
};

/** Default parameter destination settings for source audio object. */
USTRUCT(BlueprintType)
struct FSoundModulationDefaultRoutingSettings : public FSoundModulationDefaultSettings
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FSoundModulationDefaultRoutingSettings();

	/** What volume modulation settings to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta=(DisplayPriority=100))
	EModulationRouting VolumeRouting = EModulationRouting::Inherit;

	/** What pitch modulation settings to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta=(DisplayPriority=100))
	EModulationRouting PitchRouting = EModulationRouting::Inherit;

	/** What high-pass modulation settings to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta=(DisplayPriority=100))
	EModulationRouting HighpassRouting = EModulationRouting::Inherit;

	/** What low-pass modulation settings to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation", meta=(DisplayPriority=100))
	EModulationRouting LowpassRouting = EModulationRouting::Inherit;
	
	EModulationRouting GetRoutingForDestination(EModulationDestination Destination) const;
};

namespace Audio
{
	struct FModulationDestination
	{
		public:
			FModulationDestination() = default;

			ENGINE_API FModulationDestination(const FModulationDestination& InModulationDestination);
			ENGINE_API FModulationDestination(FModulationDestination&& InModulationDestination);

			ENGINE_API FModulationDestination& operator=(const FModulationDestination& InModulationDestination);
			ENGINE_API FModulationDestination& operator=(FModulationDestination&& InModulationDestination);

			/** Initializes the modulation destination
			 * InDeviceId - DeviceId associated with modulation plugin instance
			 * bInIsBuffered - Whether or not to run destination in "buffered mode," which manages an internal buffer to smooth modulation value between process calls
			 * bInValueNormalized - Whether or not to keep the output value in normalized, unitless [0.0f, 1.0f] space
			 */
			ENGINE_API void Init(FDeviceId InDeviceId, bool bInIsBuffered = false, bool bInValueNormalized = false);

			/** Initializes the modulation destination
			 * InDeviceId - DeviceId associated with modulation plugin instance
			 * InParameterName - Name of parameter used to mix/convert destination value to/from unit space
			 * bInIsBuffered - Whether or not to run destination in "buffered mode," which manages an internal buffer to smooth modulation value between process calls
			 * bInValueNormalized - Whether or not to keep the output value in normalized, unitless [0.0f, 1.0f] space
			 */
			ENGINE_API void Init(FDeviceId InDeviceId, FName InParameterName, bool bInIsBuffered = false, bool bInValueNormalized = false);

			/** returns whether or not destination references an active modulator */
			ENGINE_API bool IsActive() const;

			/* Updates internal value (or buffer if set to bIsBuffered) to current modulated result using the provided value as the base carrier value to modulate.
			 * Returns true if value was updated.
			 */
			ENGINE_API bool ProcessControl(float InValueUnitBase, int32 InNumSamples = 0);


			UE_DEPRECATED(5.1, "Deprecated in favor of supporting multiple modulators per destination. Use 'SetModulators' instead.")
			ENGINE_API void UpdateModulator(const USoundModulatorBase* InModulator);

			UE_DEPRECATED(5.8, "This function has been renamed SetModulators.")
			ENGINE_API void UpdateModulators(const TSet<TObjectPtr<USoundModulatorBase>>& InModulators);

			UE_DEPRECATED(5.8, "This function has been renamed SetModulators.")
			ENGINE_API void UpdateModulators(const TSet<USoundModulatorBase*>& InModulators);

			UE_DEPRECATED(5.8, "This function has been renamed SetModulators.")
			ENGINE_API void UpdateModulators(const TSet<const USoundModulatorBase*>& InModulators);

			/**	Updates the set of Modulators on this Destination. */
			ENGINE_API void SetModulators(const TSet<TObjectPtr<USoundModulatorBase>>& InModulators);
			ENGINE_API void SetModulators(const TSet<USoundModulatorBase*>& InModulators);
			ENGINE_API void SetModulators(const TSet<const USoundModulatorBase*>& InModulators);
		
			ENGINE_API void ClearModulators();

	private:
			ENGINE_API void UpdateModulatorsInternal(TArray<TUniquePtr<Audio::IModulatorSettings>>&& ProxySettings);

			struct FModulationDestinationData
			{
				FDeviceId DeviceId = FDeviceId(INDEX_NONE);

				float ValueTarget = 1.0f;

				bool bIsBuffered = false;
				bool bValueNormalized = false;
				bool bHasProcessed = false;

				FAlignedFloatBuffer OutputBuffer;

				TSet<FModulatorHandle> Handles;

				FModulationParameter Parameter;

				mutable FCriticalSection HandleCritSection;

				FModulationDestinationData& operator=(const FModulationDestinationData& InDestInfo);
				FModulationDestinationData& operator=(FModulationDestinationData&& InDestInfo);
				const FDeviceId& GetDeviceId() const;
				const FModulationParameter& GetParameter() const;
				void SetHandles(TSet<FModulatorHandle>&& Handles);
				void ResetHandles();
			};

			TSharedRef<FModulationDestinationData> DestinationData{ MakeShared<FModulationDestinationData>() };

		public:
			/** Returns buffer of interpolated modulation values. If not set to "IsBuffered" when initialized, returns an empty array. */
			inline const FAlignedFloatBuffer& GetBuffer() const
			{
				return DestinationData->OutputBuffer;
			}

			/** Returns whether or not the destination has requested to 
			  * process the control or not. */
			inline bool GetHasProcessed() const
			{
				return DestinationData->bHasProcessed;
			}

			/** Returns sample value last reported by modulator. Returns value in unit space, unless
			 * 'ValueNormalized' option is set on initialization.
			 */
			inline float GetValue() const
			{
				return DestinationData->ValueTarget;
			}
	};
} // namespace Audio
