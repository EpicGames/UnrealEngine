// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Math/RandomStream.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Sound/SoundWave.h"
#include "SoundNode.generated.h"

class FAudioDevice;
class UEdGraphNode;
struct FActiveSound;
struct FPropertyChangedEvent;
struct FWaveInstance;

/*-----------------------------------------------------------------------------
	USoundNode helper macros.
-----------------------------------------------------------------------------*/

struct FActiveSound;

#define DECLARE_SOUNDNODE_ELEMENT(Type,Name)													\
	Type& Name = *((Type*)(Payload));															\
	Payload += sizeof(Type);

#define DECLARE_SOUNDNODE_ELEMENT_PTR(Type,Name)												\
	Type* Name = (Type*)(Payload);																\
	Payload += sizeof(Type);

#define	RETRIEVE_SOUNDNODE_PAYLOAD( Size )														\
		uint8*	Payload					= NULL;													\
		uint32*	RequiresInitialization	= NULL;													\
		{																						\
			uint32* TempOffset = ActiveSound.SoundNodeOffsetMap.Find(NodeWaveInstanceHash);		\
			uint32 Offset;																		\
			if( !TempOffset )																	\
			{																					\
				Offset = ActiveSound.SoundNodeData.AddZeroed( Size + sizeof(uint32));				\
				ActiveSound.SoundNodeOffsetMap.Add( NodeWaveInstanceHash, Offset );				\
				RequiresInitialization = (uint32*) &ActiveSound.SoundNodeData[Offset];			\
				*RequiresInitialization = 1;													\
				Offset += sizeof(uint32);															\
			}																					\
			else																				\
			{																					\
				RequiresInitialization = (uint32*) &ActiveSound.SoundNodeData[*TempOffset];		\
				Offset = *TempOffset + sizeof(uint32);											\
			}																					\
			Payload = &ActiveSound.SoundNodeData[Offset];										\
		}

UCLASS(abstract, hidecategories=Object, editinlinenew, MinimalAPI)
class USoundNode : public UObject
{
	GENERATED_UCLASS_BODY()

	static const int32 MAX_ALLOWED_CHILD_NODES = 32;

	UPROPERTY(BlueprintReadOnly, Category = "SoundNode")
	TArray<TObjectPtr<class USoundNode>> ChildNodes;

#if WITH_EDITORONLY_DATA
	/** Node's Graph representation, used to get position. */
	UPROPERTY()
	TObjectPtr<UEdGraphNode>	GraphNode;

	ENGINE_API class UEdGraphNode* GetGraphNode() const;
#endif

	/** Stream of random numbers to be used by this instance of USoundNode */
	FRandomStream RandomStream;

	bool bIsRetainingAudio;

public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostLoad() override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif //WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual bool CanBeClusterRoot() const override;
	ENGINE_API virtual bool CanBeInCluster() const override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//
	//~ Begin USoundNode Interface.
	//

	/**
	 * Notifies the sound node that a wave instance in its subtree has finished.
	 *
	 * @param WaveInstance	WaveInstance that was finished
	 */
	virtual bool NotifyWaveInstanceFinished( struct FWaveInstance* WaveInstance )
	{
		return( false );
	}

	/**
	 * Returns the maximum distance this sound can be heard from.
	 */
	ENGINE_API virtual float GetMaxDistance() const;

	/** Returns if this node has been set to be allowed virtual. Only the sound node wave player implements this. */
	virtual bool SupportsSubtitles() const
	{
		return false;
	}

	/**
	 * Returns the maximum duration this sound node will play for.
	 *
	 * @return	float of number of seconds this sound will play for. INDEFINITELY_LOOPING_DURATION means its looping.
	 */
	ENGINE_API virtual float GetDuration();

	/** Returns whether the sound cue has a delay node. */
	ENGINE_API virtual bool HasDelayNode() const;

	/** Returns whether the sound has a sequencer node. */
	ENGINE_API virtual bool HasConcatenatorNode() const;

	/** Returns whether the sound has an attenuation node. */
	ENGINE_API virtual bool HasAttenuationNode() const;

	/** Returns true if the sound node is set to play when silent. */
	ENGINE_API virtual bool IsPlayWhenSilent() const;

	ENGINE_API virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const struct FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances );

	/**
	 * Returns an array of all (not just active) nodes.
	 */
	ENGINE_API virtual void GetAllNodes( TArray<USoundNode*>& SoundNodes );

	/**
	 * Returns the maximum number of child nodes this node can possibly have
	 */
	virtual int32 GetMaxChildNodes() const
	{
		return 1 ;
	}

	/** Returns the minimum number of child nodes this node must have */
	virtual int32 GetMinChildNodes() const
	{
		return 0;
	}

	/** Returns the number of simultaneous sounds this node instance plays back. */
	ENGINE_API virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const;


	/**
	 * Editor interface.
	 */

	/**
	 * Called by the Sound Cue Editor for nodes which allow children.  The default behaviour is to
	 * attach a single connector. Dervied classes can override to eg add multiple connectors.
	 */
	ENGINE_API virtual void CreateStartingConnectors( void );
	ENGINE_API virtual void InsertChildNode( int32 Index );
	ENGINE_API virtual void RemoveChildNode( int32 Index );
#if WITH_EDITOR
	/**
	 * Set the entire Child Node array directly, allows GraphNodes to fully control node layout.
	 * Can be overwritten to set up additional parameters that are tied to children.
	 */
	ENGINE_API virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes);

	/** Get the name of a specific input pin */
	virtual FText GetInputPinName(int32 PinIndex) const { return FText::GetEmpty(); }

	virtual FText GetTitle() const { return GetClass()->GetDisplayNameText(); }

	/** Helper function to set the position of a sound node on a grid */
	ENGINE_API void PlaceNode(int32 NodeColumn, int32 NodeRow, int32 RowCount );

	/** Called as PIE begins */
	virtual void OnBeginPIE(const bool bIsSimulating) {};

	/** Called as PIE ends */
	virtual void OnEndPIE(const bool bIsSimulating) {};
#endif //WITH_EDITOR

	/**
	 * Used to create a unique string to identify unique nodes
	 */
	static ENGINE_API UPTRINT GetNodeWaveInstanceHash(const UPTRINT ParentWaveInstanceHash, const USoundNode* ChildNode, const uint32 ChildIndex);
	static ENGINE_API UPTRINT GetNodeWaveInstanceHash(const UPTRINT ParentWaveInstanceHash, const UPTRINT ChildNodeHash, const uint32 ChildIndex);


	/**
	 * When this is called and stream caching is enabled,
	 * any wave player sound nodes childed off of this node
	 * will have their audio loaded into the cache.
	 *
	 * @param bRecurse when true, this will cause all children of child nodes to be primed as well.
	 */
	ENGINE_API virtual void PrimeChildWavePlayers(bool bRecurse);

	/**
	 * When this is called and stream caching is enabled,
	 * any wave player sound nodes childed off of this node
	 * will have their audio retained into the cache.
	 *
	 * @param bRecurse when true, this will cause all children of child nodes to be primed as well.
	 */
	ENGINE_API virtual void RetainChildWavePlayers(bool bRecurse);

	/**
	 * When this is called and stream caching is enabled,
	 * any wave player sound nodes childed off of this node
	 * with loading behavior set to "Inherited"
	 * will have their loading behavior updated and
	 * their bLoadingBehaviorOverridden flag raised
	 *
	 * @param bRecurse when true, this will cause all children of child nodes to be overridden as well.
	 */
	ENGINE_API virtual void OverrideLoadingBehaviorOnChildWaves(const bool bRecurse, const ESoundWaveLoadingBehavior InLoadingBehavior);

	ENGINE_API virtual void ReleaseRetainerOnChildWavePlayers(bool bRecurse);

	/**
	 * When called, this will find any child wave players connected to this node
	 * and null out their associated USoundWave, allowing the USoundWave to be garbage collected.
	 * This should only be called by USoundNodeQualityLevel::PostLoad when au.CullSoundWaveHardReferences is 1.
	 */
	ENGINE_API virtual void RemoveSoundWaveOnChildWavePlayers();

	/**
	 * Load wave assets for any wave player sound nodes 
	 * childed off of this node, taking into account quality nodes
	 *
	 * @param bRecurse when true, this will cause all children of child nodes to be loaded as well.
	 */
	ENGINE_API virtual void LoadChildWavePlayerAssets(bool bAddToRoot, bool bRecurse);
};

