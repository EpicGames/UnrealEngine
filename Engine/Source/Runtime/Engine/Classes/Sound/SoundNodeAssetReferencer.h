// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeAssetReferencer.generated.h"

/** 
 * Sound node that contains a reference to the raw wave file to be played
 */
UCLASS(abstract, MinimalAPI)
class USoundNodeAssetReferencer : public USoundNode
{
	GENERATED_BODY()

public:
	virtual void LoadAsset(bool bAddToRoot = false) PURE_VIRTUAL(USoundNodeAssetReferencer::LoadAsset,);
	virtual void ClearAssetReferences() PURE_VIRTUAL(USoundNodeAssetReferencer::ClearAssetReferences, );
	virtual bool ContainsProceduralSoundReference() const { return false; }

	ENGINE_API bool ShouldHardReferenceAsset(const class ITargetPlatform*) const;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditImport() override;
#endif
};

