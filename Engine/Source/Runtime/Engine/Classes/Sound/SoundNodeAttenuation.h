// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundNode.h"
#include "SoundNodeAttenuation.generated.h"

struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines how a sound's volume changes based on distance to the listener
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Attenuation" ))
class USoundNodeAttenuation : public USoundNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Attenuation, meta=(EditCondition="!bOverrideAttenuation"))
	TObjectPtr<USoundAttenuation> AttenuationSettings;

	UPROPERTY(EditAnywhere, Category=Attenuation, meta=(EditCondition="bOverrideAttenuation"))
	FSoundAttenuationSettings AttenuationOverrides;

	UPROPERTY(EditAnywhere, Category=Attenuation)
	uint32 bOverrideAttenuation:1;

public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual float GetMaxDistance() const override;
	virtual bool HasAttenuationNode() const override { return true; }
	//~ End USoundNode Interface. 

	ENGINE_API const FSoundAttenuationSettings* GetAttenuationSettingsToApply() const;
};



