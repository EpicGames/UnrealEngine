// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeBranch.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Selects a child node based on the value of a boolean parameter
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Branch" ))
class USoundNodeBranch : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/** The name of the boolean parameter to use to determine which branch we should take */
	UPROPERTY(EditAnywhere, Category=Branch)
	FName BoolParameterName;

private:

	enum BranchPurpose
	{
		ParameterTrue,
		ParameterFalse,
		ParameterUnset,
		MAX
	};

public:
	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes( void ) const override 
	{ 
		return BranchPurpose::MAX; 
	}
	virtual int32 GetMinChildNodes() const override
	{ 
		return BranchPurpose::MAX;
	}

	virtual void RemoveChildNode( int32 Index ) override
	{
		// Do nothing - we do not want to allow deleting of children nodes
	}

#if WITH_EDITOR
	virtual FText GetInputPinName(int32 PinIndex) const override;
	virtual FText GetTitle() const override;
#endif //WITH_EDITOR
	//~ End USoundNode Interface.
};



