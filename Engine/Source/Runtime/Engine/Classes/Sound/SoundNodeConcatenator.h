// Copyright Epic Games, Inc. All Rights Reserved.

 
/** 
 * A node to play sounds sequentially
 * WARNING: these are not seamless
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeConcatenator.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/**
 Plays child nodes sequentially
*/
UCLASS(hidecategories = Object, editinlinenew, MinimalAPI, meta = (DisplayName = "Concatenator"))
class USoundNodeConcatenator : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/** Volume multiplier for each input. */
	UPROPERTY(EditAnywhere, editfixedsize, Category=Concatenator)
	TArray<float> InputVolume;

public:	
	//~ Begin USoundNode Interface. 
	virtual bool NotifyWaveInstanceFinished( struct FWaveInstance* WaveInstance ) override;
	virtual float GetDuration() override;
	virtual bool HasConcatenatorNode() const override { return true; }
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const;
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes() const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES; 
	}
	virtual void CreateStartingConnectors( void ) override;
	virtual void InsertChildNode( int32 Index ) override;
	virtual void RemoveChildNode( int32 Index ) override;
#if WITH_EDITOR
	/** Ensure amount of inputs matches new amount of children */
	virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes) override;
#endif
	//~ End USoundNode Interface. 
};

