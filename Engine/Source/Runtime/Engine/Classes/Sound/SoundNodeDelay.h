// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeDelay.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines a delay
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Delay" ))
class USoundNodeDelay : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* The lower bound of delay time in seconds. */
	UPROPERTY(EditAnywhere, Category=Delay )
	float DelayMin;

	/* The upper bound of delay time in seconds. */
	UPROPERTY(EditAnywhere, Category=Delay )
	float DelayMax;

public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual float GetDuration() override;
	virtual bool HasDelayNode() const override { return true; }
	//~ End USoundNode Interface. 
};



