// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/DialogueTypes.h"
#include "Sound/SoundNode.h"
#include "SoundNodeDialoguePlayer.generated.h"

class FAudioDevice;
class UDialogueSoundWaveProxy;
class UDialogueWave;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnDialogueSoundWaveProxyParsed, const FActiveSound&, const UDialogueSoundWaveProxy*);

/**
 Sound node that contains a reference to the dialogue table to pull from and be played
*/
UCLASS(hidecategories = Object, editinlinenew, MinimalAPI, meta = (DisplayName = "Dialogue Player"))
class USoundNodeDialoguePlayer : public USoundNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DialoguePlayer)
	FDialogueWaveParameter DialogueWaveParameter;

	/* Whether the dialogue line should be played looping */
	UPROPERTY(EditAnywhere, Category=DialoguePlayer)
	uint32 bLooping:1;

public:	

	ENGINE_API UDialogueWave* GetDialogueWave() const;
	ENGINE_API void SetDialogueWave(UDialogueWave* Value);

	// Subtitles need to be able to grab the UDialogueSoundWaveProxy to for localized text and duration.
	ENGINE_API const UDialogueSoundWaveProxy* GetContextualDialogueSoundWaveProxy() const;

	// Fired on the audio thread the first time ParseNodes resolves a valid proxy for a given active sound.
	// Listeners can use this to queue subtitles only for dialogue waves that are actually playing.
	static ENGINE_API FOnDialogueSoundWaveProxyParsed OnDialogueSoundWaveProxyParsed;

	//~ Begin USoundNode Interface
	virtual int32 GetMaxChildNodes() const override;
	virtual float GetDuration();
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const { return 1; }
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
#if WITH_EDITOR
	virtual FText GetTitle() const override;
#endif
	//~ End USoundNode Interface
};

