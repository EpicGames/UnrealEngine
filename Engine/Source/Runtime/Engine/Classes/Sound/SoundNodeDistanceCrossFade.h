// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeDistanceCrossFade.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

USTRUCT()
struct FDistanceDatum
{
	GENERATED_USTRUCT_BODY()

	/* The FadeInDistance at which to start hearing this sound.
	 * If you want to hear the sound up close then setting this to 0 might be a good option. 
	 */
	UPROPERTY(EditAnywhere, Category=DistanceDatum )
	float FadeInDistanceStart;

	/* The distance at which this sound has faded in completely. */
	UPROPERTY(EditAnywhere, Category=DistanceDatum )
	float FadeInDistanceEnd;

	/* The distance at which this sound starts fading out. */
	UPROPERTY(EditAnywhere, Category=DistanceDatum )
	float FadeOutDistanceStart;

	/* The distance at which this sound is no longer audible. */
	UPROPERTY(EditAnywhere, Category=DistanceDatum )
	float FadeOutDistanceEnd;

	/* The volume for which this Input should be played. */
	UPROPERTY(EditAnywhere, Category=DistanceDatum)
	float Volume;


		FDistanceDatum()
		: FadeInDistanceStart(0)
		, FadeInDistanceEnd(0)
		, FadeOutDistanceStart(0)
		, FadeOutDistanceEnd(0)
		, Volume(1.0f)
		{
		}
	
};

/**
 * SoundNodeDistanceCrossFade
 * 
 * This node's purpose is to play different sounds based on the distance to the listener.  
 * The node mixes between the N different sounds which are valid for the distance.  One should
 * think of a SoundNodeDistanceCrossFade as Mixer node which determines the set of nodes to
 * "mix in" based on their distance to the sound.
 * 
 * Example:
 * You have a gun that plays a fire sound.  At long distances you want a different sound than
 * if you were up close.   So you use a SoundNodeDistanceCrossFade which will calculate the distance
 * a listener is from the sound and play either:  short distance, long distance, mix of short and long sounds.
 *
 * A SoundNodeDistanceCrossFade differs from an SoundNodeAttenuation in that any sound is only going
 * be played if it is within the MinRadius and MaxRadius.  So if you want the short distance sound to be 
 * heard by people close to it, the MinRadius should probably be 0
 *
 * The volume curve for a SoundNodeDistanceCrossFade will look like this:
 *
 *                          Volume (of the input) 
 *    FadeInDistance.Max --> _________________ <-- FadeOutDistance.Min
 *                          /                 \
 *                         /                   \
 *                        /                     \
 * FadeInDistance.Min -->/                       \ <-- FadeOutDistance.Max
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Crossfade by Distance" ))
class USoundNodeDistanceCrossFade : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/**
	 * Each input needs to have the correct data filled in so the SoundNodeDistanceCrossFade is able
	 * to determine which sounds to play
	 */
	UPROPERTY(EditAnywhere, export, editfixedsize, Category=CrossFade)
	TArray<struct FDistanceDatum> CrossFadeInput;


public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes() const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES;  
	}
	virtual void CreateStartingConnectors( void ) override;
	virtual void InsertChildNode( int32 Index ) override;
	virtual void RemoveChildNode( int32 Index ) override;
#if WITH_EDITOR
	/** Ensure amount of Cross Fade inputs matches new amount of children */
	virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes) override;
#endif //WITH_EDITOR
	virtual float GetMaxDistance() const override;
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const;
	//~ End USoundNode Interface. 

	virtual float GetCurrentDistance(FAudioDevice* AudioDevice, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams) const;

	/**
	 * Determines whether Crossfading is currently allowed for the active sound
	 */
	virtual bool AllowCrossfading(FActiveSound& ActiveSound) const;
};
