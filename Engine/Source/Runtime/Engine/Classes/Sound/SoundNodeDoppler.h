// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AudioDevice.h"
#include "Sound/SoundNode.h"
#include "SoundNodeDoppler.generated.h"

struct FActiveSound;
struct FSoundParseParameters;

/** 
 * Computes doppler pitch shift
 */
UCLASS(hidecategories=Object, editinlinenew, meta=( DisplayName="Doppler" ))
class USoundNodeDoppler : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* How much to scale the doppler shift (1.0 is normal). */
	UPROPERTY(EditAnywhere, Category=Doppler )
	float DopplerIntensity;

	/** Whether or not to do a smooth interp to our doppler */
	UPROPERTY(EditAnywhere, Category=Doppler)
	bool bUseSmoothing;

	/** Speed at which to interp pitch scale */
	UPROPERTY(EditAnywhere, Category = Doppler, meta = (EditCondition = "bUseSmoothing"))
	float SmoothingInterpSpeed;


public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface. 

protected:
	// @todo document
	float GetDopplerPitchMultiplier(float& CurrentPitchScale, bool bSmooth, FListener const& InListener, const FVector Location, const FVector Velocity, float DeltaTime);
};



