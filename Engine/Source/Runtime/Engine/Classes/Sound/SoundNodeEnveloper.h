// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/CurveFloat.h"
#include "Sound/SoundNode.h"
#include "SoundNodeEnveloper.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Allows manipulation of volume and pitch over a set time period
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Enveloper" ))
class USoundNodeEnveloper : public USoundNode
{
	GENERATED_UCLASS_BODY()

	// The time in seconds where the envelope's loop begins.
	UPROPERTY(EditAnywhere, Category=Looping)
	float LoopStart;

	// The time in seconds where the envelope's loop ends.
	UPROPERTY(EditAnywhere, Category=Looping)
	float LoopEnd;

	// The time in seconds it takes the evelope to fade out after the last loop is completed.
	UPROPERTY(EditAnywhere, Category=Looping)
	float DurationAfterLoop;

	// The number of times the envelope should loop if looping is enabled and the envelope is not set to loop indefinitely.
	UPROPERTY(EditAnywhere, Category=Looping)
	int32 LoopCount;

	// If enabled, the envelope will continue to loop indefenitely regardless of the Loop Count value.
	UPROPERTY(EditAnywhere, Category=Looping)
	uint32 bLoopIndefinitely:1;
	
	// If enabled, the envelope will loop using the loop settings.
	UPROPERTY(EditAnywhere, Category=Looping)
	uint32 bLoop:1;

	UPROPERTY()
	TObjectPtr<class UDistributionFloatConstantCurve> VolumeInterpCurve_DEPRECATED;

	UPROPERTY()
	TObjectPtr<class UDistributionFloatConstantCurve> PitchInterpCurve_DEPRECATED;

	// The distribution defining the volume envelope.
	UPROPERTY(EditAnywhere, Category=Envelope)
	FRuntimeFloatCurve VolumeCurve;

	// The distribution defining the pitch envelope.
	UPROPERTY(EditAnywhere, Category=Envelope)
	FRuntimeFloatCurve PitchCurve;

	// The minimum pitch for the input sound.
	UPROPERTY(EditAnywhere, Category=Modulation, meta=(ToolTip="The lower bound of pitch (1.0 is no change)"))
	float PitchMin;

	// The maximum pitch for the input sound.
	UPROPERTY(EditAnywhere, Category=Modulation, meta=(ToolTip="The upper bound of pitch (1.0 is no change)"))
	float PitchMax;

	// The minimum volume for the input sound.
	UPROPERTY(EditAnywhere, Category=Modulation, meta=(ToolTip="The lower bound of volume (1.0 is no change)"))
	float VolumeMin;

	// The maximum volume for the input sound.
	UPROPERTY(EditAnywhere, Category=Modulation, meta=(ToolTip="The upper bound of volume (1.0 is no change)"))
	float VolumeMax;


public:	
	//~ Begin UObject Interface. 
#if WITH_EDITOR
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface. 


	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual float GetDuration() override;
	//~ End USoundNode Interface. 

};



