// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeGroupControl.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Plays different sounds depending on the number of active sounds
 * Any time a new sound is played, the first group that has an available slot will be chosen
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Group Control" ))
class USoundNodeGroupControl : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* How many active sounds are allowed for each group */
	UPROPERTY(EditAnywhere, editfixedsize, Category=GroupControl)
	TArray<int32> GroupSizes;

public:
	//~ Begin USoundNode Interface.
	virtual bool NotifyWaveInstanceFinished( FWaveInstance* WaveInstance ) override;
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes() const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES; 
	}
	virtual void InsertChildNode( int32 Index ) override;
	virtual void RemoveChildNode( int32 Index ) override;
#if WITH_EDITOR
	/** Ensure amount of Groups matches new amount of children */
	virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes) override;
#endif //WITH_EDITOR
	virtual void CreateStartingConnectors() override;
	//~ End USoundNode Interface.

private:
	// Ensure the child count and group sizes are the same counts
	void FixGroupSizesArray();

	static TMap< USoundNodeGroupControl*, TArray< TMap< FActiveSound*, int32 > > > GroupControlSlotUsageMap;
};
