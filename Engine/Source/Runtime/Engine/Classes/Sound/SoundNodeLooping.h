// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "Sound/SoundNode.h"
#include "SoundNodeLooping.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines how a sound loops; either indefinitely, or for a set number of times.
 * Note: The Looping node should only be used for logical or procedural looping such as introducing a delay.
 * These sounds will not be played seamlessly. If you want a sound to loop seamlessly and indefinitely,
 * use the Looping flag on the Wave Player node for that sound.
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Looping" ))
class USoundNodeLooping : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* The amount of times to loop */
	UPROPERTY(EditAnywhere, Category = Looping, meta = (ClampMin = 1, EditCondition = "!bLoopIndefinitely"))
	int32 LoopCount;

	/* If enabled, the node will continue to loop indefinitely regardless of the Loop Count value. */
	UPROPERTY(EditAnywhere, Category = Looping)
	uint32 bLoopIndefinitely : 1;

public:	
	//~ Begin USoundNode interface. 
	virtual bool NotifyWaveInstanceFinished( struct FWaveInstance* WaveInstance ) override;
	virtual float GetDuration() override;
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const;
	//~ End USoundNode interface. 

private:
	/** Resets the bRequiresInitialization flag on child nodes to allow random nodes to pick new values */
	void ResetChildren(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, int32& CurrentLoopCount);
};



