// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeMature.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/**
 * This SoundNode uses UEngine::bAllowMatureLanguage to determine whether child nodes
 * that have USoundWave::bMature=true should be played. 
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Mature" ))
class USoundNodeMature : public USoundNode
{
	GENERATED_UCLASS_BODY()


public:
	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual void CreateStartingConnectors( void ) override;
	virtual int32 GetMaxChildNodes() const override;
	//~ End USoundNode Interface.
};

