// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeMixer.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines how concurrent sounds are mixed together
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Mixer" ))
class USoundNodeMixer : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/** A volume for each input.  Automatically sized. */
	UPROPERTY(EditAnywhere, export, editfixedsize, Category=Mixer)
	TArray<float> InputVolume;


public:
	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes() const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES; 
	}
	virtual void CreateStartingConnectors( void ) override;
	virtual void InsertChildNode( int32 Index ) override;
	virtual void RemoveChildNode( int32 Index ) override;
#if WITH_EDITOR
	/** Ensure amount of inputs matches new amount of children */
	virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes) override;
#endif //WITH_EDITOR
	//~ End USoundNode Interface.
};

