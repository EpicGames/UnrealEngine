// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeModulator.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines a random volume and pitch modification when a sound starts
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Modulator" ))
class USoundNodeModulator : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* The lower bound of pitch (1.0 is no change). */
	UPROPERTY(EditAnywhere, Category=Modulation )
	float PitchMin;

	/* The upper bound of pitch (1.0 is no change). */
	UPROPERTY(EditAnywhere, Category=Modulation )
	float PitchMax;

	/* The lower bound of volume (1.0 is no change). */
	UPROPERTY(EditAnywhere, Category=Modulation )
	float VolumeMin;

	/* The upper bound of volume (1.0 is no change). */
	UPROPERTY(EditAnywhere, Category=Modulation )
	float VolumeMax;

public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface. 
};



