// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeModulatorContinuous.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

UENUM()
enum ModulationParamMode : int
{
	MPM_Normal UMETA(DisplayName = "Normal" , Tooltip = "Clamps input value to the range (MinInput, MaxInput) then maps to the range (MinOutput, MaxOutput)"),
	MPM_Abs UMETA(DisplayName = "Absolute" , Tooltip = "Same as Normal except that the input value is treated as an absolute value"),
	MPM_Direct UMETA(DisplayName = "Direct" , Tooltip = "Use the input value directly without scaling or reference to Min or Max input or output values"),
	MPM_MAX,
};

USTRUCT(BlueprintType)
struct FModulatorContinuousParams
{
	GENERATED_USTRUCT_BODY()

	FModulatorContinuousParams()
		: Default(1.f)
		, MinInput(0.f)
		, MaxInput(1.f)
		, MinOutput(0.f)
		, MaxOutput(1.f)
		, ParamMode(MPM_Normal)
	{
	}

	/** The name of the sound instance parameter that specifies the current value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=ModulatorContinousParameters)
	FName ParameterName;

	/** The default value to be used if the parameter is not found. */
	UPROPERTY(EditAnywhere, Category=ModulatorContinousParameters)
	float Default;

	/** The minimum input value. Values will be clamped to the [MinInput, MaxInput] range.*/
	UPROPERTY(EditAnywhere, Category=ModulatorContinousParameters)
	float MinInput;

	/** The maximum input value. Values will be clamped to the [MinInput, MaxInput] range.*/
	UPROPERTY(EditAnywhere, Category = ModulatorContinousParameters)
	float MaxInput;

	/** The minimum output value. The input value will be scaled from the range [MinInput, MaxInput] to [MinOut, MaxOutput] */
	UPROPERTY(EditAnywhere, Category = ModulatorContinousParameters)
	float MinOutput;

	/** The maximum output value. The input value will be scaled from the range [MinInput, MaxInput] to [MinOut, MaxOutput] */
	UPROPERTY(EditAnywhere, Category = ModulatorContinousParameters)
	float MaxOutput;

	/** The mode with which to treat the input value */
	UPROPERTY(EditAnywhere, Category=ModulatorContinousParameters)
	TEnumAsByte<enum ModulationParamMode> ParamMode;

	float GetValue(const FActiveSound& ActiveSound) const;
};

/**
 * Allows named parameter based manipulation of pitch and volume
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Continuous Modulator" ))
class USoundNodeModulatorContinuous : public USoundNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=ContinuousModulator)
	FModulatorContinuousParams PitchModulationParams;

	UPROPERTY(EditAnywhere, Category=ContinuousModulator)
	FModulatorContinuousParams VolumeModulationParams;

public:
	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface.
};



