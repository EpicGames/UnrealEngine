// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeOscillator.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Defines how a sound oscillates
 */
UCLASS(hidecategories=Object, editinlinenew, meta=( DisplayName="Oscillator" ))
class USoundNodeOscillator : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/* Whether to oscillate volume. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	uint32 bModulateVolume:1;

	/* Whether to oscillate pitch. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	uint32 bModulatePitch:1;

	/* An amplitude of 0.25 would oscillate between 0.75 and 1.25. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float AmplitudeMin;

	/* An amplitude of 0.25 would oscillate between 0.75 and 1.25. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float AmplitudeMax;

	/* A frequency of 20 would oscillate at 10Hz. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float FrequencyMin;

	/* A frequency of 20 would oscillate at 10Hz. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float FrequencyMax;

	/* Offset into the sine wave. Value modded by 2 * PI. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float OffsetMin;

	/* Offset into the sine wave. Value modded by 2 * PI. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float OffsetMax;

	/* A center of 0.5 would oscillate around 0.5. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float CenterMin;

	/* A center of 0.5 would oscillate around 0.5. */
	UPROPERTY(EditAnywhere, Category=Oscillator )
	float CenterMax;

public:	
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface. 
};



