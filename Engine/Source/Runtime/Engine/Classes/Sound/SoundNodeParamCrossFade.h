// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNodeDistanceCrossFade.h"
#include "SoundNodeParamCrossFade.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;

/** 
 * Crossfades between different sounds based on a parameter
 */
UCLASS(editinlinenew, MinimalAPI, meta=( DisplayName="Crossfade by Param" ))
class USoundNodeParamCrossFade : public USoundNodeDistanceCrossFade
{
	GENERATED_UCLASS_BODY()

public:

	/* Parameter controlling cross fades. */
	UPROPERTY(EditAnywhere, Category=CrossFade )
	FName ParamName;

	virtual float GetCurrentDistance(FAudioDevice* AudioDevice, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams) const override;
	virtual bool AllowCrossfading(FActiveSound& ActiveSound) const override;
	virtual float GetMaxDistance() const override;

};
