// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeQualityLevel.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/**
 * This SoundNode uses GameUserSettings AudioQualityLevel (or the editor override) to choose which branch to play
 * and at runtime will only load in to memory sound waves connected to the branch that will be selected
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Quality Level" ))
class USoundNodeQualityLevel : public USoundNode
{
	GENERATED_BODY()

public:

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin USoundNode Interface.
	virtual void Serialize(FArchive& Ar) override;	
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes() const override;
	virtual int32 GetMinChildNodes() const override;
	virtual void PrimeChildWavePlayers(bool bRecurse) override;
	virtual void RetainChildWavePlayers(bool bRecurse) override;
	virtual void ReleaseRetainerOnChildWavePlayers(bool bRecurse) override;
	void LoadChildWavePlayers(bool bAddToRoot, bool bRecurse);

#if WITH_EDITOR
	virtual FText GetInputPinName(int32 PinIndex) const override;
#endif
	//~ End USoundNode Interface.

	// A Property to indicate which quality this node was cooked with. (INDEX_NONE if not cooked, or unset).
	UPROPERTY()
	int32 CookedQualityLevelIndex = INDEX_NONE;

#if WITH_EDITOR
	void ReconcileNode(bool bReconstructNode);

	// Critical section to protect Child nodes array while serializing
	FCriticalSection EditorOnlyCs;
#endif

private:
	virtual void ForCurrentQualityLevel(TFunction<void(USoundNode*)>&& Lambda);
};

