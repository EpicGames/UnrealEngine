// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeRandom.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Selects sounds from a random set
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Random" ))
class USoundNodeRandom : public USoundNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, editfixedsize, Category=Random)
	TArray<float> Weights;

	/**
	 * Internal state of which sounds have been played.  This is only used at runtime
	 * to keep track of which sounds have been played
	 */
	UPROPERTY(transient)
	TArray<bool> HasBeenUsed;

	/** Counter var so we don't have to count all of the used sounds each time we choose a sound **/
	UPROPERTY(transient)
	int32 NumRandomUsed;

	/** If greater than 0, then upon each level load such a number of inputs will be randomly selected
	 *  and the rest will be removed. This can be used to cut down the memory usage of large randomizing
	 *  cues.
	 */
	UPROPERTY(EditAnywhere, Category=Random)
	int32 PreselectAtLevelLoad;

	/** If set to true, this random node will not be culled on load for platforms with a maximum amount of preloaded random branches
	*   set in project settings
	*/
	UPROPERTY(EditAnywhere, Category = Random)
	uint8 bShouldExcludeFromBranchCulling : 1;

	/** Exclusion for branch culling set by the sound cue. */
	UPROPERTY()
	uint8 bSoundCueExcludedFromBranchCulling : 1;

	/** 
	 * Determines whether or not this SoundNodeRandom should randomize with or without
	 * replacement.  
	 *
	 * WithoutReplacement means that only nodes left will be valid for 
	 * selection.  So with that, you are guarenteed to have only one occurrence of the
	 * sound played until all of the other sounds in the set have all been played.
	 *
	 * WithReplacement means that a node will be chosen and then placed back into the set.
	 * So one could play the same sound over and over if the probabilities don't go your way :-)
	 */
	UPROPERTY(EditAnywhere, Category=Random)
	uint8 bRandomizeWithoutReplacement : 1;

#if WITH_EDITORONLY_DATA
	/** Editor only list of nodes hidden to duplicate behavior of PreselectAtLevelLoad */
	UPROPERTY(transient)
	TArray<int32> PIEHiddenNodes;
#endif //WITH_EDITORONLY_DATA

public:
	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void PostEditImport() override;
	//~ End UObject Interface

	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const override;
	virtual int32 GetMaxChildNodes() const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES; 
	}
	virtual void InsertChildNode( int32 Index ) override;
	virtual void RemoveChildNode( int32 Index ) override;
#if WITH_EDITOR
	/** Ensure Random weights and usage array matches new amount of children */
	virtual void SetChildNodes(TArray<USoundNode*>& InChildNodes) override;
	virtual void OnBeginPIE(const bool bIsSimulating) override;
#endif //WITH_EDITOR
	virtual void CreateStartingConnectors( void ) override;
	//~ End USoundNode Interface.

	// @todo document
	void FixWeightsArray( void );

	// @todo document
	void FixHasBeenUsedArray( void );

#if WITH_EDITOR
	void UpdatePIEHiddenNodes();
#endif //WITH_EDITOR

	int32 ChooseNodeIndex(FActiveSound& ActiveSound);

private:

	/*
	 * Determines the amount of branches to use on this random node, so that we can cull the rest.
	 * Based on the au.MaxRandomBranches CVar, the MaxNumRandomBranches project setting
	 * available for some platforms, and this node's PreselectAtLevelLoad FProperty.
	 * @return 0 if we do not want cull any branches.
	 */
	int32 DetermineAmountOfBranchesToPreselect();

};



