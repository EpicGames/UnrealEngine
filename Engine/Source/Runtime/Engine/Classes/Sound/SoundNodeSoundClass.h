// Copyright Epic Games, Inc. All Rights Reserved.

 
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeSoundClass.generated.h"

class USoundClass;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Remaps the SoundClass of SoundWaves underneath this
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="SoundClass" ))
class USoundNodeSoundClass : public USoundNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=SoundClass)
	TObjectPtr<USoundClass> SoundClassOverride;

public:
	//~ Begin USoundNode Interface. 
	virtual void ParseNodes( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface. 

	virtual void PostLoad() override;
	virtual void BeginDestroy() override;

private:
	// This is set to true if this node's soundclass specifically dictates that a sound should be retained.
	bool bRetainingAudioDueToSoundClass;
};
