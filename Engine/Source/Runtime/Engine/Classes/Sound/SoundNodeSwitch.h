// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeSwitch.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Selects a child node based on the value of a integer parameter
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=(DisplayName="Switch"))
class USoundNodeSwitch : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/** The name of the integer parameter to use to determine which branch we should take */
	UPROPERTY(EditAnywhere, Category=Switch)
	FName IntParameterName;

public:
	//~ Begin USoundNode Interface.
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	virtual int32 GetMaxChildNodes( void ) const override 
	{ 
		return MAX_ALLOWED_CHILD_NODES; 
	}
	virtual int32 GetMinChildNodes() const override
	{ 
		return 1;
	}
	virtual void CreateStartingConnectors( void ) override;
	
#if WITH_EDITOR
	void RenamePins();

	virtual void InsertChildNode( int32 Index ) override
	{
		Super::InsertChildNode(Index);

		RenamePins();
	}

	virtual void RemoveChildNode( int32 Index ) override
	{
		Super::RemoveChildNode(Index);
		
		RenamePins();
	}

	virtual FText GetInputPinName(int32 PinIndex) const override;
	virtual FText GetTitle() const override;
#endif //WITH_EDITOR
	//~ End USoundNode Interface.
};



