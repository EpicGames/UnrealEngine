// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sound/SoundNode.h"
#include "SoundNodeWaveParam.generated.h"

class FAudioDevice;
struct FActiveSound;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Sound node that takes a runtime parameter for the wave to play
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Wave Param" ))
class USoundNodeWaveParam : public USoundNode
{
	GENERATED_UCLASS_BODY()

	/** The name of the wave parameter to use to look up the SoundWave we should play */
	UPROPERTY(EditAnywhere, Category=WaveParam)
	FName WaveParameterName;


public:	
	//~ Begin USoundNode Interface
	virtual float GetDuration() override;
	virtual void ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances ) override;
	//~ End USoundNode Interface
};

