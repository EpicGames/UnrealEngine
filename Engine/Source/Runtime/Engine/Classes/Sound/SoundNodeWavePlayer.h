// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "Sound/SoundNodeAssetReferencer.h"
#include "SoundNodeWavePlayer.generated.h"

class FAudioDevice;
class USoundWave;
struct FActiveSound;
struct FPropertyChangedEvent;
struct FSoundParseParameters;
struct FWaveInstance;

/**
 * Sound node that contains a reference to the raw wave file to be played
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Wave Player" ))
class USoundNodeWavePlayer : public USoundNodeAssetReferencer
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category=WavePlayer, meta=(DisplayName="Sound Wave"))
	TSoftObjectPtr<USoundWave> SoundWaveAssetPtr;

	UPROPERTY(transient)
	TObjectPtr<USoundWave> SoundWave;

	void OnSoundWaveLoaded(const FName& PackageName, UPackage * Package, EAsyncLoadingResult::Type Result, bool bAddToRoot);

	uint8 bAsyncLoading:1;

public:

	UPROPERTY(EditAnywhere, Category=WavePlayer)
	uint8 bLooping:1;

private:
	// Set to true when we enqueue a task to the game thread to load
	// the asset associated with this thread.
	// This only occurs if ClearAssetReferences() was called and we still tried to play this wave player,
	// likely because the quality level was changed.
	FThreadSafeBool bAsyncLoadRequestPending;

public:

	USoundWave* GetSoundWave() const { return SoundWave; }
	ENGINE_API void SetSoundWave(USoundWave* SoundWave);

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin USoundNode Interface
	virtual int32 GetMaxChildNodes() const override;
	virtual float GetDuration() override;
	virtual bool IsPlayWhenSilent() const override;
	virtual int32 GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const { return 1; }
	virtual void ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances) override;
#if WITH_EDITOR
	virtual FText GetTitle() const override;
#endif
	//~ End USoundNode Interface

	//~ Begin USoundNodeAssetReferencer Interface
	virtual void LoadAsset(bool bAddToRoot = false) override;
	virtual void ClearAssetReferences() override;
	virtual bool ContainsProceduralSoundReference() const override;
	//~ End USoundNode Interface

	// If this returns true, this wave player currently has an async load for the USoundWave in flight.
	bool IsCurrentlyAsyncLoadingAsset() const { return bAsyncLoading; }
};

