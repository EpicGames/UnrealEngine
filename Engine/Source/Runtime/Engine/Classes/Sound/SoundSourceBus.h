// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Sound/SoundWaveProcedural.h"
#include "Sound/SoundSourceBusSend.h"
#include "Sound/AudioBus.h"
#include "SoundSourceBus.generated.h"

class USoundSourceBus;
class USoundWaveProcedural;
class FAudioDevice;

// The number of channels to mix audio into the source bus
UENUM(BlueprintType)
enum class ESourceBusChannels : uint8
{
	Mono,
	Stereo,
};

// A source bus is a type of USoundBase and can be "played" like any sound.
UCLASS(hidecategories= (Compression, SoundWave, Streaming, Subtitles, Analysis, Format, Loading, Info, ImportSettings), ClassGroup = Sound, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USoundSourceBus : public USoundWave
{
	GENERATED_UCLASS_BODY()

public:

	/** How many channels to use for the source bus if the audio bus is not specified, otherwise it will use the audio bus object's channel count. */
	UPROPERTY(EditAnywhere, Category = "Sound|Source Bus")
	ESourceBusChannels SourceBusChannels;

	/** The duration (in seconds) to use for the source bus. A duration of 0.0 indicates to play the source bus indefinitely. */
	UPROPERTY(EditAnywhere, Category = "Sound|Source Bus", meta = (UIMin = 0.0, ClampMin = 0.0))
	float SourceBusDuration;

	/** Audio bus to use as audio for this source bus. This source bus will sonify the audio from the audio bus. */
	UPROPERTY(EditAnywhere, Category = "Sound|Source Bus", meta=(DisplayName="Audio Bus Return"))
	TObjectPtr<UAudioBus> AudioBus;

	/** Stop the source bus when the volume goes to zero. */
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bAutoDeactivateWhenSilent:1;

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	virtual bool CanVisualizeAsset() const override
	{
		return false;
	}

	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Override the SoundWave behavior so that we don't ask DDC for compressed data.
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override {};
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override { return true; };
	virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override {};

#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin USoundWave Interface
#if WITH_EDITORONLY_DATA
	virtual void CachePlatformData(bool bAsyncCache = false) override {};
	virtual void BeginCachePlatformData() override {}
	virtual void FinishCachePlatformData() override {};	
#endif	//WITH_EDITOR_ONLY_DATA
	virtual void Serialize(FArchive& InArchive) override;
	virtual void SerializeCookedPlatformData(FArchive& Ar) override {}
	virtual bool IsSeekable() const override { return false; }
	//~ End USoundWave Interface

	//~ Begin USoundBase Interface.
	ENGINE_API virtual bool IsPlayable() const override;
	ENGINE_API virtual float GetDuration() const override;
	//~ End USoundBase Interface.

protected:
	ENGINE_API void Init();
	
	// SourceBus doesn't represent a wav file, don't do anything when serializing cue points
	virtual void SerializeCuePoints(FArchive& Ar, const bool bIsLoadingFromCookedArchive) {}
};
