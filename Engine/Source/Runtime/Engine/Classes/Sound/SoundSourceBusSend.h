// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "Sound/SoundSubmixSend.h"
#include "SoundSourceBusSend.generated.h"

class USoundSourceBus;
class UAudioBus;

UENUM(BlueprintType)
enum class ESourceBusSendLevelControlMethod : uint8
{
	// A send based on linear interpolation between a distance range and a remapped range
	Linear  UMETA(DisplayName = "Distance Linear"),

	// A send based on a supplied curve mapping (distance = X, send level = Y)
	CustomCurve UMETA(DisplayName = "Distance Custom"),

	// A constant send level (Uses the specified constant send level value. Useful for 2D sounds.)
	Manual UMETA(DisplayName = "Constant"),
};

USTRUCT(BlueprintType)
struct FSoundSourceBusSendInfo
{
	GENERATED_USTRUCT_BODY()

	// Changes the method we use to determine the send level - see enum value tooltips for more information
	UPROPERTY(EditAnywhere, Category = BusSend, meta = (DisplayName = "Send Level Control Method"))
	ESourceBusSendLevelControlMethod SourceBusSendLevelControlMethod;

	// The Source Bus to send the audio to
	UPROPERTY(EditAnywhere, Category = BusSend)
	TObjectPtr<USoundSourceBus> SoundSourceBus;

	// The Audio Bus to send the audio to
	UPROPERTY(EditAnywhere, Category = BusSend)
	TObjectPtr<UAudioBus> AudioBus;

	// Constant send level to use. Doesn't change as a function of distance.
	UPROPERTY(EditAnywhere, Category = BusSend, meta = (DisplayName = "Send Level", EditCondition = "SourceBusSendLevelControlMethod == ESourceBusSendLevelControlMethod::Manual", EditConditionHides))
	float SendLevel;

	// The amount to send to the bus when sound is located at a distance less than or equal to value specified in the Send Distance Min
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BusSend, meta = (DisplayName = "Send Level Min", EditCondition = "SourceBusSendLevelControlMethod == ESourceBusSendLevelControlMethod::Linear", EditConditionHides))
	float MinSendLevel;

	// The amount to send to the bus when sound is located at a distance greater than or equal to value specified in the Send Distance Max
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BusSend, meta = (DisplayName = "Send Level Max", EditCondition = "SourceBusSendLevelControlMethod == ESourceBusSendLevelControlMethod::Linear", EditConditionHides))
	float MaxSendLevel;

	/** The distance at which to start mapping between to Send Level Min/Max 
	 *  Distances LESS than this will result in a clamped Send Level Min
	 */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BusSend, meta = (DisplayName = "Send Distance Min", EditCondition = "SourceBusSendLevelControlMethod != ESourceBusSendLevelControlMethod::Manual", EditConditionHides))
	float MinSendDistance;

	/** The distance at which to stop mapping between Send Level Min/Max 
	 *  Distances GREATER than this will result in a clamped Send Level Max
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BusSend, meta = (DisplayName = "Send Distance Max", EditCondition = "SourceBusSendLevelControlMethod != ESourceBusSendLevelControlMethod::Manual", EditConditionHides))
	float MaxSendDistance;

	// The custom send curve to use for distance-based send level. (0.0-1.0 on the curve's X-axis maps to Min/Max Send Distance)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BusSend, meta = (EditCondition = "SourceBusSendLevelControlMethod == ESourceBusSendLevelControlMethod::CustomCurve", EditConditionHides))
	FRuntimeFloatCurve CustomSendLevelCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(InlineEditConditionToggle))
	uint8 bEnableLPFCutoff : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(Units="Hz", Delta="0.1", ClampMin="20.0", ClampMax="20000.0", UIMin="20.0", UIMax="20000.0", EditCondition="bEnableLPFCutoff"))
	float LPFCutoff;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(InlineEditConditionToggle))
	uint8 bEnableHPFCutoff : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(Units="Hz", Delta="0.1", ClampMin="20.0", ClampMax="20000.0", UIMin="20.0", UIMax="20000.0", EditCondition="bEnableHPFCutoff"))
	float HPFCutoff;

#if WITH_EDITORONLY_DATA
	// Legacy fields kept for tagged-load matching; migrated in PostSerialize.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use bEnableLPFCutoff + LPFCutoff instead.")
	UPROPERTY()
	TOptional<float> LPFCutoffFrequency_DEPRECATED;

	UE_DEPRECATED(5.8, "Use bEnableHPFCutoff + HPFCutoff instead.")
	UPROPERTY()
	TOptional<float> HPFCutoffFrequency_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	FSoundSourceBusSendInfo()
		: SourceBusSendLevelControlMethod(ESourceBusSendLevelControlMethod::Manual)
		, SoundSourceBus(nullptr)
		, AudioBus(nullptr)
		, SendLevel(1.0f)
		, MinSendLevel(0.0f)
		, MaxSendLevel(1.0f)
		, MinSendDistance(100.0f)
		, MaxSendDistance(1000.0f)
		, bEnableLPFCutoff(0)
		, LPFCutoff(20000.0f)
		, bEnableHPFCutoff(0)
		, HPFCutoff(20.0f)
	{
	}

#if WITH_EDITORONLY_DATA
	// Implicit special members would touch the deprecated members and warn at the synthesis site (outside the field-decl pragma). Default them explicitly inside the pragma so synthesis sits in the suppressed region.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSoundSourceBusSendInfo(const FSoundSourceBusSendInfo&) = default;
	FSoundSourceBusSendInfo(FSoundSourceBusSendInfo&&) = default;
	FSoundSourceBusSendInfo& operator=(const FSoundSourceBusSendInfo&) = default;
	FSoundSourceBusSendInfo& operator=(FSoundSourceBusSendInfo&&) = default;
	~FSoundSourceBusSendInfo() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	ENGINE_API void PostSerialize(const FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FSoundSourceBusSendInfo> : public TStructOpsTypeTraitsBase2<FSoundSourceBusSendInfo>
{
	enum
	{
		WithPostSerialize = true,
	};
};
