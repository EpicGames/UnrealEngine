// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioDeviceHandle.h"
#include "AudioLinkSettingsAbstract.h"
#include "DSP/SpectrumAnalyzer.h"
#include "IAudioEndpoint.h"
#include "ISoundfieldEndpoint.h"
#include "ISoundfieldFormat.h"
#include "SampleBufferIO.h"
#include "SoundEffectSubmix.h"
#include "SoundModulationDestination.h"
#include "SoundSubmixSend.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SoundSubmix.generated.h"


// Forward Declarations
class UEdGraph;
class USoundEffectSubmixPreset;
class USoundSubmix;
class ISubmixBufferListener;



/**
* Called when a recorded file has finished writing to disk.
*
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSubmixRecordedFileDone, const USoundWave*, ResultingSoundWave);

/**
* Called when a new submix envelope value is generated on the given audio device id (different for multiple PIE). Array is an envelope value for each channel.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSubmixEnvelope, const TArray<float>&, Envelope);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSubmixSpectralAnalysis, const TArray<float>&, Magnitudes);


UENUM(BlueprintType)
enum class EFFTSize : uint8
{
	// 512
	DefaultSize,

	// 64
	Min,

	// 256
	Small,

	// 512
	Medium,

	// 1024
	Large,

	// 2048
	VeryLarge, 

	// 4096
	Max
};

UENUM()
enum class EFFTPeakInterpolationMethod : uint8
{
	NearestNeighbor,
	Linear,
	Quadratic,
	ConstantQ,
};

UENUM()
enum class EFFTWindowType : uint8
{
	// No window is applied. Technically a boxcar window.
	None,

	// Mainlobe width of -3 dB and sidelobe attenuation of ~-40 dB. Good for COLA.
	Hamming,

	// Mainlobe width of -3 dB and sidelobe attenuation of ~-30dB. Good for COLA.
	Hann,

	// Mainlobe width of -3 dB and sidelobe attenuation of ~-60db. Tricky for COLA.
	Blackman
};

UENUM(BlueprintType)
enum class EAudioSpectrumType : uint8
{
	// Spectrum frequency values are equal to magnitude of frequency.
	MagnitudeSpectrum,

	// Spectrum frequency values are equal to magnitude squared.
	PowerSpectrum,

	// Returns decibels (0.0 dB is 1.0)
	Decibel,
};

struct FSoundSpectrumAnalyzerSettings
{
	// FFTSize used in spectrum analyzer.
	EFFTSize FFTSize;

	// Type of window to apply to audio.
	EFFTWindowType WindowType;

	// Metric used when analyzing spectrum. 
	EAudioSpectrumType SpectrumType;

	// Interpolation method used when getting frequencies.
	EFFTPeakInterpolationMethod  InterpolationMethod;

	// Hopsize between audio windows as a ratio of the FFTSize.
	float HopSize;
};

struct FSoundSpectrumAnalyzerDelegateSettings
{
	// Settings for individual bands.
	TArray<FSoundSubmixSpectralAnalysisBandSettings> BandSettings; 

	// Number of times a second the delegate is triggered. 
	float UpdateRate; 

	// The decibel level considered silence.
	float DecibelNoiseFloor; 

	// If true, returned values are scaled between 0 and 1.
	bool bDoNormalize; 

	// If true, the band values are tracked to always have values between 0 and 1. 
	bool bDoAutoRange; 

	// The time in seconds for the range to expand to a new observed range.
	float AutoRangeAttackTime; 

	// The time in seconds for the range to shrink to a new observed range.
	float AutoRangeReleaseTime;
};


#if WITH_EDITOR

/** Interface for sound submix graph interaction with the AudioEditor module. */
class ISoundSubmixAudioEditor
{
public:
	virtual ~ISoundSubmixAudioEditor() = default;

	/** Refreshes the sound class graph links. */
	virtual void RefreshGraphLinks(UEdGraph* SoundClassGraph) = 0;
};
#endif

USTRUCT()
struct FDynamicChildSubmix  
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<USoundSubmixBase>> ChildSubmixes;
};

UCLASS(config = Engine, abstract, hidecategories = Object, editinlinenew, BlueprintType, MinimalAPI)
class USoundSubmixBase : public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	/** EdGraph based representation of the SoundSubmix */
	TObjectPtr<UEdGraph> SoundSubmixGraph;
#endif

	// Auto-manage enabling and disabling the submix as a CPU optimization. It will be disabled if the submix and all child submixes are silent. It will re-enable if a sound is sent to the submix or a child submix is audible.
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = AutoDisablement)
	bool bAutoDisable = true;

	// The minimum amount of time to wait before automatically disabling a submix if it is silent. Will immediately re-enable if source audio is sent to it. 
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = AutoDisablement, meta = (EditCondition = "bAutoDisable"))
	float AutoDisableTime = 0.01f;

	// Child submixes to this sound mix
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = SoundSubmix)
	TArray<TObjectPtr<USoundSubmixBase>> ChildSubmixes;

	// Dynamic Child submixes (Map of AudioDevice -> [Submix] )
	UPROPERTY(Transient)
	TMap<uint32, FDynamicChildSubmix> DynamicChildSubmixes;
		
	/** Dynamically Connects to a parent submix.
	* @param	WorldContextObject	UObject that's used to GetWorld
	* @param	InParent	Parent Submix to connect to
	**/
	UFUNCTION(BlueprintCallable, Category = "Submix", meta = (WorldContext = "WorldContextObject", DisplayName = "Connect"))	
	virtual bool DynamicConnect(
		const UObject* WorldContextObject, 
		UPARAM(DisplayName="Parent") USoundSubmixBase* InParent) 
	{ 
		return false; 
	}

	/** Dynamically Disconnect from a parent.
	* @param	WorldContextObject	UObject that's used to GetWorld
	**/
	UFUNCTION(BlueprintCallable, Category = "Submix", meta = (WorldContext = "WorldContextObject", DisplayName = "Disconnect"))
	virtual bool DynamicDisconnect(const UObject* WorldContextObject) 
	{ 
		return false; 
	}
	
	/** Searching upwards from this Submix to the root looking for the first Submix marked Dynamic
	 *  If this Submix is Dynamic this will be returned.
	**/
	UFUNCTION(BlueprintCallable, Category = "Submix", meta = (WorldContext = "WorldContextObject"))
	virtual USoundSubmixBase* FindDynamicAncestor() 
	{ 
		return nullptr; 
	}
	
	/** If this Submix is (or any of its parents are marked dynamic).
	* @param	bIncludeAncestors	Whether to traverse upwards through the ancestors looking for anything with a dynamic flag.
	**/
	virtual bool IsDynamic(const bool bIncludeAncestors) const 
	{ 
		return false; 
	}
	
	ENGINE_API void IncrementLoadRefCount();
	ENGINE_API void DecrementLoadRefCount();
	
protected:
	//~ Begin UObject Interface.
	ENGINE_API virtual FString GetDesc() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void PostLoad() override;
	
private:
	std::atomic<int32> RefCount{ 0 };

public:

	// Sound Submix Editor functionality
#if WITH_EDITOR

	/**
	* Add Referenced objects
	*
	* @param	InThis SoundSubmix we are adding references from.
	* @param	Collector Reference Collector
	*/
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:

	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface.

private:
	static TArray<TObjectPtr<USoundSubmixBase>> BackupChildSubmixes;
#endif // WITH_EDITOR
};

/**
 * This submix class can be derived from for submixes that output to a parent submix.
 */
UCLASS(config = Engine, abstract, hidecategories = Object, editinlinenew, BlueprintType, MinimalAPI)
class USoundSubmixWithParentBase : public USoundSubmixBase
{
	GENERATED_UCLASS_BODY()
public:


	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = SoundSubmix, meta = (EditCondition = "!bIsDynamic"))
	TObjectPtr<USoundSubmixBase> ParentSubmix;

	/**
	* Holds the dynamic (not serialized) parent for this Submix.
	* Overrides the serialized one.
	*/
	
	ENGINE_API TObjectPtr<USoundSubmixBase> GetParent(Audio::FDeviceId InDeviceId) const;
	
	UPROPERTY(Transient)
	TMap<uint32, TObjectPtr<USoundSubmixBase>> DynamicParentSubmix;
	
	/**
	* Set the parent submix of this SoundSubmix, removing it as a child from its previous owner
	*
	* @param	InParentSubmix	The New Parent Submix of this submix
	* @param	bModifyAssets	Whether or not to mark the UObjects as modified.
	*/
	ENGINE_API void SetParentSubmix(USoundSubmixBase* InParentSubmix, bool bModifyAssets = true);

	ENGINE_API virtual bool DynamicConnect(const UObject* WorldContextObject, USoundSubmixBase* Parent) override;
	ENGINE_API bool DynamicConnect(FAudioDeviceHandle Handle, USoundSubmixBase* Parent);
	ENGINE_API virtual bool DynamicDisconnect(const UObject* WorldContextObject) override;
	ENGINE_API bool DynamicDisconnect(FAudioDeviceHandle Handle);
	ENGINE_API virtual bool IsDynamic(const bool bIncludeAncestors) const override;
	ENGINE_API virtual USoundSubmixBase* FindDynamicAncestor() override;

protected:

	/** Is Submix Dynamic. (i.e. allows connect/disconnect at runtime.)  **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundSubmix, meta = (DisplayPriority=-1, EditCondition = "ParentSubmix == nullptr"))
	uint8 bIsDynamic : 1;

	// Const version. 
	const USoundSubmixBase* FindDynamicAncestor() const;
	
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#endif 
};

// Whether to use linear or decibel values for audio gains
UENUM(BlueprintType)
enum class EGainParamMode : uint8
{
	Linear = 0,
	Decibels,
};

/**
 * Sound Submix class meant for applying an effect to the downmixed sum of multiple audio sources.
 */
UCLASS(config = Engine, hidecategories = Object, editinlinenew, BlueprintType, MinimalAPI)
class USoundSubmix : public USoundSubmixWithParentBase
{
	GENERATED_UCLASS_BODY()

public:

	/** Mute this submix when the application is muted or in the background. Used to prevent submix effect tails from continuing when tabbing out of application or if application is muted. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundSubmix)
	uint8 bMuteWhenBackgrounded : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SoundSubmix)
	TArray<TObjectPtr<USoundEffectSubmixPreset>> SubmixEffectChain;

	/** Optional settings used by plugins which support ambisonics file playback. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SoundSubmix)
	TObjectPtr<USoundfieldEncodingSettingsBase> AmbisonicsPluginSettings;

	/** The attack time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the envelope value of sounds played with this submix. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerAttackTime;

	/** The release time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the envelope value of sounds played with this submix. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerReleaseTime;

	/** The output volume of the submix in Decibels. Applied after submix effects and analysis are performed.*/
	UPROPERTY(EditAnywhere, BlueprintSetter=SetOutputVolumeModulation, Category = SubmixLevel, meta = (DisplayName = "Output Volume (dB)", AudioParam = "Volume", AudioParamClass = "SoundModulationParameterVolume", ClampMin = "-96.0", ClampMax = "0.0", UIMin = "-96.0", UIMax = "0.0"))
	FSoundModulationDestinationSettings OutputVolumeModulation;

	/** The wet level of the submixin Decibels. Applied after submix effects and analysis are performed. */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetWetVolumeModulation, Category = SubmixLevel, meta = (DisplayName = "Wet Level (dB)", AudioParam = "Volume", AudioParamClass = "SoundModulationParameterVolume", ClampMin = "-96.0", ClampMax = "0.0", UIMin = "-96.0", UIMax = "0.0"))
	FSoundModulationDestinationSettings WetLevelModulation;

	/** The dry level of the submix in Decibels. Applied before submix effects and analysis are performed. */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetDryVolumeModulation, Category = SubmixLevel, meta = (DisplayName = "Dry Level (dB)", AudioParam = "Volume", AudioParamClass = "SoundModulationParameterVolume", ClampMin = "-96.0", ClampMax = "0.0", UIMin = "-96.0", UIMax = "0.0"))
	FSoundModulationDestinationSettings DryLevelModulation;
	
	/** Whether to send this Submix to AudioLink (when AudioLink is Enabled)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioLink)
	uint8 bSendToAudioLink : 1;

	/** Optional Audio Link Settings Object */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioLink)
	TObjectPtr<UAudioLinkSettingsAbstract> AudioLinkSettings;

	// Blueprint delegate for when a recorded file is finished exporting.
	UPROPERTY(BlueprintAssignable)
	FOnSubmixRecordedFileDone OnSubmixRecordedFileDone;

	// Start recording the audio from this submix.
	UFUNCTION(BlueprintCallable, Category = "Audio|Bounce", meta = (WorldContext = "WorldContextObject", DisplayName = "Start Recording Submix Output"))
	ENGINE_API void StartRecordingOutput(const UObject* WorldContextObject, float ExpectedDuration);

	ENGINE_API void StartRecordingOutput(FAudioDevice* InDevice, float ExpectedDuration);

	// Finish recording the audio from this submix and export it as a wav file or a USoundWave.
	UFUNCTION(BlueprintCallable, Category = "Audio|Bounce", meta = (WorldContext = "WorldContextObject", DisplayName = "Finish Recording Submix Output"))
	ENGINE_API void StopRecordingOutput(const UObject* WorldContextObject, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundWave* ExistingSoundWaveToOverwrite = nullptr);

	ENGINE_API void StopRecordingOutput(FAudioDevice* InDevice, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundWave* ExistingSoundWaveToOverwrite = nullptr);

	// Start envelope following the submix output. Register with OnSubmixEnvelope to receive envelope follower data in BP.
	UFUNCTION(BlueprintCallable, Category = "Audio|EnvelopeFollowing", meta = (WorldContext = "WorldContextObject"))
	ENGINE_API void StartEnvelopeFollowing(const UObject* WorldContextObject);

	ENGINE_API void StartEnvelopeFollowing(FAudioDevice* InDevice);

	// Start envelope following the submix output. Register with OnSubmixEnvelope to receive envelope follower data in BP.
	UFUNCTION(BlueprintCallable, Category = "Audio|EnvelopeFollowing", meta = (WorldContext = "WorldContextObject"))
	ENGINE_API void StopEnvelopeFollowing(const UObject* WorldContextObject);

	ENGINE_API void StopEnvelopeFollowing(FAudioDevice* InDevice);

	/**
	 *	Adds an envelope follower delegate to the submix when envelope following is enabled on this submix.
	 *	@param	OnSubmixEnvelopeBP	Event to fire when new envelope data is available.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|EnvelopeFollowing", meta = (WorldContext = "WorldContextObject"))
	ENGINE_API void AddEnvelopeFollowerDelegate(const UObject* WorldContextObject, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);

	/**
	 *	Remove an envelope follower delegate.
	 *	@param	OnSubmixEnvelopeBP	The event delegate to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|EnvelopeFollowing", meta = (WorldContext = "WorldContextObject"))
	ENGINE_API void RemoveEnvelopeFollowerDelegate(const UObject* WorldContextObject, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);

	/**
	 *	Adds a spectral analysis delegate to receive notifications when this submix has spectrum analysis enabled.
	 *	@param	InBandsettings					The frequency bands to analyze and their envelope-following settings.
	 *  @param  OnSubmixSpectralAnalysisBP		Event to fire when new spectral data is available.
	 *	@param	UpdateRate						How often to retrieve the data from the spectral analyzer and broadcast the event. Max is 30 times per second.
	 *	@param  InterpMethod                    Method to used for band peak calculation.
	 *	@param  SpectrumType                    Metric to use when returning spectrum values.
	 *	@param  DecibelNoiseFloor               Decibel Noise Floor to consider as silence when using a Decibel Spectrum Type.
	 *	@param  bDoNormalize                    If true, output band values will be normalized between zero and one.
	 *	@param  bDoAutoRange                    If true, output band values will have their ranges automatically adjusted to the minimum and maximum values in the audio. Output band values will be normalized between zero and one.
	 *	@param  AutoRangeAttackTime             The time (in seconds) it takes for the range to expand to 90% of a larger range.
	 *	@param  AutoRangeReleaseTime            The time (in seconds) it takes for the range to shrink to 90% of a smaller range.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Spectrum", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 3))
	ENGINE_API void AddSpectralAnalysisDelegate(const UObject* WorldContextObject, const TArray<FSoundSubmixSpectralAnalysisBandSettings>& InBandSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP, float UpdateRate = 10.f, float DecibelNoiseFloor=-40.f, bool bDoNormalize = true, bool bDoAutoRange = false, float AutoRangeAttackTime = 0.1f, float AutoRangeReleaseTime = 60.f);

	/**
	 *	Remove a spectral analysis delegate.
	 *  @param  OnSubmixSpectralAnalysisBP		The event delegate to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Spectrum", meta = (WorldContext = "WorldContextObject"))
	ENGINE_API void RemoveSpectralAnalysisDelegate(const UObject* WorldContextObject, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP);

	/** Start spectrum analysis of the audio output. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	ENGINE_API void StartSpectralAnalysis(const UObject* WorldContextObject, EFFTSize FFTSize = EFFTSize::DefaultSize, EFFTPeakInterpolationMethod InterpolationMethod = EFFTPeakInterpolationMethod::Linear, EFFTWindowType WindowType = EFFTWindowType::Hann, float HopSize = 0, EAudioSpectrumType SpectrumType = EAudioSpectrumType::MagnitudeSpectrum);

	ENGINE_API void StartSpectralAnalysis(FAudioDevice* InDevice, EFFTSize FFTSize = EFFTSize::DefaultSize, EFFTPeakInterpolationMethod InterpolationMethod = EFFTPeakInterpolationMethod::Linear, EFFTWindowType WindowType = EFFTWindowType::Hann, float HopSize = 0, EAudioSpectrumType SpectrumType = EAudioSpectrumType::MagnitudeSpectrum);

	/** Stop spectrum analysis of the audio output. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	ENGINE_API void StopSpectralAnalysis(const UObject* WorldContextObject);

	ENGINE_API void StopSpectralAnalysis(FAudioDevice* InDevice);

	/** Sets the output volume of the submix in linear gain. This dynamic volume acts as a multiplier on the OutputVolume property of this submix.  */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject", DisplayName = "SetSubmixOutputVolume (linear gain)"))
	ENGINE_API void SetSubmixOutputVolume(const UObject* WorldContextObject, float InOutputVolume);

	/** Sets the output volume of the submix in linear gain. This dynamic level acts as a multiplier on the WetLevel property of this submix.  */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject", DisplayName = "SetSubmixWetLevel (linear gain)"))
	ENGINE_API void SetSubmixWetLevel(const UObject* WorldContextObject, float InWetLevel);

	/** Sets the output volume of the submix in linear gain. This dynamic level acts as a multiplier on the DryLevel property of this submix.  */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject", DisplayName = "SetSubmixDryLevel (linear gain)"))
	ENGINE_API void SetSubmixDryLevel(const UObject* WorldContextObject, float InDryLevel);

	static ENGINE_API FSoundSpectrumAnalyzerSettings GetSpectrumAnalyzerSettings(EFFTSize FFTSize, EFFTPeakInterpolationMethod InterpolationMethod, EFFTWindowType WindowType, float HopSize, EAudioSpectrumType SpectrumType);

	static ENGINE_API FSoundSpectrumAnalyzerDelegateSettings GetSpectrumAnalysisDelegateSettings(const TArray<FSoundSubmixSpectralAnalysisBandSettings>& InBandSettings, float UpdateRate, float DecibelNoiseFloor, bool bDoNormalize, bool bDoAutoRange, float AutoRangeAttackTime, float AutoRangeReleaseTime);
protected:

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// Custom settors for Modulation Destinations
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	void SetOutputVolumeModulation(const FSoundModulationDestinationSettings& InVolMod);

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	void SetWetVolumeModulation(const FSoundModulationDestinationSettings& InVolMod);

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	void SetDryVolumeModulation(const FSoundModulationDestinationSettings& InVolMod);

	// Send submix modulation changes to audio devices
	void PushModulationChanges();

	/** Return the audio device that corresponds to the provided world context object. */
	static FAudioDevice* GetAudioDeviceFrom(const UObject* WorldContextObject);

	// State handling for bouncing output.
	TUniquePtr<Audio::FAudioRecordingData> RecordingData;

#if WITH_EDITORONLY_DATA

	// Forever deprecated properties.
	// These must be kept to always be able to migrate older assets.
	UPROPERTY()
	float OutputVolume_DEPRECATED;
	UPROPERTY()
	float WetLevel_DEPRECATED;
	UPROPERTY()
	float DryLevel_DEPRECATED;

	void InitDeprecatedDefaults();
	void HandleVersionMigration(const int32 Version);

#endif //WITH_EDITORONLY_DATA
};
	

/**
 * Sound Submix class meant for use with soundfield formats, such as Ambisonics.
 */
UCLASS(config = Engine, hidecategories = Object, editinlinenew, BlueprintType, Meta = (DisplayName = "Sound Submix Soundfield"), MinimalAPI)
class USoundfieldSubmix : public USoundSubmixWithParentBase
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API ISoundfieldFactory* GetSoundfieldFactoryForSubmix() const;
	ENGINE_API const USoundfieldEncodingSettingsBase* GetSoundfieldEncodingSettings() const;
	ENGINE_API TArray<USoundfieldEffectBase *> GetSoundfieldProcessors() const;

public:
	/** Currently used format. */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Soundfield)
	FName SoundfieldEncodingFormat;

	/** Which encoding settings to use the sound field. */
	UPROPERTY(EditAnywhere, Category = Soundfield)
	TObjectPtr<USoundfieldEncodingSettingsBase> EncodingSettings;

	/** Soundfield effect chain to use for the sound field. */
	UPROPERTY(EditAnywhere, Category = Soundfield)
	TArray<TObjectPtr<USoundfieldEffectBase>> SoundfieldEffectChain;

	// Traverses parent submixes until we find a submix that doesn't inherit its soundfield format.
	ENGINE_API FName GetSubmixFormat() const;

	UPROPERTY()
	TSubclassOf<USoundfieldEncodingSettingsBase> EncodingSettingsClass;

	// Traverses parent submixes until we find a submix that specifies encoding settings.
	ENGINE_API const USoundfieldEncodingSettingsBase* GetEncodingSettings() const;

	// This function goes through every child submix and the parent submix to ensure that they have a compatible format with this submix's format.
	ENGINE_API void SanitizeLinks();

protected:
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};

/**
 * Sound Submix class meant for sending audio to an external endpoint, such as controller haptics or an additional audio device.
 */
UCLASS(config = Engine, hidecategories = Object, editinlinenew, BlueprintType, Meta = (DisplayName = "Sound Submix Endpoint"), MinimalAPI)
class UEndpointSubmix : public USoundSubmixBase
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API IAudioEndpointFactory* GetAudioEndpointForSubmix() const;
	ENGINE_API const UAudioEndpointSettingsBase* GetEndpointSettings() const;

public:
	/** Currently used format. */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = Endpoint)
	FName EndpointType;

	UPROPERTY()
	TSubclassOf<UAudioEndpointSettingsBase> EndpointSettingsClass;

	UPROPERTY(EditAnywhere, Category = Endpoint)
	TObjectPtr<UAudioEndpointSettingsBase> EndpointSettings;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	ENGINE_API virtual void PostLoad() override;
};

/**
 * Sound Submix class meant for sending soundfield-encoded audio to an external endpoint, such as a hardware binaural renderer that supports ambisonics.
 */
UCLASS(config = Engine, hidecategories = Object, editinlinenew, BlueprintType, Meta = (DisplayName = "Sound Submix Soundfield Endpoint"), MinimalAPI)
class USoundfieldEndpointSubmix : public USoundSubmixBase
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API ISoundfieldEndpointFactory* GetSoundfieldEndpointForSubmix() const;
	ENGINE_API const USoundfieldEndpointSettingsBase* GetEndpointSettings() const;
	ENGINE_API const USoundfieldEncodingSettingsBase* GetEncodingSettings() const;
	ENGINE_API TArray<USoundfieldEffectBase*> GetSoundfieldProcessors() const;
public:
	/** Currently used format. */
	UPROPERTY(EditAnywhere, Category = Endpoint, AssetRegistrySearchable)
	FName SoundfieldEndpointType;

	UPROPERTY()
	TSubclassOf<UAudioEndpointSettingsBase> EndpointSettingsClass;

	/**
	* @return true if the child sound class exists in the tree
	*/
	ENGINE_API bool RecurseCheckChild(const USoundSubmix* ChildSoundSubmix) const;

	// This function goes through every child submix and the parent submix to ensure that they have a compatible format.
	ENGINE_API void SanitizeLinks();

	UPROPERTY(EditAnywhere, Category = Endpoint)
	TObjectPtr<USoundfieldEndpointSettingsBase> EndpointSettings;

	UPROPERTY()
	TSubclassOf<USoundfieldEncodingSettingsBase> EncodingSettingsClass;

	UPROPERTY(EditAnywhere, Category = Soundfield)
	TObjectPtr<USoundfieldEncodingSettingsBase> EncodingSettings;

	UPROPERTY(EditAnywhere, Category = Soundfield)
	TArray<TObjectPtr<USoundfieldEffectBase>> SoundfieldEffectChain;

protected:
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

namespace SubmixUtils
{
	ENGINE_API void ForEachStaticChildRecursive(USoundSubmixBase* StartingPoint, const TFunction<void(USoundSubmixBase*)>& Op);

	ENGINE_API bool AreSubmixFormatsCompatible(const USoundSubmixBase* ChildSubmix, const USoundSubmixBase* ParentSubmix);

	ENGINE_API bool FindInGraph(const USoundSubmixBase* InEntryPoint, const USoundSubmixBase* InToMatch, bool bShouldAcsend, FAudioDeviceHandle InDevice = {});
	ENGINE_API const USoundSubmixBase* FindRoot(const USoundSubmixBase* InStartingPoint, FAudioDeviceHandle InDevice);

#if WITH_EDITOR
	ENGINE_API void RefreshEditorForSubmix(const USoundSubmixBase* InSubmix);
#endif
}
