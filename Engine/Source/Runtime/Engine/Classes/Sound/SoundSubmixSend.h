// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Curves/CurveFloat.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SoundSubmixSend.generated.h"


// Forward Declarations
class USoundSubmixBase;

UENUM(BlueprintType)
enum class EAudioSpectrumBandPresetType: uint8
{
	/** Band which contains frequencies generally related to kick drums. */
	KickDrum,

	/** Band which contains frequencies generally related to snare drums. */
	SnareDrum,

	/** Band which contains frequencies generally related to vocals. */
	Voice,

	/** Band which contains frequencies generally related to cymbals. */
	Cymbals	
};

USTRUCT(BlueprintType)
struct FSoundSubmixSpectralAnalysisBandSettings
{
	GENERATED_USTRUCT_BODY()

	// The frequency band for the magnitude to analyze
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSpectralAnalysis, meta = (ClampMin = "10.0", ClampMax = "20000.0", UIMin = "10.0", UIMax = "10000.0"))
	float BandFrequency = 440.0f;

	// The attack time for the FFT band interpolation for delegate callback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSpectralAnalysis, meta = (ClampMin = "0.0", UIMin = "10.0", UIMax = "10000.0"))
	int32 AttackTimeMsec = 10;

	// The release time for the FFT band interpolation for delegate callback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSpectralAnalysis, meta = (ClampMin = "0.0", UIMin = "10.0", UIMax = "10000.0"))
	int32 ReleaseTimeMsec = 500;

	// The ratio of the bandwidth divided by the center frequency. Only used when the spectral analysis type is set to Constant Q.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = SubmixSpectralAnalysis, meta = (ClampMin = "0.001", UIMin = "0.1", UIMax = "100.0"))
	float QFactor = 10.0f;
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnSubmixEnvelopeBP, const TArray<float>&, Envelope);

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnSubmixSpectralAnalysisBP, const TArray<float>&, Magnitude);

UENUM(BlueprintType)
enum class EAudioRecordingExportType : uint8
{
	// Exports a USoundWave.
	SoundWave,

	// Exports a WAV file.
	WavFile
};

UENUM(BlueprintType)
enum class ESendLevelControlMethod : uint8
{
	// A send based on linear interpolation between the distance range and the remapped range
	Linear UMETA(DisplayName = "Distance Linear"),

	// A send based on distance using the supplied curve mapping (distance = X, send level = Y)
	CustomCurve UMETA(DisplayName = "Distance Custom"),

	// A constant send level (Uses the specified constant send level value. Useful for 2D sounds.)
	Manual UMETA(DisplayName = "Constant"),
};

// Common set of settings that are uses as submix sends.
USTRUCT(BlueprintType)
struct FSoundSubmixSendInfoBase
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FSoundSubmixSendInfoBase();

#if WITH_EDITORONLY_DATA
	// Implicit special members would touch the deprecated members and warn at the synthesis site (outside the field-decl pragma). Default them explicitly inside the pragma so synthesis sits in the suppressed region.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSoundSubmixSendInfoBase(const FSoundSubmixSendInfoBase&) = default;
	FSoundSubmixSendInfoBase(FSoundSubmixSendInfoBase&&) = default;
	FSoundSubmixSendInfoBase& operator=(const FSoundSubmixSendInfoBase&) = default;
	FSoundSubmixSendInfoBase& operator=(FSoundSubmixSendInfoBase&&) = default;
	~FSoundSubmixSendInfoBase() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	ENGINE_API void PostSerialize(const FArchive& Ar);

	// Changes the method we use to determine the send level - see enum value tooltips for more information
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend)
	ESendLevelControlMethod SendLevelControlMethod;
	
	// The Submix to send the audio to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend)
	TObjectPtr<USoundSubmixBase> SoundSubmix;

	// Manually set the amount of audio to send
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Send Level", EditCondition = "SendLevelControlMethod == ESendLevelControlMethod::Manual", EditConditionHides))
	float SendLevel;

	// Whether to disable the internal 0-1 clamp for Constant Send Level control
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Disable Send Level Clamp", EditCondition = "SendLevelControlMethod == ESendLevelControlMethod::Manual", EditConditionHides))
	bool DisableManualSendClamp;

	// The amount to send to the Submix when sound is located at a distance less than or equal to value specified in the Send Distance Min
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Send Level Min", EditCondition = "SendLevelControlMethod == ESendLevelControlMethod::Linear", EditConditionHides))
	float MinSendLevel;

	// The amount to send to the Submix when sound is located at a distance greater than or equal to value specified in the Send Distance Max
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Send Level Max", EditCondition = "SendLevelControlMethod == ESendLevelControlMethod::Linear", EditConditionHides))
	float MaxSendLevel;

	/** The distance at which to start mapping between to Send Level Min/Max 
	 *  Distances LESS than this will result in a clamped Send Level Min
	 */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Send Distance Min", EditCondition = "SendLevelControlMethod != ESendLevelControlMethod::Manual", EditConditionHides))
	float MinSendDistance;

	/** The distance at which to stop mapping between Send Level Min/Max
	 *  Distances GREATER than this will result in a clamped Send Level Max
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (DisplayName = "Send Distance Max", EditCondition = "SendLevelControlMethod != ESendLevelControlMethod::Manual", EditConditionHides))
	float MaxSendDistance;

	// The custom send curve to use for distance-based send level. (0.0-1.0 on the curve's X-axis maps to Min/Max Send Distance)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend, meta = (EditCondition = "SendLevelControlMethod == ESendLevelControlMethod::CustomCurve", EditConditionHides))
	FRuntimeFloatCurve CustomSendLevelCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(InlineEditConditionToggle))
	uint8 bEnableLPFCutoff : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(Units="Hz", Delta="0.1", ClampMin="20.0", ClampMax="20000.0", UIMin="20.0", UIMax="20000.0", EditCondition="bEnableLPFCutoff"))
	float LPFCutoff;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(InlineEditConditionToggle))
	uint8 bEnableHPFCutoff : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="HPF/LPF", meta=(Units="Hz", Delta="0.1", ClampMin="20.0", ClampMax="20000.0", UIMin="20.0", UIMax="20000.0", EditCondition="bEnableHPFCutoff"))
	float HPFCutoff;

#if WITH_EDITORONLY_DATA
	// Legacy fields kept for tagged-load matching; migrated in PostSerialize. UHT strips _DEPRECATED from the EngineName.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use bEnableLPFCutoff + LPFCutoff instead.")
	UPROPERTY()
	TOptional<float> LPFCutoffFrequency_DEPRECATED;

	UE_DEPRECATED(5.8, "Use bEnableHPFCutoff + HPFCutoff instead.")
	UPROPERTY()
	TOptional<float> HPFCutoffFrequency_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FSoundSubmixSendInfoBase> : public TStructOpsTypeTraitsBase2<FSoundSubmixSendInfoBase>
{
	enum
	{
		WithPostSerialize = true,
	};
};

UENUM(BlueprintType)
enum class ESubmixSendStage : uint8
{
	// Whether to do the send post distance attenuation
	PostDistanceAttenuation,

	// Whether to do the send pre distance attenuation
	PreDistanceAttenuation,
};

USTRUCT(BlueprintType)
struct FSoundSubmixSendInfo : public FSoundSubmixSendInfoBase
{
	GENERATED_BODY();

	/** Defines at what mix stage the send should happen.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixSend)
	ESubmixSendStage SendStage = ESubmixSendStage::PostDistanceAttenuation;
};

// Trait specializations don't inherit; re-specialize for each serialized derived type.
template<>
struct TStructOpsTypeTraits<FSoundSubmixSendInfo> : public TStructOpsTypeTraitsBase2<FSoundSubmixSendInfo>
{
	enum
	{
		WithPostSerialize = true,
	};
};
