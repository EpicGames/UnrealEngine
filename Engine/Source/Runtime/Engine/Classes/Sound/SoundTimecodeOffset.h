// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SoundTimecodeOffset.generated.h"

USTRUCT()
struct FSoundTimecodeOffset
{
	GENERATED_BODY()

	UPROPERTY()
	double NumOfSecondsSinceMidnight = 0.0;

	constexpr bool operator==(const FSoundTimecodeOffset& InRhs) const
	{
		// We largely only care about the 0 case, so double equality is ok.
		return NumOfSecondsSinceMidnight == InRhs.NumOfSecondsSinceMidnight;
	}
};
