// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Playable sound object for raw wave files
 */

#ifndef UE_USOUNDWAVE_HOT_RELOADING_ENABLED
#define UE_USOUNDWAVE_HOT_RELOADING_ENABLED PLATFORM_DESKTOP && !UE_BUILD_SHIPPING
#endif // #ifndef UE_USOUNDWAVE_HOT_RELOADING_ENABLED

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundModulationDestination.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Async/AsyncWork.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundWaveTimecodeInfo.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Serialization/BulkData.h"
#include "Serialization/BulkDataBuffer.h"
#include "Serialization/EditorBulkData.h"
#include "Sound/SoundGroups.h"
#include "Sound/SoundWaveLoadingBehavior.h"
#include "UObject/ObjectKey.h"
#include "AudioMixerTypes.h"
#include "AudioCompressionSettings.h"
#include "UObject/PerPlatformProperties.h"
#include "ContentStreaming.h"
#include "IAudioProxyInitializer.h"
#include "IWaveformTransformation.h"
#include "ISoundWaveCloudStreaming.h"
#include "Templates/DontCopy.h"
#include "SoundWave.generated.h"

class FSoundWaveData;
class ITargetPlatform;
enum EAudioSpeakers : int;
struct FActiveSound;
struct FSoundParseParameters;
struct FPlatformAudioCookOverrides;

UENUM()
enum EDecompressionType : int
{
	DTYPE_Setup,
	DTYPE_Invalid,
	DTYPE_RealTime,
	DTYPE_Procedural,
	DTYPE_Xenon,
	DTYPE_Streaming,
	DTYPE_MAX,
};

inline const TCHAR* LexToString(const EDecompressionType InType)
{
#define CASE_TO_STRING(X) case X: return TEXT(#X);
	switch(InType)
	{
		FOREACH_ENUM_EDECOMPRESSIONTYPE(CASE_TO_STRING);
		default: break;
	}
	return TEXT("Unknown");
#undef CASE_TO_STRING
}

/**
 * Dictates whether to use the CuePoints and Loop Regions from the .wav file or from the waveform editor
 * Marker transformations during playback
 */
UENUM()
enum class ESoundWaveCuePointOrigin : uint8
{
	WaveFile = 0,
	MarkerTransformation,
};

/** Precache states */
enum class ESoundWavePrecacheState
{
	NotStarted,
	InProgress,
	Done
};

inline constexpr uint64 InvalidAudioStreamCacheLookupID = TNumericLimits<uint64>::Max();

/**
 * A chunk of streamed audio.
 */
struct FStreamedAudioChunk
{
	/** Serialization. */
	void Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex);

	/**  returns false if data retrieval failed */
	bool GetCopy(void** OutChunkData);

	/* Moves the memory out of the byte bulk-data. (Does copy with Discard original) */
	FBulkDataBuffer<uint8> MoveOutAsBuffer();

	/** Size of the chunk of data in bytes including zero padding */
	int32 DataSize = 0;

	/** Size of the audio data. (NOTE: This includes a seek-table if its present.) */
	int32 AudioDataSize = 0;

	/** Chunk position in samples frames in the stream. (NOTE: != INDEX_NONE will assume the presence of a seektable). */
	uint32 SeekOffsetInAudioFrames = uint32(INDEX_NONE);

	/** Bulk data if stored in the package. */
	FByteBulkData BulkData;

private:
	uint8* CachedDataPtr{ nullptr };

public:

#if WITH_EDITORONLY_DATA
	/** Key if stored in the derived data cache. */
	FString DerivedDataKey;

	/** True if this chunk was loaded from a cooked package. */
	bool bLoadedFromCookedPackage = false;

	/** If marked true, will attempt to inline this chunk. */
	bool bInlineChunk = false;

	/**
	 * Place chunk data in the derived data cache associated with the provided
	 * key.
	 */
	uint32 StoreInDerivedDataCache(const FString& InDerivedDataKey, const FStringView& SoundWaveName);
#endif // #if WITH_EDITORONLY_DATA
};

/**
 * Platform-specific data used streaming audio at runtime.
 */
USTRUCT()
struct FStreamedAudioPlatformData
{
	GENERATED_USTRUCT_BODY()

	/** Format in which audio chunks are stored. */
	FName AudioFormat;
	/** audio data. */
	TIndirectArray<struct FStreamedAudioChunk> Chunks;

#if WITH_EDITORONLY_DATA
	/** The key associated with this derived data. */
	FString DerivedDataKey;
	/** Protection for AsyncTask manipulation since it can be accessed from multiple threads */
	mutable TDontCopy<FRWLock> AsyncTaskLock;
	/** Async cache task if one is outstanding. */
	struct FStreamedAudioAsyncCacheDerivedDataTask* AsyncTask;
#endif // #if WITH_EDITORONLY_DATA

	/** Default constructor. */
	ENGINE_API FStreamedAudioPlatformData();

	/** Destructor. */
	ENGINE_API ~FStreamedAudioPlatformData();

	/**
	 * Try to load audio chunk from the derived data cache or build it if it isn't there.
	 * @param ChunkIndex	The Chunk index to load.
	 * @param OutChunkData	Address of pointer that will store chunk data - should
	 *						either be NULL or have enough space for the chunk
	 * @returns if > 0, the size of the chunk in bytes. If 0, the chunk failed to load.
	 */
	ENGINE_API int32 GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded = false);

	/** Get the chunks while making sure any async task are finished before returning. */
	ENGINE_API TIndirectArray<struct FStreamedAudioChunk>& GetChunks() const;

	/** Get the number of chunks while making sure any async task are finished before returning. */
	ENGINE_API int32 GetNumChunks() const;

	/** Get the audio format making sure any async task are finished before returning. */
	ENGINE_API FName GetAudioFormat() const;

	/** Serialization. */
	ENGINE_API void Serialize(FArchive& Ar, class USoundWave* Owner);

#if WITH_EDITORONLY_DATA
	ENGINE_API void Cache(class USoundWave& InSoundWave, const FPlatformAudioCookOverrides* CompressionOverrides, FName AudioFormatName, uint32 InFlags, const ITargetPlatform* InTargetPlatform=nullptr);
	ENGINE_API void FinishCache();
	ENGINE_API bool IsFinishedCache() const;
	ENGINE_API bool IsAsyncWorkComplete() const;
	ENGINE_API bool IsCompiling() const;
	ENGINE_API bool TryInlineChunkData();

	UE_DEPRECATED(5.0, "Use AreDerivedChunksAvailable with the context instead.")
	ENGINE_API bool AreDerivedChunksAvailable() const;
	
	ENGINE_API bool AreDerivedChunksAvailable(FStringView Context) const;
#endif // WITH_EDITORONLY_DATA

private:
#if WITH_EDITORONLY_DATA
	friend class USoundWave;
	/**  Utility function used internally to change task priority while maintaining thread-safety. */
	ENGINE_API bool RescheduleAsyncTask(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority);
	/**  Utility function used internally to wait or poll a task while maintaining thread-safety. */
	ENGINE_API bool WaitAsyncTaskWithTimeout(float InTimeoutInSeconds);
#endif

	/**
	 * Takes the results of a DDC operation and deserializes it into an FStreamedAudioChunk struct.
	 * @param SerializedData Serialized data resulting from DDC.GetAsynchronousResults or DDC.GetSynchronous.
	 * @param ChunkToDeserializeInto is the chunk to fill with the deserialized data.
	 * @param ChunkIndex is the index of the chunk in this instance of FStreamedAudioPlatformData.
	 * @param OutChunkData is a pointer to a pointer to populate with the chunk itself, or if pointing to nullptr, returns an allocated buffer.
	 * @returns the size of the chunk loaded in bytes, or zero if the chunk didn't load.
	 */
	ENGINE_API int32 DeserializeChunkFromDDC(TArray<uint8> SerializedData, FStreamedAudioChunk &ChunkToDeserializeInto, int32 ChunkIndex, uint8** &OutChunkData);
};

USTRUCT(BlueprintType)
struct FSoundWaveSpectralData
{
	GENERATED_USTRUCT_BODY()

	// The frequency (in Hz) of the spectrum value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float FrequencyHz = 0.0f;

	// The magnitude of the spectrum at this frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float Magnitude = 0.0f;

	// The normalized magnitude of the spectrum at this frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float NormalizedMagnitude = 0.0f;
};

USTRUCT(BlueprintType)
struct FSoundWaveSpectralDataPerSound
{
	GENERATED_USTRUCT_BODY()

	// The array of current spectral data for this sound wave
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	TArray<FSoundWaveSpectralData> SpectralData;

	// The current playback time of this sound wave
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float PlaybackTime = 0.0f;

	// The sound wave this spectral data is associated with
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	TObjectPtr<USoundWave> SoundWave = nullptr;
};

USTRUCT(BlueprintType)
struct FSoundWaveEnvelopeDataPerSound
{
	GENERATED_USTRUCT_BODY()

	// The current envelope of the playing sound
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EnvelopeData")
	float Envelope = 0.0f;

	// The current playback time of this sound wave
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EnvelopeData")
	float PlaybackTime = 0.0f;

	// The sound wave this envelope data is associated with
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "EnvelopeData")
	TObjectPtr<USoundWave> SoundWave = nullptr;
};

// Sort predicate for sorting spectral data by frequency (lowest first)
struct FCompareSpectralDataByFrequencyHz
{
	inline bool operator()(const FSoundWaveSpectralData& A, const FSoundWaveSpectralData& B) const
	{
		return A.FrequencyHz < B.FrequencyHz;
	}
};


// Struct used to store spectral data with time-stamps
USTRUCT()
struct FSoundWaveSpectralDataEntry
{
	GENERATED_USTRUCT_BODY()

	// The magnitude of the spectrum at this frequency
	UPROPERTY()
	float Magnitude = 0.0f;

	// The normalized magnitude of the spectrum at this frequency
	UPROPERTY()
	float NormalizedMagnitude = 0.0f;
};


// Struct used to store spectral data with time-stamps
USTRUCT()
struct FSoundWaveSpectralTimeData
{
	GENERATED_USTRUCT_BODY()

	// The spectral data at the given time. The array indices correspond to the frequencies set to analyze.
	UPROPERTY()
	TArray<FSoundWaveSpectralDataEntry> Data;

	// The timestamp associated with this spectral data
	UPROPERTY()
	float TimeSec = 0.0f;
};

// Struct used to store time-stamped envelope data
USTRUCT()
struct FSoundWaveEnvelopeTimeData
{
	GENERATED_USTRUCT_BODY()

	// The normalized linear amplitude of the audio
	UPROPERTY()
	float Amplitude = 0.0f;

	// The timestamp of the audio
	UPROPERTY()
	float TimeSec = 0.0f;
};

// The FFT size (in audio frames) to use for baked FFT analysis
UENUM(BlueprintType)
enum class ESoundWaveFFTSize : uint8
{
	VerySmall_64,
	Small_256,
	Medium_512,
	Large_1024,
	VeryLarge_2048,
};

// Sound Asset Compression Type
UENUM(BlueprintType)
enum class ESoundAssetCompressionType : uint8
{
	// Perceptual-based codec which supports all features across all platforms.
	BinkAudio,

	// Will encode the asset using ADPCM, a time-domain codec that has fixed-sized quality and about ~4x compression ratio, but is relatively cheap to decode.
	ADPCM,

	// Uncompressed audio. Large memory usage (streamed chunks contain less audio per chunk) but extremely cheap to decode and supports all features. 
	PCM,

	// Opus is a highly versatile audio codec. It is primarily designed for interactive speech and music transmission over the Internet, but is also applicable to storage and streaming applications.
	Opus,

	// Encodes the asset to a platform specific format and will be different depending on the platform. It does not currently support seeking.
	PlatformSpecific,

	// The project defines the codec used for this asset.
	ProjectDefined,

	// As BinkAudio, except better quality. Comparable CPU usage. Only valid sample rates are: 48000, 44100, 32000, and 24000.
	RADAudio UMETA(DisplayName = "RAD Audio"),
};


namespace Audio
{
	inline FName ToName(ESoundAssetCompressionType InDecoderType)
	{
		switch (InDecoderType)
		{
		case ESoundAssetCompressionType::RADAudio:				return NAME_RADA;
		case ESoundAssetCompressionType::BinkAudio:				return NAME_BINKA;
		case ESoundAssetCompressionType::ADPCM:					return NAME_ADPCM;
		case ESoundAssetCompressionType::PCM:					return NAME_PCM;
		case ESoundAssetCompressionType::Opus:					return NAME_OPUS;
		case ESoundAssetCompressionType::PlatformSpecific:		return NAME_PLATFORM_SPECIFIC;
		case ESoundAssetCompressionType::ProjectDefined:		return NAME_PROJECT_DEFINED;
		default:
			ensure(false);
			return TEXT("UNKNOWN");
		}
	}

	inline ESoundAssetCompressionType ToSoundAssetCompressionType(EDefaultAudioCompressionType InDefaultCompressionType)
	{
		switch (InDefaultCompressionType)
		{
			case EDefaultAudioCompressionType::RADAudio:			return ESoundAssetCompressionType::RADAudio;
			case EDefaultAudioCompressionType::BinkAudio:			return ESoundAssetCompressionType::BinkAudio;
			case EDefaultAudioCompressionType::ADPCM:				return ESoundAssetCompressionType::ADPCM;
			case EDefaultAudioCompressionType::PCM:					return ESoundAssetCompressionType::PCM;
			case EDefaultAudioCompressionType::Opus:				return ESoundAssetCompressionType::Opus;
			case EDefaultAudioCompressionType::PlatformSpecific:	return ESoundAssetCompressionType::PlatformSpecific;
			default:
				ensure(false);
				return ESoundAssetCompressionType::PlatformSpecific;
		}
	}
}

struct ISoundWaveClient
{
	ISoundWaveClient() {}
	virtual ~ISoundWaveClient() = default;
	
	// OnBeginDestroy() returns true to unsubscribe as an ISoundWaveClient
	virtual bool OnBeginDestroy(class USoundWave* Wave) = 0;
	virtual bool OnIsReadyForFinishDestroy(class USoundWave* Wave) const = 0;
	virtual void OnFinishDestroy(class USoundWave* Wave) = 0;
};

UCLASS(hidecategories=Object, editinlinenew, BlueprintType, meta= (LoadBehavior = "LazyOnDemand"), MinimalAPI)
class USoundWave : public USoundBase, public IAudioProxyDataFactory, public IInterface_AsyncCompilation
{
	GENERATED_UCLASS_BODY()

private:

	/** Platform agnostic compression quality. 1..100 with 1 being best compression and 100 being best quality. ADPCM and PCM sound asset compression types ignore this parameter. */
	UPROPERTY(Config, EditAnywhere, Category = "Info|Quality", meta = (DisplayName = "Compression", ClampMin = "1", ClampMax = "100", EditCondition = "SoundAssetCompressionType != ESoundAssetCompressionType::PCM && SoundAssetCompressionType != ESoundAssetCompressionType::ADPCM"), AssetRegistrySearchable)
	int32 CompressionQuality;

public:

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - Property is deprecated. Streaming priority has no effect with stream caching enabled."))
	int32 StreamingPriority;

	/** Determines the max sample rate to use if the platform enables "Resampling For Device" in project settings. 
	*	For example, if the platform enables Resampling For Device and specifies 32000 for High, then setting High here will
	*	force the sound wave to be _at most_ 32000. Does nothing if Resampling For Device is disabled.
	*/
	UPROPERTY(EditAnywhere, Category = "Info|Quality")
	ESoundwaveSampleRateSettings SampleRateQuality;

	/** Type of buffer this wave uses. Set once on load */
	TEnumAsByte<EDecompressionType> DecompressionType;

	UPROPERTY(EditAnywhere, Category = "Info", meta = (DisplayName = "Group"))
	TEnumAsByte<ESoundGroup> SoundGroup;

	/** If set, when played directly (not through a sound cue) the wave will be played looping. */
	UPROPERTY(EditAnywhere, Category = "Sound", AssetRegistrySearchable)
	uint8 bLooping : 1;

	/** Here for legacy code. */
	UPROPERTY()
	uint8 bStreaming : 1;

private:

	// Deprecated compression type properties
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - Property is deprecated. bSeekableStreaming now means ADPCM codec in SoundAssetCompressionType."))
	uint8 bSeekableStreaming : 1;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "5.0 - Property is deprecated. bUseBinkAudio now means Bink codec in SoundAssetCompressionType."))
	uint8 bUseBinkAudio : 1;

	/** The compression type to use for the sound wave asset. */
	UPROPERTY(Config, EditAnywhere, Category = "Info|Format", AssetRegistrySearchable)
	ESoundAssetCompressionType SoundAssetCompressionType = ESoundAssetCompressionType::PlatformSpecific;

public:
	void AddPlayingSource()
	{
		NumSourcesPlaying.Increment();
	}

	void RemovePlayingSource()
	{
		check(NumSourcesPlaying.GetValue() > 0);
		NumSourcesPlaying.Decrement();
	}

	/** Returns the sound's asset compression type. */
	UFUNCTION(BlueprintPure, Category = "Audio")
	ENGINE_API ESoundAssetCompressionType GetSoundAssetCompressionType() const;

	/** will return the raw value, (i.e. does not resolve options such as "Project Defined" to the correct codec) */
	ENGINE_API ESoundAssetCompressionType GetSoundAssetCompressionTypeEnum() const;

	/** Procedurally set the compression type. */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	ENGINE_API void SetSoundAssetCompressionType(ESoundAssetCompressionType InSoundAssetCompressionType, bool bMarkDirty = true);

	/** Filters for the cue points that are _not_ loop regions and returns those as a new array */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	ENGINE_API TArray<FSoundWaveCuePoint> GetCuePoints() const;
	ENGINE_API void SetAllCuePoints(TArray<FSoundWaveCuePoint> InCuePoints) const;

#if WITH_EDITOR
	ENGINE_API TArray<FSoundWaveCuePoint> GetSoundWaveCuePoints() const;
	ENGINE_API void SetSoundWaveCuePoints(const TArray<FSoundWaveCuePoint>& InCuePoints);

	ENGINE_API const ESoundWaveCuePointOrigin GetCuePointOrigin() const;
	ENGINE_API void SetCuePointOrigin(const ESoundWaveCuePointOrigin InCuePointOrigin);

	static FName GetCuePointOriginPropertyName() { return GET_MEMBER_NAME_CHECKED(USoundWave, CuePointOrigin); }

	/** Set gated LUFS and SamplePeakDB values */
	ENGINE_API void SetLoudnessValues(const float InLUFS, const float InSamplePeakDB);
#endif //WITH_EDITOR

	/** Filters for the cue points that _are_ loop regions and returns those as a new array */
	UFUNCTION(BlueprintCallable, Category = "Audio")
	ENGINE_API TArray<FSoundWaveCuePoint> GetLoopRegions() const;

	/** Returns the Runtime format of the wave */
	ENGINE_API FName GetRuntimeFormat() const;

	/**
	* For use when creating a new soundwave with another soundwave used as a template
	* We do not want to inherit these values from the template, as the data may be incorrect
	* rather we re-parse them from the incoming file.
	*/
	ENGINE_API void ResetInvalidTemplateArguments();

private:
	/** if UE_USOUNDWAVE_HOT_RELOADING_ENABLED is true, the sound wave will have
		its own, un-published copy of sound wave data (TSharedRef<FSoundWaveData> WorkingSoundWaveData)
		Otherwise, we use and make changes to the Proxy->PublishedSoundWaveData directly.

		This getter will return the correct one.
	*/
	const TSharedRef<FSoundWaveData>& GetWorkingSoundWaveData() const;

	TSharedRef<FSoundWaveProxy> Proxy = MakeShared<FSoundWaveProxy>();
	
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	TSharedRef<FSoundWaveData> WorkingSoundWaveData = MakeShared<FSoundWaveData>();
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

public:

	using FSoundWaveClientPtr = ISoundWaveClient*;

#if WITH_EDITORONLY_DATA
	/** Specify a sound to use for the baked analysis. Will default to this USoundWave if not set. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis")
	TObjectPtr<USoundWave> OverrideSoundToUseForAnalysis;

	/**
		Whether or not we should treat the sound wave used for analysis (this or the override) as looping while performing analysis.
		A looping sound may include the end of the file for inclusion in analysis for envelope and FFT analysis.
	*/
	UPROPERTY(EditAnywhere, Category = "Info|Analysis")
	uint8 TreatFileAsLoopingForAnalysis : 1;

	/** Whether or not to enable cook-time baked FFT analysis. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT")
	uint8 bEnableBakedFFTAnalysis : 1;

	/** Whether or not to enable cook-time amplitude envelope analysis. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|Envelope")
	uint8 bEnableAmplitudeEnvelopeAnalysis : 1;

	/** The FFT window size to use for fft analysis. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT", meta = (EditCondition = "bEnableBakedFFTAnalysis"))
	ESoundWaveFFTSize FFTSize;

	/** How many audio frames analyze at a time. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT", meta = (EditCondition = "bEnableBakedFFTAnalysis", ClampMin = "512", UIMin = "512"))
	int32 FFTAnalysisFrameSize;

	/** Attack time in milliseconds of the spectral envelope follower. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT", meta = (EditCondition = "bEnableBakedFFTAnalysis", ClampMin = "0", UIMin = "0"))
	int32 FFTAnalysisAttackTime;

	/** Release time in milliseconds of the spectral envelope follower. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT", meta = (EditCondition = "bEnableBakedFFTAnalysis", ClampMin = "0", UIMin = "0"))
	int32 FFTAnalysisReleaseTime;

	/** How many audio frames to average a new envelope value. Larger values use less memory for audio envelope data but will result in lower envelope accuracy. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|Envelope", meta = (EditCondition = "bEnableAmplitudeEnvelopeAnalysis", ClampMin = "512", UIMin = "512"))
	int32 EnvelopeFollowerFrameSize;

	/** The attack time in milliseconds. Describes how quickly the envelope analyzer responds to increasing amplitudes. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|Envelope", meta = (EditCondition = "bEnableAmplitudeEnvelopeAnalysis", ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerAttackTime;

	/** The release time in milliseconds. Describes how quickly the envelope analyzer responds to decreasing amplitudes. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|Envelope", meta = (EditCondition = "bEnableAmplitudeEnvelopeAnalysis", ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerReleaseTime;
#endif // WITH_EDITORONLY_DATA

	/** Modulation Settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Modulation")
	FSoundModulationDefaultRoutingSettings ModulationSettings;

	/** The frequencies (in hz) to analyze when doing baked FFT analysis. */
	UPROPERTY(EditAnywhere, Category = "Info|Analysis|FFT", meta = (EditCondition = "bEnableBakedFFTAnalysis"))
	TArray<float> FrequenciesToAnalyze;

	/** The cooked spectral time data. */
	UPROPERTY()
	TArray<FSoundWaveSpectralTimeData> CookedSpectralTimeData;

	/** The cooked cooked envelope data. */
	UPROPERTY()
	TArray<FSoundWaveEnvelopeTimeData> CookedEnvelopeTimeData;

	/** Helper function to get interpolated cooked FFT data for a given time value. */
	ENGINE_API bool GetInterpolatedCookedFFTDataForTime(float InTime, uint32& InOutLastIndex, TArray<FSoundWaveSpectralData>& OutData, bool bLoop);
	ENGINE_API bool GetInterpolatedCookedEnvelopeDataForTime(float InTime, uint32& InOutLastIndex, float& OutAmplitude, bool bLoop);

	/** If stream caching is enabled, allows the user to retain a strong handle to the first chunk of audio in the cache.
	 *  Please note that this USoundWave is NOT guaranteed to be still alive when OnLoadCompleted is called.
	 */
	ENGINE_API void GetHandleForChunkOfAudio(TFunction<void(FAudioChunkHandle&&)> OnLoadCompleted, bool bForceSync = false, int32 ChunkIndex = 1, ENamedThreads::Type CallbackThread = ENamedThreads::GameThread);

	/** If stream caching is enabled, set this sound wave to retain a strong handle to its first chunk.
	 *  If not called on the game thread, bForceSync must be true.
	*/
	ENGINE_API void RetainCompressedAudio(bool bForceSync = false);

	/** If stream caching is enabled and au.streamcache.KeepFirstChunkInMemory is 1, this will release this USoundWave's first chunk, allowing it to be deleted. */
	ENGINE_API void ReleaseCompressedAudio();

	ENGINE_API bool IsRetainingAudio();
	/**
	 * If Stream Caching is enabled, this can be used to override the default loading behavior of this USoundWave.
	 * This can even be called on USoundWaves that still have the RF_NeedLoad flag, and won't be stomped by serialization.
	 * NOTE: The new behavior will be ignored if it is less memory-aggressive than existing (even inherited) behavior
	 */
	ENGINE_API void OverrideLoadingBehavior(ESoundWaveLoadingBehavior InLoadingBehavior);

	/** Returns the loading behavior we should use for this sound wave.
	 *  If this is called within Serialize(), this should be called with bCheckSoundClasses = false,
	 *  Since there is no guarantee that the deserialized USoundClasses have been resolved yet.
	 */
	ENGINE_API ESoundWaveLoadingBehavior GetLoadingBehavior(bool bCheckSoundClasses = true) const;

#if WITH_EDITORONLY_DATA
	/** Please use size of First Chunk in Seconds. */
	UPROPERTY(AdvancedDisplay, meta = (DeprecatedProperty))
	int32 InitialChunkSize_DEPRECATED;
#endif

#if WITH_EDITOR
	ENGINE_API const FWaveTransformUObjectConfiguration& GetTransformationChainConfig() const;
	ENGINE_API const FWaveTransformUObjectConfiguration& UpdateTransformations();
	// Pass in a Transformations array to overwrite this soundwave with another soundwaves transformation data
	ENGINE_API void OverwriteSoundWaveDataWithTransformationData(const TArrayView<TObjectPtr<UWaveformTransformationBase>> InTransformations = {});
	ENGINE_API FWaveformTransformationInfo GetWaveformTransformationInfo() const;

#endif

private:

	/** Helper functions to search analysis data. Takes starting index to start query. Returns which data index the result was found at. Returns INDEX_NONE if not found. */
	ENGINE_API uint32 GetInterpolatedCookedFFTDataForTimeInternal(float InTime, uint32 StartingIndex, TArray<FSoundWaveSpectralData>& OutData, bool bLoop);
	ENGINE_API uint32 GetInterpolatedCookedEnvelopeDataForTimeInternal(float InTime, uint32 StartingIndex, float& OutAmplitude, bool bLoop);

	/** What state the precache decompressor is in. */
	FThreadSafeCounter PrecacheState;

public:
	/** the number of sounds currently playing this sound wave. */
	FThreadSafeCounter NumSourcesPlaying;

private:

	TArray<FSoundWaveClientPtr> SourcesPlaying;

	// This is the sample rate retrieved from platform settings.
	mutable float CachedSampleRateOverride;

	// We cache a soundwave's loading behavior on the first call to USoundWave::GetLoadingBehaviorForWave(true);
	// Caches resolved loading behavior from the SoundClass graph. Must be called on the game thread.
	ENGINE_API void CacheInheritedLoadingBehavior() const;

	// Called when we change any properties about the underlying audio asset
#if WITH_EDITOR
	ENGINE_API void UpdateAsset(bool bMarkDirty = true);
#endif

public:

	/** Set to true for programmatically generated audio. */
	uint8 bProcedural : 1;

	/** Set to true if fade is required when sound is abruptly stopped. */
	uint8 bRequiresStopFade:1;

	/** Set to true of this is a bus sound source. This will result in the sound wave not generating audio for itself, but generate audio through instances. Used only in audio mixer. */
	uint8 bIsSourceBus : 1;

	/** Set to true for procedural waves that can be processed asynchronously. */
	uint8 bCanProcessAsync : 1;

	/** Whether to free the resource data after it has been uploaded to the hardware */
	uint8 bDynamicResource : 1;

	/** If set to true if this sound is considered to contain mature/adult content. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Info", AssetRegistrySearchable)
	uint8 bMature : 1;

	/** If set to true will disable automatic generation of line breaks - use if the subtitles have been split manually. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Info|Subtitles")
	uint8 bManualWordWrap : 1;

	/** If set to true the subtitles display as a sequence of single lines as opposed to multiline. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Info|Subtitles")
	uint8 bSingleLine : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 bVirtualizeWhenSilent_DEPRECATED : 1;
#endif // WITH_EDITORONLY_DATA

	/** Whether or not this source is ambisonics file format. If set, sound always uses the
	  * 'Master Ambisonics Submix' as set in the 'Audio' category of Project Settings'
	  * and ignores submix if provided locally or in the referenced SoundClass. */
	UPROPERTY(EditAnywhere, Category = "Info")
	uint8 bIsAmbisonics : 1;

	/** Whether this SoundWave was decompressed from OGG. */
	uint8 bDecompressedFromOgg : 1;

#if WITH_EDITOR
	/** The current revision of our compressed audio data. Used to tell when a chunk in the cache is stale. */
	std::atomic<int32> CurrentChunkRevision = 0;
#endif

private:

	// This is set to false on initialization, then set to true on non-editor platforms when we cache appropriate sample rate.
	mutable uint8 bCachedSampleRateFromPlatformSettings : 1;

	// This is set when SetSampleRate is called to invalidate our cached sample rate while not re-parsing project settings.
	uint8 bSampleRateManuallyReset : 1;

#if WITH_EDITOR
	// Whether or not the thumbnail supports generation
	uint8 bNeedsThumbnailGeneration : 1;

	// Whether this asset is loaded from cooked data.
	uint8 bLoadedFromCookedData : 1;
#endif // !WITH_EDITOR

	enum class ESoundWaveResourceState : uint8
	{
		NeedsFree,
		Freeing,
		Freed
	};

	ESoundWaveResourceState ResourceState : 2;

public:
	// Loading behavior members are lazily initialized in const getters
	/** Specifies how and when compressed audio data is loaded for asset if stream caching is enabled. */
	UPROPERTY(EditAnywhere, Category = "Memory", meta = (DisplayName = "Loading Behavior Override"))
	mutable ESoundWaveLoadingBehavior LoadingBehavior;
	
private:
	/** the number of sounds currently playing this sound wave. */
	mutable UE::FMutex SourcesPlayingCs;

public:
#if WITH_EDITORONLY_DATA
   	/** How much audio to add to First Audio Chunk (in seconds) */
	UPROPERTY(EditAnywhere, Category = "Memory", meta = (UIMin = 0, UIMax = 10, EditCondition = "LoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad || LoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad", EditConditionHides), DisplayName="Size of First Audio Chunk (seconds)")
   	FPerPlatformFloat SizeOfFirstAudioChunkInSeconds = 0.0f;
#endif //WITH_EDITOR_ONLY_DATA

	/** A localized version of the text that is actually spoken phonetically in the audio. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Subtitles instead."))
	FString SpokenText_DEPRECATED;

	/** The priority of the subtitle. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Info|Subtitles")
	float SubtitlePriority;

	/** Linear volume multiplier for sound playback, default is 1.0 */
	UPROPERTY(Category = "Sound", meta = (ClampMin = "0.0"), EditAnywhere)
	float Volume;

	/** Playback rate multiplier for sound, range is 0.125 - 4.0, default is 1.0  */
	UPROPERTY(Category = "Sound", meta = (ClampMin = "0.125", ClampMax = "4.0"), EditAnywhere)
	float Pitch;

	/** Number of channels of multichannel data; 1 or 2 for regular mono and stereo files */
	UPROPERTY(Category = "Info|Format", AssetRegistrySearchable, VisibleAnywhere)
	int32 NumChannels;

#if WITH_EDITORONLY_DATA
	/** Offsets into the bulk data for the source wav data */
	UPROPERTY()
	TArray<int32> ChannelOffsets;

	/** Sizes of the bulk data for the source wav data */
	UPROPERTY()
	TArray<int32> ChannelSizes;

	/** Gated Loudness Units Full Scale of the SoundWave. Can be modified by transformations. */
	UPROPERTY(Category = "Info|Analysis", AssetRegistrySearchable, VisibleAnywhere)
	float LUFS;

	/** Highest Sample Peak in db of the SoundWave. Can be modified by transformations. */
	UPROPERTY(Category = "Info|Analysis", AssetRegistrySearchable, VisibleAnywhere)
	float SamplePeakDB;

#endif // WITH_EDITORONLY_DATA

protected:

	/** Cooked sample rate of the asset. Can be modified by sample rate override. */
	UPROPERTY(Category = "Info|Format", AssetRegistrySearchable, VisibleAnywhere)
	int32 SampleRate;

#if WITH_EDITORONLY_DATA
	/** Sample rate of the imported sound wave. */
	UPROPERTY(Category = "Info|Format", AssetRegistrySearchable, VisibleAnywhere)
	int32 ImportedSampleRate;

	/** Cue point data parsed from the .wav file. Contains "Loop Regions" as cue points as well! */
	UPROPERTY(Category = "Info|Markers", VisibleAnywhere, BlueprintGetter = GetCuePoints)
	TArray<FSoundWaveCuePoint> CuePoints;

	/** Dictates whether to use the CuePoints and Loop Regions from the .wav file or from the waveform editor 
	 *	Marker transformations during playback.
	 */
	UPROPERTY(Category = "Info|Markers", EditAnywhere)
	ESoundWaveCuePointOrigin CuePointOrigin;
#endif

	ENGINE_API virtual void SerializeCuePoints(FArchive& Ar, const bool bIsLoadingFromCookedArchive);

public:

	// Returns the compression quality of the sound asset.
	ENGINE_API int32 GetCompressionQuality() const;

	/** Resource index to cross reference with buffers */
	int32 ResourceID;

	ENGINE_API int32 GetResourceSize() const;

	/** Cache the total used memory recorded for this SoundWave to keep INC/DEC consistent */
	int32 TrackedMemoryUsage;

	/**
	 * Subtitle cues. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Info|Subtitles")
	TArray<struct FSubtitleCue> Subtitles;

#if WITH_EDITORONLY_DATA
	/** Provides contextual information for the sound to the translator. */
	UPROPERTY(EditAnywhere, Category = "Info|Subtitles")
	FString Comment;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	
	/** Information about the time-code from import, if available.  */
	UPROPERTY(VisibleAnywhere, Category = "Info", Meta=(DisplayName="Timecode"))
	FSoundWaveTimecodeInfo TimecodeInfo;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;
	
	UPROPERTY()
	FString SourceFileTimestamp_DEPRECATED;

	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

#endif // WITH_EDITORONLY_DATA

protected:

	/** Curves associated with this sound wave */
	UPROPERTY(EditAnywhere, Category = "Info|Curves")
	TObjectPtr<class UCurveTable> Curves;

	/** Hold a reference to our internal curve so we can switch back to it if we want to */
	UPROPERTY()
	TObjectPtr<class UCurveTable> InternalCurves;

#if WITH_EDITORONLY_DATA
protected:
	/** If enabled, this wave may be streamed from the cloud using the Opus format. Loading behavior must NOT be `Force Inline`. Requires a suitable support plugin to be installed. */
	UPROPERTY(EditAnywhere, Category = "Info|Format", Meta=(DisplayName="Enable Cloud Streaming", DisplayAfter="SoundAssetCompressionType", EditCondition = "LoadingBehavior != ESoundWaveLoadingBehavior::ForceInline"), AssetRegistrySearchable)
	uint8 bEnableCloudStreaming : 1;
	/** Platform specific. */
	UPROPERTY(EditAnywhere, config, Category="Info|Format", Meta=(DisplayName="Platform Specific Settings", ToolTip="Optionally disables cloud streaming per platform"))
	TMap<FGuid, FSoundWaveCloudStreamingPlatformSettings> PlatformSettings;
public:
	static FName GetCloudStreamingEnabledPropertyName() { return GET_MEMBER_NAME_CHECKED(USoundWave, bEnableCloudStreaming); }
	ENGINE_API void SetCloudStreamingEnabled(bool bEnabled);
	ENGINE_API bool IsCloudStreamingEnabled() const;
	ENGINE_API void TriggerRecookForCloudStreaming();
	static FName GetCloudStreamingPlatformSettingsPropertyName() { return GET_MEMBER_NAME_CHECKED(USoundWave, PlatformSettings); }
	TMap<FGuid, FSoundWaveCloudStreamingPlatformSettings>& GetCloudStreamingPlatformSettings() { return PlatformSettings; }
	const TMap<FGuid, FSoundWaveCloudStreamingPlatformSettings>& GetCloudStreamingPlatformSettings() const { return PlatformSettings; }
#endif // WITH_EDITORONLY_DATA

public:
	/**
	* helper function for getting the cached name of the current platform.
	*/
	static ENGINE_API ITargetPlatform* GetRunningPlatform();

	/** Used to ensure that we recook USoundWaves if the default loading behavior has changed */
	static ENGINE_API const TCHAR* GetDefaultLoadingBehaviorCVarName();

	static ENGINE_API ESoundWaveLoadingBehavior GetDefaultLoadingBehavior();

	/** Async worker that decompresses the audio data on a different thread */
	typedef FAsyncTask< class FAsyncAudioDecompressWorker > FAsyncAudioDecompress;	// Forward declare typedef
	FAsyncAudioDecompress* AudioDecompressor;

	/** Pointer to 16 bit PCM data - used to avoid synchronous operation to obtain first block of the realtime decompressed buffer */
	uint8* CachedRealtimeFirstBuffer;

	/** The number of frames which have been precached for this sound wave. */
	int32 NumPrecacheFrames;

	/** Size of RawPCMData, or what RawPCMData would be if the sound was fully decompressed */
	int32 RawPCMDataSize;

	/** Pointer to 16 bit PCM data - used to decompress data to and preview sounds */
	uint8* RawPCMData;

	/** Memory containing the data copied from the compressed bulk data */

public:
	ENGINE_API const uint8* GetResourceData() const;

#if WITH_EDITORONLY_DATA
	/** 
	* Holds the uncompressed wav data that was imported. This is guaranteed to be 16 bit, and is
	* mono or stereo - stereo not allowed for multichannel data. For multichannel data, there are
	* distinct RIFF files concatenated in the RawData - one for each channel. These can be accessed with
	* ChannelOffsets and ChannelSizes (see GetImportedSoundWaveData for example).
	* 
	* This structure is a pass-through for editor bulk data. It does an in-place conversion on the audio
	* bits to allow the audio to compress significantly better when the underlying bulk data compression hits it.
	* 
	* If you need access to the underlying audio data, use GetImportedSoundWaveData and avoid touching this.
	*/
	struct FEditorAudioBulkData
	{
		UE::Serialization::FEditorBulkData RawData;

		// The container soundwave for this raw data. This must be non-null for any non-metadata instances.
		// We need this in order to parse the multichannel layout of the raw data - and we also use this 
		// in place of the BulkData Owner parameter in many places because many call sites pass null incorrectly,
		// resulting in the bulk data not being correlated with our asset.
		USoundWave* SoundWave;

		enum class EPayloadFlags : uint8
		{
			None,
			ReturnCopyWithTransformationData = 0x01
		};

		FEditorAudioBulkData()
			: RawData()
			, SoundWave()
		{
		}
			
		FEditorAudioBulkData(USoundWave* Owner)
		{
			SoundWave = Owner;
		}

		ENGINE_API void CreateFromBulkData(FBulkData& InBulkData, const FGuid& InGuid, UObject* Owner);
		ENGINE_API void Serialize(FArchive& Ar, UObject* Owner, bool bAllowRegister=true);
		ENGINE_API TFuture<FSharedBuffer> GetPayload(const EPayloadFlags Flags = EPayloadFlags::None) const;
		ENGINE_API bool HasPayloadData() const;
		ENGINE_API void UpdatePayload(FSharedBuffer InPayload, UObject* Owner = nullptr);

		//
		// Deprecated unused API forwarding for potential backwards compatability issues.
		// As the raw data needs to be converted before use or storage, always access it via the above functions.
		//
#pragma region Deprecated Pass Thru
		UE_DEPRECATED(5.4, "CreateLegacyUniqueIdentifier is provided just for API backwards compatibility.")
		void CreateLegacyUniqueIdentifier(UObject* Owner)
		{
			RawData.CreateLegacyUniqueIdentifier(Owner);
		}
		UE_DEPRECATED(5.4, "Reset is provided just for API backwards compatibility.")
		void Reset() 
		{ 
			RawData.Reset(); 
		}
		UE_DEPRECATED(5.4, "UnloadData is provided just for API backwards compatibility.")
		void UnloadData()
		{
			RawData.UnloadData();
		}
		UE_DEPRECATED(5.4, "DetachFromDisk is provided just for API backwards compatibility.")
		void DetachFromDisk(FArchive* Ar, bool bEnsurePayloadIsLoaded)
		{
			RawData.DetachFromDisk(Ar, bEnsurePayloadIsLoaded);
		}
		UE_DEPRECATED(5.4, "GetIdentifier is provided just for API backwards compatibility.")
		FGuid GetIdentifier() const
		{
			return RawData.GetIdentifier();
		}
		UE_DEPRECATED(5.4, "GetPayloadId is provided just for API backwards compatibility.")
		const FIoHash& GetPayloadId() const
		{
			return RawData.GetPayloadId();
		}
		int64 GetPayloadSize() const
		{
			return RawData.GetPayloadSize();
		}
		UE_DEPRECATED(5.4, "DoesPayloadNeedLoading is provided just for API backwards compatibility.")
		bool DoesPayloadNeedLoading() const
		{
			return RawData.DoesPayloadNeedLoading();
		}
		UE_DEPRECATED(5.4, "GetCompressedPayload is provided just for API backwards compatibility.")
		TFuture<FCompressedBuffer> GetCompressedPayload() const
		{
			return RawData.GetCompressedPayload();
		}
		UE_DEPRECATED(5.4, "UpdatePayload is provided just for API backwards compatibility.")
		void UpdatePayload(FCompressedBuffer InPayload, UObject* Owner = nullptr)
		{
			RawData.UpdatePayload(InPayload, Owner);
		}
		UE_DEPRECATED(5.4, "UpdatePayload is provided just for API backwards compatibility.")
		void UpdatePayload(UE::Serialization::FEditorBulkData::FSharedBufferWithID InPayload, UObject* Owner = nullptr)
		{
			RawData.UpdatePayload(MoveTemp(InPayload), Owner);
		}
		UE_DEPRECATED(5.4, "SetCompressionOptions is provided just for API backwards compatibility.")
		void SetCompressionOptions(UE::Serialization::ECompressionOptions Option)
		{
			RawData.SetCompressionOptions(Option);
		}
		UE_DEPRECATED(5.4, "SetCompressionOptions is provided just for API backwards compatibility.")
		void SetCompressionOptions(ECompressedBufferCompressor Compressor, ECompressedBufferCompressionLevel CompressionLevel)
		{
			RawData.SetCompressionOptions(Compressor, CompressionLevel);
		}
		UE_DEPRECATED(5.4, "GetBulkDataVersions is provided just for API backwards compatibility.")
		void GetBulkDataVersions(FArchive& InlineArchive, FPackageFileVersion& OutUEVersion, int32& OutLicenseeUEVersion, FCustomVersionContainer& OutCustomVersions) const
		{
			RawData.GetBulkDataVersions(InlineArchive, OutUEVersion, OutLicenseeUEVersion, OutCustomVersions);
		}
		UE_DEPRECATED(5.4, "TearOff is provided just for API backwards compatibility.")
		void TearOff()
		{
			RawData.TearOff();
		}
		UE_DEPRECATED(5.4, "CopyTornOff is provided just for API backwards compatibility.")
		UE::Serialization::FEditorBulkData CopyTornOff() const
		{
			return RawData.CopyTornOff();
		}
		UE_DEPRECATED(5.4, "SerializeForRegistry is provided just for API backwards compatibility.")
		void SerializeForRegistry(FArchive& Ar)
		{
			RawData.SerializeForRegistry(Ar);
		}
		UE_DEPRECATED(5.4, "CanSaveForRegistry is provided just for API backwards compatibility.")
		bool CanSaveForRegistry() const
		{
			return RawData.CanSaveForRegistry();
		}
		UE_DEPRECATED(5.4, "HasPlaceholderPayloadId is provided just for API backwards compatibility.")
		bool HasPlaceholderPayloadId() const
		{
			return RawData.HasPlaceholderPayloadId();
		}
		UE_DEPRECATED(5.4, "IsMemoryOnlyPayload is provided just for API backwards compatibility.")
		bool IsMemoryOnlyPayload() const
		{
			return RawData.IsMemoryOnlyPayload();
		}
		UE_DEPRECATED(5.4, "UpdatePayloadId is provided just for API backwards compatibility.")
		void UpdatePayloadId()
		{
			return RawData.UpdatePayloadId();
		}
		UE_DEPRECATED(5.4, "LocationMatches is provided just for API backwards compatibility.")
		bool LocationMatches(const UE::Serialization::FEditorBulkData& Other) const
		{
			return RawData.LocationMatches(Other);
		}
		UE_DEPRECATED(5.4, "UpdateRegistrationOwner is provided just for API backwards compatibility.")
		void UpdateRegistrationOwner(UObject* Owner)
		{
			RawData.UpdateRegistrationOwner(Owner);
		}
#pragma endregion
	} RawData;

	/** 
	 * Waveform edits to be applied to this SoundWave on cook (editing transformations will trigger a cook)
	 * Transformations is a TArray because changing it to TSet will delete user data
	 */
	UPROPERTY(EditAnywhere, Instanced, Category = "Waveform Processing", meta = (NoElementDuplicate))
	TArray<TObjectPtr<class UWaveformTransformationBase>> Transformations;
#endif

	/** GUID used to uniquely identify this node so it can be found in the DDC */
	FGuid CompressedDataGuid;

#if WITH_EDITORONLY_DATA
	TMap<FName, uint32> AsyncLoadingDataFormats;

	/** FByteBulkData doesn't currently support read-only access from multiple threads, so we limit access to RawData with a critical section on cook. */
	mutable FCriticalSection RawDataCriticalSection;

#endif // WITH_EDITORONLY_DATA
	/** cooked streaming platform data for this sound */
	TSortedMap<FString, FStreamedAudioPlatformData*> CookedPlatformData;

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void PostLoad() override;

	// Returns true if the zeroth chunk is loaded, or attempts to load it if not already loaded,
	// returning true if the load was successful. Can return false if either an error was encountered
	// in attempting to load the chunk or if stream caching is not enabled for the given sound.
	ENGINE_API bool LoadZerothChunk();

	// Returns the amount of chunks this soundwave contains if it's streaming,
	// or zero if it is not a streaming source.
	ENGINE_API uint32 GetNumChunks() const;

	ENGINE_API uint32 GetSizeOfChunk(uint32 ChunkIndex) const;

	// Calculates audio streaming cache usage for this sound wave. 
	// @param OutTotalBytesOfAudioData - The accumulated number of bytes of all audio data of all chunks for this sound wave.
	// @param OutMaxChunkBytesOfAudioData - The maximum number of bytes of audio data for all audio chunks for this sound wave. 
	ENGINE_API void GetChunkSizeStats(uint32& OutTotalBytesOfAudioData, uint32& OutMaxChunkBytesOfAudioData) const;

	ENGINE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	/** IInterface_AsyncCompilation begin*/
	ENGINE_API virtual bool IsCompiling() const override;
	/** IInterface_AsyncCompilation end*/

	ENGINE_API bool IsAsyncWorkComplete() const;

	ENGINE_API void PostImport();
	
	ENGINE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;

private:
	friend class FSoundWaveCompilingManager;
	/**  Utility function used internally to change task priority while maintaining thread-safety. */
	ENGINE_API bool RescheduleAsyncTask(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority);
	/**  Utility function used internally to wait or poll a task while maintaining thread-safety. */
	ENGINE_API bool WaitAsyncTaskWithTimeout(float InTimeoutInSeconds);

public:
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	ENGINE_API TArray<Audio::FTransformationPtr> CreateTransformations() const;
#endif // WITH_EDITORONLY_DATA
	
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual FName GetExporterName() override;
	ENGINE_API virtual FString GetDesc() override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~ End UObject Interface.

	//~ Begin USoundBase Interface.
	ENGINE_API virtual bool IsPlayable() const override;
	ENGINE_API virtual void Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances) override;
	ENGINE_API virtual float GetDuration() const override;
	ENGINE_API virtual float GetSubtitlePriority() const override;
	ENGINE_API virtual bool SupportsSubtitles() const override;
	ENGINE_API virtual bool GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves) override;
	ENGINE_API virtual bool HasCookedFFTData() const override;
	ENGINE_API virtual bool HasCookedAmplitudeEnvelopeData() const override;
	virtual bool IsProcedurallyGenerated() const override { return bProcedural; }
	//~ End USoundBase Interface.

	UE_DEPRECATED(5.8, "Mutable access to proxy is deprecated. Use GetSoundWaveProxy()")
	ENGINE_API FSoundWaveProxyPtr CreateSoundWaveProxy();

	ENGINE_API TSharedRef<const FSoundWaveProxy> GetSoundWaveProxy();

	//~Begin IAudioProxyDataFactory Interface.
	ENGINE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;
	//~ End IAudioProxyDataFactory Interface.

	// Called  when the procedural sound wave begins on the render thread. Only used in the audio mixer and when bProcedural is true.
	virtual void OnBeginGenerate() {}

	// Called when the procedural sound wave is done generating on the render thread. Only used in the audio mixer and when bProcedural is true..
	virtual void OnEndGenerate() {};
	virtual void OnEndGenerate(ISoundGeneratorPtr Generator) {};

	ENGINE_API void AddPlayingSource(const FSoundWaveClientPtr& Source);
	ENGINE_API void RemovePlayingSource(const FSoundWaveClientPtr& Source);

	bool IsGeneratingAudio() const
	{
		bool bIsGeneratingAudio = false;
		UE::TUniqueLock Lock(SourcesPlayingCs);
		bIsGeneratingAudio = SourcesPlaying.Num() > 0;

		return bIsGeneratingAudio;
	}

	void SetImportedSampleRate(uint32 InImportedSampleRate)
	{
#if WITH_EDITORONLY_DATA
		ImportedSampleRate = InImportedSampleRate;
#endif
	}
	
	uint32 GetImportedSampleRate() const
	{
#if WITH_EDITORONLY_DATA
		return ImportedSampleRate;
#else
		return 0;
#endif
	}

#if WITH_EDITORONLY_DATA
	ENGINE_API void SetTimecodeInfo(const FSoundWaveTimecodeInfo& InTimecode);
	ENGINE_API TOptional<FSoundWaveTimecodeInfo> GetTimecodeInfo() const;

	// Returns true if the TimecodeInfo needed to be fixed
	ENGINE_API bool FixTimeCodeInfo();
#endif //WITH_EDITORONLY_DATA	

public:
	/**
	* Overwrite sample rate. Used for procedural soundwaves, as well as sound waves that are resampled on compress/decompress.
	*/
	ENGINE_API void SetSampleRate(uint32 InSampleRate, const bool bFromDecoders=false);

	/**
	* Overwrite number of frames. Used for procedural soundwaves, as well as sound waves that are resampled on compress/decompress.
	*/
	ENGINE_API void SetNumFrames(uint32 InNumFrames);

	/**
	 * Return the number of frames. (can change after decoders read the headers).
	 */
	ENGINE_API uint32 GetNumFrames() const;

	/**
	 *	@param		Format		Format to check
	 *
	 *	@return		Sum of the size of waves referenced by this cue for the given platform.
	 */
	ENGINE_API virtual int32 GetResourceSizeForFormat(FName Format);

	/**
	 * Frees up all the resources allocated in this class.
	 * @param bStopSoundsUsingThisResource if false, will leave any playing audio alive.
	 *        This occurs when we force a re-cook of audio while starting to play a sound.
	 */
	ENGINE_API void FreeResources(bool bStopSoundsUsingThisResource = true);

	/** Will clean up the decompressor task if the task has finished or force it finish. Returns true if the decompressor is cleaned up. */
	ENGINE_API bool CleanupDecompressor(bool bForceCleanup = false);

	/**
	 * Copy the compressed audio data from the bulk data
	 */
	ENGINE_API virtual void InitAudioResource(FByteBulkData& CompressedData);

	/**
	 * Copy the compressed audio data from derived data cache
	 *
	 * @param Format to get the compressed audio in
	 * @return true if the resource has been successfully initialized or it was already initialized.
	 */
	ENGINE_API virtual bool InitAudioResource(FName Format);

	/**
	 * Remove the compressed audio data associated with the passed in wave
	 */
	ENGINE_API void RemoveAudioResource();

	/**
	 * Handle any special requirements when the sound starts (e.g. subtitles)
	 */
	ENGINE_API FWaveInstance& HandleStart(FActiveSound& ActiveSound, const UPTRINT WaveInstanceHash) const;

	/**
	 * This is only used for DTYPE_Procedural audio. It's recommended to use USynthComponent base class
	 * for procedurally generated sound vs overriding this function. If a new component is not feasible,
	 * consider using USoundWaveProcedural base class vs USoundWave base class since as it implements
	 * GeneratePCMData for you and you only need to return PCM data.
	 */
	virtual int32 GeneratePCMData(uint8* PCMData, const int32 SamplesNeeded) { ensure(false); return 0; }

	/**
	* Return the format of the generated PCM data type. Used in audio mixer to allow generating float buffers and avoid unnecessary format conversions.
	* This feature is only supported in audio mixer. If your procedural sound wave needs to be used in both audio mixer and old audio engine,
	* it's best to generate int16 data as old audio engine only supports int16 formats. Or check at runtime if the audio mixer is enabled.
	* Audio mixer will convert from int16 to float internally.
	*/
	virtual Audio::EAudioMixerStreamDataFormat::Type GetGeneratedPCMDataFormat() const { return Audio::EAudioMixerStreamDataFormat::Int16; }

	/**
	 * Gets the compressed data size from derived data cache for the specified format
	 *
	 * @param Format	format of compressed data
	 * @param CompressionOverrides Optional argument for compression overrides.
	 * @return			compressed data size, or zero if it could not be obtained
	 */
	SIZE_T GetCompressedDataSize(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides = GetPlatformCompressionOverridesForCurrentPlatform())
	{
		FByteBulkData* Data = GetCompressedData(Format, CompressionOverrides);
		return Data ? Data->GetBulkDataSize() : 0;
	}

	ENGINE_API virtual bool HasCompressedData(FName Format, ITargetPlatform* TargetPlatform = GetRunningPlatform()) const;

#if WITH_EDITOR
	/** Utility which returns imported PCM data and the parsed header for the file. Returns true if there was data, false if there wasn't. */
	ENGINE_API bool GetImportedSoundWaveData(TArray<uint8>& OutRawPCMData, uint32& OutSampleRate, uint16& OutNumChannels) const;

	/** Utility which returns imported PCM data and the parsed header for the file. Returns true if there was data, false if there wasn't. */
	ENGINE_API bool GetImportedSoundWaveData(TArray<uint8>& OutRawPCMData, uint32& OutSampleRate, TArray<EAudioSpeakers>& OutChannelOrder) const;

	/**
	 * This function can be called before playing or using a SoundWave to check if any cook settings have been modified since this SoundWave was last cooked.
	 */
	ENGINE_API void InvalidateSoundWaveIfNeccessary();

#endif //WITH_EDITOR


	static ENGINE_API FName GetPlatformSpecificFormat(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides);

private:
#if WITH_EDITOR
	// Removes any in-progress async loading data formats. 
	ENGINE_API void FlushAsyncLoadingDataFormats();

	// Waits for audio rendering commands to execute
	ENGINE_API void FlushAudioRenderingCommands() const;

	ENGINE_API void BakeFFTAnalysis();
	ENGINE_API void BakeEnvelopeAnalysis();

	FWaveTransformUObjectConfiguration TransformationChainConfig;
#endif //WITH_EDITOR

public:

#if WITH_EDITOR
	ENGINE_API void LogBakedData();

	/** Returns if an async task for a certain platform has finished. */
	ENGINE_API bool IsCompressedDataReady(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides) const;

	ENGINE_API bool IsLoadedFromCookedData() const;
#endif //WITH_EDITOR

	ENGINE_API virtual void BeginGetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform);

	/**
	 * Gets the compressed data from derived data cache for the specified platform
	 * Warning, the returned pointer isn't valid after we add new formats
	 *
	 * @param Format	format of compressed data
	 * @param CompressionOverrides optional platform compression overrides
	 * @return	compressed data, if it could be obtained
	 */
	ENGINE_API virtual FByteBulkData* GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides = GetPlatformCompressionOverridesForCurrentPlatform(),
		const ITargetPlatform* InTargetPlatform = GetRunningPlatform());

	/**
	 * Change the guid and flush all compressed data
	 * @param bFreeResources if true, will delete any precached compressed data as well.
	 */
	ENGINE_API void InvalidateCompressedData(bool bFreeResources = false, bool bRebuildStreamingChunks = true);

	/** Returns curves associated with this sound wave */
	virtual class UCurveTable* GetCurveData() const override { return Curves; }

	// This function returns true if there are streamable chunks in this asset.
	ENGINE_API bool HasStreamingChunks() const;

#if WITH_EDITOR
	/** These functions are required for support for some custom details/editor functionality.*/

	/** Returns internal curves associated with this sound wave */
	class UCurveTable* GetInternalCurveData() const { return InternalCurves; }

	/** Returns whether this sound wave has internal curves. */
	bool HasInternalCurves() const { return InternalCurves != nullptr; }

	/** Sets the curve data for this sound wave. */
	void SetCurveData(UCurveTable* InCurves) { Curves = InCurves; }

	/** Sets the internal curve data for this sound wave. */
	void SetInternalCurveData(UCurveTable* InCurves) { InternalCurves = InCurves; }

	/** Gets the member name for the Curves property of the USoundWave object. */
	static FName GetCurvePropertyName() { return GET_MEMBER_NAME_CHECKED(USoundWave, Curves); }
#endif // WITH_EDITOR

	/** Checks whether sound has been categorized as streaming. */
public:
	ENGINE_API bool IsStreaming(const TCHAR* PlatformName = nullptr) const;
	ENGINE_API bool IsStreaming(const FPlatformAudioCookOverrides& Overrides) const;

	/**
	 * Queries if Seeking is supported by the Decoder for this Wave.
	 * @return true if Seeking is supported, false otherwise.
	 */
	ENGINE_API virtual bool IsSeekable() const;

	/**
	 * Checks whether we should use the load on demand cache.
	 */
	ENGINE_API bool ShouldUseStreamCaching() const;

	/**
	 * This returns the initial chunk of compressed data for streaming data sources.
	 */
	ENGINE_API TArrayView<const uint8> GetZerothChunk(bool bForImmediatePlayback = false);

	/**
	 * Attempts to update the cached platform data after any changes that might affect it
	 */
	ENGINE_API void UpdatePlatformData();

	ENGINE_API void CleanupCachedRunningPlatformData();

	/**
	 * Serializes cooked platform data.
	 */
	ENGINE_API virtual void SerializeCookedPlatformData(class FArchive& Ar);

	/*
	* Returns a sample rate if there is a specific sample rate override for this platform, -1.0 otherwise.
	*/
	ENGINE_API float GetSampleRateForCurrentPlatform() const;

	/**
	* Return the platform compression overrides set for the current platform.
	*/
	static ENGINE_API const FPlatformAudioCookOverrides* GetPlatformCompressionOverridesForCurrentPlatform();

	/*
	* Returns a sample rate if there is a specific sample rate override for this platform, -1.0 otherwise.
	*/
	ENGINE_API float GetSampleRateForCompressionOverrides(const FPlatformAudioCookOverrides* CompressionOverrides) const;

	ENGINE_API void SetError(const TCHAR* InErrorMsg=nullptr);
	ENGINE_API void ResetError();
	ENGINE_API bool HasError() const;

#if WITH_EDITORONLY_DATA

#if WITH_EDITOR
	
	/*
	* Used to determine the loading behavior that's applied by the owner of this wave. Will determine
	* the most appropriate loading behavior if there are multiple owners. If there is no owner "Unintialized"
	* will be returned. The results are cached in TMap below.
	*/
	ISoundWaveLoadingBehaviorUtil::FClassData GetOwnerLoadingBehavior(const ITargetPlatform* InTargetPlatform) const;
	
	mutable TMap<FName, ISoundWaveLoadingBehaviorUtil::FClassData> OwnerLoadingBehaviorCache;
	mutable FCriticalSection OwnerLoadingBehaviorCacheCS;

	/*
	* Returns this SoundWave's CuePoints array with the frame values scaled by
	* InSampleRate / ImportedSampleRate to account for resampling of the sound wave source data.
	* If no resampling is necessary, returns the CuePoints as-is.
	*
	* @param InSampleRate	The sample rate the SoundWave 
	* @return CuePoints array scaled if resampling occurred
	*/
	ENGINE_API TArray<FSoundWaveCuePoint> GetCuePointsScaledForSampleRate(const float InSampleRate) const;

	/*
	* Modifies the InOutCuePoints array with the frame values scaled by
	* InSampleRate / ImportedSampleRate to account for resampling of the sound wave source data.
	* Does not modify InOutCuePoints if no resampling is necessary
	*
	* @param InSampleRate	The sample rate the SoundWave
	* @param InOutCuePoints	The CuePoints array to re-scale
	*/
	ENGINE_API void ScaleCuePointsForSampleRate(const float InSampleRate, TArray<FSoundWaveCuePoint>& InOutCuePoints) const;

	/*
	* Returns a sample rate if there is a specific sample rate override for this platform, -1.0 otherwise.
	*/
	ENGINE_API float GetSampleRateForTargetPlatform(const ITargetPlatform* TargetPlatform);
	
	ENGINE_API float GetSizeOfFirstAudioChunkInSeconds(const ITargetPlatform* InTargetPlatform) const;

	/**
	 * Begins caching platform data in the background for the platform requested
	 */
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;

	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;

	/**
	 * Clear all the cached cooked platform data which we have accumulated with BeginCacheForCookedPlatformData calls
	 * The data can still be cached again using BeginCacheForCookedPlatformData again
	 */
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;

	ENGINE_API virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override;

	ENGINE_API virtual void WillNeverCacheCookedPlatformDataAgain() override;

	ENGINE_API virtual bool GetRedrawThumbnail() const;
	ENGINE_API virtual void SetRedrawThumbnail(bool bInRedraw);
	ENGINE_API virtual bool CanVisualizeAsset() const;

#endif // WITH_EDITOR

	/**
	 * Caches platform data for the sound.
	 */
	ENGINE_API virtual void CachePlatformData(bool bAsyncCache = false);

	/**
	 * Begins caching platform data in the background.
	 */
	ENGINE_API virtual void BeginCachePlatformData();

	/**
	 * Blocks on async cache tasks and prepares platform data for use.
	 */
	ENGINE_API virtual void FinishCachePlatformData();

	/**
	 * Forces platform data to be rebuilt.
	 */
	ENGINE_API void ForceRebuildPlatformData();
#endif // WITH_EDITORONLY_DATA

	/**
	 * Get Chunk data for a specified chunk index.
	 * @param ChunkIndex	The Chunk index to cache.
	 * @param OutChunkData	Address of pointer that will store data.
	 */
	ENGINE_API bool GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded = false);

	void SetPrecacheState(ESoundWavePrecacheState InState)
	{
		PrecacheState.Set((int32)InState);
	}

	ESoundWavePrecacheState GetPrecacheState() const
	{
		return (ESoundWavePrecacheState)PrecacheState.GetValue();
	}

	// Call this if you intend to supply an overriden decoder at playback time.
	UE_EXPERIMENTAL(5.8, "May be altered or removed")
	ENGINE_API void OverrideWaveData(int32 InNumChannels, int32 InNumFrames, int32 InSampleRate);

private:
	friend class FSoundWaveProxy;
	friend class USoundFactory;
};


class FSoundWaveData : public TSharedFromThis<FSoundWaveData>
{
public:
	UE_NONCOPYABLE(FSoundWaveData);

	FSoundWaveData()
		: bIsLooping(0)
		, bIsTemplate(0)
		, bIsStreaming(0)
		, bIsSeekable(0)
		, bShouldUseStreamCaching(0)
		, bLoadingBehaviorOverridden(0)
		, bHasError(0)
		, bHasSubtitleAssetUserData(0)
#if WITH_EDITOR
		, bLoadedFromCookedData(0)
#endif //WITH_EDITOR
	{
	}

	FSoundWaveData(const USoundWave& InWave);

	// dtor
	ENGINE_API ~FSoundWaveData();

	// non-const arg for internal call to FAudioDevice::GetRuntimeFormat
	ENGINE_API void InitializeDataFromSoundWave(const USoundWave& InWave);

	ENGINE_API void OverrideRuntimeFormat(const FName& InRuntimeFormat);

	const FGuid& GetGUID() const { return WaveGuid; }
	const FName& GetFName() const { return NameCached; }
	const FName& GetPackageName() const { return PackageNameCached; }
	const FName& GetRuntimeFormat() const { return RuntimeFormat; }
	const FObjectKey& GetFObjectKey() const { return SoundWaveKeyCached; };

	float GetSampleRate() const { return SampleRate; }
	uint32 GetNumChannels() const { return NumChannels; }
	const TArray<FSoundWaveCuePoint>& GetCuePoints() const { return CuePoints; }
	const TArray<FSoundWaveCuePoint>& GetLoopRegions() const { return LoopRegions; }
	void SetAllCuePoints(const TArray<FSoundWaveCuePoint>& InCuePoints);

	ENGINE_API uint32 GetNumChunks() const;
	ENGINE_API uint32 GetSizeOfChunk(uint32 ChunkIndex) const;
	int32 GetNumFrames() const { return NumFrames; }
	float GetDuration() const { return Duration; }

	ENGINE_API void ReleaseCompressedAudio() const;

	ENGINE_API const TArray<FSoundWaveCuePoint> GetCuePointsAndLoopRegions() const;
	ENGINE_API TArrayView<const uint8> GetZerothChunk(bool bForImmediatePlayback = false) const;

	ENGINE_API void SetError(const TCHAR* InErrorMsg=nullptr);  
	ENGINE_API bool HasError() const;
	ENGINE_API bool ResetError();

	bool IsStreaming() const { return bIsStreaming; }
	ESoundAssetCompressionType GetSoundCompressionType() const { return SoundAssetCompressionType; }
	bool IsLooping() const { return bIsLooping; }
	bool ShouldUseStreamCaching() const { return bShouldUseStreamCaching; }
	bool IsSeekable() const {  return bIsSeekable; }
	bool IsRetainingAudio() const {  return FirstChunk.IsValid(); }
	bool WasLoadingBehaviorOverridden() const { return bLoadingBehaviorOverridden; }
	ESoundWaveLoadingBehavior GetLoadingBehavior() const { return LoadingBehavior; }

	bool IsTemplate() const { return bIsTemplate; }
	bool HasSubtitleAssetUserData() const { return bHasSubtitleAssetUserData; }

 	ENGINE_API bool HasCompressedData(FName Format, ITargetPlatform* TargetPlatform = USoundWave::GetRunningPlatform()) const;
 	ENGINE_API FByteBulkData* GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides = USoundWave::GetPlatformCompressionOverridesForCurrentPlatform()) const;
 
	ENGINE_API bool GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded = false) const;
 
 	ENGINE_API bool IsZerothChunkDataLoaded() const;
 	ENGINE_API const TArrayView<uint8> GetZerothChunkDataView() const;

	ENGINE_API bool HasChunkSeekTable(int32 InChunkIndex) const;
	ENGINE_API int32 FindChunkIndexForSeeking(uint32 InTimeInAudioFrames) const;

	// Returns true if the zeroth chunk is loaded, or attempts to load it if not already loaded,
	// returning true if the load was successful. Can return false if either an error was encountered
	// in attempting to load the chunk or if stream caching is not enabled for the given sound.
	// (const because this call is used to lazy-init zeroth chunk in GetZerothChunkData())
	ENGINE_API bool LoadZerothChunk() const;
 
 #if WITH_EDITOR
 	ENGINE_API int32 GetCurrentChunkRevision() const;
 #endif // #if WITH_EDITOR
 
 	ENGINE_API FStreamedAudioChunk& GetChunk(uint32 ChunkIndex) const;
 	ENGINE_API int32 GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded) const;

#if WITH_EDITORONLY_DATA
 	ENGINE_API FString GetDerivedDataKey() const;
	FPerPlatformFloat GetSizeOfFirstAudioChunkInSeconds() const { return SizeOfFirstAudioChunkInSeconds; }
#endif // #if WITH_EDITORONLY_DATA

	int32 GetResourceSize() const { return ResourceSize; }
	const uint8* GetResourceData() const { return ResourceData.GetView().GetData(); }
	uint32 GetNumChannels() { return NumChannels; }

	void SetSampleRate(const int32 InSampleRate) { SampleRate = static_cast<float>(InSampleRate); }
	void SetNumFrames(const int32 InNumFrames) { NumFrames = InNumFrames; }

private:
	ENGINE_API void DiscardZerothChunkData();

	ENGINE_API FName FindRuntimeFormat(const USoundWave&) const;

	/**
	 * Cache state that depends on the RuntimeFormat, like things that are dependent on
	 * the decoder implementation on this platform.
	 * @param InFormatName Factory Name of the Formats Decoder "ADPCM" etc.
	 **/
	ENGINE_API void CacheRuntimeFormatDependentState(const FName InFormatName);

	/** Zeroth Chunk of audio for sources that use Load On Demand. */
	// mutable for lazy-init in GetZerothChunkData()
	mutable FBulkDataBuffer<uint8> ZerothChunkData;
#if WITH_EDITOR
	mutable FCriticalSection LoadZerothChunkDataCriticalSection;
#endif

	/* Accessor to get the zeroth chunk which might perform additional work in editor to handle async tasks. */
	ENGINE_API FBulkDataBuffer<uint8>& GetZerothChunkData() const;


	// The following members are mutable to allow lazy initialization in const getters
	// (This way we can hand out immutable refrences to objects of this type)

	/** The streaming derived data for this sound on this platform. */
	mutable FStreamedAudioPlatformData RunningPlatformData;

	mutable FAudioChunkHandle FirstChunk;

	mutable FFormatContainer CompressedFormatData;

	int32 ResourceSize = 0;

	FBulkDataBuffer<uint8> ResourceData;

	ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;

#if WITH_EDITORONLY_DATA
	// Set by CacheInheritedLoadingBehavior after traversing the Soundclass hierarchy 
	FPerPlatformFloat SizeOfFirstAudioChunkInSeconds = 0.f;
#endif //WITH_EDITORONLY_DATA
	
#if WITH_EDITOR
	int32 CurrentChunkRevision = 0;
#endif // #if WITH_EDITOR

	FName NameCached;
	FName PackageNameCached;
	FName RuntimeFormat{ "FSoundWaveProxy_InvalidFormat" };
	FObjectKey SoundWaveKeyCached;
	TArray<FSoundWaveCuePoint> CuePoints;
	TArray<FSoundWaveCuePoint> LoopRegions;
	ESoundAssetCompressionType SoundAssetCompressionType = ESoundAssetCompressionType::BinkAudio;
	FGuid WaveGuid;
	
	float SampleRate = 0;
	float Duration = 0;

	uint32 NumChannels = 0;
	int32 NumFrames = 0;

	// shared flags
	uint8 bIsLooping : 1;
	uint8 bIsTemplate : 1;
	uint8 bIsStreaming : 1;
	uint8 bIsSeekable : 1;
	uint8 bShouldUseStreamCaching : 1;
	uint8 bLoadingBehaviorOverridden : 1;
	uint8 bHasError : 1;
	uint8 bHasSubtitleAssetUserData : 1;

#if WITH_EDITOR
	uint8 bLoadedFromCookedData : 1;
#endif //WITH_EDITOR

	friend class USoundWave;
}; // class FSoundWaveData



// i.e.: "FSoundWaveProxy is a shared object between USoundWave and other systems which utilize FSoundWaveData"
/**
	The proxy acts as a broker between the USoundWave and Proxy Readers.
	if UE_USOUNDWAVE_HOT_RELOADING_ENABLED is true, then proxy clients can use this to query the availablity
	of new FSoundWaveData, and optionally update to that data.

	This lets editor changes and re-imports take effect on even playing sounds.

	In these scenarios, the owning USoundWave has its "working" (un-published) copy of FSoundWaveData
	For changes, and then calls "Publish()" to make this available
*/
using FSoundWaveProxyPtr = TSharedPtr<FSoundWaveProxy, ESPMode::ThreadSafe>;
class FSoundWaveProxy : public Audio::TProxyData<FSoundWaveProxy>
					  , public TSharedFromThis<FSoundWaveProxy, ESPMode::ThreadSafe>	
{
public:
	IMPL_AUDIOPROXY_CLASS(FSoundWaveProxy);

	ENGINE_API FSoundWaveProxy();
	
	ENGINE_API explicit FSoundWaveProxy(USoundWave* InWave);

	ENGINE_API ~FSoundWaveProxy();
	
	static ENGINE_API TSharedRef<FSoundWaveProxy> CreateNonUpdatingCopy(const TSharedRef<const FSoundWaveProxy>& Other);
	static ENGINE_API TSharedRef<const FSoundWaveProxy> CreateNonUpdatingCopy(const TSharedRef<const FSoundWaveData>& Other);
	
	friend inline uint32 GetTypeHash(const FSoundWaveProxy& InProxy)
	{
		return InProxy.TypeHash;
	}

	// USoundWave Interface
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	static ENGINE_API TArrayView<const uint8> GetZerothChunk(const FSoundWaveProxyPtr& SoundWaveProxy, bool bForImmediatePlayback = false);
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const TArray<FSoundWaveCuePoint> GetCuePointsAndLoopRegions() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API void ReleaseCompressedAudio();

	// Returns true if the zeroth chunk is loaded, or attempts to load it if not already loaded,
	// returning true if the load was successful. Can return false if either an error was encountered
	// in attempting to load the chunk or if stream caching is not enabled for the given sound
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool LoadZerothChunk();
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded = false);
	
	// Getters
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const FName& GetFName() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const FName& GetPackageName() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const FName& GetRuntimeFormat() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const FObjectKey& GetFObjectKey() const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API float GetDuration() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API float GetSampleRate() const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API int32 GetNumFrames() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API uint32 GetNumChunks() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API uint32 GetNumChannels() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API uint32 GetSizeOfChunk(uint32 ChunkIndex) const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const TArray<FSoundWaveCuePoint>& GetCuePoints() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const TArray<FSoundWaveCuePoint>& GetLoopRegions() const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsLooping() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsTemplate() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsStreaming() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsRetainingAudio() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool ShouldUseStreamCaching() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsSeekable() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool IsZerothChunkDataLoaded() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool WasLoadingBehaviorOverridden() const;
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API bool HasCompressedData(FName Format, ITargetPlatform* TargetPlatform = USoundWave::GetRunningPlatform()) const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API ESoundWaveLoadingBehavior GetLoadingBehavior() const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const TArrayView<uint8> GetZerothChunkDataView() const;
	
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API FByteBulkData* GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides = USoundWave::GetPlatformCompressionOverridesForCurrentPlatform());

#if WITH_EDITOR
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API int32 GetCurrentChunkRevision() const;
#endif // #if WITH_EDITOR

	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API FStreamedAudioChunk& GetChunk(uint32 ChunkIndex);
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API int32 GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded);

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API FString GetDerivedDataKey() const;
#endif // #if WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API int32 GetResourceSize() const;

	UE_DEPRECATED(5.8, "The FSoundWaveProxy is now just a way to get a reference to the latest FSoundWaveData. Clients should hold onto the return value of GetSoundWaveDataRef() instead")
	ENGINE_API const uint8* GetResourceData() const;

	UE_DEPRECATED(5.8, "mutable access to sound wave data is deprecated. please use GetSoundWaveDataRef()")
	TSharedPtr<FSoundWaveData> GetSoundWaveData() const
	{
		return ConstCastSharedRef<FSoundWaveData>(PublishedSoundWaveData).ToSharedPtr();
	}
	
	ENGINE_API TSharedRef<const FSoundWaveData> GetSoundWaveDataRef() const;

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	bool ENGINE_API IsLatestData(const FSoundWaveData* InData) const;
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	
private:
	friend USoundWave;
	void InitializeFromWave(USoundWave& InWave);

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED	
	void Publish(const TSharedRef<FSoundWaveData>& InNewData);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	
	TSharedRef<FSoundWaveData> PublishedSoundWaveData = MakeShared<FSoundWaveData>();

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	mutable FRWLock PublishedSoundWaveDataLock;
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	
	uint32 TypeHash = uint32(INDEX_NONE);
};

