// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundWaveLoadingBehavior.h"

#include "Audio.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/CommandLine.h"
#include "SoundClass.h"
#include "SoundCue.h"
#include "SoundWave.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "Algo/Sort.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDependencyContext.h"
#include "Cooker/CookEvents.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWaveLoadingBehavior)

#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(TXT) case TXT: return TEXT(#TXT);
#endif

const TCHAR* EnumToString(const ESoundWaveLoadingBehavior InCurrentState)
{
	switch(InCurrentState)
	{
		FOREACH_ENUM_ESOUNDWAVELOADINGBEHAVIOR(CASE_ENUM_TO_TEXT)
	} 
	return TEXT("Unknown");
}


#if WITH_EDITOR

static int32 SoundWaveLoadingBehaviorUtil_CacheAllOnStartup = 0;
FAutoConsoleVariableRef CVAR_SoundWaveLoadingBehaviorUtil_CacheAllOnStartup(
	TEXT("au.editor.SoundWaveOwnerLoadingBehaviorCacheOnStartup"),
	SoundWaveLoadingBehaviorUtil_CacheAllOnStartup,
	TEXT("Disables searching the asset registry on startup of the singleton. Otherwise it will incrementally fill cache"),
	ECVF_Default
);

static int32 SoundWaveLoadingBehaviorUtil_Enable = 1;
FAutoConsoleVariableRef CVAR_SoundWaveLoadingBehaviorUtil_Enable(
	TEXT("au.editor.SoundWaveOwnerLoadingBehaviorEnable"),
	SoundWaveLoadingBehaviorUtil_Enable,
	TEXT("Enables or disables the Soundwave owner loading behavior tagging"),
	ECVF_Default
);

namespace UE::SoundWaveLoadingUtil::Private
{
	void HashSoundWaveLoadingBehaviorDependenciesForCook(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context);
};

class FSoundWaveLoadingBehaviorUtil : public ISoundWaveLoadingBehaviorUtil
{
public:
	FSoundWaveLoadingBehaviorUtil()	
		: AssetRegistry(FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get())
	{
		if (SoundWaveLoadingBehaviorUtil_CacheAllOnStartup)
		{
			CacheAllClassLoadingBehaviors();
		}
	}
	virtual ~FSoundWaveLoadingBehaviorUtil() override = default;

private:

	struct FAnnotatedClassData
	{
		explicit FAnnotatedClassData(USoundClass* InSoundClass) 
		: BaseData(InSoundClass) 
		{
			ClassHierarchy.Add(FName(InSoundClass->GetPackage()->GetFName()));
		}
		FClassData BaseData;
		TArray<FName> ClassHierarchy;
	};

	void CacheAllClassLoadingBehaviors()
	{
		const bool bAssetRegistryInStartup = AssetRegistry.IsSearchAsync() && AssetRegistry.IsLoadingAssets();
		if (!ensureMsgf(!bAssetRegistryInStartup,TEXT("Function must not be called until after cook has started and waited on the AssetRegistry already.")))
		{
			AssetRegistry.WaitForCompletion();
		}
		
		TArray<FAssetData> SoundClasses;
		AssetRegistry.GetAssetsByClass(USoundClass::StaticClass()->GetClassPathName(), SoundClasses, true);
		
		for (const FAssetData& i : SoundClasses)
		{
			LoadAndCacheClass(i);
		}
	}

	FAnnotatedClassData WalkClassHierarchy(USoundClass* InClass) const
	{
		FAnnotatedClassData Behavior(InClass);
		while (InClass->ParentClass && InClass->Properties.LoadingBehavior == ESoundWaveLoadingBehavior::Inherited)
		{
			InClass = InClass->ParentClass;
			InClass->ConditionalPreload();
			InClass->ConditionalPostLoad();
			Behavior.BaseData = FClassData(InClass);
			Behavior.ClassHierarchy.Add(FName(InClass->GetPackage()->GetFName()));
		}
		
		// If we failed to find anything other than inherited, use the default which is cvar'd.
		if (Behavior.BaseData.LoadingBehavior == ESoundWaveLoadingBehavior::Inherited ||
			Behavior.BaseData.LoadingBehavior == ESoundWaveLoadingBehavior::Uninitialized )
		{
			Behavior.BaseData.LoadingBehavior = USoundWave::GetDefaultLoadingBehavior();
			Behavior.BaseData.LengthOfFirstChunkInSeconds = 0;
		}
		return Behavior;
	}

	FClassData LoadAndCacheClass(const FAssetData& InAssetData) const
	{
		if (USoundClass* SoundClass = Cast<USoundClass>(InAssetData.GetAsset()))
		{
			FAnnotatedClassData Result = WalkClassHierarchy(SoundClass);
			CacheClassLoadingBehaviors.Add(InAssetData.PackageName,Result);
			return Result.BaseData;
		}
		return {};
	}

	void CollectAllRelevantSoundClassAssetData(const FName InWaveName, TSet<FAssetData>& OutClassAssetData) const
	{
		TArray<FName> SoundWaveReferencerNames;
		if (!AssetRegistry.GetReferencers(InWaveName, SoundWaveReferencerNames))
		{
			return;
		}

		if (SoundWaveReferencerNames.IsEmpty())
		{
			return;
		}

		// Filter on SoundCues.
		FARFilter Filter;
		// Don't rely on the AR to filter out classes as it's going to gather all assets for the specified classes first, then filtering for the provided packages names afterward,
		// resulting in execution time being over 100 times slower than filtering for classes after package names.
		//Filter.ClassPaths.Add(USoundCue::StaticClass()->GetClassPathName());
		//Filter.bRecursiveClasses = true;
		Filter.PackageNames = SoundWaveReferencerNames;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// Enumerate only on-disk packages when cooking to avoid indeterministic results
		TArray<FAssetData> ReferencingSoundCueAssetDataArray;
		if (!AssetRegistry.GetAssets(Filter, ReferencingSoundCueAssetDataArray))
		{
			return;
		}

		// Filter out unwanted classes, see above comment for details.
		for (int32 AssetIndex = 0; AssetIndex < ReferencingSoundCueAssetDataArray.Num(); AssetIndex++)
		{
			if (UClass* AssetClass = ReferencingSoundCueAssetDataArray[AssetIndex].GetClass(); !AssetClass || !AssetClass->IsChildOf<USoundCue>())
			{
				ReferencingSoundCueAssetDataArray.RemoveAtSwap(AssetIndex--);
			}
		}

		if (ReferencingSoundCueAssetDataArray.IsEmpty())
		{
			return;
		}

		for (const FAssetData& CueAsset : ReferencingSoundCueAssetDataArray)
		{
			// Query for class references from the Cue instead of loading and opening it.
			TArray<FName> SoundCueReferences;
			if (!AssetRegistry.GetDependencies(CueAsset.PackageName, SoundCueReferences))
			{
				UE_LOGF(LogAudio, Warning, "Failed to query SoundCue '%ls' for it's dependencies.",
					*CueAsset.PackagePath.ToString());
				continue;
			}

			if (SoundCueReferences.Num() == 0)
			{
				continue;
			}

			// Filter for Classes.
			FARFilter ClassFilter;
			ClassFilter.ClassPaths.Add(USoundClass::StaticClass()->GetClassPathName());
			ClassFilter.PackageNames = SoundCueReferences;
			ClassFilter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// only enumerate disk-only assets during cooking
			TArray<FAssetData> ReferencedSoundClasses;
			if (!AssetRegistry.GetAssets(ClassFilter, ReferencedSoundClasses))
			{
				UE_LOGF(LogAudio, Warning, "Failed to filter for Soundclasses from the SoundCue dependencies for '%ls'", *CueAsset.PackagePath.ToString());
				continue;
			}
			for (FAssetData& SoundClass : ReferencedSoundClasses)
			{
				OutClassAssetData.Add(MoveTemp(SoundClass));
			}
		}
	}

	virtual FClassData FindOwningLoadingBehavior(const USoundWave* InWave, const ITargetPlatform* InTargetPlatform) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindOwningLoadingBehavior);

		// This code: Given a wave, finds all cues that references it. (reverse lookup)
		// Then finds the SoundClasses those cues use, traverses the heirarchy (from lookup) to determine the loading behavior.
		// Then stack ranks the most important behavior. (RetainOnLoad (Highest), PrimeOnLoad (Medium), LoadOnDemand (Lowest))
		// Which ever wins, we also capture the "SizeOfFirstChunk" to use for that wave.

		if (!InWave)
		{
			return {};
		}

		const bool bIsAssetRegistryStartup = AssetRegistry.IsSearchAsync() && AssetRegistry.IsLoadingAssets();

		// Disallow during startup of registry (cookers will have already done this)
		if (bIsAssetRegistryStartup)
		{
			UE_LOGF(LogAudio, Warning, "FindOwningLoadingBehavior called before AssetRegistry is ready. SoundWave=%ls", *InWave->GetName());
			return {};
		}
	
		const UPackage* WavePackage = InWave->GetPackage();
		if (!WavePackage)
		{
			return {};
		}

		TSet<FAssetData> SoundClassesToConsider;
		CollectAllRelevantSoundClassAssetData(WavePackage->GetFName(), SoundClassesToConsider);

		FClassData MostImportantLoadingBehavior;

		// If there's more than one, rank them.
		FScopeLock Lock(&CacheCS);
		for (const FAssetData& Class : SoundClassesToConsider)
		{
			FClassData CacheLoadingBehavior;
			if (const FAnnotatedClassData* Found = CacheClassLoadingBehaviors.Find(Class.PackageName))
			{
				CacheLoadingBehavior = Found->BaseData;
			}
			else
			{
				CacheLoadingBehavior = LoadAndCacheClass(Class);
			}

			// Compare if this is more important
			if (MostImportantLoadingBehavior.CompareGreater(CacheLoadingBehavior, InTargetPlatform))
			{
				MostImportantLoadingBehavior = CacheLoadingBehavior;
			}
		}
		
		// Return the most important one we found.
		return MostImportantLoadingBehavior;
	}

	void GetChainOfClassesForLoadingBehaviorInheritance(FName StartingClass, TArray<FName>& OutClasses)
	{
		FScopeLock Lock(&CacheCS);
		const FAnnotatedClassData* CacheEntry = CacheClassLoadingBehaviors.Find(StartingClass);
		if (CacheEntry)
		{
			OutClasses = CacheEntry->ClassHierarchy;
		}
		else
		{
			OutClasses.Reset();
		}
	}


	// Make cache here.
	IAssetRegistry& AssetRegistry;
	mutable TMap<FName, FAnnotatedClassData> CacheClassLoadingBehaviors;
	mutable FCriticalSection CacheCS;
	friend void UE::SoundWaveLoadingUtil::Private::HashSoundWaveLoadingBehaviorDependenciesForCook(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context);
	friend void UE::SoundWaveLoadingUtil::Private::RecordSoundWaveLoadingBehaviorDependenciesForCook(UE::Cook::FCookEventContext& CookContext, const USoundWave* SoundWave);
};

namespace UE { namespace SoundWaveLoadingUtil { namespace Private {
constexpr int32 HashSoundWaveLoadingBehaviorDependenciesForCookArgsVersion = 1;
void HashSoundWaveLoadingBehaviorDependenciesForCook(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
{
	int32 ArgsVersion = -1;
	bool bValid = false;

	FCbFieldViewIterator ArgField(Args);
	ArgsVersion = (ArgField++).AsInt32();
	if (ArgsVersion == HashSoundWaveLoadingBehaviorDependenciesForCookArgsVersion)
	{
		bValid = true;
	}
	if (!bValid)
	{
		Context.LogError(FString::Printf(TEXT("Unsupported arguments version %d."), ArgsVersion));
		return;
	}

	TSet<FAssetData> ClassDependenciesFromAssetRegistryQuery;
	FSoundWaveLoadingBehaviorUtil* Util = static_cast<FSoundWaveLoadingBehaviorUtil*>(FSoundWaveLoadingBehaviorUtil::Get());
	Util->CollectAllRelevantSoundClassAssetData(Context.GetPackageName(), ClassDependenciesFromAssetRegistryQuery);

	TArray<FName> SortedClassDependencies;
	for (const FAssetData& AssetData : ClassDependenciesFromAssetRegistryQuery)
	{
		SortedClassDependencies.Add(AssetData.PackageName);
	}
	Algo::Sort(SortedClassDependencies, FNameLexicalLess());

	for (const FName& ClassDependency : SortedClassDependencies)
	{
		TStringBuilder<256> ClassNameString(InPlace, ClassDependency);
		Context.Update(ClassNameString.GetData(), ClassNameString.Len() * sizeof(ClassNameString.GetData()[0]));
	}
}

UE_COOK_DEPENDENCY_FUNCTION(HashSoundWaveLoadingBehaviorDependenciesForCook, HashSoundWaveLoadingBehaviorDependenciesForCook);

void RecordSoundWaveLoadingBehaviorDependenciesForCook(UE::Cook::FCookEventContext& CookContext,
	const USoundWave* SoundWave)
{
	// Cooking of USoundWaves depends on a bunch of indirect state that we want to cooker to know about for incremental cook invalidation
	// This is due to how we determine the sound wave loading behavior and associated chunk sizes
	// Anything that impacts the result of FindOwningLoadingBehavior needs to be encapsulated so that the cooker can
	// determine whether the results might be different from when the base cook was made
	FCbWriter Writer;
	Writer << HashSoundWaveLoadingBehaviorDependenciesForCookArgsVersion;

	// 1. We depend on the set of USoundClass objects which are dependencies of any USoundCue that depends on our USoundWave
	// Those classes are identified via a series of asset registry queries encapsulated in CollectAllRelevantSoundClassAssetData
	// HashSoundWaveLoadingBehaviorDependenciesForCook is used to package up that query and make it deterministic for the cooker
	// dependency checking
	
	CookContext.AddLoadBuildDependency(
		UE::Cook::FCookDependency::Function(
			UE_COOK_DEPENDENCY_FUNCTION_CALL(HashSoundWaveLoadingBehaviorDependenciesForCook), Writer.Save()));

	FSoundWaveLoadingBehaviorUtil* Util = static_cast<FSoundWaveLoadingBehaviorUtil*>(FSoundWaveLoadingBehaviorUtil::Get());

	// 2. For each class in 1, we have an additional dependency on a its inheritance hierarchy as identified by 
	// GetChainOfClassesForLoadingBehaviorInheritance 
	TSet<FAssetData> ClassDependenciesFromAssetRegistryQuery;
	Util->CollectAllRelevantSoundClassAssetData(SoundWave->GetPackage()->GetFName(), ClassDependenciesFromAssetRegistryQuery);

	for (const FAssetData& DependencyFromRegistry : ClassDependenciesFromAssetRegistryQuery)
	{
		TArray<FName> SoundCueHierarchyDependencies;
		Util->GetChainOfClassesForLoadingBehaviorInheritance(DependencyFromRegistry.PackageName, SoundCueHierarchyDependencies);

		for (FName SoundCue : SoundCueHierarchyDependencies)
		{
			CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::Package(SoundCue));
		}
	}

	// 3. Finally, we depend on the value of the default loading behavior cvar
	CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::ConsoleVariable(USoundWave::GetDefaultLoadingBehaviorCVarName()));
}
}}}

ISoundWaveLoadingBehaviorUtil* ISoundWaveLoadingBehaviorUtil::Get()
{
	// Cvar disable system if necessary
	if (!SoundWaveLoadingBehaviorUtil_Enable)
	{
		return nullptr;
	}
		
	// Only run while the cooker is active.
	if (!IsRunningCookCommandlet())
	{		
		return nullptr;
	}
		
	static FSoundWaveLoadingBehaviorUtil Instance;
	return &Instance;
}

ISoundWaveLoadingBehaviorUtil::FClassData::FClassData(const USoundClass* InClass)
	: LoadingBehavior(InClass->Properties.LoadingBehavior)
	, LengthOfFirstChunkInSeconds(InClass->Properties.SizeOfFirstAudioChunkInSeconds)
{
}

bool ISoundWaveLoadingBehaviorUtil::FClassData::CompareGreater(const FClassData& InOther, const ITargetPlatform* InPlatform) const
{
	if (InOther.LoadingBehavior != LoadingBehavior)
	{
		// If loading behavior enum is less, it's more important.
		return InOther.LoadingBehavior < LoadingBehavior;
	}
	else 
	{
		// If we are using Prime/Retain, use one with the higher Length.
		if (LoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad ||
			LoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad )
		{
			const float Length = LengthOfFirstChunkInSeconds.GetValueForPlatform(*InPlatform->PlatformName());
			const float OtherLength = InOther.LengthOfFirstChunkInSeconds.GetValueForPlatform(*InPlatform->PlatformName());
				
			if (OtherLength > Length)
			{
				return true;
			}
		}
		return false;
	}
}

#endif //WITH_EDITOR
