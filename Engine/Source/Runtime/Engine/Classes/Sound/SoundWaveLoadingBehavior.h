// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#if WITH_EDITOR
#include "UObject/PerPlatformProperties.h"
#endif //WITH_EDITOR

#include "SoundWaveLoadingBehavior.generated.h"

/**
 * Only used when stream caching is enabled. Determines how we are going to load or retain a given audio asset.
 * A USoundWave's loading behavior can be overridden in the USoundWave itself, the sound wave's USoundClass, or by cvars.
 * The order of priority is defined as:
 * 1) The loading behavior set on the USoundWave
 * 2) The loading behavior set on the USoundWave's USoundClass.
 * 3) The loading behavior set on the nearest parent of a USoundWave's USoundClass.
 * 4) The loading behavior set via the au.streamcache cvars.
 */
UENUM()
enum class ESoundWaveLoadingBehavior : uint8
{
	// If set on a USoundWave, use the setting defined by the USoundClass. If set on the next parent USoundClass, or the default behavior defined via the au.streamcache cvars.
	Inherited = 0,
	// the first chunk of audio for this asset will be retained in the audio cache until a given USoundWave is either destroyed or USoundWave::ReleaseCompressedAudioData is called.
	RetainOnLoad = 1,
	// the first chunk of audio for this asset will be loaded into the cache from disk when this asset is loaded, but may be evicted to make room for other audio if it isn't played for a while.
	PrimeOnLoad = 2,
	// the first chunk of audio for this asset will not be loaded until this asset is played or primed.
	LoadOnDemand = 3,
	// Force all audio data for this audio asset to live outside of the cache and use the non-streaming decode pathways. Only usable if set on the USoundWave.
	ForceInline = 4 UMETA(DisplayName = "Force Inline"),
	// This value is used to delineate when the value of ESoundWaveLoadingBehavior hasn't been cached on a USoundWave yet.
	Uninitialized = 0xff UMETA(Hidden)
};

ENGINE_API const TCHAR* EnumToString(ESoundWaveLoadingBehavior InCurrentState);

#if WITH_EDITOR

class USoundWave;
class USoundClass;
class ITargetPlatform;
namespace UE::Cook { class FCookEventContext; }

class ISoundWaveLoadingBehaviorUtil
{
public:
	virtual ~ISoundWaveLoadingBehaviorUtil() = default;

	struct FClassData
	{
		FClassData() = default;
		FClassData(const USoundClass* InClass);

		bool CompareGreater(const FClassData&, const ITargetPlatform*) const;

		ESoundWaveLoadingBehavior LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
		FPerPlatformFloat LengthOfFirstChunkInSeconds;
	};

	static ISoundWaveLoadingBehaviorUtil* Get();	

	virtual FClassData FindOwningLoadingBehavior(const USoundWave*, const ITargetPlatform*) const = 0;
};

namespace UE::SoundWaveLoadingUtil::Private
{
	void RecordSoundWaveLoadingBehaviorDependenciesForCook(UE::Cook::FCookEventContext& CookContext, const USoundWave* SoundWave);
};

#endif //WITH_EDITOR
