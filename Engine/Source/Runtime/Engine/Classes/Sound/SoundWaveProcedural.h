// Copyright Epic Games, Inc. All Rights Reserved.


/** 
 * Playable sound object for wave files that are streamed, particularly VOIP
 */

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectMacros.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Sound/SoundWave.h"
#include "SoundWaveProcedural.generated.h"

#if PLATFORM_IOS
#define DEFAULT_PROCEDURAL_SOUNDWAVE_BUFFER_SIZE (8 * 1024)
#else
#define DEFAULT_PROCEDURAL_SOUNDWAVE_BUFFER_SIZE 1024
#endif

DECLARE_DELEGATE_TwoParams( FOnSoundWaveProceduralUnderflow, class USoundWaveProcedural*, int32 );

UCLASS(MinimalAPI)
class USoundWaveProcedural : public USoundWave
{
	GENERATED_BODY()

private:
	// A thread safe queue for queuing audio to be consumed on audio thread
	TQueue<TArray<uint8>> QueuedAudio;

	// The amount of bytes queued and not yet consumed
	FThreadSafeCounter AvailableByteCount;

	// The actual audio buffer that can be consumed. QueuedAudio is fed to this buffer. Accessed only audio thread.
	TArray<uint8> AudioBuffer;

	// Flag to reset the audio buffer
	FThreadSafeBool bReset;

	// Pumps audio queued from game thread
	ENGINE_API void PumpQueuedAudio();

protected:

	// Number of samples to pad with 0 if there isn't enough audio available
	int32 NumBufferUnderrunSamples;

	// The number of PCM samples we want to generate. This can't be larger than SamplesNeeded in GeneratePCMData callback, but can be less.
	int32 NumSamplesToGeneratePerCallback;

	// Procedural Sounds don't represent a wav file, don't do anything when serializing cue points
	virtual void SerializeCuePoints(FArchive& Ar, const bool bIsLoadingFromCookedArchive) {}

public:
	ENGINE_API USoundWaveProcedural(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject Interface. 
	ENGINE_API virtual void Serialize( FArchive& Ar ) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~ End UObject Interface. 

	//~ Begin USoundWave Interface.
	ENGINE_API virtual int32 GeneratePCMData(uint8* PCMData, const int32 SamplesNeeded) override;
	ENGINE_API virtual bool HasCompressedData(FName Format, ITargetPlatform* TargetPlatform) const override;
	ENGINE_API virtual void BeginGetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform) override;
	ENGINE_API virtual FByteBulkData* GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform) override;
	ENGINE_API virtual void InitAudioResource( FByteBulkData& CompressedData ) override;
	ENGINE_API virtual bool InitAudioResource(FName Format) override;
	ENGINE_API virtual int32 GetResourceSizeForFormat(FName Format) override;
	virtual bool IsSeekable() const override { return false; }
	//~ End USoundWave Interface.

	// Virtual function to generate PCM audio from the audio render thread. 
	// Returns number of samples generated
	virtual int32 OnGeneratePCMAudio(TArray<uint8>& OutAudio, int32 NumSamples) { return 0; }

	/** Add data to the FIFO that feeds the audio device. */
	ENGINE_API void QueueAudio(const uint8* AudioData, const int32 BufferSize);

	/** Remove all queued data from the FIFO. This is only necessary if you want to start over, or GeneratePCMData() isn't going to be called, since that will eventually drain it. */
	ENGINE_API void ResetAudio();

	/** Query bytes queued for playback */
	ENGINE_API int32 GetAvailableAudioByteCount();

	/** Called when GeneratePCMData is called but not enough data is available. Allows more data to be added, and will try again */
	FOnSoundWaveProceduralUnderflow OnSoundWaveProceduralUnderflow;

	/** Size in bytes of a single sample of audio in the procedural audio buffer. */
	int32 SampleByteSize;
};
