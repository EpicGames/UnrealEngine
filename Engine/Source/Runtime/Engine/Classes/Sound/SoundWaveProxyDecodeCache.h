// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioDecompress.h"
#include "HAL/LowLevelMemTracker.h"
#include "UObject/PerPlatformProperties.h"

LLM_DECLARE_TAG_API(Audio_SoundWaveDecodeCache, ENGINE_API);

class FSoundWaveData;
class FSoundWaveProxy;
class ICompressedAudioInfo;

/**
 * A sound wave proxy decode cache.
 * Supports caching sound wave proxy decoding to prevent decoding the exact same audio multiple times.
 * Useful to significantly reduce CPU usage when decoding short-duration or oft-repeated audio files.
 */
namespace SoundWaveProxyDecodeCache
{
	using FSoundWaveProxyRef = TSharedRef<FSoundWaveProxy, ESPMode::ThreadSafe>;
	using FSoundWaveDataRef = TSharedRef<const FSoundWaveData, ESPMode::ThreadSafe>;

	/**
	* Queries if the decode cache is enabled.
	*/
	bool ENGINE_API IsEnabled();

	/**
	* Used to create an ICompressedAudioInfo instance (a decoder) that will interact with the decode cache.
	* The cache is keyed off the immutable identity carried on the data snapshot (GUID/format/etc.),
	* so callers only need a TSharedRef to the FSoundWaveData they intend to decode.
	* @param InSoundWaveData The sound wave data snapshot to decode.
	* @return A new ICompressedAudioInfo unique ptr that serves as a decoder for the sound wave data.
	*/
	TUniquePtr<ICompressedAudioInfo> ENGINE_API CreateDecoderInstance(const FSoundWaveDataRef& InSoundWaveData);

	UE_DEPRECATED(5.8, "Use the TSharedRef<const FSoundWaveData>& overload instead. The proxy is a broker for FSoundWaveData — pass the snapshot via InSoundWaveProxy->GetSoundWaveDataRef().")
	TUniquePtr<ICompressedAudioInfo> ENGINE_API CreateDecoderInstance(const FSoundWaveProxyRef& InSoundWaveProxy);
};
