// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDecompress.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DSP/ChannelMap.h"
#include "DSP/ConvertDeinterleave.h"
#include "DSP/RuntimeResampler.h"
#include "HAL/Platform.h"

// Forward declare
class FSoundWaveProxy;
class ICompressedAudioInfo;

#define UE_API ENGINE_API

class UE_INTERNAL FSoundWaveProxyPlayer
{
public:
	using FSoundWaveProxyRef = TSharedRef<FSoundWaveProxy, ESPMode::ThreadSafe>;

	struct FSettings
	{
		FSettings(float InOutputSampleRate, int32 InOutputNumChannels)
			: OutputSampleRate(InOutputSampleRate)
			, OutputNumChannels(InOutputNumChannels)
		{
			check(OutputSampleRate > 0.0f);
			check(OutputNumChannels > 0);
			check(OutputNumChannels <= 8);
		}
		
		float OutputSampleRate = 0.0f;
		int32 OutputNumChannels = 0;
		// Method for upmixing mono audio to multichannel output
		Audio::EChannelMapMonoUpmixMethod MonoUpmixMethod = Audio::EChannelMapMonoUpmixMethod::FullVolume;
		int32 MaxDecodeSizeFrames = 1024;
		bool bMaintainAudioSync = false;
	};

	struct FSourceEvent
	{
		enum EType
		{
			CuePoint,
			Loop,
			OnFinished,
			OnNearlyFinished
		};
		
		FSourceEvent(EType InType, int32 InSourceFrameIndex, int32 InOutputFrameIndex, int32 InCuePointIndex = INDEX_NONE)
			: Type(InType)
			, SourceFrameIndex(InSourceFrameIndex)
			, OutputFrameIndex(InOutputFrameIndex)
			, CuePointIndex(InCuePointIndex)
		{
		}

		EType Type = Loop;

		// The Source Frame into the Source Sound Wave
		int32 SourceFrameIndex = INDEX_NONE;

		// The Output Frame Index relative to the render block
		int32 OutputFrameIndex = INDEX_NONE;

		// For Cue Point Events only - the index into the sorted cue point
		// default to INDEX_NONE 
		int32 CuePointIndex = INDEX_NONE;
	};

	~FSoundWaveProxyPlayer() {}

	/**
	 * Create a SoundWaveProxyPlayer initialized with the given output settings
	 * (Note: Settings can be changed after creation, so no need to create a new one if you need to change the settings)
	 * 
	 * @param InSettings 
	 * @return a UniquePtr to a new WaveProxyPlayer
	 */
	UE_API static TUniquePtr<FSoundWaveProxyPlayer> Create(const FSettings& InSettings);

	/**
	 * Set the SoundWave Data to play
	 * 
	* if the FSoundWaveData has invalid data or is unabled to be decoded,
	 * the player will be reset and this will return 'false'
	 * 
	 * @param InWaveData 
	 * @return whether the player is valid for playback
	 */
	UE_API bool SetSoundWave(const TSharedRef<const FSoundWaveData>& InWaveData); 
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	UE_API bool SetSoundWave(FSoundWaveProxyPtr InWaveProxy);

	/**
	 * 
	 * @return the number of channels this player will render out to
	 */
	UE_API int32 GetOutputNumChannels() const;

	/**
	 *
	 * @return the sample rate this player will render out to in Hz 
	 */
	UE_API float GetOutputSampleRate() const;

	/**
	 * 
	 * @return the number of channels of the source sound wave
	 */
	UE_API int32 GetSourceNumChannels() const;

	/**
	 * 
	 * @return Number of frames in the source sound wave
	 */
	UE_API int32 GetSourceNumFrames() const;

	/**
	 * 
	 * @return Duration (in seconds) of the source sound wave 
	 */
	UE_API float GetSourceDuration() const;

	/**
	 * 
	 * @return the sample rate of the source sound wave
	 */
	UE_API float GetSourceSampleRate() const;

	/**
	 * 
	 * @return whether this player has a valid Sound Wave and Decoder
	 */
	UE_API bool IsPlayerValid() const;
	
	/**
	 * Reset the Wave Player with a new Wave Proxy and new settings
	 * Helpful to avoid creating a whole new Wave Player with new buffers etc.
	 * 
	 * @param NewSettings
	 */
	UE_API void Reset(const FSettings& NewSettings);

	/**
	 * Render audio from the Wave Source based on the current Speed, Loop Settings, and Desired output Sample Rate
	 * Expects the number of channels in the output buffer be the same as the initial settings
	 *
	 * Asserts the SoundWave is set
	 * 
	 * @param OutMultiChannelAudio 
	 */
	UE_API void RenderMultiChannelAudio(const Audio::FMultichannelBufferView& OutMultiChannelAudio);

	/**
	 * Render audio from the Wave Source based on the current Speed, Loop Settings, and Desired output Sample Rate
	 * Expects the number of channels in the output buffer be the same as the initial settings
	 *
	 * Also, Generate a list of source events (Cue Points & Loop Points) based on the current location of the player
	 *
	 * Output Event Output Frames will be relative to the given Block Frame Index (accounts for resampling)
	 * Output Event Source Frames will be in absolute Frames into the Source Sound (no resampling)
	 *
	 * Asserts the SoundWave is set
	 * 
	 * @param BlockFrameIndex The Current BlockFrameIndex for this render block
	 * @param OutMultiChannelAudio The output audio buffer view
	 * @param OutEvents the output source events triggered during this render
	 */
	UE_API void RenderMultiChannelAudio(int32 BlockFrameIndex, Audio::FMultichannelBufferView& OutMultiChannelAudio, TArray<FSourceEvent>& OutEvents);

	/**
	 * Directly generate audio from the Wave Source into an interleaved buffer
	 *
	 * This will update the current playback state of the wave player
	 *
	 * will honor loop settings
	 *
	 * Asserts that the SoundWave is set
	 * 
	 * @param OutAudio 
	 * @return The number of frames generated
	 */
	UE_API int32 GenerateSourceAudio(TArrayView<float> OutAudio);

	/**
	 * Set the speed of the wave player
	 * clamped to MinSpeed & MaxSpeed
	 * @param InSpeed 
	 */
	UE_API void SetSpeed(float InSpeed);

	/**
	 * @return the current speed of the wave player
	 */
	UE_API float GetSpeed() const;

	/**
	 * Seek this Wave Player to the specified time
	 * Can fail if the wave player is not initialized or the wave asset does not properly support seeking
	 * 
	 * @param TimeSeconds 
	 * @return whether the wave player successfully seeked to the desired time
	 */
	UE_API bool SeekToTime(float TimeSeconds);

	/**
	 * Seek this wave player to a specified frame.
	 * Can fail if the wave player is not initialized or the wave asset does not properly support seeking
	 * 
	 * @param InFrame 
	 * @return whether the wave player successfully seeked to the desired frame
	 */
	UE_API bool SeekToFrame(int32 InFrame);

	/**
	 * Set whether this wave player should loop with optional loop parameters
	 * 
	 * @param InIsLooping 
	 * @param InLoopStartTimeSeconds 
	 * @param InLoopDurationSeconds 
	 */
	UE_API void SetLoop(bool InIsLooping, float InLoopStartTimeSeconds = 0.0f, float InLoopDurationSeconds = -1.0f);

	/**
	 * @return whether this wave player is set to looping
	 */
	UE_API bool IsLooping() const;

	/**
	 * 
	 * @return true if the Player has reached the end of the SoundWave
	 */
	UE_API bool IsFinished() const;
	
	/**
	 * Returns the current playback position into the Source 
	 *
	 * @return Current Playback Position in Seconds
	 */
	UE_API float GetCurrentPlaybackTimeSeconds() const;

	/**
	 * The Current playback position into the Source 
	 * 
	 * @return Current Playback Position In Frames
	 */
	UE_API int32 GetCurrentPlaybackFrame() const;

	/**
	 * Returns the current playback progress based on the playback time into the source file
	 *
	 * @return PlayProgress in the range [0.0, 1.0]
	 */
	UE_API float GetPlaybackProgress() const;

	/**
	 * Returns the current loop progress between the current loop region
	 * based on the playback time in the source file
	 *
	 * @return LoopProgress in the range [0.0, 1.0]
	 */
	UE_API float GetLoopProgress() const;

	/**
	 * Get the Current Cue Points as a sorted array
	 * If the Sound Wave is not valid, cue points will be empty
	 * 
	 * @return Sorted Array of Cue Points
	 */
	UE_API const TArray<FSoundWaveCuePoint>& GetCuePoints() const;

private:
	
	/**
	 * Intentionally Private constructor. call the static method "Create" to make a UniquePtr to this object
	 * 
	 * @param InSettings 
	 */
	FSoundWaveProxyPlayer(const FSettings& InSettings);

	/**
	 * Check whether the Wave Proxy has valid data (valid Sample Rate, Num Channels, Num Frames)
	 * 
	 * @param InWaveData
	 * @return whether the WaveProxy is valid
	 */
	static bool IsValidSoundWaveData(const FSoundWaveData& InWaveData);

	// 
	/**
	 * performs the actual streaming/reading of the audio from the decoder into an interleaved output buffer
	 *
	 * Called by GenerateSourceAudio to generate subblock range of audio
	 * 
	 * @param OutAudio 
	 * @return 
	 */
	int32 GenerateSourceAudioInternal(TArrayView<float> OutAudio);

	/**
	 * Generate a list of source events (Cue Points & Loop Points) given a current Source Start Frame Index
	 *
	 * Called immediately before RenderMultiChannelAudio
	 *
	 * @param BlockFrameIndex 
	 * @param NumFrames 
	 * @param OutEvents 
	 */
	void GenerateSourceEvents(int32 BlockFrameIndex, int32 NumFrames, TArray<FSourceEvent>& OutEvents);

	/**
	 * Generate a list of source events (Cue Points & Loop Points) given a current Source Start Frame Index
	 *
	 * Used by GenerateSourceEvents to generate subblock ranges of events
	 *
	 * @param InSourceStartFrameIndex 
	 * @param InOutputStartFrameIndex 
	 * @param NumFrames 
	 * @param OutEvents 
	 */
	void GenerateSourceEventsInternal(int32 InSourceStartFrameIndex, int32 InOutputStartFrameIndex, int32 NumFrames, TArray<FSourceEvent>& OutEvents);
	
	/**
	 * "seek forward" by discarding the requested number of frames from the decoder and advancing the "source" frame
	 * does NOT update "audio sync" frame
	 * 
	 * @param NumFrames 
	 * @return Number of frames that were actually seeked
	 */
	int32 SimulateSeekForward(int32 NumFrames);

	/**
	 * Calculate the current frame ratio based on Speed & Resampling (if needed)
	 * 
	 * @return Current Frame Ratio
	 */
	float GetCurrentFrameRatio() const;

	/**
	 * Initialize the Decoder with the given wave proxy
	 * also initialize the decode buffers
	 * @param MaxDecodeSizeInFrames 
	 * @return whether the decoder was successfully initialized
	 */
	bool InitializeDecoder(int32 MaxDecodeSizeInFrames);

	void InitializeResampler(int32 NumChannels);

	TSharedPtr<const FSoundWaveData> SoundWaveDataPtr;
	FSettings Settings;
	TUniquePtr<ICompressedAudioInfo> CompressedAudioInfo;
	TUniquePtr<Audio::IConvertDeinterleave> ConvertDeinterleave;
	Audio::FRuntimeResampler RuntimeResampler;

	// working "decode buffer" to write decoded audio to before converting it to floats
	TArray<int16, Audio::FAudioBufferAlignedAllocator> DecodeBuffer;

	// "source audio" buffer to write "interleaved" audio to before de-interleaving
	Audio::FAlignedFloatBuffer SourceAudio;
	
	// "resampled audio" buffer to render "Resampled" audio to before de-interleaving
	Audio::FAlignedFloatBuffer ResampledAudio;

	// source wave data
	float DurationSeconds = 0.0f;
	float SampleRate = 0.0f;
	int32 NumChannels = 0;
	int32 NumFramesInWave = 0;
	TArray<FSoundWaveCuePoint> SortedCuePoints;

	// playback
	int32 SourceFrameIndex = 0;
	int32 AudioSyncFrameIndex = 0;
	float Speed = 1.0f;
	bool bIsLooping = false;
	float CurrentLoopStartTime = 0.0f;
	float CurrentLoopDuration = -1.0f;
	int32 LoopStartFrameIndex = 0;
	int32 LoopEndFrameIndex = -1;
	bool bRequiresResampling = false;
	bool bHasNearlyFinished = false;

	static constexpr float MinLoopDurationSeconds = 0.05f;
	static constexpr float MinSpeed = 0.01f;
	static constexpr float MaxSpeed = 100.0f;
	
	bool bFallbackSeekMethodWarningLogged = false;

	// Track how many samples played before queueing another subtitle chunk.
	int64 Chan0SamplesSinceQueuedSubtitles = 0;
};

#undef UE_API