// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/BufferVectorOperations.h"
#include "AudioDecompress.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

// Forward declare
class FSoundWaveProxy;

/** FSoundWaveProxyReader reads a FWaveProxy and outputs 32 bit interleaved audio.
*
* FSoundWaveProxyReader provides controls for looping and relevant frame index
* values.
*/
class UE_DEPRECATED(5.8, "Use FSoundWaveProxyPlayer instead.") FSoundWaveProxyReader
{

public:
	using FSoundWaveProxyRef = TSharedRef<FSoundWaveProxy, ESPMode::ThreadSafe>;
	using FSoundWaveDataRef = TSharedRef<const FSoundWaveData, ESPMode::ThreadSafe>;

	/** Minimum number of frames to decode per a call to the decoder.  */
	static constexpr uint32 DefaultMinDecodeSizeInFrames = 128;

	/** Some codecs have strict requirements on decode size. In order to be
	 * functional with all supported codecs, the decode size must be a multiple
	 * of 128.
	 */
	static constexpr uint32 DecodeSizeQuantizationInFrames = 128;

	static uint32 ConformDecodeSize(uint32 InMaxDesiredDecodeSizeInFrames);

	/** Minimum duration of a loop. */
	static constexpr float MinLoopDurationInSeconds = 0.05f;

	/** Maximum duration of a loop.
	 *
	 * 10 years. ridiculously high. This exists to prevent floating point
	 * undefined overflow behavior if performing calculations with
	 * TNumericLimits<float>::Max().
	 */
	static constexpr float MaxLoopDurationInSeconds = 60.f * 60.f * 365.f * 10.f;

	/** Settings for a FSoundWaveProxyReader. */
	struct FSettings
	{
		uint32 MaxDecodeSizeInFrames = 1024;
		float StartTimeInSeconds = 0.f;
		bool bIsLooping = false;
		float LoopStartTimeInSeconds = 0.f;
		float LoopDurationInSeconds = 0.f;
		bool bMaintainAudioSync = false;
	};

private:
	/** Construct a wave proxy reader.
	 *
	 * @param InWaveData - A TSharedRef of a const FSoundWaveData which is to be played.
	 * @param InSettings - Reader settings.
	 */
	FSoundWaveProxyReader(FSoundWaveDataRef InWaveData, const FSettings& InSettings);
	
	UE_DEPRECATED(5.8, "Please use the FSoundWaveDataRef overload instead")
	FSoundWaveProxyReader(FSoundWaveProxyRef InWaveProxy, const FSettings& InSettings);
	

public:
	/** Create a wave proxy reader.
	 *
	 * @param InWaveProxy - A TSharedRef of a const FSoundWaveData which is to be played.
	 * @param InSettings - Reader settings.
	 */
	static ENGINE_API TUniquePtr<FSoundWaveProxyReader> Create(FSoundWaveDataRef InWaveProxy, const FSettings& InSettings);
	
	UE_DEPRECATED(5.8, "Please use the FSoundWaveDataRef overload instead")
	static ENGINE_API TUniquePtr<FSoundWaveProxyReader> Create(FSoundWaveProxyRef InWaveProxy, const FSettings& InSettings);

	/** Set whether the reader should loop the audio or not. */
	ENGINE_API void SetIsLooping(bool bInIsLooping);

	/** Returns true if the audio will be looped, false otherwise. */
	inline bool IsLooping() const
	{
		return Settings.bIsLooping;
	}

	/** Sets the beginning position of the loop. */
	ENGINE_API void SetLoopStartTime(float InLoopStartTimeInSeconds);

	/** Sets the duration of the loop in seconds.
	 *
	 * If the value is negative, the MaxLoopDurationInSeconds will be used
	 * which will effectively loop at the end of the file.
	 */
	ENGINE_API void SetLoopDuration(float InLoopDurationInSeconds);

	inline float GetSampleRate() const
	{
		return SampleRate;
	}

	inline int32 GetNumChannels() const
	{
		return NumChannels;
	}

	/** Returns the index of the playhead within the complete wave. */
	inline int32 GetFrameIndex() const
	{
		return CurrentFrameIndex;
	}

	inline int32 GetNumFramesInWave() const
	{
		return NumFramesInWave;
	}

	inline int32 GetNumFramesInLoop() const
	{
		return LoopEndFrameIndex - LoopStartFrameIndex;
	}

	inline int32 GetLoopStartFrameIndex() const
	{
		return LoopStartFrameIndex;
	}

	inline int32 GetLoopEndFrameIndex() const
	{
		return LoopEndFrameIndex;
	}

	/** Seeks to position in wave.
	 *
	 * @param InSeconds - The location to seek the playhead
	 *
	 * @return true on success, false on failure.
	 */
	ENGINE_API bool SeekToTime(float InSeconds);

	/** Seeks to position in wave at a specific frame.
	 *
	 * @param InFrameNum - The specific frame to seek the playhead
	 *
	 * @return true on success, false on failure.
	 */
	ENGINE_API bool SeekToFrame(uint32 InFrameNum);


	/** Pops audio from reader and copies audio into OutBuffer. It returns the number of samples copied.
	 * Samples not written to will be set to zero.
	 */
	ENGINE_API int32 PopAudio(Audio::FAlignedFloatBuffer& OutBuffer);

	/** Returns TRUE if the reader can produce audio (eg. has a valid decoder, it can still decode, etc.)
	 *
	 * @return true on success, false on failure.
	 */
	ENGINE_API bool CanProduceMoreAudio() const;

	/** Returns TRUE if the reader has encountered a decoder failure.
	 *
	 * @return true if there is a failure, false if no failure.
	 */
	bool HasFailed() const
	{ 
		return DecodeResult == EDecodeResult::Fail; 
	}

private:

	enum class EDecodeResult
	{
		Fail,					// Decoder failed somehow.
		MoreDataRemaining,		// Data has produced and there's more remaining
		Finished				// The decoder has reached the end of the wave data
	};

	EDecodeResult Decode();

	int32 PopAudioFromDecoderOutput(TArrayView<float> OutBufferView);
	bool InitializeDecoder(float InStartTimeInSeconds);
	int32 DiscardSamples(int32 InNumSamplesToDiscard);
	float ClampLoopStartTime(float InStartTimeInSeconds);
	float ClampLoopDuration(float InDurationInSeconds);
	void UpdateLoopBoundaries();

	FSoundWaveDataRef WaveDataRef;

	FSettings Settings;

	float SampleRate = 0.f;
	int32 NumChannels = 0;
	EDecodeResult DecodeResult = EDecodeResult::MoreDataRemaining;
	int32 NumFramesInWave = 0;
	int32 NumDecodeSamplesToDiscard = 0;
	int32 CurrentFrameIndex = 0;
	int32 LoopStartFrameIndex = 0;
	int32 LoopEndFrameIndex = -1;
	float DurationInSeconds = 0.f;
	float MaxLoopStartTimeInSeconds = 0.f;

	bool bIsDecoderValid = false;
	bool bFallbackSeekMethodWarningLogged = false;

private:
	mutable TUniquePtr<class ICompressedAudioInfo> CompressedAudioInfo;
	Audio::TCircularAudioBuffer<float> DecoderOutput;

	TArray<int16> ResidualBuffer;
	Audio::FAlignedFloatBuffer SampleConversionBuffer;

	static constexpr uint32 MinNumFramesPerDecode = 1;

	bool bIsFirstDecode = true;
	bool bPreviousIsStreaming = true;
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	uint32 NumFramesPerDecode = MinNumFramesPerDecode;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
