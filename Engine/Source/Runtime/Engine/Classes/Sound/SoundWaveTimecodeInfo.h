// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/FrameRate.h"

#include "SoundWaveTimecodeInfo.generated.h"

USTRUCT()
struct FSoundWaveTimecodeInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	uint64 NumSamplesSinceMidnight = ~0UL;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	uint32 NumSamplesPerSecond = 0;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FString Description;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FString OriginatorTime;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FString OriginatorDate;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FString OriginatorDescription;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FString OriginatorReference;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	FFrameRate TimecodeRate;

	UPROPERTY(Category = Timecode, VisibleAnywhere)
	bool bTimecodeIsDropFrame = false;

	inline bool operator==(const FSoundWaveTimecodeInfo& InRhs) const
	{
		// Note, we don't compare the strings.
		return NumSamplesSinceMidnight == InRhs.NumSamplesSinceMidnight &&
			NumSamplesPerSecond == InRhs.NumSamplesPerSecond &&
			TimecodeRate == InRhs.TimecodeRate &&
			bTimecodeIsDropFrame == InRhs.bTimecodeIsDropFrame;
	}

	inline double GetNumSecondsSinceMidnight() const
	{
		if( NumSamplesSinceMidnight != ~0 && NumSamplesPerSecond > 0)
		{
			return (double)NumSamplesSinceMidnight / NumSamplesPerSecond;
		}
		return 0.0;
	}
};

