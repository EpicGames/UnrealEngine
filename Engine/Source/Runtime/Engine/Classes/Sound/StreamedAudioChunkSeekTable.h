// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Templates/PimplPtr.h"

#include "StreamedAudioChunkSeekTable.generated.h"

// Forward declarations.
class FArchive;

UENUM()
enum class EChunkSeekTableMode : uint8
{
	ConstantSamplesPerEntry = 0,
	VariableSamplesPerEntry,
};

/**
 * Class representing an audio seek-table. Typically embedded in a bit-stream.
 */
class FStreamedAudioChunkSeekTable
{
public:	
	/**
	 * Gets the current version of Seek table. This can be used in the DDC key.
	 * @return int16 Seek-table version
	 */
	static ENGINE_API int16 GetVersion();

	/**
	 * Parse a seek-table from memory.
	 *
	 * @param InMemory Pointer to memory where to load from.
	 * @param InSize Size of memory
	 * @param InOutOffset Output the offset of the end of the table
	 * @param OutTable Table that's loaded
	 * @return true on Success, false on Failure
	 */
	static ENGINE_API bool Parse(const uint8* InMemory, uint32 InSize, uint32& InOutOffset, FStreamedAudioChunkSeekTable& OutTable);

	/**
	 * Calculate the size needed for the table. 
	 * 
	 * This is cheaper than serializing the entire struct and counting the bytes.
	 *
	 * @param InMode The Mode the table is running in (i.e. constant/variable)
	 * @return The Size of the Table in bytes including any header.
	 */
	static ENGINE_API int32 CalcSize(int32 NumEntries, EChunkSeekTableMode InMode = EChunkSeekTableMode::ConstantSamplesPerEntry);

	/**
	 * Member version. Calculate the size needed for the table.	 	 
	 *	 
	 * @return The Size of the Table in bytes including any header.
	 */
	ENGINE_API int32 CalcSize() const;
	
	struct ISeekTableImpl
	{
		virtual ~ISeekTableImpl() = default;
		virtual int32 Num() const = 0;
		virtual uint32 FindOffset(uint32 InTimeInAudioFrames) const = 0;
		virtual uint32 FindTime(uint32 InOffset) const = 0;
		virtual void Add(uint32 InTimeInAudioFrames, uint32 InOffset) = 0;
		virtual bool Serialize(FArchive& Ar) = 0;
		virtual bool GetAt(const uint32 InIndex, uint32& OutOffset, uint32& OutTime) const = 0;
	};

	/**
	 * Construct a seek-table. The mode determines the internal representation.
	 *
	 * @param InMode Mode the table will operate in.
	 */
	ENGINE_API FStreamedAudioChunkSeekTable(EChunkSeekTableMode InMode = EChunkSeekTableMode::ConstantSamplesPerEntry);	

	/**
	 * Add an item to the seek table.
	 *
	 * @param InTimeInAudioFrames Time of this entry in audio frames
	 * @param InOffset Offset in the file of where this time happens
	 */
	void Add(uint32 InTimeInAudioFrames, uint32 InOffset)
	{
		Impl->Add(InTimeInAudioFrames,InOffset);
	}

	/**
	 * Finds an offset for seeking given a time
	 * 
	 * @param InTimeInAudioFrames Time to search for in audio frames
	 * @return Offset indicated by the table. INDEX_NONE (~0) on error.
	 */
	uint32 FindOffset(uint32 InTimeInAudioFrames) const
	{
		return Impl->FindOffset(InTimeInAudioFrames);
	}

	/**
	 * Finds a time given an offset (reverse look up)
	 *
	 * @param InOffset Offset in the file
	 * @return Time indicated by the table. INDEX_NONE (~0) on error. 
	 */
	uint32 FindTime(uint32 InOffset) const
	{
		return Impl->FindTime(InOffset);
	}
	
	/**
	 * Get the number of Entries in the seek table.
	 * @return Num of Items.
	 */
	int32 Num() const
	{
		return Impl->Num();
	}
	
	/**
	 * Serialize the table to/from the archive
	 *
	 * @param Ar Archive to load/save
	 * @return True on success, false on failure.
	 */
	ENGINE_API bool Serialize(FArchive& Ar);

	/**
	 * Empties the table
	 *
	 */
	void Reset();

	/**
	 * Retrieves the Indexed item from the table
	 * @param InIndex Index of item to retrieve (if out of bounds will fail)
	 * @param OutOffset The found offset (if successful)
	 * @param OutTime The found time (if successful)
	 * @return True on Success, False otherwise.
	 */
	bool GetAt(const uint32 InIndex, uint32& OutOffset, uint32& OutTime) const
	{
		return Impl->GetAt(InIndex, OutOffset, OutTime);
	}

private:

	static uint32 GetMagic();
	static TUniquePtr<ISeekTableImpl> CreateImpl(EChunkSeekTableMode InMode);

	void SetMode(EChunkSeekTableMode InMode);
	
	TUniquePtr<ISeekTableImpl> Impl;
	EChunkSeekTableMode Mode = EChunkSeekTableMode::ConstantSamplesPerEntry;
};

