// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/VolumeTexture.h"

#include "Containers/Array.h"
#include "Containers/StaticArray.h"
#include "Engine/TextureDefines.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Serialization/EditorBulkData.h"
#include "UnrealClient.h"
#include "UObject/ObjectSaveContext.h"

#include "SparseVolumeTexture.generated.h"

namespace UE { namespace Shader	{ enum class EValueType : uint8; } }
namespace UE { namespace DerivedData { class FRequestOwner; } }

// SVT_TODO: Unify with macros in SparseVolumeTextureCommon.ush
#define SPARSE_VOLUME_TILE_RES 16
#define SPARSE_VOLUME_TILE_BORDER 1
#define SPARSE_VOLUME_TILE_RES_PADDED (SPARSE_VOLUME_TILE_RES + 2 * SPARSE_VOLUME_TILE_BORDER)

namespace UE
{
namespace SVT
{

struct FTextureData;
class FStreamingManager;

struct FHeader
{
	FIntVector3 VirtualVolumeResolution = FIntVector3(0, 0, 0);
	FIntVector3 VirtualVolumeAABBMin = FIntVector3(INT32_MAX, INT32_MAX, INT32_MAX);
	FIntVector3 VirtualVolumeAABBMax = FIntVector3(INT32_MIN, INT32_MIN, INT32_MIN);
	FIntVector3 PageTableVolumeResolution = FIntVector3(0, 0, 0);
	FIntVector3 PageTableVolumeAABBMin = FIntVector3(INT32_MAX, INT32_MAX, INT32_MAX);
	FIntVector3 PageTableVolumeAABBMax = FIntVector3(INT32_MIN, INT32_MIN, INT32_MIN);
	TStaticArray<EPixelFormat, 2> AttributesFormats = TStaticArray<EPixelFormat, 2>(InPlace, PF_Unknown);
	TStaticArray<FVector4f, 2> FallbackValues = TStaticArray<FVector4f, 2>(InPlace, FVector4f(0.0f, 0.0f, 0.0f, 0.0f));

	FHeader() = default;
	ENGINE_API FHeader(const FIntVector3& AABBMin, const FIntVector3& AABBMax, EPixelFormat FormatA, EPixelFormat FormatB, const FVector4f& FallbackValueA, const FVector4f& FallbackValueB);

	// PageTableVolumeAABBMin needs to be aligned to a power of two such that (PageTableVolumeAABBMin / pow(2, MipLevel)) results in an integer value for all mip levels. Otherwise we could end up
	// with higher mip levels that are shifted in world space (due to PageTableVolumeAABBMin getting rounded down for every mip level) and would therefore not have corresponding voxels for voxels of lower mip levels.
	// Such a shifted page table mip level causes clearly visible artifacts because the volume looks cut off where the page table ends.
	// This problem can be avoided by aligning PageTableVolumeAABBMin to pow(2, (NumMipLevelsGlobal - 1)), where NumMipLevelsGlobal is the maximum number of mip levels in the entire animated SVT sequence.
	ENGINE_API void UpdatePageTableFromGlobalNumMipLevels(int32 NumMipLevelsGlobal);
	ENGINE_API bool Validate(bool bPrintToLog);
};

// Linear struct of arrays octree describing the mipped SVT topology
struct FPageTopology
{
	struct FMip
	{
		uint32 PageOffset; // Offset in Pages array where the pages for this mip begin
		uint32 PageCount; // Number of pages in this mip
	};

	TArray<FMip> MipInfo;
	TArray<uint32> PackedPageTableCoords; // 11|11|10 packed coords of the page within the dense page table. One entry per (sparse) page.
	TArray<uint32> TileIndices; // Index of tile data the page points to. One entry per (sparse) page.
	TArray<uint32> ParentIndices; // Parent index. One entry per (sparse) page.

	void Reset();
	void Serialize(FArchive& Ar);
	uint32 NumPages() const { return PackedPageTableCoords.Num(); }
	bool IsValidPageIndex(uint32 PageIndex) const { return PackedPageTableCoords.IsValidIndex(PageIndex); }
	void GetTileRange(uint32 PageOffset, uint32 PageCount, uint32& OutTileOffset, uint32& OutTileCount) const;
};

// Describes a mip level of a SVT frame in terms of the sizes and offsets of the data in the built bulk data.
// Each mip level consists of 4 buffer sections:
// 
// Page table data is stored as two consecutive uint32 arrays, where the first array stores packed coordinates into the page table.
// The second array stores the physical tile indices to be written to the page table. Only non-zero page table entries are stored.
//
// Since often about 30%-50% of all voxels in the physical tiles have values equal to the fallback value, the tile data (voxels) is also compressed.
// For each tile, a bit mask with one bit per voxel is stored (occupancy bits). A set bit indicates that the voxel is actually stored and not equal
// to the fallback value. When streaming in the data, the compressed voxel data is expanded on the GPU in the upload shader.
// This bit mask costs 183 uint32 -> 732 bytes. A full 18x18x18 tile (16+2 padding) of 8bit unorm data is 5832 bytes, so if 30%-50% of that can be saved,
// then paying 732 bytes for the occupancy bits should almost always be worth it.
// 
// Due to the above compression technique, the start offset of the voxel data for a given tile can now no longer be computed as NumVoxelsPerPaddedTile * TileIndex,
// so a precomputed offset into the shared array of voxel data is needed (tile data offsets). This means that we need to store one additional uint32 per tile,
// arriving at a total overhead of 736 bytes per tile.
// 
// Finally there is the actual voxel data, which is simply the all non-fallback voxels in contiguous memory.
// 
struct FMipLevelStreamingInfo
{
	int32 BulkOffset;
	int32 BulkSize;
	int32 PageTableOffset; // relative to BulkOffset
	int32 PageTableSize;
	TStaticArray<int32, 2> OccupancyBitsOffset; // relative to BulkOffset
	TStaticArray<int32, 2> OccupancyBitsSize;
	TStaticArray<int32, 2> TileDataOffsetsOffset; // relative to BulkOffset. Per-tile offset into this mip levels voxel data
	TStaticArray<int32, 2> TileDataOffsetsSize;
	TStaticArray<int32, 2> TileDataOffset; // relative to BulkOffset
	TStaticArray<int32, 2> TileDataSize;
	int32 NumPhysicalTiles;
};

// All the tiles are essentially stored as an array of structs with each tile having different memory sections:
// 
//    | OccupancyBitsA | OccupancyBitsB | VoxelsA | VoxelsB |
// 
// OccupancyBitsA and OccupancyBitsB will only be stored if the respective texture/attributes group exists (Format is != Unknown).
// Each set of occupancy bits has a fixed size, so it doesn't need to be stored explicitely.
// VoxelsA and VoxelsB are the compacted non-fallback-value voxels of the tile. The sizes of these sections varies with the number of active
// voxels in the tile (and in each texture).

struct FTileInfo
{
	uint32 Offset;
	uint32 Size;
	TStaticArray<uint32, 2> OccupancyBitsOffsets; // Relative to Offset
	TStaticArray<uint32, 2> OccupancyBitsSizes; // Relative to Offset
	TStaticArray<uint32, 2> VoxelDataOffsets; // Relative to Offset
	TStaticArray<uint32, 2> VoxelDataSizes;
	TStaticArray<uint32, 2> NumVoxels;
};

// Compactly stores data to construct a FTileInfo for every stored (and compressed) tile.
struct FTileStreamingMetaData
{
	// Array of byte offsets into the tile data. Has N+1 entries, with the last entry effectively being the total size of all tiles. This is a "logical" offset
	// so we can compute the size of each tile as TileDataOffsets[N+1] - TileDataOffsets[N]. Tiles < FirstStreamingTileIndex are not actually stored with the rest of the
	// streaming tiles, so to get the actual file offset of a streaming tile, the size of the root/non-streaming tile needs to be subtracted first. GetTileInfo() takes this into account.
	TArray<uint32> TileDataOffsets;
	TArray<uint16> NumVoxelsA; // Number of stored voxels for texture A for every tile. We reconstruct the number of voxels for texture B based on NumVoxelsA and the tile size.
	uint32 FirstStreamingTileIndex; // Index of first tile that does not belong to the root mip level. Root tiles are always in memory/resident and do not stream.

	void Reset()
	{
		TileDataOffsets.Reset();
		NumVoxelsA.Reset();
		FirstStreamingTileIndex = 0;
	}

	uint32 GetNumTiles() const
	{
		return NumVoxelsA.Num();
	}

	uint32 GetNumStreamingTiles() const
	{
		return GetNumTiles() - FirstStreamingTileIndex;
	}

	bool HasRootTile() const
	{
		return FirstStreamingTileIndex > 0;
	}

	uint32 GetRootTileSize() const
	{
		return TileDataOffsets[FirstStreamingTileIndex] - TileDataOffsets[0];
	}

	uint32 GetTileMemorySize(uint32 TileIndex) const
	{
		return TileDataOffsets[TileIndex + 1] - TileDataOffsets[TileIndex];
	}

	void GetTileRangeMemoryOffsetSize(uint32 TileOffset, uint32 TileCount, uint32& OutMemoryOffset, uint32& OutMemorySize) const
	{
		checkf(!(TileOffset < FirstStreamingTileIndex && (TileOffset + TileCount) > FirstStreamingTileIndex), TEXT("Tile range must not straddle the root tile and streaming tiles!"));
		const uint32 RootTileSize = TileOffset < FirstStreamingTileIndex ? 0 : GetRootTileSize();
		OutMemoryOffset = TileDataOffsets[TileOffset] - RootTileSize;
		const uint32 ReadEnd = TileDataOffsets[TileOffset + TileCount] - RootTileSize;
		OutMemorySize = ReadEnd - OutMemoryOffset;
	}

	FTileInfo GetTileInfo(uint32 TileIndex, uint32 FormatSizeA, uint32 FormatSizeB) const;

	void GetNumVoxelsInTileRange(uint32 TileOffset, uint32 TileCount, uint32 FormatSizeA, uint32 FormatSizeB, const TBitArray<>* OptionalValidTiles, uint32& OutNumVoxelsA, uint32& OutNumVoxelsB) const;
};

enum EResourceFlag : uint32
{
	EResourceFlag_StreamingDataInDDC = 1 << 0u, // FResources was cached, so MipLevelStreamingInfo can be streamed from DDC
};

// Represents the derived data of a SVT that is needed by the streaming manager.
struct FResources
{
public:
	FHeader Header;
	uint32 ResourceFlags = 0;
	int32 NumMipLevels = 0;
	// Info about offsets into the tile data. 
	FTileStreamingMetaData StreamingMetaData;
	// Data for the highest/"root" mip level
	TArray<uint8> RootData;
	// Data for all streamable mip levels
	FByteBulkData StreamableMipLevels;

	FPageTopology Topology;

	// These are used for logging and retrieving StreamableMipLevels from DDC in FStreamingManager
#if WITH_EDITORONLY_DATA
	FString ResourceName;
	FIoHash DDCKeyHash;
	TArray<TStaticArray<uint8, 12>> DDCChunkIds; // 12-byte hashes used to reconstruct the FValueId required to look up the chunks.
	TArray<uint32> DDCChunkMaxTileIndices; // Maximum tile index of each chunk. Used to look up the chunk a given tile belongs to.
#endif

	// Called when serializing to/from DDC buffers and when serializing the owning USparseVolumeTextureFrame.
	void Serialize(FArchive& Ar, UObject* Owner, bool bCooked);
	// Returns true if there are streamable mip levels.
	bool HasStreamingData() const;
#if WITH_EDITORONLY_DATA
	// Removes the StreamableMipLevels bulk data if it was successfully cached to DDC.
	void DropBulkData();
	bool HasBuildFromDDCError() const;
	void SetHasBuildFromDDCError(bool bHasError);
	// Fills StreamableMipLevels with data from DDC. Returns true when done.
	bool RebuildBulkDataFromCacheAsync(const UObject* Owner, bool& bFailed);
	// Builds all the data from SourceData. Is called by Cache().
	bool Build(USparseVolumeTextureFrame* Owner, UE::Serialization::FEditorBulkData& SourceData);
	// Cache the built data to/from DDC. If bLocalCachingOnly is true, the read/write queries will only use the local DDC; otherwise the remote DDC will also be used.
	void Cache(USparseVolumeTextureFrame* Owner, UE::Serialization::FEditorBulkData& SourceData, bool bLocalCachingOnly);
	// Sets empty default data. This is used when caching/building is canceled but some form of valid data is needed.
	void SetDefault(EPixelFormat FormatA, EPixelFormat FormatB, const FVector4f& FallbackValueA, const FVector4f& FallbackValueB);
#endif

private:
#if WITH_EDITORONLY_DATA
	enum class EDDCRebuildState : uint8
	{
		Initial,
		InitialAfterFailed,
		Pending,
		Succeeded,
		Failed,
	};
	static bool IsInitialState(EDDCRebuildState State)
	{
		return State == EDDCRebuildState::Initial || State == EDDCRebuildState::InitialAfterFailed;
	}
	TDontCopy<TPimplPtr<UE::DerivedData::FRequestOwner>> DDCRequestOwner;
	std::atomic<EDDCRebuildState> DDCRebuildState;
	std::atomic_int DDCRebuildNumFinishedRequests;
	void BeginRebuildBulkDataFromCache(const UObject* Owner);
	void EndRebuildBulkDataFromCache();
#endif

	static FTileStreamingMetaData CompressTiles(const FPageTopology& Topology, const struct FDerivedTextureData& DerivedTextureData, TArray<uint8>& OutRootBulkData, TArray64<uint8>& OutStreamingBulkData);
};

// Encapsulates RHI resources needed to render a SparseVolumeTexture.
class FTextureRenderResources : public ::FRenderResource
{
	friend class FStreamingManager;
public:
	const FHeader& GetHeader() const							{ check(IsInParallelRenderingThread()); return Header; }
	FIntVector3 GetTileDataTextureResolution() const			{ check(IsInParallelRenderingThread()); return TileDataTextureResolution; }
	int32 GetFrameIndex() const									{ check(IsInParallelRenderingThread()); return FrameIndex; }
	int32 GetNumLogicalMipLevels() const						{ check(IsInParallelRenderingThread()); return NumLogicalMipLevels; }
	FRHITextureReference* GetPageTableTexture() const			{ check(IsInParallelRenderingThread()); return PageTableTextureReferenceRHI.GetReference(); }
	FRHITextureReference* GetPhysicalTileDataATexture() const	{ check(IsInParallelRenderingThread()); return PhysicalTileDataATextureReferenceRHI.GetReference(); }
	FRHITextureReference* GetPhysicalTileDataBTexture() const	{ check(IsInParallelRenderingThread()); return PhysicalTileDataBTextureReferenceRHI.GetReference(); }
	ENGINE_API void GetPackedUniforms(FUintVector4& OutPacked0, FUintVector4& OutPacked1) const;
	// Updates the GlobalVolumeResolution member in a thread-safe way.
	ENGINE_API void SetGlobalVolumeResolution_GameThread(const FIntVector3& GlobalVolumeResolution);

	//~ Begin FRenderResource Interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface.

private:
	FHeader Header;
	FIntVector3 GlobalVolumeResolution = FIntVector3::ZeroValue; // The virtual resolution of the union of the AABBs of all frames. Needed for GetPackedUniforms().
	FIntVector3 TileDataTextureResolution = FIntVector3::ZeroValue;
	int32 FrameIndex = INDEX_NONE;
	int32 NumLogicalMipLevels = 0; // Might not all be resident in GPU memory
	FTextureReferenceRHIRef PageTableTextureReferenceRHI;
	FTextureReferenceRHIRef PhysicalTileDataATextureReferenceRHI;
	FTextureReferenceRHIRef PhysicalTileDataBTextureReferenceRHI;
};

}
}

FArchive& operator<<(FArchive& Ar, UE::SVT::FHeader& Header);
FArchive& operator<<(FArchive& Ar, UE::SVT::FPageTopology::FMip& Mip);

enum ESparseVolumeTextureShaderUniform
{
	ESparseVolumeTexture_TileSize,
	ESparseVolumeTexture_PageTableSize,
	ESparseVolumeTexture_UVScale,
	ESparseVolumeTexture_UVBias,
	ESparseVolumeTexture_Count,
};

// SparseVolumeTexture base interface to communicate with material graph and shader bindings.
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class USparseVolumeTexture : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	ENGINE_API USparseVolumeTexture();
	virtual ~USparseVolumeTexture() = default;

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual int32 GetSizeX() const { return GetVolumeResolution().X; }

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual int32 GetSizeY() const { return GetVolumeResolution().Y; }

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual int32 GetSizeZ() const { return GetVolumeResolution().Z; }

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual int32 GetNumFrames() const { return 0; }

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual int32 GetNumMipLevels() const { return 0; }

	UFUNCTION(BlueprintCallable, Category = "Texture")
	virtual FTransform GetFrameTransform() const { return FTransform::Identity; }

	virtual FIntVector GetVolumeResolution() const { return FIntVector(); }
	virtual EPixelFormat GetFormat(int32 AttributesIndex) const { return PF_Unknown; }
	virtual FVector4f GetFallbackValue(int32 AttributesIndex) const { return FVector4f(); }
	virtual TextureAddress GetTextureAddressX() const { return TA_Wrap; }
	virtual TextureAddress GetTextureAddressY() const { return TA_Wrap; }
	virtual TextureAddress GetTextureAddressZ() const { return TA_Wrap; }
	virtual const UE::SVT::FTextureRenderResources* GetTextureRenderResources() const { return nullptr; }
	// Computes the optimal mip level to stream the SVT at, based on projected screen space size and voxel resolution.
	float GetOptimalStreamingMipLevel(const FBoxSphereBounds& Bounds, float MipBias) const;

	/** Getter for the shader uniform parameters with index as ESparseVolumeTextureShaderUniform. */
	FVector4 GetUniformParameter(int32 Index) const { return FVector4(ForceInitToZero); } // SVT_TODO: This mechanism is no longer needed and can be removed

	/** Getter for the shader uniform parameter type with index as ESparseVolumeTextureShaderUniform. */
	static ENGINE_API UE::Shader::EValueType GetUniformParameterType(int32 Index);

#if WITH_EDITOR
	enum class ENotifyMaterialsEffectOnShaders
	{
		Default,
		DoesNotInvalidate
	};

	/** Notify any loaded material instances that the texture has changed. */
	ENGINE_API void NotifyMaterials(const ENotifyMaterialsEffectOnShaders EffectOnShaders = ENotifyMaterialsEffectOnShaders::Default);
#endif // WITH_EDITOR
};

// Represents a frame in a SparseVolumeTexture sequence and owns the actual data needed for rendering. Is owned by a UStreamableSparseVolumeTexture object.
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class USparseVolumeTextureFrame : public USparseVolumeTexture
{
	GENERATED_UCLASS_BODY()

	friend class UE::SVT::FStreamingManager;
	friend class UStreamableSparseVolumeTexture;

public:

	ENGINE_API USparseVolumeTextureFrame();
	virtual ~USparseVolumeTextureFrame() = default;

	// Retrieves a frame from the given SparseVolumeTexture and also issues a streaming request for it. 
	// StreamingInstanceKey can be any arbitrary value that is suitable to keep track of the source of requests for a given SVT. This key is used internally to associate
	// incoming requests with prior requests issued for the same SVT. A good value to pass here might be the pointer of the component the SVT is used within.
	// FrameRate is an optional argument which helps to more accurately predict the required bandwidth when using non-blocking requests.
	// FrameIndex is of float type so that the streaming system can use the fractional part to more easily keep track of playback speed and direction (forward/reverse playback).
	// MipLevel is the lowest mip level that the caller intends to use but does not guarantee that the mip is actually resident.
	// If bBlocking is true, DDC streaming requests will block on completion, guaranteeing that the requested frame will have been streamed in after the next streaming system update.
	// If streaming cooked data from disk, the highest priority will be used, but no guarantee is given.
	// if bHasValidFrameRate is true, the FrameRate argument will be use to predict the required streaming IO bandwidth.
	static ENGINE_API USparseVolumeTextureFrame* GetFrameAndIssueStreamingRequest(USparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, bool bBlocking, bool bHasValidFrameRate);

	ENGINE_API bool Initialize(USparseVolumeTexture* InOwner, int32 InFrameIndex, const FTransform& InFrameTransform, UE::SVT::FTextureData& UncookedFrame);
	int32 GetFrameIndex() const { return FrameIndex; }
	UE::SVT::FResources* GetResources() { return &Resources; }
	// Creates TextureRenderResources if they don't already exist. Returns false if they already existed.
	ENGINE_API bool CreateTextureRenderResources();

#if WITH_EDITORONLY_DATA
	// Caches the derived data (FResources) of this frame to/from DDC and ensures that FTextureRenderResources exists.
	ENGINE_API void Cache(bool bSkipDDCAndSetResourcesToDefault);
#endif

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITOR
	ENGINE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void WillNeverCacheCookedPlatformDataAgain() override;
	ENGINE_API virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
#endif
	//~ End UObject Interface.

	//~ Begin USparseVolumeTexture Interface.
	virtual int32 GetNumFrames() const override { return 1; }
	virtual int32 GetNumMipLevels() const override { return Owner->GetNumMipLevels(); }
	virtual FTransform GetFrameTransform() const override { return Transform; }
	virtual FIntVector GetVolumeResolution() const override { return Owner->GetVolumeResolution(); }
	virtual EPixelFormat GetFormat(int32 AttributesIndex) const override { return Owner->GetFormat(AttributesIndex); }
	virtual FVector4f GetFallbackValue(int32 AttributesIndex) const override { return Owner->GetFallbackValue(AttributesIndex); }
	virtual TextureAddress GetTextureAddressX() const override { return Owner->GetTextureAddressX(); }
	virtual TextureAddress GetTextureAddressY() const override { return Owner->GetTextureAddressY(); }
	virtual TextureAddress GetTextureAddressZ() const override { return Owner->GetTextureAddressZ(); }
	virtual const UE::SVT::FTextureRenderResources* GetTextureRenderResources() const override { return TextureRenderResources; }
	//~ End USparseVolumeTexture Interface.

private:

	UPROPERTY()
	TObjectPtr<USparseVolumeTexture> Owner;

	UPROPERTY()
	int32 FrameIndex;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	FTransform Transform;

#if WITH_EDITORONLY_DATA
	// FTextureData from which the FResources data can be built with a call to FResources::Build()
	UE::Serialization::FEditorBulkData SourceData;
#endif

	// Derived data used at runtime
	UE::SVT::FResources Resources;
	// Runtime render data
	UE::SVT::FTextureRenderResources* TextureRenderResources;
};

// Represents a streamable SparseVolumeTexture asset and serves as base class for UStaticSparseVolumeTexture and UAnimatedSparseVolumeTexture. It has an array of USparseVolumeTextureFrame.
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class UStreamableSparseVolumeTexture : public USparseVolumeTexture, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	FIntVector VolumeResolution;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	int32 NumMipLevels;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	int32 NumFrames;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	TEnumAsByte<enum EPixelFormat> FormatA;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AssetRegistrySearchable)
	TEnumAsByte<enum EPixelFormat> FormatB;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AdvancedDisplay)
	FVector4f FallbackValueA;

	UPROPERTY(VisibleAnywhere, Category = "Texture", AdvancedDisplay)
	FVector4f FallbackValueB;

	// The addressing mode to use for the X axis.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (DisplayName = "X-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressX;

	// The addressing mode to use for the Y axis.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (DisplayName = "Y-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressY;

	// The addressing mode to use for the Z axis.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (DisplayName = "Z-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<enum TextureAddress> AddressZ;

	// If enabled, the SparseVolumeTexture is only going to use the local DDC. For certain assets it might be reasonable to also use the remote DDC, but for larger assets this will mean long up- and download times.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", AdvancedDisplay)
	bool bLocalDDCOnly = true;

	// The SVT streaming pool is sized such that it can hold the largest frame multiplied by this value. There should be some slack to allow for prefetching frames.
	UPROPERTY(EditAnywhere, Category = "Texture", AdvancedDisplay)
	float StreamingPoolSizeFactor = 3.0f;

	// When using non-blocking streaming requests, upcoming frames are loaded into memory in advance. This property controls how many frames to prefetch.
	UPROPERTY(EditAnywhere, Category = "Texture", AdvancedDisplay)
	int32 NumberOfPrefetchFrames = 3;

	// When using non-blocking streaming requests, upcoming frames are loaded into memory in advance. This property controls the size reduction in percent of each additional prefetched frames.
	// A value of 20.0 would prefetch frame N+1 at 80%, N+2 at 60%, N+3 at 40% etc.
	UPROPERTY(EditAnywhere, Category = "Texture", AdvancedDisplay)
	float PrefetchPercentageStepSize = 20.0f;

	// When using non-blocking streaming requests, upcoming frames are loaded into memory in advance. This property applies a bias in percent to how much data is prefetched for every frame.
	// A value of 20.0 adds 20% to all prefetch percentages. So if PrefetchPercentageStepSize is set to 20.0, frame N+1 is prefetched at 80% + 20% = 100%, frame N+2 at 60% + 20% = 80%, N+3 at 40% + 20% = 60% etc.
	UPROPERTY(EditAnywhere, Category = "Texture", AdvancedDisplay)
	float PrefetchPercentageBias = 20.0f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif // WITH_EDITORONLY_DATA

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = "Texture")
	TArray<TObjectPtr<class UAssetUserData>> AssetUserData;

	UStreamableSparseVolumeTexture();
	virtual ~UStreamableSparseVolumeTexture() = default;

	// Multi-phase initialization: Call BeginInitialize(), then call AppendFrame() for each frame to add and then finish initialization with a call to EndInitialize().
	// The NumExpectedFrames parameter on BeginInitialize() just serves as a potential optimization to reserve memory for the frames to be appended
	// and doesn't need to match the exact number if it is not known at the time.
	ENGINE_API virtual bool BeginInitialize(int32 NumExpectedFrames);
	ENGINE_API virtual bool AppendFrame(UE::SVT::FTextureData& UncookedFrame, const FTransform& FrameTransform);
	ENGINE_API virtual bool EndInitialize();

	// Convenience function wrapping the multi-phase initialization functions above
	virtual bool Initialize(const TArrayView<UE::SVT::FTextureData>& UncookedData, const TArrayView<FTransform>& FrameTransforms);
	// Consider using USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest() if the frame should have streaming requests issued.
	USparseVolumeTextureFrame* GetFrame(int32 FrameIndex) const { return Frames.IsValidIndex(FrameIndex) ? Frames[FrameIndex] : nullptr; }

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~ End UObject Interface.

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	//~ Begin USparseVolumeTexture Interface.
	virtual int32 GetNumFrames() const override { return Frames.Num(); }
	virtual int32 GetNumMipLevels() const override { return NumMipLevels; }
	virtual FTransform GetFrameTransform() const override { return Frames.IsEmpty() ? FTransform::Identity : Frames[0]->GetFrameTransform(); }
	virtual FIntVector GetVolumeResolution() const override { return VolumeResolution; };
	virtual EPixelFormat GetFormat(int32 AttributesIndex) const override { check(AttributesIndex >= 0 && AttributesIndex < 2) return AttributesIndex == 0 ? FormatA : FormatB; }
	virtual FVector4f GetFallbackValue(int32 AttributesIndex) const override { check(AttributesIndex >= 0 && AttributesIndex < 2) return AttributesIndex == 0 ? FallbackValueA : FallbackValueB; }
	virtual TextureAddress GetTextureAddressX() const override { return AddressX; }
	virtual TextureAddress GetTextureAddressY() const override { return AddressY; }
	virtual TextureAddress GetTextureAddressZ() const override { return AddressZ; }
	virtual const UE::SVT::FTextureRenderResources* GetTextureRenderResources() const override { return Frames.IsEmpty() ? nullptr : Frames[0]->GetTextureRenderResources(); }
	//~ End USparseVolumeTexture Interface.

#if WITH_EDITOR
	UE_DEPRECATED(5.5, "OnAssetsAddExtraObjectsToDelete signature has been deprecated please use OnAddExtraObjectsToDelete instead")
	ENGINE_API void OnAssetsAddExtraObjectsToDelete(TArray<UObject*>& ObjectsToDelete);
	ENGINE_API void OnAddExtraObjectsToDelete(const TArray<UObject*>& InObjectsToDelete, TSet<UObject*>& OutSecondaryObjects);
#endif

protected:

	UPROPERTY(Export)
	TArray<TObjectPtr<USparseVolumeTextureFrame>> Frames;

#if WITH_EDITORONLY_DATA

	enum EInitState : uint8
	{
		EInitState_Uninitialized,
		EInitState_Pending,
		EInitState_Done,
		EInitState_Failed,
	};
	
	UPROPERTY()
	FIntVector VolumeBoundsMin;
	
	UPROPERTY()
	FIntVector VolumeBoundsMax;
	
	UPROPERTY()
	uint8 InitState = EInitState_Uninitialized;

	// Ensures all frames have derived data (based on the source data and the current settings like TextureAddress modes etc.) cached to DDC and are ready for rendering.
	// Disconnects this SVT from the streaming manager, calls Cache() on all frames and finally connects to FStreamingManager again.
	void RecacheFrames();

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API bool ShouldRegisterDelegates();
	ENGINE_API void RegisterEditorDelegates();
	ENGINE_API void UnregisterEditorDelegates();
#endif // WITH_EDITOR
};

// Represents a streamable SparseVolumeTexture asset with a single frame. Although there is only a single frame, it is still recommended to use USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest().
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class UStaticSparseVolumeTexture : public UStreamableSparseVolumeTexture
{
	GENERATED_UCLASS_BODY()

public:

	ENGINE_API UStaticSparseVolumeTexture();
	virtual ~UStaticSparseVolumeTexture() = default;

	//~ Begin UStreamableSparseVolumeTexture Interface.
	// Override AppendFrame() to ensure that there is never more than a single frame in a static SVT
	ENGINE_API virtual bool AppendFrame(UE::SVT::FTextureData& UncookedFrame, const FTransform& InFrameTransform) override;
	//~ End UStreamableSparseVolumeTexture Interface.

	//~ Begin USparseVolumeTexture Interface.
	int32 GetNumFrames() const override { return 1; }
	//~ End USparseVolumeTexture Interface.

private:
};

// Represents a streamable SparseVolumeTexture with one or more frames. Use USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest() to bind extract a particular frame to be used for rendering.
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class UAnimatedSparseVolumeTexture : public UStreamableSparseVolumeTexture
{
	GENERATED_UCLASS_BODY()

public:

	ENGINE_API UAnimatedSparseVolumeTexture();
	virtual ~UAnimatedSparseVolumeTexture() = default;

	//~ Begin USparseVolumeTexture Interface.
	virtual const UE::SVT::FTextureRenderResources* GetTextureRenderResources() const override { return Frames.IsValidIndex(PreviewFrameIndex) ? Frames[PreviewFrameIndex]->GetTextureRenderResources() : nullptr; }
	//~ End USparseVolumeTexture Interface.

private:
	int32 PreviewFrameIndex;
};

// Utility (blueprint) class for controlling SparseVolumeTexture playback.
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class UAnimatedSparseVolumeTextureController : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(BlueprintReadWrite, Category = "Animation")
	TObjectPtr<USparseVolumeTexture> SparseVolumeTexture;
	
	UPROPERTY(BlueprintReadWrite, Category = "Animation")
	float Time;

	UPROPERTY(BlueprintReadWrite, Category = "Animation")
	bool bIsPlaying;

	UPROPERTY(BlueprintReadWrite, Category = "Animation")
	float FrameRate = 24.0f;

	UPROPERTY(BlueprintReadWrite, Category = "Rendering")
	int32 MipLevel = 0;

	UPROPERTY(BlueprintReadWrite, Category = "Rendering")
	bool bBlockingStreamingRequests = false;

	ENGINE_API UAnimatedSparseVolumeTextureController();
	virtual ~UAnimatedSparseVolumeTextureController() = default;

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void Play();

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void Pause();

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void Stop();

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void Update(float DeltaTime);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API float GetFractionalFrameIndex();

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API USparseVolumeTextureFrame* GetFrameByIndex(int32 FrameIndex);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API USparseVolumeTextureFrame* GetCurrentFrame();

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API void GetCurrentFramesForInterpolation(USparseVolumeTextureFrame*& Frame0, USparseVolumeTextureFrame*& Frame1, float& LerpAlpha);

	UFUNCTION(BlueprintCallable, Category = "Animation")
	ENGINE_API float GetDuration();
};
