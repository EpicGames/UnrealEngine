// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AutomationTestSettings.h: Declares the UAutomationTestSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "UObject/SoftObjectPath.h"
#include "AutomationTestSettings.generated.h"


/**
 * Structure for defining an external tool
 */
USTRUCT()
struct FExternalToolDefinition
{
	GENERATED_USTRUCT_BODY()

	/* The name of the tool / test. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools)
	FString ToolName;

	/* The executable to run. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools, meta=(FilePathFilter = "*"))
	FFilePath ExecutablePath;

	/* The command line options to pass to the executable. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools)
	FString CommandLineOptions;

	/* The working directory for the new process. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools, meta=(RelativePath))
	FDirectoryPath WorkingDirectory;

	/* If set, look for scripts with this extension. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools)
	FString ScriptExtension;

	/* If the ScriptExtension is set, look here for the script files. */
	UPROPERTY(config, EditAnywhere, Category=ExternalTools, meta=(RelativePath))
	FDirectoryPath ScriptDirectory;
};

/**
 * Holds FProperty names and values to customize factory settings
 */
USTRUCT()
struct FImportFactorySettingValues
{
	GENERATED_USTRUCT_BODY()

	/* The name of the FProperty to change */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (ToolTip = "Name of the property to change.  Nested settings can be modified using \"Outer.Property\""))
	FString SettingName;

	/* The value to apply to the FProperty */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (ToolTip = "Value to import for the specified property."))
	FString Value;
};


/**
 * Holds settings for the asset import / export automation test
 */
USTRUCT()
struct FEditorImportExportTestDefinition
{
	GENERATED_USTRUCT_BODY()

	/* The file to import */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "*"))
	FFilePath ImportFilePath;

	/* The file extension to use when exporting */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (ToolTip = "The file extension to use when exporting this asset.  Used to find a supporting exporter"))
	FString ExportFileExtension;

	/* If true, the export step will be skipped */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	bool bSkipExport = false;

	/* Settings for the import factory */
	UPROPERTY(config, EditAnywhere, Category=Automation)
	TArray<FImportFactorySettingValues> FactorySettings;
};

/**
* Holds settings for the asset import workflow test
*/
USTRUCT()
struct FEditorImportWorkflowDefinition
{
	GENERATED_USTRUCT_BODY()

	/* The file to import */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "*"))
	FFilePath ImportFilePath;

	/* Settings for the import factory */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	TArray<FImportFactorySettingValues> FactorySettings;
};

/**
* Holds settings for the import workflow stage of the build promotion test
*/
USTRUCT()
struct FBuildPromotionImportWorkflowSettings
{
	GENERATED_USTRUCT_BODY()

	/* Import settings for the Diffuse texture */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition Diffuse;

	/* Import settings for the Normalmap texture */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition Normal;

	/* Import settings for the static mesh */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition StaticMesh;

	/* Import settings for the static mesh to re-import */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition ReimportStaticMesh;

	/* Import settings for the blend shape */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition BlendShapeMesh;

	/* Import settings for the morph mesh */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition MorphMesh;

	/* Import settings for the skeletal mesh */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition SkeletalMesh;

	/* Import settings for the animation asset.  (Will automatically use the skeleton of the skeletal mesh above) */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition Animation;

	/* Import settings for the sound */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition Sound;

	/* Import settings for the surround sound (Select any of the channels.  It will auto import the rest)*/
	UPROPERTY(config, EditAnywhere, Category = Automation)
	FEditorImportWorkflowDefinition SurroundSound;

	/* Import settings for any other assets you may want to import */
	UPROPERTY(config, EditAnywhere, Category = Automation)
	TArray<FEditorImportWorkflowDefinition> OtherAssetsToImport;
};

/**
* Holds settings for the open assets stage of the build promotion test
*/
USTRUCT()
struct FBuildPromotionOpenAssetSettings
{
	GENERATED_USTRUCT_BODY()

	/* The blueprint asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath BlueprintAsset;

	/* The material asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath MaterialAsset;
	
	/* The particle system asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath ParticleSystemAsset;
	
	/* The skeletal mesh asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath SkeletalMeshAsset;
	
	/* The static mesh asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath StaticMeshAsset;
	
	/* The texture asset to open */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath TextureAsset;
};

/**
* Holds settings for the new project stage of the build promotion test
*/
USTRUCT()
struct FBuildPromotionNewProjectSettings
{
	GENERATED_USTRUCT_BODY()

	/** The path for the new project */
	UPROPERTY(EditAnywhere, Category = Automation)
	FDirectoryPath NewProjectFolderOverride;

	/** The name of the project **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FString NewProjectNameOverride;
};

/**
* Holds settings for the material editor build promotion tests
*/
USTRUCT()
struct FMaterialEditorPromotionSettings
{
	GENERATED_USTRUCT_BODY()

	/** Default material asset to apply to static meshes **/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath DefaultMaterialAsset;

	/** Default material asset to apply to static meshes **/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath DefaultDiffuseTexture;

	/** Default material asset to apply to static meshes **/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath DefaultNormalTexture;

};


/**
* Holds settings for the particle editor build promotion tests
*/
USTRUCT()
struct FParticleEditorPromotionSettings
{
	GENERATED_USTRUCT_BODY()

	/** Default particle asset to use for tests*/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath DefaultParticleAsset;
};

/**
* Holds settings for the blueprint editor build promotion tests
*/
USTRUCT()
struct FBlueprintEditorPromotionSettings
{
	GENERATED_USTRUCT_BODY()

	/** The starting mesh for the blueprint **/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath FirstMeshPath;

	/** The mesh to set on the blueprint after the delay **/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath SecondMeshPath;

	/** Default particle asset to use for tests*/
	UPROPERTY(EditAnywhere, Category = Automation, meta = (FilePathFilter = "uasset"))
	FFilePath DefaultParticleAsset;
};


/**
* Holds settings for the editor build promotion test
*/
USTRUCT()
struct FBuildPromotionTestSettings
{
	GENERATED_USTRUCT_BODY()

	/** Default static mesh asset to apply materials to **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FFilePath DefaultStaticMeshAsset;

	/** Import workflow settings **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FBuildPromotionImportWorkflowSettings	ImportWorkflow;

	/** Open assets settings **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FBuildPromotionOpenAssetSettings	OpenAssets;

	/** New project settings **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FBuildPromotionNewProjectSettings NewProjectSettings;

	/** Material to modify for the content browser step **/
	UPROPERTY(EditAnywhere, Category = Automation)
	FFilePath	SourceControlMaterial;
};

/**
* Holds settings for the asset import / export automation test
*/
USTRUCT()
struct FEditorMapPerformanceTestDefinition
{
	GENERATED_USTRUCT_BODY()
	
	/** Map to be used for the Performance Capture **/
	UPROPERTY(config, EditAnywhere, Category = Automation, meta=( AllowedClasses="/Script/Engine.World" ))
	FSoftObjectPath PerformanceTestmap;

	/** How long is this test expected to run before stopping **/
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (ToolTip = "This is the length of time in seconds that this test will run for before stopping."))
	int32 TestTimer = 0;
};

/**
* Holds settings for the editor Launch On With Map Iterations test.
*/
USTRUCT()
struct FLaunchOnTestSettings
{
	GENERATED_USTRUCT_BODY()

	/** Map to be used for the Launch On test **/
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter = "umap"))
	FFilePath LaunchOnTestmap;

	/** Device to be used **/
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (ToolTip = "This is the device to be used for launch on. Example: WindowsClient, Android, IOS, Linux"))
	FString DeviceID;
};



/**
 * Implements the Editor's user settings.
 */
UCLASS(config=Engine, defaultconfig, MinimalAPI)
class UAutomationTestSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Modules to load that have engine tests
	 */
	UPROPERTY(EditAnywhere, config, Category = Loading)
	TArray<FString> EngineTestModules;

	/**
	 * Modules to load that have editor tests
	 */
	UPROPERTY(EditAnywhere, config, Category = Loading)
	TArray<FString> EditorTestModules;

	/**
	 * The automation test map to be used for several of the automation tests.
	 */
	UPROPERTY(config, EditAnywhere, Category = Automation, meta=( AllowedClasses="/Script/Engine.World" ))
	FSoftObjectPath AutomationTestmap;

	/**
	* The map to be used for the editor performance capture tool.
	*/
	UPROPERTY(config, EditAnywhere, Category = Automation)
	TArray<FEditorMapPerformanceTestDefinition> EditorPerformanceTestMaps;

	/**
	 * Asset to test for open in automation process
	 */
	UPROPERTY(EditAnywhere, config, Category="Open Asset Tests")
	TArray<FString> AssetsToOpen;

	/**
	 * Maps to PIE during the PIE test
	 */
	UPROPERTY(EditAnywhere, config, Category = "PIE Test Maps")
	TArray<FString> MapsToPIETest;

	/**
	 * Use all Maps from project for PlayMapInPIE test
	 */
	UPROPERTY(EditAnywhere, config, Category = "Play all project Maps In PIE")
	bool bUseAllProjectMapsToPlayInPIE;

	/**
	* Editor build promotion test settings
	*/
	UPROPERTY(EditAnywhere, config, Category = Automation)
	FBuildPromotionTestSettings BuildPromotionTest;

	/**
	* Material editor promotion test settings
	*/
	UPROPERTY(EditAnywhere, config, Category = Automation)
	FMaterialEditorPromotionSettings MaterialEditorPromotionTest;

	/**
	* Particle editor promotion test settings
	*/
	UPROPERTY(EditAnywhere, config, Category = Automation)
	FParticleEditorPromotionSettings ParticleEditorPromotionTest;

	/**
	* Blueprint editor promotion test settings
	*/
	UPROPERTY(EditAnywhere, config, Category = Automation)
	FBlueprintEditorPromotionSettings BlueprintEditorPromotionTest;

	/**
	* Folders containing levels to exclude from automated tests
	*/
	UPROPERTY(EditAnywhere, config, Category = MiscAutomationSetups)
	TArray<FString> TestLevelFolders;
	
	/**
	 * External executables and scripts to run as part of automation.
	 */
	UPROPERTY(EditAnywhere, config, Category=ExternalTools)
	TArray<FExternalToolDefinition> ExternalTools;

	/**
	 * Asset import / Export test settings
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	TArray<FEditorImportExportTestDefinition> ImportExportTestDefinitions;

	/**
	* The map and device type to be used for the editor Launch On With Map Iterations test.
	*/
	UPROPERTY(config, EditAnywhere, Category = Automation, meta = (FilePathFilter="umap"))
	TArray<FLaunchOnTestSettings> LaunchOnSettings;

	/**
	 * The default resolution to take all automation screenshots at.
	 */
	UPROPERTY(EditAnywhere, config, Category=Screenshots)
	FIntPoint DefaultScreenshotResolution;

	/**
	 * PIE test duration in seconds. Only applied if you do PIETest
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float PIETestDuration;

	/**
	 * Default value used for FWaitForInteractiveFrameRate. This is a framerate determine to be suitably "interactive", but may be
	 * less than the target framerate of the game as this is used for evaluating time to PIE, load the editor etc
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float DefaultInteractiveFramerate;

	/**
	 * Default wait time in seconds for FWaitForInteractiveFrameRate. After this time a test will fail.
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float DefaultInteractiveFramerateWaitTime;

	/**
	 *  Default time in seconds that DefaultInteractiveFramerate must remain true in FWaitForInteractiveFrameRate
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float DefaultInteractiveFramerateDuration;

	/**
	 * Timeout in seconds for FWaitForEditorNavMeshBuildCommand. If the editor world's navmesh
	 * has not finished building within this time, the command completes with a warning log and
	 * the test proceeds — PIE may then start with incomplete navmesh data.
	 * Increase this for projects with very large or complex navmeshes.
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float NavMeshBuildTimeoutSeconds;

	/**
	 * Timeout in seconds for FStartPIEForAutomationCommand. If PIE fails to start, or starts
	 * but the game world does not reach a ready state within this time, the command completes
	 * and marks the test as failed via AddError().
	 * Increase this for maps with slow initialization or on slower machines.
	 */
	UPROPERTY(EditAnywhere, config, Category = Automation)
	float PIEStartTimeoutSeconds;
};
