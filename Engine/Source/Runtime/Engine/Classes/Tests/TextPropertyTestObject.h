// Copyright Epic Games, Inc. All Rights Reserved.






#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "TextPropertyTestObject.generated.h"
UCLASS()
class UTextPropertyTestObject : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	FText DefaultedText;
	UPROPERTY()
	FText UndefaultedText;
	UPROPERTY()
	FText TransientText;
};
