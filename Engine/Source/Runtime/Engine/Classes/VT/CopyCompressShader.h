// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "SceneView.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"

/** Copy with compression shader. */
class FCopyCompressCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyCompressCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyCompressCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(Texture2D, SourceTextureA)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTextureB)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DestTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DestCompressTexture_64bit)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DestCompressTexture_128bit)
		SHADER_PARAMETER(FVector2f, SourceUV)
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER(FVector2f, TexelOffsets)
		SHADER_PARAMETER(FIntVector4, DestRect)
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int32 GroupSize = 8;

	class FSourceTextureSelector : SHADER_PERMUTATION_BOOL("SOURCE_TEXTURE_A");
	class FDestSrgb : SHADER_PERMUTATION_BOOL("TEXTURE_SRGB");
	class FCompressionFormatDim : SHADER_PERMUTATION_INT("COMPRESSION_FORMAT", 7);
	using FPermutationDomain = TShaderPermutationDomain<FSourceTextureSelector, FDestSrgb, FCompressionFormatDim>;

	/** Get index to use for FCompressionFormatDim. */
	static int32 GetCompressionPermutation(EPixelFormat InFormat)
	{
		const EPixelFormat CompressedFormats[] = { PF_DXT1, PF_DXT5, PF_BC4, PF_BC5, PF_BC6H, PF_BC7 };
		return MakeArrayView(CompressedFormats).Find(InFormat) + 1;
	}
};
