// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "LightmapVirtualTexture.generated.h"

enum class ELightMapVirtualTextureType
{
	LightmapLayer0,
	LightmapLayer1,
	ShadowMask,
	SkyOcclusion,
	AOMaterialMask,

	Count,
};

UCLASS(ClassGroup = Rendering, MinimalAPI)
class ULightMapVirtualTexture2D : public UTexture2D
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = VirtualTexture)
	TArray<int8> TypeToLayer;

	ENGINE_API void SetLayerForType(ELightMapVirtualTextureType InType, uint8 InLayer);
	ENGINE_API uint32 GetLayerForType(ELightMapVirtualTextureType InType) const;

	inline bool HasLayerForType(ELightMapVirtualTextureType InType) const { return GetLayerForType(InType) != ~0u; }

	/** Whether this virtual texture is used for preview lightmaps (no underlying FVirtualTexture2DResource) */
	bool bPreviewLightmap;
};
