// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "RHIFwd.h"

#include "MeshPaintVirtualTexture.generated.h"

/**
 * Mesh paint virtual texture asset.
 * This is a virtual texture that which will be owned by a mesh component to store the mesh painting on that component.
 * All mesh paint virtual textures will be stored using a shared virtual texture page table and physical space.
 * This shared space means that all mesh paint virtual textures can be accessed in a "bindless" way using a small descriptor.
 */
UCLASS(ClassGroup = Rendering, hidecategories = Texture, hidecategories = Compression, hidecategories = Adjustments, hidecategories = Compositing, MinimalAPI)
class UMeshPaintVirtualTexture : public UTexture2D
{
	GENERATED_UCLASS_BODY()

	/** Weak refererence to the owning primitive component. */
	UPROPERTY()
	TWeakObjectPtr<UPrimitiveComponent> OwningComponent;

	//~ Begin UTexture Interface.
	virtual void GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const override;
	virtual void UpdateResourceWithParams(EUpdateResourceFlags InFlags) override;
	virtual bool IsVirtualTexturedWithSinglePhysicalPool() const override { return true; }

#if WITH_EDITOR
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
#endif
	//~ End UTexture Interface.

	/** Returns the delegate called once UpdateResourceWithParams has completed. */
	DECLARE_MULTICAST_DELEGATE(FOnResourceUpdated);
	FOnResourceUpdated& GetOnResourceUpdated() { return OnResourceUpdated; }

private:
	/** Delegate that this texture will call once UpdateResourceWithParams has completed. */
	FOnResourceUpdated OnResourceUpdated;
};

namespace MeshPaintVirtualTexture
{
	/** Returns per platform support of mesh paint virtual textures. */
	ENGINE_API bool IsSupported(EShaderPlatform InShaderPlatform);
	/** Returns per target platform support of mesh paint virtual textures. (This combines the ShaderPlatform results for a TargetPlatform). */
	ENGINE_API bool IsSupported(ITargetPlatform const* InTargetPlatform);

	/** Returns the fallback color to use for unmapped virtual textures. Use white (same as default vertex color). */
	inline uint32 GetDefaultFallbackColor() { return 0xFFFFFFFF; }

	/** Returns the passed in size after it is rounded up meet any size constraints. */
	ENGINE_API uint32 GetAlignedTextureSize(int32 InSize);
	/** Returns the default texture size to use for a mesh based on the number of vertices. */
	ENGINE_API uint32 GetDefaultTextureSize(int32 InNumVertices);

	/** 
	 * Get the 2 dword texture descriptor from texture resource. 
	 * Will return a null descriptor if the texture resource is not from a UMeshPaintVirtualTexture.
	 */
	FUintVector2 GetTextureDescriptor(FTextureResource* InTextureResource, uint32 InOptionalCoordinateIndex = 0);

	/** Scene view parameters that describe the virtual texture space shared by all UMeshPaintVirtualTexture objects. */
	struct FUniformParams
	{
		FTextureRHIRef PageTableTexture;
		FTextureRHIRef PhysicalTexture;
		FUintVector4 PackedUniform = FUintVector4(GetDefaultFallbackColor(), 0, 0, 0);
	};
	
	/** Get the global scene view parameters shared by all UMeshPaintVirtualTexture objects. */
	ENGINE_API FUniformParams GetUniformParams();
}
