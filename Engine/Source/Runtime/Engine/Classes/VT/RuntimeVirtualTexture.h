// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureDefines.h"
#include "VirtualTexturing.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "VirtualTextureEnum.h"
#include "RuntimeVirtualTexture.generated.h"

enum EShaderPlatform : uint16;
namespace UE { namespace Shader	{ enum class EValueType : uint8; } }

/** Runtime virtual texture UObject */
UCLASS(ClassGroup = Rendering, BlueprintType, MinimalAPI)
class URuntimeVirtualTexture : public UObject
{
	GENERATED_UCLASS_BODY()
	ENGINE_API ~URuntimeVirtualTexture();

	friend class FRuntimeVirtualTextureDetailsCustomization;

protected:
	/** 
	 * Size of virtual texture in tiles. (Actual values increase in powers of 2).
	 * This is applied to the largest axis in world space and the size for any shorter axis is chosen to maintain aspect ratio.  
	 */
	UPROPERTY(EditAnywhere, BluePrintGetter = GetTileCount, Category = Size, meta = (UIMin = "0", UIMax = "12", DisplayName = "Size of the virtual texture in tiles"))
	int32 TileCount = 8; // 256

	/** Page tile size. (Actual values increase in powers of 2) */
	UPROPERTY(EditAnywhere, BluePrintGetter = GetTileSize, Category = Size, meta = (UIMin = "0", UIMax = "4", DisplayName = "Size of each virtual texture tile"))
	int32 TileSize = 2; // 256

	/** Page tile border size divided by 2 (Actual values increase in multiples of 2). Higher values trigger a higher anisotropic sampling level. */
	UPROPERTY(EditAnywhere, BluePrintGetter = GetTileBorderSize, Category = Size, meta = (UIMin = "0", UIMax = "4", DisplayName = "Border padding for each virtual texture tile"))
	int32 TileBorderSize = 2; // 4

	/** Contents of virtual texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (DisplayName = "Virtual texture content"), AssetRegistrySearchable)
	ERuntimeVirtualTextureMaterialType MaterialType = ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular;

	/** Enable storing the virtual texture in GPU supported compression formats. Using uncompressed is only recommended for debugging and quality comparisons. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (DisplayName = "Enable texture compression"))
	bool bCompressTextures = true;

	/**
	* Use low quality textures (RGB565/RGB555A1) to replace runtime compression
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Performance, meta = (DisplayName = "Use Low Quality Compression", editcondition = "MaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness && bCompressTextures == true"))
	bool bUseLowQualityCompression = false;

	/** Allows to override the default priority that this runtime virtual texture has, relative to other virtual texture producers. This allows to get the pages from this virtual texture to update faster than others in case of high contention. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Performance, meta = (EditCondition = bUseCustomPriority))
	EVTProducerPriority CustomPriority;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Performance, meta = (InlineEditConditionToggle)) 
	bool bUseCustomPriority = false;

	/** Enable clear before rendering a page of the virtual texture. Disabling this can be an optimization if you know that the texture will always be fully covered by rendering.  */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Layout, meta = (DisplayName = "Enable clear before render"))
	bool bClearTextures = true;

	/** Enable page table channel packing. This reduces page table memory and update cost but can reduce the ability to share physical memory with other virtual textures.  */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Layout, meta = (DisplayName = "Enable packed page table"))
	bool bSinglePhysicalSpace = true;

	/** Enable private page table allocation. This can reduce total page table memory allocation but can also reduce the total number of virtual textures supported. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Layout, meta = (DisplayName = "Enable private page table"))
	bool bPrivateSpace = true;

	/** Enable sparse adaptive page tables. This supports larger tile counts but adds an indirection cost when sampling the virtual texture. It is recommended only when very large virtual resolutions are necessary. */
	UPROPERTY(EditAnywhere, Category = Layout, meta = (DisplayName = "Enable adaptive page table"))
	bool bAdaptive = false;

	/** Enable continuous update of the virtual texture pages. This round-robin updates already mapped pages and can help fix pages that are mapped before dependent textures are fully streamed in.  */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Layout, meta = (DisplayName = "Enable continuous page updates"))
	bool bContinuousUpdate = false;

	/** Number of low mips to cut from the virtual texture. This can reduce peak virtual texture update cost but will also increase the probability of mip shimmering. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Layout, meta = (UIMin = "0", UIMax = "5", DisplayName = "Number of low mips to remove from the virtual texture"))
	int32 RemoveLowMips = 0;

	/** A float4 custom value that can be read in the material that writes this virtual texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Settings)
	FVector4f CustomMaterialData;

	/** Texture group this texture belongs to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayName = "Texture Group"), AssetRegistrySearchable)
	TEnumAsByte<enum TextureGroup> LODGroup = TEXTUREGROUP_World;
	
	/** Deprecated size of virtual texture. */
	UPROPERTY()
	int32 Size_DEPRECATED = -1;

	/** Deprecated texture object containing streamed low mips. */
	UPROPERTY()
	TObjectPtr<class URuntimeVirtualTextureStreamingProxy> StreamingTexture_DEPRECATED = nullptr;

public:
	/** Get the material set that this virtual texture stores. */
	ERuntimeVirtualTextureMaterialType GetMaterialType() const { return MaterialType; }
	
	/** Public getter for virtual texture tile count */
	UFUNCTION(BlueprintGetter)
	int32 GetTileCount() const { return GetClampedTileCount(TileCount, bAdaptive); }
	static int32 GetClampedTileCount(int32 InTileCount, bool InAdaptive) { return 1 << FMath::Clamp(InTileCount, 0, GetMaxTileCountLog2(InAdaptive)); }
	static ENGINE_API int32 GetMaxTileCountLog2(bool InAdaptive);
	/** Public getter for virtual texture tile size */
	UFUNCTION(BlueprintGetter)
	int32 GetTileSize() const { return GetClampedTileSize(TileSize); }
	static int32 GetClampedTileSize(int32 InTileSize) { return 1 << FMath::Clamp(InTileSize + 6, 6, 10); }
	/** Public getter for virtual texture tile border size */
	UFUNCTION(BlueprintGetter)
	int32 GetTileBorderSize() const { return 2 * FMath::Clamp(TileBorderSize, 0, 4); }

	/** Public getter for virtual texture size. This is derived from the TileCount and TileSize. */
	UFUNCTION(BlueprintPure, Category = Size)
	int32 GetSize() const { return GetTileCount() * GetTileSize(); }
	/** Public getter for virtual texture page table size. This is only different from GetTileCount() when using an adaptive page table.  */
	UFUNCTION(BlueprintPure, Category = Size)
	ENGINE_API int32 GetPageTableSize() const;

	/** Get if this virtual texture uses compressed texture formats. */
	bool GetCompressTextures() const { return bCompressTextures; }
	/** Returns true if texture pages should be cleared before render */
	bool GetClearTextures() const { return bClearTextures; }
	/** Public getter for virtual texture using single physical space flag. */
	bool GetSinglePhysicalSpace() const { return bSinglePhysicalSpace; }
	/** Public getter for virtual texture using private space flag. */
	bool GetPrivateSpace() const { return bPrivateSpace; }
	/** Public getter for virtual texture using adaptive flag. */
	bool GetAdaptivePageTable() const { return bAdaptive; }
	/** Public getter for virtual texture using continuous update flag. */
	bool GetContinuousUpdate() const { return bContinuousUpdate; }
	/** Public getter for virtual texture removed low mips */
	int32 GetRemoveLowMips() const { return FMath::Clamp(RemoveLowMips, 0, 5); }
	/** Public getter for virtual texture using low quality compression flag. */
	bool GetLQCompression() const { return bUseLowQualityCompression; }
	/** Public getter for the custom material data. */
	FVector4f GetCustomMaterialData() const { return CustomMaterialData; }
	/** Public getter for texture LOD Group */
	TEnumAsByte<enum TextureGroup> GetLODGroup() const { return LODGroup; }

	/** Structure containing additional settings for initializing the producer. */
	struct FInitSettings
	{
		/** Bias to apply to the virtual texture tile count. */
		int32 TileCountBias = 0;
	};

	/** Get virtual texture description based on the properties of this object and the passed in volume transform. */
	ENGINE_API void GetProducerDescription(FVTProducerDescription& OutDesc, FInitSettings const& InitSettings, FTransform const& VolumeToWorld) const;

	/** Returns number of texture layers in the virtual texture */
	ENGINE_API int32 GetLayerCount() const;
	/** Returns number of texture layers in the virtual texture of a given material type */
	static ENGINE_API int32 GetLayerCount(ERuntimeVirtualTextureMaterialType InMaterialType);
	/** Returns the texture format for the virtual texture layer */
	ENGINE_API EPixelFormat GetLayerFormat(int32 LayerIndex) const;
	/** Return true if the virtual texture layer should be sampled as sRGB */
	ENGINE_API bool IsLayerSRGB(int32 LayerIndex) const;
	/** Return true if the virtual texture layer should be sampled as YCoCg */
	ENGINE_API bool IsLayerYCoCg(int32 LayerIndex) const;

	/**
	 * Declare a delegate funtion signiture for a function that is registered in Initialize() to be called if we call Initialize() without calling Release().
	 * This can happen for the case where the RuntimeVirtualTexture is registered twice. For example in PIE and the main editor.
	 * In that case we want to notify the first caller.
	 */
	DECLARE_DELEGATE(FOnReInitDelegate);

	/** (Re)Initialize this object. Call this whenever we modify the producer or transform. */
	ENGINE_API void Initialize(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc, FTransform const& InVolumeToWorld, FBox const& InWorldBounds, FOnReInitDelegate InDelegate);
	UE_DEPRECATED(5.6, "Use the version that takes a FOnReInitDelegate instead.")
	ENGINE_API void Initialize(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc, FTransform const& InVolumeToWorld, FBox const& InWorldBounds);

	/** Release the resources for this object This will need to be called if our producer becomes stale and we aren't doing a full reinit with a new producer. */
	ENGINE_API void Release();

	/** Getter for the associated virtual texture producer. Call on render thread only. */
	ENGINE_API FVirtualTextureProducerHandle GetProducerHandle() const;
	/** Getter for the associated virtual texture allocation. Call on render thread only. */
	ENGINE_API IAllocatedVirtualTexture* GetAllocatedVirtualTexture() const;

	/** Getter for the shader uniform parameters. */
	ENGINE_API FVector4 GetUniformParameter(int32 Index) const;
	/** Getter for the shader uniform parameter type. */
	static ENGINE_API UE::Shader::EValueType GetUniformParameterType(int32 Index);

	ENGINE_API EVTProducerPriority GetPriority() const;
	ENGINE_API EVTProducerPriority GetDefaultPriority() const;
	bool GetUseCustomPriority() const
	{
		return bUseCustomPriority;
	}

protected:
	/** Initialize the render resources. This kicks off render thread work. */
	ENGINE_API void InitResource(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc);
	/** Initialize the render resources with a null producer. This kicks off render thread work. */
	ENGINE_API void InitNullResource();

	//~ Begin UObject Interface.
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	ENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface.

private:
	/** Render thread resource container. */
	class FRuntimeVirtualTextureRenderResource* Resource;

	/** Material uniform parameters to support transform from world to UV coordinates. */
	FVector4 WorldToUVTransformParameters[3];
	/** Material uniform parameter used to pack world height. */
	FVector4 WorldHeightUnpackParameter;

	/** Delegate to be called on ReInitialize(). */
	FOnReInitDelegate OnReInitDelegate; 
};

class UVirtualTexture2D;

namespace RuntimeVirtualTexture
{
	/** Helper function to create a streaming virtual texture producer. */
	ENGINE_API IVirtualTexture* CreateStreamingTextureProducer(
		UVirtualTexture2D* InStreamingTexture,
		FVTProducerDescription const& InOwnerProducerDesc,
		FVTProducerDescription& OutStreamingProducerDesc);

	/** Helper function to bind a runtime virtual texture producer to a streaming producer. Returns the new combined producer. */
	ENGINE_API IVirtualTexture* BindStreamingTextureProducer(
		IVirtualTexture* InProducer,
		IVirtualTexture* InStreamingProducer,
		int32 InTransitionLevel);

	/** Get if the material type is supported. Individual material types can be disabled in project settings to reduce shader permutations. */
	ENGINE_API bool IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType MaterialType);
	ENGINE_API bool IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType MaterialType, EShaderPlatform InPlatform);
}
