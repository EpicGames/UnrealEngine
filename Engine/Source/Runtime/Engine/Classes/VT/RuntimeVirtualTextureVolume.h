// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "RuntimeVirtualTextureVolume.generated.h"

/** Actor used to place a URuntimeVirtualTexture in the world. */
UCLASS(Blueprintable, HideCategories = (Actor, Collision, Cooking, HLOD, Input, LOD, Networking, Physics, Replication), MinimalAPI)
class ARuntimeVirtualTextureVolume : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	/** Component that owns the runtime virtual texture. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = VirtualTexture, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class URuntimeVirtualTextureComponent> VirtualTextureComponent;

#if WITH_EDITORONLY_DATA
	/** Box for visualizing virtual texture extents. */
	UPROPERTY(Transient)
	TObjectPtr<class UBoxComponent> Box = nullptr;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif

protected:
	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar);
	virtual bool NeedsLoadForServer() const override { return false; }
	//~ End UObject Interface.
	//~ Begin AActor Interface.
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	//~ End AActor Interface
};
