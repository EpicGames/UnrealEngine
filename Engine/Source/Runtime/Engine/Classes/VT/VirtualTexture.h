// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "VT/VirtualTextureBuildSettings.h"
#include "VirtualTexture.generated.h"

/** Deprecated class */
UCLASS(ClassGroup = Rendering, MinimalAPI)
class UVirtualTexture : public UObject
{
	GENERATED_UCLASS_BODY()
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
};

/** Deprecated class */
UCLASS(ClassGroup = Rendering, MinimalAPI)
class ULightMapVirtualTexture : public UVirtualTexture
{
	GENERATED_UCLASS_BODY()
};

/** Deprecated class. */
UCLASS(ClassGroup = Rendering)
class URuntimeVirtualTextureStreamingProxy : public UTexture2D
{
	GENERATED_UCLASS_BODY()
};


/**
 * Virtual Texture with locally configurable build settings.
 * A raw UTexture2D can also represent a Virtual Texture but uses the one and only per-project build settings.
 */
UCLASS(ClassGroup = Rendering)
class UVirtualTexture2D : public UTexture2D
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FVirtualTextureBuildSettings Settings;

	UPROPERTY()
	bool bContinuousUpdate_DEPRECATED;

	UPROPERTY()
	bool bSinglePhysicalSpace;

	//~ Begin UTexture Interface.
	virtual void GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const override { OutSettings = Settings; }
	virtual bool IsVirtualTexturedWithSinglePhysicalSpace() const override { return bSinglePhysicalSpace; }
#if WITH_EDITOR
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	virtual void ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
#endif
	//~ End UTexture Interface.
};
