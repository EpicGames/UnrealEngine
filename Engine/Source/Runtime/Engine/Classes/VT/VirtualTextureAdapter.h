// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "VirtualTextureAdapter.generated.h"

/** 
 * Adapter object that wraps a regular UTexture and allows it to be referenced as a virtual texture in materials.
 * The virtual texture pages are filled on demand by copying from the wrapped texture.
 * This isn't memory efficient or performant, but can be useful for certain debugging or previewing modes where we don't
 * want to change the material, but do want to bind a non-virtual texture to an existing virtual texture sample.
 */
UCLASS(ClassGroup = Rendering, BlueprintType, MinimalAPI)
class UVirtualTextureAdapter : public UTexture
{
	GENERATED_UCLASS_BODY()

	//~ Begin UTexture Interface.
	virtual FTextureResource* CreateResource() override;
	virtual ETextureClass GetTextureClass() const override;
	virtual EMaterialValueType GetMaterialType() const override;
	virtual bool IsCurrentlyVirtualTextured() const override { return true; }
	virtual float GetSurfaceWidth() const override;
	virtual float GetSurfaceHeight() const override;
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	//~ End UTexture Interface.

public:
	/** The UTexture object to wrap. */
	UPROPERTY(EditAnywhere, Category = Texture)
	TObjectPtr<UTexture> Texture;

	/** Optional UTexture object that has the final texture format that we would like to use. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Texture)
	TObjectPtr<class UTexture2D> OverrideWithTextureFormat;

	/** Set to true if we want to use the default project virtual texture tile settings. */
	UPROPERTY(EditAnywhere, Category = Texture)
	bool bUseDefaultTileSizes = true;

	/** Page tile size. (Will be rounded up to power of 2). */
	UPROPERTY(EditAnywhere, Category = Texture, meta = (UIMin = "0", UIMax = "512", editcondition="!bUseDefaultTileSizes"))
	int32 TileSize = 0;

	/** Page tile border size. (Will be rounded up to multiple of 2). */
	UPROPERTY(EditAnywhere, Category = Texture, meta = (UIMin = "0", UIMax = "8", editcondition = "!bUseDefaultTileSizes"))
	int32 TileBorderSize = 0;

	/** Flush the virtual texture page contents. Call this whenever the wrapped UTexture is modified. */
	ENGINE_API void Flush(FBox2f const& UVRect);
};
