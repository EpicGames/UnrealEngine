// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureBuildSettings.generated.h"

/** Build settings used for virtual textures. */
USTRUCT()
struct FVirtualTextureBuildSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 TileSize = 0;

	UPROPERTY()
	int32 TileBorderSize = 0;

	/** Initialize with the default build settings. These are defined by the current project setup. */
	ENGINE_API void Init();

	/** Helper to clamp tile size and align to next power of two. */
	ENGINE_API static int32 ClampAndAlignTileSize(int32 InTileSize);
	/** Helper to clamp tile border size and align to next multiple of two. */
	ENGINE_API static int32 ClampAndAlignTileBorderSize(int32 InTileBorderSize);
};
