// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "VirtualTextureBuilder.generated.h"

enum class EShadingPath;

#if WITH_EDITOR

/** Description object used to build the contents of a UVirtualTextureBuilder. */
struct FVirtualTextureBuildDesc
{
	uint64 BuildHash = 0;

	int32 LayerCount = 0;
	TArray<ETextureSourceFormat, TInlineAllocator<4>> LayerFormats;
	TArray<FTextureFormatSettings, TInlineAllocator<4>> LayerFormatSettings;

	int32 TileSize = 0;
	int32 TileBorderSize = 0;

	TEnumAsByte<enum TextureGroup> LODGroup = TEXTUREGROUP_World;
	ETextureLossyCompressionAmount LossyCompressionAmount = TLCA_Default;

	UE_DEPRECATED(5.6, "bContinuousUpdate is not used.")
	bool bContinuousUpdate = false;
	bool bSinglePhysicalSpace = false;

	int32 NumMips = 0;

	uint32 InSizeX = 0;
	uint32 InSizeY = 0;
	uint8 const* InData = nullptr;
};

#endif

/**
 * Container for a UVirtualTexture2D that can be built from a FVirtualTextureBuildDesc description.
 * This has a simple BuildTexture() interface but we may want to extend in the future to support partial builds
 * or other more blueprint driven approaches for data generation.
 */
UCLASS(ClassGroup = Rendering, BlueprintType, MinimalAPI, meta = (DisplayName = "Streaming Runtime Virtual Texture"))
class UVirtualTextureBuilder : public UObject
{
public:
	GENERATED_UCLASS_BODY()
	ENGINE_API ~UVirtualTextureBuilder();

	/** The (embedded) texture asset. Use Build Virtual Textures in the Build menu, or the Build button in the Runtime Virtual Texture Component to create/update it. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Texture)
	TObjectPtr<class UVirtualTexture2D> Texture;

	/** The (embedded) texture asset for mobile rendering. Use Build Virtual Textures or the Build button in the Runtime Virtual Texture Component to create/update it. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Texture, meta = (EditCondition=bSeparateTextureForMobile))
	TObjectPtr<class UVirtualTexture2D> TextureMobile;

	/** Some client defined hash of that defines how the Texture was built. */
	UPROPERTY()
	uint64 BuildHash;

	/** Virtual texture for a specific shading path */
	ENGINE_API UVirtualTexture2D* GetVirtualTexture(EShadingPath ShadingPath) const;

	/** Whether to use a separate texture for Mobile rendering. A separate texture will be built using mobile preview editor mode. Use this in case there is too much discrepancy between the RVT used for desktop vs. mobile. */
	UPROPERTY(EditAnywhere, Category = Texture)
	bool bSeparateTextureForMobile = false;

#if WITH_EDITOR
	/** Creates a new UVirtualTexture2D and stores it in the contained Texture. */
	ENGINE_API void BuildTexture(EShadingPath ShadingPath, FVirtualTextureBuildDesc const& BuildDesc, bool bWaitForCompilation = false);
#endif

protected:
	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~ End UObject Interface
};
