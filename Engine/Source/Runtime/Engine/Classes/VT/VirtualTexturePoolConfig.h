// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "RenderResource.h"

#include "VirtualTexturePoolConfig.generated.h"

/** Settings for a single virtual texture physical pool. */
USTRUCT()
struct FVirtualTextureSpacePoolConfig
{
	GENERATED_USTRUCT_BODY()

	/** Formats of the layers in the physical pool. Leave empty to match any format. */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Filter")
	TArray< TEnumAsByte<EPixelFormat> > Formats;

	/** Minimum tile size to match (including tile border). */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Filter")
	int32 MinTileSize = 0;

	/** Maximum tile size to match (including tile border). Set to 0 to match any tile size. */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Filter")
	int32 MaxTileSize = 0;

	/** Upper limit size in megabytes to allocate for the pool. The allocator will allocate as close as possible below this limit. */
	UPROPERTY(EditAnywhere, Category = PoolConfig)
	int32 SizeInMegabyte = 0;

	/** Enable MipMapBias based on pool residency tracking. */
	UPROPERTY(EditAnywhere, Category = PoolConfig)
	bool bEnableResidencyMipMapBias = false;

	/** Group can be in range 0-3. Pools with the same group share a mip map bias. */
	UPROPERTY(EditAnywhere, Category = PoolConfig, meta = (ClampMin = "0", ClampMax = "3", UIMin = "0", UIMax = "3"))
	int32 ResidencyMipMapBiasGroup = 0;

	/** Allow the size to allocate for the pool to be scaled by scalability settings. */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Scalabiity")
	bool bAllowSizeScale = false;

	/** Lower limit of size in megabytes to allocate for the pool after size scaling. */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Scalability")
	int32 MinScaledSizeInMegabyte = 0;

	/** Upper limit of size in megabytes to allocate for the pool after size scaling. Set to 0 to ignore. */
	UPROPERTY(EditAnywhere, Category = "PoolConfig|Scalability")
	int32 MaxScaledSizeInMegabyte = 0;

	/** Is this the default config? Use this setting when we can't find any other match. */
	bool IsDefault() const { return Formats.Num() == 0 && SizeInMegabyte > 0; }
};

/** Configuration for virtual texture physical pool sizes. */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Virtual Texture Pool"), MinimalAPI)
class UVirtualTexturePoolConfig : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()
	
	/** Upper size limit in megabytes for any pools not explicitly matched by a config entry in the Pools array. */
	UPROPERTY(config, EditAnywhere, Category = PoolConfig)
	int32 DefaultSizeInMegabyte = 64;

	/** 
	 * Enable physical pools growing on oversubscription.
	 * Each physical pool will grow to the maximum size so far requested.
	 * This setting applies to the editor only. To have similar behavior in a cooked build use r.VT.PoolAutoGrow.
	 */
	UPROPERTY(config, EditAnywhere, Category = PoolConfig)
	bool bPoolAutoGrowInEditor = true;

	/** 
	 * Serialized array of configs. 
	 * A virtual texture physical pool iterates these from last to first and uses the first matching config that it finds. 
	 */
	UPROPERTY(config, EditAnywhere, Category = PoolConfig, meta=(DisplayName="Fixed Pools", TitleProperty = "Formats"))
	TArray<FVirtualTextureSpacePoolConfig> Pools;

	/** 
	 * Transient array of runtime detected configs used by the PoolAutoGrow system.
	 * A virtual texture physical pool searches these to find a match before searching the configs in Pools.
	 * These tracked configs can be copied to the serialized Pools as a good estimation of the fixed pool sizes that a cooked project needs.
	 */
	UPROPERTY(Transient, EditAnywhere, Category = PoolConfig, meta = (TitleProperty = "Formats"))
	TArray<FVirtualTextureSpacePoolConfig> TransientPools;

	/** Find a matching config by first searching the TransientPools and then the Pools. */
	void FindPoolConfig(TEnumAsByte<EPixelFormat> const* InFormats, int32 InNumLayers, int32 InTileSize, FVirtualTextureSpacePoolConfig& OutConfig) const;

	/** Add a transient config. If there is already a matching config, then this will just copy the SizeInMegabyte setting to the existing config. Returns false if no change is made. */
	bool AddOrModifyTransientPoolConfig(FVirtualTextureSpacePoolConfig const& InConfig);

#if WITH_EDITOR
	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

/** Namespace container for static helper functions. */
struct VirtualTexturePool
{
	/** Find a matching pool config. */
	ENGINE_API static void FindPoolConfig(TEnumAsByte<EPixelFormat> const* InFormats, int32 InNumLayers, int32 InTileSize, FVirtualTextureSpacePoolConfig& OutConfig);
	/** Add pool configs. If one of the configs already has a matching config, then this will just copy the SizeInMegabyte setting to the existing config. */
	ENGINE_API static void AddOrModifyTransientPoolConfigs_RenderThread(TArray<FVirtualTextureSpacePoolConfig>& InConfigs);
	/** Get the scalability setting for scaling virtual texture physical pool sizes. */
	ENGINE_API static float GetPoolSizeScale();
	/** Get if the virtual texture physical pools should auto grow on oversubscription. */
	ENGINE_API static bool GetPoolAutoGrow();
	/** Get if we support 16bit page table formats. */
	ENGINE_API static bool Support16BitPageTables();
	/** Get the scalability setting whether we can create multiple physical pools for the same format. */
	ENGINE_API static int32 GetSplitPhysicalPoolSize();
	/** Get a hash of all settings that affect virtual texture physical pool creation. */
	ENGINE_API static uint32 GetConfigHash();
};
