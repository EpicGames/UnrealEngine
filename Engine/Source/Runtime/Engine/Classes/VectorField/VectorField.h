// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorField: A 3D grid of vectors.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VectorField.generated.h"

UCLASS(MinimalAPI)
class UVectorField : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Bounds of the volume in local space. */
	UPROPERTY(EditAnywhere, Category=VectorFieldBounds)
	FBox Bounds;

	/** The intensity with which to multiplie vectors in this field. */
	UPROPERTY(EditAnywhere, Category=VectorField)
	float Intensity;


	/**
	 * Initializes an instance for use with this vector field.
	 */
	virtual void InitInstance(class FVectorFieldInstance* Instance, bool bPreviewInstance);
};



