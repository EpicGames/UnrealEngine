// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VectorFieldAnimated: An animated 3D grid of vectors.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VectorField/VectorField.h"
#include "VectorFieldAnimated.generated.h"

struct FPropertyChangedEvent;

/**
 * Operation used to construct the vector field from a 2D texture.
 */
UENUM()
enum EVectorFieldConstructionOp : int
{
	VFCO_Extrude UMETA(DisplayName="Extrude"),
	VFCO_Revolve UMETA(DisplayName="Revolve"),
	VFCO_MAX,
};

UCLASS(MinimalAPI)
class UVectorFieldAnimated : public UVectorField
{
	GENERATED_UCLASS_BODY()

	/** The texture from which to create the vector field. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	TObjectPtr<class UTexture2D> Texture;

	/** The operation used to construct the vector field. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	TEnumAsByte<enum EVectorFieldConstructionOp> ConstructionOp;

	/** The size of the volume. Valid sizes: 16, 32, 64. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 VolumeSizeX;

	/** The size of the volume. Valid sizes: 16, 32, 64. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 VolumeSizeY;

	/** The size of the volume. Valid sizes: 16, 32, 64. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 VolumeSizeZ;

	/** The number of horizontal subimages in the texture atlas. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 SubImagesX;

	/** The number of vertical subimages in the texture atlas. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 SubImagesY;

	/** The number of frames in the atlas. */
	UPROPERTY(EditAnywhere, Category=Reconstruction)
	int32 FrameCount;

	/** The rate at which to interpolate between frames. */
	UPROPERTY(EditAnywhere, Category=Animation)
	float FramesPerSecond;

	/** Whether or not the simulation should loop. */
	UPROPERTY(EditAnywhere, Category=Animation)
	uint32 bLoop:1;

	/** A static vector field used to add noise. */
	UPROPERTY(EditAnywhere, Category=Noise)
	TObjectPtr<class UVectorFieldStatic> NoiseField;

	/** Scale to apply to vectors in the noise field. */
	UPROPERTY(EditAnywhere, Category=Noise)
	float NoiseScale;

	/** The maximum magnitude of noise vectors to apply. */
	UPROPERTY(EditAnywhere, Category=Noise)
	float NoiseMax;


	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UVectorField Interface
	virtual void InitInstance(class FVectorFieldInstance* Instance, bool bPreviewInstance) override;
	//~ End UVectorField Interface

};



