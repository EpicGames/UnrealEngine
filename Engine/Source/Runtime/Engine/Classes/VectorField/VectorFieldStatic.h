// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorField: A 3D grid of vectors.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Math/Float16.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/BulkData.h"
#include "VectorField/VectorField.h"
#include "VectorFieldStatic.generated.h"

class FRHITexture;
struct FPropertyChangedEvent;

// The internal representation of the CPU data (if used) is based on how we can
// effectively sample it.  Currently we'll decide that based on if we have ISPC
// supported
#define VECTOR_FIELD_DATA_AS_HALF (INTEL_ISPC)

UCLASS(hidecategories=VectorFieldBounds, MinimalAPI)
class UVectorFieldStatic : public UVectorField
{
	GENERATED_UCLASS_BODY()

	/** Size of the vector field volume. */
	UPROPERTY(Category=VectorFieldStatic, VisibleAnywhere)
	int32 SizeX;

	/** Size of the vector field volume. */
	UPROPERTY(Category=VectorFieldStatic, VisibleAnywhere)
	int32 SizeY;

	/** Size of the vector field volume. */
	UPROPERTY(Category=VectorFieldStatic, VisibleAnywhere)
	int32 SizeZ;

	/** Whether to keep vector field data accessible to the CPU. */
	UPROPERTY(Category=VectorFieldStatic, EditAnywhere)
	bool bAllowCPUAccess;

public:
	/** The resource for this vector field. */
	class FVectorFieldResource* Resource;

	/** Source vector data. */
	FByteBulkData SourceData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif // WITH_EDITORONLY_DATA

	//~ Begin UObject Interface.
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual void PostInitProperties() override;
#endif
	//~ End UObject Interface.

	//~ Begin UVectorField Interface
	virtual void InitInstance(class FVectorFieldInstance* Instance, bool bPreviewInstance) override;
	//~ End UVectorField Interface

	/**
	 * Initialize resources.
	 */
	ENGINE_API void InitResource();

	/** Takes a local copy of the source bulk data so that it is readable at runtime on the CPU. 
	* @param bDiscardData If the internal loaded data should be discarded after calling this or not.
	*/
	ENGINE_API void UpdateCPUData(bool bDiscardData);

#if WITH_EDITOR
	/** Sets the bAllowCPUAccess flag and calls UpdateCPUData(). */
	ENGINE_API void SetCPUAccessEnabled();
#endif // WITH_EDITOR

	/** Returns a reference to a 3D texture handle for the GPU data. */
	ENGINE_API FRHITexture* GetVolumeTextureRef();

	ENGINE_API FVector FilteredSample(const FVector& SamplePosition, const FVector& TilingAxes) const;
	ENGINE_API FVector Sample(const FIntVector& SamplePosition) const;

	using InternalFloatType = 
#if VECTOR_FIELD_DATA_AS_HALF
		FFloat16;
#else
		float;
#endif

	TConstArrayView<InternalFloatType> ReadCPUData() const
	{
		return MakeArrayView<const InternalFloatType>(reinterpret_cast<const InternalFloatType*>(CPUData.GetData()), CPUData.Num() / sizeof(InternalFloatType));
	}

	bool HasCPUData() const { return bAllowCPUAccess && CPUData.Num(); }

private:
	/** Permit the factory class to update and release resources externally. */
	friend class UVectorFieldStaticFactory;

	/**
	 * Update resources. This must be implemented by subclasses as the Resource
	 * pointer must always be valid.
	 */
	ENGINE_API void UpdateResource();

	/**
	 * Release the static vector field resource.
	 */
	ENGINE_API void ReleaseResource();

	inline FVector SampleInternal(int32 SampleIndex) const;

	/** Local copy of the source vector data.  May be stored in half precision if it can be sampled efficiently */
	TArray<uint8> CPUData;
};

// work around for the private nature of FVectorFieldResource
struct FVectorFieldTextureAccessor
{
	ENGINE_API FVectorFieldTextureAccessor(UVectorField* InVectorField);
	ENGINE_API FVectorFieldTextureAccessor(const FVectorFieldTextureAccessor& rhs);
	ENGINE_API ~FVectorFieldTextureAccessor();

	ENGINE_API FRHITexture* GetTexture() const;

private:
	TUniquePtr<struct FVectorFieldTextureAccessorImpl> Impl;
};
