// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorFieldVolume: Volume encompassing a vector field.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "VectorFieldVolume.generated.h"

class UBillboardComponent;

UCLASS(hidecategories=(Object, Advanced, Collision), MinimalAPI)
class AVectorFieldVolume : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = VectorFieldVolume, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UVectorFieldComponent> VectorFieldComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif

public:
	/** Returns VectorFieldComponent subobject **/
	class UVectorFieldComponent* GetVectorFieldComponent() const { return VectorFieldComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif
};



