// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DataAsset.h"
#include "TireType.generated.h"

/** DEPRECATED - Only used to allow conversion to new TireConfig in PhysXVehicles plugin */
UCLASS(MinimalAPI)
class UTireType : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(VisibleAnywhere, Category = Deprecated)
	float FrictionScale;
};
