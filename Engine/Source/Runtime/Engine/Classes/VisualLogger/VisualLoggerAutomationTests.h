// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VisualLoggerAutomationTests.generated.h"

UCLASS(NotBlueprintable, Transient, hidecategories = UObject)
class UVisualLoggerAutomationTests : public UObject
{
	GENERATED_UCLASS_BODY()
};

