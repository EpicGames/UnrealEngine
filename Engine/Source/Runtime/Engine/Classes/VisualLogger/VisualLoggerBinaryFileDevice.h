// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "EngineDefines.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "VisualLogger/VisualLoggerDefines.h"

#define VISLOG_FILENAME_EXT TEXT("bvlog")

#if UE_DEBUG_RECORDING_USING_VLOG

#include "VisualLogger/VisualLoggerTypes.h"

class FVisualLoggerBinaryFileDevice : public FVisualLogDevice
{
public:
	static FVisualLoggerBinaryFileDevice& Get()
	{
		static FVisualLoggerBinaryFileDevice GDevice;
		return GDevice;
	}

	FVisualLoggerBinaryFileDevice();
	virtual void Cleanup(bool bReleaseMemory = false) override;
	virtual void StartRecordingToFile(double TimeStamp) override;
	virtual void StopRecordingToFile(double TimeStamp) override;
	virtual void DiscardRecordingToFile() override;
	virtual void SetFileName(const FString& InFileName) override;
	virtual void Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) override;
	virtual void GetRecordedLogs(TArray<FVisualLogEntryItem>& RecordedLogs) const override { RecordedLogs = FrameCache; }
	virtual bool HasFlags(int32 InFlags) const override { return !!(InFlags & (EVisualLoggerDeviceFlags::CanSaveToFile | EVisualLoggerDeviceFlags::StoreLogsLocally)); }

protected:
	int32 bUseCompression : 1;
	float FrameCacheLenght;
	double StartRecordingTime;
	double LastLogTimeStamp;
	FArchive* FileArchive;
	FString TempFileName;
	FString FileName;
	TArray<FVisualLogEntryItem> FrameCache;
};
#endif
