// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "VisualLoggerKismetLibrary.generated.h"

UCLASS(MinimalAPI, meta=(ScriptName="VisualLoggerLibrary"))
class UVisualLoggerKismetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (DisplayName = "Enable VisLog Recording", CallableWithoutWorldContext, DevelopmentOnly))
	static void EnableRecording(bool bEnabled);

	/** Makes SourceOwner log to DestinationOwner's vislog*/
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, CallableWithoutWorldContext, DevelopmentOnly))
	static void RedirectVislog(UObject* SourceOwner, UObject* DestinationOwner);

	/** Logs simple text string with Visual Logger - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Text", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogText(UObject* WorldContextObject, FString Text, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false);

	/** Logs location as sphere with given radius - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Location", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogLocation(UObject* WorldContextObject, FVector Location, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, float Radius = 10, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false);

	/** Logs sphere shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Sphere Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogSphere(UObject* WorldContextObject, FVector Center, float Radius, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs cone shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Cone Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogCone(UObject* WorldContextObject, FVector Origin, FVector Direction, float Length, float Angle, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs cylinder shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Cylinder Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogCylinder(UObject* WorldContextObject, FVector Start, FVector End, float Radius, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs capsule shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Capsule Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogCapsule(UObject* WorldContextObject, FVector Base, float HalfHeight, float Radius, FQuat Rotation, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs box shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Box Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogBox(UObject* WorldContextObject, FBox BoxShape, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs oriented box shape - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Oriented Box Shape", AdvancedDisplay = "WorldContextObject, bAddToMessageLog", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogOrientedBox(UObject* WorldContextObject, FBox BoxShape, FTransform Transform, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName LogCategory = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs arrow - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Arrow", AdvancedDisplay = "WorldContextObject, bAddToMessageLog, Thickness", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogArrow(UObject* WorldContextObject, const FVector SegmentStart, const FVector SegmentEnd, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, FName CategoryName = TEXT("VisLogBP"), bool bAddToMessageLog = false, float ArrowHeadSize = 8.0f);

	/** Logs circle - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Circle", AdvancedDisplay = "WorldContextObject, bAddToMessageLog, Thickness", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogCircle(UObject* WorldContextObject, FVector Center, FVector UpAxis, float Radius, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, const float Thickness = 0.f, FName CategoryName = TEXT("VisLogBP"), bool bAddToMessageLog = false, bool bWireframe = false);

	/** Logs segment - recording for Visual Logs has to be enabled to record this data
	 * @param bAddToMessageLog Indicates whether a log should also be produced.
	 *  Message log is used for calls on game thread, otherwise regular output log is used for thread safety.
	 */
	UFUNCTION(BlueprintCallable, Category = "Debug|VisualLogger", meta = (BlueprintThreadSafe, DisplayName = "VisLog Segment", AdvancedDisplay = "WorldContextObject, bAddToMessageLog, Thickness", CallableWithoutWorldContext, DevelopmentOnly, DefaultToSelf = "WorldContextObject"))
	static void LogSegment(UObject* WorldContextObject, const FVector SegmentStart, const FVector SegmentEnd, FString Text, FLinearColor ObjectColor = FLinearColor::Blue, const float Thickness = 0.f, FName CategoryName = TEXT("VisLogBP"), bool bAddToMessageLog = false);

private:
	static constexpr ELogVerbosity::Type DefaultVerbosity = ELogVerbosity::Log;
};
