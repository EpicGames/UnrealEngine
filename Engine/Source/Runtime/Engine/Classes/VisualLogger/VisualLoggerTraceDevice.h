// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "EngineDefines.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "VisualLogger/VisualLoggerTypes.h"
#include "Delegates/DelegateCombinations.h"

#define UE_API ENGINE_API

DECLARE_DELEGATE_TwoParams(FImmediateRenderDelegate, const UObject*, const FVisualLogEntry&);

class FVisualLoggerTraceDevice : public FVisualLogDevice
{
public:
	static UE_API FVisualLoggerTraceDevice& Get();

	UE_API FVisualLoggerTraceDevice();
	UE_API virtual void Cleanup(bool bReleaseMemory = false) override;
	UE_API virtual void StartRecordingToFile(double TimeStamp) override;
	UE_API virtual void StopRecordingToFile(double TimeStamp) override;
	UE_API virtual void DiscardRecordingToFile() override;
	UE_API virtual void SetFileName(const FString& InFileName) override;
	UE_API virtual void Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) override;
	UE_API virtual bool HasFlags(int32 InFlags) const override;

	FImmediateRenderDelegate ImmediateRenderDelegate;
};

#undef UE_API

#endif // UE_DEBUG_RECORDING_ENABLED
