// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Engine : ModuleRules
{
	public Engine(ReadOnlyTargetRules Target) : base(Target)
	{
		NumIncludedBytesPerUnityCPPOverride = 589824; // best unity size found from using UBT ProfileUnitySizes mode

		PrivatePCHHeaderFile = "Private/EnginePrivatePCH.h";

		SharedPCHHeaderFile = "Public/EngineSharedPCH.h";

		StaticAnalyzerDisabledCheckers.Add("core.uninitialized.ArraySubscript");

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"AnimationCore",
				"AudioExtensions",
				"AudioMixer", 
				"AudioMixerCore",
				"InputCore",
				"MovieSceneCapture", 
				"PacketHandler", 
				"Renderer",
				"RHI",
				"Shaders"
			}
		);

		PrivateIncludePaths.AddRange(
			new string[] {
				Path.Combine(GetModuleDirectory("NetCore"), "Private"),
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
				"DesktopPlatform",
				"DistributedBuildInterface",
				"TargetPlatform",
				"ImageWrapper",
				"ImageWriteQueue",
				"HeadMountedDisplay",
				"EyeTracker",
				"MRMesh",
				"Advertising",
				"AutomationWorker",
			}
		);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping || Target.bCompileAgainstEditor)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SlateReflector",
					"TranslationEditor",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SlateReflector",
					"TranslationEditor",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorAnalyticsSession"
				}
			);
		}

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreOnline",
				"CoreUObject",
				"FieldNotification",
				"NetCore",
				"ImageCore",
				"Json",
				"JsonUtilities",
				"SlateCore",
				"Slate",
				"InputCore",
				"Messaging",
				"MessagingCommon",
				"RenderCore",
				"AnalyticsET",
				"RHI",
				"Sockets",
				"AssetRegistry", // Here until we update all modules using AssetRegistry to add a dependency on it
				"EngineMessages",
				"EngineSettings",
				"SynthBenchmark",
				"GameplayTags",
				"PacketHandler",
				"AudioPlatformConfiguration",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"PakFile",
				"NetworkReplayStreaming",
				"PhysicsCore",
				"SignalProcessing",
				"StateStream",
				"AudioExtensions",
				"DeveloperSettings",
				"AudioLinkCore",
				"CookOnTheFly",
				"IoStoreOnDemandCore"
			}
		);

		if (Target.bCompileAgainstApplicationCore)
		{
			PublicDependencyModuleNames.Add("ApplicationCore");
		}

		if (Target.bCompileIoStoreOnDemand)
		{
			PrivateDependencyModuleNames.Add("IoStoreOnDemand");
			PublicDefinitions.Add("UE_WITH_IOSTOREONDEMAND=1");
		}

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"TypedElementFramework",
				"TypedElementRuntime",
				"NetCore",
				"RenderCore",
				"CoreUObject",
				"CoreOnline",
				"PhysicsCore",
				"ChaosCore",
				"DeveloperSettings",
				"NetCommon",
				"Slate",
				"Sockets",
				"MeshDescription"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AnimationCore",
				"AutoRTFM",
				"AppFramework",
				"BandwidthMeasurementTool",
				"BuildSettings",
				"Networking",
				"Landscape",
				"UMG",
				"Projects",
				"TypedElementFramework",
				"TypedElementRuntime",
				"MaterialShaderQualitySettings",
				"MoviePlayerProxy",
				"CinematicCamera",
				"Analytics",
				"AudioMixer",
				"AudioMixerCore",
				"IntelISPC",
				"TraceLog",
				"Icmp",
				"UniversalObjectLocator",
				"XmlParser",
				"IoStoreOnDemand",
				"HTTP",
				"GeometryCore",
				"DownlinkBandwidthManager",
			}
		);

		if (Target.bBuildWithEditorOnlyData && Target.bBuildEditor)
		{
			// The SparseVolumeTexture module containing the importer is only loaded and used in the editor.
			DynamicallyLoadedModuleNames.Add("SparseVolumeTexture");
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"UEWavComp"
				);
		}

		// Cross platform Audio Codecs: (we build here, but don't depend on them directly)
		DynamicallyLoadedModuleNames.Add("RadAudioDecoder");
		DynamicallyLoadedModuleNames.Add("BinkAudioDecoder");
		DynamicallyLoadedModuleNames.Add("VorbisAudioDecoder");
		DynamicallyLoadedModuleNames.Add("OpusAudioDecoder");
		DynamicallyLoadedModuleNames.Add("AdpcmAudioDecoder");
		
		DynamicallyLoadedModuleNames.Add("EyeTracker");

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.Add("Localization");
			DynamicallyLoadedModuleNames.Add("Localization");
		}

		// to prevent "causes WARNING: Non-editor build cannot depend on non-redistributable modules."
		if (Target.bCompileAgainstEditor)
		{

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"TextureBuildUtilities",
					"Horde"
				}
			);

			// for now we depend on these
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RawMesh",
					"Zen"
				}
			);

			// editor telemetry events
			PrivateDependencyModuleNames.Add("EditorToolEvents");
		}

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MessagingRpc",
				"PortalRpc",
				"PortalServices",
			}
		);

		if (Target.bCompileAgainstEditor)
		{
			// these modules require variadic templates
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MessagingRpc",
					"PortalRpc",
					"PortalServices",
				}
			);
		}

		CircularlyReferencedDependentModules.AddRange(
			[
				"GameplayTags",
				"Landscape",
				"UMG",
				"MaterialShaderQualitySettings",
				"CinematicCamera",
				"AudioMixer",
				"MaterialUtilities",
			]);
		

		if (Target.bCompileAgainstEditor)
		{
			PrivateIncludePathModuleNames.Add("Foliage");
		}

		// The AnimGraphRuntime module is not needed by Engine proper, but it is loaded in LaunchEngineLoop.cpp,
		// and needs to be listed in an always-included module in order to be compiled into standalone games
		DynamicallyLoadedModuleNames.Add("AnimGraphRuntime");

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"MovieScene",
				"MovieSceneCapture",
				"MovieSceneTracks",
				"LevelSequence",
				"HeadMountedDisplay",
				"MRMesh",
				"StreamingPauseRendering",
			}
		);

		if (Target.Type != TargetType.Server)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"SlateNullRenderer",
					"SlateRHIRenderer"
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
					"SlateNullRenderer",
					"SlateRHIRenderer"
				}
			);
		}

		if (Target.Type == TargetType.Server || Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.Add("PerfCounters");
		}

		if (Target.bCompileAgainstEditor)
		{
			PrivateIncludePathModuleNames.Add("MaterialUtilities");
			PrivateDependencyModuleNames.Add("MaterialUtilities");

			PrivateIncludePathModuleNames.Add("MeshUtilities");
			DynamicallyLoadedModuleNames.Add("MeshUtilities");

			PrivateIncludePathModuleNames.Add("MeshUtilitiesCommon");

			PublicIncludePathModuleNames.Add("AnimationDataController");
			DynamicallyLoadedModuleNames.Add("AnimationDataController");

			PublicIncludePathModuleNames.Add("AnimationBlueprintEditor");
			DynamicallyLoadedModuleNames.Add("AnimationBlueprintEditor");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RawMesh"
				}
			);

			PrivateDependencyModuleNames.Add("CollisionAnalyzer");
			CircularlyReferencedDependentModules.Add("CollisionAnalyzer");

			PrivateDependencyModuleNames.Add("LogVisualizer");
			CircularlyReferencedDependentModules.Add("LogVisualizer");

			PrivateIncludePathModuleNames.Add("GraphEditor");
			PrivateDependencyModuleNames.Add("GraphEditor");

			PrivateDependencyModuleNames.Add("MaterialEditor");

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"WindowsTargetPlatform",
						"WindowsPlatformEditor",
					}
				);
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"MacTargetPlatform",
						"MacPlatformEditor",
					}
				);
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
			{
				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"LinuxTargetPlatform",
						"LinuxPlatformEditor",
					}
				);
			}
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"NullNetworkReplayStreaming",
				"LocalFileNetworkReplayStreaming",
				"HttpNetworkReplayStreaming",
				"Advertising"
			}
		);

		if (Target.bWithLiveCoding)
		{
			DynamicallyLoadedModuleNames.Add("LiveCoding");
		}

		if (Target.Type != TargetType.Server)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"ImageWrapper"
				}
			);
		}

		AllowedRestrictedFolders.Add("Private/NotForLicensees");

		if (!Target.bBuildRequiresCookedData && Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("DeveloperToolSettings");


		}

		DynamicallyLoadedModuleNames.Add("MassCore");
		DynamicallyLoadedModuleNames.Add("MassEntity");
		DynamicallyLoadedModuleNames.Add("MassSignals");

		if (Target.bBuildEditor == true)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"InterchangeCore",
					"Kismet",
					"ToolMenus",
					"UnrealEd",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AssetTools",
					"AssetDefinition",
					"Documentation",
					"HierarchicalLODUtilities",
					"MeshBuilder",
					"NaniteBuilder",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DerivedDataCache",
					"Kismet",
					"TextureCompressor",
					"UnrealEd"
				}
			);

			CircularlyReferencedDependentModules.AddRange(
				new string[] {
					"UnrealEd",
					"Kismet"
				}
			);
			

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"AnimationModifiers",
					"AssetTools",
					"HierarchicalLODUtilities",
					"LevelInstanceEditor",
					"NaniteBuilder",
					"MassEntityEditor",
					"SVGDistanceField"
				}
			);
		}

		SetupModulePhysicsSupport(Target);

		// Engine public headers need to know about some types (enums etc.)
		PublicIncludePathModuleNames.Add("ClothingSystemRuntimeInterface");
		PublicDependencyModuleNames.Add("ClothingSystemRuntimeInterface");

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.Add("ClothingSystemEditorInterface");
			PrivateIncludePathModuleNames.Add("ClothingSystemEditorInterface");
			PrivateDependencyModuleNames.Add("DesktopPlatform");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// Head Mounted Display support
			//			PrivateIncludePathModuleNames.AddRange(new string[] { "HeadMountedDisplay" });
			//			DynamicallyLoadedModuleNames.AddRange(new string[] { "HeadMountedDisplay" });
		}

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			// Allow VirtualTextureUploadCache to use the UpdateTexture path.
			PublicDefinitions.Add("ALLOW_UPDATE_TEXTURE=1");

			PublicFrameworks.AddRange(new string[] { "AVFoundation", "CoreVideo", "CoreMedia" });
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Android))
		{
			PrivateIncludePathModuleNames.Add("AndroidRuntimeSettings");
			PrivateDefinitions.Add("ENABLE_PLATFORM_COMPRESSION_OVERRIDES=1");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PublicIncludePathModuleNames.Add("IOSPlatformFeatures");
			PrivateIncludePathModuleNames.Add("IOSRuntimeSettings");
			PrivateDefinitions.Add("ENABLE_PLATFORM_COMPRESSION_OVERRIDES=1");
		}

		PublicDefinitions.Add("GPUPARTICLE_LOCAL_VF_ONLY=0");

		// Add a reference to the stats HTML files referenced by UEngine::DumpFPSChartToHTML. Previously staged by CopyBuildToStagingDirectory.
		if (Target.bBuildEditor || Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			RuntimeDependencies.Add("$(EngineDir)/Content/Stats/...", StagedFileType.UFS);
		}

		if (Target.bBuildEditor == false && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PublicDefinitions.Add("WITH_ODSC=1");
		}
        else
        {
			PublicDefinitions.Add("WITH_ODSC=0");
		}

		const bool bIrisAddAsPublicDependency = true;
		SetupIrisSupport(Target, bIrisAddAsPublicDependency);

		PrivateDefinitions.Add("UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES=1");

		bAllowUETypesInNamespaces = true;
	}
}
