// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "RenderTransform.h"

#if WITH_EDITOR
#include "InstanceDataSceneProxy.h"
#endif

struct FInstancedStaticMeshRandomSeed;

/**
 * Shared utility functions for instance data processing.
 * Used by FInstanceDataManager, FPrimitiveInstanceDataManager, and FastGeo to avoid code duplication.
 */
class FInstanceDataHelpers
{
public:
	/**
	 * Hash a primitive transform for spatial hash validation.
	 * Rounds components to int to reduce floating point precision sensitivity.
	 * Used to detect whether a primitive's transform has changed since spatial hashes were computed.
	 */
	ENGINE_API static uint32 HashPrimitiveLocalToWorld(const FRenderTransform& PrimitiveToRelativeWorld, const FVector& PrimitiveWorldSpaceOffset);

	/**
	 * Generate per-instance random IDs using the same deterministic algorithm as UInstancedStaticMeshComponent.
	 * Iterates through instances, advancing a random stream seeded with InstancingRandomSeed,
	 * and resetting the stream at indices specified by AdditionalRandomSeeds.
	 * Fills the provided array in-place. The array must be pre-sized to the desired instance count.
	 */
	ENGINE_API static void GenerateInstanceRandomIDs(int32 InstancingRandomSeed, const TArray<FInstancedStaticMeshRandomSeed>& AdditionalRandomSeeds, TArrayView<float> OutRandomIDs);

	/**
	 * Generate per-instance random IDs and return them as a new array.
	 * @see GenerateInstanceRandomIDs(int32, const TArray<FInstancedStaticMeshRandomSeed>&, TArrayView<float>)
	 */
	ENGINE_API static TArray<float> GenerateInstanceRandomIDs(int32 InstanceCount, int32 InstancingRandomSeed, const TArray<FInstancedStaticMeshRandomSeed>& AdditionalRandomSeeds);

#if WITH_EDITOR
	/** Result of BuildSpatialHashData, containing the compressed spatial hashes and the instance reorder table. */
	struct FSpatialHashResult
	{
		/** Compressed spatial hash items. Each item represents a contiguous range of instances sharing the same spatial hash cell. */
		TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> SpatialHashes;
		/** Reorder table mapping new index -> original index. Empty if instances were already in optimal order. */
		TArray<int32> ReorderTable;
	};

	/**
	 * Build compressed spatial hash data from instance world-space spheres.
	 * Sorts instances by spatial locality using FSpatialHashSortBuilder, then compresses
	 * consecutive same-location instances into FCompressedSpatialHashItem ranges.
	 *
	 * @param NumInstances                  Number of instances to process.
	 * @param GetWorldSpaceInstanceSphere   Callback returning the world-space bounding sphere for each instance index.
	 * @return Compressed spatial hash items and reorder table.
	 */
	ENGINE_API static FSpatialHashResult BuildSpatialHashData(int32 NumInstances, TFunctionRef<FSphere(int32)> GetWorldSpaceInstanceSphere);
#endif
};
