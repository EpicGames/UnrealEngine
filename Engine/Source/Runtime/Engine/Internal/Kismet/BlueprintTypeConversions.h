// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BlueprintTypeConversions.generated.h"

namespace UE::Kismet::BlueprintTypeConversions
{
	using ConversionFunctionT = void(*)(const void*, void*);
	using ConversionFunctionPairT = TPair<ConversionFunctionT, UFunction*>;

	class FStructConversionTable
	{
	public:
		static ENGINE_API const FStructConversionTable& Get();

		ENGINE_API TOptional<UE::Kismet::BlueprintTypeConversions::ConversionFunctionPairT>
			GetConversionFunction(const UScriptStruct* InFrom, const UScriptStruct* InTo) const;

	private:
		FStructConversionTable();

		using StructVariantPairT = TPair<const UScriptStruct*, const UScriptStruct*>;
		using ImplicitCastLookupTableT = TMap<StructVariantPairT, UE::Kismet::BlueprintTypeConversions::ConversionFunctionPairT>;
		using StructVariantLookupTableT = TMap<const UScriptStruct*, const UScriptStruct*>;

		StructVariantPairT GetVariantsFromStructs(const UScriptStruct* InStruct1, const UScriptStruct* InStruct2) const;

		ImplicitCastLookupTableT ImplicitCastLookupTable;
		StructVariantLookupTableT StructVariantLookupTable;

		static FStructConversionTable* Instance;
	};
}

#define MAKE_CONVERSION_EXEC_FUNCTION_NAME(SourceType, DestType)						\
	execConvert##SourceType##To##DestType

#define DECLARE_CONVERSION_FUNCTIONS(BaseType, VariantType1, VariantType2)				\
	DECLARE_FUNCTION(MAKE_CONVERSION_EXEC_FUNCTION_NAME(VariantType1, VariantType2));	\
	DECLARE_FUNCTION(MAKE_CONVERSION_EXEC_FUNCTION_NAME(VariantType2, VariantType1));

UCLASS(MinimalAPI)
class UBlueprintTypeConversions : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	// Container conversions

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API TArray<int> ConvertArrayType(const TArray<int>& InArray);
	DECLARE_FUNCTION(execConvertArrayType);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API TSet<int> ConvertSetType(const TSet<int>& InSet);
	DECLARE_FUNCTION(execConvertSetType);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API TMap<int, int> ConvertMapType(const TMap<int, int>& InMap);
	DECLARE_FUNCTION(execConvertMapType);


	// Custom struct conversions

	DECLARE_CONVERSION_FUNCTIONS(FVector, FVector3f, FVector3d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector3dToFVector3f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector3fToFVector3d(int32 InFromData);

	DECLARE_CONVERSION_FUNCTIONS(FVector2D, FVector2f, FVector2d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector2dToFVector2f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector2fToFVector2d(int32 InFromData);

	DECLARE_CONVERSION_FUNCTIONS(FVector4, FVector4f, FVector4d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector4dToFVector4f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFVector4fToFVector4d(int32 InFromData);


	DECLARE_CONVERSION_FUNCTIONS(FPlane, FPlane4f, FPlane4d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFPlane4dToFPlane4f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFPlane4fToFPlane4d(int32 InFromData);


	DECLARE_CONVERSION_FUNCTIONS(FQuat, FQuat4f, FQuat4d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFQuat4dToFQuat4f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFQuat4fToFQuat4d(int32 InFromData);
	

	DECLARE_CONVERSION_FUNCTIONS(FRotator, FRotator3f, FRotator3d);
	
	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFRotator3dToFRotator3f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFRotator3fToFRotator3d(int32 InFromData);


	DECLARE_CONVERSION_FUNCTIONS(FTransform, FTransform3f, FTransform3d);
	
	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFTransform3dToFTransform3f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFTransform3fToFTransform3d(int32 InFromData);


	DECLARE_CONVERSION_FUNCTIONS(FMatrix, FMatrix44f, FMatrix44d);
	
	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFMatrix44dToFMatrix44f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFMatrix44fToFMatrix44d(int32 InFromData);

	DECLARE_CONVERSION_FUNCTIONS(FBox2D, FBox2f, FBox2d);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFBox2dToFBox2f(int32 InFromData);

	UFUNCTION(CustomThunk, meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API int32 ConvertFBox2fToFBox2d(int32 InFromData);
};
