// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheTagSceneData.h"
#include "PrimitiveComponentId.h"

struct FGuid;
struct FVTProducerDescription;
struct FMaterialCacheTagLayout;
struct FMaterialCacheVirtualTextureAllocation;
class  FSceneInterface;

using FMaterialCacheTagProviderSceneInvalidationDelegate = void(*)(void* Baton);

/** Serves as a general interface to decouple rendering */
class IMaterialCacheTagProvider
{
public:
	virtual ~IMaterialCacheTagProvider() = default;

	/**
	 * Get the scene resource binding data for a given tag
	 * @param Guid Optional, null for default tag
	 * @return Always valid
	 */
	virtual FMaterialCacheTagBindingData GetBindingData(const FGuid& Guid) = 0;

	/**
	 * Get the scene uniform data for a given tag
	 * @param Guid Optional, null for default tag
	 * @return Always valid
	 */
	virtual FMaterialCacheTagUniformData GetUniformData(const FGuid& Guid) = 0;

	/**
	 * Register a new virtual texture for a given primitive
	 * @param Scene The scene it's registered against
	 * @param PrimitiveComponentId The primitive that is to own the virtual texture
	 * @param TagLayout Generated layout of the virtual texture
	 * @param VirtualTexture Texture to be registered
	 */
	virtual void Register(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture) = 0;

	/**
	 * Deregister an existing virtual texture from a given primitive
	 * @param Scene The scene it's registered against
	 * @param PrimitiveComponentId The primitive that owns the virtual texture
	 * @param TagGuid Optional, null for default tag
	 * @param VirtualTexture Texture to be deregistered
	 */
	virtual void Unregister(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture) = 0;
	
	/**
	 * Register a new virtual texture
	 * @param VirtualTexture Texture to be registered
	 */
	virtual void Register(FMaterialCacheVirtualTextureAllocation* VirtualTexture) = 0;
	
	/**
	 * Deregister an existing virtual texture
	 * @param VirtualTexture Texture to be deregistered
	 */
	virtual void Unregister(FMaterialCacheVirtualTextureAllocation* VirtualTexture) = 0;

	/**
	 * Flush all pages of a given tag
	 * @param Scene Scene to flush against
	 * @param PrimitiveComponentId Primitive to flush against
	 * @param TagGuid Optional, null for default tag
	 */
	virtual void Flush(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, const FGuid& TagGuid) = 0;

	/**
	 * Add a callback for whenever the tag scene bindings / data has changed
	 * @param TagGuid The tag to subscribe to
	 * @param Delegate Delegate to be called on invalidations
	 * @param Baton Baton to pass into the callback, also used for broad deregistration
	 */
	virtual void AddTagSceneInvalidationCallback(const FGuid& TagGuid, FMaterialCacheTagProviderSceneInvalidationDelegate Delegate, void* Baton) = 0;

	/**
	 * Remove all tag scene bindings / data listeners
	 * @param Baton Baton whose listeners are to be removed
	 */
	virtual void RemoveTagSceneInvalidationCallbacks(void* Baton) = 0;

	/**
	 * Invalidate all tag scenes
	 */
	virtual void NotifyAllTagSceneInvalidation() = 0;
};
