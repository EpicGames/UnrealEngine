// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheTagSceneData.h"
#include "VirtualTexturing.h"
#include "PrimitiveComponentId.h"

class IAllocatedVirtualTexture;
class FMaterialCacheVirtualProducer;

struct FMaterialCacheVirtualTextureDescription
{
	/** Allow for a single shading pass over the full tile */
	bool bSharedShading = false;
	
	/** Desired/fine producer desc */
	FVTProducerDescription ProducerDesc;
		
	/** Physical memory layout of the cache */
	FMaterialCacheTagLayout TagLayout;
};

struct FMaterialCacheVirtualTextureAllocation
{
	/** Reset backing handles */
	void ResetHandles()
	{
		VirtualTexture = nullptr;
		ProducerHandle = {};
		Producer = nullptr;
	}
	
	/** 
	 * Handle lifetime guaranteed up to allocator updates,
	 * invalidations handled by tag provider subscriptions.
	 */
	
	/** The currently allocated virtual texture */
	IAllocatedVirtualTexture* VirtualTexture = nullptr;
	
	/** The currently allocated producer */
	FMaterialCacheVirtualProducer* Producer = nullptr;
	FVirtualTextureProducerHandle ProducerHandle;
	
	FMaterialCacheVirtualTextureDescription Description;
	
	/** Stable allocation index */
	uint32_t AllocationIndex = 0;
};
