// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveComponentId.h"
#include "IMaterialCacheVirtualTextureAllocation.h"

struct FVTProducerDescription;
struct FMaterialCacheTagLayout;
class  FSceneInterface;

/** Serves as a general interface to decouple rendering */
class IMaterialCacheVirtualTextureAllocator
{
public:
	virtual ~IMaterialCacheVirtualTextureAllocator() = default;

	/**
	 * Allocate a virtual texture
	 * @param RHICmdList Command list used for initialization
	 * @param Scene Owning scene
	 * @param Desc The texture description
	 */
	virtual FMaterialCacheVirtualTextureAllocation* Allocate(
		FRHICommandListBase& RHICmdList,
		FSceneInterface* Scene,
		const FMaterialCacheVirtualTextureDescription& Desc
	) = 0;

	/**
	 * Reallocate a virtual texture
	 * @param Allocation must originate from Allocate(...) 
	 */
	virtual void Reallocate(
		FRHICommandListBase& RHICmdList,
		FMaterialCacheVirtualTextureAllocation* Allocation
	) = 0;

	/**
	 * Deallocate a virtual texture
	 * @param Allocation must originate from Allocate(...) 
	 */
	virtual void Deallocate(FMaterialCacheVirtualTextureAllocation* Allocation) = 0;

	/**
	 * Flush a virtual texture
	 * @param Allocation must originate from Allocate(...) 
	 */
	virtual void Flush(
		FMaterialCacheVirtualTextureAllocation* Allocation,
		const FVector2f& MinUV,
		const FVector2f& MaxUV
	) = 0;
};
