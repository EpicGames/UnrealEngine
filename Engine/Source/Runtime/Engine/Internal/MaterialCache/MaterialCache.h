// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIShaderPlatform.h"
#include "Misc/Optional.h"
#include "Math/IntPoint.h"

/** Get the width of a tile */
ENGINE_API uint32 GetMaterialCacheTileWidth();

/** Get the border width of a tile */
ENGINE_API uint32 GetMaterialCacheTileBorderWidth();

/** Is object space supported (e.g., cook data) for the given platform? */
ENGINE_API bool IsMaterialCacheSupported(FStaticShaderPlatform Platform);

/** Is object space enabled at runtime? May be toggled. */
ENGINE_API bool IsMaterialCacheEnabled(FStaticShaderPlatform Platform);

/** Get the number of tiles needed to represent a surface */
ENGINE_API FIntPoint GetMaterialCacheTileCount(float TriangleArea, float TexelSize, TOptional<FIntPoint> MaxTileCount = {});
