// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelFormat.h"
#include "VirtualTexturingFwd.h"
#include "MaterialValueType.h"
#include "Containers/Array.h"
#include "MaterialCacheAttribute.generated.h"

/** Max number of runtime layers (i.e., render targets and VT layers) */
static constexpr uint32 MaterialCacheMaxRuntimeLayers    = VIRTUALTEXTURE_SPACE_MAXLAYERS;

/** Max number of written tags for a given primitive */
static constexpr uint32 MaterialCacheMaxTagsPerPrimitive = 4u;

/** Debug toggles */
static constexpr bool MaterialCacheDebugUseIdentities = true;

/** Attributes which each material cache texture may store */
UENUM(BlueprintType)
enum class EMaterialCacheAttribute : uint8
{
	/**
	 * General material attributes
	 * Always prefer these over generic formats due to packing and compression constraints
	 **/
	BaseColor,
	Normal,
	Roughness,
	Specular,
	Metallic,
	Opacity,
	WorldPosition,

	/** Displacement, [0-1] normalized */
	Displacement,

	/** World height */
	/** TODO[MP]: Store local to the primitives bounds */
	WorldHeight,

	/** Generic 8-bit mask */
	Mask,

	/** Generic 32-bit float */
	Float
};

/** Attribute identities, effectively known packing schemes */
UENUM(BlueprintType)
enum class EMaterialCacheAttributeIdentity : uint8
{
	None,
	BaseColorRoughness,
	NormalSpecularOpacity,
	MetallicWorldPositionOffset,
	SpecularRoughnessMetallicOpacity,
	RoughnessMetallic,
	Displacement
};

/** All default attributes */
static EMaterialCacheAttribute DefaultMaterialCacheAttributes[] = {
	EMaterialCacheAttribute::BaseColor,
	EMaterialCacheAttribute::Roughness,
	EMaterialCacheAttribute::Normal,
	EMaterialCacheAttribute::Specular,
	EMaterialCacheAttribute::Metallic,
	EMaterialCacheAttribute::Opacity
};

static bool IsMaterialAttribute(EMaterialCacheAttribute Attribute)
{
	switch (Attribute)
	{
	default:
		return false;
	case EMaterialCacheAttribute::BaseColor:
	case EMaterialCacheAttribute::Normal:
	case EMaterialCacheAttribute::Roughness:
	case EMaterialCacheAttribute::Specular:
	case EMaterialCacheAttribute::Metallic:
	case EMaterialCacheAttribute::Opacity:
	case EMaterialCacheAttribute::WorldPosition:
	case EMaterialCacheAttribute::Displacement:
		return true;
	}
}

USTRUCT()
struct FMaterialCacheLayer
{
	GENERATED_BODY()

	/** The intermediate (transient) render format */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	TEnumAsByte<EPixelFormat> RenderFormat = PF_Unknown;

	/** The compressed (stored) format */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	TEnumAsByte<EPixelFormat> StorageFormat = PF_Unknown;

	/** Total number of components in this layer */
	UPROPERTY()
	uint8 ComponentCount = 0;

	/** Is this layer stored in SRGB? */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	bool bIsSRGB = false;

	/** Is this layer stored in YCoCg? */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	bool bYCoCg = false;
	
	/** Optional, attribute identity */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	EMaterialCacheAttributeIdentity Identity = EMaterialCacheAttributeIdentity::None;

	/** All contained attributes */
	UPROPERTY(VisibleAnywhere, Category = Layer)
	TArray<EMaterialCacheAttribute> Attributes;
};

using FMaterialCacheLayerArray = TArray<FMaterialCacheLayer, TInlineAllocator<MaterialCacheMaxRuntimeLayers>>;

/** Thread safe tag layout */
struct FMaterialCacheTagLayout
{
	/** Optional, tag guid */
	FGuid Guid;

	/** All runtime layers of this tag */
	FMaterialCacheLayerArray Layers;
};

/** Test an identity against an attribute set */
static bool MaterialCacheAttributeIdentityTest(TArray<EMaterialCacheAttribute>& Attributes, const TConstArrayView<EMaterialCacheAttribute>& IdentityAttributes)
{
	// Check if the current attribute set contains all identity attributes
	for (EMaterialCacheAttribute IdentityAttribute : IdentityAttributes)
	{
		int32 Index = Attributes.Find(IdentityAttribute);
		if (Index == INDEX_NONE)
		{
			// Attribute missing, identity not valid
			return false;
		}
	}

	// Identity valid, remove all identity attributes
	for (EMaterialCacheAttribute IdentityAttribute : IdentityAttributes)
	{
		Attributes.Remove(IdentityAttribute);
	}

	return true;
}

static uint8 GetMaterialCacheAttributeComponentCount(EMaterialCacheAttribute Attribute, bool bIsStore)
{
	switch (Attribute)
	{
	case EMaterialCacheAttribute::BaseColor:
		return 3;
	case EMaterialCacheAttribute::Normal:
		// Store's in either tangent-space or encoded world-space, which is .xy
		// TODO[MP]: Optionally store in world-space
		return bIsStore ? 2 : 3;
	case EMaterialCacheAttribute::Roughness:
		return 1;
	case EMaterialCacheAttribute::Specular:
		return 1;
	case EMaterialCacheAttribute::Metallic:
		return 1;
	case EMaterialCacheAttribute::Opacity:
		return 1;
	case EMaterialCacheAttribute::WorldPosition:
		return 3;
	case EMaterialCacheAttribute::WorldHeight:
		return 1;
	case EMaterialCacheAttribute::Mask:
		return 1;
	case EMaterialCacheAttribute::Float:
		return 1;
	case EMaterialCacheAttribute::Displacement:
		return 1;
	}

	checkNoEntry();
	return 0;
}

static uint8 GetMaterialCacheLayerAttributeSwizzleOffset(const FMaterialCacheLayer& Layer, EMaterialCacheAttribute Attribute, bool bIsStore)
{
	uint8 Offset = 0;

	for (EMaterialCacheAttribute Contained : Layer.Attributes)
	{
		if (Attribute == Contained)
		{
			return Offset;
		}
		
		Offset += GetMaterialCacheAttributeComponentCount(Contained, bIsStore);
	}

	checkf(false, TEXT("Attribute not present in layer"));
	return 0;
}

static EMaterialValueType GetMaterialCacheAttributeValueType(EMaterialCacheAttribute Attribute)
{
	switch (Attribute)
	{
	case EMaterialCacheAttribute::BaseColor:
		return MCT_Float3;
	case EMaterialCacheAttribute::Normal:
		return MCT_Float3;
	case EMaterialCacheAttribute::Roughness:
		return MCT_Float1;
	case EMaterialCacheAttribute::Specular:
		return MCT_Float1;
	case EMaterialCacheAttribute::Metallic:
		return MCT_Float1;
	case EMaterialCacheAttribute::Opacity:
		return MCT_Float1;
	case EMaterialCacheAttribute::WorldPosition:
		return MCT_Float3;
	case EMaterialCacheAttribute::WorldHeight:
		return MCT_Float1;
	case EMaterialCacheAttribute::Mask:
		return MCT_Float1;
	case EMaterialCacheAttribute::Float:
		return MCT_Float1;
	case EMaterialCacheAttribute::Displacement:
		return MCT_Float1;
	}

	checkNoEntry();
	return MCT_Unknown;
}

static FString GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute Attribute)
{
	switch (Attribute)
	{
	case EMaterialCacheAttribute::BaseColor:
		return TEXT("BaseColor");
	case EMaterialCacheAttribute::Normal:
		return TEXT("Normal");
	case EMaterialCacheAttribute::Roughness:
		return TEXT("Roughness");
	case EMaterialCacheAttribute::Specular:
		return TEXT("Specular");
	case EMaterialCacheAttribute::Metallic:
		return TEXT("Metallic");
	case EMaterialCacheAttribute::Opacity:
		return TEXT("Opacity");
	case EMaterialCacheAttribute::WorldPosition:
		return TEXT("WorldPosition");
	case EMaterialCacheAttribute::WorldHeight:
		return TEXT("WorldHeight");
	case EMaterialCacheAttribute::Mask:
		return TEXT("Mask");
	case EMaterialCacheAttribute::Float:
		return TEXT("Float");
	case EMaterialCacheAttribute::Displacement:
		return TEXT("Displacement");
	}

	checkNoEntry();
	return TEXT("");
}

static FString GetMaterialCacheLayerDecoration(const FMaterialCacheLayer& Layer)
{
	switch (Layer.Identity)
	{
		case EMaterialCacheAttributeIdentity::None:
			break;
		case EMaterialCacheAttributeIdentity::BaseColorRoughness:
			return TEXT("BaseColorSpecular");
		case EMaterialCacheAttributeIdentity::NormalSpecularOpacity:
			return TEXT("NormalSpecularOpacity");
		case EMaterialCacheAttributeIdentity::MetallicWorldPositionOffset:
			return TEXT("MetallicWorldPositionOffset");
		case EMaterialCacheAttributeIdentity::RoughnessMetallic:
			return TEXT("RoughnessMetallic");
		case EMaterialCacheAttributeIdentity::Displacement:
			return TEXT("Displacement");
	}

	FString Composite;

	for (EMaterialCacheAttribute Attribute : Layer.Attributes)
	{
		switch (Attribute)
		{
			case EMaterialCacheAttribute::BaseColor:
				Composite += TEXT("BaseColor");
				break;
			case EMaterialCacheAttribute::Normal:
				Composite += TEXT("Normal");
				break;
			case EMaterialCacheAttribute::Roughness:
				Composite += TEXT("Roughness");
				break;
			case EMaterialCacheAttribute::Specular:
				Composite += TEXT("Specular");
				break;
			case EMaterialCacheAttribute::Metallic:
				Composite += TEXT("Metallic");
				break;
			case EMaterialCacheAttribute::Opacity:
				Composite += TEXT("Opacity");
				break;
			case EMaterialCacheAttribute::WorldPosition:
				Composite += TEXT("WorldPosition");
				break;
			case EMaterialCacheAttribute::WorldHeight:
				Composite += TEXT("WorldHeight");
				break;
			case EMaterialCacheAttribute::Mask:
				Composite += TEXT("Mask");
				break;
			case EMaterialCacheAttribute::Float:
				Composite += TEXT("Float");
				break;
			case EMaterialCacheAttribute::Displacement:
				Composite += TEXT("Displacement");
				break;
		}
	}

	return Composite;
}

struct FMaterialCachePackInfo
{
	/** Enable compression in general? */
	bool bEnableCompression = true;
	
	/** Enable YCoCg for base color storage? */
	bool bEnableBaseColorHQ = true;
	
	/** Enable separate storage for normals? */
	bool bEnableNormalHQ = true;
};

/** Try to pack all attributes down to a set of runtime layers */
template<typename A>
static void PackMaterialCacheAttributeLayers(TArray<EMaterialCacheAttribute>& Attributes, const FMaterialCachePackInfo& Info, TArray<FMaterialCacheLayer, A>& Out)
{
	/** First, try to find the set of identities that we can optimally represent */

	if (MaterialCacheDebugUseIdentities)
	{
		// BaseColor.xyz Roughness.w (-YCoCg)
		if (!Info.bEnableBaseColorHQ && MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::BaseColor, EMaterialCacheAttribute::Roughness })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::BaseColorRoughness;
			Layer.Attributes = { EMaterialCacheAttribute::BaseColor, EMaterialCacheAttribute::Roughness };
			Layer.RenderFormat = PF_R8G8B8A8;
			Layer.StorageFormat = PF_DXT5;
			Layer.ComponentCount = 4;
			Layer.bIsSRGB = true;
			Out.Add(Layer);
		}

		// Normal.xy Specular.z Opacity.w
		if (!Info.bEnableNormalHQ && MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::Normal, EMaterialCacheAttribute::Specular, EMaterialCacheAttribute::Opacity })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::NormalSpecularOpacity;
			Layer.Attributes = { EMaterialCacheAttribute::Normal, EMaterialCacheAttribute::Specular, EMaterialCacheAttribute::Opacity };
			Layer.RenderFormat = PF_A2B10G10R10;
			Layer.StorageFormat = PF_DXT5;
			Layer.ComponentCount = 4;
			Out.Add(Layer);
		}

		// Specular.x Roughness.y Metallic.z Opacity.w
		if (MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::Specular, EMaterialCacheAttribute::Roughness, EMaterialCacheAttribute::Metallic, EMaterialCacheAttribute::Opacity })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::SpecularRoughnessMetallicOpacity;
			Layer.Attributes = { EMaterialCacheAttribute::Specular, EMaterialCacheAttribute::Roughness, EMaterialCacheAttribute::Metallic, EMaterialCacheAttribute::Opacity };
			Layer.RenderFormat = PF_A2B10G10R10;
			Layer.StorageFormat = PF_DXT5;
			Layer.ComponentCount = 4;
			Out.Add(Layer);
		}

		// Metallic.x WorldPosition.yzw
		if (MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::Metallic, EMaterialCacheAttribute::WorldPosition })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::MetallicWorldPositionOffset;
			Layer.Attributes = { EMaterialCacheAttribute::Metallic, EMaterialCacheAttribute::WorldPosition };
			Layer.RenderFormat = PF_R8G8B8A8;
			Layer.StorageFormat = PF_DXT5;
			Layer.ComponentCount = 4;
			Out.Add(Layer);
		}

		// Roughness.x Metallic.y
		if (MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::Roughness, EMaterialCacheAttribute::Metallic })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::RoughnessMetallic;
			Layer.Attributes = { EMaterialCacheAttribute::Roughness, EMaterialCacheAttribute::Metallic };
			Layer.RenderFormat = PF_R8G8;
			Layer.StorageFormat = PF_BC7;
			Layer.ComponentCount = 2;
			Out.Add(Layer);
		}

		// Displacement.x
		if (MaterialCacheAttributeIdentityTest(Attributes, TConstArrayView<EMaterialCacheAttribute>({ EMaterialCacheAttribute::Displacement })))
		{
			FMaterialCacheLayer Layer;
			Layer.Identity = EMaterialCacheAttributeIdentity::Displacement;
			Layer.Attributes = { EMaterialCacheAttribute::Displacement };
			Layer.RenderFormat  = PF_G16;
			Layer.StorageFormat = PF_BC4;
			Layer.ComponentCount = 1;
			Out.Add(Layer);
		}
	}

	/** After that, try to pack together the attributes automatically */

	// TODO[MP]: We're currently allocating a separate layer for each attribute
	// This is temporary of course, we can pack similar attributes down to the same
	// layer which avoids VT limitations. One problem at a time.

	for (EMaterialCacheAttribute Attribute : Attributes)
	{
		FMaterialCacheLayer Layer;
		Layer.Attributes.Add(Attribute);
		
		switch (Attribute)
		{
		case EMaterialCacheAttribute::BaseColor:
			Layer.RenderFormat  = PF_R8G8B8A8;
			Layer.StorageFormat = PF_DXT5;
			Layer.bYCoCg        = Info.bEnableBaseColorHQ;
			Layer.bIsSRGB       = !Info.bEnableBaseColorHQ;
			break;
		case EMaterialCacheAttribute::Normal:
			// bEnableNormalHQ or just not any matching identity
			Layer.RenderFormat  = PF_A2B10G10R10;
			Layer.StorageFormat = PF_BC5;
			break;
		case EMaterialCacheAttribute::Roughness:
		case EMaterialCacheAttribute::Specular:
		case EMaterialCacheAttribute::Metallic:
		case EMaterialCacheAttribute::Opacity:
			Layer.RenderFormat  = PF_R8;
			Layer.StorageFormat = PF_DXT1;
			break;
		case EMaterialCacheAttribute::WorldPosition:
			Layer.RenderFormat  = PF_R8G8B8A8;
			Layer.StorageFormat = PF_DXT5;
			break;
		case EMaterialCacheAttribute::WorldHeight:
			Layer.RenderFormat  = PF_R16F;
			Layer.StorageFormat = PF_R16F;
			break;
		case EMaterialCacheAttribute::Mask:
			Layer.RenderFormat  = PF_R8;
			Layer.StorageFormat = PF_DXT1;
			break;
		case EMaterialCacheAttribute::Float:
			Layer.RenderFormat  = PF_R32_FLOAT;
			Layer.StorageFormat = PF_R32_FLOAT;
			break;
		case EMaterialCacheAttribute::Displacement:
			Layer.RenderFormat  = PF_G16;
			Layer.StorageFormat = PF_BC4;
			break;
		}

		Layer.ComponentCount = GetMaterialCacheAttributeComponentCount(Attribute, true);
		Out.Add(Layer);
	}	
		
	if (!Info.bEnableCompression)
	{
		for (FMaterialCacheLayer& Layer : Out)
		{
			Layer.StorageFormat = Layer.RenderFormat;
			Layer.bYCoCg        = false;
			Layer.bIsSRGB       = false;
		}
	}
}

template<typename A>
static void PackMaterialCacheAttributeLayers(const TArrayView<EMaterialCacheAttribute>& Attributes, const FMaterialCachePackInfo& Info, TArray<FMaterialCacheLayer, A>& Out)
{
	TArray<EMaterialCacheAttribute> AttributesCopy(Attributes);
	PackMaterialCacheAttributeLayers(AttributesCopy, Info, Out);
}

template<typename A>
static void PackMaterialCacheAttributeLayers(const TArrayView<EMaterialCacheAttribute>& Attributes, TArray<FMaterialCacheLayer, A>& Out)
{
	TArray<EMaterialCacheAttribute> AttributesCopy(Attributes);
	PackMaterialCacheAttributeLayers(AttributesCopy, FMaterialCachePackInfo {}, Out);
}
