// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <cstdint>

struct FMaterialCacheDescriptor
{
	union
	{
		struct
		{
			uint32 TagOffset             : 16;
			uint32 Pad                   : 14;
			uint32 UVChannel             : 2;
		};
		
		uint32 Opaque;
	};
};

static FMaterialCacheDescriptor InvalidMaterialCacheDescriptor = {.Opaque = UINT32_MAX };

static_assert(sizeof(FMaterialCacheDescriptor) == sizeof(uint32), "Unexpected size");

ENGINE_API FMaterialCacheDescriptor PackMaterialCacheDescriptor(
	uint32 TagOffset,
	uint32 UVChannel
);
