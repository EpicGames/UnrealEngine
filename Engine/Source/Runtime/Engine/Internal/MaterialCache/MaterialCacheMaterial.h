// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIShaderPlatform.h"

class UMaterialExpression;

ENGINE_API bool MaterialCacheIsExpressionNonUVDerived(const UMaterialExpression* Expression, uint64& UVChannelsUsedMask);
