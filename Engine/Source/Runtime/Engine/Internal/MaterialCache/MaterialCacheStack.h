// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class FMaterialRenderProxy;

struct FMaterialCacheStackEntry
{
	/** The material to be rendered on top of the proxy, one entry per section */
	TArray<const FMaterialRenderProxy*, TInlineAllocator<4u>> SectionMaterials;
};

struct FMaterialCacheStack
{
	/** All material stacks to be composited, respects the given order */
	TArray<FMaterialCacheStackEntry, TInlineAllocator<8u>> Stack;
};
