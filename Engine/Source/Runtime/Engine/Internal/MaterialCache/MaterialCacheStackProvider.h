// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box2D.h"
#include "MaterialCacheStack.h"
#include "MaterialCacheStackProvider.generated.h"

class FMaterialCacheStackProviderRenderProxy
{
public:
	virtual ~FMaterialCacheStackProviderRenderProxy() = default;
	
	/**
	 * Evaluate the material stack.
	 * Called on the render thread.
	 */
	virtual void Evaluate(const FMaterialCacheStack* OutStack) = 0;

#if WITH_EDITOR
	/**
	 * Called prior to stack evaluation to check if all relevant resources are ready.
	 * Called on the render thread.
	 */
	virtual bool IsMaterialResourcesReady() = 0;
#endif // WITH_EDITOR
};

UCLASS(MinimalAPI, abstract)
class UMaterialCacheStackProvider : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Get the representing render proxy
	 * Called on the game thread
	 */
	virtual FMaterialCacheStackProviderRenderProxy* CreateRenderProxy()
	{
		checkNoEntry();
		return nullptr;
	}
};
