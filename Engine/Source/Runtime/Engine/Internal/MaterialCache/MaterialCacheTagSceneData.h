// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "MaterialCacheAttribute.h"

struct FMaterialCacheTagBindingData
{
	/** The tag indirection buffer */
	FRHIShaderResourceView* TagBufferSRV = nullptr;

	/** The shared page table for a given tag, each physical texture of a tag shares the same layout */
	FRHITexture* PageTableSRV = nullptr;

	/** All physical textures of the tag */
	TArray<FRHIShaderResourceView*, TInlineAllocator<MaterialCacheMaxRuntimeLayers>> PhysicalTextureSRVs;
};

struct FMaterialCacheTagUniformData
{
	/** Packed physical texture uniforms, page table uniforms stored in the tag buffer */
	FUintVector4 PackedTableUniform;
};
