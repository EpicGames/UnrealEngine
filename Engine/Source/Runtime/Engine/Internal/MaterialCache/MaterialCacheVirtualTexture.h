// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "MaterialCacheVirtualTextureTag.h"
#include "MaterialCacheVirtualTexture.generated.h"

#define UE_API ENGINE_API

struct FMaterialCacheVirtualTextureAllocation;
class UMaterialCacheStackProvider;
class FMaterialCacheVirtualTextureRenderProxy;
class FMaterialCacheVirtualTextureResource;
class FSceneInterface;
struct FMaterialCacheVirtualBaton;

UCLASS(MinimalAPI, Experimental)
class UMaterialCacheVirtualTexture : public UTexture
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual ~UMaterialCacheVirtualTexture() override;

	/** The primitive component that the cache is rendering on */
	UPROPERTY(VisibleAnywhere, Category="Material Cache")
	TWeakObjectPtr<USceneComponent> OwningComponent;

	/** Optional, the stack provider for compositing */
	UPROPERTY(VisibleAnywhere, Category="Material Cache")
	TWeakObjectPtr<UMaterialCacheStackProvider> MaterialStackProvider;

	/** Optional, tag describing the cache contents */
	UPROPERTY(VisibleAnywhere, Category="Material Cache")
	TObjectPtr<UMaterialCacheVirtualTextureTag> Tag;

	/** The number of tiles to allocate for this given texture, optionally modified by the tag */
	UPROPERTY(VisibleAnywhere, Category="Material Cache")
	FIntPoint TileCount = FIntPoint(8, 8);

	/** Allow for a single shading pass over the full tile */
	UPROPERTY(VisibleAnywhere, Category="Material Cache")
	bool bSharedShading = false;

public:
	/** Flush all relevant pages and re-composite */
	UE_API void Flush();

	/** Deregister this texture from the scene */
	UE_API void Unregister();
	
	/** Recreate the underlying allocation */
	UE_API void RecreateAllocation();

	/** Get the number of tiles to be allocated */
	UE_API FIntPoint GetRuntimeTileCount() const;

	/** Create the render proxy representing this texture */
	UE_API FMaterialCacheVirtualTextureRenderProxy* CreateRenderProxy(const FBox2f& UVRegion, uint32 UVCoordinateIndex);

	/** Get the tag runtime layer layout */
	UE_API FMaterialCacheTagLayout GetRuntimeLayout() const;

public: /** UTexture */
	UE_API virtual void GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const override;
	UE_API virtual EMaterialValueType GetMaterialType() const override;
	UE_API virtual uint32 GetSurfaceArraySize() const override;
	UE_API virtual float GetSurfaceDepth() const override;
	UE_API virtual float GetSurfaceHeight() const override;
	UE_API virtual float GetSurfaceWidth() const override;
	UE_API virtual ETextureClass GetTextureClass() const override;
	UE_API virtual FTextureResource* CreateResource() override;

private:	
	FMaterialCacheVirtualTextureResource* GetPrivateResource();
	FSceneInterface* GetScene();
};

#undef UE_API
