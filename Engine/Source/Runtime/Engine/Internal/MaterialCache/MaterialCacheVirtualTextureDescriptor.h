// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMath.h"
#include "NaniteDefinitions.h"
#include "TextureResource.h"
#include "VirtualTexturing.h"
#include "IMaterialCacheVirtualTextureAllocation.h"

class FTextureResource;

struct UMaterialCacheVirtualTextureDescriptor
{
	operator FUintVector2() const
	{
		FUintVector2 Out;
		FMemory::Memcpy(&Out, this, sizeof(FUintVector2));
		return Out;
	}
	
	// DWord0
	uint32_t PageX : 12;              // 12
	uint32_t PageY : 12;              // 24
	uint32_t PageTableMipBias : 4;    // 28
	uint32_t SpaceID : 4;             // 32

	// DWord1
	uint32_t WidthHeightInPages : 12; // 12
	uint32_t PageFormat : 1;          // 13
	uint32_t Pad : 11;                // 24
	uint32_t MaxLevel : 4;            // 28
	uint32_t AdaptiveBias : 4;        // 32
};

inline UMaterialCacheVirtualTextureDescriptor PackMaterialCacheTextureDescriptor(FMaterialCacheVirtualTextureAllocation* Allocation, uint32_t UVCoordinateIndex) 
{
	checkf(UVCoordinateIndex <= 3 && UVCoordinateIndex < NANITE_MAX_UVS, TEXT("Out of bounds coordinate index, consider expanding bit-width of UVCoordinateIndex"));

	UMaterialCacheVirtualTextureDescriptor Descriptor{};
	if (!Allocation)
	{
		return Descriptor;
	}

	// The backing virtual texture may be null if the allocation failed
	IAllocatedVirtualTexture* VTAllocation = Allocation->VirtualTexture;
	if (!VTAllocation)
	{
		return Descriptor;
	}
	
	check(VTAllocation->GetWidthInBlocks() == VTAllocation->GetHeightInBlocks());
	
	Descriptor.PageX              = VTAllocation->GetVirtualPageX();
	Descriptor.PageY              = VTAllocation->GetVirtualPageY();
	Descriptor.WidthHeightInPages = VTAllocation->GetWidthInTiles();
	Descriptor.PageTableMipBias   = FMath::FloorLog2(VTAllocation->GetVirtualTileSize());
	Descriptor.SpaceID            = VTAllocation->GetSpaceID();
	Descriptor.MaxLevel           = VTAllocation->GetMaxLevel();
	Descriptor.PageFormat         = VTAllocation->GetPageTableFormat() == EVTPageTableFormat::UInt32 ? 1 : 0;
	Descriptor.AdaptiveBias       = VTAllocation->GetAllocatedAdaptiveBias();
	return Descriptor;
}

static_assert(sizeof(UMaterialCacheVirtualTextureDescriptor) == sizeof(FUintVector2), "Unexpected descriptor size");
