// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheStackProvider.h"
#include "Templates/UniquePtr.h"
#include "PrimitiveComponentId.h"
#include "Math/MathFwd.h"

class FSceneInterface;
class FMaterialCacheStackProviderRenderProxy;
struct FMaterialCacheVirtualTextureAllocation;

class FMaterialCacheVirtualTextureRenderProxy
{
public:
	/** Flush all pages of this texture */
	ENGINE_API void Flush(FSceneInterface* Scene);

public:
	/** Optional, effective UV region of this proxy */
	FBox2f UVRegion = FBox2f(FVector2f::Zero(), FVector2f::One());

	/** Optional, owned stack provider render proxy */
	TUniquePtr<FMaterialCacheStackProviderRenderProxy> StackProviderRenderProxy;

	/** Persistent allocation */
	FMaterialCacheVirtualTextureAllocation* Allocation = nullptr;

	/** UV channel used for unwrapping */
	uint32 UVCoordinateIndex = 0;
};
