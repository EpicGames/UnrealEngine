// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "MaterialCacheAttribute.h"
#include "MaterialCacheVirtualTextureTag.generated.h"

#define UE_API ENGINE_API

UCLASS(MinimalAPI, Experimental)
class UMaterialCacheVirtualTextureTag : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** All stored attributes */
	UPROPERTY(EditAnywhere, Category="Material Cache")
	TArray<EMaterialCacheAttribute> Attributes;

	/** Optional, general multiplier to the tile counts of virtual textures using this tag */
	UPROPERTY(EditAnywhere, Category="Material Cache")
	FIntPoint TileCountMultiplier = FIntPoint(1, 1);

	/** Enable compression for all attributes? */
	UPROPERTY(EditAnywhere, Category="Material Cache")
	bool bEnableCompression = true;

	/** Enable high quality base colors? Uses YCoCg for base color, higher memory pressure */
	UPROPERTY(EditAnywhere, Category="Material Cache", meta = (EditCondition = "bEnableCompression"))
	bool bEnableBaseColorHQ = true;

	/** Enable high quality normals? Separates storage into 8:8 octahedron */
	UPROPERTY(EditAnywhere, Category="Material Cache", meta = (EditCondition = "bEnableCompression"))
	bool bEnableNormalHQ = true;

	/** Generated, the packed runtime layers of this tag */
	UPROPERTY(VisibleAnywhere, Category="Material Cache", AdvancedDisplay)
	TArray<FMaterialCacheLayer> RuntimeLayers;

	/** Internal guid for associations */
	UPROPERTY(VisibleAnywhere, Category="Material Cache", AdvancedDisplay)
	FGuid Guid = FGuid::NewGuid();

public:
	/** Get the thread-safe runtime layout of this tag */
	UE_API FMaterialCacheTagLayout GetRuntimeLayout();

public: /** UObject */
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
	void PackRuntimeLayers();

};

#undef UE_API
