// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionMaterialSample.h"
#include "Materials/MaterialParameters.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionMaterialSample)

UMaterialExpressionMaterialSample::UMaterialExpressionMaterialSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if ENABLE_MATERIAL_SAMPLE_PROTOTYPE
	bCollapsed = false;
	bShowOutputNameOnPin = true;
	Outputs.Empty();
	if (Substrate::IsSubstrateEnabled())
	{
		Outputs.Add(FExpressionOutput(TEXT("Front Material")));
	}
#endif
}

#if WITH_EDITOR 
#if ENABLE_MATERIAL_SAMPLE_PROTOTYPE
int32 UMaterialExpressionMaterialSample::DynamicCompile(FMaterialCompiler* Compiler, int32 OutputIndex, bool bCompilePreview)
{
	int32 Result = INDEX_NONE;
	if (MaterialReference)
	{
		if (MaterialReference->GetMaterial() != Material->GetMaterial())
		{
			if (UMaterialEditorOnlyData* EditorData = Cast<UMaterialEditorOnlyData>(MaterialReference->GetEditorOnlyData()))
			{
				UMaterialExpression* CompileExpression = nullptr;
				if (Substrate::IsSubstrateEnabled())
				{
					CompileExpression = EditorData->FrontMaterial.Expression;
				}
				else
				{
					Result = Compiler->Errorf(TEXT("Material Sample is only compatible with Substrate materials."));
				}
				//else MatAttributes?

				if (CompileExpression)
				{
					Result = bCompilePreview ? CompileExpression->CompilePreview(Compiler, OutputIndex) : CompileExpression->Compile(Compiler, OutputIndex);
				}
			}
		}
		else
		{
			Result = Compiler->Errorf(TEXT("Material Sample cannot reference this material or one of it's child instances."));
		}
	}
	return Result;
}

int32 UMaterialExpressionMaterialSample::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return DynamicCompile(Compiler, OutputIndex, false);
}

int32 UMaterialExpressionMaterialSample::CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return DynamicCompile(Compiler, OutputIndex, true);
}

EMaterialValueType UMaterialExpressionMaterialSample::GetOutputValueType(int32 OutputIndex)
{
	if (Substrate::IsSubstrateEnabled())
	{
		return MCT_Substrate;
	}
	else 
	{
		return MCT_MaterialAttributes;
	}
}

bool UMaterialExpressionMaterialSample::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return Substrate::IsSubstrateEnabled() && MaterialReference != nullptr;
}

FSubstrateOperator* UMaterialExpressionMaterialSample::SubstrateGenerateMaterialTopologyTree(class FMaterialCompiler* Compiler, class UMaterialExpression* Parent, int32 OutputIndex)
{
	if (MaterialReference)
	{
		if (MaterialReference->GetMaterial() != Material->GetMaterial())
		{
			if (UMaterialEditorOnlyData* EditorData = Cast<UMaterialEditorOnlyData>(MaterialReference->GetEditorOnlyData()))
			{
				return EditorData->FrontMaterial.Expression->SubstrateGenerateMaterialTopologyTree(Compiler, Parent, OutputIndex);
			}
		}
	}
	return nullptr;
}
#endif //ENABLE_MATERIAL_SAMPLE_PROTOTYPE

void UMaterialExpressionMaterialSample::GetCaption(TArray<FString>& OutCaptions) const
{
	if (MaterialReference)
	{
		OutCaptions.Add(FString("MS ") + MaterialReference->GetName());
	}
	else
	{
		OutCaptions.Add(TEXT("Material Sample"));
	}
}
#endif // WITH_EDITOR
