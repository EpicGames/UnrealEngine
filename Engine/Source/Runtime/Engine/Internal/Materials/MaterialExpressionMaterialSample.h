// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialExpressionMaterialSample - an expression which allows a material to reference another's functional output
 */

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionMaterialSample.generated.h"

#define ENABLE_MATERIAL_SAMPLE_PROTOTYPE 0

class FMaterialCompiler;

UCLASS(MinimalAPI)
class UMaterialExpressionMaterialSample: public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionMaterialSample)
	TObjectPtr<class UMaterialInterface> MaterialReference;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR 
#if ENABLE_MATERIAL_SAMPLE_PROTOTYPE
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual int32 CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual FSubstrateOperator * SubstrateGenerateMaterialTopologyTree(class FMaterialCompiler* Compiler, class UMaterialExpression* Parent, int32 OutputIndex) override; 

	int32 DynamicCompile(FMaterialCompiler* Compiler, int32 OutputIndex, bool bCompilePreview);
#endif
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif

	//~ End UMaterialExpression Interface
};