// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/OverriddenObjectsExternalHandler.h"

#include "OverriddenObjectsExternalPackageHelpers.generated.h"

#if WITH_EDITOR

struct FObjectExport;
class FLinkerLoad;

class FExternalObjectsOverrideHelper
{
	public:

	ENGINE_API FExternalObjectsOverrideHelper(UObject* ForOuter, UClass* InDesiredClass, FName InExternalsPropertyName);
	virtual ~FExternalObjectsOverrideHelper() = default;

	struct FExternalAsset 
	{
		EOverriddenPropertyOperation OverrideOperation;
		FName ObjectPackageName;
		FName AssetName;
		FName RemovedObjectDefaultValue;
		TWeakObjectPtr<UObject> AssetObject;
		FSoftObjectPath AssetPath;
	};

	ENGINE_API void GetOverridesFromExternalObjects();
	ENGINE_API void GetOverridesFromExternalObjects(TArray<FExternalAsset>& InExternalAssets, bool bOnlyDiskAssets = true ) const;
	ENGINE_API void LoadExternalObjects(bool bFlush);
	ENGINE_API void Flush();

	ENGINE_API void SetConsiderExternalsWithoutPropertyTagsAsAdded(bool bValue);
	
	ENGINE_API void HandleRemove(UObject* RemovedObject, UObject* RemovedObjectArchetype);
	ENGINE_API void HandleAdd(UObject* RemovedObject);
	ENGINE_API void HandleRename(UObject* Object);

	ENGINE_API void RemoveExternalObjectPackage(UObject* Object, bool bCanDirtyPackage);
	ENGINE_API void RemoveExternalObjectPackage(FString ExternalAssetPackageName, FName AssetName, bool bForAnExternalRemove, bool bCanDirtyPackage);
	ENGINE_API void CreateExternalObjectPackage(UObject* Object, bool bCanDirtyPackage);
	ENGINE_API void CreateExternalRemoveObjectPackage(UObject* RemovedObjectArchetype);
	
	ENGINE_API void RemoveExternalPackages(bool bCanDirtyPackages);
	ENGINE_API void CreateExternalPackages(bool bCanDirtyPackages);

	ENGINE_API void GetAssetRegistryTags(const UObject* InObjectInOuter, FAssetRegistryTagsContext Context);

	ENGINE_API static void ImportExternalOverrides(const FLinkerLoad& Linker, const FObjectExport& Export, TArray<FDynamicImport>& OutDynamicImports);

	template<class T, typename RemoveFn, typename AddFn, typename ModFn>
	void ApplyTo(TArray<T>& Objects, RemoveFn RemoveFunctor, AddFn AddFunctor, ModFn ModifyFunctor)
	{
		for (const FExternalAsset& ExternalAsset : ExternalAssets)
		{	
			if (!ExternalAsset.AssetObject.IsValid())
			{
				// In the case where no asset object was able to be loaded we need to skip the apply there's nothing that can be done
				// The loading code will a warning about this
				continue;
			}

			switch (ExternalAsset.OverrideOperation)
			{
				case EOverriddenPropertyOperation::Add:
				case EOverriddenPropertyOperation::Replace:
				{
					AddFunctor(ExternalAsset.AssetObject.Get());
				}
				break;
				case EOverriddenPropertyOperation::Remove:
				{
					int32 IndexOfRemoved = Objects.IndexOfByPredicate([&ExternalAsset](UObject* Object)
					{
						if (Object->GetArchetype() == ExternalAsset.AssetObject.Get())
						{
							return true;
						}

						// before subobject reinstance we sometime have the archetypes subobjects in the array, remove them too
						if (Object == ExternalAsset.AssetObject.Get())
						{
							return true;
						}

						return false;
					});

					if (IndexOfRemoved != INDEX_NONE)
					{
						RemoveFunctor(Objects[IndexOfRemoved], IndexOfRemoved);
					}
				}
				break;
				case EOverriddenPropertyOperation::None:
				{
					int32 IndexOfArchetype= Objects.IndexOfByPredicate([&ExternalAsset](UObject* Object)
					{
						if (Object == ExternalAsset.AssetObject->GetArchetype())
						{
							return true;
						}

						return false;
					});

					if (IndexOfArchetype != INDEX_NONE)
					{
						ModifyFunctor(ExternalAsset.AssetObject.Get(), IndexOfArchetype);
					}
				}
				break;
				default:
				{
					// nothing
				}
			}
		}
	}

	ENGINE_API void InitializeObjectArrayOverridesFromExternalAssets(TNonNullPtr<FOverridableObjectArrayOverrides> InArrayOverrides, TArray<FExternalAsset>& InFromExternalAssets, bool bLogInvalidAssets);
	ENGINE_API void InitializeObjectArrayOverridesFromObject(TNonNullPtr<FOverridableObjectArrayOverrides> InArrayOverrides, TNonNullPtr<FProperty> InProperty, TNonNullPtr<UObject> InObject);

	virtual FString GetPackageName(UObject* Object) = 0;
	virtual UObject* GetOuterForRemovePlaceholder(UObject* Object, UObject* ObjectOuteredIn) = 0;
	
	TArray<FExternalAsset> ExternalAssets;
	TArray<int32> PackagesAsyncLoadRequests;
	UObject* Outer = nullptr;
	UClass* DesiredClass =  nullptr;	
	FName PackageNameInMemory;
	FName ExternalsPropertyName;	
	bool bExternalAssetsScanned = false;
	bool bConsiderExternalsWithoutPropertyTagsAsAdded = false;
};
#endif //#if WITH_EDITOR

UCLASS(MinimalAPI)
class UOverrideRemovePlaceholder : public UObject
{
	GENERATED_BODY()

public:
	UOverrideRemovePlaceholder ();

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FSoftObjectPath OverrideRemoved;

	UPROPERTY() 
	FString OverrideProperty;

	ENGINE_API virtual bool IsEditorOnly() const override;
	ENGINE_API virtual bool IsAsset() const override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif
};
