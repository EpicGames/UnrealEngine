// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "Async/TaskGraphInterfaces.h"
#include "PhysicsEngine/BodySetup.h"

struct FTriMeshCollisionData;

namespace Chaos
{
	class FImplicitObject;
	class FTriangleMeshImplicitObject;
}

namespace Chaos
{
	namespace Cooking
	{
		ENGINE_API Chaos::FTriangleMeshImplicitObjectPtr BuildSingleTrimesh(const FTriMeshCollisionData& Desc, TArray<int32>& OutFaceRemap, TArray<int32>& OutVertexRemap);

		void BuildConvexMeshes(TArray<Chaos::FImplicitObjectPtr>& OutTriangleMeshes, const FCookBodySetupInfo& InParams);
		void BuildTriangleMeshes(TArray<Chaos::FTriangleMeshImplicitObjectPtr>& OutTriangleMeshes, TArray<int32>& OutFaceRemap, TArray<int32>& OutVertexRemap, const FCookBodySetupInfo& InParams);
	}

	struct FCookHelper
	{
		FCookHelper() = delete;
		FCookHelper(UBodySetup* InSetup);

		TArray<Chaos::FImplicitObjectPtr> SimpleImplicits;
		TArray<Chaos::FTriangleMeshImplicitObjectPtr> ComplexImplicits;
		FBodySetupUVInfo UVInfo;
		TArray<int32> FaceRemap;
		TArray<int32> VertexRemap;

		void Cook();
		void CookAsync(FSimpleDelegateGraphTask::FDelegate CompletionDelegate);
		bool HasWork() const;

		// CancelCookAsync is not guaranteed to have any effect on the work done.
		// If it is called the cook work may be abandoned and the CookAsync may return early.
		// If bCancel is true in the CompletionDelegate the results must be ignored.
		void CancelCookAsync() { bCanceled = true; }
		bool WasCanceled() const { return bCanceled; }

	private:
		UBodySetup* SourceSetup;
		FCookBodySetupInfo CookInfo;
		std::atomic<bool> bCanceled;
	};
}