// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSceneInterface;
class FSceneViewFamily;
class ISceneRenderBuilder;

/**
 * Internal function called from FRendererModule::BeginRenderingViewFamilies.  List of view families is required to handle captures where
 * USceneCaptureComponent2D::ShouldRenderWithMainViewFamily() is true.
 */
ENGINE_API void SceneCaptureUpdateDeferredCapturesInternal(FSceneInterface* Scene, TConstArrayView<const FSceneViewFamily*> ViewFamilies, ISceneRenderBuilder& SceneRenderBuilder);
