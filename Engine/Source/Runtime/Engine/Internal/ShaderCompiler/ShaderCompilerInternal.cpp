// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerInternal.cpp: Implements FShaderCompileInternalUtilities.
=============================================================================*/

#include "ShaderCompilerInternal.h"
#include "ShaderCompiler.h"


void FShaderCompileInternalUtilities::ExecuteShaderCompileJob(FShaderCommonCompileJob& Job)
{
	FShaderCompileUtilities::ExecuteShaderCompileJob(Job);
}

void FShaderCompileInternalUtilities::EnableDumpDebugInfoForRetry(FShaderCommonCompileJob& Job)
{
	Job.ForEachSingleShaderJob(
		[](FShaderCompileJob& SingleJob)
		{
			if (SingleJob.Input.DumpDebugInfoPath.IsEmpty())
			{
				SingleJob.Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(SingleJob.Input);
				// Any reissued jobs due to this condition will dump debug information, so increment the dump count here
				GShaderCompilingManager->IncrementNumDumpedShaderSources();
			}
		}
	);
}

void FShaderCompileInternalUtilities::DumpDebugInfo(FShaderCommonCompileJob& Job)
{
	FShaderCompileInternalUtilities::EnableDumpDebugInfoForRetry(Job);
	FShaderDebugDataContext Ctx;
	Job.OnComplete(Ctx);
}
