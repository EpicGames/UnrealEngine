// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompiler.h: Platform independent shader compilation definitions.
=============================================================================*/

#pragma once

#include "ShaderCompilerCore.h"
#include "ShaderCompilerJobTypes.h"


/** Wrapper for internal shader compiler utilities that can be accessed by plugins for internal use. */
class FShaderCompileInternalUtilities
{
public:
	/** Execute the specified (single or pipeline) shader compile job. */
	static ENGINE_API void ExecuteShaderCompileJob(FShaderCommonCompileJob& Job);

	/** Ensures DumpDebugInfoPath is assigned and tracked by the shader compiling manager. */
	static ENGINE_API void EnableDumpDebugInfoForRetry(FShaderCommonCompileJob& Job);

	/** Explicitly dumps debug information for this shader compile job. Call this if debug info must be dumped earlier than the regular compile job completion, e.g. right before a Fatal error. */
	static ENGINE_API void DumpDebugInfo(FShaderCommonCompileJob& Job);
};

