// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ClosableMpscQueue.h"
#include "Containers/IntrusiveDoubleLinkedList.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Tasks/Task.h"
#include "RHICommandList.h"

class FRDGBuilder;
class FRHICommandList;
class FRenderCommandList;
class FGPUSkinCache;
class FSceneInterface;
class FSkeletalMeshObject;
class FSkeletalMeshDynamicData;
class FSkeletalMeshUpdater;
class FSkeletalMeshUpdateChannel;

/*
	The Skeletal Mesh Updater is an optimized pipeline for processing batches of skeletal mesh updates in parallel
	with other scene rendering tasks. Each scene has an instance of this class. The system supports processing
	commands from multiple channels, where each channel is a separate backend implementation. The resulting work
	is processed in the same update tasks on the rendering timeline.

	Frontend Usage:

	Mesh objects are registered with the system on the game thread to receive a handle. The handle provides an interface to
	push updates as well as release the instance:

		FSkeletalMeshUpdater* Updater = Scene->GetSkeletalMeshUpdater();

		// Creates a new handle associated with the object.
		FSkeletalMeshUpdateHandle Handle = Updater->Create(MyMeshObject);

		// Pushes a new update for the associated object.
		Handle.Update(MyDynamicData);

		// Releases the object from the updater.
		Handle.Release();

	Commands are automatically pushed by the system from the game thread to the render thread at sync points prior to rendering.
	A delegate associated with UE::RenderCommandPipe::FSyncScope is used for this, as it already instruments key sync points.
	Updates are not actually replayed until the next scene render, so an instance can very well receive multiple update requests
	and then get removed. Removal is handled safely by releasing all pending updates and unregistering the instance.

	Backend Usage:

	Updates are processed in stages based on the work that needs to be synced first in the pipeline. Mesh Deformer related work is synced
	first, followed by skin cache, and finally inline skinning that touches mesh draw commands. A background task is kicked afterwards and this
	is useful for performing RHI related updates; e.g. filling data into bone buffers or updating vertex factories, since that work does not
	need to be synced until much later. Ideally only setup work is performed during the foreground stages.

	Implementing a backend requires deriving from TSkeletalMeshUpdatePacket, which is associated with derived FSkeletalMeshObject / FSkeletalMeshDynamicData
	types. As commands are replayed they are filtered into the user-derived TSkeletalMeshUpdatePacket class using template method overrides. Later the process
	method associated with each stage is called in order. See the following example code for usage:

	class FMyMeshObject : public FSkeletalMeshObject { ... };
	class FMyMeshDynamicData : public FSkeletalMeshDynamicData { ... };

	class FMySkeletalMeshUpdatePacket : public TSkeletalMeshUpdatePacket<FMyMeshObject, FMyMeshDynamicData>
	{
	public:
		void Add(MeshObjectType* MeshObject, MeshDynamicDataType* MeshDynamicData)
		{
			// Filter the mesh object into the N stages of processing.
		}

		void Process_SkinCache(FRHICommandList& RHICmdList)
		{
			// Do setup work associated with the skin cache (create buffers, register skin cache entries, etc).
		}

		void Process_Background(FRHICommandList& RHICmdList)
		{
			// Do update work for RHI resources that were allocated during setup.
		}
	}

	// Registers the implementation and sets up a new channel in the updater.
	REGISTER_SKELETAL_MESH_UPDATE_BACKEND(FMySkeletalMeshUpdatePacket);
*/

///////////////////////////////////////////////////////////////////////////////

class FSkeletalMeshDynamicData
{
public:
	virtual ~FSkeletalMeshDynamicData() = default;

	int32 LODIndex = 0;

private:
	FSkeletalMeshDynamicData* Next = nullptr;
	friend FSkeletalMeshUpdateChannel;

	int32 PoolBucketIndex = 0;
	int32 PoolBucketSize  = 0;

	template <typename DynamicDataType>
	friend class TSkeletalMeshDynamicDataPool;
};

template <typename InMeshObjectType, typename InDynamicDataType>
class TSkeletalMeshDynamicData : public FSkeletalMeshDynamicData
{
	using MeshObjectType      = InMeshObjectType;
	using MeshDynamicDataType = InDynamicDataType;

public:
	static void TrimPool() {}

	static MeshDynamicDataType* Acquire(int32 LODIndex)
	{
		return new MeshDynamicDataType;
	}

	static void Release(MeshDynamicDataType* DynamicData)
	{
		delete DynamicData;
	}

protected:
	TSkeletalMeshDynamicData() = default;
};

class FSkeletalMeshDynamicDataPool
{
public:
	static int64 GetPoolBudget();

protected:
	static constexpr int32 NumPoolBuckets = 5;

	int32 GetBucketIndex(int32 NumTransforms) const;

#if COUNTERSTRACE_ENABLED
	void AddStatsMemory(int32 BucketIndex, int32 Size);
#endif
};

template <typename DynamicDataType>
class TSkeletalMeshDynamicDataPool : public FSkeletalMeshDynamicDataPool
{
public:
	DynamicDataType* Acquire(int32 NumTransforms)
	{
		const int32 PoolBucketIndex  = GetBucketIndex(NumTransforms);
		DynamicDataType* DynamicData = GetBucket(PoolBucketIndex).Pop();

		if (DynamicData)
		{
#if COUNTERSTRACE_ENABLED
			AddStatsMemory(PoolBucketIndex, -DynamicData->PoolBucketSize);
#endif
			NumFreeBytes.fetch_sub(DynamicData->PoolBucketSize, std::memory_order_relaxed);
			return DynamicData;
		}

		DynamicData = new DynamicDataType;
		DynamicData->PoolBucketIndex = PoolBucketIndex;
		return DynamicData;
	}

	void Release(DynamicDataType* DynamicData)
	{
		if (DynamicData)
		{
			const int32 DynamicDataSize = DynamicData->Reset();
			DynamicData->PoolBucketSize = DynamicDataSize;
			NumFreeBytes.fetch_add(DynamicDataSize, std::memory_order_relaxed);
#if COUNTERSTRACE_ENABLED
			AddStatsMemory(DynamicData->PoolBucketIndex, DynamicDataSize);
#endif
			GetBucket(DynamicData->PoolBucketIndex).Push(DynamicData);
		}
	}

	void Trim()
	{
		int32 PoolBucketIndex = 0;

		while (NumFreeBytes.load(std::memory_order_relaxed) > GetPoolBudget())
		{
			int32 NumFreeItemBytes = 0;

			// Release more of the less detailed items relative to the high detailed ones to help balance the pools.
			const int32 NumPoolBucketItems = NumPoolBuckets - PoolBucketIndex;

			for (int32 Index = 0; Index < NumPoolBucketItems; ++Index)
			{
				DynamicDataType* DynamicData = GetBucket(PoolBucketIndex).Pop();
				if (!DynamicData)
				{
					break;
				}
				NumFreeItemBytes += DynamicData->PoolBucketSize;
				delete DynamicData;
			}

			if (NumFreeItemBytes > 0)
			{
#if COUNTERSTRACE_ENABLED
				AddStatsMemory(PoolBucketIndex, -NumFreeItemBytes);
#endif
				NumFreeBytes.fetch_sub(NumFreeItemBytes, std::memory_order_relaxed);
			}

			PoolBucketIndex = (PoolBucketIndex + 1) % NumPoolBuckets;
		}
	}

private:
	static TLockFreePointerListFIFO<DynamicDataType, 0>& GetBucket(int32 BucketIndex)
	{
		static TStaticArray<TLockFreePointerListFIFO<DynamicDataType, 0>, NumPoolBuckets> Buckets;
		return Buckets[BucketIndex];
	}

	std::atomic<int64> NumFreeBytes = 0;
};

template <typename DynamicDataType>
class TPooledSkeletalMeshDynamicData : public FSkeletalMeshDynamicData
{
	friend class TSkeletalMeshDynamicDataPool<DynamicDataType>;
	friend class FSkeletalMeshUpdater;

	static TSkeletalMeshDynamicDataPool<DynamicDataType> Pool;

public:
	static void TrimPool()
	{
		Pool.Trim();
	}

	static DynamicDataType* Acquire(int32 LODIndex)
	{
		return Pool.Acquire(LODIndex);
	}

	static void Release(DynamicDataType* DynamicData)
	{
		Pool.Release(DynamicData);
	}

protected:
	TPooledSkeletalMeshDynamicData() = default;
	void Reset() {}
};

template <typename DynamicDataType>
TSkeletalMeshDynamicDataPool<DynamicDataType> TPooledSkeletalMeshDynamicData<DynamicDataType>::Pool;

///////////////////////////////////////////////////////////////////////////////

enum class ESkeletalMeshUpdateStage : uint8
{
	// Filtering of dynamic datas to mesh objects.
	Filter,

	// Processing inline mesh object allocations.
	Inline,

	// Processing mesh deformer mesh object allocations.
	MeshDeformer,

	// Processing skin cache mesh object allocations.
	SkinCache
};

class FSkeletalMeshUpdatePacket
{
public:
	struct FInitializer
	{
		int32 NumAdds = 0;
		int32 NumRemoves = 0;
		int32 NumUpdates = 0;
		ERHIPipeline SkinCachePipeline = ERHIPipeline::Graphics;
	};

	virtual ~FSkeletalMeshUpdatePacket() = default;

	//////////////////////////////////////////////////////////////////////////////
	// Virtual method overrides to process updates by stage. Each method is called in order, and each process stage is synced in order.

	// Called before adding any skeletal mesh elements.
	virtual void Init(const FInitializer& Initializer) {}

	// Process all enqueued commands that must be synced prior to manipulating mesh deformers.
	virtual void ProcessStage_MeshDeformer(FRHICommandList&, UE::Tasks::FTaskEvent& TaskEvent) {}

	// Process all enqueued commands that must be synced prior to manipulating skin cache.
	virtual void ProcessStage_SkinCache(FRHICommandList&, UE::Tasks::FTaskEvent& TaskEvent) {}

	// Process all enqueued commands that must be synced prior to processing mesh draw commands.
	virtual void ProcessStage_Inline(FRHICommandList&, UE::Tasks::FTaskEvent& TaskEvent) {}

	// Process all enqueued commands that must be synced prior to completing the scene render.
	virtual void ProcessStage_Upload(FRHICommandList&) {}

	//////////////////////////////////////////////////////////////////////////////

	void InvalidatePathTracedOutput()
	{
#if RHI_RAYTRACING
		bInvalidatePathTracedOutput = true;
#endif
	}

	bool IsSkinCacheForRayTracingSupported() const
	{
#if RHI_RAYTRACING
		return bSkinCacheForRayTracingSupported;
#else
		return false;
#endif
	}

protected:
	FSceneInterface* Scene = nullptr;
	FGPUSkinCache* GPUSkinCache = nullptr;
	ERHIPipeline GPUSkinCachePipeline = ERHIPipeline::Graphics;

private:
	void Init(FSceneInterface* Scene, FGPUSkinCache* GPUSkinCache, ERHIPipeline GPUSkinCachePipeline, const FInitializer& Initializer);
	void Finalize();

	virtual void TrimPool() {}

#if RHI_RAYTRACING
	bool bSkinCacheForRayTracingSupported = false;
	bool bInvalidatePathTracedOutput = false;
#endif

	friend FSkeletalMeshUpdater;
};

// The base class for implementing a new backend to the skeletal mesh updater.
template <typename InMeshObjectType, typename InMeshDynamicDataType>
class TSkeletalMeshUpdatePacket : public FSkeletalMeshUpdatePacket
{
public:
	using MeshObjectType      = InMeshObjectType;
	using MeshDynamicDataType = InMeshDynamicDataType;

	////////////////////////////////////////////////////////////////////////////////
	// Template method overrides to filter update requests.

	// Filter the update into a container to process by stage.
	void Add(MeshObjectType* MeshObject, MeshDynamicDataType* MeshDynamicData) {}

	///////////////////////////////////////////////////////////////////////////////

	void TrimPool() override
	{
		MeshDynamicDataType::TrimPool();
	}
};

///////////////////////////////////////////////////////////////////////////////

// Handle associated with a registered mesh object. It has move-only semantics. You must call Release() prior to destruction.
class FSkeletalMeshUpdateHandle
{
public:
	FSkeletalMeshUpdateHandle() = default;

	FSkeletalMeshUpdateHandle(FSkeletalMeshUpdateHandle&& RHS)
		: Channel(RHS.Channel)
		, Index(RHS.Index)
	{
		RHS = {};
	}

	FSkeletalMeshUpdateHandle& operator=(FSkeletalMeshUpdateHandle&& RHS)
	{
		Channel     = RHS.Channel;
		Index       = RHS.Index;
		RHS.Channel = nullptr;
		RHS.Index   = INDEX_NONE;
		return *this;
	}

	~FSkeletalMeshUpdateHandle()
	{
		checkf(!Channel, TEXT("Call Release prior to destructing this handle"));
	}

	bool IsValid() const
	{
		return Channel != nullptr;
	}

	template <typename SkeletalMeshDynamicDataType>
	void Update(SkeletalMeshDynamicDataType* MeshDynamicData);

	void Release();

private:
	FSkeletalMeshUpdateChannel* Channel = nullptr;
	uint32 Index = INDEX_NONE;

	friend FSkeletalMeshUpdateChannel;
};

///////////////////////////////////////////////////////////////////////////////

struct FSkeletalMeshUpdateEvent
{
	const FSkeletalMeshObject* MeshObject;
	int32 LODIndex;
};

class FSkeletalMeshUpdater
{
public:
	ENGINE_API FSkeletalMeshUpdater(FSceneInterface* InScene, FGPUSkinCache* InGPUSkinCache);

	//////////////////////////////////////////////////////////////////////////
	// Game Thread Methods

	// Call at creation time to register a new mesh object with the updater.
	template <typename SkeletalMeshObjectType>
	FSkeletalMeshUpdateHandle Create(SkeletalMeshObjectType* MeshObject);

	ENGINE_API void Shutdown();

	void BeginAsyncPushCommands()
	{
		check(IsInGameThread());
		check(!bInAsyncPushCommandsRegion);
		bInAsyncPushCommandsRegion = true;
	}

	void EndAsyncPushCommands()
	{
		check(IsInGameThread());
		check(bInAsyncPushCommandsRegion);
		bInAsyncPushCommandsRegion = false;
		PushCommandsTask.Wait();
		PushCommandsTask = {};
	}

	// This method is added for StateStream purposes and not to change to much code
	ENGINE_API void PushChangesToStream();

	// Call to insert a task to push commands when the game thread task completes.
	ENGINE_API UE::Tasks::FTask AddPushCommandsTask(const UE::Tasks::FTask& PrerequisiteTask);

	//////////////////////////////////////////////////////////////////////////
	// Render Thread Methods

	class FSubmitData
	{
	public:
		static ENGINE_API const FSubmitData EmptyData;

		FSubmitData() = default;
		FSubmitData(const FSubmitData&) = delete;

		// Returns set of events that occurred during this update. Must wait on the Filter task before calling.
		TConstArrayView<FSkeletalMeshUpdateEvent> GetUpdateEvents() const
		{
			check(Tasks.Filter.IsCompleted());
			return UpdateEvents;
		}

		const UE::Tasks::FTask& GetFilterTask() const
		{
			return Tasks.Filter;
		}

		const UE::Tasks::FTask& GetInlineTask() const
		{
			return Tasks.Inline;
		}

		const UE::Tasks::FTask& GetMeshDeformerTask() const
		{
			return Tasks.MeshDeformer;
		}

		const UE::Tasks::FTask& GetSkinCacheTask() const
		{
			return Tasks.SkinCache;
		}

	protected:
		TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator> UpdateEvents;

		struct
		{
			UE::Tasks::FTask Filter;
			UE::Tasks::FTask Inline;
			UE::Tasks::FTask MeshDeformer;
			UE::Tasks::FTask SkinCache;

		} Tasks;

		friend FSkeletalMeshUpdater;
	};

	// Issues setup tasks to process commands pushed from the game side. Use the provided tasks to sync stages as needed. The builder automatically syncs otherwise.
	ENGINE_API const FSubmitData& Submit(FRDGBuilder& GraphBuilder, ERHIPipeline GPUSkinCachePipeline);

	// Waits for tasks associated with the provided update stage.
	ENGINE_API static void WaitForStage(FRDGBuilder& GraphBuilder, ESkeletalMeshUpdateStage Stage);

	//////////////////////////////////////////////////////////////////////////

private:
	struct FTaskData;

	FSceneInterface* Scene;
	FGPUSkinCache* GPUSkinCache;
	FDelegateHandle DelegateHandle;
	TArray<FSkeletalMeshUpdateChannel> Channels;
	UE::Tasks::FTask PushCommandsTask;
	bool bSubmitting = false;
	bool bInAsyncPushCommandsRegion = false;
};

///////////////////////////////////////////////////////////////////////////////

// Macro to register a new backend using the derived packet type. This should only be used in a cpp file as it creates a statically initialized global.
#define REGISTER_SKELETAL_MESH_UPDATE_BACKEND(SkeletalMeshUpdatePacketType)                                      \
	template <>                                                                                                  \
	struct FSkeletalMeshUpdateChannel::TLookupPacket<typename SkeletalMeshUpdatePacketType::MeshObjectType>      \
	{                                                                                                            \
		using Type = SkeletalMeshUpdatePacketType;                                                               \
	};                                                                                                           \
	template <>                                                                                                  \
	struct FSkeletalMeshUpdateChannel::TLookupPacket<typename SkeletalMeshUpdatePacketType::MeshDynamicDataType> \
	{                                                                                                            \
		using Type = SkeletalMeshUpdatePacketType;                                                               \
	};                                                                                                           \
	FSkeletalMeshUpdateChannel::TBackend<SkeletalMeshUpdatePacketType> GSkeletalMeshUpdateChannelBackend_##SkeletalMeshUpdatePacketType;

///////////////////////////////////////////////////////////////////////////////
// Implementation only Classes

// Class for pushing commands associated with a specific backend down the pipeline to be replayed into a packet.
class FSkeletalMeshUpdateChannel
{
	struct FIndexAllocator
	{
		void Free(int32 Index);
		int32 Allocate();
		int32 NumAllocated() const { return Max - FreeList.Num(); }

		UE::FMutex Mutex;
		TArray<int32> FreeList;
		int32 Max = 0;
	};

	struct FOp : public TConcurrentLinearObject<FOp>
	{
		enum class EType : uint8
		{
			Add,
			Update,
			Remove
		};

		int32 HandleIndex;
		EType Type;

		union
		{
			struct
			{
				FSkeletalMeshDynamicData* MeshDynamicData;

			} Data_Update;

			struct
			{
				FSkeletalMeshObject* MeshObject;

			} Data_Add;
		};
	};

	struct FOpQueue
	{
		TClosableMpscQueue<FOp> Queue;
		std::atomic_int32_t NumAdds = 0;
		std::atomic_int32_t NumUpdates = 0;
		std::atomic_int32_t NumRemoves = 0;
		std::atomic_int32_t Num = 0;
	};

	struct FOpStream
	{
		TArray<FOp, FConcurrentLinearArrayAllocator> Ops;
		int32 NumAdds = 0;
		int32 NumRemoves = 0;
		int32 NumUpdates = 0;
		int32 Num = 0;
	};

	struct FSlot
	{
		FSkeletalMeshObject* MeshObject = nullptr;
		FSkeletalMeshDynamicData* DynamicData = nullptr;
		int32 LODIndex = INDEX_NONE;
	};

	struct FSlotRegistry
	{
		TBitArray<> SlotBits;
		TArray<FSlot> Slots;
	};

	struct FBackend : public TIntrusiveDoubleLinkedListNode<FBackend>
	{
		struct FGlobalList
		{
			TIntrusiveDoubleLinkedList<FBackend> List;
			int32 Num = 0;
		};

		static FGlobalList& GetGlobalList();

		ENGINE_API FBackend();
		ENGINE_API virtual ~FBackend();

		virtual TUniquePtr<FSkeletalMeshUpdatePacket> CreatePacket() const = 0;
		virtual void Replay(FRHICommandList& RHICmdList, FSkeletalMeshUpdateChannel& Channel, FSkeletalMeshUpdatePacket& Packet, TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator>& UpdateEvents) const = 0;

		int32 GlobalListIndex = INDEX_NONE;
		TArray<FSkeletalMeshUpdateChannel*> Channels;
	};

	template <typename T>
	struct TLookupPacket
	{
		static_assert(std::is_same<T, void>::value, "TLookupPacket: No specialization available for the given type!");
	};

public:
	template <typename SkeletalMeshUpdatePacketType>
	class TBackend final : private FBackend
	{
	public:
		TBackend()
		{
			checkf(!Instance, TEXT("Multiple skeletal mesh update backends of the same type were instantiated. Only one is allowed at a time."));
			Instance = this;
		}

	private:
		static TBackend* Instance;

		static int32 GetGlobalListIndex()
		{
			checkf(Instance, TEXT("Skeletal mesh backend instance was not instantiated."));
			return Instance->GlobalListIndex;
		}

		static TBackend* GetInstance()
		{
			return Instance;
		}

		TUniquePtr<FSkeletalMeshUpdatePacket> CreatePacket() const override
		{
			return TUniquePtr<FSkeletalMeshUpdatePacket>(new SkeletalMeshUpdatePacketType());
		}

		void Replay(FRHICommandList& RHICmdList, FSkeletalMeshUpdateChannel& Channel, FSkeletalMeshUpdatePacket& Packet, TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator>& UpdateEvents) const override
		{
			Channel.Replay(RHICmdList, static_cast<SkeletalMeshUpdatePacketType&>(Packet), UpdateEvents);
		}

		friend FSkeletalMeshUpdateChannel;
	};

	template <typename T>
	static int32 GetChannelIndex()
	{
		return TBackend<typename TLookupPacket<T>::Type>::GetGlobalListIndex();
	}

	template <typename T>
	bool IsChannelFor() const
	{
		return TBackend<typename TLookupPacket<T>::Type>::GetInstance() == Backend;
	}

	static TArray<FSkeletalMeshUpdateChannel> GetChannels();
	FSkeletalMeshUpdateChannel(FBackend* InBackend);
	ENGINE_API ~FSkeletalMeshUpdateChannel();
	
	FSkeletalMeshUpdateHandle Create(FSkeletalMeshObject* MeshObject);
	void Update(const FSkeletalMeshUpdateHandle& Handle, FSkeletalMeshDynamicData* MeshDynamicData);
	void Release(FSkeletalMeshUpdateHandle&& Handle);

private:
	TUniquePtr<FSkeletalMeshUpdatePacket> CreatePacket()
	{
		checkf(Backend, TEXT("Backend was released but the channel is still being used."));
		return Backend->CreatePacket();
	}

	void Replay(FRHICommandList& RHICmdList, FSkeletalMeshUpdatePacket& Packet, TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator>& UpdateEvents)
	{
		checkf(Backend, TEXT("Backend was released but the channel is still being used."));
		Backend->Replay(RHICmdList, *this, Packet, UpdateEvents);
	}

	void Shutdown();

	// Methods to push ops from the game thread queue to render thread op stream.
	TUniquePtr<FOpQueue> PopFromQueue();
	void PushToStream(TUniquePtr<FOpQueue>&& Ops);

	template <typename SkeletalMeshUpdatePacketType>
	void Replay(FRHICommandList& RHICmdList, SkeletalMeshUpdatePacketType& Packet, TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator>& UpdateEvents);

	FSkeletalMeshUpdatePacket::FInitializer GetPacketInitializer() const
	{
		return FSkeletalMeshUpdatePacket::FInitializer
		{
			  .NumAdds    = OpStream.NumAdds
			, .NumRemoves = OpStream.NumRemoves
			, .NumUpdates = OpStream.NumUpdates
		};
	}

	FIndexAllocator IndexAllocator;
	TUniquePtr<FOpQueue> OpQueue;

	// Protects OpQueue pointer lifetime for concurrent producer access.
	// Producers (Create/Update/Release) take a shared lock to ensure the queue
	// is not swapped out mid-Enqueue. PopFromQueue takes an exclusive lock to
	// swap the queue. The MPSC queue handles concurrent enqueues; this lock
	// only protects the pointer, not the individual enqueue operations.
	FRWLock OpQueueLock;

	FOpStream OpStream;
	FSlotRegistry SlotRegistry;
	FBackend* Backend = nullptr;
	int32 ChannelIndex;

	friend FSkeletalMeshUpdater;
};

template <typename SkeletalMeshUpdatePacketType>
FSkeletalMeshUpdateChannel::TBackend<SkeletalMeshUpdatePacketType>* FSkeletalMeshUpdateChannel::TBackend<SkeletalMeshUpdatePacketType>::Instance = nullptr;

///////////////////////////////////////////////////////////////////////////////

#include "SkeletalMeshUpdater.inl"