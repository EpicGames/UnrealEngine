// Copyright Epic Games, Inc. All Rights Reserved.

///////////////////////////////////////////////////////////////////////////////

template <typename SkeletalMeshMeshDynamicDataType>
void FSkeletalMeshUpdateHandle::Update(SkeletalMeshMeshDynamicDataType* MeshDynamicData)
{
	if (ensure(Channel))
	{
		checkf(Channel->IsChannelFor<SkeletalMeshMeshDynamicDataType>(), TEXT("Provided MeshDynamicData is not the correct type for this handle."));
		Channel->Update(*this, MeshDynamicData);
	}
}

inline void FSkeletalMeshUpdateHandle::Release()
{
	if (Channel)
	{
		Channel->Release(MoveTemp(*this));
	}
}

///////////////////////////////////////////////////////////////////////////////

template <typename SkeletalMeshUpdatePacketType>
void FSkeletalMeshUpdateChannel::Replay(FRHICommandList& RHICmdList, SkeletalMeshUpdatePacketType& UpdatePacket, TArray<FSkeletalMeshUpdateEvent, FConcurrentLinearArrayAllocator>& Events)
{
	check(IsInParallelRenderingThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshUpdateChannel::Replay);

	using MeshObjectType      = typename SkeletalMeshUpdatePacketType::MeshObjectType;
	using MeshDynamicDataType = typename SkeletalMeshUpdatePacketType::MeshDynamicDataType;

	Events.Reserve(OpStream.NumUpdates);

	for (FOp Op : OpStream.Ops)
	{
		switch (Op.Type)
		{
		case FOp::EType::Add:
		{
			const int32 ExpectedSize = Op.HandleIndex + 1;
			if (SlotRegistry.Slots.Num() < ExpectedSize)
			{
				SlotRegistry.Slots.SetNum(ExpectedSize);
				SlotRegistry.SlotBits.SetNum(ExpectedSize, false);
			}
			SlotRegistry.Slots[Op.HandleIndex].MeshObject = Op.Data_Add.MeshObject;
		}
		break;
		case FOp::EType::Remove:
		{
			FSlot& Slot = SlotRegistry.Slots[Op.HandleIndex];
			if (Slot.DynamicData)
			{
				UpdatePacket.Free(static_cast<MeshDynamicDataType*>(Slot.DynamicData));
			}
			Slot = {};
			SlotRegistry.SlotBits[Op.HandleIndex] = false;
		}
		break;
		case FOp::EType::Update:
		{
			FSlot& Slot = SlotRegistry.Slots[Op.HandleIndex];
			check(Slot.MeshObject);
			const int32 LODIndex = Op.Data_Update.MeshDynamicData->LODIndex;
			Slot.DynamicData = Op.Data_Update.MeshDynamicData;
			Slot.LODIndex = LODIndex;
			SlotRegistry.SlotBits[Op.HandleIndex] = true;
		}
		break;
		}
	}

	for (TConstSetBitIterator<> It(SlotRegistry.SlotBits); It; ++It)
	{
		FSlot& Slot = SlotRegistry.Slots[It.GetIndex()];
		auto* MeshObject = static_cast<MeshObjectType*>(Slot.MeshObject);
		UpdatePacket.Add(MeshObject, static_cast<MeshDynamicDataType*>(Slot.DynamicData));

		Events.Emplace(FSkeletalMeshUpdateEvent
		{
			  .MeshObject = Slot.MeshObject
			, .LODIndex   = Slot.LODIndex
		});

		Slot.DynamicData = nullptr;
	}

	SlotRegistry.SlotBits.Init(false, SlotRegistry.SlotBits.Num());
	OpStream = {};
}

///////////////////////////////////////////////////////////////////////////////

template <typename SkeletalMeshObjectType>
FSkeletalMeshUpdateHandle FSkeletalMeshUpdater::Create(SkeletalMeshObjectType* MeshObject)
{
	return Channels[FSkeletalMeshUpdateChannel::GetChannelIndex<SkeletalMeshObjectType>()].Create(MeshObject);
}