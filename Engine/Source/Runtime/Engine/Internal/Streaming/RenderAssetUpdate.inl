// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
RenderAssetUpdate.inl: Base class of helpers to stream in and out texture/mesh LODs
=============================================================================*/

#pragma once

#include "RenderAssetUpdate.h"

template <typename TContext>
TRenderAssetUpdate<TContext>::TRenderAssetUpdate(const UStreamableRenderAsset* InAsset)
	: FRenderAssetUpdate(InAsset)
	, TaskThread(TT_None)
	, TaskCallback(nullptr)
	, CancelationThread(TT_None)
	, CancelationCallback(nullptr)
{

}

template <typename TContext>
void TRenderAssetUpdate<TContext>::PushTask(const FContext& Context, EThreadType InTaskThread, const FCallback& InTaskCallback, EThreadType InCancelationThread, const FCallback& InCancelationCallback)
{
	// PushTask can only be called by the one thread/callback that is doing the processing. 
	// This means we don't need to check whether other threads could be trying to push tasks.
	check(TaskState == TS_Locked || TaskState == TS_Init);
	checkSlow((bool)InTaskCallback == (InTaskThread != TT_None));
	checkSlow((bool)InCancelationCallback == (InCancelationThread != TT_None));

	TaskThread = InTaskThread;
	TaskCallback = InTaskCallback;
	CancelationThread = InCancelationThread;
	CancelationCallback = InCancelationCallback;
}

template <typename TContext>
FRenderAssetUpdate::ETaskState TRenderAssetUpdate<TContext>::TickInternal(EThreadType InCurrentThread, bool bCheckForSuspension)
{
	check(TaskState == TS_Locked);

	// Thread, callback and asset must be coherent because Abort() could be called while this is executing.
	EThreadType RelevantThread = TaskThread;
	FCallback RelevantCallback = TaskCallback;
	const UStreamableRenderAsset* RelevantAsset = StreamableAsset;
	if (bIsCancelled || !RelevantAsset)
	{
		RelevantThread = CancelationThread;
		RelevantCallback = CancelationCallback;
		// RelevantAsset = nullptr; // TODO once Cancel supports it!
	}

	if (RelevantThread == TT_None)
	{
		ClearCallbacks();
		return TS_Done;
	}
	else if (TaskSynchronization.GetValue() > 0)
	{
		return TS_Suspended;
	}
	else if (bCheckForSuspension && IsAssetStreamingSuspended())
	{
		return TS_Suspended;
	}
	else if (bDeferExecution)
	{
		bDeferExecution = false;
		return TS_Suspended;
	}
	else if (RelevantThread == InCurrentThread)
	{
		ClearCallbacks();
		FContext Context(RelevantAsset, InCurrentThread);
		RelevantCallback(Context);
		return TS_Locked;
	} 
	else if (RelevantThread == FRenderAssetUpdate::TT_GameThread && !ScheduledGTTasks)
	{
		ScheduleGTTask();
		return TS_Locked;
	}
	else if (RelevantThread == FRenderAssetUpdate::TT_Render && !ScheduledRenderTasks)
	{
		if (GIsThreadedRendering)
		{
			// Note that it could execute now if this is the renderthread.
			ScheduleRenderTask();
			return TS_Locked;
		}
		else
		{
			// If render commands do not have their own thread, avoid unrelated thread to enqueue commands.
			// This is to avoid issues from using RHIAsyncCreateTexture2D which doesn't give any feedback of when it is ready.
			return TS_InProgress;
		}
	}
	else if (RelevantThread == FRenderAssetUpdate::TT_Async && !ScheduledAsyncTasks)
	{
		ScheduleAsyncTask();
		return TS_Locked;
	}
	else
	{
		return TS_InProgress;
	}
}
