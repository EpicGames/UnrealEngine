// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
SkeletalMeshUpdate.cpp: Helpers to stream in and out skeletal mesh LODs.
=============================================================================*/

#include "Streaming/SkeletalMeshUpdate.h"
#include "HAL/PlatformFile.h"
#include "RenderUtils.h"
#include "ContentStreaming.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Serialization/MemoryReader.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "Components/SkinnedMeshComponent.h"
#include "Streaming/RenderAssetUpdate.inl"
#include "RHIResourceReplace.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

extern int32 GStreamingMaxReferenceChecks;

template class TRenderAssetUpdate<FSkelMeshUpdateContext>;

static constexpr uint32 GSkelMeshMaxNumResourceUpdatesPerLOD = 16;
static constexpr uint32 GSkelMeshMaxNumResourceUpdatesPerBatch = (MAX_MESH_LOD_COUNT - 1) * GSkelMeshMaxNumResourceUpdatesPerLOD;

FSkelMeshUpdateContext::FSkelMeshUpdateContext(const USkeletalMesh* InMesh, EThreadType InCurrentThread)
	: Mesh(InMesh)
	, CurrentThread(InCurrentThread)
{
	check(InMesh);
	checkSlow(InCurrentThread != FSkeletalMeshUpdate::TT_Render || IsInRenderingThread());
	RenderData = Mesh->GetResourceForRendering();
	AssetLODBias = Mesh->GetStreamableResourceState().AssetLODBias;
	if (RenderData)
	{
		LODResourcesView = TArrayView<FSkeletalMeshLODRenderData*>(RenderData->LODRenderData.GetData() + AssetLODBias, Mesh->GetStreamableResourceState().MaxNumLODs);
	}
}

FSkelMeshUpdateContext::FSkelMeshUpdateContext(const UStreamableRenderAsset* InMesh, EThreadType InCurrentThread)
#if UE_BUILD_SHIPPING
	: FSkelMeshUpdateContext(static_cast<const USkeletalMesh*>(InMesh), InCurrentThread)
#else
	: FSkelMeshUpdateContext(Cast<USkeletalMesh>(InMesh), InCurrentThread)
#endif
{}

FSkeletalMeshUpdate::FSkeletalMeshUpdate(const USkeletalMesh* InMesh)
	: TRenderAssetUpdate<FSkelMeshUpdateContext>(InMesh)
{
}

void FSkeletalMeshStreamIn::FIntermediateBuffers::CreateFromCPUData(FRHICommandListBase& RHICmdList, FSkeletalMeshLODRenderData& LODResource)
{
	FStaticMeshVertexBuffers& VBs = LODResource.StaticVertexBuffers;
	TangentsVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTangentsRHIBuffer(RHICmdList);
	TexCoordVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTexCoordRHIBuffer(RHICmdList);
	PositionVertexBuffer = VBs.PositionVertexBuffer.CreateRHIBuffer(RHICmdList);
	ColorVertexBuffer = VBs.ColorVertexBuffer.CreateRHIBuffer(RHICmdList);
	LODResource.SkinWeightProfilesData.CreateRHIBuffers(RHICmdList, AltSkinWeightVertexBuffers);
	SkinWeightVertexBuffer = LODResource.SkinWeightVertexBuffer.CreateRHIBuffer(RHICmdList);
	ClothVertexBuffer = LODResource.ClothVertexBuffer.CreateRHIBuffer(RHICmdList);
	IndexBuffer = LODResource.MultiSizeIndexContainer.CreateRHIBuffer(RHICmdList);
	HalfEdgeBuffer = LODResource.HalfEdgeBuffer.CreateRHIBuffer(RHICmdList);
}

void FSkeletalMeshStreamIn::FIntermediateBuffers::TransferBuffers(FSkeletalMeshLODRenderData& LODResource, FRHIResourceReplaceBatcher& Batcher)
{
	FStaticMeshVertexBuffers& VBs = LODResource.StaticVertexBuffers;
	VBs.StaticMeshVertexBuffer.InitRHIForStreaming(TangentsVertexBuffer, TexCoordVertexBuffer, Batcher);
	VBs.PositionVertexBuffer.InitRHIForStreaming(PositionVertexBuffer, Batcher);
	VBs.ColorVertexBuffer.InitRHIForStreaming(ColorVertexBuffer, Batcher);
	LODResource.SkinWeightVertexBuffer.InitRHIForStreaming(SkinWeightVertexBuffer, Batcher);
	LODResource.ClothVertexBuffer.InitRHIForStreaming(ClothVertexBuffer, Batcher);
	LODResource.MultiSizeIndexContainer.InitRHIForStreaming(IndexBuffer, Batcher);
	LODResource.SkinWeightProfilesData.InitRHIForStreaming(AltSkinWeightVertexBuffers, Batcher);
	LODResource.HalfEdgeBuffer.InitRHIForStreaming(HalfEdgeBuffer, Batcher);
}

#if RHI_RAYTRACING

void FSkeletalMeshStreamIn::FIntermediateRayTracingGeometry::CreateFromCPUData(FRHICommandListBase& RHICmdList, FRayTracingGeometry& RayTracingGeometry)
{
	Initializer = RayTracingGeometry.GetInitializer();
	Initializer.Type = ERayTracingGeometryInitializerType::StreamingSource;

	if (RayTracingGeometry.RawData.Num())
	{
		check(!RayTracing::ShouldForceRuntimeBLAS());
		check(Initializer.OfflineData == nullptr);
		Initializer.OfflineData = &RayTracingGeometry.RawData;
	}

	RayTracingGeometryRHI = RHICmdList.CreateRayTracingGeometry(Initializer);
	bRequiresBuild = Initializer.OfflineData == nullptr || RayTracingGeometryRHI->IsCompressed();
}

void FSkeletalMeshStreamIn::FIntermediateRayTracingGeometry::SafeRelease()
{
	Initializer = {};
	RayTracingGeometryRHI.SafeRelease();
}

void FSkeletalMeshStreamIn::FIntermediateRayTracingGeometry::TransferRayTracingGeometry(FRayTracingGeometry& RayTracingGeometry, FRHIResourceReplaceBatcher& Batcher)
{
	if (ensureMsgf(RayTracingGeometryRHI.IsValid(),
		TEXT("FIntermediateRayTracingGeometry should have a valid RHI object. Was r.RayTracing.Enable toggled between FStaticMeshStreamIn::CreateBuffers(...) and FStaticMeshStreamIn::DoFinishUpdate(...)?")))
	{
		RayTracingGeometry.InitRHIForStreaming(RayTracingGeometryRHI, Batcher);
		RayTracingGeometry.SetRequiresBuild(bRequiresBuild);
	}
}

#endif

FSkeletalMeshStreamIn::FSkeletalMeshStreamIn(const USkeletalMesh* InMesh, EThreadType CreateResourcesThread)
	: FSkeletalMeshUpdate(InMesh)
	, CreateResourcesThread(CreateResourcesThread)
{
	if (!ensure(PendingFirstLODIdx < CurrentFirstLODIdx))
	{
		bIsCancelled = true;
	}
}

FSkeletalMeshStreamIn::~FSkeletalMeshStreamIn()
{
	check(!StreamingRHICmdList);
}

void FSkeletalMeshStreamIn::CreateBuffers(const FContext& Context)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);

	check(Context.Mesh && Context.RenderData);

	StreamingRHICmdList = new FRHICommandList();
	StreamingRHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
	{
		SCOPED_DRAW_EVENTF(*StreamingRHICmdList, SkeletalMesh_StreamIn, TEXT("SkeletalMesh - StreamIn: %s"), Context.Mesh->GetFName());

		for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
		{
			FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];
			IntermediateBuffersArray[LODIndex].CreateFromCPUData(*StreamingRHICmdList, LODResource);

#if RHI_RAYTRACING
			// Skip LODs that have their render data stripped
			if (IsRayTracingEnabled() && Context.Mesh->GetSupportRayTracing() && LODResource.GetNumVertices() > 0 && LODResource.bReferencedByStaticSkeletalMeshObjects_RenderThread)
			{
				IntermediateRayTracingGeometry[LODIndex].CreateFromCPUData(*StreamingRHICmdList, LODResource.StaticRayTracingGeometry);
			}
#endif
		}

		// Use a scope to flush the batcher before updating CurrentFirstLODIdx
		{
			FRHIResourceReplaceBatcher Batcher(*StreamingRHICmdList, GSkelMeshMaxNumResourceUpdatesPerBatch);

			for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
			{
				FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];
				LODResource.IncrementMemoryStats(Context.Mesh->GetHasVertexColors());
				LODResource.InitMorphResources();
				IntermediateBuffersArray[LODIndex].TransferBuffers(LODResource, Batcher);
			}
		}
	}

	StreamingRHICmdList->FinishRecording();
}

void FSkeletalMeshStreamIn::DiscardNewLODs(const FContext& Context)
{
	if (Context.RenderData)
	{
		for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
		{
			FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];
			LODResource.ReleaseCPUResources(true);
		}
	}
}

void FSkeletalMeshStreamIn::DoFinishUpdate(const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);
	check(IsInRenderingThread());

	if (StreamingRHICmdList)
	{
		FRHICommandListImmediate::Get().QueueAsyncCommandListSubmit(StreamingRHICmdList);
		StreamingRHICmdList = nullptr;
	}

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && Context.Mesh->GetSupportRayTracing())
	{
		// Use a scope to flush the batcher before updating CurrentFirstLODIdx
		{
			FRHIResourceReplaceBatcher Batcher(FRHICommandListImmediate::Get(), GSkelMeshMaxNumResourceUpdatesPerBatch);
			for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
			{
				FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIdx];

				if (IsRayTracingEnabled() && LODResource.GetNumVertices() > 0 && LODResource.bReferencedByStaticSkeletalMeshObjects_RenderThread && !LODResource.StaticRayTracingGeometry.IsEvicted())
				{
					IntermediateRayTracingGeometry[LODIdx].TransferRayTracingGeometry(LODResource.StaticRayTracingGeometry, Batcher);
				}

				IntermediateRayTracingGeometry[LODIdx].SafeRelease();
			}
		}

		// Must happen after the batched updates have been flushed
		for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
		{
			FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];

			// Skip LODs that have their render data stripped
			if (LODResource.GetNumVertices() > 0 && LODResource.bReferencedByStaticSkeletalMeshObjects_RenderThread)
			{
				// Under very rare circumstances that we switch ray tracing on/off right in the middle of streaming RayTracingGeometryRHI might not be valid.
				if (IsRayTracingEnabled() && ensure(LODResource.StaticRayTracingGeometry.IsValid() && !LODResource.StaticRayTracingGeometry.IsEvicted()))
				{
					LODResource.StaticRayTracingGeometry.RequestBuildIfNeeded(FRHICommandListImmediate::Get(), ERTAccelerationStructureBuildPriority::Normal);
				}
			}
		}

	}
#endif

	Context.RenderData->PendingFirstLODIdx = Context.RenderData->CurrentFirstLODIdx = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumRequestedLODs);

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && Context.RenderData->bSupportRayTracing)
	{
		((FRayTracingGeometryManager*)GRayTracingGeometryManager)->SetRayTracingGeometryGroupCurrentFirstLODIndex(FRHICommandListImmediate::Get(), Context.RenderData->RayTracingGeometryGroupHandle, Context.RenderData->CurrentFirstLODIdx);
		GRayTracingGeometryManager->RequestUpdateCachedRenderState(Context.RenderData->RayTracingGeometryGroupHandle);
	}
#endif

	MarkAsSuccessfullyFinished();
}

void FSkeletalMeshStreamIn::DoCancel(const FContext& Context)
{
	// TODO: support streaming CPU data for editor builds
	if (!GIsEditor)
	{
		DiscardNewLODs(Context);
	}

	check(!StreamingRHICmdList);
}

FSkeletalMeshStreamOut::FSkeletalMeshStreamOut(const USkeletalMesh* InMesh)
	: FSkeletalMeshUpdate(InMesh)
{
	PushTask(FContext(InMesh, TT_None), TT_GameThread, SRA_UPDATE_CALLBACK(ConditionalMarkComponentsDirty), TT_None, nullptr);
}

void FSkeletalMeshStreamOut::ConditionalMarkComponentsDirty(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshStreamOut::ConditionalMarkComponentsDirty"), STAT_SkeletalMeshStreamOut_ConditionalMarkComponentsDirty, STATGROUP_StreamingDetails);
	CSV_SCOPED_TIMING_STAT_GLOBAL(SkStreamingMarkDirtyTime);
	check(Context.CurrentThread == TT_GameThread);

	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	if (!IsCancelled() && Mesh && RenderData)
	{
		RenderData->PendingFirstLODIdx = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumRequestedLODs);

		TArray<const IPrimitiveComponent*> Comps;
		IStreamingManager::Get().GetRenderAssetStreamingManager().GetAssetComponents(Mesh, Comps);
		for (const IPrimitiveComponent* Component : Comps)
		{
			const_cast<IPrimitiveComponent*>(Component)->OnRenderAssetFirstLodChange(Mesh, RenderData->PendingFirstLODIdx);
		}
	}
	else
	{
		Abort();
	}
	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(WaitForReferences), (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
}

void FSkeletalMeshStreamOut::WaitForReferences(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshStreamOut::WaitForReferences"), STAT_SkeletalMeshStreamOut_WaitForReferences, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	uint32 NumExternalReferences = 0;

	if (Mesh && RenderData)
	{
		for (int32 LODIndex = CurrentFirstLODIdx; LODIndex < PendingFirstLODIdx; ++LODIndex)
		{
			// Minus 1 since the LODResources reference is not considered external
			NumExternalReferences += Context.LODResourcesView[LODIndex]->GetRefCount() - 1;
		}

		if (NumExternalReferences > PreviousNumberOfExternalReferences && NumReferenceChecks > 0)
		{
			PreviousNumberOfExternalReferences = NumExternalReferences;
			UE_LOGF(LogSkeletalMesh, Warning, "[%ls] Streamed out LODResources got referenced while in pending stream out.", *Mesh->GetName());
		}
	}

	if (!NumExternalReferences || NumReferenceChecks >= GStreamingMaxReferenceChecks)
	{
		PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(ReleaseBuffers), (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
		
		// This is required to allow the engine to generate the bone buffers for the PendingFirstLODIdx. See logic in FSkeletalMeshSceneProxy::GetMeshElementsConditionallySelectable().
		if (NumReferenceChecks == 0)
		{
			bDeferExecution = true;
		}
	}
	else
	{
		++NumReferenceChecks;
		if (NumReferenceChecks >= GStreamingMaxReferenceChecks)
		{
			UE_LOGF(LogSkeletalMesh, Warning, "[%ls] Streamed out LODResources references are not getting released.", *Mesh->GetName());
		}

		bDeferExecution = true;
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(WaitForReferences),  (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FSkeletalMeshStreamOut::ReleaseBuffers(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshStreamOut::ReleaseBuffers"), STAT_SkeletalMeshStreamOut_ReleaseBuffers, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);
	check(IsInRenderingThread());

	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	if (!IsCancelled() && Mesh && RenderData)
	{
		RenderData->CurrentFirstLODIdx = Context.RenderData->PendingFirstLODIdx;

		{
			FRHIResourceReplaceBatcher Batcher(FRHICommandListImmediate::Get(), GSkelMeshMaxNumResourceUpdatesPerBatch);

			for (int32 LODIndex = CurrentFirstLODIdx; LODIndex < PendingFirstLODIdx; ++LODIndex)
			{
				FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];
				FStaticMeshVertexBuffers& VBs = LODResource.StaticVertexBuffers;
				LODResource.DecrementMemoryStats();
				VBs.StaticMeshVertexBuffer.ReleaseRHIForStreaming(Batcher);
				VBs.PositionVertexBuffer.ReleaseRHIForStreaming(Batcher);
				VBs.ColorVertexBuffer.ReleaseRHIForStreaming(Batcher);
				LODResource.SkinWeightVertexBuffer.ReleaseRHIForStreaming(Batcher);
				LODResource.ClothVertexBuffer.ReleaseRHIForStreaming(Batcher);
				LODResource.MultiSizeIndexContainer.ReleaseRHIForStreaming(Batcher);
				LODResource.SkinWeightProfilesData.ReleaseRHIForStreaming(Batcher);
				LODResource.HalfEdgeBuffer.ReleaseRHIForStreaming(Batcher);

				if (!FPlatformProperties::HasEditorOnlyData())
				{
					// TODO requires more testing : LODResource.ReleaseCPUResources(true);
				}

#if RHI_RAYTRACING
				if (IsRayTracingAllowed() && RenderData->LODRenderData[LODIndex].bReferencedByStaticSkeletalMeshObjects_RenderThread && !LODResource.StaticRayTracingGeometry.IsEvicted())
				{
					LODResource.StaticRayTracingGeometry.ReleaseRHIForStreaming(Batcher);
				}
#endif
			}
		}

#if RHI_RAYTRACING
		if (IsRayTracingAllowed() && Context.RenderData->bSupportRayTracing)
		{
			((FRayTracingGeometryManager*)GRayTracingGeometryManager)->SetRayTracingGeometryGroupCurrentFirstLODIndex(FRHICommandListImmediate::Get(), Context.RenderData->RayTracingGeometryGroupHandle, Context.RenderData->CurrentFirstLODIdx);
			GRayTracingGeometryManager->RequestUpdateCachedRenderState(Context.RenderData->RayTracingGeometryGroupHandle);
		}
#endif

		MarkAsSuccessfullyFinished();
	}
}

void FSkeletalMeshStreamOut::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshStreamOut::Cancel"), STAT_SkeletalMeshStreamOut_Cancel, STATGROUP_StreamingDetails);

	if (Context.RenderData)
	{
		Context.RenderData->PendingFirstLODIdx = Context.RenderData->CurrentFirstLODIdx;
	}
}

void FSkeletalMeshStreamIn_IO::FCancelIORequestsTask::DoWork()
{
	check(PendingUpdate);
	// Acquire the lock of this object in order to cancel any pending IO.
	// If the object is currently being ticked, wait.
	ETaskState OldState = PendingUpdate->DoLock();
	PendingUpdate->CancelIORequest();
	PendingUpdate->DoUnlock(OldState);
}

void FSkeletalMeshStreamIn_IO::Abort()
{
	if (!IsCancelled() && !IsCompleted())
	{
		FSkeletalMeshStreamIn::Abort();

		if (BulkDataRequest.IsPending())
		{
			// Prevent the update from being considered done before this is finished.
			// By checking that it was not already cancelled, we make sure this doesn't get called twice.
			(new FAsyncCancelIORequestsTask(this))->StartBackgroundTask();
		}
	}
}

void FSkeletalMeshStreamIn_IO::SetIORequest(const FContext& Context)
{
	if (IsCancelled())
	{
		return;
	}
	check(BulkDataRequest.IsNone() && PendingFirstLODIdx < CurrentFirstLODIdx);

	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	if (Mesh && RenderData)
	{
		TRACE_IOSTORE_METADATA_SCOPE_TAG(*Mesh->GetName());

		const int32 BatchCount = CurrentFirstLODIdx - PendingFirstLODIdx;
		FBulkDataBatchRequest::FScatterGatherBuilder Batch = FBulkDataBatchRequest::ScatterGather(BatchCount);
		for (int32 Index = PendingFirstLODIdx; Index < CurrentFirstLODIdx; ++Index)
		{
			Batch.Read(Context.LODResourcesView[Index]->StreamingBulkData);
		}

		// Increment as we push the request. If a request complete immediately, then it will call the callback
		// but that won't do anything because the tick would not try to acquire the lock since it is already locked.
		TaskSynchronization.Increment();

		EAsyncIOPriorityAndFlags Priority = AIOP_Low;
		if (bHighPrioIORequest)
		{
			static IConsoleVariable* CVarAsyncLoadingPrecachePriority = IConsoleManager::Get().FindConsoleVariable(TEXT("s.AsyncLoadingPrecachePriority"));
			const bool bLoadBeforeAsyncPrecache = CVarStreamingLowResHandlingMode.GetValueOnAnyThread() == (int32)FRenderAssetStreamingSettings::LRHM_LoadBeforeAsyncPrecache;

			if (CVarAsyncLoadingPrecachePriority && bLoadBeforeAsyncPrecache)
			{
				const int32 AsyncIOPriority = CVarAsyncLoadingPrecachePriority->GetInt();
				// Higher priority than regular requests but don't go over max
				Priority = (EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AsyncIOPriority + 1, AIOP_BelowNormal, AIOP_MAX);
			}
			else
			{
				Priority = AIOP_BelowNormal;
			}
		}

		Batch.Issue(BulkData, Priority, [this](FBulkDataRequest::EStatus Status)
		{
			// At this point task synchronization would hold the number of pending requests.
			TaskSynchronization.Decrement();

			if (FBulkDataRequest::EStatus::Ok != Status)
			{
				// If IO requests was cancelled but the streaming request wasn't, this is an IO error.
				if (!bIsCancelled)
				{
					bFailedOnIOError = true;
				}
				MarkAsCancelled();
			}

#if !UE_BUILD_SHIPPING
			// On some platforms the IO is too fast to test cancelation requests timing issues.
			if (FRenderAssetStreamingSettings::ExtraIOLatency > 0 && TaskSynchronization.GetValue() == 0)
			{
				FPlatformProcess::Sleep(FRenderAssetStreamingSettings::ExtraIOLatency * .001f); // Slow down the streaming.
			}
#endif
			// The tick here is intended to schedule the success or cancel callback.
			// Using TT_None ensure gets which could create a dead lock.
			Tick(FSkeletalMeshUpdate::TT_None);
		},
		BulkDataRequest);
	}
	else
	{
		MarkAsCancelled();
	}
}

void FSkeletalMeshStreamIn_IO::ClearIORequest(const FContext& Context)
{
	if (BulkDataRequest.IsPending())
	{
		BulkDataRequest.Cancel();
		BulkDataRequest.Wait();
	}
	
	BulkDataRequest = FBulkDataBatchRequest();
	BulkData = FIoBuffer();
}

void FSkeletalMeshStreamIn_IO::ReportIOError(const FContext& Context)
{
	// Invalidate the cache state of all initial mips (note that when using FIoChunkId each mip has a different value).
	if (bFailedOnIOError && Context.Mesh)
	{
		IRenderAssetStreamingManager& StreamingManager = IStreamingManager::Get().GetRenderAssetStreamingManager();
		for (int32 MipIndex = 0; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			StreamingManager.MarkMountedStateDirty(Context.Mesh->GetMipIoFilenameHash(MipIndex));
		}

		UE_LOGF(LogContentStreaming, Warning, "[%ls] SkeletalMesh stream in request failed due to IO error (LOD %d-%d).", *Context.Mesh->GetName(), PendingFirstLODIdx, CurrentFirstLODIdx - 1);
	}
}

void FSkeletalMeshStreamIn_IO::SerializeLODData(const FContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("SkeletalMesh/Serialize"));

	check(!TaskSynchronization.GetValue());
	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	if (!IsCancelled() && Mesh && RenderData)
	{
		check(BulkData.GetSize() >= 0 && BulkData.GetSize() <= TNumericLimits<uint32>::Max());

		FMemoryReaderView Ar(BulkData.GetView(), true);
		for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
		{
			FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];
			const bool bForceKeepCPUResources = FSkeletalMeshLODRenderData::ShouldForceKeepCPUResources();
			const bool bNeedsCPUAccess = FSkeletalMeshLODRenderData::ShouldKeepCPUResources(Mesh, LODIndex + Context.AssetLODBias, bForceKeepCPUResources);
			constexpr uint8 DummyStripFlags = 0;
			LODResource.SerializeStreamedData(Ar, const_cast<USkeletalMesh*>(Mesh), LODIndex + Context.AssetLODBias, DummyStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);

			// Attempt to recover from possibly corrupted data
			if (Ar.IsError())
			{
				UE_LOGF(LogContentStreaming, Error,
					"[%ls] SkeletalMesh stream in failed due to possibly corrupted data. LOD %d %d-%d. BulkData %#x offset %lld size %lld flags %#x. bForceKeepCPUResources %d. bNeedsCPUAccess %d.",
					*Mesh->GetPathName(),
					LODIndex,
					PendingFirstLODIdx,
					CurrentFirstLODIdx - 1,
					LODResource.StreamingBulkData.GetIoFilenameHash(),
					LODResource.StreamingBulkData.GetBulkDataOffsetInFile(),
					LODResource.StreamingBulkData.GetBulkDataSize(),
					LODResource.StreamingBulkData.GetBulkDataFlags(),
					bForceKeepCPUResources,
					bNeedsCPUAccess);

#if STREAMING_RETRY_ON_DESERIALIZATION_ERROR
				bFailedOnIOError = true;
				MarkAsCancelled();
				break;
#else
				GLog->FlushThreadedLogs();
				GLog->Flush();
				UE_LOGF(LogContentStreaming, Fatal, "Possibly corrupted skeletal mesh LOD data detected.");
#endif
			}
		}

		BulkData = FIoBuffer();
	}
}

void FSkeletalMeshStreamIn_IO::Cancel(const FContext& Context)
{
	DoCancel(Context);
	ReportIOError(Context);
}

void FSkeletalMeshStreamIn_IO::CancelIORequest()
{
	if (BulkDataRequest.IsPending())
	{
		BulkDataRequest.Cancel();
	}
}

FSkeletalMeshStreamIn_IO::FSkeletalMeshStreamIn_IO(const USkeletalMesh* InMesh, bool bHighPrio, EThreadType CreateResourcesThread)
	: FSkeletalMeshStreamIn(InMesh, CreateResourcesThread)
	, bHighPrioIORequest(bHighPrio)
{
	PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(DoInitiateIO), TT_None, nullptr);
}

void FSkeletalMeshStreamIn_IO::DoInitiateIO(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);

	SetIORequest(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(DoSerializeLODData), TT_Async, SRA_UPDATE_CALLBACK(DoCancelIO));
}

void FSkeletalMeshStreamIn_IO::DoSerializeLODData(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);
	SerializeLODData(Context);
	ClearIORequest(Context);

	PushTask(Context
		, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers)
		, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
}

void FSkeletalMeshStreamIn_IO::DoCreateBuffers(const FContext& Context)
{
	CreateBuffers(Context);

	check(!TaskSynchronization.GetValue());

	// We cannot cancel once DoCreateBuffers has started executing, as there's an RHICmdList that must be submitted.
	// Pass the same callback for both task and cancel.
	PushTask(Context
		, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate)
		, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate)
	);
}

void FSkeletalMeshStreamIn_IO::DoCancelIO(const FContext& Context)
{
	ClearIORequest(Context);
	PushTask(Context, TT_None, nullptr, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
}

#if WITH_EDITOR
FSkeletalMeshStreamIn_DDC::FSkeletalMeshStreamIn_DDC(const USkeletalMesh* InMesh, EThreadType CreateResourcesThread)
	: FSkeletalMeshStreamIn(InMesh, CreateResourcesThread)
{
	PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(DoLoadNewLODsFromDDC), TT_None, nullptr);
}

void FSkeletalMeshStreamIn_DDC::LoadNewLODsFromDDC(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);
	// TODO: support streaming CPU data for editor builds
}

void FSkeletalMeshStreamIn_DDC::DoLoadNewLODsFromDDC(const FContext& Context)
{
	LoadNewLODsFromDDC(Context);
	check(!TaskSynchronization.GetValue());

	PushTask(Context
		, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers)
		, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(DoCancel));
}

void FSkeletalMeshStreamIn_DDC::DoCreateBuffers(const FContext& Context)
{
	CreateBuffers(Context);
	
	check(!TaskSynchronization.GetValue());
	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate), TT_None, nullptr);
}
#endif
