// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
SkeletalMeshUpdate.h: Helpers to stream in and out skeletal mesh LODs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "IO/IoDispatcher.h"
#include "RenderAssetUpdate.h"
#include "Rendering/SkeletalMeshHalfEdgeBuffer.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Serialization/BulkData.h"
#include "RayTracingGeometry.h"

#define UE_API ENGINE_API


struct FSkinWeightProfileStack;

/**
* A context used to update or proceed with the next update step.
* The mesh and render data references could be stored in the update object
* but are currently kept outside to avoid lifetime management within the object.
*/
struct FSkelMeshUpdateContext
{
	typedef int32 EThreadType;

	UE_API FSkelMeshUpdateContext(const USkeletalMesh* InMesh, EThreadType InCurrentThread);

	UE_API FSkelMeshUpdateContext(const UStreamableRenderAsset* InMesh, EThreadType InCurrentThread);

	EThreadType GetCurrentThread() const
	{
		return CurrentThread;
	}

	/** The mesh to update, this must be the same one as the one used when creating the FSkeletalMeshUpdate object. */
	const USkeletalMesh* Mesh;
	/** The current render data of this mesh. */
	FSkeletalMeshRenderData* RenderData;
	/** The array view of streamable LODs from the asset. Takes into account FStreamableRenderResourceState::AssetLODBias and FStreamableRenderResourceState::MaxNumLODs. */
	TArrayView<FSkeletalMeshLODRenderData*> LODResourcesView;
	/** Cached value of mesh its LOD bias (MinLOD for SkeletalMesh). */
	int32 AssetLODBias = 0;

	/** The thread on which the context was created. */
	EThreadType CurrentThread;
};

extern template class TRenderAssetUpdate<FSkelMeshUpdateContext>;

/**
* This class provides a framework for loading and unloading the LODs of skeletal meshes.
* Each thread essentially calls Tick() until the job is done.
* The object can be safely deleted when IsCompleted() returns true.
*/
class FSkeletalMeshUpdate : public TRenderAssetUpdate<FSkelMeshUpdateContext>
{
public:
	FSkeletalMeshUpdate(const USkeletalMesh* InMesh);

	virtual ~FSkeletalMeshUpdate() = default;

	virtual void Abort()
	{
		TRenderAssetUpdate<FSkelMeshUpdateContext>::Abort();
	}
};

class FSkeletalMeshStreamIn : public FSkeletalMeshUpdate
{
public:
	UE_API FSkeletalMeshStreamIn(const USkeletalMesh* InMesh, EThreadType CreateResourcesThread);

	UE_API virtual ~FSkeletalMeshStreamIn();

protected:
	/** Correspond to the buffers in FSkeletalMeshLODRenderData */
	struct FIntermediateBuffers
	{
		FBufferRHIRef TangentsVertexBuffer;
		FBufferRHIRef TexCoordVertexBuffer;
		FBufferRHIRef PositionVertexBuffer;
		FBufferRHIRef ColorVertexBuffer;
		FSkinWeightRHIInfo SkinWeightVertexBuffer;
		FBufferRHIRef ClothVertexBuffer;
		FBufferRHIRef IndexBuffer;
		TArray<TPair<FSkinWeightProfileStack, FSkinWeightRHIInfo>> AltSkinWeightVertexBuffers;
		FSkeletalMeshHalfEdgeBuffer::FRHIInfo HalfEdgeBuffer;

		void CreateFromCPUData(FRHICommandListBase& RHICmdList, FSkeletalMeshLODRenderData& LODResource);

		/** Transfer ownership of buffers to a LOD resource */
		void TransferBuffers(FSkeletalMeshLODRenderData& LODResource, FRHIResourceReplaceBatcher& Batcher);
	};

#if RHI_RAYTRACING
	struct FIntermediateRayTracingGeometry
	{
	private:
		FRayTracingGeometryInitializer Initializer;
		FRayTracingGeometryRHIRef RayTracingGeometryRHI;
		bool bRequiresBuild = false;

	public:
		void CreateFromCPUData(FRHICommandListBase& RHICmdList, FRayTracingGeometry& RayTracingGeometry);

		void SafeRelease();

		void TransferRayTracingGeometry(FRayTracingGeometry& RayTracingGeometry, FRHIResourceReplaceBatcher& Batcher);
	};
#endif

	/** Create buffers with new LOD data */
	UE_API void CreateBuffers(const FContext& Context);

	/** Discard newly streamed-in CPU data */
	UE_API void DiscardNewLODs(const FContext& Context);

	/** Apply the new buffers (if not cancelled) and finish the update process. When cancelled, the intermediate buffers simply gets discarded. */
	UE_API void DoFinishUpdate(const FContext& Context);

	/** Discard streamed-in CPU data and intermediate RHI buffers */
	UE_API void DoCancel(const FContext& Context);

	/** The intermediate buffers created in the update process. */
	FIntermediateBuffers IntermediateBuffersArray[MAX_MESH_LOD_COUNT];

#if RHI_RAYTRACING
	FIntermediateRayTracingGeometry IntermediateRayTracingGeometry[MAX_MESH_LOD_COUNT];
#endif

	/** RHI command list used for creating buffers and replacing the streaming placeholders. Submitted in DoFinishUpdate */
	FRHICommandList* StreamingRHICmdList = nullptr;

	/** The thread to use for recording the above command list */
	const EThreadType CreateResourcesThread;
};

class FSkeletalMeshStreamOut : public FSkeletalMeshUpdate
{
public:
	FSkeletalMeshStreamOut(const USkeletalMesh* InMesh);

	virtual ~FSkeletalMeshStreamOut() = default;

private:

	int32 NumReferenceChecks = 0;
	uint32 PreviousNumberOfExternalReferences = 0;


	/** Notify components that the LOD is being streamed out so that they can release references. */
	void ConditionalMarkComponentsDirty(const FContext& Context);

	/** Wait for all references to be released. */
	void WaitForReferences(const FContext& Context);

	/** Release RHI buffers and update SRVs. */
	void ReleaseBuffers(const FContext& Context);

	/** Cancel the pending mip change. */
	void Cancel(const FContext& Context);
};

class FSkeletalMeshStreamIn_IO : public FSkeletalMeshStreamIn
{
public:
	FSkeletalMeshStreamIn_IO(const USkeletalMesh* InMesh, bool bHighPrio, EThreadType CreateResourcesThread);

	virtual ~FSkeletalMeshStreamIn_IO() = default;

	virtual void Abort() override;

protected:
	class FCancelIORequestsTask : public FNonAbandonableTask
	{
	public:
		FCancelIORequestsTask(FSkeletalMeshStreamIn_IO* InPendingUpdate)
			: PendingUpdate(InPendingUpdate)
		{}

		void DoWork();

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FCancelIORequestsTask_SkeletalMesh, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		TRefCountPtr<FSkeletalMeshStreamIn_IO> PendingUpdate;
	};

	typedef FAutoDeleteAsyncTask<FCancelIORequestsTask> FAsyncCancelIORequestsTask;
	friend class FCancelIORequestsTask;

	/** Create a new async IO request to read in LOD data */
	void SetIORequest(const FContext& Context);

	/** Release IORequest and IOFileHandle. IORequest will be cancelled if still inflight */
	void ClearIORequest(const FContext& Context);

	/** Report IO errors if any. */
	void ReportIOError(const FContext& Context);

	/** Serialize data of new LODs to corresponding FStaticMeshLODResources */
	void SerializeLODData(const FContext& Context);

	/** Cancel and report IO error */
	void Cancel(const FContext& Context);

	/** Called by FAsyncCancelIORequestsTask to cancel inflight IO request if any */
	void CancelIORequest();

	/** Handle to bulk data I/O request */
	FBulkDataBatchRequest BulkDataRequest;

	/** Bulk data I/O buffer */
	FIoBuffer BulkData;

	bool bHighPrioIORequest;

	// Whether an IO error was detected (when files do not exists).
	bool bFailedOnIOError = false;

	void DoInitiateIO(const FContext& Context);

	void DoSerializeLODData(const FContext& Context);

	void DoCreateBuffers(const FContext& Context);

	void DoCancelIO(const FContext& Context);
};

#if WITH_EDITOR
class FSkeletalMeshStreamIn_DDC : public FSkeletalMeshStreamIn
{
public:
	FSkeletalMeshStreamIn_DDC(const USkeletalMesh* InMesh, EThreadType CreateResourcesThread);

	virtual ~FSkeletalMeshStreamIn_DDC() = default;

protected:
	void LoadNewLODsFromDDC(const FContext& Context);

private:
	/** Load new LOD buffers from DDC and queue a task to create RHI buffers on RT */
	void DoLoadNewLODsFromDDC(const FContext& Context);

	/** Create RHI buffers for newly streamed-in LODs and queue a task to rename references on RT */
	void DoCreateBuffers(const FContext& Context);
};
#endif

#undef UE_API
