// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "AsyncLoadingTests_Shared.generated.h"

UCLASS()
class UAsyncLoadingTests_Shared : public UObject
{
	GENERATED_BODY()

public:
	UAsyncLoadingTests_Shared()
	{
		OnConstruct.ExecuteIfBound( this);
	}

	UPROPERTY()
	TSoftObjectPtr<UObject> SoftReference;

	UPROPERTY()
	TObjectPtr<UObject> HardReference;

	UPROPERTY()
	TObjectPtr<UObject> HardReference2;

	UPROPERTY()
	FInstancedStruct StructReference;

	// Those delegates allow to easily change behavior between tests
	// Avoid using thread-safe delegates since calling them currently is an exclusive lock and can cause deadlocks in the test when serialize is multithreaded.
	// Registration to these delegates happens safely when the test begins before threads are involved.
	using FOnPostLoadDelegate = TDelegate<void(UAsyncLoadingTests_Shared*), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	ENGINE_API static FOnPostLoadDelegate OnPostLoad;

	using FOnConstructDelegate = TDelegate<void(UAsyncLoadingTests_Shared*), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	ENGINE_API static FOnConstructDelegate OnConstruct;

	using FOnSerializeDelegate = TDelegate<void(FArchive& Ar, UAsyncLoadingTests_Shared*), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	ENGINE_API static FOnSerializeDelegate OnSerialize;

	using FOnIsReadyForAsyncPostLoadDelegate = TDelegate<bool(const UAsyncLoadingTests_Shared*), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	ENGINE_API static FOnIsReadyForAsyncPostLoadDelegate OnIsReadyForAsyncPostLoad;

	using FOnIsPostLoadThreadSafeDelegate = TDelegate<bool(const UAsyncLoadingTests_Shared*), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	ENGINE_API static FOnIsReadyForAsyncPostLoadDelegate OnIsPostLoadThreadSafe;

	virtual void PostLoad() override
	{
		Super::PostLoad();

		OnPostLoad.ExecuteIfBound(this);
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		OnSerialize.ExecuteIfBound(Ar, this);
	}

	virtual bool IsReadyForAsyncPostLoad() const override
	{
		if (OnIsReadyForAsyncPostLoad.IsBound())
		{
			return OnIsReadyForAsyncPostLoad.Execute(this);
		}

		return Super::IsReadyForAsyncPostLoad();
	}

	virtual bool IsPostLoadThreadSafe() const override
	{
		if (OnIsPostLoadThreadSafe.IsBound())
		{
			return OnIsPostLoadThreadSafe.Execute(this);
		}

		return Super::IsPostLoadThreadSafe();
	}
};

#if WITH_DEV_AUTOMATION_TESTS

class FLoadingTestsScope
{
private:
	FAutomationTestBase& AutomationTest;

	TArray<FString>     PackageNames;
	std::atomic<uint32> PackageIndex;
public:
	static constexpr const TCHAR* ObjectName = TEXT("TestObject");
	static constexpr const TCHAR* PackagePath1 = TEXT("/Engine/LoadingTestsScope_Package1");
	static constexpr const TCHAR* ObjectPath1 = TEXT("/Engine/LoadingTestsScope_Package1.TestObject");
	static constexpr const TCHAR* PackagePath2 = TEXT("/Engine/LoadingTestsScope_Package2");
	static constexpr const TCHAR* ObjectPath2 = TEXT("/Engine/LoadingTestsScope_Package2.TestObject");
	static constexpr const TCHAR* PackagePath3 = TEXT("/Engine/LoadingTestsScope_Package3");
	static constexpr const TCHAR* ObjectPath3 = TEXT("/Engine/LoadingTestsScope_Package3.TestObject");
	static constexpr const TCHAR* PackagePath4 = TEXT("/Engine/LoadingTestsScope_Package4");
	static constexpr const TCHAR* ObjectPath4 = TEXT("/Engine/LoadingTestsScope_Package4.TestObject");

	UPackage* Package1 = nullptr;
	UPackage* Package2 = nullptr;
	UPackage* Package3 = nullptr;
	UPackage* Package4 = nullptr;

	UAsyncLoadingTests_Shared* Object1 = nullptr;
	UAsyncLoadingTests_Shared* Object2 = nullptr;
	UAsyncLoadingTests_Shared* Object3 = nullptr;
	UAsyncLoadingTests_Shared* Object4 = nullptr;

	UPackage* CreatePackage()
	{
		FString PackageName = FString::Printf(TEXT("/Engine/LoadingTestsScope_Package%u"), ++PackageIndex);
		UPackage* Package = ::CreatePackage(*PackageName);
		PackageNames.Add(PackageName);
		return Package;
	}

	ENGINE_API void CreateObjects();
	ENGINE_API void DefaultMutateObjects();
	ENGINE_API void SavePackages();
	ENGINE_API void LoadObjects();
	ENGINE_API void CleanupObjects();
	ENGINE_API void GarbageCollect();
	static void GarbageCollect(const TArray<FString>& PackageNames, FAutomationTestBase& AutomationTest);

	FLoadingTestsScope(FAutomationTestBase* InAutomationTest, TFunction<void (FLoadingTestsScope&)> InMutateObjects = nullptr)
		: AutomationTest(*InAutomationTest)
	{
		// Just make sure the async loading queue is empty before beginning.
		FlushAsyncLoading();

		CreateObjects();

		if (InMutateObjects)
		{
			InMutateObjects(*this);
		}
		else
		{
			DefaultMutateObjects();
		}

		SavePackages();

		GarbageCollect();
	}

	virtual ~FLoadingTestsScope()
	{
		// Just to be sure in case the test fails, don't leave some pending loads behind
		FlushAsyncLoading();

		CleanupObjects();
	}
};

class FLoadingTests_ZenLoaderOnly_Base : public FAutomationTestBase
{
public:
	using FAutomationTestBase::FAutomationTestBase;
protected:
	bool CanRunInEnvironment(const FString& TestParams, FString* OutReason, bool* OutWarn) const override
	{
		ELoaderType LoaderType = GetLoaderType();
		if (LoaderType != ELoaderType::ZenLoader)
		{
			if (OutReason)
			{
				*OutReason = FString::Printf(TEXT("Test %s is for ZenLoader only. Cannot run on non-compliant loader currently active: %s"), *GetTestName(), LexToString(LoaderType));
			}
			return false;
		}

		return true;
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS

namespace LoadingTestsUtils
{
	extern bool IsAssetSuitableForTests(const FAssetData& AssetData);
}
