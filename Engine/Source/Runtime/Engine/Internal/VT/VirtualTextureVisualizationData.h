// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureVisualizationData.generated.h"

/** The virtual texture visualization modes. */
UENUM()
enum class EVirtualTextureVisualizationMode : uint8
{
	None,
	PendingMips,
	StackCount,
};

/** Manager for virtual texture visualization modes. */
class FVirtualTextureVisualizationData
{
public:
	FVirtualTextureVisualizationData()
		: bIsInitialized(false)
	{
	}

	/** Initialize the system. */
	void Initialize();

	/** Check if system was initialized. */
	inline bool IsInitialized() const { return bIsInitialized; }

	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString ModeString;
		FName ModeName;
		FText ModeText;
		FText ModeDesc;
		EVirtualTextureVisualizationMode ModeID;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TArray<FModeRecord> TModeArray;

	/** Get the mode map. */
	inline const TModeArray& GetModes() const { return ModeArray; }

	/** Get the mode id of the active mode. */
	ENGINE_API FName GetActiveMode(class FSceneView const& InView) const;

	/** Get the mode id from the mode name. **/
	ENGINE_API EVirtualTextureVisualizationMode GetModeID(FName const& InModeName) const;

	/** Get the display name from the mode name. **/
	ENGINE_API FText GetModeDisplayName(FName const& InModeName) const;

	/** Get the display description from the mode name. **/
	ENGINE_API FText GetModeDisplayDesc(FName const& InModeName) const;

	/** Set the current mode through the console and return the previously set mode. */
	ENGINE_API EVirtualTextureVisualizationMode SetModeUsingConsoleCommand(EVirtualTextureVisualizationMode InMode);

	/** Return the console command name for enabling single mode visualization. */
	static const TCHAR* GetVisualizeConsoleCommandName();

private:
	void AddVisualizationMode(
		const TCHAR* ModeString,
		const FText& ModeText,
		const FText& ModeDesc,
		EVirtualTextureVisualizationMode ModeID);

	void ConfigureConsoleCommand();

private:
	/** Flag indicating if system is initialized. **/
	bool bIsInitialized;

	/** The registered modes. */
	TModeArray ModeArray;

	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;
};

ENGINE_API FVirtualTextureVisualizationData& GetVirtualTextureVisualizationData();
