// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

struct FVolumeBoundsMesh
{
	// Hack to make this compatible with both the actor framework and SceneGraph,
	// and avoid holding onto stale FPhysicsObject*.
	// This function should hold a weak ptr on the owner of the physics body, 
	// and not cache the physics object handles directly, as those may be invalidated.
	TFunction<TArray<Chaos::FPhysicsObjectHandle>()> PhysicsObjectsAccessor;
};
