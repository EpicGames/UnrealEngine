// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/AISystemBase.h"
#include "GameFramework/GameModeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISystemBase)


UAISystemBase::UAISystemBase(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

FName UAISystemBase::GetAISystemModuleName()
{
	UAISystemBase* AISystemDefaultObject = Cast<UAISystemBase>(StaticClass()->GetDefaultObject());
	return AISystemDefaultObject != NULL ? AISystemDefaultObject->AISystemModuleName : TEXT("");
}

FSoftClassPath UAISystemBase::GetAISystemClassName()
{
	UAISystemBase* AISystemDefaultObject = Cast<UAISystemBase>(StaticClass()->GetDefaultObject());
	return AISystemDefaultObject != NULL ? AISystemDefaultObject->AISystemClassName : FSoftClassPath();
}

void UAISystemBase::CleanupWorld(bool bSessionEnded, bool bCleanupResources, UWorld* NewWorld)
{
	CleanupWorld(bSessionEnded, bCleanupResources);
}

void UAISystemBase::CleanupWorld(bool bSessionEnded, bool bCleanupResources)
{
	FGameModeEvents::OnGameModeMatchStateSetEvent().Remove(OnMatchStateSetHandle);
}

void UAISystemBase::StartPlay()
{
	OnMatchStateSetHandle = FGameModeEvents::OnGameModeMatchStateSetEvent().AddUObject(this, &UAISystemBase::OnMatchStateSet);
}

void UAISystemBase::OnMatchStateSet(FName NewMatchState)
{

}

bool UAISystemBase::ShouldInstantiateInNetMode(ENetMode NetMode)
{
	UAISystemBase* AISystemDefaultObject = Cast<UAISystemBase>(StaticClass()->GetDefaultObject());
	return AISystemDefaultObject && (AISystemDefaultObject->bInstantiateAISystemOnClient == true || NetMode != NM_Client);
}

