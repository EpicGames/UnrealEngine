// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/AvoidanceManager.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AI/Navigation/NavEdgeProviderInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AvoidanceManager)

DEFINE_STAT(STAT_AI_ObstacleAvoidance);

FNavAvoidanceData::FNavAvoidanceData(UAvoidanceManager* Manager, IRVOAvoidanceInterface* AvoidanceComp)
{
	Init(Manager,
		AvoidanceComp->GetRVOAvoidanceOrigin(),
		AvoidanceComp->GetRVOAvoidanceRadius(),
		AvoidanceComp->GetRVOAvoidanceHeight(),
		AvoidanceComp->GetVelocityForRVOConsideration(),
		AvoidanceComp->GetRVOAvoidanceWeight(),
		AvoidanceComp->GetAvoidanceGroupMask(),
		AvoidanceComp->GetGroupsToAvoidMask(),
		AvoidanceComp->GetGroupsToIgnoreMask(),
		AvoidanceComp->GetRVOAvoidanceConsiderationRadius()
		);
}

void FNavAvoidanceData::Init(UAvoidanceManager* Avoidance, const FVector& InCenter, float InRadius, float InHalfHeight,
							 const FVector& InVelocity, float InWeight,
							 int32 InGroupMask, int32 InGroupsToAvoid, int32 InGroupsToIgnore,
							 float InTestRadius2D)
{
	Center = InCenter;
	Velocity = InVelocity;
	Radius = InRadius * Avoidance->ArtificialRadiusExpansion;
	HalfHeight = InHalfHeight;
	Weight = FMath::Clamp<float>(InWeight, 0.0f, 1.0f);
	GroupMask = InGroupMask;
	GroupsToAvoid = InGroupsToAvoid;
	GroupsToIgnore = InGroupsToIgnore;
	OverrideWeightTime = 0.0f;
	RemainingTimeToLive = Avoidance->DefaultTimeToLive;
	TestRadius2D = InTestRadius2D;

	//RickH - This is kind of a hack, but ultimately this is going to be a 2D solution with 3D culling/broad-phase anyway.
	Velocity.Z = 0.0f;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
bool UAvoidanceManager::bSystemActive = true;
#endif

UAvoidanceManager::UAvoidanceManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DefaultTimeToLive = 1.5f;
	LockTimeAfterAvoid = 0.2f;
	LockTimeAfterClean = 0.001f;
	DeltaTimeToPredict = 0.5f;
	ArtificialRadiusExpansion = 1.5f;
	TestHeightDifference_DEPRECATED = 500.0f;
	bRequestedUpdateTimer = false;
	bAutoPurgeOutdatedObjects = true;
	HeightCheckMargin = 10.0f;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bDebugAll = false;
#endif
}

void UAvoidanceManager::RemoveAvoidanceObject(const int32 AvoidanceUID)
{
	if (AvoidanceUID < 0)
	{
		return;
	}

	FNavAvoidanceData* ExistingDataPtr = AvoidanceObjects.Find(AvoidanceUID);
	if (ExistingDataPtr)
	{
		ExistingDataPtr->RemainingTimeToLive = 0.0f;
	}

	//Expired, not in pool yet, assign to pool
	//DrawDebugLine(GetWorld(), AvoidanceData.Center, AvoidanceData.Center + FVector(0,0,500), FColor(64,255,64), true, 2.0f, SDPG_MAX, 20.0f);
	NewKeyPool.AddUnique(AvoidanceUID);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DebugUIDs.Contains(AvoidanceUID))
	{
		DebugUIDs.Remove(AvoidanceUID);
	}
#endif
}

void UAvoidanceManager::RemoveOutdatedObjects()
{
	if (bAutoPurgeOutdatedObjects == false)
	{
		// will be handled manually
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AI_ObstacleAvoidance);
	bRequestedUpdateTimer = false;

	//Update RVO stuff. Mainly, this involves removing outdated RVO entries.
	bool bHasActiveObstacles = false;
	for (auto& AvoidanceObj : AvoidanceObjects)
	{
		FNavAvoidanceData& AvoidanceData = AvoidanceObj.Value;

		if (AvoidanceData.RemainingTimeToLive > (DefaultTimeToLive * 0.5f))
		{
			//Update record with reduced TTL
			AvoidanceData.RemainingTimeToLive -= (DefaultTimeToLive * 0.5f);
			bHasActiveObstacles = true;
		}
		else if (!AvoidanceData.ShouldBeIgnored())
		{
			const int32 ObjectId = AvoidanceObj.Key;
			AvoidanceData.RemainingTimeToLive = 0.0f;

			//Expired, not in pool yet, assign to pool
			//DrawDebugLine(GetWorld(), AvoidanceData.Center, AvoidanceData.Center + FVector(0,0,500), FColor(64,255,64), true, 2.0f, SDPG_MAX, 20.0f);
			NewKeyPool.AddUnique(ObjectId);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (DebugUIDs.Contains(ObjectId))
			{
				DebugUIDs.Remove(ObjectId);
			}
#endif
		}
	}

	if (bHasActiveObstacles)
	{
		RequestUpdateTimer();
	}
}

void UAvoidanceManager::RequestUpdateTimer()
{
	UWorld* MyWorld = Cast<UWorld>(GetOuter());
	if (!bRequestedUpdateTimer && MyWorld)
	{
		bRequestedUpdateTimer = true;
		MyWorld->GetTimerManager().SetTimer(TimerHandle_RemoveOutdatedObjects, this, &UAvoidanceManager::RemoveOutdatedObjects, DefaultTimeToLive * 0.5f, false);
	}
}

int32 UAvoidanceManager::GetObjectCount()
{
	return AvoidanceObjects.Num();
}

int32 UAvoidanceManager::GetNewAvoidanceUID()
{
	int32 NewUID = AvoidanceObjects.Num();
	if (NewKeyPool.Num())
	{
		NewUID = NewKeyPool[NewKeyPool.Num() - 1];
		NewKeyPool.RemoveAt(NewKeyPool.Num() - 1, EAllowShrinking::No);
	}
	return NewUID;
}

bool UAvoidanceManager::RegisterMovementComponent(UMovementComponent* MovementComp, float AvoidanceWeight)
{
	if (IRVOAvoidanceInterface* AvoidingComp = Cast<IRVOAvoidanceInterface>(MovementComp))
	{
		const int32 NewAvoidanceUID = GetNewAvoidanceUID();
		AvoidingComp->SetRVOAvoidanceUID(NewAvoidanceUID);
		AvoidingComp->SetRVOAvoidanceWeight(AvoidanceWeight);

		RequestUpdateTimer();

		FNavAvoidanceData AvoidanceData(this, AvoidingComp);
		UpdateRVO_Internal(NewAvoidanceUID, AvoidanceData);

		return true;
	}

	return false;
}

bool UAvoidanceManager::RegisterMovementComponent(UCharacterMovementComponent* MovementComp, float AvoidanceWeight)
{
	const int32 NewAvoidanceUID = GetNewAvoidanceUID();
	MovementComp->SetRVOAvoidanceUID(NewAvoidanceUID);
	MovementComp->SetRVOAvoidanceWeight(AvoidanceWeight);

	RequestUpdateTimer();

	FNavAvoidanceData AvoidanceData(this, MovementComp);
	UpdateRVO_Internal(NewAvoidanceUID, AvoidanceData);

	return true;
}

FVector UAvoidanceManager::GetAvoidanceVelocityForComponent(UMovementComponent* MovementComp)
{
	if (IRVOAvoidanceInterface* AvoidingComp = Cast<IRVOAvoidanceInterface>(MovementComp))
	{
		FNavAvoidanceData AvoidanceData(this, AvoidingComp);
		return GetAvoidanceVelocityIgnoringUID(AvoidanceData, DeltaTimeToPredict, AvoidingComp->GetRVOAvoidanceUID());
	}

	return FVector::ZeroVector;
}

FVector UAvoidanceManager::GetAvoidanceVelocityForComponent(UCharacterMovementComponent* MovementComp)
{
	check(MovementComp);
	FNavAvoidanceData AvoidanceData(this, MovementComp);
	return GetAvoidanceVelocityIgnoringUID(AvoidanceData, DeltaTimeToPredict, MovementComp->GetRVOAvoidanceUIDFast());
}

FVector UAvoidanceManager::GetAvoidanceVelocityIgnoringUID(const FNavAvoidanceData& inAvoidanceData, float DeltaTime, int32 inIgnoreThisUID)
{
	return GetAvoidanceVelocity_Internal(inAvoidanceData, DeltaTime, &inIgnoreThisUID);
}

FVector UAvoidanceManager::GetAvoidanceVelocity(const FNavAvoidanceData& inAvoidanceData, float DeltaTime)
{
	return GetAvoidanceVelocity_Internal(inAvoidanceData, DeltaTime);
}

void UAvoidanceManager::UpdateRVO(UMovementComponent* MovementComp)
{
	if (IRVOAvoidanceInterface* AvoidingComp = Cast<IRVOAvoidanceInterface>(MovementComp))
	{
		FNavAvoidanceData NewAvoidanceData(this, AvoidingComp);
		UpdateRVO_Internal(AvoidingComp->GetRVOAvoidanceUID(), NewAvoidanceData);
	}
}

void UAvoidanceManager::UpdateRVO(UCharacterMovementComponent* MovementComp)
{
	check(MovementComp);
	FNavAvoidanceData NewAvoidanceData(this, MovementComp);
	UpdateRVO_Internal(MovementComp->GetRVOAvoidanceUIDFast(), NewAvoidanceData);
}

void UAvoidanceManager::UpdateRVO_Internal(int32 inAvoidanceUID, const FNavAvoidanceData& inAvoidanceData)
{
	if (FNavAvoidanceData* existingData = AvoidanceObjects.Find(inAvoidanceUID))
	{
		double OverrideWeightTime = existingData->OverrideWeightTime;		//Hold onto this one value
		*existingData = inAvoidanceData;
		existingData->OverrideWeightTime = OverrideWeightTime;
	}
	else
	{
		AvoidanceObjects.Add(inAvoidanceUID, inAvoidanceData);
	}
}

FVector AvoidCones(TArray<FVelocityAvoidanceCone>& AllCones, const FVector& BasePosition, const FVector& DesiredPosition, const int NumConesToTest)
{
	FVector CurrentPosition = DesiredPosition;
	FVector::FReal DistanceInsidePlane_Current[2];
	FVector::FReal DistanceInsidePlane_Base[2];
	FVector::FReal Weighting[2];
	int ConePlaneIndex;

	//AllCones is non-const so that it can be reordered, but nothing should be added or removed from it.
	checkSlow(NumConesToTest <= AllCones.Num());
	TArray<FVelocityAvoidanceCone>::TIterator It = AllCones.CreateIterator();
	for (int i = 0; i < NumConesToTest; ++i, ++It)
	{
		FVelocityAvoidanceCone& CurrentCone = *It;

		//See if CurrentPosition is outside this cone. If it is, then this cone doesn't obstruct us.
		DistanceInsidePlane_Current[0] = (CurrentPosition|CurrentCone.ConePlane[0]) - CurrentCone.ConePlane[0].W;
		DistanceInsidePlane_Current[1] = (CurrentPosition|CurrentCone.ConePlane[1]) - CurrentCone.ConePlane[1].W;
		if ((DistanceInsidePlane_Current[0] <= 0.0f) || (DistanceInsidePlane_Current[1] <= 0.0f))
		{
			//We're not inside this cone, continue past it.
			//If we wanted to, we could check if CurrentPosition and BasePosition are on the same side of the cone, in which case the cone can be removed from this segment test entirely.
			continue;
		}

		//If we've gotten here, CurrentPosition is inside the cone. If BasePosition is also inside the cone, this entire segment is blocked.
		DistanceInsidePlane_Base[0] = (BasePosition|CurrentCone.ConePlane[0]) - CurrentCone.ConePlane[0].W;
		DistanceInsidePlane_Base[1] = (BasePosition|CurrentCone.ConePlane[1]) - CurrentCone.ConePlane[1].W;

		//We know that the BasePosition isn't in the cone, but CurrentPosition is. We should find the point where the line segment between CurrentPosition and BasePosition exits the cone.
#define CALCULATE_WEIGHTING(index) Weighting[index] = -DistanceInsidePlane_Base[index] / (DistanceInsidePlane_Current[index] - DistanceInsidePlane_Base[index]);
		if (DistanceInsidePlane_Base[0] <= 0.0f)
		{
			CALCULATE_WEIGHTING(0);
			if (DistanceInsidePlane_Base[1] <= 0.0f)
			{
				CALCULATE_WEIGHTING(1);
				ConePlaneIndex = (Weighting[1] > Weighting[0]) ? 1 : 0;
			}
			else
			{
				ConePlaneIndex = 0;
			}
		}
		else if (DistanceInsidePlane_Base[1] <= 0.0f)
		{
			CALCULATE_WEIGHTING(1);
			ConePlaneIndex = 1;
		}
		else
		{
			//BasePosition is also in the cone. This entire line segment of movement is invalidated. I'm considering a way to return false/NULL here.
			return BasePosition;
		}
		//Weighted average of points based on planar distance gives us the answer we want without needing to make a direction vector.
		CurrentPosition = (CurrentPosition * Weighting[ConePlaneIndex]) + (BasePosition * (1.0f - Weighting[ConePlaneIndex]));
#undef CALCULATE_WEIGHTING

		//This cone doesn't need to be used again, so drop it from the list (by shuffling it to the end and decrementing the cone count).
		//This probably ruins our iterator, but it doesn't matter because we're done.
		AllCones.Swap(i, NumConesToTest - 1);		//Don't care if this is swapping an element with itself; the swap function checks this already.
		return AvoidCones(AllCones, BasePosition, CurrentPosition, NumConesToTest - 1);
	}

	return CurrentPosition;
}

static bool AvoidsNavEdges(const FVector& OrgLocation, const FVector& TestVelocity, const TArray<FNavEdgeSegment>& NavEdges, float MaxZDiff)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Avoidance: avoid nav edges"), STAT_AIAvoidanceEdgeAvoid, STATGROUP_AI);

	for (int32 Idx = 0; Idx < NavEdges.Num(); Idx++)
	{
		const FVector2D Seg0ToSeg1(NavEdges[Idx].P1 - NavEdges[Idx].P0);
		const FVector2D OrgToNewPos(TestVelocity);
		const FVector2D OrgToSeg0(NavEdges[Idx].P0 - OrgLocation);
		const FVector2D::FReal CrossD = FVector2D::CrossProduct(Seg0ToSeg1, OrgToNewPos);
		if (FMath::Abs(CrossD) < UE_KINDA_SMALL_NUMBER)
		{
			continue;
		}

		const FVector2D::FReal CrossS = FVector2D::CrossProduct(OrgToNewPos, OrgToSeg0) / CrossD;
		const FVector2D::FReal CrossT = FVector2D::CrossProduct(Seg0ToSeg1, OrgToSeg0) / CrossD;
		if (CrossS < 0.0f || CrossS > 1.0f || CrossT < 0.0f || CrossT > 1.0f)
		{
			continue;
		}

		const FVector CrossPt = FMath::Lerp(NavEdges[Idx].P0, NavEdges[Idx].P1, CrossS);
		const FVector2D::FReal ZDiff = FMath::Abs(OrgLocation.Z - CrossPt.Z);
		if (ZDiff > MaxZDiff)
		{
			continue;
		}

		return false;
	}
	
	return true;
}

//RickH - We could probably significantly improve speed if we put separate Z checks in place and did everything else in 2D.
FVector UAvoidanceManager::GetAvoidanceVelocity_Internal(const FNavAvoidanceData& inAvoidanceData, float DeltaTime, int32* inIgnoreThisUID)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!bSystemActive)
	{
		return inAvoidanceData.Velocity;
	}
#endif
	if (DeltaTime <= 0.0f)
	{
		return inAvoidanceData.Velocity;
	}

	FVector ReturnVelocity = inAvoidanceData.Velocity * DeltaTime;
	FVector::FReal MaxSpeed = ReturnVelocity.Size2D();
	double CurrentTime;

	UWorld* MyWorld = Cast<UWorld>(GetOuter());
	if (MyWorld)
	{
		CurrentTime = MyWorld->TimeSeconds;
	}
	else
	{
		//No world? OK, just quietly back out and don't alter anything.
		return inAvoidanceData.Velocity;
	}

	bool Unobstructed = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool DebugMode = IsDebugOnForAll() || (inIgnoreThisUID ? IsDebugOnForUID(*inIgnoreThisUID) : false);
#endif

	//If we're moving very slowly, just push forward. Not sure it's worth avoiding at this speed, though I could be wrong.
	if (MaxSpeed < 0.01f)
	{
		return inAvoidanceData.Velocity;
	}
	AllCones.Empty(AllCones.Max());

	//DrawDebugDirectionalArrow(GetWorld(), inAvoidanceData.Center, inAvoidanceData.Center + inAvoidanceData.Velocity, 2.5f, FColor(0,255,255), true, 0.05f, SDPG_MAX);

	for (auto& AvoidanceObj : AvoidanceObjects)
	{
		if ((inIgnoreThisUID) && (*inIgnoreThisUID == AvoidanceObj.Key))
		{
			continue;
		}
		FNavAvoidanceData& OtherObject = AvoidanceObj.Value;

		//
		//Start with a few fast-rejects
		//

		//If the object has expired, ignore it
		if (OtherObject.ShouldBeIgnored())
		{
			continue;
		}

		//If other object is not in avoided group, ignore it
		if (inAvoidanceData.ShouldIgnoreGroup(OtherObject.GroupMask))
		{
			continue;
		}

		//RickH - We should have a max-radius parameter/option here, so I'm just going to hardcode one for now.
		//if ((OtherObject.Radius + _AvoidanceData.Radius + MaxSpeed + OtherObject.Velocity.Size2D()) < FVector::Dist(OtherObject.Center, _AvoidanceData.Center))
		if (FVector2D(OtherObject.Center - inAvoidanceData.Center).SizeSquared() > FMath::Square(inAvoidanceData.TestRadius2D))
		{
			continue;
		}

		if (FMath::Abs(OtherObject.Center.Z - inAvoidanceData.Center.Z) > OtherObject.HalfHeight + inAvoidanceData.HalfHeight + HeightCheckMargin)
		{
			continue;
		}

		//If we are moving away from the obstacle, ignore it. Even if we're the slower one, let the other obstacle path around us.
		if ((ReturnVelocity | (OtherObject.Center - inAvoidanceData.Center)) <= 0.0f)
		{
			continue;
		}

		//Create data for the avoidance routine
		{
			FVector PointAWorld = inAvoidanceData.Center;
			FVector PointBRelative = OtherObject.Center - PointAWorld;
			FVector TowardB, SidewaysFromB;
			float RadiusB = OtherObject.Radius + inAvoidanceData.Radius;

			PointBRelative.Z = 0.0f;
			TowardB = PointBRelative.GetSafeNormal2D();		//Don't care about height for this game. Rough height-checking will come in later, but even then it will be acceptable to do this.
			if (TowardB.IsZero())
			{
				//Already intersecting, or aligned vertically, scrap this whole object.
				continue;
			}
			SidewaysFromB.Set(-TowardB.Y, TowardB.X, 0.0f);

			//Build collision cone (two planes) and store for later use. We might consider some fast rejection here to see if we can skip the cone entirely.
			//RickH - If we built these cones in 2D, we could avoid all the cross-product matrix stuff and just use (y, -x) 90-degree rotation.
			{
				FVector PointPlane[2];
				FVector EffectiveVelocityB;
				FVelocityAvoidanceCone NewCone;

				//Use RVO (as opposed to VO) only for objects that are not overridden to max weight AND that are currently moving toward us.
				if ((OtherObject.OverrideWeightTime <= CurrentTime) && ((OtherObject.Velocity|PointBRelative) < 0.0f))
				{
					float OtherWeight = (OtherObject.Weight + (1.0f - inAvoidanceData.Weight)) * 0.5f;			//Use the average of what the other wants to be and what we want it to be.
					EffectiveVelocityB = ((inAvoidanceData.Velocity * (1.0f - OtherWeight)) + (OtherObject.Velocity * OtherWeight)) * DeltaTime;
				}
				else
				{
					EffectiveVelocityB = OtherObject.Velocity * DeltaTime;		//This is equivalent to VO (not RVO) because the other object is not going to reciprocate our avoidance.
				}
				checkSlow(EffectiveVelocityB.Z == 0.0f);

				//Make the left plane
				PointPlane[0] = EffectiveVelocityB + (PointBRelative + (SidewaysFromB * RadiusB));
				PointPlane[1].Set(PointPlane[0].X, PointPlane[0].Y, PointPlane[0].Z + 100.0f);
				NewCone.ConePlane[0] = FPlane(EffectiveVelocityB, PointPlane[0], PointPlane[1]);		//First point is relative to A, which is ZeroVector in this implementation
				checkSlow((((PointBRelative+EffectiveVelocityB)|NewCone.ConePlane[0]) - NewCone.ConePlane[0].W) > 0.0f);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (DebugMode)
				{
//					DrawDebugDirectionalArrow(MyWorld, EffectiveVelocityB + PointAWorld, PointPlane[0] + PointAWorld, 50.0f, FColor(64,64,64), true, 0.05f, SDPG_MAX);
//					DrawDebugLine(MyWorld, PointAWorld, EffectiveVelocityB + PointAWorld, FColor(64,64,64), true, 0.05f, SDPG_MAX, 5.0f);
				}
#endif

				//Make the right plane
				PointPlane[0] = EffectiveVelocityB + (PointBRelative - (SidewaysFromB * RadiusB));
				PointPlane[1].Set(PointPlane[0].X, PointPlane[0].Y, PointPlane[0].Z - 100.0f);
				NewCone.ConePlane[1] = FPlane(EffectiveVelocityB, PointPlane[0], PointPlane[1]);		//First point is relative to A, which is ZeroVector in this implementation
				checkSlow((((PointBRelative+EffectiveVelocityB)|NewCone.ConePlane[1]) - NewCone.ConePlane[1].W) > 0.0f);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (DebugMode)
				{
//					DrawDebugDirectionalArrow(MyWorld, EffectiveVelocityB + PointAWorld, PointPlane[0] + PointAWorld, 50.0f, FColor(64,64,64), true, 0.05f, SDPG_MAX);
				}
#endif

				if ((((ReturnVelocity|NewCone.ConePlane[0]) - NewCone.ConePlane[0].W) > 0.0f)
					&& (((ReturnVelocity|NewCone.ConePlane[1]) - NewCone.ConePlane[1].W) > 0.0f))
				{
					Unobstructed = false;
				}

				AllCones.Add(NewCone);
			}
		}
	}
	if (Unobstructed)
	{
		//Trivial case, our ideal velocity is available.
		return inAvoidanceData.Velocity;
	}

	TArray<FNavEdgeSegment> NavEdges;
	if (EdgeProviderOb.IsValid())
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Avoidance: collect nav edges"), STAT_AIAvoidanceEdgeCollect, STATGROUP_AI);
		EdgeProviderInterface->GetEdges(inAvoidanceData.Center, inAvoidanceData.TestRadius2D, NavEdges);
	}

	//Find a good velocity that isn't inside a cone.
	if (AllCones.Num())
	{
		FVector::FReal AngleCurrent;
		FVector::FReal AngleF = ReturnVelocity.HeadingAngle();
		FVector::FReal BestScore = 0.0f;
		FVector::FReal BestScorePotential;
		FVector BestVelocity = FVector::ZeroVector;		//Worst case is we just stand completely still. Should we also allow backing up? Should we test standing still?
		const int AngleCount = 4;		//Every angle will be tested both right and left.
		FVector::FReal AngleOffset[AngleCount] = {FMath::DegreesToRadians<float>(23.0f), FMath::DegreesToRadians<float>(40.0f), FMath::DegreesToRadians<float>(55.0f), FMath::DegreesToRadians<float>(85.0f)};
		FVector AngleVector[AngleCount<<1];

		//Determine check angles
		for (int i = 0; i < AngleCount; ++i)
		{
			AngleCurrent = AngleF - AngleOffset[i];
			AngleVector[(i<<1)].Set(FMath::Cos(AngleCurrent), FMath::Sin(AngleCurrent), 0.0f);
			AngleCurrent = AngleF + AngleOffset[i];
			AngleVector[(i<<1) + 1].Set(FMath::Cos(AngleCurrent), FMath::Sin(AngleCurrent), 0.0f);
		}

		//Sample velocity-space destination points and drag them back to form lines
		for (int AngleToCheck = 0; AngleToCheck < (AngleCount<<1); ++AngleToCheck)
		{
			FVector VelSpacePoint = AngleVector[AngleToCheck] * MaxSpeed;

			//Skip testing if we know we can't possibly get a better score than what we have already.
			//Note: This assumes the furthest point is the highest-scoring value (i.e. VelSpacePoint is not moving backward relative to ReturnVelocity)
			BestScorePotential = (VelSpacePoint|ReturnVelocity) * (VelSpacePoint|VelSpacePoint);
			if (BestScorePotential > BestScore)
			{
				const bool bAvoidsNavEdges = NavEdges.Num() > 0 ? AvoidsNavEdges(inAvoidanceData.Center, VelSpacePoint, NavEdges, inAvoidanceData.HalfHeight) : true;
				if (bAvoidsNavEdges)
				{
					FVector CandidateVelocity = AvoidCones(AllCones, FVector::ZeroVector, VelSpacePoint, AllCones.Num());
					FVector::FReal CandidateScore = (CandidateVelocity|ReturnVelocity) * (CandidateVelocity|CandidateVelocity);

					//Vectors are rated by their length and their overall forward movement.
					if (CandidateScore > BestScore)
					{
						BestScore = CandidateScore;
						BestVelocity = CandidateVelocity;
					}
				}
			}
		}
		ReturnVelocity = BestVelocity;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (DebugMode)
		{
			DrawDebugDirectionalArrow(MyWorld, inAvoidanceData.Center + inAvoidanceData.Velocity, inAvoidanceData.Center + (ReturnVelocity / DeltaTime), 75.0f, FColor(64,255,64), true, 2.0f, SDPG_MAX);
		}
#endif
	}

	return ReturnVelocity / DeltaTime;		//Remove prediction-time scaling
}

void UAvoidanceManager::OverrideToMaxWeight(int32 AvoidanceUID, float Duration)
{
	if (FNavAvoidanceData *AvoidObj = AvoidanceObjects.Find(AvoidanceUID))
	{
		UWorld* MyWorld = Cast<UWorld>(GetOuter());
		AvoidObj->OverrideWeightTime = MyWorld->TimeSeconds + Duration;
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
bool UAvoidanceManager::IsDebugOnForUID(int32 AvoidanceUID)
{
	return (DebugUIDs.Find(AvoidanceUID) != INDEX_NONE);
}

bool UAvoidanceManager::IsDebugOnForAll()
{
	return bDebugAll;
}

bool UAvoidanceManager::IsDebugEnabled(int32 AvoidanceUID)
{
	return IsDebugOnForAll() || IsDebugOnForUID(AvoidanceUID);
}

void UAvoidanceManager::AvoidanceDebugForUID(int32 AvoidanceUID, bool TurnOn)
{
	if (TurnOn)
	{
		DebugUIDs.AddUnique(AvoidanceUID);
	}
	else
	{
		DebugUIDs.Remove(AvoidanceUID);
	}
}

void UAvoidanceManager::AvoidanceDebugForAll(bool TurnOn)
{
	bDebugAll = TurnOn;
}

void UAvoidanceManager::AvoidanceSystemToggle(bool TurnOn)
{
	bSystemActive = TurnOn;
}

void UAvoidanceManager::HandleToggleDebugAll( const TCHAR* Cmd, FOutputDevice& Ar )
{
	bDebugAll = !bDebugAll;
}

void UAvoidanceManager::HandleToggleAvoidance( const TCHAR* Cmd, FOutputDevice& Ar )
{
	bSystemActive = !bSystemActive;
	Ar.Logf(TEXT("Avoidance system: %s"), bSystemActive ? TEXT("enabled") : TEXT("disabled"));
}

#endif

bool UAvoidanceManager::Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (FParse::Command(&Cmd, TEXT("AvoidanceDisplayAll")))
	{
		HandleToggleDebugAll( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("AvoidanceSystemToggle")))
	{
		HandleToggleAvoidance( Cmd, Ar );
		return true;
	}
#endif

	return false;
}

void UAvoidanceManager::SetNavEdgeProvider(INavEdgeProviderInterface* InEdgeProvider)
{
	EdgeProviderInterface = InEdgeProvider;
	EdgeProviderOb = Cast<UObject>(InEdgeProvider);
}

