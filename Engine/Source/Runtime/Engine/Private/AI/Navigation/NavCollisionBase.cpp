// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavCollisionBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavCollisionBase)


UNavCollisionBase::FConstructNew UNavCollisionBase::ConstructNewInstanceDelegate;
UNavCollisionBase::FDelegateInitializer UNavCollisionBase::DelegateInitializer;

UNavCollisionBase::FDelegateInitializer::FDelegateInitializer()
{
	UNavCollisionBase::ConstructNewInstanceDelegate.BindLambda([](UObject&) { return nullptr; });
}
	
UNavCollisionBase::UNavCollisionBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsDynamicObstacle = false;
	bUseSurfaceArea = false;
	bHasConvexGeometry = false;
}

