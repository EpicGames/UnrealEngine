// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavigationDirtyArea.h"
#include "AI/Navigation/NavigationElement.h"
#if !NO_LOGGING
#include "AI/NavigationSystemBase.h"
#endif

FNavigationDirtyArea::FNavigationDirtyArea(const FBox& InBounds, const ENavigationDirtyFlag InFlags, const TSharedPtr<const FNavigationElement>& InOptionalSourceElement)
	: Bounds(InBounds)
	, OptionalSourceElement(InOptionalSourceElement)
	, Flags(InFlags)
{
#if !NO_LOGGING
	if (!Bounds.IsValid || Bounds.ContainsNaN())
	{
		const FNavigationElement* Element = OptionalSourceElement.Get();
		UE_LOGF(LogNavigation, Warning, "Creation of FNavigationDirtyArea with invalid bounds%ls. Bounds: %ls, SourceElement: %ls.",
			Bounds.ContainsNaN() ? TEXT(" (contains NaN)") : TEXT(""), *Bounds.ToString(), *GetFullNameSafe(Element));
	}
#endif //!NO_LOGGING
}

FString FNavigationDirtyArea::GetSourceDescription() const
{
	const FNavigationElement* SourceElement = OptionalSourceElement.Get();
	return SourceElement ? SourceElement->GetFullName() : TEXT("");
}

// Deprecated
FNavigationDirtyArea::FNavigationDirtyArea(const FBox& InBounds, const int32 InFlags, UObject* const InOptionalSourceObject)
	: FNavigationDirtyArea(InBounds, static_cast<ENavigationDirtyFlag>(InFlags), /*InOptionalSourceElement*/nullptr)
{
}

FNavigationDirtyArea::FNavigationDirtyArea(const FNavigationDirtyArea& Other)
	: Bounds(Other.Bounds)
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, OptionalSourceObject(Other.OptionalSourceObject)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, OptionalSourceElement(Other.OptionalSourceElement)
	, Flags(Other.Flags)
{
}

FNavigationDirtyArea::FNavigationDirtyArea(FNavigationDirtyArea&& Other)
	: Bounds(MoveTemp(Other.Bounds))
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, OptionalSourceObject(MoveTemp(Other.OptionalSourceObject))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, OptionalSourceElement(MoveTemp(Other.OptionalSourceElement))
	, Flags(Other.Flags)
{
}

FNavigationDirtyArea& FNavigationDirtyArea::operator=(const FNavigationDirtyArea& Other)
{
	Bounds = Other.Bounds;
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OptionalSourceObject = Other.OptionalSourceObject;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	OptionalSourceElement = Other.OptionalSourceElement;
	Flags = Other.Flags;
	return *this;
}

FNavigationDirtyArea& FNavigationDirtyArea::operator=(FNavigationDirtyArea&& Other)
{
	Bounds = MoveTemp(Other.Bounds);
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OptionalSourceObject = MoveTemp(Other.OptionalSourceObject);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	OptionalSourceElement = MoveTemp(Other.OptionalSourceElement);
	Flags = Other.Flags;
	return *this;
}
