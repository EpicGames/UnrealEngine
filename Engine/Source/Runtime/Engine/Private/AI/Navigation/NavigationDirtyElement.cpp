// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavigationDirtyElement.h"
#include "AI/Navigation/NavigationDirtyArea.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavigationTypes.h"

FNavigationDirtyElement::FNavigationDirtyElement(
	const TSharedRef<const FNavigationElement>& InNavigationElement,
	const ENavigationDirtyFlag InFlagsOverride,
	const bool bUseWorldPartitionedDynamicMode /*= false*/)
	: NavigationElement(InNavigationElement)
	, FlagsOverride(InFlagsOverride)
	, PrevFlags(ENavigationDirtyFlag::None)
	, bIsFromVisibilityChange(bUseWorldPartitionedDynamicMode && NavigationElement->IsFromLevelVisibilityChange())
	, bIsInBaseNavmesh(bUseWorldPartitionedDynamicMode && NavigationElement->IsInBaseNavigationData())
{
}

FNavigationDirtyElement::FNavigationDirtyElement(const TSharedRef<const FNavigationElement>& InNavigationElement, const bool bUseWorldPartitionedDynamicMode /*= false*/)
	: FNavigationDirtyElement(InNavigationElement, ENavigationDirtyFlag::None, bUseWorldPartitionedDynamicMode)
{
}

FNavigationDirtyElement::FNavigationDirtyElement(const FNavigationDirtyElement& Other)
	: ExplicitAreasToDirty(Other.ExplicitAreasToDirty)
	, NavigationElement(Other.NavigationElement)
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, Owner(Other.Owner)
	, NavInterface(Other.NavInterface)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, PrevBounds(Other.PrevBounds)
	, FlagsOverride(Other.FlagsOverride)
	, PrevFlags(Other.PrevFlags)
	, bHasPrevData(Other.bHasPrevData)
	, bInvalidRequest(Other.bInvalidRequest)
	, bIsFromVisibilityChange(Other.bIsFromVisibilityChange)
	, bIsInBaseNavmesh(Other.bIsInBaseNavmesh)
{
}

FNavigationDirtyElement& FNavigationDirtyElement::operator=(const FNavigationDirtyElement& Other)
{
	ExplicitAreasToDirty = Other.ExplicitAreasToDirty;
	NavigationElement = Other.NavigationElement;
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Owner = Other.Owner;
	NavInterface = Other.NavInterface;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	PrevBounds = Other.PrevBounds;
	FlagsOverride = Other.FlagsOverride;
	PrevFlags = Other.PrevFlags;
	bHasPrevData = Other.bHasPrevData;
	bInvalidRequest = Other.bInvalidRequest;
	bIsFromVisibilityChange = Other.bIsFromVisibilityChange;
	bIsInBaseNavmesh = Other.bIsInBaseNavmesh;
	return *this;
}

uint32 GetTypeHash(const FNavigationDirtyElement& Info)
{
	return GetTypeHash(Info.NavigationElement.Get());
}


//----------------------------------------------------------------------//
// Deprecated methods
//----------------------------------------------------------------------//
PRAGMA_DISABLE_DEPRECATION_WARNINGS

// Deprecated
FNavigationDirtyElement::FNavigationDirtyElement()
	: FNavigationDirtyElement(FNavigationElement::MakeFromUObject_DEPRECATED(nullptr))
{
}

// Deprecated
FNavigationDirtyElement::FNavigationDirtyElement(UObject* InOwner, INavRelevantInterface*, int32 InFlagsOverride /*= 0*/, const bool bUseWorldPartitionedDynamicMode /*= false*/)
	: FNavigationDirtyElement(FNavigationElement::MakeFromUObject_DEPRECATED(InOwner), static_cast<ENavigationDirtyFlag>(InFlagsOverride), bUseWorldPartitionedDynamicMode)
{
}

// Deprecated
FNavigationDirtyElement::FNavigationDirtyElement(UObject* InOwner)
	: FNavigationDirtyElement(FNavigationElement::MakeFromUObject_DEPRECATED(InOwner))
{
}

// Deprecated
bool FNavigationDirtyElement::operator==(const UObject*& OtherOwner) const
{ 
	return NavigationElement->GetHandle() == FNavigationElementHandle(OtherOwner);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
