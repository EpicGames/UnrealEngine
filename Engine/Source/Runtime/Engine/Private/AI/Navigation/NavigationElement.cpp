// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavigationElement.h"

#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/Object.h"
#include "Templates/TypeHash.h"

namespace UE::NavigationElement::Private
{
#if !UE_BUILD_SHIPPING
bool bValidateNavigationElementInitialization = false;

FAutoConsoleVariableRef ConsoleVariables[] =
{
	FAutoConsoleVariableRef(
		TEXT("ai.debug.nav.validateNavigationElementInitialization"),
		bValidateNavigationElementInitialization,
		TEXT("Used to validate that the values returned by INavRelevantInterface when initializing the FNavigationElement makes sense."
			"Those warnings might not be critical since an update can be sent after and update then pending element but might still worth "
			"investigating the use case to reduce redundant operations."))
};
#endif // !UE_BUILD_SHIPPING

} // UE::NavigationOctree::Private

const FNavigationElementHandle FNavigationElementHandle::Invalid = FNavigationElementHandle(nullptr, INDEX_NONE);

FNavigationElement::FNavigationElement(const UObject& Object, const uint64 SubElementId /*= INDEX_NONE*/)
	: OwnerUObject(&Object)
	, SubElementId(SubElementId)
	, GeometryExportType(EHasCustomNavigableGeometry::No)
	, GeometryGatheringMode(ENavDataGatheringMode::Default)
	, bIsInBaseNavigationData(FNavigationSystem::IsInBaseNavmesh(&Object))
	, bIsFromLevelVisibilityChange(FNavigationSystem::IsLevelVisibilityChanging(&Object))
{
}

FNavigationElement::FNavigationElement(FPrivateToken, const UObject* Object, uint64 SubElementId /*= INDEX_NONE*/)
	: OwnerUObject(Object)
	, SubElementId(SubElementId)
	, GeometryExportType(EHasCustomNavigableGeometry::No)
	, GeometryGatheringMode(ENavDataGatheringMode::Default)
	, bIsInBaseNavigationData(FNavigationSystem::IsInBaseNavmesh(Object))
	, bIsFromLevelVisibilityChange(FNavigationSystem::IsLevelVisibilityChanging(Object))
{
}

FNavigationElement::FNavigationElement(const INavRelevantInterface& NavRelevant, const uint64 SubElementId)
	: SubElementId(SubElementId)
	, GeometryExportType(EHasCustomNavigableGeometry::No)
	, GeometryGatheringMode(ENavDataGatheringMode::Default)

{
	if (const UObject* Object = Cast<UObject>(&NavRelevant))
	{
		OwnerUObject = Object;
		bIsInBaseNavigationData = FNavigationSystem::IsInBaseNavmesh(Object);
		bIsFromLevelVisibilityChange = FNavigationSystem::IsLevelVisibilityChanging(Object);
	}
	InitializeFromInterface(&NavRelevant);
}

FNavigationElement::FNavigationElement(const UObject& Object, const uint64 SubElementId, const bool bTryInitializeFromInterface)
	: FNavigationElement(Object, SubElementId)
{
	if (bTryInitializeFromInterface)
	{
		InitializeFromInterface(Cast<INavRelevantInterface>(&Object));
	}
}

void FNavigationElement::InitializeFromInterface(const INavRelevantInterface* NavRelevantInterface)
{
	if (NavRelevantInterface == nullptr)
	{
		return;
	}

	INavRelevantInterface* MutableInterface = const_cast<INavRelevantInterface*>(NavRelevantInterface);

	bDirtyAreaOnRegistration = !NavRelevantInterface->ShouldSkipDirtyAreaOnAddOrRemove();

	Bounds = NavRelevantInterface->GetNavigationBounds();
	GeometryGatheringMode = NavRelevantInterface->GetGeometryGatheringMode();
	BodySetup = MutableInterface->GetNavigableGeometryBodySetup();
	GeometryTransform = NavRelevantInterface->GetNavigableGeometryTransform();
	GeometryExportType = NavRelevantInterface->HasCustomNavigableGeometry();
	NavigationParent = NavRelevantInterface->GetNavigationParent();

	NavigationDataExportDelegate.BindWeakLambda(Cast<UObject>(NavRelevantInterface),
		[NavRelevantInterface](const FNavigationElement&, FNavigationRelevantData& OutNavigationRelevantData)
		{
			NavRelevantInterface->GetNavigationData(OutNavigationRelevantData);
		});

	CustomGeometryExportDelegate.BindWeakLambda(Cast<UObject>(NavRelevantInterface),
		[NavRelevantInterface](const FNavigationElement&, FNavigableGeometryExport& OutGeometry, bool& bOutShouldExportDefaultGeometry)
		{
			bOutShouldExportDefaultGeometry = NavRelevantInterface->DoCustomNavigableGeometryExport(OutGeometry);
		});

	if (NavRelevantInterface->SupportsGatheringGeometrySlices())
	{
		GeometrySliceExportDelegate.BindWeakLambda(Cast<UObject>(NavRelevantInterface),
			[MutableInterface](const FNavigationElement&, FNavigableGeometryExport& OutGeometryExport, const FBox& InSliceBox)
			{
				MutableInterface->PrepareGeometryExportSync();
				MutableInterface->GatherGeometrySlice(OutGeometryExport, InSliceBox);
			});
	}

#if !UE_BUILD_SHIPPING
	if (UE::NavigationElement::Private::bValidateNavigationElementInitialization)
	{
		if (!Bounds.IsValid)
		{
			UE_LOGF(LogNavigation, Warning,
			   "Initializing a FNavigationElement from '%ls' that provides invalid navigation bounds.",
			   *GetPathNameSafe(Cast<UObject>(NavRelevantInterface)));
		}
		else if (bDirtyAreaOnRegistration && FVector2d(Bounds.GetSize()).IsNearlyZero())
		{
			UE_LOGF(LogNavigation, Warning,
			   "Initializing a FNavigationElement from '%ls' that provides empty navigation bounds.",
			   *GetPathNameSafe(Cast<UObject>(NavRelevantInterface)));
		}

		if (!NavRelevantInterface->IsNavigationRelevant())
		{
			UE_LOGF(LogNavigation, Warning,
			   "Initializing a FNavigationElement from '%ls' for which 'IsNavigationRelevant()' returns 'false'.",
			   *GetPathNameSafe(Cast<UObject>(NavRelevantInterface)));
		}
	}
#endif // !UE_BUILD_SHIPPING
}

TSharedRef<const FNavigationElement> FNavigationElement::CreateFromNavRelevantInterface(const INavRelevantInterface& NavRelevantInterface)
{
	return MakeShared<const FNavigationElement>(NavRelevantInterface, /*SubElementId*/INDEX_NONE);
}

void FNavigationElement::SetBodySetup(UBodySetup* InBodySetup)
{
	BodySetup = InBodySetup;
}

FNavigationElementHandle FNavigationElement::GetHandle() const
{
	return FNavigationElementHandle(OwnerUObject, SubElementId);
}

FString FNavigationElement::GetName() const
{
	const UObject* Owner = OwnerUObject.Get();

	if (SubElementId != INDEX_NONE)
	{
		return FString::Printf(TEXT("%s - %llu"), *GetNameSafe(Owner), SubElementId);
	}
	return FString::Printf(TEXT("%s"), *GetNameSafe(Owner));
}

FString FNavigationElement::GetPathName() const
{
	const UObject* Owner = OwnerUObject.Get();
	if (SubElementId != INDEX_NONE)
	{
		return FString::Printf(TEXT("%s - %llu"), *GetPathNameSafe(Owner), SubElementId);
	}
	return FString::Printf(TEXT("%s"), *GetPathNameSafe(Owner));
}

FString FNavigationElement::GetFullName() const
{
	const UObject* Owner = OwnerUObject.Get();
	if (SubElementId != INDEX_NONE)
	{
		return FString::Printf(TEXT("%s - %llu"), *GetFullNameSafe(Owner), SubElementId);
	}
	return FString::Printf(TEXT("%s"), *GetFullNameSafe(Owner));
}

uint32 GetTypeHash(const FNavigationElement& Element)
{
	return HashCombine(GetTypeHash(Element.OwnerUObject), GetTypeHash(Element.SubElementId));
}

FString LexToString(const FNavigationElement& Element)
{
	return Element.GetName();
}

FString LexToString(const FNavigationElementHandle& Handle)
{
	if (Handle.SubElementId != INDEX_NONE)
	{
		return FString::Printf(TEXT("%s - %llu"), *GetNameSafe(Handle.OwnerUObject.Get()), Handle.SubElementId);
	}
	return FString::Printf(TEXT("%s"), *GetNameSafe(Handle.OwnerUObject.Get()));
}