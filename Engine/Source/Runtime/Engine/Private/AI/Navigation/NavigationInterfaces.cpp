// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/Navigation/NavPathObserverInterface.h"
#include "AI/Navigation/NavEdgeProviderInterface.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/RVOAvoidanceInterface.h"
#include "AI/Navigation/NavigationDataInterface.h"
#include "AI/Navigation/NavigationInvokerInterface.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "UObject/Interface.h"

URVOAvoidanceInterface::URVOAvoidanceInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNavAgentInterface::UNavAgentInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FNavAgentProperties& INavAgentInterface::GetNavAgentPropertiesRef() const
{
	return FNavAgentProperties::DefaultProperties;
}

UNavRelevantInterface::UNavRelevantInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNavEdgeProviderInterface::UNavEdgeProviderInterface(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
}

UNavPathObserverInterface::UNavPathObserverInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNavigationDataInterface::UNavigationDataInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPathFollowingAgentInterface::UPathFollowingAgentInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNavigationInvokerInterface::UNavigationInvokerInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
