// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/NavigationModifier.h"
#include "Math/ConvexHull2d.h"
#include "UObject/UnrealType.h"
#include "EngineStats.h"
#include "Components/BrushComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavAreaBase.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"

// if square distance between two points is less than this the those points
// will be considered identical when calculating convex hull
// should be less than voxel size (recast navmesh)
static const FVector::FReal CONVEX_HULL_POINTS_MIN_DISTANCE_SQ = 4.0f * 4.0f;

bool FCompositeNavModifier::bEnableNavMeshResolutions = true;

namespace UE::Navigation::Private
{
	FAutoConsoleVariableRef CVarEnableNavMeshResolutions(TEXT("ai.nav.EnableNavMeshResolutions"), FCompositeNavModifier::bEnableNavMeshResolutions, TEXT("When set to false, navmesh resoutions will be ignored."), ECVF_Default);
}

//----------------------------------------------------------------------//
// FNavigationLinkBase
//----------------------------------------------------------------------//
FNavigationLinkBase::FNavigationLinkBase() 
	: LeftProjectHeight(0.0f), MaxFallDownLength(1000.0f), SnapRadius(30.f), SnapHeight(50.0f),
	  Direction(ENavLinkDirection::BothWays), bUseSnapHeight(false), bSnapToCheapestArea(true),
	  bCustomFlag0(false), bCustomFlag1(false), bCustomFlag2(false), bCustomFlag3(false), bCustomFlag4(false),
	  bCustomFlag5(false), bCustomFlag6(false), bCustomFlag7(false),
	  bIsGenerated(false)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UserId = InvalidUserId;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	AreaClass = nullptr;
	SupportedAgentsBits = 0xFFFFFFFF;
}

void FNavigationLinkBase::SetAreaClass(UClass* InAreaClass)
{
	if (InAreaClass)
	{
		AreaClassOb = InAreaClass;
		AreaClass = InAreaClass;
	}
}

UClass* FNavigationLinkBase::GetAreaClass() const
{
	UClass* ClassOb = AreaClassOb.Get();
	return ClassOb ? ClassOb : *FNavigationSystem::GetDefaultWalkableArea();
}

void FNavigationLinkBase::InitializeAreaClass(const bool bForceRefresh)
{
	AreaClassOb = (UClass*)AreaClass;
}

bool FNavigationLinkBase::HasMetaArea() const
{
	return AreaClass && AreaClass->GetDefaultObject<UNavAreaBase>()->IsMetaArea();
}

#if WITH_EDITORONLY_DATA
void FNavigationLinkBase::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_NAVIGATION_AGENT_SELECTOR)
	{
		SupportedAgents.bSupportsAgent0 = bSupportsAgent0;
		SupportedAgents.bSupportsAgent1 = bSupportsAgent1;
		SupportedAgents.bSupportsAgent2 = bSupportsAgent2;
		SupportedAgents.bSupportsAgent3 = bSupportsAgent3;
		SupportedAgents.bSupportsAgent4 = bSupportsAgent4;
		SupportedAgents.bSupportsAgent5 = bSupportsAgent5;
		SupportedAgents.bSupportsAgent6 = bSupportsAgent6;
		SupportedAgents.bSupportsAgent7 = bSupportsAgent7;
		SupportedAgents.bSupportsAgent8 = bSupportsAgent8;
		SupportedAgents.bSupportsAgent9 = bSupportsAgent9;
		SupportedAgents.bSupportsAgent10 = bSupportsAgent10;
		SupportedAgents.bSupportsAgent11 = bSupportsAgent11;
		SupportedAgents.bSupportsAgent12 = bSupportsAgent12;
		SupportedAgents.bSupportsAgent13 = bSupportsAgent13;
		SupportedAgents.bSupportsAgent14 = bSupportsAgent14;
		SupportedAgents.bSupportsAgent15 = bSupportsAgent15;
		SupportedAgents.MarkInitialized();
	}

	// can't initialize at this time, used UClass may not be ready yet
}
#endif

#if WITH_EDITOR

void FNavigationLinkBase::DescribeCustomFlags(const TArray<FString>& EditableFlagNames, UClass* NavLinkPropertiesOwnerClass)
{
	if (NavLinkPropertiesOwnerClass == nullptr)
	{
		NavLinkPropertiesOwnerClass = UNavLinkDefinition::StaticClass();
	}

	const int32 MaxFlags = FMath::Min(8, EditableFlagNames.Num());
	const FString CustomNameMeta = "DisplayName";

	for (TFieldIterator<FProperty> PropertyIt(NavLinkPropertiesOwnerClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Prop = *PropertyIt;

		FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop);
		FStructProperty* StructProp = CastField<FStructProperty>(ArrayProp ? ArrayProp->Inner : Prop);

		if (StructProp)
		{
			for (UStruct* StructIt = StructProp->Struct; StructIt; StructIt = StructIt->GetSuperStruct())
			{
				if (StructIt->GetFName() == TEXT("NavigationLinkBase"))
				{
					for (int32 Idx = 0; Idx < 8; Idx++)
					{
						FString PropName("bCustomFlag");
						PropName += TTypeToString<int32>::ToString(Idx);

						FProperty* FlagProp = FindFProperty<FProperty>(StructIt, *PropName);
						if (FlagProp)
						{
							if (Idx < MaxFlags)
							{
								FlagProp->SetPropertyFlags(CPF_Edit);
								FlagProp->SetMetaData(*CustomNameMeta, *EditableFlagNames[Idx]);
							}
							else
							{
								FlagProp->ClearPropertyFlags(CPF_Edit);
							}
						}
					}

					break;
				}
			}
		}
	}
}

#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// UNavLinkDefinition
//----------------------------------------------------------------------//
UNavLinkDefinition::UNavLinkDefinition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bHasInitializedAreaClasses(false)
	, bHasDeterminedMetaAreaClass(false)
	, bHasMetaAreaClass(false)
	, bHasDeterminedAdjustableLinks(false)
	, bHasAdjustableLinks(false)
{
}

const TArray<FNavigationLink>& UNavLinkDefinition::GetLinksDefinition(UClass* LinkDefinitionClass)
{
	const UNavLinkDefinition* LinkDefCDO = LinkDefinitionClass ? LinkDefinitionClass->GetDefaultObject<UNavLinkDefinition>() : nullptr;
	if (LinkDefCDO)
	{
		LinkDefCDO->InitializeAreaClass();
		return LinkDefCDO->Links;
	}

	static const TArray<FNavigationLink> DummyDefinition;
	return DummyDefinition;
}

const TArray<FNavigationSegmentLink>& UNavLinkDefinition::GetSegmentLinksDefinition(UClass* LinkDefinitionClass)
{
	const UNavLinkDefinition* LinkDefCDO = LinkDefinitionClass ? LinkDefinitionClass->GetDefaultObject<UNavLinkDefinition>() : nullptr;
	if (LinkDefCDO)
	{
		LinkDefCDO->InitializeAreaClass();
		return LinkDefCDO->SegmentLinks;
	}

	static const TArray<FNavigationSegmentLink> DummyDefinition;
	return DummyDefinition;
}

#if WITH_EDITOR
void UNavLinkDefinition::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// In case relevant data has changed, clear the flag that says we've determined whether there's a meta area class
	// so it will be recalculated the next time it's needed.
	bHasDeterminedMetaAreaClass = false;

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UNavLinkDefinition::InitializeAreaClass() const
{
	if (bHasInitializedAreaClasses)
	{
		return;
	}

	UNavLinkDefinition* MutableThis = (UNavLinkDefinition*)this;
	MutableThis->bHasInitializedAreaClasses = true;

	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		FNavigationLink& LinkData = MutableThis->Links[Idx];
		LinkData.InitializeAreaClass();
	}

	for (int32 Idx = 0; Idx < SegmentLinks.Num(); Idx++)
	{
		FNavigationSegmentLink& LinkData = MutableThis->SegmentLinks[Idx];
		LinkData.InitializeAreaClass();
	}
}

bool UNavLinkDefinition::HasMetaAreaClass() const
{
	if (bHasDeterminedMetaAreaClass)
	{
		return bHasMetaAreaClass;
	}

	UNavLinkDefinition* MutableThis = (UNavLinkDefinition*)this;
	MutableThis->bHasDeterminedMetaAreaClass = true;

	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		const FNavigationLink& LinkData = Links[Idx];
		if (!bHasMetaAreaClass && LinkData.HasMetaArea())
		{
			MutableThis->bHasMetaAreaClass = true;
			return true;
		}
	}

	for (int32 Idx = 0; Idx < SegmentLinks.Num(); Idx++)
	{
		const FNavigationSegmentLink& LinkData = SegmentLinks[Idx];
		if (!bHasMetaAreaClass && LinkData.HasMetaArea())
		{
			MutableThis->bHasMetaAreaClass = true;
			return true;
		}
	}

	return false;
}

bool UNavLinkDefinition::HasAdjustableLinks() const
{
	if (bHasDeterminedAdjustableLinks && !GIsEditor)
	{
		return bHasAdjustableLinks;
	}

	UNavLinkDefinition* MutableThis = (UNavLinkDefinition*)this;
	MutableThis->bHasDeterminedAdjustableLinks = true;

	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		if (Links[Idx].MaxFallDownLength > 0)
		{
			MutableThis->bHasAdjustableLinks = true;
			return true;
		}
	}

	for (int32 Idx = 0; Idx < SegmentLinks.Num(); Idx++)
	{
		if (SegmentLinks[Idx].MaxFallDownLength > 0)
		{
			MutableThis->bHasAdjustableLinks = true;
			return true;
		}
	}

	return false;
}



//----------------------------------------------------------------------//
// FAreaNavModifier
//----------------------------------------------------------------------//

FString LexToString(ENavigationShapeType::Type Type)
{
	switch (Type)
	{
	case ENavigationShapeType::Unknown: return TEXT("Unknown");
	case ENavigationShapeType::Cylinder: return TEXT("Cylinder");
	case ENavigationShapeType::Box: return TEXT("Box");
	case ENavigationShapeType::Convex: return TEXT("Convex");
	case ENavigationShapeType::InstancedConvex: return TEXT("InstancedConvex");
	default:
		checkNoEntry();
		return FString();
	}
}

FString LexToString(ENavigationAreaMode::Type Type)
{
	switch (Type)
	{
	case ENavigationAreaMode::Apply: return TEXT("Apply");
	case ENavigationAreaMode::Replace: return TEXT("Replace");
	case ENavigationAreaMode::ApplyInLowPass: return TEXT("ApplyInLowPass");
	case ENavigationAreaMode::ReplaceInLowPass: return TEXT("ReplaceInLowPass");
	default:
		checkNoEntry();
		return FString();
	}
}

FAreaNavModifier::FAreaNavModifier() 
: Cost(0.0f)
, FixedCost(0.0f)
, Bounds(ForceInitToZero)
, ShapeType(ENavigationShapeType::Unknown)
, ApplyMode(ENavigationAreaMode::Apply)
, bExpandTopByCellHeight(false)
, bIncludeAgentHeight(false)
, bIsLowAreaModifier(false)
{
}

FAreaNavModifier::FAreaNavModifier(float Radius, float Height, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	Init(InAreaClass);

	// check if the cylinder is vertical (Z axis pointing straight up or straight down)
	const FVector TransformZAxis = LocalToWorld.GetUnitAxis(EAxis::Z);
	const float VerticalDotProduct = TransformZAxis.Dot(FVector::UpVector);
	// About 1 degree off vertical
	if (FMath::IsNearlyEqual(FMath::Abs(VerticalDotProduct), 1.f, UE_KINDA_SMALL_NUMBER))
	{
		FVector Scale3D = LocalToWorld.GetScale3D().GetAbs();
		const FVector::FReal RadiusScaled = Radius * FMath::Max(Scale3D.X, Scale3D.Y);
		const FVector::FReal HeightScaled = Height * Scale3D.Z;

		Points.SetNumUninitialized(2);
		Points[0] = LocalToWorld.GetLocation();
		Points[1].X = RadiusScaled;
		Points[1].Z = HeightScaled;
		ShapeType = ENavigationShapeType::Cylinder;

		Bounds = FBox::BuildAABB(LocalToWorld.GetLocation(), FVector(RadiusScaled, RadiusScaled, HeightScaled));
	}
	// Fallback to a box if the cylinder is not vertical
	else
	{
		const float HalfHeight = Height * 0.5f;
		FVector Center(0.f, 0.f, HalfHeight);
		FVector Extents(Radius, Radius, HalfHeight);
		SetBox(FBox::BuildAABB(Center, Extents), LocalToWorld);
	}
}

FAreaNavModifier::FAreaNavModifier(const FVector& Extent, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	Init(InAreaClass);
	SetBox(FBox::BuildAABB(FVector::ZeroVector, Extent), LocalToWorld);
}

FAreaNavModifier::FAreaNavModifier(const FBox& Box, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	Init(InAreaClass);
	SetBox(Box, LocalToWorld);
}

FAreaNavModifier::FAreaNavModifier(const TArray<FVector>& InPoints, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	Init(InAreaClass);
	SetConvex(InPoints.GetData(), 0, InPoints.Num(), CoordType, LocalToWorld);
}

FAreaNavModifier::FAreaNavModifier(const TConstArrayView<FVector> InPoints, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	Init(InAreaClass);
	SetConvex(InPoints.GetData(), 0, InPoints.Num(), CoordType, LocalToWorld);
}

FAreaNavModifier::FAreaNavModifier(const TArray<FVector>& InPoints, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	check(InPoints.IsValidIndex(FirstIndex) && InPoints.IsValidIndex(LastIndex-1));

	Init(InAreaClass);
	SetConvex(InPoints.GetData(), FirstIndex, LastIndex, CoordType, LocalToWorld);
}

FAreaNavModifier::FAreaNavModifier(const TNavStatArray<FVector>& InPoints, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	check(InPoints.IsValidIndex(FirstIndex) && InPoints.IsValidIndex(LastIndex-1));

	Init(InAreaClass);
	SetConvex(InPoints.GetData(), FirstIndex, LastIndex, CoordType, LocalToWorld);
}

void FAreaNavModifier::InitializeConvex(const TNavStatArray<FVector>& InPoints, const int32 FirstIndex, const int32 LastIndex, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	check(InPoints.IsValidIndex(FirstIndex) && InPoints.IsValidIndex(LastIndex-1));

	Init(InAreaClass);
	SetConvex(UE::LWC::ConvertArrayType<FVector>(InPoints).GetData(), FirstIndex, LastIndex, ENavigationCoordSystem::Unreal, LocalToWorld);	// LWC_TODO: Perf pessimization
}

void FAreaNavModifier::InitializePerInstanceConvex(const TNavStatArray<FVector>& InPoints, const int32 FirstIndex, const int32 LastIndex, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	check(InPoints.IsValidIndex(FirstIndex) && InPoints.IsValidIndex(LastIndex - 1));

	Init(InAreaClass);
	SetPerInstanceConvex(UE::LWC::ConvertArrayType<FVector>(InPoints).GetData(), FirstIndex, LastIndex);	// LWC_TODO: Perf pessimization
}

FAreaNavModifier::FAreaNavModifier(const UBrushComponent* BrushComponent, const TSubclassOf<UNavAreaBase> InAreaClass)
{
	check(BrushComponent != NULL);

	TArray<FVector> Verts;
	if(BrushComponent->BrushBodySetup)
	{
		for (int32 ElemIndex = 0; ElemIndex < BrushComponent->BrushBodySetup->AggGeom.ConvexElems.Num(); ElemIndex++)
		{
			const FKConvexElem& Convex = BrushComponent->BrushBodySetup->AggGeom.ConvexElems[ElemIndex];
			for (int32 VertexIndex = 0; VertexIndex < Convex.VertexData.Num(); VertexIndex++)
			{
				Verts.AddUnique(Convex.VertexData[VertexIndex]);
			}
		}
	}

	Init(InAreaClass);
	SetConvex(Verts.GetData(), 0, Verts.Num(), ENavigationCoordSystem::Unreal, BrushComponent->GetComponentTransform());
}

void FAreaNavModifier::GetCylinder(FCylinderNavAreaData& Data) const
{
	check(Points.Num() == 2 && ShapeType == ENavigationShapeType::Cylinder);
	Data.Origin = Points[0];
	Data.Radius = FloatCastChecked<float>(Points[1].X, UE::LWC::DefaultFloatPrecision);
	Data.Height = FloatCastChecked<float>(Points[1].Z, UE::LWC::DefaultFloatPrecision);
}

void FAreaNavModifier::GetBox(FBoxNavAreaData& Data) const
{
	check(Points.Num() == 2 && ShapeType == ENavigationShapeType::Box);
	Data.Origin = Points[0];
	Data.Extent = Points[1];
}

void FAreaNavModifier::GetConvex(FConvexNavAreaData& Data) const
{
	check(ShapeType == ENavigationShapeType::Convex);
	Data.Points.Append(Points);
	FVector LastPoint = Data.Points.Pop();
	Data.MinZ = LastPoint.X;
	Data.MaxZ = LastPoint.Y;
}

void FAreaNavModifier::GetPerInstanceConvex(const FTransform& InLocalToWorld, FConvexNavAreaData& OutConvexData) const
{
	ensure(ShapeType == ENavigationShapeType::InstancedConvex);

	FBox TmpBounds;
	FillConvexNavAreaData(Points.GetData(), Points.Num(), InLocalToWorld, OutConvexData, TmpBounds);
}

void FAreaNavModifier::Init(const TSubclassOf<UNavAreaBase> InAreaClass)
{
	bExpandTopByCellHeight = false;
	bIncludeAgentHeight = false;
	ApplyMode = ENavigationAreaMode::Apply;
	Cost = 0.0f;
	FixedCost = 0.0f;
	Bounds = FBox(ForceInitToZero);
	SetAreaClass(InAreaClass);
}

bool IsMetaAreaClass(UClass& AreaClass)
{
	const UNavAreaBase* AreaClassCDO = GetDefault<UNavAreaBase>(&AreaClass);
	return AreaClassCDO && AreaClassCDO->IsMetaArea();
}

void FAreaNavModifier::SetAreaClass(const TSubclassOf<UNavAreaBase> InAreaClass)
{
	AreaClassOb = (UClass*)InAreaClass;

	UClass* AreaClass1 = AreaClassOb.Get();
	UClass* AreaClass2 = ReplaceAreaClassOb.Get();
	bHasMetaAreas = (AreaClass1 && IsMetaAreaClass(*AreaClass1))
		|| (AreaClass2 && IsMetaAreaClass(*AreaClass2));
}

void FAreaNavModifier::SetAreaClassToReplace(const TSubclassOf<UNavAreaBase> InAreaClass)
{
	ReplaceAreaClassOb = (UClass*)InAreaClass;

	UClass* AreaClass1 = AreaClassOb.Get();
	UClass* AreaClass2 = ReplaceAreaClassOb.Get();
	bHasMetaAreas = (AreaClass1 && IsMetaAreaClass(*AreaClass1))
		|| (AreaClass2 && IsMetaAreaClass(*AreaClass2));

	if (AreaClass2)
	{
		bIsLowAreaModifier = AreaClass2->GetDefaultObject<UNavAreaBase>()->IsLowArea();
		ApplyMode = bIsLowAreaModifier ? ENavigationAreaMode::ReplaceInLowPass : ENavigationAreaMode::Replace;
	}
	else if (ApplyMode == ENavigationAreaMode::ReplaceInLowPass || ApplyMode == ENavigationAreaMode::Replace)
	{
		// since we no longer have ReplaceAreaClass the new value of ApplyMode and bIsLowAreaModifier should depend on previous value of ApplyMode
		bIsLowAreaModifier = ApplyMode == ENavigationAreaMode::ReplaceInLowPass;
		ApplyMode = ApplyMode == ENavigationAreaMode::ReplaceInLowPass ? ENavigationAreaMode::ApplyInLowPass : ENavigationAreaMode::Apply;		
	}
}

void FAreaNavModifier::SetApplyMode(ENavigationAreaMode::Type InApplyMode)
{
	ApplyMode = InApplyMode;
	bIsLowAreaModifier = (InApplyMode == ENavigationAreaMode::ApplyInLowPass) || (InApplyMode == ENavigationAreaMode::ReplaceInLowPass);
}

bool IsAngleMatching(FRotator::FReal Angle)
{
	const float AngleThreshold = 1.0f; // degrees
	return (Angle < AngleThreshold) || ((90.0f - Angle) < AngleThreshold);
}

void FAreaNavModifier::SetBox(const FBox& Box, const FTransform& LocalToWorld)
{
	const FVector BoxOrigin = Box.GetCenter();
	const FVector BoxExtent = Box.GetExtent();

	TArray<FVector> Corners;
	for (int32 i = 0; i < 8; i++)
	{
		const FVector Dir(((i / 4) % 2) ? 1 : -1, ((i / 2) % 2) ? 1 : -1, (i % 2) ? 1 : -1);
		Corners.Add(LocalToWorld.TransformPosition(BoxOrigin + BoxExtent * Dir));
	}

	// check if it can be used as AABB
	const FRotator Rotation = LocalToWorld.GetRotation().Rotator();
	const FRotator::FReal PitchMod = FMath::Fmod(FMath::Abs(Rotation.Pitch), 90.0f);
	const FRotator::FReal YawMod = FMath::Fmod(FMath::Abs(Rotation.Yaw), 90.0f);
	const FRotator::FReal RollMod = FMath::Fmod(FMath::Abs(Rotation.Roll), 90.0f);
	if (IsAngleMatching(PitchMod) && IsAngleMatching(YawMod) && IsAngleMatching(RollMod))
	{
		Bounds = FBox(ForceInit);
		for (int32 i = 0; i < Corners.Num(); i++)
		{
			Bounds += Corners[i];
		}

		Points.SetNumUninitialized(2);
		Points[0] = Bounds.GetCenter();
		Points[1] = Bounds.GetExtent();
		ShapeType = ENavigationShapeType::Box;
	}
	else
	{
		SetConvex(Corners.GetData(), 0, Corners.Num(), ENavigationCoordSystem::Unreal, FTransform::Identity);
	}
}

void FAreaNavModifier::FillConvexNavAreaData(const FVector* InPoints, const int32 InNumPoints, const FTransform& InTotalTransform, FConvexNavAreaData& OutConvexData, FBox& OutBounds)
{
	OutBounds = FBox(ForceInit);
	OutConvexData.Points.Reset();
	OutConvexData.MinZ = UE_MAX_FLT;
	OutConvexData.MaxZ = -UE_MAX_FLT;

	if (InNumPoints <= 0)
	{
		return;
	}

	const int MaxConvexPoints = 8;
	TArray<FVector, TInlineAllocator<MaxConvexPoints>> HullVertices;
	HullVertices.Empty(MaxConvexPoints);

	for (int32 i = 0; i < InNumPoints; i++)
	{
		FVector TransformedPoint = InTotalTransform.TransformPosition(InPoints[i]);
		OutConvexData.MinZ = FMath::Min(OutConvexData.MinZ, TransformedPoint.Z);
		OutConvexData.MaxZ = FMath::Max(OutConvexData.MaxZ, TransformedPoint.Z);
		TransformedPoint.Z = 0.f;

		// check if there's a similar point already in HullVertices array
		bool bUnique = true;
		const FVector* RESTRICT Start = HullVertices.GetData();
		for (const FVector* RESTRICT Data = Start, *RESTRICT DataEnd = Data + HullVertices.Num(); Data != DataEnd; ++Data)
		{
			if (FVector::DistSquared(*Data, TransformedPoint) < CONVEX_HULL_POINTS_MIN_DISTANCE_SQ)
			{
				bUnique = false;
				break;
			}
		}

		if (bUnique)
		{
			HullVertices.Add(TransformedPoint);
		}
	}

	TArray<int32, TInlineAllocator<MaxConvexPoints>> HullIndices;
	HullIndices.Empty(MaxConvexPoints);

	ConvexHull2D::ComputeConvexHullLegacy(HullVertices, HullIndices);
	
	// ConvexHull implementation requires at least 3 vertices  (i.e. GrowConvexHull)
	const int32 MIN_NUM_POINTS = 3;

	if (HullIndices.Num() >= MIN_NUM_POINTS)
	{
		for (int32 i = 0; i < HullIndices.Num(); ++i)
		{
			const FVector& HullVert = HullVertices[HullIndices[i]];
			OutConvexData.Points.Add(HullVert);
			OutBounds += HullVert;
		}

		OutBounds.Min.Z = OutConvexData.MinZ;
		OutBounds.Max.Z = OutConvexData.MaxZ;
	}
}

void FAreaNavModifier::SetPerInstanceConvex(const FVector* InPoints, const int32 InFirstIndex, const int32 InLastIndex)
{
	// Per Instance modifiers requires that we keep all unique points until we receive the instance transform for
    // ConvexHull to be computed. Local Bounds must be computed right away.
	Bounds = FBox(ForceInit);
	for (int32 i = InFirstIndex; i < InLastIndex; i++)
	{
		const FVector& CurrentPoint = InPoints[i];
		FVector* SamePoint = Points.FindByPredicate([&CurrentPoint](FVector& Point) { return FMath::IsNearlyZero(FVector::DistSquared(Point, CurrentPoint)); });
		if (SamePoint == nullptr)
		{
			Points.Add(CurrentPoint);
			Bounds += CurrentPoint;
		}
	}
	ShapeType = ENavigationShapeType::InstancedConvex;
}

void FAreaNavModifier::SetConvex(const FVector* InPoints, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld)
{
	const FTransform& TotalTransform = LocalToWorld * FNavigationSystem::GetCoordTransform(CoordType, ENavigationCoordSystem::Unreal);

	FConvexNavAreaData ConvexData;
	FillConvexNavAreaData(InPoints + FirstIndex, LastIndex - FirstIndex, TotalTransform, ConvexData, Bounds);

	if (ConvexData.Points.Num() > 0)
	{
		Points.Append(ConvexData.Points);
		Points.Add(FVector(ConvexData.MinZ, ConvexData.MaxZ, 0));
		ShapeType = ENavigationShapeType::Convex;
	}
	else
	{
		ShapeType = ENavigationShapeType::Unknown;
	}
}

//----------------------------------------------------------------------//
// FCustomLinkNavModifier
//----------------------------------------------------------------------//
void FCustomLinkNavModifier::Set(TSubclassOf<UNavLinkDefinition> InPresetLinkClass, const FTransform& InLocalToWorld)
{
	LinkDefinitionClassOb = (UClass*)InPresetLinkClass;
	LocalToWorld = InLocalToWorld;

	const UNavLinkDefinition* LinkDefOb = InPresetLinkClass->GetDefaultObject<UNavLinkDefinition>();
	if (LinkDefOb)
	{
		LinkDefOb->InitializeAreaClass();

		bHasMetaAreas = LinkDefOb->HasMetaAreaClass();
	}
	else
	{
		bHasMetaAreas = false;
	}
}

//----------------------------------------------------------------------//
// FSimpleLinkNavModifier
//----------------------------------------------------------------------//
void FSimpleLinkNavModifier::SetLinks(const TArray<FNavigationLink>& InLinks)
{
	Links = InLinks;
	bHasMetaAreasPoint = false;

	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		FNavigationLink& LinkData = Links[Idx];

		bHasMetaAreasPoint |= LinkData.HasMetaArea();
		bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	}

	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::SetSegmentLinks(const TArray<FNavigationSegmentLink>& InLinks)
{
	SegmentLinks = InLinks;

	bHasMetaAreasSegment = false;
	for (int32 Idx = 0; Idx < SegmentLinks.Num(); Idx++)
	{
		FNavigationSegmentLink& LinkData = SegmentLinks[Idx];
		LinkData.NavLinkId = NavLinkId;

		bHasMetaAreasSegment |= LinkData.HasMetaArea();
		bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	}
	
	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::AppendLinks(const TArray<FNavigationLink>& InLinks)
{
	const int32 LinkBase = SegmentLinks.Num();
	Links.Append(InLinks);

	for (int32 Idx = 0; Idx < InLinks.Num(); Idx++)
	{
		FNavigationLink& LinkData = Links[LinkBase + Idx];

		bHasMetaAreasPoint |= LinkData.HasMetaArea();
		bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	}

	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::AppendSegmentLinks(const TArray<FNavigationSegmentLink>& InLinks)
{
	const int32 LinkBase = SegmentLinks.Num();
	SegmentLinks.Append(InLinks);

	for (int32 Idx = 0; Idx < InLinks.Num(); Idx++)
	{
		FNavigationSegmentLink& LinkData = SegmentLinks[LinkBase + Idx];
		LinkData.NavLinkId = NavLinkId;

		bHasMetaAreasSegment |= LinkData.HasMetaArea();
		bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	}

	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::AddLink(const FNavigationLink& InLink)
{
	const int32 LinkIdx = Links.Add(InLink);

	FNavigationLink& LinkData = Links[LinkIdx];

	bHasMetaAreasPoint |= LinkData.HasMetaArea();
	bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::AddSegmentLink(const FNavigationSegmentLink& InLink)
{
	const int32 LinkIdx = SegmentLinks.Add(InLink);

	FNavigationSegmentLink& LinkData = SegmentLinks[LinkIdx];
	LinkData.NavLinkId = NavLinkId;

	bHasMetaAreasSegment |= LinkData.HasMetaArea();
	bHasFallDownLinks |= LinkData.MaxFallDownLength > 0.f;
	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}

void FSimpleLinkNavModifier::UpdateFlags()
{
	bHasMetaAreasPoint = false;
	bHasMetaAreasSegment = false;
	bHasFallDownLinks = false;

	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		bHasMetaAreasPoint |= Links[Idx].HasMetaArea();
		bHasFallDownLinks |= Links[Idx].MaxFallDownLength > 0.f;
	}
	
	bHasMetaAreas = bHasMetaAreasSegment || bHasMetaAreasPoint;
}


//----------------------------------------------------------------------//
// FCompositeNavMeshModifier
//----------------------------------------------------------------------//
void FCompositeNavModifier::Shrink()
{
	Areas.Shrink();
	SimpleLinks.Shrink();
	CustomLinks.Shrink();
}

void FCompositeNavModifier::Reset()
{
	Areas.Reset();
	SimpleLinks.Reset();
	CustomLinks.Reset();
	bHasPotentialLinks = false;
	bAdjustHeight = false;
	bIsPerInstanceModifier = false;
	bFillCollisionUnderneathForNavmesh = false;
	bMaskFillCollisionUnderneathForNavmesh = false;
	bRasterizeAsFilledConvexVolume = false;
	NavMeshResolution = ENavigationDataResolution::Invalid;
}

void FCompositeNavModifier::Empty()
{
	Areas.Empty();
	SimpleLinks.Empty();
	CustomLinks.Empty();
	bHasPotentialLinks = false;
	bAdjustHeight = false;
	bFillCollisionUnderneathForNavmesh = false;
	bMaskFillCollisionUnderneathForNavmesh = false;
	bRasterizeAsFilledConvexVolume = false;
	NavMeshResolution = ENavigationDataResolution::Invalid;
}

FCompositeNavModifier FCompositeNavModifier::GetInstantiatedMetaModifier(const FNavAgentProperties* NavAgent, const TWeakObjectPtr<UObject> WeakOwnerPtr) const
{
	return GetInstantiatedMetaModifier(NavAgent, static_cast<const TWeakObjectPtr<const UObject>&>(WeakOwnerPtr));
}

FCompositeNavModifier FCompositeNavModifier::GetInstantiatedMetaModifier(const FNavAgentProperties* NavAgent, const TWeakObjectPtr<const UObject>& WeakOwnerPtr) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_MetaAreaTranslation);
	FCompositeNavModifier Result;

	check(NavAgent);
	// should not be called when HasMetaAreas == false since it's a waste of performance
	ensure(HasMetaAreas() == true);

	const UObject* ObjectOwner = WeakOwnerPtr.Get();
	if (ObjectOwner == nullptr)
	{
		return Result;
	}
	
	auto FindActorOwner = [](const UObject* Obj) -> const AActor*
	{
		while (Obj)
		{
			if (const AActor* ActorOwner = Cast<AActor>(Obj->GetOuter()))
			{
				return ActorOwner;
			}
			Obj = Obj->GetOuter();
		}
		return nullptr;
	};

	const AActor* ActorOwner = Cast<AActor>(ObjectOwner);
	if (ActorOwner == nullptr)
	{
		ActorOwner = FindActorOwner(ObjectOwner);
	}

	if (ActorOwner == nullptr)
	{
		return Result;
	}

	// copy values
	Result = *this;

	{
		Result.bHasMetaAreas = false;
		FAreaNavModifier* Area = Result.Areas.GetData();
		for (int32 Index = 0; Index < Result.Areas.Num(); ++Index, ++Area)
		{
			if (Area->HasMetaAreas())
			{
				Area->SetAreaClass(UNavAreaBase::PickAreaClassForAgent(Area->GetAreaClass(), *ActorOwner, *NavAgent));
				Area->SetAreaClassToReplace(UNavAreaBase::PickAreaClassForAgent(Area->GetAreaClassToReplace(), *ActorOwner, *NavAgent));
				Result.bHasMetaAreas = -1;
			}
		}
	}

	{
		FSimpleLinkNavModifier* SimpleLink = Result.SimpleLinks.GetData();
		for (int32 Index = 0; Index < Result.SimpleLinks.Num(); ++Index, ++SimpleLink)
		{
			if (SimpleLink->HasMetaAreas())
			{
				for (int32 LinkIndex = 0; LinkIndex < SimpleLink->Links.Num(); ++LinkIndex)
				{
					FNavigationLink& Link = SimpleLink->Links[LinkIndex];
					Link.SetAreaClass(UNavAreaBase::PickAreaClassForAgent(Link.GetAreaClass(), *ActorOwner, *NavAgent));
				}

				for (int32 LinkIndex = 0; LinkIndex < SimpleLink->SegmentLinks.Num(); ++LinkIndex)
				{
					FNavigationSegmentLink& Link = SimpleLink->SegmentLinks[LinkIndex];
					Link.SetAreaClass(UNavAreaBase::PickAreaClassForAgent(Link.GetAreaClass(), *ActorOwner, *NavAgent));
				}
			}
		}
	}

	{
		// create new entry in CustomLinks, and put there all regular links that have meta area class
		// making plain FNavigationLink from them first
		Result.SimpleLinks.Reserve(Result.CustomLinks.Num() + Result.SimpleLinks.Num());
		
		for (int32 Index = Result.CustomLinks.Num() - 1; Index >= 0; --Index)
		{
			FCustomLinkNavModifier* CustomLink = &Result.CustomLinks[Index];
			if (CustomLink->HasMetaAreas())
			{
				const TArray<FNavigationLink>& Links = UNavLinkDefinition::GetLinksDefinition(CustomLink->GetNavLinkClass());

				FSimpleLinkNavModifier& SimpleLink = Result.SimpleLinks[Result.SimpleLinks.AddZeroed(1)];
				SimpleLink.LocalToWorld = CustomLink->LocalToWorld;
				SimpleLink.Links.Reserve(Links.Num());

				// and copy all links to FCompositeNavMeshModifier::CustomLinks 
				// updating AreaClass if it's meta area
				for (int32 LinkIndex = 0; LinkIndex < Links.Num(); ++LinkIndex)
				{
					const int32 AddedIdx = SimpleLink.Links.Add(Links[LinkIndex]);
					FNavigationLink& NavLink = SimpleLink.Links[AddedIdx];
					NavLink.SetAreaClass(UNavAreaBase::PickAreaClassForAgent(NavLink.GetAreaClass(), *ActorOwner, *NavAgent));
				}				

				const TArray<FNavigationSegmentLink>& SegmentLinks = UNavLinkDefinition::GetSegmentLinksDefinition(CustomLink->GetNavLinkClass());

				FSimpleLinkNavModifier& SimpleSegLink = Result.SimpleLinks[Result.SimpleLinks.AddZeroed(1)];
				SimpleSegLink.LocalToWorld = CustomLink->LocalToWorld;
				SimpleSegLink.SegmentLinks.Reserve(SegmentLinks.Num());

				// and copy all links to FCompositeNavMeshModifier::CustomLinks 
				// updating AreaClass if it's meta area
				for (int32 LinkIndex = 0; LinkIndex < SegmentLinks.Num(); ++LinkIndex)
				{
					const int32 AddedIdx = SimpleSegLink.SegmentLinks.Add(SegmentLinks[LinkIndex]);
					FNavigationSegmentLink& NavLink = SimpleSegLink.SegmentLinks[AddedIdx];
					NavLink.SetAreaClass(UNavAreaBase::PickAreaClassForAgent(NavLink.GetAreaClass(), *ActorOwner, *NavAgent));
				}

				Result.CustomLinks.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
	}

	return Result;
}

void FCompositeNavModifier::CreateAreaModifiers(const UPrimitiveComponent* PrimComp, const TSubclassOf<UNavAreaBase> AreaClass, const TSubclassOf<UNavAreaBase> AreaClassToReplace)
{
	UBodySetup* BodySetup = PrimComp ? ((UPrimitiveComponent*)PrimComp)->GetBodySetup() : nullptr;
	if (BodySetup == nullptr)
	{
		return;
	}

	for (int32 Idx = 0; Idx < BodySetup->AggGeom.BoxElems.Num(); Idx++)
	{
		const FKBoxElem& BoxElem = BodySetup->AggGeom.BoxElems[Idx];
		const FBox BoxSize = BoxElem.CalcAABB(FTransform::Identity, 1.0f);

		FAreaNavModifier AreaMod(BoxSize, PrimComp->GetComponentTransform(), AreaClass);
		if (AreaClassToReplace)
		{
			AreaMod.SetAreaClassToReplace(AreaClassToReplace);
		}
		Add(AreaMod);
	}

	for (int32 Idx = 0; Idx < BodySetup->AggGeom.SphylElems.Num(); Idx++)
	{
		const FKSphylElem& SphylElem = BodySetup->AggGeom.SphylElems[Idx];
		const float SphylHalfHeight = SphylElem.Length * 0.5f + SphylElem.Radius;
		const FTransform AreaOffset(FVector(0, 0, -SphylHalfHeight));

		FAreaNavModifier AreaMod(SphylElem.Radius, 2.f * SphylHalfHeight, AreaOffset * SphylElem.GetTransform() * PrimComp->GetComponentTransform(), AreaClass);
		if (AreaClassToReplace)
		{
			AreaMod.SetAreaClassToReplace(AreaClassToReplace);
		}
		Add(AreaMod);
	}

	for (int32 Idx = 0; Idx < BodySetup->AggGeom.ConvexElems.Num(); Idx++)
	{
		const FKConvexElem& ConvexElem = BodySetup->AggGeom.ConvexElems[Idx];
		if (ConvexElem.VertexData.Num() > 0)
		{
			FAreaNavModifier AreaMod(UE::LWC::ConvertArrayType<FVector>(ConvexElem.VertexData), 0, ConvexElem.VertexData.Num(), ENavigationCoordSystem::Unreal, PrimComp->GetComponentTransform(), AreaClass);
			if (AreaClassToReplace)
			{
				AreaMod.SetAreaClassToReplace(AreaClassToReplace);
			}
			Add(AreaMod);
		}
		else
		{
			UE_LOGF(LogNavigation, Warning, "CreateAreaModifiers called for component %ls whose BodySetup contains ConvexElem with no vertex data at index %d. Not adding nav modifier.", *GetPathNameSafe(PrimComp), Idx);
		}
	}
	
	for (int32 Idx = 0; Idx < BodySetup->AggGeom.SphereElems.Num(); Idx++)
	{
		const FKSphereElem& SphereElem = BodySetup->AggGeom.SphereElems[Idx];
		const FTransform AreaOffset(FVector(0, 0, -SphereElem.Radius));

		FAreaNavModifier AreaMod(SphereElem.Radius, SphereElem.Radius * 2.0f, AreaOffset * PrimComp->GetComponentTransform(), AreaClass);
		if (AreaClassToReplace)
		{
			AreaMod.SetAreaClassToReplace(AreaClassToReplace);
		}
		Add(AreaMod);
	}
}

void FCompositeNavModifier::CreateAreaModifiers(const FCollisionShape& CollisionShape, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass, const bool bIncludeAgentHeight /*= false*/)
{
	if (CollisionShape.IsBox())
	{
		const FVector BoxExtent = CollisionShape.GetBox();
		FAreaNavModifier AreaMod(FBox(-BoxExtent, BoxExtent), LocalToWorld, AreaClass);
		AreaMod.SetIncludeAgentHeight(bIncludeAgentHeight);
		Add(AreaMod);
	}
	else if (CollisionShape.IsCapsule())
	{
		const float CapsuleHalfHeight = CollisionShape.GetCapsuleHalfHeight();
		const FTransform AreaOffset(FVector(0.0f, 0.0f, -CapsuleHalfHeight));
		FAreaNavModifier AreaMod(CollisionShape.GetCapsuleRadius(), CapsuleHalfHeight * 2.0f, AreaOffset * LocalToWorld, AreaClass); // Note: FAreaNavModifier creates a cylinder shape under the hood
		AreaMod.SetIncludeAgentHeight(bIncludeAgentHeight);
		Add(AreaMod);
	}
	else if (CollisionShape.IsSphere())
	{
		const float SphereRadius = CollisionShape.GetSphereRadius();
		const FTransform AreaOffset(FVector(0.0f, 0.0f, -SphereRadius));
		FAreaNavModifier AreaMod(SphereRadius, SphereRadius * 2.0f, AreaOffset * LocalToWorld, AreaClass); // Note: FAreaNavModifier creates a cylinder shape under the hood
		AreaMod.SetIncludeAgentHeight(bIncludeAgentHeight);
		Add(AreaMod);
	}
	else
	{
		UE_LOGF(LogNavigation, Error, "Asked to create a FAreaNavModifier with an unknown collision shape type! Collision Shape Type = %d", CollisionShape.ShapeType);
	}
}

uint32 FCompositeNavModifier::GetAllocatedSize() const
{
	SIZE_T MemUsed = Areas.GetAllocatedSize() + SimpleLinks.GetAllocatedSize() + CustomLinks.GetAllocatedSize();

	const FSimpleLinkNavModifier* SimpleLink = SimpleLinks.GetData();
	for (int32 Index = 0; Index < SimpleLinks.Num(); ++Index, ++SimpleLink)
	{
		MemUsed += SimpleLink->Links.GetAllocatedSize();
	}

	return IntCastChecked<uint32>(MemUsed);
}
