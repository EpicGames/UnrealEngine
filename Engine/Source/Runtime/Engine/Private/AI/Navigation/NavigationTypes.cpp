// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAreaBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavQueryFilter.h"
#include "AI/NavigationSystemBase.h"
#include "Components/ShapeComponent.h"
#include "Engine/Level.h"
#include "EngineStats.h"
#include "GameFramework/WorldSettings.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationTypes)

DEFINE_STAT(STAT_Navigation_MetaAreaTranslation);

static constexpr uint32 MAX_NAV_SEARCH_NODES = 2048;

namespace FNavigationSystem
{
	// these are totally arbitrary values, and it should haven happen these are ever used.
	// in any reasonable case UNavigationSystemV1::SupportedAgents should be filled in ini file
	// and only those values will be used
	const float FallbackAgentRadius = 35.f;
	const float FallbackAgentHeight = 144.f;

	bool IsLevelVisibilityChanging(const UObject* Object)
	{
		const ULevel* Level = nullptr;

		if (const UActorComponent* ObjectAsComponent = Cast<UActorComponent>(Object))
		{
			Level = ObjectAsComponent->GetComponentLevel();
		}
		else if (const AActor* Actor = Cast<AActor>(Object))
		{
			Level = Actor->GetLevel();
		}
		else if (const UAssetUserData* AssetUserData = Cast<UAssetUserData>(Object))
		{
			Level = AssetUserData->GetTypedOuter<ULevel>();
		}

		return Level ? Level->HasVisibilityChangeRequestPending() : false;
	}
	
	bool IsInBaseNavmesh(const UObject* Object)
	{
		const UActorComponent* ObjectAsComponent = Cast<UActorComponent>(Object);
		if (const AActor* Actor = ObjectAsComponent ? ObjectAsComponent->GetOwner() : Cast<AActor>(Object))
		{
			if (!Actor->HasDataLayers())
			{
				return true;
			}
		
			if (const UWorld* World = Object->GetWorld())
			{
				if (const AWorldSettings* WorldSettings = World->GetWorldSettings())
				{
					const TArray<TObjectPtr<UDataLayerAsset>>& BaseNavmeshLayers = WorldSettings->BaseNavmeshDataLayers;
					for (const TObjectPtr<UDataLayerAsset>& DataLayer : BaseNavmeshLayers)
					{
						if (Actor->ContainsDataLayer(DataLayer))
						{
							return true;
						}
					}
				}
			}
		}

		return false;
	}	
}

//----------------------------------------------------------------------//
// FNavigationQueryFilter
//----------------------------------------------------------------------//
const uint32 FNavigationQueryFilter::DefaultMaxSearchNodes = MAX_NAV_SEARCH_NODES;

//----------------------------------------------------------------------//
// FNavPathType
//----------------------------------------------------------------------//
uint32 FNavPathType::NextUniqueId = 0;

//----------------------------------------------------------------------//
// FNavDataConfig
//----------------------------------------------------------------------//
FNavDataConfig::FNavDataConfig(float Radius, float Height)
	: FNavAgentProperties(Radius, Height)
	, Name(TEXT("Default"))
	, Color(38, 75, 0, 164) // do not change this default value or the universe will explode!
	, DefaultQueryExtent(DEFAULT_NAV_QUERY_EXTENT_HORIZONTAL, DEFAULT_NAV_QUERY_EXTENT_HORIZONTAL, DEFAULT_NAV_QUERY_EXTENT_VERTICAL)
	, NavDataClass(FNavigationSystem::GetDefaultNavDataClass())
{
}

FNavDataConfig::FNavDataConfig(const FNavDataConfig& Other) = default;
FNavDataConfig& FNavDataConfig::operator=(const FNavDataConfig& Other) = default;

void FNavDataConfig::SetNavDataClass(UClass* InNavDataClass)
{
	NavDataClass = InNavDataClass;
}

void FNavDataConfig::SetNavDataClass(TSoftClassPtr<AActor> InNavDataClass)
{
	NavDataClass = InNavDataClass;
}

bool FNavDataConfig::IsValid() const 
{
	return FNavAgentProperties::IsValid() && NavDataClass.IsValid();
}

void FNavDataConfig::Invalidate()
{
	new(this) FNavAgentProperties();
	SetNavDataClass(nullptr);
}

FString FNavDataConfig::GetDescription() const
{
	return FString::Printf(TEXT("Name %s class %s agent radius %.1f")
		, *Name.ToString(), *NavDataClass.ToString(), AgentRadius);
}

//----------------------------------------------------------------------//
// FNavigationRelevantData
//----------------------------------------------------------------------//
FNavigationRelevantData::FNavigationRelevantData(const FNavigationRelevantData& Other)
	: TSharedFromThis<FNavigationRelevantData, ESPMode::ThreadSafe>(Other)
	, CollisionData(Other.CollisionData)
	, VoxelData(Other.VoxelData)
	, Bounds(Other.Bounds)
	, NavDataPerInstanceTransformDelegate(Other.NavDataPerInstanceTransformDelegate)
	, ShouldUseGeometryDelegate(Other.ShouldUseGeometryDelegate)
	, Modifiers(Other.Modifiers)
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, SourceObject(Other.SourceObject)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, SourceElement(Other.SourceElement)
	, bPendingLazyGeometryGathering(Other.bPendingLazyGeometryGathering)
	, bPendingLazyModifiersGathering(Other.bPendingLazyModifiersGathering)
	, bPendingChildLazyModifiersGathering(Other.bPendingChildLazyModifiersGathering)
	, bSupportsGatheringGeometrySlices(Other.bSupportsGatheringGeometrySlices)
	, bShouldSkipDirtyAreaOnAddOrRemove(Other.bShouldSkipDirtyAreaOnAddOrRemove)
	, bLoadedData(Other.bLoadedData)
{
}

FNavigationRelevantData::FNavigationRelevantData(FNavigationRelevantData&& Other)
	: TSharedFromThis<FNavigationRelevantData, ESPMode::ThreadSafe>(MoveTemp(Other))
	, CollisionData(MoveTemp(Other.CollisionData))
	, VoxelData(MoveTemp(Other.VoxelData))
	, Bounds(MoveTemp(Other.Bounds))
	, NavDataPerInstanceTransformDelegate(MoveTemp(Other.NavDataPerInstanceTransformDelegate))
	, ShouldUseGeometryDelegate(MoveTemp(Other.ShouldUseGeometryDelegate))
	, Modifiers(MoveTemp(Other.Modifiers))
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, SourceObject(MoveTemp(Other.SourceObject))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, SourceElement(MoveTemp(Other.SourceElement))
	, bPendingLazyGeometryGathering(Other.bPendingLazyGeometryGathering)
	, bPendingLazyModifiersGathering(Other.bPendingLazyModifiersGathering)
	, bPendingChildLazyModifiersGathering(Other.bPendingChildLazyModifiersGathering)
	, bSupportsGatheringGeometrySlices(Other.bSupportsGatheringGeometrySlices)
	, bShouldSkipDirtyAreaOnAddOrRemove(Other.bShouldSkipDirtyAreaOnAddOrRemove)
	, bLoadedData(Other.bLoadedData)
{
}

FNavigationRelevantData& FNavigationRelevantData::operator=(FNavigationRelevantData&& Other)
{
	TSharedFromThis<FNavigationRelevantData, ESPMode::ThreadSafe>::operator =(MoveTemp(Other));
	CollisionData = MoveTemp(Other.CollisionData);
	VoxelData = MoveTemp(Other.VoxelData);
	Bounds = MoveTemp(Other.Bounds);
	NavDataPerInstanceTransformDelegate = MoveTemp(Other.NavDataPerInstanceTransformDelegate);
	ShouldUseGeometryDelegate = MoveTemp(Other.ShouldUseGeometryDelegate);
	Modifiers = MoveTemp(Other.Modifiers);
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SourceObject = MoveTemp(Other.SourceObject);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	SourceElement = MoveTemp(Other.SourceElement);
	bPendingLazyGeometryGathering = Other.bPendingLazyGeometryGathering;
	bPendingLazyModifiersGathering = Other.bPendingLazyModifiersGathering;
	bPendingChildLazyModifiersGathering = Other.bPendingChildLazyModifiersGathering;
	bSupportsGatheringGeometrySlices = Other.bSupportsGatheringGeometrySlices;
	bShouldSkipDirtyAreaOnAddOrRemove = Other.bShouldSkipDirtyAreaOnAddOrRemove;
	bLoadedData = Other.bLoadedData;
	return *this;
}

FNavigationRelevantData& FNavigationRelevantData::operator=(const FNavigationRelevantData& Other)
{
	TSharedFromThis<FNavigationRelevantData, ESPMode::ThreadSafe>::operator =(Other);
	CollisionData = Other.CollisionData;
	VoxelData = Other.VoxelData;
	Bounds = Other.Bounds;
	NavDataPerInstanceTransformDelegate = Other.NavDataPerInstanceTransformDelegate;
	ShouldUseGeometryDelegate = Other.ShouldUseGeometryDelegate;
	Modifiers = Other.Modifiers;
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SourceObject = Other.SourceObject;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	SourceElement = Other.SourceElement;
	bPendingLazyGeometryGathering = Other.bPendingLazyGeometryGathering;
	bPendingLazyModifiersGathering = Other.bPendingLazyModifiersGathering;
	bPendingChildLazyModifiersGathering = Other.bPendingChildLazyModifiersGathering;
	bSupportsGatheringGeometrySlices = Other.bSupportsGatheringGeometrySlices;
	bShouldSkipDirtyAreaOnAddOrRemove = Other.bShouldSkipDirtyAreaOnAddOrRemove;
	bLoadedData = Other.bLoadedData;
	return *this;
}

bool FNavigationRelevantData::FCollisionDataHeader::IsValid(const uint8* RawData, int32 RawDataSize)
{
	constexpr int32 HeaderSize = sizeof(FCollisionDataHeader);
	return (RawDataSize == 0) || ((RawDataSize >= HeaderSize) && (((const FCollisionDataHeader*)RawData)->DataSize == RawDataSize));
}

FCompositeNavModifier FNavigationRelevantData::GetModifierForAgent(const FNavAgentProperties* NavAgent) const
{
	return Modifiers.HasMetaAreas() ? Modifiers.GetInstantiatedMetaModifier(NavAgent, SourceElement->GetWeakUObject()) : Modifiers;
}

bool FNavigationRelevantData::HasPerInstanceTransforms() const
{
	return NavDataPerInstanceTransformDelegate.IsBound();
}

bool FNavigationRelevantData::IsMatchingFilter(const FNavigationRelevantDataFilter& Filter) const
{
	if (Filter.bExcludeLoadedData && bLoadedData)
	{
		return false;
	}
	return (Filter.bIncludeGeometry && HasGeometry()) ||
		(Filter.bIncludeOffmeshLinks && (Modifiers.HasPotentialLinks() || Modifiers.HasLinks())) ||
		(Filter.bIncludeAreas && Modifiers.HasAreas()) ||
		(Filter.bIncludeMetaAreas && Modifiers.HasMetaAreas());
}

void FNavigationRelevantData::Shrink()
{
	CollisionData.Shrink();
	VoxelData.Shrink();
	Modifiers.Shrink();
}

bool FNavigationRelevantData::IsCollisionDataValid() const
{
	const bool bIsValid = FCollisionDataHeader::IsValid(CollisionData.GetData(), CollisionData.Num());
	if (!ensure(bIsValid))
	{
		UE_LOGF(LogNavigation, Error, "NavOctree element has corrupted collision data! Owner:%ls Bounds:%ls", *SourceElement->GetName(), *Bounds.ToString());
		return false;
	}

	return true;
}

//----------------------------------------------------------------------//
// FNavigationQueryFilter
//----------------------------------------------------------------------//
FNavigationQueryFilter::FNavigationQueryFilter(const FNavigationQueryFilter& Source)
{
	Assign(Source);
}

FNavigationQueryFilter::FNavigationQueryFilter(const FNavigationQueryFilter* Source)
	: MaxSearchNodes(DefaultMaxSearchNodes)
{
	if (Source != nullptr)
	{
		Assign(*Source);
	}
}

FNavigationQueryFilter::FNavigationQueryFilter(const FSharedNavQueryFilter Source)
	: MaxSearchNodes(DefaultMaxSearchNodes)
{
	if (Source.IsValid())
	{
		SetFilterImplementation(Source->GetImplementation());
	}
}

FNavigationQueryFilter& FNavigationQueryFilter::operator=(const FNavigationQueryFilter& Source)
{
	Assign(Source);
	return *this;
}

void FNavigationQueryFilter::Assign(const FNavigationQueryFilter& Source)
{
	if (Source.GetImplementation() != nullptr)
	{
		QueryFilterImpl = Source.QueryFilterImpl;
	}
	MaxSearchNodes = Source.GetMaxSearchNodes();
}

FSharedNavQueryFilter FNavigationQueryFilter::GetCopy() const
{
	FSharedNavQueryFilter Copy = MakeShareable(new FNavigationQueryFilter());
	Copy->QueryFilterImpl = MakeShareable(QueryFilterImpl->CreateCopy());
	Copy->MaxSearchNodes = MaxSearchNodes;

	return Copy;
}

void FNavigationQueryFilter::SetAreaCost(uint8 AreaType, float Cost)
{
	check(QueryFilterImpl.IsValid());
	QueryFilterImpl->SetAreaCost(AreaType, Cost);
}

void FNavigationQueryFilter::SetFixedAreaEnteringCost(uint8 AreaType, float Cost)
{
	check(QueryFilterImpl.IsValid());
	QueryFilterImpl->SetFixedAreaEnteringCost(AreaType, Cost);
}

void FNavigationQueryFilter::SetExcludedArea(uint8 AreaType)
{
	QueryFilterImpl->SetExcludedArea(AreaType);
}

void FNavigationQueryFilter::SetAllAreaCosts(const TArray<float>& CostArray)
{
	SetAllAreaCosts(CostArray.GetData(), CostArray.Num());
}

void FNavigationQueryFilter::SetAllAreaCosts(const float* CostArray, const int32 Count)
{
	QueryFilterImpl->SetAllAreaCosts(CostArray, Count);
}

void FNavigationQueryFilter::GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const
{
	QueryFilterImpl->GetAllAreaCosts(CostArray, FixedCostArray, Count);
}

void FNavigationQueryFilter::SetIncludeFlags(uint16 Flags)
{
	QueryFilterImpl->SetIncludeFlags(Flags);
}

uint16 FNavigationQueryFilter::GetIncludeFlags() const
{
	return QueryFilterImpl->GetIncludeFlags();
}

void FNavigationQueryFilter::SetExcludeFlags(uint16 Flags)
{
	QueryFilterImpl->SetExcludeFlags(Flags);
}

uint16 FNavigationQueryFilter::GetExcludeFlags() const
{
	return QueryFilterImpl->GetExcludeFlags();
}

//----------------------------------------------------------------------//
// FNavAgentSelector
//----------------------------------------------------------------------//
FNavAgentSelector::FNavAgentSelector(const uint32 InBits)
	: PackedBits(InBits)
{
}

bool FNavAgentSelector::Serialize(FArchive& Ar)
{
	Ar << PackedBits;
	return true;
}

//----------------------------------------------------------------------//
// FNavHeightfieldSample
//----------------------------------------------------------------------//
FNavHeightfieldSamples::FNavHeightfieldSamples()
{
}

void FNavHeightfieldSamples::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this) + Heights.GetAllocatedSize() + Holes.GetAllocatedSize());
}

void FNavHeightfieldSamples::Empty()
{
	Heights.Empty();
	Holes.Empty();
}


namespace UE::Navigation::NavLinkIdHelpers::Private
{
	uint64 MakeIdFromGUID(const FGuid Guid)
	{
		// Create array to guarantee contiguous memory layout 
		const uint32 GuidArray[] = { Guid.A, Guid.B, Guid.C, Guid.D };
		return CityHash64(reinterpret_cast<const char*>(GuidArray), sizeof(GuidArray));
	}

	uint64 MakeIdFromGUID(FNavLinkAuxiliaryId AuxiliaryId, const FGuid Guid)
	{
		const uint64 ActorGuidHash = MakeIdFromGUID(Guid);
		return CityHash128to64({ AuxiliaryId.GetId(), ActorGuidHash});
	}
} // UE::Navigation::NavLinkHelpers

const FNavLinkId FNavLinkId::Invalid = FNavLinkId();
const FNavLinkAuxiliaryId FNavLinkAuxiliaryId::Invalid = FNavLinkAuxiliaryId();

FNavLinkAuxiliaryId FNavLinkAuxiliaryId::GenerateUniqueAuxiliaryId()
{
	const uint64 AuxiliaryId = UE::Navigation::NavLinkIdHelpers::Private::MakeIdFromGUID(FGuid::NewGuid());

	return FNavLinkAuxiliaryId(AuxiliaryId);
}

FNavLinkAuxiliaryId FNavLinkAuxiliaryId::GenerateUniqueAuxiliaryId(FStringView PathName)
{
	check(!PathName.IsEmpty());

	const uint64 AuxiliaryId = UE::Navigation::NavLinkIdHelpers::Private::MakeIdFromGUID(FGuid::NewDeterministicGuid(PathName));

	return FNavLinkAuxiliaryId(AuxiliaryId);
}

FNavLinkId FNavLinkId::GenerateUniqueId()
{
	// Apply NavLinkIdBitMask to differentiate Legacy Ids (that do not have the mask set).
	const uint64 UniqueId = UE::Navigation::NavLinkIdHelpers::Private::MakeIdFromGUID(FGuid::NewGuid()) | NavLinkIdBitMask;
	UE_LOG(LogNavLink, VeryVerbose, TEXT("%hs id: %" UINT64_FMT "."), __FUNCTION__, UniqueId);
	return FNavLinkId(UniqueId);
}

FNavLinkId FNavLinkId::GenerateUniqueId(FNavLinkAuxiliaryId AuxiliaryId, FGuid ActorInstanceGuid)
{
	// Apply NavLinkIdBitMask to differentiate Legacy Ids (that do not have the mask set).
	const uint64 UniqueId = UE::Navigation::NavLinkIdHelpers::Private::MakeIdFromGUID(AuxiliaryId, ActorInstanceGuid) | NavLinkIdBitMask;

	UE_LOG(LogNavLink, VeryVerbose, TEXT("%hs id: %" UINT64_FMT "."), __FUNCTION__, UniqueId);

	return FNavLinkId(UniqueId);
}

//----------------------------------------------------------------------//
// FNavAgentProperties
//----------------------------------------------------------------------//
const FNavAgentProperties FNavAgentProperties::DefaultProperties;

FNavAgentProperties::FNavAgentProperties(const FNavAgentProperties& Other) = default;
FNavAgentProperties& FNavAgentProperties::operator=(const FNavAgentProperties& Other) = default;

void FNavAgentProperties::UpdateWithCollisionComponent(UShapeComponent* CollisionComponent)
{
	check(CollisionComponent != NULL);
	AgentRadius = FloatCastChecked<float>(CollisionComponent->Bounds.SphereRadius, UE::LWC::DefaultFloatPrecision);
}

bool FNavAgentProperties::IsNavDataMatching(const FNavAgentProperties& Other) const
{
	return (PreferredNavData == Other.PreferredNavData || PreferredNavData.IsNull() || Other.PreferredNavData.IsNull());
}

void FNavAgentProperties::SetPreferredNavData(TSubclassOf<AActor> NavDataClass)
{
	PreferredNavData = FSoftClassPath(NavDataClass.Get());
}

//----------------------------------------------------------------------//
// UNavAreaBase
//----------------------------------------------------------------------//
UNavAreaBase::UNavAreaBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsMetaArea = false;
}

TSubclassOf<UNavAreaBase> UNavAreaBase::PickAreaClassForAgent(const AActor& Actor, const FNavAgentProperties& NavAgent) const
{
	UE_CLOGF(IsMetaArea(), LogNavigation, Warning, "UNavAreaBase::PickAreaClassForAgent called for meta class %ls. Please override PickAreaClass."
		, *(GetClass()->GetName()));

	return GetClass();
}

//----------------------------------------------------------------------//
// Deprecated methods
//----------------------------------------------------------------------//

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// Deprecated
FNavigationRelevantData::FNavigationRelevantData(UObject& Source)
	: SourceElement(MakeShared<const FNavigationElement>(Source))
	, bPendingLazyGeometryGathering(false)
	, bPendingLazyModifiersGathering(false)
	, bPendingChildLazyModifiersGathering(false)
	, bSupportsGatheringGeometrySlices(false)
	, bShouldSkipDirtyAreaOnAddOrRemove(false)
	, bLoadedData(false)
{
}

// Deprecated
const UObject* FNavigationRelevantData::GetOwner() const
{
	return SourceElement->GetWeakUObject().Get();
}

// Deprecated
TWeakObjectPtr<UObject> FNavigationRelevantData::GetOwnerPtr() const
{
	return TWeakObjectPtr(const_cast<UObject*>(GetOwner()));
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
