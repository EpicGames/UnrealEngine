// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/NavigationSystemBase.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GameFramework/WorldSettings.h"
#include "AI/Navigation/NavAreaBase.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "AI/NavigationSystemConfig.h"
#include "AI/Navigation/NavigationDataChunk.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/Navigation/NavigationElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationSystemBase)

DEFINE_LOG_CATEGORY(LogNavigation);
DEFINE_LOG_CATEGORY(LogNavigationDataBuild);
DEFINE_LOG_CATEGORY(LogNavigationHistory);
DEFINE_LOG_CATEGORY(LogNavInvokers);
DEFINE_LOG_CATEGORY(LogNavLink);
DEFINE_LOG_CATEGORY(LogAStar);

namespace FNavigationSystem
{
	void DiscardNavigationDataChunks(UWorld& InWorld)
	{
		const auto& Levels = InWorld.GetLevels();
		for (ULevel* Level : Levels)
		{
			UE_LOGF(LogNavigation, Verbose, "%s for %ls", __FUNCTION__, *GetFullNameSafe(Level));
			
			for (UNavigationDataChunk* NavChunk : Level->NavDataChunks)
			{
				if (NavChunk != nullptr)
				{
					NavChunk->MarkAsGarbage();
				}
			}
			Level->NavDataChunks.Empty();
		}
	}

	FNavigationSystemRunMode FindRunModeFromWorldType(const UWorld& World)
	{
		switch (World.WorldType)
		{
		case EWorldType::Editor:
		case EWorldType::EditorPreview:
			return FNavigationSystemRunMode::EditorMode;

		case EWorldType::PIE:
			return FNavigationSystemRunMode::PIEMode;

		case EWorldType::Game:
		case EWorldType::GamePreview:
		case EWorldType::GameRPC:
			return FNavigationSystemRunMode::GameMode;

		case EWorldType::Inactive:
		case EWorldType::None:
			return FNavigationSystemRunMode::InvalidMode;

		default:
			UE_LOGF(LogNavigation, Warning, "%s Unhandled world type, defaulting to FNavigationSystemRunMode::InvalidMode.", __FUNCTION__);
			return FNavigationSystemRunMode::InvalidMode;
		}
	}

	bool IsEditorRunMode(const FNavigationSystemRunMode Mode)
	{
		return Mode == FNavigationSystemRunMode::EditorMode || Mode == FNavigationSystemRunMode::EditorWorldPartitionBuildMode;
	}

	void AddNavigationSystemToWorld(UWorld& WorldOwner, const FNavigationSystemRunMode RunMode, UNavigationSystemConfig* NavigationSystemConfig, const bool bInitializeForWorld, const bool bOverridePreviousNavSys)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNavigationSystem::AddNavigationSystemToWorld);

		UE_LOGF(LogNavigation, Verbose, "%s bOverridePreviousNavSys=%i (NavigationSystemConfig: %ls) (WorldOwner: %ls) (DefaultAgentName: %ls)",
			__FUNCTION__,
			bOverridePreviousNavSys,
			*GetNameSafe(NavigationSystemConfig),
			*WorldOwner.GetOuter()->GetName(),
			NavigationSystemConfig ? *NavigationSystemConfig->DefaultAgentName.ToString() : TEXT("none"));

		const FNavigationSystemRunMode ResolvedRunMode = (RunMode == FNavigationSystemRunMode::InferFromWorldMode) ? FindRunModeFromWorldType(WorldOwner) : RunMode;

		if (WorldOwner.GetNavigationSystem() == nullptr || bOverridePreviousNavSys)
		{
			if (NavigationSystemConfig == nullptr)
			{
				AWorldSettings* WorldSettings = WorldOwner.GetWorldSettings();
				if (WorldSettings)
				{
					NavigationSystemConfig = WorldSettings->GetNavigationSystemConfig();
				}
			}

			UNavigationSystemBase* NavSysInstance = NavigationSystemConfig 
				? NavigationSystemConfig->CreateAndConfigureNavigationSystem(WorldOwner)
				: nullptr;
			// we're setting to an instance or null, both are correct
			WorldOwner.SetNavigationSystem(NavSysInstance);			
		}

		if (bInitializeForWorld)
		{
			if (WorldOwner.GetNavigationSystem())
			{
				WorldOwner.GetNavigationSystem()->InitializeForWorld(WorldOwner, ResolvedRunMode);
			}
			else if (IsEditorRunMode(ResolvedRunMode))
			{
				DiscardNavigationDataChunks(WorldOwner);
			}
		}
	}
	
	const FNavDataConfig& GetFallbackSupportedAgent() 
	{ 
		static FNavDataConfig FallbackSupportedAgent;
		return FallbackSupportedAgent; 
	}

	const FNavDataConfig& GetFallbackSupportedAgent(const UWorld* World) 
	{ 
		static FNavDataConfig FallbackSupportedAgent;
		return FallbackSupportedAgent; 
	}
	
	bool bWantsComponentChangeNotifies = true;
	
	class FDelegates
	{
	public:
		FSupportsDynamicChangesSignature SupportsDynamicChangesDelegate;
		FAddNavigationElementSignature AddNavigationElementDelegate;
		FUpdateNavigationElementSignature UpdateNavigationElementDelegate;
		FRemoveNavigationElementSignature RemoveNavigationElementDelegate;
		FUpdateNavigationElementBoundsSignature UpdateNavigationElementBoundsDelegate;

		FObjectBasedSignature RegisterNavRelevantObject;
		FObjectBasedSignature UpdateNavRelevantObject;
		FObjectBasedSignature UnregisterNavRelevantObject;
		FObjectBoundsChangedSignature OnObjectBoundsChanged;

		FActorBasedSignature UpdateActorData;
		FActorComponentBasedSignature UpdateComponentData;
		FSceneComponentBasedSignature UpdateComponentDataAfterMove;
		FActorBasedSignature OnActorBoundsChanged;
		FActorBasedSignature OnPostEditActorMove;
		FSceneComponentBasedSignature OnComponentTransformChanged;
		FActorBasedSignature OnActorRegistered;
		FActorBasedSignature OnActorUnregistered;
		FActorComponentBasedSignature OnComponentRegistered;
		FActorComponentBasedSignature OnComponentUnregistered;
		FActorComponentBasedSignature RegisterComponent;
		FActorComponentBasedSignature UnregisterComponent;
		FActorBasedSignature RemoveActorData;
		FControllerBasedSignature StopMovement;
		FBoolControllerBasedSignature IsFollowingAPath;
		FBoolActorComponentBasedSignature HasComponentData;
		FNavDataConfigBasedSignature GetDefaultSupportedAgent;
		FNavDataConfigAndWorldSignature GetBiggestSupportedAgent;
		FActorBooleBasedSignature UpdateActorAndComponentData;
		FNavDataForActorSignature GetNavDataForActor;
		FNavDataClassFetchSignature GetDefaultNavDataClass;
		FWorldBoolBasedSignature VerifyNavigationRenderingComponents;
		FWorldBasedSignature Build;
		FOnNavigationInitSignature OnNavigationInitStart;
		FOnNavigationInitSignature OnNavigationInitDone;
		FOnNavAreaGenericEvent OnNavAreaRegistered;
		FOnNavAreaGenericEvent OnNavAreaUnregistered;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FComponentBoundsChangeSignature OnComponentBoundsChanged;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
#if WITH_EDITOR
		FWorldBasedSignature OnPIEStart;
		FWorldBasedSignature OnPIEEnd;
		FLevelBasedSignature UpdateLevelCollision;
		FNavigationAutoUpdateEnableSignature SetNavigationAutoUpdateEnable;
		FWorldByteBasedSignature AddNavigationUpdateLock;
		FWorldByteBasedSignature RemoveNavigationUpdateLock;
		FDoubleWorldBasedSignature GetWorldPartitionNavigationDataBuilderOverlap;
#endif // WITH_EDITOR

		FDelegates()
		{
			AddNavigationElementDelegate.BindLambda([](UWorld* World, FNavigationElement&&) { return FNavigationElementHandle::Invalid; });
			UpdateNavigationElementDelegate.BindLambda([](UWorld* World, FNavigationElementHandle, FNavigationElement&&) {});
			RemoveNavigationElementDelegate.BindLambda([](UWorld* World, FNavigationElementHandle) {});
			UpdateNavigationElementBoundsDelegate.BindLambda([](UWorld* World, FNavigationElementHandle, const FBox&, TConstArrayView<FBox>) {});

			RegisterNavRelevantObject.BindLambda([](UObject&) {});
			UpdateNavRelevantObject.BindLambda([](UObject&) {});
			UnregisterNavRelevantObject.BindLambda([](UObject&) {});
			OnObjectBoundsChanged.BindLambda([](UObject&, const FBox&, TConstArrayView<FBox>) {});

			UpdateActorData.BindLambda([](AActor&) {});
			UpdateComponentData.BindLambda([](UActorComponent&) {});
			UpdateComponentDataAfterMove.BindLambda([](UActorComponent&) {});
			OnActorBoundsChanged.BindLambda([](AActor&) {});
			OnPostEditActorMove.BindLambda([](AActor&) {});
			OnComponentTransformChanged.BindLambda([](USceneComponent&) {});
			OnActorRegistered.BindLambda([](AActor&) {});
			OnActorUnregistered.BindLambda([](AActor&) {});
			OnComponentRegistered.BindLambda([](UActorComponent&) {});
			OnComponentUnregistered.BindLambda([](UActorComponent&) {});
			RegisterComponent.BindLambda([](UActorComponent&) {});
			UnregisterComponent.BindLambda([](UActorComponent&) {});
			RemoveActorData.BindLambda([](AActor&) {});
			StopMovement.BindLambda([](const AController&) {});
			IsFollowingAPath.BindLambda([](const AController&) { return false; });
			HasComponentData.BindLambda([](UActorComponent&) { return false; });
			GetDefaultSupportedAgent.BindStatic(&GetFallbackSupportedAgent);
			GetBiggestSupportedAgent.BindStatic(&GetFallbackSupportedAgent);
			UpdateActorAndComponentData.BindLambda([](AActor&, bool) {});
			OnComponentBoundsChanged.BindLambda([](UActorComponent&, const FBox&, const FBox&) {});
			GetNavDataForActor.BindLambda([](const AActor&) { return nullptr; });
			GetDefaultNavDataClass.BindLambda([]() { return AActor::StaticClass(); });
			VerifyNavigationRenderingComponents.BindLambda([](UWorld&, bool) {});
			Build.BindLambda([](UWorld&) {});
#if WITH_EDITOR
			OnPIEStart.BindLambda([](UWorld&) {});
			OnPIEEnd.BindLambda([](UWorld&) {});
			UpdateLevelCollision.BindLambda([](ULevel&) {});
			SetNavigationAutoUpdateEnable.BindLambda([](const bool, UNavigationSystemBase*) {});
			AddNavigationUpdateLock.BindLambda([](UWorld&, uint8) {});
			RemoveNavigationUpdateLock.BindLambda([](UWorld&, uint8) {});
			GetWorldPartitionNavigationDataBuilderOverlap.BindLambda([](const UWorld&){ return 0; });
#endif // WITH_EDITOR
		}
	};

	FDelegates Delegates;

	void ResetDelegates()
	{
		new(&Delegates)FDelegates();
	}

	bool SupportsDynamicChanges(UWorld* World)
	{
		return Delegates.SupportsDynamicChangesDelegate.Execute(World);
	}

	FNavigationElementHandle AddNavigationElement(UWorld* World, FNavigationElement&& Element)
	{
		return Delegates.AddNavigationElementDelegate.Execute(World, MoveTemp(Element));
	}

	void UpdateNavigationElement(UWorld* World, const FNavigationElementHandle ElementHandle, FNavigationElement&& Element)
	{
		Delegates.UpdateNavigationElementDelegate.Execute(World, ElementHandle, MoveTemp(Element));
	}

	void RemoveNavigationElement(UWorld* World, const FNavigationElementHandle ElementHandle)
	{
		Delegates.RemoveNavigationElementDelegate.Execute(World, ElementHandle);
	}

	void OnNavigationElementBoundsChanged(UWorld* World, const FNavigationElementHandle ElementHandle, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas)
	{
		Delegates.UpdateNavigationElementBoundsDelegate.Execute(World, ElementHandle, NewBounds, DirtyAreas);
	}

	void RegisterNavRelevantObject(UObject& Object) { Delegates.RegisterNavRelevantObject.Execute(Object); }
	void UpdateNavRelevantObject(UObject& Object) { Delegates.UpdateNavRelevantObject.Execute(Object); }
	void UnregisterNavRelevantObject(UObject& Object) { Delegates.UnregisterNavRelevantObject.Execute(Object); }
	void OnObjectBoundsChanged(UObject& Object, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas) { Delegates.OnObjectBoundsChanged.Execute(Object, NewBounds, DirtyAreas); }

	void UpdateActorData(AActor& Actor) { Delegates.UpdateActorData.Execute(Actor); }
	void UpdateComponentData(UActorComponent& Comp) { Delegates.UpdateComponentData.Execute(Comp); }
	void UpdateActorAndComponentData(AActor& Actor, bool bUpdateAttachedActors) { Delegates.UpdateActorAndComponentData.Execute(Actor, bUpdateAttachedActors); }
	void UpdateComponentDataAfterMove(USceneComponent& Comp) { Delegates.UpdateComponentDataAfterMove.Execute(Comp); }
	void OnActorBoundsChanged(AActor& Actor) { Delegates.OnActorBoundsChanged.Execute(Actor); }
	void OnPostEditActorMove(AActor& Actor) { Delegates.OnPostEditActorMove.Execute(Actor); }
	void OnComponentBoundsChanged(UActorComponent& Comp, const FBox& NewBounds, const FBox& DirtyArea) { OnObjectBoundsChanged(Comp, NewBounds, { DirtyArea }); }
	void OnComponentTransformChanged(USceneComponent& Comp) { Delegates.OnComponentTransformChanged.Execute(Comp); }
	void OnActorRegistered(AActor& Actor) { Delegates.OnActorRegistered.Execute(Actor); }
	void OnActorUnregistered(AActor& Actor) { Delegates.OnActorUnregistered.Execute(Actor); }
	void OnComponentRegistered(UActorComponent& Comp) { Delegates.OnComponentRegistered.Execute(Comp); }
	void OnComponentUnregistered(UActorComponent& Comp) { Delegates.OnComponentUnregistered.Execute(Comp); }
	void RegisterComponent(UActorComponent& Comp) { Delegates.RegisterComponent.Execute(Comp); }
	void UnregisterComponent(UActorComponent& Comp) { Delegates.UnregisterComponent.Execute(Comp); }
	void RemoveActorData(AActor& Actor) { Delegates.RemoveActorData.Execute(Actor); }
	bool HasComponentData(UActorComponent& Comp) { return Delegates.HasComponentData.Execute(Comp);	}
	const FNavDataConfig& GetDefaultSupportedAgent() { return Delegates.GetDefaultSupportedAgent.Execute(); }
	const FNavDataConfig& GetBiggestSupportedAgent(const UWorld* World) { return Delegates.GetBiggestSupportedAgent.Execute(World); }
#if WITH_EDITOR	
	double GetWorldPartitionNavigationDataBuilderOverlap(const UWorld& World) { return Delegates.GetWorldPartitionNavigationDataBuilderOverlap.Execute(World); }
#endif	

	TSubclassOf<UNavAreaBase> DefaultWalkableArea; 
	TSubclassOf<UNavAreaBase> DefaultObstacleArea;
	TSubclassOf<UNavAreaBase> GetDefaultWalkableArea() { return DefaultWalkableArea; }
	TSubclassOf<UNavAreaBase> GetDefaultObstacleArea() { return DefaultObstacleArea; }
		
	bool WantsComponentChangeNotifies()
	{
		return bWantsComponentChangeNotifies;
	}

	//INavigationDataInterface* GetNavDataForProps(const FNavAgentProperties& AgentProperties) { return Delegates.GetNavDataForProps.Execute(AgentProperties); }
	INavigationDataInterface* GetNavDataForActor(const AActor& Actor) { return Delegates.GetNavDataForActor.Execute(Actor); }
	TSubclassOf<AActor> GetDefaultNavDataClass() { return Delegates.GetDefaultNavDataClass.Execute(); }

	void VerifyNavigationRenderingComponents(UWorld& World, const bool bShow) { Delegates.VerifyNavigationRenderingComponents.Execute(World, bShow); }
	void Build(UWorld& World) { Delegates.Build.Execute(World); }

	// pathfollowing
	bool IsFollowingAPath(const AController& Controller) { return Delegates.IsFollowingAPath.Execute(Controller); }
	void StopMovement(const AController& Controller) { Delegates.StopMovement.Execute(Controller); }
	IPathFollowingAgentInterface* FindPathFollowingAgentForActor(const AActor& Actor)
	{
		const TSet<UActorComponent*>& Components = Actor.GetComponents();
		for (UActorComponent* Component : Components)
		{
			IPathFollowingAgentInterface* AsPFAgent = Cast<IPathFollowingAgentInterface>(Component);
			if (AsPFAgent)
			{
				return AsPFAgent;
			}
		}
		return nullptr;
	}

#if WITH_EDITOR
	void OnPIEStart(UWorld& World) { Delegates.OnPIEStart.Execute(World); }
	void OnPIEEnd(UWorld& World) { Delegates.OnPIEEnd.Execute(World); }
	void SetNavigationAutoUpdateEnabled(const bool bNewEnable, UNavigationSystemBase* InNavigationSystem) { Delegates.SetNavigationAutoUpdateEnable.Execute(bNewEnable, InNavigationSystem); }
	void UpdateLevelCollision(ULevel& Level) { Delegates.UpdateLevelCollision.Execute(Level); }
#endif // WITH_EDITOR

	struct FCoordTransforms
	{
		FTransform& Get(const ENavigationCoordSystem::Type FromCoordType, const ENavigationCoordSystem::Type ToCoordType)
		{
			static FTransform CoordTypeTransforms[ENavigationCoordSystem::MAX][ENavigationCoordSystem::MAX] = {
				{FTransform::Identity, FTransform::Identity}
				, {FTransform::Identity, FTransform::Identity}
			};

			return CoordTypeTransforms[uint8(FromCoordType)][uint8(ToCoordType)];
		}
	};

	FCoordTransforms& GetCoordTypeTransforms()
	{
		static FCoordTransforms CoordTypeTransforms;
		return CoordTypeTransforms;
	}

	const FTransform& GetCoordTransform(const ENavigationCoordSystem::Type FromCoordType, const ENavigationCoordSystem::Type ToCoordType)
	{
		return GetCoordTypeTransforms().Get(FromCoordType, ToCoordType);
	}

	UWorld* GetWorldFromContextObject(UObject* WorldContextObject)
	{
		return (WorldContextObject != nullptr)
			? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull)
			: nullptr;
	}
}

//----------------------------------------------------------------------//
// FNavigationLockContext                                                                
//----------------------------------------------------------------------//
void FNavigationLockContext::LockUpdates()
{
#if WITH_EDITOR
	bIsLocked = true;

	if (bSingleWorld)
	{
		if (MyWorld)
		{
			FNavigationSystem::Delegates.AddNavigationUpdateLock.Execute(*MyWorld, LockReason);
		}
	}
	else
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.World())
			{
				FNavigationSystem::Delegates.AddNavigationUpdateLock.Execute(*Context.World(), LockReason);
			}
		}
	}
#endif
}

void FNavigationLockContext::UnlockUpdates()
{
#if WITH_EDITOR
	if (!bIsLocked)
	{
		return;
	}

	if (bSingleWorld)
	{
		if (MyWorld)
		{
			FNavigationSystem::Delegates.RemoveNavigationUpdateLock.Execute(*MyWorld, LockReason);
		}
	}
	else
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.World())
			{
				FNavigationSystem::Delegates.RemoveNavigationUpdateLock.Execute(*Context.World(), LockReason);
			}
		}
	}
#endif
}

//----------------------------------------------------------------------//
// UNavigationSystemBase
//----------------------------------------------------------------------//
void UNavigationSystemBase::SetCoordTransform(const ENavigationCoordSystem::Type FromCoordType, const ENavigationCoordSystem::Type ToCoordType, const FTransform& Transform, bool bAddInverse)
{
	FNavigationSystem::GetCoordTypeTransforms().Get(FromCoordType, ToCoordType) = Transform;
	if (bAddInverse)
	{
		FNavigationSystem::GetCoordTypeTransforms().Get(ToCoordType, FromCoordType) = Transform.Inverse();
	}
}

void UNavigationSystemBase::SetWantsComponentChangeNotifies(const bool bEnable)
{
	FNavigationSystem::bWantsComponentChangeNotifies = bEnable;
}

void UNavigationSystemBase::SetDefaultWalkableArea(TSubclassOf<UNavAreaBase> InAreaClass)
{
	FNavigationSystem::DefaultWalkableArea = InAreaClass;
}

void UNavigationSystemBase::SetDefaultObstacleArea(TSubclassOf<UNavAreaBase> InAreaClass)
{
	FNavigationSystem::DefaultObstacleArea = InAreaClass;
}


void UNavigationSystemBase::ResetEventDelegates() { FNavigationSystem::ResetDelegates(); }

FNavigationSystem::FSupportsDynamicChangesSignature& UNavigationSystemBase::GetSupportsDynamicChangesDelegate()
{
	return FNavigationSystem::Delegates.SupportsDynamicChangesDelegate;
}

FNavigationSystem::FAddNavigationElementSignature& UNavigationSystemBase::GetAddNavigationElementDelegate()
{
	return FNavigationSystem::Delegates.AddNavigationElementDelegate;
}

FNavigationSystem::FUpdateNavigationElementSignature& UNavigationSystemBase::GetUpdateNavigationElementDelegate()
{
	return FNavigationSystem::Delegates.UpdateNavigationElementDelegate;
}

FNavigationSystem::FRemoveNavigationElementSignature& UNavigationSystemBase::GetRemoveNavigationElementDelegate()
{
	return FNavigationSystem::Delegates.RemoveNavigationElementDelegate;
}

FNavigationSystem::FUpdateNavigationElementBoundsSignature& UNavigationSystemBase::GetUpdateNavigationElementBoundsDelegate()
{
	return FNavigationSystem::Delegates.UpdateNavigationElementBoundsDelegate;
}

FNavigationSystem::FObjectBasedSignature& UNavigationSystemBase::RegisterNavRelevantObjectDelegate() { return FNavigationSystem::Delegates.RegisterNavRelevantObject; }
FNavigationSystem::FObjectBasedSignature& UNavigationSystemBase::UpdateNavRelevantObjectDelegate() { return FNavigationSystem::Delegates.UpdateNavRelevantObject; }
FNavigationSystem::FObjectBasedSignature& UNavigationSystemBase::UnregisterNavRelevantObjectDelegate() { return FNavigationSystem::Delegates.UnregisterNavRelevantObject; }
FNavigationSystem::FObjectBoundsChangedSignature& UNavigationSystemBase::OnObjectBoundsChangedDelegate() { return FNavigationSystem::Delegates.OnObjectBoundsChanged; }

FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::UpdateActorDataDelegate() { return FNavigationSystem::Delegates.UpdateActorData; }
FNavigationSystem::FActorComponentBasedSignature& UNavigationSystemBase::UpdateComponentDataDelegate() { return FNavigationSystem::Delegates.UpdateComponentData; }
FNavigationSystem::FSceneComponentBasedSignature& UNavigationSystemBase::UpdateComponentDataAfterMoveDelegate() { return FNavigationSystem::Delegates.UpdateComponentDataAfterMove; }
FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::OnActorBoundsChangedDelegate() { return FNavigationSystem::Delegates.OnActorBoundsChanged; }
FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::OnPostEditActorMoveDelegate() { return FNavigationSystem::Delegates.OnPostEditActorMove; }
FNavigationSystem::FSceneComponentBasedSignature& UNavigationSystemBase::OnComponentTransformChangedDelegate() { return FNavigationSystem::Delegates.OnComponentTransformChanged; }
FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::OnActorRegisteredDelegate() { return FNavigationSystem::Delegates.OnActorRegistered; }
FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::OnActorUnregisteredDelegate() { return FNavigationSystem::Delegates.OnActorUnregistered; }
FNavigationSystem::FActorComponentBasedSignature& UNavigationSystemBase::OnComponentRegisteredDelegate() { return FNavigationSystem::Delegates.OnComponentRegistered; }
FNavigationSystem::FActorComponentBasedSignature& UNavigationSystemBase::OnComponentUnregisteredDelegate() { return FNavigationSystem::Delegates.OnComponentUnregistered; }
FNavigationSystem::FActorComponentBasedSignature& UNavigationSystemBase::RegisterComponentDelegate() { return FNavigationSystem::Delegates.RegisterComponent; }
FNavigationSystem::FActorComponentBasedSignature& UNavigationSystemBase::UnregisterComponentDelegate() { return FNavigationSystem::Delegates.UnregisterComponent; }
FNavigationSystem::FActorBasedSignature& UNavigationSystemBase::RemoveActorDataDelegate() { return FNavigationSystem::Delegates.RemoveActorData; }
FNavigationSystem::FBoolActorComponentBasedSignature& UNavigationSystemBase::HasComponentDataDelegate() { return FNavigationSystem::Delegates.HasComponentData; }
FNavigationSystem::FNavDataConfigBasedSignature& UNavigationSystemBase::GetDefaultSupportedAgentDelegate() { return FNavigationSystem::Delegates.GetDefaultSupportedAgent; }
FNavigationSystem::FNavDataConfigAndWorldSignature& UNavigationSystemBase::GetBiggestSupportedAgentDelegate() { return FNavigationSystem::Delegates.GetBiggestSupportedAgent; }
FNavigationSystem::FActorBooleBasedSignature& UNavigationSystemBase::UpdateActorAndComponentDataDelegate() { return FNavigationSystem::Delegates.UpdateActorAndComponentData; }
FNavigationSystem::FNavDataForActorSignature& UNavigationSystemBase::GetNavDataForActorDelegate() { return FNavigationSystem::Delegates.GetNavDataForActor; }
FNavigationSystem::FNavDataClassFetchSignature& UNavigationSystemBase::GetDefaultNavDataClassDelegate() { return FNavigationSystem::Delegates.GetDefaultNavDataClass; }
FNavigationSystem::FWorldBoolBasedSignature& UNavigationSystemBase::VerifyNavigationRenderingComponentsDelegate() { return FNavigationSystem::Delegates.VerifyNavigationRenderingComponents; }
FNavigationSystem::FWorldBasedSignature& UNavigationSystemBase::BuildDelegate() { return FNavigationSystem::Delegates.Build; }
FNavigationSystem::FOnNavigationInitSignature& UNavigationSystemBase::OnNavigationInitStartStaticDelegate() { return FNavigationSystem::Delegates.OnNavigationInitStart; }
FNavigationSystem::FOnNavigationInitSignature& UNavigationSystemBase::OnNavigationInitDoneStaticDelegate() { return FNavigationSystem::Delegates.OnNavigationInitDone; }
FNavigationSystem::FOnNavAreaGenericEvent& UNavigationSystemBase::OnNavAreaRegisteredDelegate() { return FNavigationSystem::Delegates.OnNavAreaRegistered; }
FNavigationSystem::FOnNavAreaGenericEvent& UNavigationSystemBase::OnNavAreaUnregisteredDelegate() { return FNavigationSystem::Delegates.OnNavAreaUnregistered; }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FNavigationSystem::FComponentBoundsChangeSignature& UNavigationSystemBase::OnComponentBoundsChangedDelegate() { return FNavigationSystem::Delegates.OnComponentBoundsChanged; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
FNavigationSystem::FWorldBasedSignature& UNavigationSystemBase::OnPIEStartDelegate() { return FNavigationSystem::Delegates.OnPIEStart; }
FNavigationSystem::FWorldBasedSignature& UNavigationSystemBase::OnPIEEndDelegate() { return FNavigationSystem::Delegates.OnPIEEnd; }
FNavigationSystem::FLevelBasedSignature& UNavigationSystemBase::UpdateLevelCollisionDelegate() { return FNavigationSystem::Delegates.UpdateLevelCollision; }
FNavigationSystem::FNavigationAutoUpdateEnableSignature& UNavigationSystemBase::SetNavigationAutoUpdateEnableDelegate() { return FNavigationSystem::Delegates.SetNavigationAutoUpdateEnable; }
FNavigationSystem::FWorldByteBasedSignature& UNavigationSystemBase::AddNavigationUpdateLockDelegate() { return FNavigationSystem::Delegates.AddNavigationUpdateLock; }
FNavigationSystem::FWorldByteBasedSignature& UNavigationSystemBase::RemoveNavigationUpdateLockDelegate() { return FNavigationSystem::Delegates.RemoveNavigationUpdateLock; }
FNavigationSystem::FDoubleWorldBasedSignature& UNavigationSystemBase::GetWorldPartitionNavigationDataBuilderOverlapDelegate() { return FNavigationSystem::Delegates.GetWorldPartitionNavigationDataBuilderOverlap; }
#endif // WITH_EDITOR
//----------------------------------------------------------------------//
// IPathFollowingManagerInterface
//----------------------------------------------------------------------//
FNavigationSystem::FControllerBasedSignature& IPathFollowingManagerInterface::StopMovementDelegate() { return FNavigationSystem::Delegates.StopMovement; }
FNavigationSystem::FBoolControllerBasedSignature& IPathFollowingManagerInterface::IsFollowingAPathDelegate() { return FNavigationSystem::Delegates.IsFollowingAPath; }

