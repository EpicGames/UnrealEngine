// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/NavigationSystemConfig.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/Engine.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationSystemConfig)


UNavigationSystemConfig::UNavigationSystemConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (GEngine)
	{
		NavigationSystemClass = GEngine->NavigationSystemClassName;
	}
}

TSubclassOf<UNavigationSystemConfig> UNavigationSystemConfig::GetDefaultConfigClass()
{
	return GEngine ? GEngine->NavigationSystemConfigClass : nullptr;
}

UNavigationSystemBase* UNavigationSystemConfig::CreateAndConfigureNavigationSystem(UWorld& World) const
{
	UNavigationSystemBase* NavSys = nullptr;

	const TSubclassOf<UNavigationSystemBase> NavSysClass = NavigationSystemClass.ResolveClass();
	ensure(NavSysClass != nullptr || NavigationSystemClass.IsValid() == false);
	
	if (NavSysClass
		&& NavSysClass->GetDefaultObject<UNavigationSystemBase>()->ShouldCreateNavigationSystemInstance(&World))
	{
		NavSys = NewObject<UNavigationSystemBase>(&World, NavSysClass);
		if (NavSys)
		{
			NavSys->Configure(*this);
		}
	}

	return NavSys;
}

#if WITH_EDITOR
void UNavigationSystemConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_NavigationSystemClass = GET_MEMBER_NAME_CHECKED(UNavigationSystemConfig, NavigationSystemClass);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == NAME_NavigationSystemClass)
		{
			if (NavigationSystemClass.IsValid() == false)
			{
				NavigationSystemClass = *GEngine->NavigationSystemClass;
			}
			else
			{
				NavigationSystemClass.TryLoad();
				TSubclassOf<UNavigationSystemBase> NavSysClass = NavigationSystemClass.ResolveClass();
			}
		}
	}
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UNullNavSysConfig::UNullNavSysConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NavigationSystemClass.Reset();
}

