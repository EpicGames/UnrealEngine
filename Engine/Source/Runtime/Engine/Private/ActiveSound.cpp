// Copyright Epic Games, Inc. All Rights Reserved.
#include "ActiveSound.h"

#include "Audio/AudioDebug.h"
#include "Audio/AudioTraceUtil.h"
#include "Misc/App.h"
#include "AudioDevice.h"
#include "AudioLinkSettingsAbstract.h"
#include "Sound/SoundConcurrency.h"
#include "Sound/SoundCue.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "Sound/SoundNodeAttenuation.h"
#include "IAudioParameterTransmitter.h"
#include "SubtitleManager.h"

static int32 AudioOcclusionDisabledCvar = 0;
FAutoConsoleVariableRef CVarAudioOcclusionEnabled(
	TEXT("au.DisableOcclusion"),
	AudioOcclusionDisabledCvar,
	TEXT("Disables (1) or enables (0) audio occlusion.\n"),
	ECVF_Default);

static bool bUseAsyncOcclusionTracesCVar = false;
FAutoConsoleVariableRef CVarUseAsyncOcclusionTraces(
	TEXT("au.UseAsyncOcclusionTraces"),
	bUseAsyncOcclusionTracesCVar,
	TEXT("0: Run sound occlusion traces from the audio device update, 1: Run sound occlusion traces through the gt async trace api."),
	ECVF_Default);

static int32 GatherInteriorDataFromAudioVolumesCVar = 1;
FAutoConsoleVariableRef CVarGatherInteriorDataFromAudioVolumes(
	TEXT("au.InteriorData.UseAudioVolumes"),
	GatherInteriorDataFromAudioVolumesCVar,
	TEXT("When set to 1, allows gathering of interior data from audio volumes (Legacy).\n")
	TEXT("0: Disabled, 1: Enabled (default)"),
	ECVF_Default);

static int32 GatherInteriorDataFromIActiveSoundUpdateCVar = 1;
FAutoConsoleVariableRef CVarGatherInteriorDataFromIActiveSoundUpdate(
	TEXT("au.InteriorData.UseIActiveSoundUpdate"),
	GatherInteriorDataFromIActiveSoundUpdateCVar,
	TEXT("When set to 1, allows gathering of interior data from subsystems that implement the IActiveSoundUpdate interface.\n")
	TEXT("0: Disabled, 1: Enabled (default)"),
	ECVF_Default);

static int32 InitializeFocusFactorOnFirstUpdateCVar = 1;
FAutoConsoleVariableRef CVarInitializeFocusFactorOnFirstUpdateCVar(
	TEXT("au.FocusData.InitializeFocusFactorOnFirstUpdate"),
	InitializeFocusFactorOnFirstUpdateCVar,
	TEXT("When set to 1, focus factor will be initialized on first update to the proper value, instead of interpolating from 0 to the proper value.\n")
	TEXT("0: Disabled, 1: Enabled (default)"),
	ECVF_Default);


static int32 FocusDependsOnSpatializationCVar = 0;
FAutoConsoleVariableRef CVarFocusDependsOnSpatialization(
	TEXT("au.FocusData.FocusDependsOnSpatialization"),
	FocusDependsOnSpatializationCVar,
	TEXT("When set to 1, focus factor on attenuation settings will not be enabled if spatialization is not enabled on the attenuation setting.\n")
	TEXT("0: Focus and Spatialization are orthogonal, 1: Focus will be disabled if spatialization is not also enabled"),
	ECVF_Default);

static int32 IncludeModulationInConcurrencyEvaluationCVar = 0;
FAutoConsoleVariableRef CVarIncludeModulationInConcurrencyEvaluation(
	TEXT("au.Concurrency.IncludeVolumeModulation"),
	IncludeModulationInConcurrencyEvaluationCVar,
	TEXT("0: Ignore Audio Modulation Volume when evaluating concurrency (Default), 1: Apply Modulation Destination to logical volume for one-shots only, 2: Apply Modulation Destination to logical volume on all sounds."),
	ECVF_Default);

static int32 bAudioLinkSpatializeAllSoundsCVar = 0;
FAutoConsoleVariableRef CVarAudioLinkSpatializeAllSounds(
	TEXT("au.AudioLink.SpatializeAllSounds"),
	bAudioLinkSpatializeAllSoundsCVar,
	TEXT("0 (off), !=0 (on)"),
	ECVF_Default);

static int32 OverrideSeekStartTimeCVar = 0;
FAutoConsoleVariableRef CVarOverrideSeekStartTime(
	TEXT("au.Virtualization.OverrideSeekStartTimeMode"),
	OverrideSeekStartTimeCVar,
	TEXT("Optionally force all sounds with virtualization mode set to 'Seek Start Time' to another Virtualization Mode.\n")
	TEXT("0: Leave as-is at 'Seek Start Time', 1: Set to 'Disabled', 2: Set to 'Play When Silent', 3: Set to 'Restart'"),
	ECVF_Default);

FTraceDelegate FActiveSound::ActiveSoundTraceDelegate;
TMap<FTraceHandle, FActiveSound::FAsyncTraceDetails> FActiveSound::TraceToActiveSoundMap;

FActiveSound::FActiveSound()
	: World(nullptr)
	, WorldID(0)
	, Sound(nullptr)
	, SourceEffectChain(nullptr)
	, SoundAttenuation(nullptr)
	, AudioComponentID(0)
	, OwnerID(0)
	, PlayOrder(INDEX_NONE)
	, AudioDevice(nullptr)
	, SoundClassOverride(nullptr)
	, VirtualizationModeOverride(EVirtualizationMode::Restart)
	, bEnableVirtualizationModeOverride(false)
	, bHasCheckedOcclusion(false)
	, bAllowSpatialization(true)
	, bHasAttenuationSettings(false)
	, bShouldRemainActiveIfDropped(false)
	, bFinished(false)
	, bIsPaused(false)
	, bShouldStopDueToMaxConcurrency(false)
	, bHasVirtualized(false)
	, bRadioFilterSelected(false)
	, bApplyRadioFilter(false)
	, bHandleSubtitles(true)
	, bHasExternalSubtitles(false)
	, bLocationDefined(false)
	, bIgnoreForFlushing(false)
	, bAlwaysPlay(false)
	, bIsUISound(false)
	, bIsMusic(false)
	, bReverb(false)
	, bCenterChannelOnly(false)
	, bIsPreviewSound(false)
#if WITH_EDITORONLY_DATA
	, bIsDryPreviewPlayback(false)
#endif //WITH_EDITORONLY_DATA

	, bGotInteriorSettings(false)
	, bApplyInteriorVolumes(false)
#if !(NO_LOGGING || UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, bWarnedAboutOrphanedLooping(false)
#endif
	, bEnableLowPassFilter(false)
	, bEnableHighPassFilter(false)
	, bUpdatePlayPercentage(false)
	, bUpdateSingleEnvelopeValue(false)
	, bUpdateMultiEnvelopeValue(false)
	, bUpdatePlaybackTime(false)
	, bIsPlayingAudio(false)
	, bIsStopping(false)
	, bHasActiveBusSendRoutingOverride(false)
	, bHasActiveMainSubmixOutputOverride(false)
	, bHasActiveSubmixSendRoutingOverride(false)
	, bEnableBusSendRoutingOverride(false)
	, bEnableMainSubmixOutputOverride(false)
	, bEnableSubmixSendRoutingOverride(false)
	, bIsFirstAttenuationUpdate(true)
	, bStartedWithinNonBinauralRadius(false)
	, bModulationRoutingUpdated(false)
	, bIsAttenuationSettingsOverridden(false)
	, UserIndex(0)
	, FadeOut(EFadeOut::None)
	, bIsOccluded(false)
	, bAsyncOcclusionPending(false)
	, PlaybackTime(0.0f)
	, PlaybackTimeUnscaled(0.0f)
	, PlaybackTimeNonVirtualized(0.0f)
	, MinCurrentPitch(1.0f)
	, RequestedStartTime(0.0f)
	, VolumeMultiplier(1.0f)
	, PitchMultiplier(1.0f)
	, LowPassFilterFrequency(MAX_FILTER_FREQUENCY)
	, HighPassFilterFrequency(MIN_FILTER_FREQUENCY)
	, CurrentOcclusionFilterFrequency(MAX_FILTER_FREQUENCY)
	, CurrentOcclusionVolumeAttenuation(1.0f)
	, SubtitlePriority(DEFAULT_SUBTITLE_PRIORITY)
	, Priority(1.0f)
	, VolumeConcurrency(0.0f)
	, MaxSourceModulationValue(-1.0f)
	, LastOcclusionCheckTime(TNumericLimits<float>::Lowest())
	, MaxDistance(FAudioDevice::GetMaxWorldDistance())
	, LastLocation(FVector::ZeroVector)
	, AudioVolumeID(0)
	, LastUpdateTime(0.0f)
	, SourceInteriorVolume(1.0f)
	, SourceInteriorLPF(MAX_FILTER_FREQUENCY)
	, CurrentInteriorVolume(1.0f)
	, CurrentInteriorLPF(MAX_FILTER_FREQUENCY)
	, EnvelopeFollowerAttackTime(10)
	, EnvelopeFollowerReleaseTime(100)
	, bHasNewBusSends(false)
#if ENABLE_AUDIO_DEBUG
	, DebugColor(FColor::Black)
#endif // ENABLE_AUDIO_DEBUG
{
	static uint32 TotalPlayOrder = 0;
	PlayOrder = TotalPlayOrder++;

	if (!ActiveSoundTraceDelegate.IsBound())
	{
		ActiveSoundTraceDelegate.BindStatic(&OcclusionTraceDone);
	}

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	check(AudioSettings);
	OcclusionCheckInterval = AudioSettings->DefaultOcclusionCheckInterval.GetValue();
}

FActiveSound::FActiveSound(const FActiveSound&) = default;

FActiveSound::~FActiveSound()
{
	ensureMsgf(WaveInstances.Num() == 0, TEXT("Destroyed an active sound that had active wave instances."));
	check(CanDelete());
}

FActiveSound* FActiveSound::CreateVirtualCopy(const FActiveSound& InActiveSoundToCopy, FAudioDevice& InAudioDevice)
{
	check(!InActiveSoundToCopy.bIsStopping);

	FActiveSound* ActiveSound = new FActiveSound(InActiveSoundToCopy);

	ActiveSound->bAsyncOcclusionPending = false;
	ActiveSound->bHasVirtualized = true;
	ActiveSound->bIsPlayingAudio = false;
	ActiveSound->bShouldStopDueToMaxConcurrency = false;
	ActiveSound->AudioDevice = &InAudioDevice;
	ActiveSound->PlaybackTimeNonVirtualized = 0.0f;

	// If we are the restart virtual mode, we need to reset our sound cue parse state and our playback time.
	USoundBase* Sound = InActiveSoundToCopy.GetSound();
	check(Sound);
	if (InActiveSoundToCopy.GetVirtualizationMode() == EVirtualizationMode::Restart)
	{
		ActiveSound->SoundNodeOffsetMap.Reset();
	}

	// If volume concurrency tracking is enabled, reset the value,
	// otherwise keep disabled
	if (InActiveSoundToCopy.VolumeConcurrency >= 0.0f)
	{
		ActiveSound->VolumeConcurrency = 1.0f;
	}

	ActiveSound->ConcurrencyGroupData.Reset();
	ActiveSound->WaveInstances.Reset();

	FAudioThread::RunCommandOnGameThread([AudioComponentID = ActiveSound->GetAudioComponentID()]()
	{
		if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
		{
			AudioComponent->SetIsVirtualized(true);
		}
	});

	return ActiveSound;
}

FArchive& operator<<(FArchive& Ar, FActiveSound* ActiveSound)
{
	if (!Ar.IsLoading() && !Ar.IsSaving())
	{
		Ar << ActiveSound->Sound;
		Ar << ActiveSound->WaveInstances;
		Ar << ActiveSound->SoundNodeOffsetMap;
	}
	return(Ar);
}

void FActiveSound::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto WaveInstanceIt(WaveInstances.CreateConstIterator()); WaveInstanceIt; ++WaveInstanceIt)
	{
		FWaveInstance* WaveInstance = WaveInstanceIt.Value();
		// Avoid recursing back to the wave instance that sourced this active sound
		if (WaveInstance)
		{
			WaveInstance->AddReferencedObjects(Collector);
		}
	}

	Collector.AddReferencedObject(SoundClassOverride);

	Collector.AddReferencedObject(SourceEffectChain);
	if (SourceEffectChain)
	{
		SourceEffectChain->AddReferencedEffects(Collector);
	}

	Collector.AddReferencedObject(Sound);
	if (Sound && Sound->SourceEffectChain)
	{
		Sound->SourceEffectChain->AddReferencedEffects(Collector);
	}

	Collector.AddReferencedObject(SoundAttenuation);

	for (auto& Concurrency : ConcurrencySet)
	{
		if (Concurrency)
		{
			Collector.AddReferencedObject(Concurrency);
		}
	}

	if (InstanceTransmitter.IsValid())
	{
		InstanceTransmitter->AddReferencedObjects(Collector);
	}

	// Adding ref'd objects in AttenuationSettings
	Collector.AddReferencedObject(AttenuationSettings.AudioLinkSettingsOverride);
	Collector.AddReferencedObjects(AttenuationSettings.PluginSettings.OcclusionPluginSettingsArray);
	Collector.AddReferencedObjects(AttenuationSettings.PluginSettings.ReverbPluginSettingsArray);
	Collector.AddReferencedObjects(AttenuationSettings.PluginSettings.SourceDataOverridePluginSettingsArray);
	Collector.AddReferencedObjects(AttenuationSettings.PluginSettings.SpatializationPluginSettingsArray);

	for (FAudioVolumeSubmixSendSettings& SendSettings : AudioVolumeSubmixSendSettings)
	{
		for (FSoundSubmixSendInfo& SendInfo : SendSettings.SubmixSends)
		{
			Collector.AddReferencedObject(SendInfo.SoundSubmix);
		}
	}

	for (FAudioVolumeSubmixSendSettings& SendSettings : PreviousAudioVolumeSubmixSendSettings)
	{
		for (FSoundSubmixSendInfo& SendInfo : SendSettings.SubmixSends)
		{
			Collector.AddReferencedObject(SendInfo.SoundSubmix);
		}
	}

	for (TTuple<EBusSendType, FSoundSourceBusSendInfo>& BusSendTuple : NewBusSends)
	{
		Collector.AddReferencedObject(BusSendTuple.Value.AudioBus);
		Collector.AddReferencedObject(BusSendTuple.Value.SoundSourceBus);
	}

	for (TArray<FSoundSourceBusSendInfo>& SendArray : BusSendsOverride)
	{
		for (FSoundSourceBusSendInfo& BusSend : SendArray)
		{
			Collector.AddReferencedObject(BusSend.AudioBus);
			Collector.AddReferencedObject(BusSend.SoundSourceBus);
		}
	}

	for (FSoundSubmixSendInfo& SendInfo : SoundSubmixSendsOverride)
	{
		Collector.AddReferencedObject(SendInfo.SoundSubmix);
	}

	Collector.AddReferencedObjects(ModulationRouting.VolumeModulationDestination.Modulators);
	Collector.AddReferencedObjects(ModulationRouting.PitchModulationDestination.Modulators);
	Collector.AddReferencedObjects(ModulationRouting.HighpassModulationDestination.Modulators);
	Collector.AddReferencedObjects(ModulationRouting.LowpassModulationDestination.Modulators);
}

int32 FActiveSound::GetPlayCount() const
{
	if (!Sound || !AudioDevice)
	{
		return 0;
	}

	if (const int32* PlayCount = Sound->CurrentPlayCount.Find(AudioDevice->DeviceID))
	{
		return *PlayCount;
	}

	return 0;
}

void FActiveSound::SetPitch(float Value)
{
	PitchMultiplier = Value;
}

void FActiveSound::SetVolume(float Value)
{
	VolumeMultiplier = Value;
}

void FActiveSound::SetWorld(UWorld* InWorld)
{
	World = InWorld;
	WorldID = (InWorld ? InWorld->GetUniqueID() : 0);
}

void FActiveSound::SetSound(USoundBase* InSound)
{
	Sound = InSound;
	
	if (SoundClassOverride)
	{
		bApplyInteriorVolumes = SoundClassOverride->Properties.bApplyAmbientVolumes;
	}
	else
	{
		bApplyInteriorVolumes = Sound && Sound->ShouldApplyInteriorVolumes();
	}
}

void FActiveSound::SetSourceEffectChain(USoundEffectSourcePresetChain* InSourceEffectChain)
{
	SourceEffectChain = InSourceEffectChain;
}

void FActiveSound::SetSoundClass(USoundClass* SoundClass)
{
	SoundClassOverride = SoundClass;
	
	if (SoundClassOverride)
	{
		bApplyInteriorVolumes = SoundClassOverride->Properties.bApplyAmbientVolumes;
	}
	else
	{
		bApplyInteriorVolumes = Sound && Sound->ShouldApplyInteriorVolumes();
	}
}

void FActiveSound::SetAttenuationSettingsAsset(TObjectPtr<USoundAttenuation> InSoundAttenuation)
{
	SoundAttenuation = InSoundAttenuation;
}

void FActiveSound::SetAttenuationSettingsOverride(bool bInattenuationSettingsOverride)
{
	bIsAttenuationSettingsOverridden = bInattenuationSettingsOverride;
}

bool FActiveSound::IsPlayWhenSilent() const
{
	if (!AudioDevice || !AudioDevice->PlayWhenSilentEnabled())
	{
		return false;
	}
	
	return GetVirtualizationMode() == EVirtualizationMode::PlayWhenSilent;
}

float FActiveSound::GetConcurrencyPriority() const
{
	if (GetAlwaysPlay())
	{
		return GetHighestPriority(true) * Priority + MAX_SOUND_PRIORITY + 1.0f;
	}
	else
	{
		return GetHighestPriority() * Priority;
	}
}

EVirtualizationMode FActiveSound::GetVirtualizationMode() const
{
	EVirtualizationMode Mode = EVirtualizationMode::Disabled;

	if (Sound)
	{
		Mode = Sound->VirtualizationMode;
	}
	
	if (bEnableVirtualizationModeOverride)
	{
		Mode = VirtualizationModeOverride;
	}

	if (Mode == EVirtualizationMode::SeekRestart)
	{
		switch (OverrideSeekStartTimeCVar)
		{
		case 1:
			Mode = EVirtualizationMode::Disabled;
			break;
		case 2:
			Mode = EVirtualizationMode::PlayWhenSilent;
			break;
		case 3:
			Mode = EVirtualizationMode::Restart;
			break;
		default:
			break;
		}
	}
	return Mode;
}

void FActiveSound::OverrideVirtualizationMode(EVirtualizationMode Mode)
{
	VirtualizationModeOverride = Mode;
	bEnableVirtualizationModeOverride = true;
}

void FActiveSound::ClearVirtualizationModeOverride()
{
	bEnableVirtualizationModeOverride = false;
}

void FActiveSound::ClearAudioComponent()
{
	AudioComponentID = 0;
	AudioComponentUserID = NAME_None;
	AudioComponentName = NAME_None;

	OwnerID = 0;
	OwnerName = NAME_None;
	OwnerKey = FObjectKey{};
}

void FActiveSound::SetAudioComponent(const FActiveSound& ActiveSound)
{
	AudioComponentID = ActiveSound.AudioComponentID;
	AudioComponentUserID = ActiveSound.AudioComponentUserID;
	AudioComponentName = ActiveSound.AudioComponentName;

	OwnerID = ActiveSound.OwnerID;
	OwnerName = ActiveSound.OwnerName;
	OwnerKey = ActiveSound.OwnerKey;
}

void FActiveSound::SetAudioComponent(const UAudioComponent& Component)
{
	check(IsInGameThread());

	AActor* Owner = Component.GetOwner();

	AudioComponentID = Component.GetAudioComponentID();
	AudioComponentUserID = Component.GetAudioComponentUserID();
	AudioComponentName = Component.GetFName();

	SetOwner(Owner);
}

void FActiveSound::SetOwner(const AActor* Actor)
{
	if (Actor)
	{
		OwnerID = Actor->GetUniqueID();
		OwnerName = Actor->GetFName();
		OwnerKey = FObjectKey{Actor};
	}
	else
	{
		OwnerID = 0;
		OwnerName = NAME_None;
		OwnerKey = FObjectKey{};
	}
}

FString FActiveSound::GetAudioComponentName() const
{
	return (AudioComponentID > 0 ? AudioComponentName.ToString() : TEXT("NO COMPONENT"));
}

FString FActiveSound::GetOwnerName() const
{
	return (OwnerID > 0 ? OwnerName.ToString() : TEXT("None"));
}

USoundClass* FActiveSound::GetSoundClass() const
{
	if (SoundClassOverride)
	{
		return SoundClassOverride;
	}
	else if (Sound)
	{
		return Sound->GetSoundClass();
	}

	return nullptr;
}

USoundSubmixBase* FActiveSound::GetSoundSubmix() const
{
	return Sound ? Sound->GetSoundSubmix() : nullptr;
}

void FActiveSound::SetSubmixSend(const FSoundSubmixSendInfo& SubmixSendInfo)
{
	// Override send level if the submix send already included in active sound
	for (FSoundSubmixSendInfo& Info : SoundSubmixSendsOverride)
	{
		if (Info.SoundSubmix == SubmixSendInfo.SoundSubmix)
		{
			Info.SendLevel = SubmixSendInfo.SendLevel;
			return;
		}
	}

	// Otherwise, add it to the submix send overrides
	SoundSubmixSendsOverride.Add(SubmixSendInfo);
}

void FActiveSound::SetSourceBusSend(EBusSendType BusSendType, const FSoundSourceBusSendInfo& SendInfo)
{
	// Override send level if the source bus send is already included in active sound
	for (FSoundSourceBusSendInfo& Info : BusSendsOverride[(int32)BusSendType])
	{
		if ((SendInfo.SoundSourceBus && Info.SoundSourceBus == SendInfo.SoundSourceBus) ||
			(SendInfo.AudioBus && Info.AudioBus == SendInfo.AudioBus))
		{
			Info.SendLevel = SendInfo.SendLevel;
			Info.bEnableLPFCutoff = SendInfo.bEnableLPFCutoff;
			Info.LPFCutoff = SendInfo.LPFCutoff;
			Info.bEnableHPFCutoff = SendInfo.bEnableHPFCutoff;
			Info.HPFCutoff = SendInfo.HPFCutoff;

			bHasNewBusSends = true;
			NewBusSends.Add(TTuple<EBusSendType, FSoundSourceBusSendInfo>(BusSendType, SendInfo));
			return;
		}
	}

	// Otherwise, add it to the source bus send overrides
	BusSendsOverride[(int32)BusSendType].Add(SendInfo);

	bHasNewBusSends = true;
	NewBusSends.Add(TTuple<EBusSendType, FSoundSourceBusSendInfo>(BusSendType,SendInfo));
}

bool FActiveSound::HasNewBusSends() const
{
	return bHasNewBusSends;
}

TArray< TTuple<EBusSendType, FSoundSourceBusSendInfo> > const& FActiveSound::GetNewBusSends() const
{
	return NewBusSends;
}

void FActiveSound::ResetNewBusSends()
{
	NewBusSends.Empty();
	bHasNewBusSends = false;
}

void FActiveSound::SetNewModulationRouting(const FSoundModulationDefaultRoutingSettings& NewRouting)
{
	ModulationRouting = NewRouting;
	bModulationRoutingUpdated = true;
}

void FActiveSound::AddModulationRouting(const TSet<TObjectPtr<USoundModulatorBase>>& NewModulators, EModulationDestination Destination)
{
	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulationRouting.VolumeRouting = EModulationRouting::Union;
			ModulationRouting.VolumeModulationDestination.Modulators.Append(NewModulators);
			break;
		case EModulationDestination::Pitch:
			ModulationRouting.PitchRouting = EModulationRouting::Union;
			ModulationRouting.PitchModulationDestination.Modulators.Append(NewModulators);
			break;
		case EModulationDestination::Lowpass:
			ModulationRouting.LowpassRouting = EModulationRouting::Union;
			ModulationRouting.LowpassModulationDestination.Modulators.Append(NewModulators);
			break;
		case EModulationDestination::Highpass:
			ModulationRouting.HighpassRouting = EModulationRouting::Union;
			ModulationRouting.HighpassModulationDestination.Modulators.Append(NewModulators);
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return;
		}
	}
	
	bModulationRoutingUpdated = true;
}

void FActiveSound::RemoveModulationRouting(const TSet<TObjectPtr<USoundModulatorBase>>& NewModulators, EModulationDestination Destination)
{
	switch (Destination)
	{
		case EModulationDestination::Volume:
			ModulationRouting.VolumeModulationDestination.Modulators = ModulationRouting.VolumeModulationDestination.Modulators.Difference(NewModulators);
			break;
		case EModulationDestination::Pitch:
			ModulationRouting.PitchModulationDestination.Modulators = ModulationRouting.PitchModulationDestination.Modulators.Difference(NewModulators);
			break;
		case EModulationDestination::Lowpass:
			ModulationRouting.LowpassModulationDestination.Modulators = ModulationRouting.LowpassModulationDestination.Modulators.Difference(NewModulators);
			break;
		case EModulationDestination::Highpass:
			ModulationRouting.HighpassModulationDestination.Modulators = ModulationRouting.HighpassModulationDestination.Modulators.Difference(NewModulators);
			break;
		default:
		{
			static_assert(static_cast<int32>(EModulationDestination::Count) == 4, "Possible missing ELiteralType case coverage.");
			ensureMsgf(false, TEXT("Failed to set input node default: Literal type not supported"));
			return;
		}
	}

	bModulationRoutingUpdated = true;
}

TArray<TSet<TObjectPtr<USoundModulatorBase>>> FActiveSound::GetEffectiveVolumeModulators() const
{
	TArray<TSet<TObjectPtr<USoundModulatorBase>>> ModulatorSets;
	for (const TPair<UPTRINT, FWaveInstance*>& Pair : GetWaveInstances())
	{
		if (const FWaveInstance* WaveInstance = Pair.Value)
		{
			TSet<TObjectPtr<USoundModulatorBase>> VolumeModulators = WaveInstance->GetEffectiveModulationSettings(EModulationDestination::Volume).Modulators;
			if (!VolumeModulators.IsEmpty())
			{
				ModulatorSets.Add(VolumeModulators);
			}
		}
	}

	return ModulatorSets;
}

void FActiveSound::Stop()
{
	if (AudioDevice)
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		Audio::Trace::EventLog::SendActiveSoundEvent(*this, Audio::Trace::EventLog::ID::StopRequestActiveSound);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		AudioDevice->AddSoundToStop(this);
	}
}

void FActiveSound::GetSoundSubmixSends(TArray<FSoundSubmixSendInfo>& OutSends) const
{
	if (Sound)
	{
		// Get the base sends
		Sound->GetSoundSubmixSends(OutSends);

		// Loop through the overrides, which may append or override the existing send
		for (const FSoundSubmixSendInfo& SendInfo : SoundSubmixSendsOverride)
		{
			bool bOverridden = false;
			for (FSoundSubmixSendInfo& OutSendInfo : OutSends)
			{
				if (OutSendInfo.SoundSubmix == SendInfo.SoundSubmix)
				{
					OutSendInfo.SendLevel = SendInfo.SendLevel;
					bOverridden = true;
					break;
				}
			}

			if (!bOverridden)
			{
				OutSends.Add(SendInfo);
			}
		}
	}
}

void FActiveSound::GetBusSends(EBusSendType BusSendType, TArray<FSoundSourceBusSendInfo>& OutSends) const
{
	if (Sound)
	{
		// Get the base sends
		Sound->GetSoundSourceBusSends(BusSendType, OutSends);

		// Loop through the overrides, which may append or override the existing send
		for (const FSoundSourceBusSendInfo& SendInfo : BusSendsOverride[(int32)BusSendType])
		{
			bool bOverridden = false;
			for (FSoundSourceBusSendInfo& OutSendInfo : OutSends)
			{
				const bool bSameSourceBus = (OutSendInfo.SoundSourceBus != nullptr) && (OutSendInfo.SoundSourceBus == SendInfo.SoundSourceBus);
				const bool bSameAudioBus = (OutSendInfo.AudioBus != nullptr) && (OutSendInfo.AudioBus == SendInfo.AudioBus);

				if (bSameSourceBus || bSameAudioBus)
				{
					OutSendInfo.SendLevel = SendInfo.SendLevel;
					OutSendInfo.bEnableLPFCutoff = SendInfo.bEnableLPFCutoff;
					OutSendInfo.LPFCutoff = SendInfo.LPFCutoff;
					OutSendInfo.bEnableHPFCutoff = SendInfo.bEnableHPFCutoff;
					OutSendInfo.HPFCutoff = SendInfo.HPFCutoff;
					bOverridden = true;
					break;
				}
			}

			if (!bOverridden)
			{
				OutSends.Add(SendInfo);
			}
		}
	}
}

int32 FActiveSound::FindClosestListener(const TArray<struct FListener>& InListeners) const
{
	return AudioDevice ? AudioDevice->FindClosestListenerIndex(Transform, InListeners) : INDEX_NONE;
}

int32 FActiveSound::FindClosestListener() const
{
	return AudioDevice ? AudioDevice->FindClosestListenerIndex(Transform) : INDEX_NONE;
}

void FActiveSound::GetConcurrencyHandles(TArray<FConcurrencyHandle>& OutConcurrencyHandles) const
{
	OutConcurrencyHandles.Reset();

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	check(AudioSettings);

	if (ConcurrencySet.Num() > 0)
	{
		for (const USoundConcurrency* Concurrency : ConcurrencySet)
		{
			if (Concurrency)
			{
				OutConcurrencyHandles.Emplace(*Concurrency);
			}
		}
	}
	else if (Sound)
	{
		Sound->GetConcurrencyHandles(OutConcurrencyHandles);
	}
	else if (const USoundConcurrency* DefaultConcurrency = AudioSettings->GetDefaultSoundConcurrency())
	{
		OutConcurrencyHandles.Emplace(*DefaultConcurrency);
	}
}

bool FActiveSound::GetConcurrencyFadeDuration(float& OutFadeDuration) const
{
	OutFadeDuration = -1.0f;
	TArray<FConcurrencyHandle> Handles;
	GetConcurrencyHandles(Handles);
	for (FConcurrencyHandle& Handle : Handles)
	{
		// Resolution rules that don't support eviction (effectively requiring a sound to start before culling)
		// can spam if a looping ActiveSound isn't active longer than a virtualization update period, which
		// can happen when a concurrency group is maxed and constantly evicting.  If the voice steal fade time is particularly
		// long, this can flood the active sound count. Therefore, only use the voice steal fade time if the sound has been
		// active for a sufficient period of time.
		if (!Handle.Settings.IsEvictionSupported() && IsLooping() && FMath::IsNearlyZero(PlaybackTimeNonVirtualized, 0.1f))
		{
			OutFadeDuration = 0.0f;
			return false;
		}

		OutFadeDuration = OutFadeDuration < 0.0f
			? Handle.Settings.VoiceStealReleaseTime
			: FMath::Min(Handle.Settings.VoiceStealReleaseTime, OutFadeDuration);
	}

	// Negative if no handles are found, so return no fade required.
	if (OutFadeDuration <= 0.0f)
	{
		OutFadeDuration = 0.0f;
		return false;
	}

	return true;
}

void FActiveSound::UpdateInterfaceParameters(const TArray<FListener>& InListeners)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FActiveSound::UpdateInterfaceParameters);

	using namespace Audio;

	if (!InstanceTransmitter.IsValid())
	{
		return;
	}

	if (!Sound)
	{
		return;
	}

	if (!InListeners.IsValidIndex(ClosestListenerIndex))
	{
		return;
	}

	FParameterInterfacePtr AttenuationInterface = AttenuationInterface::GetInterface();
	FParameterInterfacePtr SpatializationInterface = SpatializationInterface::GetInterface();
	FParameterInterfacePtr SourceOrientationInterface = SourceOrientationInterface::GetInterface();
	FParameterInterfacePtr ListenerOrientationInterface = ListenerOrientationInterface::GetInterface();
	FParameterInterfacePtr SourceLocationInterface = SourceLocationInterface::GetInterface();
	FParameterInterfacePtr ListenerLocationInterface = ListenerLocationInterface::GetInterface();
	FParameterInterfacePtr SourceOcclusionInterface = SourceOcclusionInterface::GetInterface();
	FParameterInterfacePtr WorldTimeInterface = WorldTimeInterface::GetInterface();

	bool bImplementsAttenuation = Sound->ImplementsParameterInterface(AttenuationInterface);
	bool bImplementsSpatialization = Sound->ImplementsParameterInterface(SpatializationInterface);
	bool bImplementsSourceOrientation = Sound->ImplementsParameterInterface(SourceOrientationInterface);
	bool bImplementsListenerOrientation = Sound->ImplementsParameterInterface(ListenerOrientationInterface);
	bool bImplementsSourceLocation = Sound->ImplementsParameterInterface(SourceLocationInterface);
	bool bImplementsListenerLocation = Sound->ImplementsParameterInterface(ListenerLocationInterface);
	bool bImplementsSourceOcclusion = Sound->ImplementsParameterInterface(SourceOcclusionInterface);
	bool bImplementsWorldTime = Sound->ImplementsParameterInterface(WorldTimeInterface);

	for (const TPair<UPTRINT, FWaveInstance*>& InstancePair : WaveInstances)
	{
		const FWaveInstance* Instance = InstancePair.Value;

		if (Instance == nullptr || Instance->WaveData == Sound)
		{
			continue;
		}

		if (const USoundBase* SoundInstance = Instance->WaveData)
		{
			bImplementsAttenuation |= SoundInstance->ImplementsParameterInterface(AttenuationInterface);
			bImplementsSpatialization |= SoundInstance->ImplementsParameterInterface(SpatializationInterface);
			bImplementsSourceOrientation |= SoundInstance->ImplementsParameterInterface(SourceOrientationInterface);
			bImplementsListenerOrientation |= SoundInstance->ImplementsParameterInterface(ListenerOrientationInterface);
			bImplementsSourceLocation |= SoundInstance->ImplementsParameterInterface(SourceLocationInterface);
			bImplementsListenerLocation |= SoundInstance->ImplementsParameterInterface(ListenerLocationInterface);
			bImplementsSourceOcclusion |= SoundInstance->ImplementsParameterInterface(SourceOcclusionInterface);
			bImplementsWorldTime |= SoundInstance->ImplementsParameterInterface(WorldTimeInterface);
		}
	}

	if (false == (bImplementsAttenuation || bImplementsSpatialization || bImplementsSourceOrientation || bImplementsListenerOrientation 
		|| bImplementsSourceLocation || bImplementsListenerLocation || bImplementsSourceOcclusion || bImplementsWorldTime))
	{
		return;
	}

	TArray<FAudioParameter> ParamsToUpdate;

	// World time is available in preview, so we can add params to the update
	if (bImplementsWorldTime)
	{
		const float CurrentWorldTimeSeconds = World.Get() ? World->GetTimeSeconds() : FApp::GetGameTime();
		const float ElapsedWorldTimeSeconds = PlaybackTimeUnscaled;
		const float WorldTimeWhenPlayedSeconds = CurrentWorldTimeSeconds - PlaybackTimeUnscaled;
		
		ParamsToUpdate.Append(
			{
				{ WorldTimeInterface::Inputs::WorldTimeWhenPlayed,  WorldTimeWhenPlayedSeconds},
				{ WorldTimeInterface::Inputs::CurrentWorldTime,  CurrentWorldTimeSeconds},
				{ WorldTimeInterface::Inputs::ElapsedWorldTime,  ElapsedWorldTimeSeconds}
			});
	}

	// If source is not a Preview Sound, resume adding Audio Params related to the Listener's 3D environment
	if (!IsPreviewSound())
	{
		const FListener& Listener = InListeners[ClosestListenerIndex];
		const FVector ListenerLocation = Listener.bUseAttenuationOverride ? Listener.AttenuationOverride : Listener.Transform.GetLocation();
		const FVector SourceLocation = Transform.GetLocation();
		const FVector SourceDirection = SourceLocation - ListenerLocation;
		const bool bSourceOccluded = bIsOccluded;

		if (bImplementsAttenuation)
		{
			const float Distance = SourceDirection.Size();
			ParamsToUpdate.Add({ AttenuationInterface::Inputs::Distance, Distance });
		}

		if (bImplementsSpatialization)
		{
			const FVector SourceDirectionNormal = Listener.Transform.InverseTransformVectorNoScale(SourceDirection).GetSafeNormal();
			const FVector2D SourceAzimuthAndElevation = FMath::GetAzimuthAndElevation(SourceDirectionNormal, FVector::ForwardVector, FVector::RightVector, FVector::UpVector);
			const float Azimuth = FMath::RadiansToDegrees(SourceAzimuthAndElevation.X);
			const float Elevation = FMath::RadiansToDegrees(SourceAzimuthAndElevation.Y);
			ParamsToUpdate.Append(
				{
					{ SpatializationInterface::Inputs::Azimuth, Azimuth },
					{ SpatializationInterface::Inputs::Elevation, Elevation }
				});
		}

		if (bImplementsSourceOrientation)
		{
			const FVector ListenerDirectionNormal = Transform.InverseTransformVectorNoScale(-SourceDirection).GetSafeNormal();
			const FVector2D ListenerAzimuthAndElevation = FMath::GetAzimuthAndElevation(ListenerDirectionNormal, FVector::ForwardVector, FVector::RightVector, FVector::UpVector);
			const float Azimuth = FMath::RadiansToDegrees(ListenerAzimuthAndElevation.X);
			const float Elevation = FMath::RadiansToDegrees(ListenerAzimuthAndElevation.Y);
			ParamsToUpdate.Append(
				{
					{ SourceOrientationInterface::Inputs::Azimuth, Azimuth },
					{ SourceOrientationInterface::Inputs::Elevation, Elevation }
				});
		}

		if (bImplementsListenerOrientation)
		{
			const FVector ListenerDirectionNormal = Listener.Transform.GetRotation().GetForwardVector();
			const FVector2D ListenerAzimuthAndElevation = FMath::GetAzimuthAndElevation(ListenerDirectionNormal, FVector::ForwardVector, FVector::RightVector, FVector::UpVector);
			const float Azimuth = FMath::RadiansToDegrees(ListenerAzimuthAndElevation.X);
			const float Elevation = FMath::RadiansToDegrees(ListenerAzimuthAndElevation.Y);

			ParamsToUpdate.Append(
				{
					{ ListenerOrientationInterface::Inputs::Azimuth, Azimuth },
					{ ListenerOrientationInterface::Inputs::Elevation, Elevation }
				});
		}

		if (bImplementsSourceLocation)
		{
			const float SourceX = SourceLocation.X;
			const float SourceY = SourceLocation.Y;
			const float SourceZ = SourceLocation.Z;

			ParamsToUpdate.Append(
				{
					{ SourceLocationInterface::Inputs::X, SourceX },
					{ SourceLocationInterface::Inputs::Y, SourceY },
					{ SourceLocationInterface::Inputs::Z, SourceZ }
				});
		}

		if (bImplementsListenerLocation)
		{
			const float ListenerX = ListenerLocation.X;
			const float ListenerY = ListenerLocation.Y;
			const float ListenerZ = ListenerLocation.Z;

			ParamsToUpdate.Append(
				{
					{ ListenerLocationInterface::Inputs::X, ListenerX },
					{ ListenerLocationInterface::Inputs::Y, ListenerY },
					{ ListenerLocationInterface::Inputs::Z, ListenerZ }
				});
		}

		if (bImplementsSourceOcclusion)
		{
			ParamsToUpdate.Append(
				{
					{ SourceOcclusionInterface::Inputs::IsOccluded, bSourceOccluded }
				});
		}
	}

	InstanceTransmitter->SetParameters(MoveTemp(ParamsToUpdate));
}

int32 FActiveSound::GetClosestListenerIndex() const
{
	// If we haven't cached the closest listener index, just directly query it from the audio device.
	if (ClosestListenerIndex == INDEX_NONE && AudioDevice)
	{
		CacheClosestListener();
	}
	return ClosestListenerIndex;
}

void FActiveSound::CacheClosestListener() const
{
	// Early outs.
	if (!AudioDevice || Sound == nullptr || !Sound->IsPlayable())
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AudioFindNearestLocation);
	ClosestListenerIndex = AudioDevice->FindClosestListenerIndex(Transform);
}

void FActiveSound::UpdateWaveInstances(TArray<FWaveInstance*> &InWaveInstances, const float DeltaTime)
{
	// Reset whether or not the active sound is playing audio.
	bIsPlayingAudio = false;

	// Reset the active sound's min current pitch value. This is updated as sounds try to play and determine their pitch values.
	MinCurrentPitch = 1.0f;

	// Early outs.
	if (!AudioDevice || Sound == nullptr || !Sound->IsPlayable())
	{
		return;
	}

	FocusData.PriorityHighest = 1.0f;

	FSoundParseParameters ParseParams;
	ParseParams.Transform = Transform;
	ParseParams.StartTime = RequestedStartTime;

	// Report back to component if necessary once initial fade is complete
	const bool bIsInitFade = PlaybackTimeNonVirtualized < ComponentVolumeFader.GetFadeDuration();
	if (bIsInitFade)
	{
		bool bWasFading = ComponentVolumeFader.IsFadingIn();
		ComponentVolumeFader.Update(DeltaTime);
		const bool bIsFading = ComponentVolumeFader.IsFading();
		if (bWasFading && !bIsFading)
		{
			FAudioThread::RunCommandOnGameThread([AudioComponentID = GetAudioComponentID()]()
			{
				if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
				{
					AudioComponent->SetFadeInComplete();
				}
			});
		}
	}
	else
	{
		ComponentVolumeFader.Update(DeltaTime);
	}

	ParseParams.VolumeMultiplier = GetVolume();

	ParseParams.Priority = Priority;
	ParseParams.Pitch *= GetPitch() * Sound->GetPitchMultiplier();
	ParseParams.bEnableLowPassFilter = bEnableLowPassFilter;
	ParseParams.LowPassFilterFrequency = LowPassFilterFrequency;
	ParseParams.bEnableHighPassFilter = bEnableHighPassFilter;
	ParseParams.HighPassFilterFrequency = HighPassFilterFrequency;
	ParseParams.SoundClass = GetSoundClass();
	ParseParams.bIsPaused = bIsPaused;

	ParseParams.SoundSubmix = GetSoundSubmix();
	GetSoundSubmixSends(ParseParams.SoundSubmixSends);

	ParseParams.bEnableBusSends = Sound->bEnableBusSends;
	ParseParams.bEnableBaseSubmix = Sound->bEnableBaseSubmix;
	ParseParams.bEnableSubmixSends = Sound->bEnableSubmixSends;

	for (int32 BusSendType = 0; BusSendType < (int32)EBusSendType::Count; ++BusSendType)
	{
		GetBusSends((EBusSendType)BusSendType, ParseParams.BusSends[BusSendType]);
	}

	// Set up the base source effect chain.
	ParseParams.SourceEffectChain = GetSourceEffectChain();

	// Setup the envelope attack and release times
	ParseParams.EnvelopeFollowerAttackTime = EnvelopeFollowerAttackTime;
	ParseParams.EnvelopeFollowerReleaseTime = EnvelopeFollowerReleaseTime;

	if (bApplyInteriorVolumes)
	{
		// Additional inside/outside processing for ambient sounds
		// If we aren't in a world there is no interior volumes to be handled.
		const bool bNeedsInteriorUpdate = (!bGotInteriorSettings || (ParseParams.Transform.GetTranslation() - LastLocation).SizeSquared() > UE_KINDA_SMALL_NUMBER);
		const bool bUseAudioVolumes = GatherInteriorDataFromAudioVolumesCVar != 0;
		const bool bUseActiveSoundUpdate = GatherInteriorDataFromIActiveSoundUpdateCVar != 0;

		// Gather data from interior spaces
		if (bNeedsInteriorUpdate)
		{
			if (bUseAudioVolumes)
			{
				GatherInteriorData(ParseParams);
			}

			if (bUseActiveSoundUpdate)
			{
				AudioDevice->GatherInteriorData(*this, ParseParams);
			}

			bGotInteriorSettings = true;
		}

		// Apply data to the wave instances
		if (bUseAudioVolumes)
		{
			HandleInteriorVolumes(ParseParams);
		}

		if (bUseActiveSoundUpdate)
		{
			AudioDevice->ApplyInteriorSettings(*this, ParseParams);
		}
	}

	// for velocity-based effects like doppler
	if (DeltaTime > 0.f)
	{
		ParseParams.Velocity = (ParseParams.Transform.GetTranslation() - LastLocation) / DeltaTime;
		LastLocation = ParseParams.Transform.GetTranslation();
	}

	TArray<FWaveInstance*> ThisSoundsWaveInstances;

	// Recurse nodes, have SoundWave's create new wave instances and update bFinished unless we finished fading out.
	bFinished = true;
	if (FadeOut == EFadeOut::None || ComponentVolumeFader.IsActive())
	{
		bool bReverbSendLevelWasSet = false;
		if (bHasAttenuationSettings || bAudioLinkSpatializeAllSoundsCVar)
		{
			UpdateAttenuation(DeltaTime, ParseParams, ClosestListenerIndex);
			bReverbSendLevelWasSet = true;
		}
		else
		{
			ParseParams.ReverbSendMethod = EReverbSendMethod::Manual;
			if (ParseParams.SoundClass)
			{
				ParseParams.ManualReverbSendLevel = ParseParams.SoundClass->Properties.Default2DReverbSendAmount;
			}
			else
			{
				ParseParams.ManualReverbSendLevel = AudioDevice->GetDefaultReverbSendLevel();
			}
		}

		// Allow subsystems one last opportunity to adjust ParseParams
		AudioDevice->SubsystemsAdjustParseParams(*this, ParseParams);

		Sound->Parse(AudioDevice, 0, *this, ParseParams, ThisSoundsWaveInstances);

		// Track this active sound's min pitch value. This is used to scale it's possible duration value.
		if (ParseParams.Pitch < MinCurrentPitch)
		{
			MinCurrentPitch = ParseParams.Pitch;
		}
	}

	if (bFinished)
	{
		AudioDevice->AddSoundToStop(this);
	}
	else if (ThisSoundsWaveInstances.Num() > 0)
	{
		// Let the wave instance know that this active sound is stopping. This will result in the wave instance getting a lower sort for voice prioritization
		if (IsStopping())
		{
			for (FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
			{
				WaveInstance->SetStopping(true);
			}
		}


		// each wave instance needs its own copy of the quantization command
		for (FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
		{
			check(WaveInstance);

			if (!WaveInstance->QuantizedRequestData && QuantizedRequestData.QuantizedCommandPtr)
			{
				// shallow copy of the FQuartzQuantizedRequestData struct
				WaveInstance->QuantizedRequestData = MakeUnique<Audio::FQuartzQuantizedRequestData>(QuantizedRequestData);

				// manually deep copy the QuantizedCommandPtr object itself
				WaveInstance->QuantizedRequestData->QuantizedCommandPtr = QuantizedRequestData.QuantizedCommandPtr->GetDeepCopyOfDerivedObject();
			}
			
			// each wave instance needs its own copy of the source buffer listener.
			WaveInstance->SourceBufferListener = SourceBufferListener;
			WaveInstance->bShouldSourceBufferListenerZeroBuffer = bShouldSourceBufferListenerZeroBuffer;
		}

		// If the concurrency volume is negative (as set by ConcurrencyManager on creation),
		// skip updating as its been deemed unnecessary
		if (VolumeConcurrency >= 0.0f)
		{
			// Now that we have this sound's active wave instances, lets find the loudest active wave instance to represent the "volume" of this active sound
			VolumeConcurrency = 0.0f;
			for (const FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
			{
				check(WaveInstance);

				float WaveInstanceVolume = WaveInstance->GetVolumeWithDistanceAndOcclusionAttenuation() * WaveInstance->GetDynamicVolume();
				if (WaveInstanceVolume > VolumeConcurrency)
				{
					VolumeConcurrency = WaveInstanceVolume;
				}
			}

			const bool bUseModulationInConcurrency = (IncludeModulationInConcurrencyEvaluationCVar == 2) || (IncludeModulationInConcurrencyEvaluationCVar == 1 && IsOneShot());
			if (bUseModulationInConcurrency)
			{
				// If there are no modulators set on any wave instance, then MaxSourceModulationValue may still be negative (see line below),
				// so we take the absolute value in that case (1.0).
				VolumeConcurrency *= FMath::Abs(MaxSourceModulationValue);

				// We set the value as negative to mark it as "read", so the next place to update this value will always update it.
				// By Multiplying by -1, we can reset the "read state" while preserving the previous value.
				if (MaxSourceModulationValue >= 0.0f)
				{
					MaxSourceModulationValue *= -1.0f;
				}
			}

			// Remove concurrency volume scalars as this can cause ping-ponging to occur with virtualization and loops
			// utilizing concurrency with rules that don't support eviction (removal from concurrency system prior to playback).
			const float VolumeScale = GetTotalConcurrencyVolumeScale();
			if (VolumeScale > UE_SMALL_NUMBER)
			{
				VolumeConcurrency /= VolumeScale;
			}
			else
			{
				VolumeConcurrency = 0.0f;
			}
		}

		// Check to see if we need to broadcast the envelope value of sounds playing with this active sound
		if (AudioComponentID > 0)
		{
			if (bUpdateMultiEnvelopeValue)
			{
				int32 NumWaveInstances = ThisSoundsWaveInstances.Num();

				// Add up the envelope value for every wave instance so we get a sum of the envelope value for all sources.
				float EnvelopeValueSum = 0.0f;
				float MaxEnvelopeValue = 0.0f;
				for (FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
				{
					const float WaveInstanceEnvelopeValue = WaveInstance->GetEnvelopeValue();
					EnvelopeValueSum += WaveInstanceEnvelopeValue;
					MaxEnvelopeValue = FMath::Max(WaveInstanceEnvelopeValue, MaxEnvelopeValue);
				}

				// Now divide by the number of instances to get the average
				float AverageEnvelopeValue = EnvelopeValueSum / NumWaveInstances;
				uint64 AudioComponentIDCopy = AudioComponentID;
				FAudioThread::RunCommandOnGameThread([AudioComponentIDCopy, AverageEnvelopeValue, MaxEnvelopeValue, NumWaveInstances]()
				{
					if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentIDCopy))
					{
						if (AudioComponent->OnAudioMultiEnvelopeValue.IsBound())
						{
							AudioComponent->OnAudioMultiEnvelopeValue.Broadcast(AverageEnvelopeValue, MaxEnvelopeValue, NumWaveInstances);
						}

						if (AudioComponent->OnAudioMultiEnvelopeValueNative.IsBound())
						{
							AudioComponent->OnAudioMultiEnvelopeValueNative.Broadcast(AudioComponent, AverageEnvelopeValue, MaxEnvelopeValue, NumWaveInstances);
						}
					}
				});
			}

			if (bUpdatePlaybackTime)
			{
				TMap<uint32, float> WaveInstancePlaybackTimes;

				// Update each of the wave instances playback time based on delta time and the wave instances pitch value
				for (FWaveInstance* WaveInstance : ThisSoundsWaveInstances)
				{
					WaveInstance->PlaybackTime += DeltaTime * WaveInstance->Pitch;

					// For looping sounds, we need to check the wrapping condition
					if (WaveInstance->LoopingMode != LOOP_Never)
					{
						float Duration = WaveInstance->WaveData->Duration;
						if (WaveInstance->PlaybackTime > Duration)
						{
							WaveInstance->PlaybackTime = 0.0f;
						}
					}
					WaveInstancePlaybackTimes.Add(WaveInstance->WaveData->GetUniqueID(), WaveInstance->PlaybackTime);
				}
				uint64 AudioComponentIDCopy = AudioComponentID;
				FAudioThread::RunCommandOnGameThread([AudioComponentIDCopy, WaveInstancePlaybackTimes]()
				{
					if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentIDCopy))
					{
						AudioComponent->SetPlaybackTimes(WaveInstancePlaybackTimes);
					}
				});
			}
		}

	}

#if ENABLE_AUDIO_DEBUG
	if (DebugColor == FColor::Black)
	{
		DebugColor = FColor::MakeRandomColor();
	}
	Audio::FAudioDebugger::DrawDebugInfo(*this, ThisSoundsWaveInstances, DeltaTime);
#endif // ENABLE_AUDIO_DEBUG

	InWaveInstances.Append(ThisSoundsWaveInstances);
}

void FActiveSound::MarkPendingDestroy(bool bDestroyNow)
{
	check(AudioDevice);

	bool bWasStopping = bIsStopping;

	if (Sound && !bIsStopping)
	{
		int32* PlayCount = AudioDevice ? Sound->CurrentPlayCount.Find(AudioDevice->DeviceID) : nullptr;
		if (PlayCount)
		{
			*PlayCount = FMath::Max(*PlayCount - 1, 0);
			if (*PlayCount == 0)
			{
				Sound->CurrentPlayCount.Remove(AudioDevice->DeviceID);
			}
		}
	}

	TArray<FWaveInstance*> ToDelete;
	for (TPair<UPTRINT, FWaveInstance*> WaveInstanceIt : WaveInstances)
	{
		FWaveInstance* WaveInstance = WaveInstanceIt.Value;

		// Stop the owning sound source
		FSoundSource* Source = AudioDevice->GetSoundSource(WaveInstance);
		if (Source)
		{
			bool bStopped = false;
			if (AudioDevice->IsStoppingVoicesEnabled())
			{
				if (bDestroyNow || !AudioDevice->GetNumFreeSources())
				{
					Source->StopNow();
					bStopped = true;
				}
			}

			if (!bStopped)
			{
				Source->Stop();
			}
		}

		if (!bIsStopping)
		{
			// Dequeue subtitles for this sounds on the game thread
			DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.KillSubtitles"), STAT_AudioKillSubtitles, STATGROUP_TaskGraphTasks);
			const PTRINT WaveInstanceID = (PTRINT)WaveInstance;
			FAudioThread::RunCommandOnGameThread([WaveInstanceID]()
			{
				FSubtitleManager::GetSubtitleManager()->KillSubtitles(WaveInstanceID);
			}, GET_STATID(STAT_AudioKillSubtitles));
		}

		if (Source)
		{
			if (!Source->IsStopping())
			{
				Source->StopNow();

				ToDelete.Add(WaveInstance);
			}
			else
			{
				// This source is doing a fade out, so stopping. Can't remove the wave instance yet.
				bIsStopping = true;
			}
		}
		else
		{
			// Have a wave instance but no source.
			ToDelete.Add(WaveInstance);
		}
	}

	for (FWaveInstance* WaveInstance : ToDelete)
	{
		RemoveWaveInstance(WaveInstance->WaveInstanceHash);
	}

	if (bDestroyNow)
	{
		bIsStopping = false;
	}

	if (!bWasStopping)
	{
		AudioDevice->RemoveActiveSound(this);
	}
}

bool FActiveSound::UpdateStoppingSources(uint64 CurrentTick, bool bEnsureStopped)
{
	// If we're not stopping, then just return true (we can be cleaned up)
	if (!bIsStopping)
	{
		return true;
	}

	bIsStopping = false;

	TArray<FWaveInstance*> ToDelete;
	for (TPair<UPTRINT, FWaveInstance*> WaveInstanceIt : WaveInstances)
	{
		FWaveInstance* WaveInstance = WaveInstanceIt.Value;

		// Some wave instances in the list here may be nullptr if some sounds have already stopped or didn't need to do a stop
		if (WaveInstance)
		{
			// Stop the owning sound source
			FSoundSource* Source = AudioDevice->GetSoundSource(WaveInstance);
			if (Source)
			{
				// The source has finished (totally faded out)
				if (Source->IsFinished() || bEnsureStopped)
				{
					Source->StopNow();

					// Delete the wave instance
					ToDelete.Add(WaveInstance);
				}
				else
				{
					// We are not finished yet so touch it
					Source->LastUpdate = CurrentTick;
					Source->LastHeardUpdate = CurrentTick;

					// flag that we're still stopping (return value)
					bIsStopping = true;
				}
			}
			else
			{
				// We have a wave instance but no source for it, so just delete it.
				ToDelete.Add(WaveInstance);
			}
		}
	}

	for (FWaveInstance* WaveInstance : ToDelete)
	{
		RemoveWaveInstance(WaveInstance->WaveInstanceHash);
	}

	// Return true to indicate this active sound can be cleaned up
	// If we've reached this point, all sound waves have stopped so we can clear this wave instance out.
	if (!bIsStopping)
	{
		check(WaveInstances.Num() == 0);
		return true;
	}

	// still stopping!
	return false;
}

FWaveInstance* FActiveSound::FindWaveInstance(const UPTRINT WaveInstanceHash)
{
	return WaveInstances.FindRef(WaveInstanceHash);
}

void FActiveSound::RemoveWaveInstance(const UPTRINT WaveInstanceHash)
{
	if (FWaveInstance* WaveInstance = WaveInstances.FindRef(WaveInstanceHash))
	{
		WaveInstances.Remove(WaveInstanceHash);
		delete WaveInstance;
	}
}

void FActiveSound::OcclusionTraceDone(const FTraceHandle& TraceHandle, FTraceDatum& TraceDatum)
{
	// Look for any results that resulted in a blocking hit
	bool bFoundBlockingHit = false;
	for (const FHitResult& HitResult : TraceDatum.OutHits)
	{
		if (HitResult.bBlockingHit)
		{
			bFoundBlockingHit = true;
			break;
		}
	}

	FAsyncTraceDetails TraceDetails;
	if (TraceToActiveSoundMap.RemoveAndCopyValue(TraceHandle, TraceDetails))
	{
		if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(TraceDetails.AudioDeviceID))
			{
				FActiveSound* ActiveSound = TraceDetails.ActiveSound;

				DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.OcclusionTraceDone"), STAT_OcclusionTraceDone, STATGROUP_AudioThreadCommands);

				FAudioThread::RunCommandOnAudioThread([AudioDevice, ActiveSound, bFoundBlockingHit]()
				{
					AudioDevice->NotifyActiveSoundOcclusionTraceDone(ActiveSound, bFoundBlockingHit);
				}, GET_STATID(STAT_OcclusionTraceDone));
			}
		}
	}
}

void FActiveSound::CheckOcclusion(const FVector ListenerLocation, const FVector SoundLocation, const FSoundAttenuationSettings* AttenuationSettingsPtr)
{
	check(AttenuationSettingsPtr);
	check(AttenuationSettingsPtr->bEnableOcclusion);

	float InterpolationTime = 0.0f;

	// If occlusion is disabled by cvar, we're always going to be not occluded
	if (AudioOcclusionDisabledCvar == 1)
	{
		bIsOccluded = false;
	}
	else
	{
		// If our occlusion check interval is less than 0, we only want to check occlusion once.  Otherwise, retest when the interval passes.
		const bool bOcclusionCheckDue = OcclusionCheckInterval >= 0.f && (PlaybackTime - LastOcclusionCheckTime) > OcclusionCheckInterval;
		const bool bNeedsInitialOcclusionCheck = LastOcclusionCheckTime < 0.f;

		if (!bAsyncOcclusionPending && (bNeedsInitialOcclusionCheck || bOcclusionCheckDue))
		{
			LastOcclusionCheckTime = PlaybackTime;

			const bool bUseComplexCollisionForOcclusion = AttenuationSettingsPtr->bUseComplexCollisionForOcclusion;
			const ECollisionChannel OcclusionTraceChannel = AttenuationSettingsPtr->OcclusionTraceChannel;

			if (!bHasCheckedOcclusion || !bUseAsyncOcclusionTracesCVar)
			{
				FCollisionQueryParams Params(SCENE_QUERY_STAT(SoundOcclusion), bUseComplexCollisionForOcclusion);
				if (OwnerID > 0)
				{
					Params.AddIgnoredActor(OwnerID);
				}

				if (UWorld* WorldPtr = World.Get())
				{
					// LineTraceTestByChannel is generally threadsafe, but there is a very narrow race condition here
					// if World goes invalid before the scene lock and queries begin.
					bIsOccluded = WorldPtr->LineTraceTestByChannel(SoundLocation, ListenerLocation, OcclusionTraceChannel, Params);
				}
			}
			else
			{
				bAsyncOcclusionPending = true;

				const uint32 SoundOwnerID = OwnerID;
				TWeakObjectPtr<UWorld> SoundWorld = World;
				FAsyncTraceDetails TraceDetails;
				TraceDetails.AudioDeviceID = AudioDevice->DeviceID;
				TraceDetails.ActiveSound = this;

				FAudioThread::RunCommandOnGameThread([SoundWorld, SoundLocation, ListenerLocation, OcclusionTraceChannel, SoundOwnerID, bUseComplexCollisionForOcclusion, TraceDetails]
				{
					if (UWorld* WorldPtr = SoundWorld.Get())
					{
						FCollisionQueryParams Params(SCENE_QUERY_STAT(SoundOcclusion), bUseComplexCollisionForOcclusion);
						if (SoundOwnerID > 0)
						{
							Params.AddIgnoredActor(SoundOwnerID);
						}

						FTraceHandle TraceHandle = WorldPtr->AsyncLineTraceByChannel(EAsyncTraceType::Test, SoundLocation, ListenerLocation, OcclusionTraceChannel, Params, FCollisionResponseParams::DefaultResponseParam, &ActiveSoundTraceDelegate);
						TraceToActiveSoundMap.Add(TraceHandle, TraceDetails);
					}
				});
			}
		}

		// Update the occlusion values
		if (bHasCheckedOcclusion)
		{
			InterpolationTime = AttenuationSettingsPtr->OcclusionInterpolationTime;
		}
		bHasCheckedOcclusion = true;
	}

	if (bIsOccluded)
	{
		if (CurrentOcclusionFilterFrequency.GetTargetValue() > AttenuationSettingsPtr->OcclusionLowPassFilterFrequency)
		{
			CurrentOcclusionFilterFrequency.Set(AttenuationSettingsPtr->OcclusionLowPassFilterFrequency, InterpolationTime);
		}

		if (CurrentOcclusionVolumeAttenuation.GetTargetValue() > AttenuationSettingsPtr->OcclusionVolumeAttenuation)
		{
			CurrentOcclusionVolumeAttenuation.Set(AttenuationSettingsPtr->OcclusionVolumeAttenuation, InterpolationTime);
		}
	}
	else
	{
		CurrentOcclusionFilterFrequency.Set(MAX_FILTER_FREQUENCY, InterpolationTime);
		CurrentOcclusionVolumeAttenuation.Set(1.0f, InterpolationTime);
	}

	const float DeltaTime = FApp::GetDeltaTime();
	CurrentOcclusionFilterFrequency.Update(DeltaTime);
	CurrentOcclusionVolumeAttenuation.Update(DeltaTime);
}

void FActiveSound::GatherInteriorData(FSoundParseParameters& ParseParams)
{
	// Query for new settings using audio volumes
	FAudioDevice::FAudioVolumeSettings AudioVolumeSettings;
	AudioDevice->GetAudioVolumeSettings(WorldID, ParseParams.Transform.GetTranslation(), AudioVolumeSettings);

	InteriorSettings = AudioVolumeSettings.InteriorSettings;
	AudioVolumeSubmixSendSettings = AudioVolumeSettings.SubmixSendSettings;
	AudioVolumeID = AudioVolumeSettings.AudioVolumeID;
}

void FActiveSound::HandleInteriorVolumes(FSoundParseParameters& ParseParams)
{
	check(IsInAudioThread());
	const TArray<FListener>& Listeners = AudioDevice->GetListeners();
	check(ClosestListenerIndex < Listeners.Num());
	const FListener& Listener = Listeners[ClosestListenerIndex];

	// Check to see if we've moved to a new audio volume
	if (LastUpdateTime < Listener.InteriorStartTime)
	{
		SourceInteriorVolume = CurrentInteriorVolume;
		SourceInteriorLPF = CurrentInteriorLPF;
		LastUpdateTime = FApp::GetCurrentTime();
	}

	EAudioVolumeLocationState LocationState;
	if (Listener.AudioVolumeID == AudioVolumeID || !bAllowSpatialization)
	{
		// Ambient and listener in same ambient zone
		CurrentInteriorVolume = FMath::Lerp(SourceInteriorVolume, 1.0f, Listener.InteriorVolumeInterp);
		ParseParams.InteriorVolumeMultiplier = CurrentInteriorVolume;

		CurrentInteriorLPF = FMath::Lerp(SourceInteriorLPF, MAX_FILTER_FREQUENCY, Listener.InteriorLPFInterp);
		ParseParams.AmbientZoneFilterFrequency = CurrentInteriorLPF;

		LocationState = EAudioVolumeLocationState::InsideTheVolume;
	}
	else
	{
		// Ambient and listener in different ambient zone
		if (InteriorSettings.bIsWorldSettings)
		{
			// The ambient sound is 'outside' - use the listener's exterior volume
			CurrentInteriorVolume = FMath::Lerp(SourceInteriorVolume, Listener.InteriorSettings.ExteriorVolume, Listener.ExteriorVolumeInterp);
			ParseParams.InteriorVolumeMultiplier = CurrentInteriorVolume;

			CurrentInteriorLPF = FMath::Lerp(SourceInteriorLPF, Listener.InteriorSettings.ExteriorLPF, Listener.ExteriorLPFInterp);
			ParseParams.AmbientZoneFilterFrequency = CurrentInteriorLPF;

			LocationState = EAudioVolumeLocationState::InsideTheVolume;
		}
		else
		{
			// The ambient sound is 'inside' - use the ambient sound's interior volume multiplied with the listeners exterior volume
			CurrentInteriorVolume = FMath::Lerp(SourceInteriorVolume, InteriorSettings.InteriorVolume, Listener.InteriorVolumeInterp);
			CurrentInteriorVolume *= FMath::Lerp(SourceInteriorVolume, Listener.InteriorSettings.ExteriorVolume, Listener.ExteriorVolumeInterp);
			ParseParams.InteriorVolumeMultiplier = CurrentInteriorVolume;

			float AmbientLPFValue = FMath::Lerp(SourceInteriorLPF, InteriorSettings.InteriorLPF, Listener.InteriorLPFInterp);
			float ListenerLPFValue = FMath::Lerp(SourceInteriorLPF, Listener.InteriorSettings.ExteriorLPF, Listener.ExteriorLPFInterp);

			// The current interior LPF value is the less of the LPF due to ambient zone and LPF due to listener settings
			if (AmbientLPFValue < ListenerLPFValue)
			{
				CurrentInteriorLPF = AmbientLPFValue;
				ParseParams.AmbientZoneFilterFrequency = AmbientLPFValue;
			}
			else
			{
				CurrentInteriorLPF = ListenerLPFValue;
				ParseParams.AmbientZoneFilterFrequency = ListenerLPFValue;
			}

			LocationState = EAudioVolumeLocationState::OutsideTheVolume;
		}
	}

	AddVolumeSubmixSends(ParseParams, LocationState);
}

FWaveInstance& FActiveSound::AddWaveInstance(const UPTRINT WaveInstanceHash)
{
	FWaveInstance* WaveInstance = new FWaveInstance(WaveInstanceHash, *this);
	WaveInstances.Add(WaveInstanceHash, WaveInstance);
	return *WaveInstance;
}

void FActiveSound::ApplyRadioFilter(const FSoundParseParameters& ParseParams)
{
	check(AudioDevice);
	if (AudioDevice->GetMixDebugState() != DEBUGSTATE_DisableRadio)
	{
		// Make sure the radio filter is requested
		if (ParseParams.SoundClass)
		{
			const float RadioFilterVolumeThreshold = ParseParams.VolumeMultiplier * ParseParams.SoundClass->Properties.RadioFilterVolumeThreshold;
			if (RadioFilterVolumeThreshold > UE_KINDA_SMALL_NUMBER)
			{
				bApplyRadioFilter = (ParseParams.Volume < RadioFilterVolumeThreshold);
			}
		}
	}
	else
	{
		bApplyRadioFilter = false;
	}

	bRadioFilterSelected = true;
}

float FActiveSound::GetVolume() const
{
	const float Volume = VolumeMultiplier * ComponentVolumeFader.GetVolume() * GetTotalConcurrencyVolumeScale();
	return Sound ? Volume * Sound->GetVolumeMultiplier() : Volume;
}

float FActiveSound::GetTotalConcurrencyVolumeScale() const
{
	float OutVolume = 1.0f;

	for (const TPair<FConcurrencyGroupID, FConcurrencySoundData>& ConcurrencyPair : ConcurrencyGroupData)
	{
		OutVolume *= ConcurrencyPair.Value.GetVolume();
	}

	return OutVolume;
}

void FActiveSound::UpdateConcurrencyVolumeScalars(const float DeltaTime)
{
	for (TPair<FConcurrencyGroupID, FConcurrencySoundData>& ConcurrencyPair : ConcurrencyGroupData)
	{
		ConcurrencyPair.Value.Update(DeltaTime);
	}
}

void FActiveSound::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const
{
	bool bFoundAttenuationSettings = false;

	if (bHasAttenuationSettings)
	{
		if (bIsAttenuationSettingsOverridden)
		{
			AttenuationSettings.CollectAttenuationShapesForVisualization(ShapeDetailsMap);
		}
		else if (SoundAttenuation)
		{
			SoundAttenuation->Attenuation.CollectAttenuationShapesForVisualization(ShapeDetailsMap);
		}
	}

	// For sound cues we'll dig in and see if we can find any attenuation sound nodes that will affect the settings
	USoundCue* SoundCue = Cast<USoundCue>(Sound);
	if (SoundCue)
	{
		TArray<USoundNodeAttenuation*> AttenuationNodes;
		SoundCue->RecursiveFindAttenuation(SoundCue->FirstNode, AttenuationNodes);
		for (int32 NodeIndex = 0; NodeIndex < AttenuationNodes.Num(); ++NodeIndex)
		{
			const FSoundAttenuationSettings* AttenuationSettingsToApply = AttenuationNodes[NodeIndex]->GetAttenuationSettingsToApply();
			if (AttenuationSettingsToApply)
			{
				AttenuationSettingsToApply->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
			}
		}
	}
}

float FActiveSound::GetAttenuationFrequency(const FSoundAttenuationSettings* Settings, const FAttenuationListenerData& ListenerData, const FVector2D& FrequencyRange, const FRuntimeFloatCurve& CustomCurve)
{
	float OutputFrequency = 0.0f;

	// If the frequency mapping is the same no matter what, no need to do any mapping
	if (FrequencyRange.X == FrequencyRange.Y)
	{
		OutputFrequency = FrequencyRange.X;
	}
	// If the transition band is instantaneous, just set it to before/after frequency value
	else if (Settings->LPFRadiusMin == Settings->LPFRadiusMax)
	{
		if (ListenerData.AttenuationDistance > Settings->LPFRadiusMin)
		{
			OutputFrequency = FrequencyRange.Y;
		}
		else
		{
			OutputFrequency = FrequencyRange.X;
		}
	}
	else if (Settings->AbsorptionMethod == EAirAbsorptionMethod::Linear)
	{
		FVector2D AbsorptionDistanceRange = { Settings->LPFRadiusMin, Settings->LPFRadiusMax };

		// Do log-scaling if we've been told to do so. This applies a log function to perceptually smooth filter frequency between target frequency ranges
		if (Settings->bEnableLogFrequencyScaling)
		{
			OutputFrequency = Audio::GetLogFrequencyClamped(ListenerData.AttenuationDistance, AbsorptionDistanceRange, FrequencyRange);
		}
		else
		{
			OutputFrequency = FMath::GetMappedRangeValueClamped(AbsorptionDistanceRange, FrequencyRange, ListenerData.AttenuationDistance);
		}
	}
	else
	{
		// In manual absorption mode, the frequency ranges are interpreted as a true "range"
		FVector2D ActualFreqRange(FMath::Min(FrequencyRange.X, FrequencyRange.Y), FMath::Max(FrequencyRange.X, FrequencyRange.Y));

		// Normalize the distance values to a value between 0 and 1
		FVector2f AbsorptionDistanceRange = { Settings->LPFRadiusMin, Settings->LPFRadiusMax };
		check(AbsorptionDistanceRange.Y != AbsorptionDistanceRange.X);
		const float Alpha = FMath::Clamp<float>((ListenerData.AttenuationDistance - AbsorptionDistanceRange.X) / (AbsorptionDistanceRange.Y - AbsorptionDistanceRange.X), 0.0f, 1.0f);

		// Perform the curve mapping
		const float MappedFrequencyValue = FMath::Clamp<float>(CustomCurve.GetRichCurveConst()->Eval(Alpha), 0.0f, 1.0f);

		if (Settings->bEnableLogFrequencyScaling)
		{
			// Use the mapped value in the log scale mapping
			OutputFrequency = Audio::GetLogFrequencyClamped(MappedFrequencyValue, FVector2D(0.0f, 1.0f), ActualFreqRange);
		}
		else
		{
			// Do a straight linear interpolation between the absorption frequency ranges
			OutputFrequency = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, 1.0f), FVector2f(ActualFreqRange), MappedFrequencyValue);
		}
	}

	return FMath::Clamp<float>(OutputFrequency, MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
}

bool FActiveSound::GetAlwaysPlay() const
{
	if (bAlwaysPlay)
	{
		return true;
	}

	for (const TPair<UPTRINT, FWaveInstance*>& Pair : WaveInstances)
	{
		if (Pair.Value && Pair.Value->SoundClass)
		{
			const FSoundClassProperties* SoundClassProperties = AudioDevice->GetSoundClassCurrentProperties(Pair.Value->SoundClass);
			check(SoundClassProperties);
			if (SoundClassProperties->bAlwaysPlay)
			{
				return true;
			}
		}
	}

	return false;
}

float FActiveSound::GetHighestPriority(bool bIgnoreAlwaysPlay) const
{ 
	if (!bIgnoreAlwaysPlay)
	{
		if (GetAlwaysPlay())
		{
			static constexpr float AlwaysPlayPriority = TNumericLimits<float>::Max();
			return AlwaysPlayPriority;
		}
	}

	const float HighestPriority = FocusData.PriorityHighest * FocusData.PriorityScale;
	return FMath::Clamp(HighestPriority, 0.0f, 100.0f);
}

void FActiveSound::UpdateFocusData(float DeltaTime, const FAttenuationListenerData& ListenerData, FAttenuationFocusData* OutFocusData)
{
	FAttenuationFocusData* FocusDataToUpdate = OutFocusData ? OutFocusData : &FocusData;

	AudioDevice->GetAzimuth(ListenerData, FocusDataToUpdate->Azimuth, FocusDataToUpdate->AbsoluteAzimuth);

	FocusDataToUpdate->DistanceScale = 1.0f;
	FocusDataToUpdate->PriorityScale = 1.0f;

	if (!ListenerData.AttenuationSettings->bEnableListenerFocus)
	{
		return;
	}

	// This cvar is used to recover the incorrect state of spatialization and focus being dependent on each other
	// The original design of this system was each attenuation setting category was orthogonal (i.e. focus should be applicable independent of spatialization)
	// This cvar exists to allow a swap back to the old state if needed
	if (FocusDependsOnSpatializationCVar == 1 && !ListenerData.AttenuationSettings->bSpatialize)
	{
		return;
	}

	const FGlobalFocusSettings& FocusSettings = AudioDevice->GetGlobalFocusSettings();
	const float TargetFocusFactor = AudioDevice->GetFocusFactor(FocusDataToUpdate->Azimuth, *ListenerData.AttenuationSettings);

	// Enabling InitializeFocusVolumeBeforeInterpCVar will fix a bug related to focus factor always needing to interpolate up from 0 to the correct value.
	// Could affect game mix, so enable with caution.
	const bool bShouldInterpolateFocusFactor = (!FocusDataToUpdate->bFirstFocusUpdate || !InitializeFocusFactorOnFirstUpdateCVar);

	// User opt-in for focus interpolation
	if (ListenerData.AttenuationSettings->bEnableFocusInterpolation && bShouldInterpolateFocusFactor)
	{
		// Determine which interpolation speed to use (attack/release)
		float InterpSpeed;
		if (TargetFocusFactor <= FocusDataToUpdate->FocusFactor)
		{
			InterpSpeed = ListenerData.AttenuationSettings->FocusAttackInterpSpeed;
		}
		else
		{
			InterpSpeed = ListenerData.AttenuationSettings->FocusReleaseInterpSpeed;
		}

		FocusDataToUpdate->FocusFactor = FMath::FInterpTo(FocusDataToUpdate->FocusFactor, TargetFocusFactor, DeltaTime, InterpSpeed);
	}
	else
	{
		// Set focus directly to target value
		FocusDataToUpdate->FocusFactor = TargetFocusFactor;
	}

	// No longer first update
	FocusDataToUpdate->bFirstFocusUpdate = false;

	// Scale the volume-weighted priority scale value we use for sorting this sound for voice-stealing
	FocusDataToUpdate->PriorityScale = ListenerData.AttenuationSettings->GetFocusPriorityScale(FocusSettings, FocusDataToUpdate->FocusFactor);
	FocusDataToUpdate->DistanceScale = ListenerData.AttenuationSettings->GetFocusDistanceScale(FocusSettings, FocusDataToUpdate->FocusFactor);
	FocusDataToUpdate->VolumeScale = ListenerData.AttenuationSettings->GetFocusAttenuation(FocusSettings, FocusDataToUpdate->FocusFactor);
}

void FActiveSound::AddVolumeSubmixSends(FSoundParseParameters& ParseParams, EAudioVolumeLocationState LocationState)
{
	if (ensureMsgf(IsInAudioThread(), TEXT("AddVolumeSubmixSends called on something other than audio thread!")))
	{
		if (AudioVolumeSubmixSendSettings.Num() > 0)
		{
			for (const FAudioVolumeSubmixSendSettings& SendSetting : AudioVolumeSubmixSendSettings)
			{
				if (SendSetting.ListenerLocationState == LocationState)
				{
					for (const FSoundSubmixSendInfo& SubmixSendInfo : SendSetting.SubmixSends)
					{
						ParseParams.SoundSubmixSends.Add(SubmixSendInfo);
					}
				}
			}
		}
	}
}

void FActiveSound::ParseAttenuation(FSoundParseParameters& OutParseParams, int32 ListenerIndex, const FSoundAttenuationSettings& InAttenuationSettings)
{
	UpdateAttenuation(0.0f, OutParseParams, ListenerIndex, &InAttenuationSettings);
}

void FActiveSound::UpdateAttenuation(float DeltaTime, FSoundParseParameters& ParseParams, int32 ListenerIndex, const FSoundAttenuationSettings* SettingsAttenuationNode)
{
	// We default to using the copied off "overridden" settings (or default constructed settings)
	const FSoundAttenuationSettings* Settings = &AttenuationSettings;

	// Get the attenuation settings to use for this application to the active sound
	// Use the passed-in attenuation settings
	if (!bIsAttenuationSettingsOverridden)
	{
		if (SettingsAttenuationNode)
		{
			Settings = SettingsAttenuationNode;
		}
		// We fallback to using the asset's settings directly
		else if (SoundAttenuation)
		{
			Settings = &SoundAttenuation->Attenuation;
		}
	}

	// Reset Focus data and recompute if necessary
	FAttenuationFocusData FocusDataToApply;
	FocusDataToApply.PriorityHighest = FocusData.PriorityHighest;

	if (Settings->bEnableReverbSend)
	{
		ParseParams.ReverbSendMethod = Settings->ReverbSendMethod;
		ParseParams.ManualReverbSendLevel = Settings->ManualReverbSendLevel;
		ParseParams.CustomReverbSendCurve = Settings->CustomReverbSendCurve;
		ParseParams.ReverbSendLevelRange = { Settings->ReverbWetLevelMin, Settings->ReverbWetLevelMax };
		ParseParams.ReverbSendLevelDistanceRange = { Settings->ReverbDistanceMin, Settings->ReverbDistanceMax };
	}

	if (Settings->bEnableSubmixSends)
	{
		ParseParams.AttenuationSubmixSends.Reset();
		for (const FAttenuationSubmixSendSettings& SendSettings : Settings->SubmixSendSettings)
		{
			if (SendSettings.SoundSubmix)
			{
				ParseParams.AttenuationSubmixSends.Add(SendSettings);
			}
		}
	}

	check(AudioDevice);
	FAttenuationListenerData ListenerData = FAttenuationListenerData::Create(*AudioDevice, ListenerIndex, ParseParams.Transform, *Settings);

	// Apply priority attenuation if it's enabled
	if (Settings->bEnablePriorityAttenuation)
	{
		float PriorityScale = 1.0f;
		if (Settings->PriorityAttenuationMethod == EPriorityAttenuationMethod::Manual)
		{
			PriorityScale = Settings->ManualPriorityAttenuation;
		}
		else
		{
			const float Denom = FMath::Max(Settings->PriorityAttenuationDistanceMax - Settings->PriorityAttenuationDistanceMin, UE_SMALL_NUMBER);
			const float Alpha = FMath::Clamp((ListenerData.ListenerToSoundDistance - Settings->PriorityAttenuationDistanceMin) / Denom, 0.0f, 1.0f);

			if (Settings->PriorityAttenuationMethod == EPriorityAttenuationMethod::Linear)
			{
				PriorityScale = FMath::Max(FMath::Lerp(Settings->PriorityAttenuationMin, Settings->PriorityAttenuationMax, Alpha), 0.0f);
			}
			else
			{
				PriorityScale = FMath::Max(Settings->CustomPriorityAttenuationCurve.GetRichCurveConst()->Eval(Alpha), 0.0f);
			}
		}

		ParseParams.Priority *= FMath::Max(PriorityScale, 0.0f);
	}

	if (Settings->bSpatialize || Settings->bEnableListenerFocus)
	{
		// Feed prior focus factor on update to allow for proper interpolation.
		FocusDataToApply.FocusFactor = FocusData.FocusFactor;

		// Feed first update flag
		FocusDataToApply.bFirstFocusUpdate = FocusData.bFirstFocusUpdate;

		// Update azimuth angles prior to updating focus as it uses this in calculating
		// in and out of focus values.
		UpdateFocusData(DeltaTime, ListenerData, &FocusDataToApply);

		// Update FocusData's highest priority copy prior to applying cached scalar immediately following
		// to avoid applying scalar twice
		FocusDataToApply.PriorityHighest = FMath::Max(FocusDataToApply.PriorityHighest, ParseParams.Priority);

		ParseParams.Volume *= FocusDataToApply.VolumeScale;
		ParseParams.Priority *= FocusDataToApply.PriorityScale;
		if (Settings->bSpatialize)
		{
			ParseParams.AttenuationDistance = ListenerData.AttenuationDistance;
			ParseParams.ListenerToSoundDistance = ListenerData.ListenerToSoundDistance;
			ParseParams.ListenerToSoundDistanceForPanning = ListenerData.ListenerToSoundDistanceForPanning;
			ParseParams.AbsoluteAzimuth = FocusDataToApply.AbsoluteAzimuth;
		}
	}

	// Attenuate the volume based on the model. Note we don't apply the distance attenuation immediately to the sound.
	// The audio mixer applies distance-based attenuation as a separate stage to feed source audio through source effects and buses.
	// The old audio engine will scale this together when the wave instance is queried for GetActualVolume.
	if (Settings->bAttenuate)
	{
		// Apply the sound-class-based distance scale
		if (ParseParams.SoundClass)
		{
			FSoundClassDynamicProperties* DynamicSoundClassProperties = AudioDevice->GetSoundClassDynamicProperties(ParseParams.SoundClass);
			if (DynamicSoundClassProperties)
			{
				FocusDataToApply.DistanceScale *= FMath::Max(DynamicSoundClassProperties->AttenuationScaleParam.GetValue(), 0.0f);
			}
		}

		if (Settings->AttenuationShape == EAttenuationShape::Sphere)
		{
			// Update attenuation data in-case it hasn't been updated
			ParseParams.DistanceAttenuation *= Settings->AttenuationEval(ListenerData.AttenuationDistance, Settings->FalloffDistance, FocusDataToApply.DistanceScale);
		}
		else
		{
			const FVector ListenerTranslation = ListenerData.ListenerTransform.GetTranslation();
			ParseParams.DistanceAttenuation *= Settings->Evaluate(ParseParams.Transform, ListenerTranslation, FocusDataToApply.DistanceScale);
		}
	}

	// Only do occlusion traces if the sound is audible and we're not using a occlusion plugin
	if (Settings->bEnableOcclusion)
	{
		// If we've got a occlusion plugin settings, then the plugin will handle occlusion calculations
		if (Settings->PluginSettings.OcclusionPluginSettingsArray.Num())
		{
			UClass* PluginClass = GetAudioPluginCustomSettingsClass(EAudioPlugin::OCCLUSION);
			if (PluginClass)
			{
				for (UOcclusionPluginSourceSettingsBase* SettingsBase : Settings->PluginSettings.OcclusionPluginSettingsArray)
				{
					if (SettingsBase != nullptr && SettingsBase->IsA(PluginClass))
					{
						ParseParams.OcclusionPluginSettings = SettingsBase;
						break;
					}
				}
			}
		}
		else if (ParseParams.Volume > 0.0f && !AudioDevice->IsAudioDeviceMuted())
		{
			check(ClosestListenerIndex != INDEX_NONE);
			FVector ListenerPosition;
			const bool bAllowOverride = false;
			AudioDevice->GetListenerPosition(ClosestListenerIndex, ListenerPosition, bAllowOverride);
			CheckOcclusion(ListenerPosition, ParseParams.Transform.GetTranslation(), Settings);

			// Apply the volume attenuation due to occlusion (using the interpolating dynamic parameter)
			ParseParams.OcclusionAttenuation = CurrentOcclusionVolumeAttenuation.GetValue();
			ParseParams.bIsOccluded = bIsOccluded;
			ParseParams.OcclusionFilterFrequency = CurrentOcclusionFilterFrequency.GetValue();
		}
	}

	// Figure out which attenuation settings to use
	if (Settings->bSpatialize)
	{
		if (Settings->PluginSettings.SpatializationPluginSettingsArray.Num() > 0)
		{
			UClass* PluginClass = GetAudioPluginCustomSettingsClass(EAudioPlugin::SPATIALIZATION);
			if (PluginClass)
			{
				for (USpatializationPluginSourceSettingsBase* SettingsBase : Settings->PluginSettings.SpatializationPluginSettingsArray)
				{
					if (SettingsBase != nullptr && SettingsBase->IsA(PluginClass))
					{
						ParseParams.SpatializationPluginSettings = SettingsBase;
						break;
					}
				}
			}
		}
	}

	if (Settings->bEnableReverbSend)
	{
		if (Settings->PluginSettings.ReverbPluginSettingsArray.Num() > 0)
		{
			UClass* PluginClass = GetAudioPluginCustomSettingsClass(EAudioPlugin::REVERB);
			if (PluginClass)
			{
				for (UReverbPluginSourceSettingsBase* SettingsBase : Settings->PluginSettings.ReverbPluginSettingsArray)
				{
					if (SettingsBase != nullptr && SettingsBase->IsA(PluginClass))
					{
						ParseParams.ReverbPluginSettings = SettingsBase;
						break;
					}
				}
			}
		}
	}

	if (Settings->bEnableSourceDataOverride)
	{
		if (Settings->PluginSettings.SourceDataOverridePluginSettingsArray.Num() > 0)
		{
			UClass* PluginClass = GetAudioPluginCustomSettingsClass(EAudioPlugin::SOURCEDATAOVERRIDE);
			if (PluginClass)
			{
				for (USourceDataOverridePluginSourceSettingsBase* SettingsBase : Settings->PluginSettings.SourceDataOverridePluginSettingsArray)
				{
					if (SettingsBase != nullptr && SettingsBase->IsA(PluginClass))
					{
						ParseParams.SourceDataOverridePluginSettings = SettingsBase;
						break;
					}
				}
			}
		}
	}

	if (Settings->AudioLinkSettingsOverride)
	{
		ParseParams.AudioLinkSettingsOverride = Settings->AudioLinkSettingsOverride;
	}

	// Attenuate with the absorption filter if necessary
	if (Settings->bAttenuateWithLPF)
	{
		FVector2D AbsorptionLowPassFrequencyRange = { Settings->LPFFrequencyAtMin, Settings->LPFFrequencyAtMax };
		FVector2D AbsorptionHighPassFrequencyRange = { Settings->HPFFrequencyAtMin, Settings->HPFFrequencyAtMax };
		const float AttenuationLowpassFilterFrequency = GetAttenuationFrequency(Settings, ListenerData, AbsorptionLowPassFrequencyRange, Settings->CustomLowpassAirAbsorptionCurve);
		const float AttenuationHighPassFilterFrequency = GetAttenuationFrequency(Settings, ListenerData, AbsorptionHighPassFrequencyRange, Settings->CustomHighpassAirAbsorptionCurve);

		// Only apply the attenuation filter frequency if it results in a lower attenuation filter frequency than is already being used by ParseParams (the struct pass into the sound cue node tree)
		// This way, subsequently chained attenuation nodes in a sound cue will only result in the lowest frequency of the set.
		if (AttenuationLowpassFilterFrequency < ParseParams.AttenuationLowpassFilterFrequency)
		{
			ParseParams.AttenuationLowpassFilterFrequency = AttenuationLowpassFilterFrequency;
		}

		// Same with high pass filter frequency
		if (AttenuationHighPassFilterFrequency > ParseParams.AttenuationHighpassFilterFrequency)
		{
			ParseParams.AttenuationHighpassFilterFrequency = AttenuationHighPassFilterFrequency;
		}
	}

	ParseParams.NonSpatializedRadiusStart = Settings->NonSpatializedRadiusStart;
	ParseParams.NonSpatializedRadiusEnd = Settings->NonSpatializedRadiusEnd;
	ParseParams.NonSpatializedRadiusMode = Settings->NonSpatializedRadiusMode;
	ParseParams.StereoSpread = Settings->StereoSpread;
	ParseParams.bApplyNormalizationToStereoSounds = Settings->bApplyNormalizationToStereoSounds;
	ParseParams.bUseSpatialization |= Settings->bSpatialize || bAudioLinkSpatializeAllSoundsCVar;
	ParseParams.bEnableSourceDataOverride |= Settings->bEnableSourceDataOverride;
	ParseParams.bEnableSendToAudioLink |= Settings->bEnableSendToAudioLink;

	// Check the binaural radius to determine if we're going to HRTF spatialize, cache the result
	if (bIsFirstAttenuationUpdate)
	{
		bStartedWithinNonBinauralRadius = ListenerData.ListenerToSoundDistance < Settings->BinauralRadius;
	}

	if (bStartedWithinNonBinauralRadius)
	{
		ParseParams.SpatializationMethod = ESoundSpatializationAlgorithm::SPATIALIZATION_Default;
	}
	else
	{
		if (Settings->SpatializationAlgorithm == ESoundSpatializationAlgorithm::SPATIALIZATION_Default && AudioDevice->IsHRTFEnabledForAll())
		{
			ParseParams.SpatializationMethod = ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF;
		}
		else if (Settings->SpatializationAlgorithm == ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF && AudioDevice->IsHRTFDisabled())
		{
			ParseParams.SpatializationMethod = ESoundSpatializationAlgorithm::SPATIALIZATION_Default;
		}
		else
		{
			ParseParams.SpatializationMethod = Settings->SpatializationAlgorithm;
		}

		ParseParams.bSpatializationIsExternalSend = AudioDevice->GetCurrentSpatializationPluginInterfaceInfo().bSpatializationIsExternalSend;
	}

	// If not overriding from a node, set focus data
	if (!SettingsAttenuationNode)
	{
		FocusData = FocusDataToApply;
	}
	// Make sure to always update highest priority
	else
	{
		FocusData.PriorityHighest = FocusDataToApply.PriorityHighest;
	}

	bIsFirstAttenuationUpdate = false;
}
